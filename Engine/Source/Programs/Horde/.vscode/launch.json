{
    "version": "0.2.0",
    "configurations": [
        {
            "name": "HordeServer",
            "type": "coreclr",
            "request": "launch",
            "program": "${workspaceFolder}/HordeServer/bin/Debug/net8.0/HordeServer.exe",
            "args": [],
            "cwd": "${workspaceFolder}/HordeServer/bin/Debug/net8.0/",
            "stopAtEntry": false,
            "console": "externalTerminal"
        },
        {
            "name": "HordeAgent",
            "type": "coreclr",
            "request": "launch",
            "program": "${workspaceFolder}/HordeAgent/bin/Debug/net8.0/HordeAgent.exe",
            "args": [],
            "cwd": "${workspaceFolder}/HordeAgent/bin/Debug/net8.0",
            "stopAtEntry": false,
            "console": "externalTerminal"
        }
    ],
    "compounds": [
        {
            "name": "Debug Server and Agent",
            "configurations": [
                "HordeServer",
                "HordeAgent"
            ],
            // Optional: Controls if debugging should stop if one process exits.
            "stopAll": true
        }
    ]
}