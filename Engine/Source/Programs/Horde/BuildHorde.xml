<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../../../Engine/Build/Graph/Schema.xsd" >
	<EnvVar Name="UE_HORDE_JOBID"/>
	
	<Option Name="PreflightChange" DefaultValue="" Description="Preflight changelist number"/>
	<Option Name="Configuration" DefaultValue="release" Description="Configuration to build"/>
	<Option Name="SignOutputs" Restrict="true|false" DefaultValue="false" Description="Whether to sign the agent included with server tools"/>
	<Option Name="HordeDashboardConfig" DefaultValue="Production" Description="Whether to build a development or production dashboard"/>

	<Property Name="DefaultImageTag" Value="Unversioned"/>
	<Property Name="DefaultImageTag" Value="CL-$(Change)" If="'$(Change)' != ''"/>
	<Property Name="DefaultImageTag" Value="$(DefaultImageTag)-PF-$(PreflightChange)" If="'$(PreflightChange)' != ''" />
	<Property Name="DefaultImageTag" Value="$(DefaultImageTag)-$(UE_HORDE_JOBID)" If="'$(UE_HORDE_JOBID)' != ''" />
	<Option Name="ImageRepoUrl" DefaultValue="" Description="URL to Docker/OCI image repository"/>
	<Option Name="ImageName" DefaultValue="horde-server" Description="Name of Docker/OCI image"/>
	<Option Name="ImageTag" DefaultValue="$(DefaultImageTag)" Description="Tag to apply to image"/>
	<Option Name="ImageTagSuffix" DefaultValue="" Description="Text to append to tag"/>
	<Option Name="ImageEnvFile" DefaultValue="" Description="Path to environment file for used for Docker invocations"/>
	<Option Name="ImageRepoAuthFile" DefaultValue="" Description="Path to file with repository auth credentials"/>

	<Property Name="EngineDir" Value="$(RootDir)/Engine"/>

	<Property Name="WixDir" Value="$(RootDir)/Engine/Source/ThirdParty/WiX/3.8"/>
	<Property Name="UgsDir" Value="$(RootDir)/Engine/Source/Programs/UnrealGameSync"/>

	<Property Name="Version" Value="$(EngineMajorVersion).$(EngineMinorVersion).$(EnginePatchVersion)"/>
	<Property Name="InformationalVersion" Value="$(Version)-$(Change)"/>
	<Property Name="InformationalVersion" Value="$(InformationalVersion)-PF-$(PreflightChange)" If="'$(PreflightChange)' != ''"/>
	<Property Name="VersionArguments" Value="/p:Version=$(Version).0 /p:InformationalVersion=$(InformationalVersion)"/>

	<Property Name="DotNetRootPath" Value="$(RootDir)/Engine/Binaries/ThirdParty/DotNet/10.0"/>
	<Property Name="DotNetPath" Value=""/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/win-x64/dotnet.exe" If="$(HostPlatform) == 'Win64' And $(HostArchitecture) == 'X64'"/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/win-arm64/dotnet.exe" If="$(HostPlatform) == 'Win64' And $(HostArchitecture) == 'Arm64'"/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/linux-x64/dotnet" If="$(HostPlatform) == 'Linux' And $(HostArchitecture) == 'X64'"/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/linux-arm64/dotnet" If="$(HostPlatform) == 'Linux' And $(HostArchitecture) == 'Arm64'"/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/mac-x64/dotnet" If="$(HostPlatform) == 'Mac' And $(HostArchitecture) == 'X64'"/>
	<Property Name="DotNetPath" Value="$(DotNetRootPath)/mac-arm64/dotnet" If="$(HostPlatform) == 'Mac' And $(HostArchitecture) == 'Arm64'"/>

	<!-- SERVER -->

	<Property Name="StagePaths">
		Engine/Binaries/DotNET/EpicGames.Perforce.Native/...
		Engine/Source/Programs/Shared/...
		Engine/Source/Programs/Horde/...
		Engine/Source/Programs/AutomationTool/AutomationUtils/Matchers/...
		Engine/Source/Programs/UnrealBuildTool/Matchers/...
	</Property>

	<Agent Name="HordeServer Analyze" Type="Linux;Win64_Docker">
		<Node Name="Check Documentation">
			<CheckMarkdown Files="$(RootDir)/Engine/Source/Programs/Horde/README.md;$(RootDir)/Engine/Source/Programs/Horde/Docs/..."/>
		</Node>

		<Node Name="Analyze HordeServer" Requires="Check Documentation">
			<DotNet Arguments="build &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; /p:Configuration=Analyze $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<DotNet Arguments="build &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer.Tests/HordeServer.Tests.csproj&quot; /p:Configuration=Analyze $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
		</Node>

		<Node Name="Check HordeServer Licenses" Requires="Analyze HordeServer">
			<Property Name="IgnorePackages">
				NETStandard.Library@2.0.0
				Serilog.Enrichers.OpenTracing@0.0.2
				SharpCompress@0.30.1
				Humanizer.Core@2.2.0
			</Property>
			<NuGet-LicenseCheck BaseDir="Engine/Source/Programs/Horde/HordeServer" IgnorePackages="$(IgnorePackages)" LicenseDir="Engine/Source/Programs/Horde/ThirdParty/Licenses" DotNetPath="$(DotNetPath)"/>
		</Node>
	</Agent>
	
	<Agent Name="HordeServer" Type="Linux;Win64_Docker">
		<Property Name="StagingDir" Value="$(RootDir)/Engine/Saved/HordeServer"/>

		<Node Name="Docker Prune">
			<Docker Arguments="system prune --all --force --volumes"/>
		</Node>

		<Node Name="Build HordeServer">
			<!-- Tag all the files that need to be staged to build -->
			<Tag Files="$(StagePaths)" Except=".../.vs/...;.../.git/...;.../bin/...;.../obj/..." With="#InputFiles"/>

			<!-- Build the Docker image and publish it -->
			<Docker-Build BaseDir="Engine" Files="#InputFiles" Tag="horde-server-bare" DockerFile="Engine/Source/Programs/Horde/HordeServer/Dockerfile" Arguments="--build-arg msbuild_args=&quot;$(VersionArguments)&quot;"/>
		</Node>

		<Node Name="Test HordeServer Code Coverage">
			<Tag Files="$(StagePaths)" Except=".../.vs/...;.../.git/...;.../bin/...;.../obj/..." With="#InputFiles"/>
			<Docker-Build BaseDir="Engine" Files="#InputFiles" Tag="hordeserver-codecov" UseBuildKit="true" Target="build-env-output" Output="type=local,dest=/tmp/docker-output" DockerFile="Engine/Source/Programs/Horde/HordeServer/Dockerfile" Arguments="--build-arg msbuild_args=&quot;$(VersionArguments)&quot; --build-arg code_coverage=&quot;true&quot;" />
			<Copy From="/tmp/docker-output/dotcover-report.zip" To="$(RootDir)/Engine/Programs/AutomationTool/Saved/Logs/dotcover-report.zip" />
		</Node>

		<!-- Build server and dashboard on same agent as we rely on local, unpublished Docker images for bundling dashboard with server below -->
		<Node Name="Build HordeDashboard">
			<Copy From="Engine/Source/Programs/Horde/Docs/..." To="Engine/Source/Programs/Horde/HordeDashboard/documentation/Docs/..."/>
			<Copy From="Engine/Source/Programs/Horde/README.md" To="Engine/Source/Programs/Horde/HordeDashboard/documentation/README.md"/>
			<Tag Files="Engine/Source/Programs/Horde/Directory.Build.props" With="#InputFiles"/>
			<Tag Files="Engine/Source/Programs/Horde/HordeDashboard/..." Except="Engine/Source/Programs/Horde/HordeDashboard/node_modules/..." With="#InputFiles"/>

			<!-- Build the dashboard Docker image and publish it -->
			<Docker-Build BaseDir="Engine" Files="#InputFiles" Tag="hordedashboard-public" DockerFile="Engine/Source/Programs/Horde/HordeDashboard/Dockerfile" Arguments="--build-arg &quot;VersionInfo=$(InformationalVersion)&quot; --build-arg &quot;DashboardConfig=$(HordeDashboardConfig)&quot;"/>
		</Node>
		
		<Node Name="Build HordeServer with Dashboard" Requires="Build HordeServer;Build HordeDashboard">
			<!-- Create a new image by combining server and dashboard image into one -->
			<Docker-Build BaseDir="Engine/Source/Programs/Horde/HordeServer" Files="Dockerfile*" Tag="horde-server" DockerFile="Engine/Source/Programs/Horde/HordeServer/Dockerfile.dashboard" />
		</Node>
	</Agent>
	
	<Agent Name="UnrealToolbox" Type="Win64">
		<Node Name="Analyze UnrealToolbox">
			<DotNet Arguments="build &quot;$(EngineDir)/Source/Programs/UnrealToolbox/UnrealToolbox.csproj&quot; -p:Configuration=Analyze --nologo" DotNetPath="$(DotNetPath)"/>
		</Node>	
		
		<Node Name="Check UnrealToolbox Licenses" Requires="Analyze UnrealToolbox">
			<Property Name="IgnorePackages">
				NETStandard.Library@2.0.0
				Serilog.Enrichers.OpenTracing@0.0.2
				SharpCompress@0.30.1
				Humanizer.Core@2.2.0
				Avalonia.BuildServices@0.0.29
			</Property>
			<NuGet-LicenseCheck BaseDir="Engine/Source/Programs/UnrealToolbox" IgnorePackages="$(IgnorePackages)" LicenseDir="Engine/Source/Programs/Horde/ThirdParty/Licenses" DotNetPath="$(DotNetPath)"/>
		</Node>
	</Agent>
	
	<!-- INSTALLER -->
	
	<Property Name="InstallerDir" Value="$(RootDir)/Engine/Source/Programs/Horde/Installer"/>
	<Property Name="InstallerConfig" Value="Debug"/>
	<Property Name="InstallerOutputDir" Value="$(RootDir)/Engine/Source/Programs/Horde/Installer/bin/$(InstallerConfig)"/>
	
	<Property Name="ServerToolsStagingDir" Value="$(RootDir)/Staging/ServerTools"/>
	<Property Name="ServerToolsStagingDir_HordeCmd" Value="$(ServerToolsStagingDir)/HordeCmd"/>
	<Property Name="ServerToolsStagingDir_HordeAgent" Value="$(ServerToolsStagingDir)/HordeAgent"/>
	<Property Name="ServerToolsStagingDir_HordeAgentWin64" Value="$(ServerToolsStagingDir)/HordeAgentWin64"/>
	<Property Name="ServerToolsStagingDir_HordeAgentMsi" Value="$(ServerToolsStagingDir)/HordeAgentMsi"/>
	<Property Name="ServerToolsStagingDir_HordeAgentMsiToolbox" Value="$(ServerToolsStagingDir)/HordeAgentMsiToolbox"/>
	<Property Name="ServerToolsStagingDir_UnrealToolbox" Value="$(ServerToolsStagingDir)/UnrealToolbox"/>
	<Property Name="ServerToolsStagingDir_UnrealToolboxMsi" Value="$(ServerToolsStagingDir)/UnrealToolboxMsi"/>
	<Property Name="ServerToolsStagingDir_P4VUtils" Value="$(ServerToolsStagingDir)/P4VUtils"/>
	<Property Name="ServerToolsStagingDir_UnrealGameSync" Value="$(ServerToolsStagingDir)/UnrealGameSync"/>
	<Property Name="ServerToolsStagingDir_UnrealGameSyncCmd" Value="$(ServerToolsStagingDir)/UnrealGameSyncCmd"/>
	<Property Name="ServerToolsStagingDir_UnrealGameSyncMsi" Value="$(ServerToolsStagingDir)/UnrealGameSyncMsi"/>

    <!-- Changing the windows architecture value here means updates elsewhere in this XML, and in the Installer wxs file -->
	<Property Name="ServerArchList" Value="win-x64;osx-arm64;linux-x64"/>
	
	<Agent Name="HordeInstallerTools" Type="Win64_Licensee">
		<Node Name="Build HordeCmd" Produces="#HordeCmdFiles">
			<Delete Files="$(ServerToolsStagingDir_HordeCmd)/..."/>
			<DotNet	Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeCmd/HordeCmd.csproj&quot; --output &quot;$(ServerToolsStagingDir_HordeCmd)&quot; --runtime win-x64 --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)" />
			<Sign Files="$(ServerToolsStagingDir_HordeCmd)/Horde.exe;$(ServerToolsStagingDir_HordeCmd)/Horde.dll;$(ServerToolsStagingDir_HordeCmd)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Tag Files="$(ServerToolsStagingDir_HordeCmd)/..." With="#HordeCmdFiles"/>
		</Node>
		<Node Name="Build HordeAgent" Produces="#HordeAgentFiles">
			<Delete Files="$(ServerToolsStagingDir_HordeAgent)/..."/>
			<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeAgent/HordeAgent.csproj&quot; --output &quot;$(ServerToolsStagingDir_HordeAgent)&quot; $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/Drivers/JobDriver/JobDriver.csproj&quot; --output &quot;$(ServerToolsStagingDir_HordeAgent)/JobDriver&quot; $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Delete Files="$(ServerToolsStagingDir_HordeAgent)/appsettings.Local.json"/>
			<Sign Files="$(ServerToolsStagingDir_HordeAgent)/HordeAgent.exe;$(ServerToolsStagingDir_HordeAgent)/HordeAgent.dll;$(ServerToolsStagingDir_HordeAgent)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Sign Files="$(ServerToolsStagingDir_HordeAgent)/JobDriver/JobDriver.exe;$(ServerToolsStagingDir_HordeAgent)/JobDriver/JobDriver.dll;$(ServerToolsStagingDir_HordeAgent)/JobDriver/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Tag Files="$(ServerToolsStagingDir_HordeAgent)/..." With="#HordeAgentFiles"/>
		</Node>
		<Node Name="Build HordeAgent Win64" Produces="#HordeAgentWin64Files">
			<Delete Files="$(ServerToolsStagingDir_HordeAgentWin64)/..."/>
			
			<!-- Build the agent -->
			<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeAgent/HordeAgent.csproj&quot; --output &quot;$(ServerToolsStagingDir_HordeAgentWin64)&quot; --runtime win-x64 --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Sign Files="$(ServerToolsStagingDir_HordeAgentWin64)/HordeAgent.exe;$(ServerToolsStagingDir_HordeAgentWin64)/HordeAgent.dll;$(ServerToolsStagingDir_HordeAgentWin64)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>

			<!-- Build the driver -->
			<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/Drivers/JobDriver/JobDriver.csproj&quot; --output &quot;$(ServerToolsStagingDir_HordeAgentWin64)/JobDriver&quot; --runtime win-x64 --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Sign Files="$(ServerToolsStagingDir_HordeAgentWin64)/JobDriver/JobDriver.exe;$(ServerToolsStagingDir_HordeAgentWin64)/JobDriver/JobDriver.dll;$(ServerToolsStagingDir_HordeAgentWin64)/JobDriver/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>

			<Delete Files="$(ServerToolsStagingDir_HordeAgentWin64)/appsettings.Local.json"/>
			<Tag Files="$(ServerToolsStagingDir_HordeAgentWin64)/..." With="#HordeAgentWin64Files"/>
		</Node>
		<Node Name="Build UnrealToolbox" Produces="#UnrealToolboxFiles">
			<Delete Files="$(ServerToolsStagingDir_UnrealToolbox)/..."/>
			<DotNet	Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/UnrealToolbox/UnrealToolbox.csproj&quot; --output &quot;$(ServerToolsStagingDir_UnrealToolbox)&quot; --runtime win-x64 --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)" />
			<WriteTextFile File="$(ServerToolsStagingDir_UnrealToolbox)/Version.json" Text="{ &quot;version&quot;: &quot;$(InformationalVersion)&quot; }"/>
			<Sign Files="$(ServerToolsStagingDir_UnrealToolbox)/UnrealToolbox.exe;$(ServerToolsStagingDir_UnrealToolbox)/UnrealToolbox.dll;$(ServerToolsStagingDir_UnrealToolbox)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Tag Files="$(ServerToolsStagingDir_UnrealToolbox)/..." With="#UnrealToolboxFiles"/>
		</Node>
		<Node Name="Build UnrealToolbox Msi" Requires="Build UnrealToolbox" Produces="#UnrealToolboxMsiFiles">
			<Property Name="StagingDir" Value="$(RootDir)/Staging"/>
			<Property Name="CommonArgs" Value="-dPlatform=x64 -arch x64"/>

			<!-- Create the toolbox file list -->
			<Delete Files="$(StagingDir)/Toolbox-Bulk/..."/>
			<Tag Files="#UnrealToolboxFiles" Except="UnrealToolbox.exe" With="#ToolboxBulkFiles"/>
			<Copy Files="#ToolboxBulkFiles" From="$(ServerToolsStagingDir_UnrealToolbox)" To="$(StagingDir)/Toolbox-Bulk"/>

			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(StagingDir)/Toolbox-Bulk&quot; -cg UnrealToolboxFiles -scom -sreg -gg -dr ToolboxInstallDir -srd -var var.SourceDir -out &quot;$(InstallerDir)/UnrealToolboxFiles.wxs&quot;"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) UnrealToolboxFiles.wxs -dSourceDir=&quot;$(StagingDir)/Toolbox-Bulk&quot; -out &quot;obj/UnrealToolboxFiles.wixobj&quot;" WorkingDir="$(InstallerDir)"/>

			<!-- Main component -->
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) ToolboxInstaller.wxs -dStagingDir=&quot;$(ServerToolsStagingDir_UnrealToolbox)&quot; -out &quot;obj/UnrealToolbox.wixobj&quot; -ext WixUtilExtension" WorkingDir="$(InstallerDir)"/>
			<Spawn Exe="$(WixDir)/light.exe" Arguments="-out &quot;bin/UnrealToolbox.msi&quot; -sw1076 -pdbout &quot;bin/UnrealToolbox.pdb&quot; &quot;obj/UnrealToolbox.wixobj&quot; &quot;obj/UnrealToolboxFiles.wixobj&quot; -ext WixUtilExtension -ext WixUIExtension" WorkingDir="$(InstallerDir)"/>

			<!-- We need to sign here as will be included in the server msi -->
			<Sign Files="$(InstallerDir)/bin/UnrealToolbox.msi" Description="Unreal Toolbox" If="'$(SignOutputs)' == 'true'"/>
			
			<!-- Copy files to the output directory -->
			<Copy Files="UnrealToolbox.msi" From="$(InstallerDir)/bin" To="$(ServerToolsStagingDir_UnrealToolboxMsi)"/>
			<Tag Files="$(ServerToolsStagingDir_UnrealToolboxMsi)/..." With="#UnrealToolboxMsiFiles"/>
		</Node>
		<Node Name="Build P4VUtils" Produces="#P4VUtilsFiles">
			<Delete Files="$(ServerToolsStagingDir_P4VUtils)/..."/>
			<DotNet Arguments="publish &quot;$(RootDir)/Engine/Extras/P4VUtils/P4VUtils.csproj&quot; --configuration Release --runtime win-x64 --self-contained --output &quot;$(ServerToolsStagingDir_P4VUtils)&quot; /p:IsWindows=true /p:IsOSX=false /p:IsLinux=false /p:WithRestricted=false $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Tag Files="$(ServerToolsStagingDir_P4VUtils)/..." With="#P4VUtilsFiles"/>
		</Node>
		<Node Name="Build UnrealGameSync" Produces="#UnrealGameSyncFiles">
			<Delete Files="$(ServerToolsStagingDir_UnrealGameSync)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(ServerToolsStagingDir_UnrealGameSync)&quot; --self-contained -c ReleaseAutoUpdate -r win-x64 $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSync/UnrealGameSync.csproj&quot; --output=&quot;$(ServerToolsStagingDir_UnrealGameSync)&quot; --self-contained -c ReleaseAutoUpdate -r win-x64 $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Sign Files="$(ServerToolsStagingDir_UnrealGameSync)/UnrealGameSync.exe;$(ServerToolsStagingDir_UnrealGameSync)/ugs.exe;$(ServerToolsStagingDir_UnrealGameSync)/ugs.dll;$(ServerToolsStagingDir_UnrealGameSync)/UnrealGameSync*.dll;$(ServerToolsStagingDir_UnrealGameSync)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Tag Files="$(ServerToolsStagingDir_UnrealGameSync)/..." With="#UnrealGameSyncFiles"/>
		</Node>
		<Node Name="Build UnrealGameSync Cmd" Produces="#UnrealGameSyncCmdFiles">
			<Delete Files="$(ServerToolsStagingDir_UnrealGameSyncCmd)/..."/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(ServerToolsStagingDir_UnrealGameSyncCmd)&quot; --no-self-contained -c Release -r osx-x64 $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(ServerToolsStagingDir_UnrealGameSyncCmd)&quot; --no-self-contained -c Release -r linux-x64 $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<DotNet Arguments="publish &quot;$(UgsDir)/UnrealGameSyncCmd/UnrealGameSyncCmd.csproj&quot; --output=&quot;$(ServerToolsStagingDir_UnrealGameSyncCmd)&quot; --no-self-contained -c Release -r win-x64 $(VersionArguments)" DotNetPath="$(DotNetPath)"/>
			<Sign Files="$(ServerToolsStagingDir_UnrealGameSyncCmd)/ugs.dll;$(ServerToolsStagingDir_UnrealGameSyncCmd)/UnrealGameSync*.dll;$(ServerToolsStagingDir_UnrealGameSyncCmd)/EpicGames.*.dll" If="'$(SignOutputs)' == 'true'"/>
			<Tag Files="$(ServerToolsStagingDir_UnrealGameSyncCmd)/..." With="#UnrealGameSyncCmdFiles"/>
		</Node>
		<Node Name="Build UnrealGameSync Msi" Requires="Build UnrealGameSync" Produces="#UnrealGameSyncMsiFiles">
			<Property Name="UgsInstallerDir" Value="$(RootDir)/Engine/Source/Programs/UnrealGameSync/Installer"/>
			<Property Name="UgsPublishDir" Value="$(ServerToolsStagingDir_UnrealGameSync)"/>
			
			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(UgsPublishDir)&quot; -cg UGSLauncher_Project -dr INSTALLFOLDER -scom -sreg -srd -var var.BasePath -gg -sfrag -out obj/Release/UGSLauncher.wxs" WorkingDir="$(UgsInstallerDir)"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="-dBasePath=&quot;$(UgsPublishDir)&quot; -out obj\Release\ -dConfiguration=Release -dPlatform=x64 -arch x86 -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll Product.wxs obj/Release/UGSLauncher.wxs" WorkingDir="$(UgsInstallerDir)"/>
			<Spawn Exe="$(WixDir)/light.exe" Arguments="-out &quot;bin/Release/UnrealGameSync.msi&quot; -pdbout &quot;bin/Release/UnrealGameSync.wixpdb&quot; -cultures:null -ext ..\..\..\ThirdParty\WiX\3.8\WixUtilExtension.dll -sice:ICE69 &quot;obj\Release\Product.wixobj&quot; &quot;obj\Release\UGSLauncher.wixobj&quot;" WorkingDir="$(UgsInstallerDir)"/>

			<Sign Files="$(UgsInstallerDir)/bin/Release/UnrealGameSync.msi" Description="UnrealGameSync" If="'$(SignOutputs)' == 'true'"/>
			<Copy From="$(UgsInstallerDir)/bin/Release/UnrealGameSync.msi" To="$(ServerToolsStagingDir_UnrealGameSyncMsi)/UnrealGameSync.msi" Tag="#UnrealGameSyncMsiFiles"/>
		</Node>
		
	</Agent>
	
	<Agent Name="HordeInstallerDashboard" Type="Linux">
		<Property Name="StagingDir" Value="$(RootDir)/Staging"/>

		<Node Name="Stage Horde Installer Dashboard" Produces="#DashboardOutputFiles">
			<Delete Files="$(StagingDir)/Dashboard/..."/>		
			<WriteTextFile File="$(StagingDir)/Dashboard/index.html" Text="placeholder"/>			
		
			<!-- Build the dashboard -->
			<Tag Files="Engine/Source/Programs/Horde/Directory.Build.props" With="#DashboardInputFiles"/>
			<Tag Files="Engine/Source/Programs/Horde/HordeDashboard/..." With="#DashboardInputFiles"/>
			<Docker-Build BaseDir="Engine/Source" Files="#DashboardInputFiles" Tag="hordedashboard:installer" DockerFile="Engine/Source/Programs/Horde/HordeDashboard/installer/Dockerfile" Arguments="--build-arg &quot;VersionInfo=$(InformationalVersion)&quot; --build-arg &quot;DashboardConfig=$(HordeDashboardConfig)&quot;"/>
			
			<!-- Create a container from the image that can copy files from, note that this does not start the container -->
			<Docker Arguments="create --name horde-installer-container hordedashboard:installer"/>
			
			<!-- Copy the dashboard build to the server -->
			<Docker Arguments="cp horde-installer-container:/app/Dashboard/dist/. &quot;$(StagingDir)/Dashboard&quot;"/>

			<!-- Remove the container now that files are copied -->
			<Docker Arguments="rm horde-installer-container"/>
	
			<!-- Copy the Horde documentation to the root folder -->
			<Copy From="$(RootDir)/Engine/Source/Programs/Horde/Docs/..." To="$(StagingDir)/Dashboard/documentation/Docs/..."/>		
			<Copy From="$(RootDir)/Engine/Source/Programs/Horde/README.md" To="$(StagingDir)/Dashboard/documentation/README.md"/>
			
			<Tag Files="$(StagingDir)/Dashboard/..." With="#DashboardOutputFiles"/>
		</Node>
	</Agent>
	
	<Agent Name="HordeInstaller" Type="Win64;CompileWin64">
		<Property Name="StagingDir" Value="$(RootDir)/Staging"/>
		<Property Name="ObjDir" Value="obj/$(InstallerConfig)"/>
		<Property Name="BinDir" Value="bin/$(InstallerConfig)"/>
		<Property Name="CommonArgs" Value="-dConfiguration=$(InstallerConfig) -dPlatform=x64 -arch x64"/>

		<Node Name="Build Horde Agent Installer" Requires="#HordeAgentWin64Files" Produces="#HordeAgentMsiFiles">

			<!-- Create the agent file list -->
			<Delete Files="$(StagingDir)/Agent-Bulk/..."/>
			<Tag Files="#HordeAgentWin64Files" Except="HordeAgent.exe;UnrealToolbox.exe" With="#AgentFiles"/>
			<Copy Files="#AgentFiles" From="$(ServerToolsStagingDir_HordeAgentWin64)" To="$(StagingDir)/Agent-Bulk/Agent"/>

			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(StagingDir)/Agent-Bulk&quot; -cg HordeAgentFiles -scom -sreg -gg -dr InstallDir -srd -var var.SourceDir -out &quot;$(InstallerDir)/HordeAgentFiles.wxs&quot;"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) HordeAgentFiles.wxs -dSourceDir=&quot;$(StagingDir)/Agent-Bulk&quot; -out &quot;$(ObjDir)/HordeAgentFiles.wixobj&quot;" WorkingDir="$(InstallerDir)"/>

			<!-- Main component -->
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) AgentInstaller.wxs -dStagingDir=&quot;$(ServerToolsStagingDir_HordeAgentWin64)&quot; -out &quot;$(ObjDir)/AgentInstaller.wixobj&quot; -ext WixUtilExtension" WorkingDir="$(InstallerDir)"/>
			<Spawn Exe="$(WixDir)/light.exe" Arguments="-out &quot;$(BinDir)/UnrealHordeAgent.msi&quot; -sw1076 -pdbout &quot;$(BinDir)/UnrealHordeAgent.pdb&quot; &quot;$(ObjDir)/AgentInstaller.wixobj&quot; &quot;$(ObjDir)/HordeAgentFiles.wixobj&quot; -ext WixUtilExtension -ext WixUIExtension" WorkingDir="$(InstallerDir)"/>

			<!-- We need to sign here as will be included in the server msi -->
			<Sign Files="$(InstallerOutputDir)/UnrealHordeAgent.msi" Description="Horde Agent" If="'$(SignOutputs)' == 'true'"/>
			
			<!-- Copy files to the output directory -->
			<Copy Files="UnrealHordeAgent.msi" From="$(InstallerOutputDir)" To="$(ServerToolsStagingDir_HordeAgentMsi)"/>
			<Tag Files="$(ServerToolsStagingDir_HordeAgentMsi)/..." With="#HordeAgentMsiFiles"/>

		</Node>
		
		<ForEach Name="ServerArch" Values="$(ServerArchList)">
			<Node Name="Stage Horde Server Installer ($(ServerArch))" Requires="#DashboardOutputFiles;#HordeCmdFiles;#HordeAgentFiles;#HordeAgentMsiFiles;#UnrealToolboxFiles;#UnrealToolboxMsiFiles;#P4VUtilsFiles;#UnrealGameSyncFiles;#UnrealGameSyncCmdFiles;#UnrealGameSyncMsiFiles;" Produces="#ServerWithToolsFiles_$(ServerArch)">
		
				<!-- Stage the server -->
				<Delete Files="$(StagingDir)/Server/$(ServerArch)/..."/>
				<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; --output &quot;$(StagingDir)/Server/$(ServerArch)&quot; --runtime $(ServerArch) --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)" />

				<!-- Remove the local config file. This is only used for local iteration. -->
				<Delete Files="$(StagingDir)/Server/$(ServerArch)/appsettings.Local.json"/>

				<!-- Add the bundled tools into the installer -->

				<!-- General Tools -->
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=horde-agent -name=&quot;Horde Agent (Cross-Platform)&quot; -group=&quot;horde-agent&quot; -platforms=any -description=&quot;Cross-platform Horde Agent distribution. Allows hosts to take on work from the Horde Server.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_HordeAgent)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=horde-agent-msi -name=&quot;Horde Agent (Windows Installer)&quot; -group=&quot;horde-agent&quot; -platforms=win-x64 -description=&quot;Installer for the Horde Agent on Windows.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_HordeAgentMsi)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showintoolbox -metadata=msi-product-id:1A853492-7FD0-4F88-B120-25292474A829" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=unreal-toolbox-msi -name=&quot;Unreal Toolbox (Windows Installer)&quot; -description=&quot;Windows installer for the Unreal Toolbox.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_UnrealToolboxMsi)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showinugs" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=ugs-cmd -name=&quot;UnrealGameSync (Command-Line)&quot; -group=&quot;ugs&quot; -platforms=any -description=&quot;Command-line version of UnrealGameSync. Supports Windows, MacOS and Linux.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_UnrealGameSyncCmd)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showintoolbox" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=ugs-msi -name=&quot;UnrealGameSync (Windows Installer)&quot; -group=&quot;ugs&quot; -platforms=win-x64 -description=&quot;Utility to allow syncing and building from Perforce.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_UnrealGameSyncMsi)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showintoolbox -metadata=msi-product-id:6465D644-E775-44B5-B978-7B9EF8F0EF33" DotNetPath="$(DotNetPath)" />

				<!-- Internal Category -->
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=horde-cmd -name=&quot;Horde (Command-Line)&quot; -Category=Internal -description=&quot;Allows interacting with Horde via the command-line.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_HordeCmd)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showinugs -showintoolbox" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=horde-agent-win64 -name=&quot;Horde Agent (Windows)&quot; -category=Internal -description=&quot;Update package for older installations of Horde Agent through the MSI installer.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_HordeAgentWin64)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showindashboard=false" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=unreal-toolbox -name=&quot;Unreal Toolbox&quot; -Category=Internal -description=&quot;Update package for the Unreal Toolbox.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_UnrealToolbox)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showinugs" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=p4v-utils -name=&quot;P4V Extensions (Requires P4V Restart)&quot; -category=Internal -description=&quot;Adds common Horde workflows such as preflight-and-submit to P4V.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_P4VUtils)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showinugs -showintoolbox" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=ugs-win -name=&quot;UnrealGameSync (Windows Update Package)&quot; -category=Internal -description=&quot;Update package for UGS on Windows.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(ServerToolsStagingDir_UnrealGameSync)&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showindashboard=false" DotNetPath="$(DotNetPath)" />
				<DotNet Arguments="run --project &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeServer/HordeServer.csproj&quot; -- generate tooldata -id=ushell -name=&quot;Ushell&quot; -Category=Internal -description=&quot;Implements many command line workflows for UE.&quot; -version=&quot;$(InformationalVersion)&quot; -inputdir=&quot;$(RootDir)/Engine/Extras/ushell&quot; -serverdir=&quot;$(StagingDir)/Server/$(ServerArch)&quot; -public -showinugs -showintoolbox" DotNetPath="$(DotNetPath)" />

				<!-- Separate copy of the command line tool for installation with the server on Windows -->
				<Delete Files="$(StagingDir)/Tools/..."/>
				<DotNet Arguments="publish &quot;$(RootDir)/Engine/Source/Programs/Horde/HordeCmd/HordeCmd.csproj&quot; --output &quot;$(StagingDir)/Tools&quot; --runtime win-x64 --self-contained $(VersionArguments)" DotNetPath="$(DotNetPath)" />

				<!-- Copy the Dashboard -->
				<Copy Files="#DashboardOutputFiles" From="$(StagingDir)/Dashboard/..." To="$(StagingDir)/Server/$(ServerArch)/DashboardApp/..." />
				
				<!-- Copy the VCREDIST files -->
				<Copy From="$(StagingDir)/Server/$(ServerArch)/ThirdParty/Mongo/*.dll" To="$(StagingDir)/Server/$(ServerArch)"/>

				<Tag Files="$(StagingDir)/Server/$(ServerArch)/..." With="#ServerWithToolsFiles_$(ServerArch)" />
			</Node>
		</ForEach>

		<Node Name="Build Horde Server Installer" Requires="Stage Horde Server Installer (win-x64)">

			<!-- Create the tools file list -->
			<Delete Files="$(StagingDir)/Tools-Bulk/..."/>
			<Tag Files="$(StagingDir)/Tools/..." With="#ToolsFiles"/>
			<Copy Files="#ToolsFiles" From="$(StagingDir)" To="$(StagingDir)/Tools-Bulk"/>

			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(StagingDir)/Tools-Bulk&quot; -cg HordeToolsFiles -scom -sreg -gg -dr InstallDir -srd -var var.SourceDir -out &quot;$(InstallerDir)/HordeToolsFiles.wxs&quot;"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) HordeToolsFiles.wxs -dSourceDir=&quot;$(StagingDir)&quot; -out $(ObjDir)/HordeToolsFiles.wixobj" WorkingDir="$(InstallerDir)"/>

			<!-- Create the server file list -->
			<Delete Files="$(StagingDir)/Server-Bulk/..."/>
			<Tag Files="$(StagingDir)/Server/win-x64/..." Except="HordeServer.exe" With="#ServerFiles"/>
			<Copy Files="#ServerFiles" From="$(StagingDir)/Server/win-x64" To="$(StagingDir)/Server-Bulk"/>

			<Spawn Exe="$(WixDir)/heat.exe" Arguments="dir &quot;$(StagingDir)/Server-Bulk&quot; -cg HordeServerFiles -scom -sreg -gg -dr ServerInstallDir -srd -var var.SourceDir -out &quot;$(InstallerDir)/HordeServerFiles.wxs&quot;"/>
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) HordeServerFiles.wxs -dSourceDir=&quot;$(StagingDir)/Server/win-x64&quot; -out $(ObjDir)/HordeServerFiles.wixobj" WorkingDir="$(InstallerDir)"/>

			<!-- Main component -->
			<Spawn Exe="$(WixDir)/candle.exe" Arguments="$(CommonArgs) ServerInstaller.wxs -dStagingDir=&quot;$(StagingDir)&quot; -out $(ObjDir)/ServerInstaller.wixobj -ext WixUtilExtension" WorkingDir="$(InstallerDir)"/>
			<Spawn Exe="$(WixDir)/light.exe" Arguments="-out $(BinDir)/UnrealHordeServer.msi -sw1076 -pdbout $(BinDir)/UnrealHordeServer.pdb $(ObjDir)/ServerInstaller.wixobj $(ObjDir)/HordeServerFiles.wixobj $(ObjDir)/HordeToolsFiles.wixobj -ext WixUtilExtension -ext WixUIExtension" WorkingDir="$(InstallerDir)"/>

			<!-- We need to sign here as will be included in the server msi -->
			<Sign Files="$(InstallerOutputDir)/UnrealHordeServer.msi" Description="Horde Server" If="'$(SignOutputs)' == 'true'"/>
			<Copy Files="UnrealHordeServer.msi" From="$(InstallerOutputDir)" To="$(StagingDir)/Server-Installer"/>

		</Node>
	</Agent>

	<Agent Name="HordeInstallerLinux" Type="Linux">
		<Property Name="StagingDir" Value="$(RootDir)/Staging"/>
		
		<Node Name="Build Bundled Docker Image" Requires="Stage Horde Server Installer (win-x64);#ServerWithToolsFiles_win-x64">
			<Tag Files="$(StagePaths)" Except=".../.vs/...;.../.git/...;.../bin/...;.../obj/..." With="#InputFiles"/>
			<Docker-Build BaseDir="Engine" Files="#InputFiles" Tag="horde-server-bare" DockerFile="Engine/Source/Programs/Horde/HordeServer/Dockerfile" Arguments="--build-arg msbuild_args=&quot;$(VersionArguments)&quot;"/>

			<!-- A Horde server image has already been built, and is referenced inside Dockerfile.bundled -->
			<Copy From="Engine/Source/Programs/Horde/HordeServer/Dockerfile.bundled" To="$(StagingDir)/Dockerfile.bundled" />
			<Tag Files="$(StagingDir)/Dockerfile*;$(StagingDir)/Dashboard/...;$(StagingDir)/Server/win-x64/Tools/...;$(StagingDir)/Server/win-x64/*.json" With="#DockerFiles"/>
			<Docker-Build BaseDir="$(StagingDir)" Files="#DockerFiles" Tag="horde-server:bundled" DockerFile="$(StagingDir)/Dockerfile.bundled" />
		</Node>

		<Node Name="Push Bundled Docker Image" Requires="Build Bundled Docker Image">
			<Docker-Push Repository="$(ImageRepoUrl)" Image="horde-server:bundled" TargetImage="$(ImageName):$(ImageTag)$(ImageTagSuffix)" EnvironmentFile="$(ImageEnvFile)" RepositoryAuthFile="$(ImageRepoAuthFile)" />
		</Node>
	</Agent>

</BuildGraph>
