<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="Current" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <!-- Import Shared UnrealEngine props for all Horde projects -->
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <!-- Common Horde project properties -->
  <PropertyGroup>
    <TargetFramework>net10.0</TargetFramework>
    <Nullable>enable</Nullable>
  </PropertyGroup>

  <!-- Non-test project default properties -->
  <PropertyGroup Condition="!$(MSBuildProjectName.EndsWith('.Tests')) AND !$(MSBuildProjectName.EndsWith('.Tests.Shared'))">
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
  </PropertyGroup>

  <!-- Test project default properties -->
  <PropertyGroup Condition="$(MSBuildProjectName.EndsWith('.Tests')) OR $(MSBuildProjectName.EndsWith('.Tests.Shared'))">
    <IsPackable>false</IsPackable>
    <IsTestProject>true</IsTestProject>
    <!-- We don't require public documentation for test projects -->
    <NoWarn>$(NoWarn);1591</NoWarn>
  </PropertyGroup>

  <!-- Test project default includes -->
  <ItemGroup Condition="$(MSBuildProjectName.EndsWith('.Tests')) OR $(MSBuildProjectName.EndsWith('.Tests.Shared'))">
    <PackageReference Include="Microsoft.NET.Test.Sdk" Version="18.4.0">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestAdapter" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="MSTest.TestFramework" Version="4.2.1">
      <TreatAsUsed>true</TreatAsUsed>
    </PackageReference>
    <PackageReference Include="coverlet.collector" Version="8.0.1">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
  </ItemGroup>

</Project>
