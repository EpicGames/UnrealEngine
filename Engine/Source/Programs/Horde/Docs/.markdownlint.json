{
    "MD013": { "line_length": 120 },
    "MD041": false,
    "MD033": false
}