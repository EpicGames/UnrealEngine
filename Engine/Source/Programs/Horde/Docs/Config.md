[Horde](../README.md) > Configuration

# Configuration

This section targets operating and managing Horde installations
and shares some best practices learned from supporting it for
teams at Epic.

## General

* [Orientation](Config/Orientation.md): Get familiar with how
  Horde is configured, how to store configuration data in
  revision control, and how to set up a schema server.
* [Permissions](Config/Permissions.md): Understand how to use the
  Horde permissions model.

## Plugins

Horde is implemented as a set of plugins on a host server. For
information about enabling or disabling specific plugins, see
[Configuration > Plugins](Config/Plugins.md).

### Analytics

Experimental solution for studio-wide analytics and metrics
gathering, with support integrated into AutomationTool,
UnrealBuildTool and Unreal Editor.

* [Analytics](Config/Analytics.md): Setting up a telemetry sink
  and creating dashboards showing KPIs.

### Build

Mature build automation system similar to Jenkins or TeamCity,
designed and streamlined for Unreal Engine projects and best
practices adopted by Epic.

* [Build Automation](Config/BuildAutomation.md): Introduction to
  Horde's Build Automation and CI/CD system.
* [Artifacts](Config/Artifacts.md): Managing and distributing
  artifacts produced by build steps.
* [Devices](Config/Devices.md): Adding mobile devices and console
  development kits as shared resources.
* [Automation Hub](Config/TestAutomationHub.md): Dashboard for
  surfacing test jamdata and trends across multiple projects and
  streams.
* [UGS Metadata Server](Config/UgsMetadataServer.md): Surfaces
  team metadata to users of UnrealGameSync.

### Compute

Manages worker machines that can be leased out to perform
workloads, including build automation and remote execution use
cases.

* [Compute](Config/Compute.md): Configuring agent pools and compute clusters.
* [Agents](Config/Agents.md): Configuring worker machines to connect
  to the Horde server for CI and remote execution workspaces.

### Experimental

Experimental research and development functionality within Horde.
While functional, this plugin is subject to change without notice.

* [Experimental](Config/Experimental.md): Configuring the current
  functionality of this plugin.

### Scheduled Downtime

Manages the maintenance windows.

* [Scheduled Downtime](Config/ScheduledDowntime.md): Configuring scheduled
  downtimes.

### Secrets

Managed access to sensitive data integrated with Horde's
permissions model.Secrets may be stored in Horde itself, or in an
external secret store.

* [Secrets](Config/Secrets.md): Configuring secrets and external
  secret providers.

### Storage

Flexible, low overhead storage abstraction suitable for many
different use cases. Used by Horde internally, but it can also be
used directly by client applications.

* [Storage](Config/Storage.md): Setting up and managing the
  storage system.  

### Symbols

Implements a Windows symbol store indexing artifacts uploaded to
Horde's storage system.

* [Symbols](Config/Symbols.md): Adding symbol stores and
  configuring clients to use them.

### Tools

Hosts tools and utilities that can be downloaded by users using
UnrealGameSync or Unreal Toolbox.

* [Tools](Config/Tools.md): Configure tools hosted by Horde.

## Reference

* [Globals.json Reference](Config/Schema/Globals.md)
* [*.project.json Reference](Config/Schema/Projects.md)
* [*.stream.json Reference](Config/Schema/Streams.md)
* [*.telemetry.json Reference](Config/Schema/Telemetry.md)
* [*.dashboard.json Reference](Config/Schema/Dashboard.md)
