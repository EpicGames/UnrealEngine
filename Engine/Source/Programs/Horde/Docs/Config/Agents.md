[Horde](../../README.md) > [Configuration](../Config.md) > Agents

# Agents

## Installing the Horde Agent

For information about deploying new agents, see [Horde > Deployment > Agent](../Deployment/Agent.md).

## Pools

Pools are groups of machines that can be used interchangeably, typically due to being a particular platform or
hardware class. At Epic, certain pools are dedicated to (incremental) compilation, cooking and Unreal Build Accelerator (UBA). 
Pools simplify the management of build pipelines by allowing infrastructure engineers to configure a mapping
from agent types to physical machines. An agent can belong to multiple pools.

Pools are defined in the [.globals.json](Schema/Globals.md) file, via the `pools` property. Agents may be added to a
pool manually through the Horde Dashboard or automatically by matching a particular condition. For example,
the following configuration block defines a pool that automatically includes all Windows machines:

        {
            "name": "WinLargeRam",
            "condition": "Platform == 'Win64' && RAM > 64gb"
        }

See also: [Condition expression syntax](Conditions.md)

### Auto-scaling

Pools in Horde serve as the primary configuration point for auto-scaling,
a feature particularly valuable in cloud environments where hardware resources can be rented on an hourly or per-second basis.
This functionality allows for resource allocation based on factors such as time of week or the current phase of your game project,
optimizing cost-efficiency. For on-premise setups where hardware is statically allocated or owned, this feature is less useful.

Each pool can be configured to automatically adjust its size based on specific metrics.
The pool size strategy defines the ideal number of agents, with two primary approaches: `JobQueue` and `LeaseUtilization`.
`JobQueue`, the preferred method, proactively examines the queue of pending jobs waiting to run.
In contrast, `LeaseUtilization` reactively adjusts based on average CPU usage across all agents in the pool, which may result in a slight lag.

Fleet managers handle cloud-specific implementations for hardware allocation,
such as the `AwsRecycle` manager that controls AWS EC2 instances.

To fine-tune scaling behavior, time-based cooldowns for scale-in and scale-out operations can be set,
with scale-out typically configured more aggressively to accommodate rapid bursts of activity without excessive wait times.

## Remoting to Agents

If you have a fleet of machines that require identical login credentials, you can configure UnrealGameSync to open
Remote Desktop sessions from links in the Horde dashboard.

To enable this functionality, open **Credential Manager** from the Windows Control Panel and select **Windows
Credentials**. Click the **Add a new generic credential...** link to create a new entry and name it
`UnrealGameSync:RDP`. Enter the login username and password as appropriate.

The **Remote Desktop** button on agent dialogs in Horde will open a URL of the form `ugs://rdp?host=[NameOrIP]`.
UnrealGameSync is configured to handle `ugs://` links by default, intercepts these links, and adds a Windows login
entry for the given `NameOrIP` before launching the remote desktop application.
