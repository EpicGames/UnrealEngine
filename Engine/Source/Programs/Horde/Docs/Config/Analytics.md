[Horde](../../README.md) > [Configuration](../Config.md) > Analytics

# Analytics

Horde implements HTTP endpoints to collect telemetry data sent by Unreal Editor. This data can
provide insights into bottlenecks and workflow issues that a team and the Horde dashboard can aggregate and
chart it to highlight improvements and regressions over time.

The [Getting Started > Analytics](../Tutorials/Analytics.md) guide explains how to configure a project to send
telemetry data to Horde.

## Telemetry Stores

Horde supports multiple orthogonal telemetry stores, allowing you to group telemetry data for different projects
as you see fit. Each telemetry store has its own set of metrics, and the dashboard allows switching contexts
to view the same charts using data from different stores.

To send data to a particular telemetry store, include the telemetry store name in the `APIEndpointET` property
in the project's `DefaultEngine.ini` file. The `engine` store uses the following URL, for example:

    APIEndpointET="api/v1/telemetry/engine"

## Metrics

To provide efficient aggregation of analytics data over large time periods, Horde aggregates telemetry events into
running metrics for each time interval. This aggregation is performed according to rules specified in the
`Telemetry.Metrics` section of the globals.json file (see [MetricConfig](Schema/Telemetry.md#metricconfig)).

## Charting

The Horde dashboard render charts showing metrics collected on the server. These views are configured
using the `Dashboard.Analytics` section of the globals.json file (see [TelemetryViewConfig](Schema/Telemetry.md#telemetryviewconfig)).

## Telemetry Sinks

Horde can collect raw telemetry data in its own database as well as forwarding it to other telemetry sinks.

You can configure telemetry sinks through the `Telemetry` property in the server's
[Server.json](../Deployment/ServerSettings.md) file. It is not necessary to configure a telemetry sink
in order to compute metrics from aggregated data.
