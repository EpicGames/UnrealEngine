[Horde](../../README.md) > [Configuration](../Config.md) > Artifacts

# Artifacts

An artifact in Horde refers to a set of archived files produced by a build
process. The archive corresponds to a particular commit in VCS, allowing
for searching for artifacts produced between in a particular stream across
a range of time, and includes other indexed keys and metadata that can
help it to be manipulated by external tools.

Amongst other things, artifacts can be used to store:

- Packaged builds
- Precompiled editor builds for UnrealGameSync (often referred to as "PCBs")
- Intermediate artifacts transferred between agents by BuildGraph
- Test results
- Logs and traces

## Configuration

Settings for artifacts are configured by their **artifact type**, which
are specified at upload time. Artifact types are arbitrary string
identifiers (case insensitive alpha-numeric strings, or the characters
`-`, `_`, `.`).

Artifact types are customizable, and new types may be added as needed.
They must be declared through Horde config files before Horde will allow
clients to upload artifacts of that type.

Types may be configured through the `artifactTypes` property at the
[global](Schema/Globals.md#buildconfig), [project](Schema/Projects.md),
or [stream](Schema/Streams.md) scopes.

### Standard Artifact Types

Horde produces certain artifact types by default:

- `step-output`: Output from a build step which can be used by other build
steps. These artifacts are produced based on dependencies between steps
configured through BuildGraph.

- `step-saved`: Contents of the AutomationTool log folder, harvested from
the agent executing a job once it completes. The Horde Agent sets the path
to this output folder for jobs via the `uebp_LogFolder` environment
variable (normally `Engine/Programs/AutomationTool/Saved/Logs`).

- `step-trace`: Telemetry and trace data for a build step, harvested from
the directory given by the `UE_TELEMETRY_DIR` environment variable
(normally `Engine/Programs/AutomationTool/Saved/Telemetry`).
AutomationTool and UnrealBuildTool write timing information for their
execution to this folder using the in JSON format, which are replicated to
the configured telemetry provider.

- `step-testdata`: Data written to the directory given by the
`UE_TESTDATA_DIR` environment variable by a build step (normally
`Engine/Programs/AutomationTool/Saved/TestData`). Json files in this
directory will also be harvested for test data results and surfaced to
Automation Hub.

### Expiration

Artifacts may be expired automatically using the following settings:

- [KeepCount](Schema/Globals.md#artifacttypeconfig): Retains the most recent `N` artifacts of a particular type in each stream.
- [KeepDays](Schema/Globals.md#artifacttypeconfig): Retains each artifact for `N` days.

If both settings are specified, the artifact will be retained until neither retention policy applies.

If the configuration for an artifact is removed from Horde - because a
stream is deleted, for example - any orphanned artifacts will be expired
according to the last configured settings.

## Uploading Artifacts

Artifacts can be produced by build processes using the `CreateArtifact`
BuildGraph task within a node, or using the `Artifact` element to indicate
that a produced tag should be published as an artifact.

Artifacts produced by build jobs will be listed at the top of the dashboard page
for that job.

An example of using the `CreateArtifact` task can be found in the
`Engine/Build/InstalledEngineBuild.xml` script, which uploads the complete
installed build as an `installed-build` artifact type:

  ```xml
  <Node Name="Upload To Horde Win64" Requires="Make Installed Build Win64">
      <CreateArtifact
          Name="installed-build-win64"
          Type="installed-build"
          Description="Installed Build (Windows)"
          BaseDir="$(LocalInstalledDir)"
          Files="..."/>
  </Node>
  ```

An example of using the `Artifact` element can be found in the
`Engine/Build/Graph/Examples/BuildEditorAndTools.xml` script, which
uploads a set of files as a `ugs-pcb` artifact type. UnrealGameSync
searches for this artifact type to determine which archives are available,
along with their associated project and archive type.

  ```xml
  <Artifact
      Name="editor"
      Description="Editor PCBs"
      Type="ugs-pcb"
      BasePath="$(ArchiveStagingRelativeDir)"
      Keys="$(UgsProjectKey)"
      Metadata="ArchiveType=Editor"
      Tag="#PublishBinaries"/>
  ```

Artifacts may also be uploaded programmatically using the `EpicGames.Horde` library.

## Downloading Artifacts

The easiest way to download an artifact is via the Horde dashboard. While
convenient, downloading a zip file directly from the dashboard can be slow
and inefficient due to the work the server has to do to re-pack data in
that format.

To download artifacts in their native format - which is typically much
faster and more efficient - the following methods are available:

- **UnrealGameSync**: The Horde dashboard allows downloading an artifact as a
  `.uartifact` file; a JSON file containing metadata about an artifact.
  Double-clicking on these files will open them in UnrealGameSync and allow
  downloading them to a local folder.

- **Command line**: The Horde command line tool allows downloading artifacts to an output folder using the following syntax:

    ```bat
    horde artifact download -Id=12345678abcd12345678abcd -OutputDir=C:\Folder
    ```

  Artifacts can be enumerated using the `artifact find` command:

    ```bat
    horde artifact find -streamid=ue5-main -type=step-saved -jobid=670fce208961cf36f3855ff0 -stepid=4994
    ```

- **Programmatically**: The `EpicGames.Horde` library provides interfaces for
  enumerating and downloading artifacts programmatically. The Horde command-line
  tool serves as an example for using this API to query artifacts.

The `RetrieveArtifact` task can be used to find and download artifacts from
a build script. By default, the task will search for an artifact in the current
stream, at the current changelist - though the `Stream` and `Commit` attributes
can be used to choose a different source. 

  ```xml
  <RetrieveArtifact
      Name="build-name"
      Type="installed-build"
      OutputDir="My/Output/Dir" />
  ```

Additionally, specifying the `MaxCommit` parameter will find the most recent artifact at or 
before the given change.

## Keys & Metadata

Artifacts are designed to be consumed programmatically and include various
metadata fields allowing them to be manipulated in a standard way, such as a
name, type, stream, and commit identifier.

Artifacts also support user-defined **keys and metadata**. Both are arbitrary
strings attached to an artifact, with the difference that keys are globally
indexed and support searching, and metadata is not.

All artifacts uploaded by a particular job will automatically have
`job:12345678abcd12345678abcd` and `job:12345678abcd12345678abcd/step:abcd` keys
added to them.
