[Horde](../Home.md) > [Configuration](../Config.md) > Automation Hub

# Automation Hub

> **_Note:_** This view is deprecated. The version 2 named Test Automation Hub should be used instead.
> To keep using this view, add this argument to the Gauntlet command line `-UseTestDataV1`.

The Horde Automation Hub surfaces individual and suite [Gauntlet](https://docs.unrealengine.com/5.3/en-US/gauntlet-automation-framework-overview-in-unreal-engine)
test results.  Horde efficiently generates searchable metadata for
streams, platforms, configurations, rendering apis, etc.  The automation
hub is used at Epic by QA, release managers, and code owners to quickly
view and investigate the latest test results across platforms and
streams.  It provides historical data and views which drill into specific
test events which can include screenshots, logging, and callstacks.

In addition to enabling Horde [build automation](BuildAutomation.md), the
configuration required to surface test results is simply adding the  
`-WriteTestResultsForHorde` Gauntlet test command line argument.  Please
see the BuildGraph [example](#buildgraph-example) below for details.

## Automation Filters

The automation UI is data driven and automatically populated from test
metadata.  Test results can be filtered by project, tests, streams,
platforms, configurations, targets, rendering hardware interface, and
variations.  Detailed selections may also be linked for sharing or to be
bookmarked.  

![Automation Selection](../Images/AutomationHub-LeftPanel.png)

## Test Tiles

Test results are presented as tiles which surface relative test health based on the platforms and streams selected.  

![Test Results](../Images/AutomationHub-TestResults.png)

Test tiles can be expanded to view further details such as platforms and
changelists, which are linked to individual [Horde CI](BuildAutomation.md)
job steps to assist in issue investigation.  

![Test Card](../Images/AutomationHub-TestPanel.png)

Test history graphs and detailed failure reports are also available which include logging and callstacks.  

![Test History](../Images/AutomationHub-TestHistory.png)

## Test Suites

Gauntlet suite tests may contains thousands of individual unit tests.  The
automation hub can drill into historical data for each unit test with
cross stream comparison.  

![Suite Results](../Images/AutomationHub-SuiteResults.png)

Suite tests generate test events which can  be viewed to help diagnose
issues.  Test events can include additional data such as logging and
screenshots to catch regressions.  Alternate platforms of the unit test
may also be selected for comparison purposes.  

![Screenshot Comparison](../Images/AutomationHub-ScreenShotCompare.png)

## BuildGraph Example

The following [**BuildGraph**](https://docs.unrealengine.com/5.0/en-US/buildgraph-for-unreal-engine/) fragment declares:

* `HordeDeviceService` and `HordeDevicePool` properties that specify your Horde server and which device pool to use.
* Adds a `BootTest Android` node which specifies
`-WriteTestResultsForHorde` and will automatically generate test data to
be ingested by Horde, parsed to efficient meta data, and surfaces by the
automation hub

```xml
    <Property Name="HordeDeviceService" Value="http://localhost:13440" />
    <Property Name="HordeDevicePool" Value="UE5" />

    <Node Name="BootTest Android">
        <Command Name="RunUnreal" Arguments="-test=UE.BootTest -platform=Android -deviceurl=&quot;$(HordeDeviceService)&quot; -devicepool=&quot;$(HordeDevicePool)&quot; -WriteTestResultsForHorde"/>
    </Node>
```
