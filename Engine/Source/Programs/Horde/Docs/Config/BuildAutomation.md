[Horde](../../README.md) > [Configuration](../Config.md) > Build Automation

# Build Automation

**Build Automation** (aka CI/CD) was the first use case targeted for Horde at Epic and the most mature.

It is designed to support [**BuildGraph**](https://docs.unrealengine.com/en-US/buildgraph-for-unreal-engine/) as
a first-class citizen, allowing efficient distributed and parallelized build pipelines to be scripted, with automatic
tracking and transfer of intermediate build artifacts between agents.

It also supports the following features:

* [Build Health](BuildHealth.md)
* Perforce metadata caching and workspace management
* Support for autoscaling based on job queues
* Support for structured logging, with many common UE-types automatically annotated with additional metadata.
* Profiling and telemetry functionality

CI works hand-in-hand with Horde's remote execution capabilities to distribute compute-heavy workloads between multiple
agents, and the storage functionality to archive and retrieve cached, intermediate and final artifacts.

## BuildGraph

BuildGraph describes the build pipeline as a parameterized graph, with each node in the graph performing some set of
sequential operations on a set of inputs (in the form of files produced by another node) and producing a set of outputs
(in the form of a set of files). One or more nodes execute sequentially on an agent with a synced Perforce workspace.

When running a job on Horde, you specify a BuildGraph script, any command line arguments to pass to it, and the name
of one or more nodes to evaluate. Horde takes care of provisioning machines, syncing from Perforce, and transferring
inputs and outputs to temporary storage.

## Configuration

To enable CI functionality in Horde, you will need to perform the following steps:

* Write a BuildGraph script and submit it to source control.
* Define a project in the [`Globals.json`](Schema/Globals.md) file, which includes a [`*.project.json`](Schema/Projects.md)
  configuration file.
* Define a stream in the [`.project.json`](Schema/Projects.md) file, which includes a [`*.stream.json`](Schema/Streams.md)
  configuration file.
* Declare an [`agent type`](#agent-types) that defines a machine that can execute steps in your BuildGraph script.
* Declare a [`job template`](#job-templates) that defines parameters for your BuildGraph script and references it in
  source control.

See also: [Quick Start - Build Automation](../Tutorials/BuildAutomation.md).

## Agent Types

Agent type specifies the machines desired to execute a particular type of work, given a name specified in a BuildGraph script.
This layer of indirection can be useful when running the same BuildGraph script in multiple streams or parameterizing the 
same BuildGraph script to run for different projects.

For ease of management, agents are typically grouped into static or dynamic [pools](Agents.md#pools) for inclusion in
an agent-type definition. Agent types can also reference a [workspace](#workspaces) that specifies which files need to
be synced on the machine to execute a job. Agents configured for use in a particular stream will keep a 'hot'
sync of the data from that stream, allowing them to start servicing a job more quickly when required, so filtering the
files to sync can reduce disk space and allow higher packing of workspaces per machine.

For more information on configuring agent types, see the [.stream.json reference](Schema/Streams.md#agentconfig).

## Job Templates

A job template describes the parameters and default arguments required to run a job and the widgets the Horde
dashboard can use to present these parameters to the user. A typical template is as follows:

        {
            "id": "editor-only",
            "name": "Editor Only",
            "arguments": [
                "-Target=Editor Only",
                "-Script=Engine/Restricted/NotForLicensees/Build/DevStreams.xml"
            ],
            "parameters": [
                {
                    "type": "List",
                    "style": "List",
                    "label": "Toolchain Versions",
                    "items": [
                        {
                            "text": "Latest",
                            "argumentIfEnabled": "-set:WithLatest=true",
                            "argumentIfDisabled": "-set:WithLatest=false",
                            "default": true
                        },
                        {
                            "text": "Preview",
                            "argumentIfEnabled": "-set:WithPreview=true",
                            "argumentIfDisabled": "-set:WithPreview=false",
                            "default": true
                        },
                        {
                            "text": "Clang",
                            "argumentIfEnabled": "-set:WithClang=true",
                            "argumentIfDisabled": "-set:WithClang=false",
                            "default": true
                        }
                    ]
                }
            ]
        }

You can specify various widget types in the `parameters` block. Screenshots and config excerpts below are taken
from the Parameter Demo job type in the tutorial.

### Text Parameters

Allows entering arbitrary text for an argument, with an optional regex for validation.

![Text Parameter](../Images/NewBuild-Param-Text.png)

    {
        "type": "Text",
        "label": "Text Parameter",
        "argument": "-set:TextParameter=",
        "default": "",
        "validation": "[a-zA-Z0-9_]+",
        "validationError": "Should be a sequence of alphanumeric characters",
        "hint": "Enter a string of alphanumeric characters",
        "toolTip": "Tooltip for text parameter"
    }

See [TemplateTextParameterConfig](Schema/Streams.md#templatetextparameterconfig) for valid properties.

### List Parameters

Allows the user to select one or more options from a predefined list.

![Text Parameter](../Images/NewBuild-Param-List.png)

    {
        "type": "List",
        "label": "List Parameter (Default Style)",
        "toolTip": "Tooltip for list parameter",
        "items": [
            {
                "text": "Option 1",
                "argumentIfEnabled": "-set:ListParameter=Option1"
            },
            {
                "text": "Option 2",
                "argumentIfEnabled": "-set:ListParameter=Option2",
                "default": true
            },
            {
                "text": "Option 3",
                "argumentIfEnabled": "-set:ListParameter=Option3"
            }
        ]
    }

See [TemplateListParameterConfig](Schema/Streams.md#templatelistparameterconfig) for valid properties.

### Multi-List Parameters

Multi-list parameters are created as regular lists but have the `Style` property set to `MultiList`. You can specify multiple 
options by checking items from the drop-down list. The `Group` property on list items specifies a heading
to group items underneath.

![Multi-List Parameter](../Images/NewBuild-Param-List-MultiList.png)

    {
        "type": "List",
        "style": "MultiList",
        "label": "List Parameter (MultiList Style)",
        "toolTip": "Tooltip for list parameter (MultiList)",
        "items": [
            {
                "group": "First group",
                "text": "Option 1",
                "argumentIfEnabled": "-set:MultiListOption1=true",
                "default": true
            },
            {
                "group": "First group",
                "text": "Option 2",
                "argumentIfEnabled": "-set:MultiListOption2=true"
            },
            {
                "group": "Second group",
                "text": "Option 3",
                "argumentIfEnabled": "-set:MultiListOption3=true",
                "default": true
            },
            {
                "group": "Second group",
                "text": "Option 4",
                "argumentIfEnabled": "-set:MultiListOption4=true"
            }
        ]
    },

See [TemplateListParameterConfig](Schema/Streams.md#templatelistparameterconfig) for valid properties.

### Tag-Picker Parameters

Tag-picker parameters are similar to [multi-list parameters](#multi-list-parameters) but enable item selection
by typing a few characters rather than picking from a list.

![Tag-Picker Parameter](../Images/NewBuild-Param-List-TagPicker.png)

    {
        "type": "List",
        "style": "TagPicker",
        "label": "List Parameter (TagPicker Style)",
        "toolTip": "Tooltip for list parameter (TagPicker)",
        "items": [
            {
                "text": "Option 1",
                "argumentIfEnabled": "-set:TagPickerOption1=true",
                "default": true
            },
            {
                "text": "Option 2",
                "argumentIfEnabled": "-set:TagPickerOption2=true"
            },
            {
                "text": "Option 3",
                "argumentIfEnabled": "-set:TagPickerOption3=true",
                "default": true
            },
            {
                "text": "Option 4",
                "argumentIfEnabled": "-set:TagPickerOption4=true"
            }
        ]
    }

### Bool Parameters

Allows toggling whether to enable an option or not.

![Bool Parameter](../Images/NewBuild-Param-Bool.png)

    {
        "type": "Bool",
        "label": "Bool Parameter",
        "toolTip": "Tooltip for bool parameter",
        "argumentIfEnabled": "-set:BoolParameter=true",
        "argumentIfDisabled": "-set:BoolParameter=false"
    }

See [TemplateBoolParameterConfig](Schema/Streams.md#templateboolparameterconfig) for valid properties.

## Artifacts

See [Configuration > Artifacts](Artifacts.md).

## Build Health

See [Configuration > Build Health](BuildHealth.md).

## Schedules

Templates may also specify a schedule and policy on which to trigger automatically; running for every submitted change,
for the last `n` changes, whenever certain files are modified, and so on. For more information, see the
[ScheduleConfig](Schema/Streams.md#scheduleconfig) section in the config reference.

## Workspaces

Workspace definitions tell an agent what and how to sync data from source control to execute a job.

The `view` property refines the stream definition with an additional set of filters to apply to the stream
view, with wildcards specified in standard Perforce syntax.

The `incremental` property defines incremental workspaces. By default, Horde will return a workspace to a
pristine state before running a job. If this flag is set, any tracked files will be synced back to a pristine
state (i.e. any modified files, as determined by timestamp, will be reset to their original version), but other files in
the workspace will be left alone. This allows keeping intermediate state, such as compile artifacts, when tools are
robust enough in their dependency tracking to determine whether files need to be rebuilt.

## AutoSDK

The AutoSDK system provides a mechanism for distributing target platform SDKs and configuring them for use by the
engine on demand. This is particularly useful when managing multiple code lines on a build farm, as it reduces the
amount of manual configuration required for build agents.

For more information about creating an AutoSDK repository, see the documentation under `Engine/Extras/AutoSDK` in the
Unreal Engine source tree.

AutoSDK can be configured for Horde from the `perforceClusters` section of the globals.json file. Different AutoSDK
streams may be specified according to property matches from the agent, for example:

    "autoSdk": [
        {
            "properties": [ "OSFamily=Windows" ],
            "stream": "//UE5/Private-AutoSDK-Windows"
        },
        {
            "properties": [ "OSFamily=MacOS" ],
            "stream": "//UE5/Private-AutoSDK-Mac"
        },
        {
            "properties": [ "OSFamily=Linux" ],
            "stream": "//UE5/Private-AutoSDK-Linux"
        }
    ]
