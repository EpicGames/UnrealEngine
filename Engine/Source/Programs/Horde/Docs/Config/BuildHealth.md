[Horde](../../README.md) > [Configuration](../Config.md) > [Build Automation](BuildAutomation.md) > Build Health

# Build Health

Horde's Build Health system is an automated issue tracking and triage pipeline for
build breakages. It ingests structured log events from build steps, runs
**issue handlers** to generate **fingerprints**, groups those fingerprints into
**issues**, identifies **suspect commits**, and notifies developers via **Slack**.

It works best with trunk-based development where commits to release branches are
merged back to mainline, and scales to large teams where manual triage is slow.

## Anatomy of a Build Breakage

Each branch is modeled as a linear sequence of commits. When a build failure occurs,
Horde identifies which commit since the last successful build is responsible using
heuristics such as:

* Matching modified source files or assets to compile/cook errors.
* Matching linker symbol names to modified source file names.
* Tracing merge history to find the originating commit across branches.

A single change can cause widespread errors — different compiler diagnostics on
MSVC vs Clang vs GCC, cook failures across platforms, etc. Build Health groups
these together and gets eyes on them quickly while avoiding duplicate notifications.

## Marking Issues as Fixed

Issues are marked as fixed automatically after a successful build, but users can
also indicate a fix via:

* **Slack** — interactive buttons on the issue thread.
* **Horde Dashboard** — the issue detail view.
* **Commit tags** — including `#horde 1234` on a separate line in a commit description.

The tag name can be customized via `IssueFixedTag` in
[globals.json](../Config/Schema/Globals.md#buildconfig).

## Issues, Spans, and Fingerprints

Horde creates a build health **issue** for each detected breakage.

Each issue contains multiple **spans** — one per stream the issue is observed in —
identifying the last successful and first failing commit. A span resolves once the
error no longer occurs; the issue resolves once all spans resolve.

Errors are attached to spans via their **fingerprint**, which describes the error
programmatically and controls how it matches with other errors.

### Contents of a Fingerprint

See `IIssueFingerprint.cs` for the source definition.

| Field | Description |
|-------|-------------|
| **Type** | Identifies the issue type. Fingerprints only match others of the same type (e.g. `Compile`, `Content`, `Linker`). |
| **SummaryTemplate** | Populates the issue description. May contain [placeholders](#summary-templates). |
| **Keys** | Identifying data (e.g. filenames) used to match and group fingerprints. |
| **RejectKeys** | Keys this fingerprint should _not_ match with. |
| **Metadata** | Arbitrary key/value pairs for summary templates and debugging. |
| **ChangeFilter** | Semicolon-delimited file wildcards for suspect identification (e.g. `*.c;*.cpp;*.h`). |

### Matching and Merging Fingerprints

1. A fingerprint is generated for each error or warning in a build step.
2. Each is compared against unresolved spans in the same stream. A match requires:
   - Identical type and summary template.
   - At least one common key (or both have no keys).
   - No keys in the other's reject set.
3. Unmatched fingerprints are grouped using relaxed conditions (identical type,
   no conflicting reject keys).
4. For new spans, suspect commits are found via merge history. If an existing issue
   shares a fingerprint and suspect commit, the span joins that issue; otherwise a
   new issue is created.

### Creating Fingerprints

Fingerprints can be created by:

* Including fingerprint data directly in a
  [structured log event](../Internals/StructuredLogging.md) (preferred — most control).
* Post-processing structured log events on the Horde server after a step completes.

#### Adding Fingerprints to Errors (C#)

The `EpicGames.Horde` library provides extension methods using the standard .NET
`ILogger` interface (exposed via `CommandUtils.Logger` in AutomationTool):

```cs
IssueFingerprint fingerprint = new IssueFingerprint(
    "Compile",
    "Compile {Severity} in {Files}",
    IssueChangeFilter.Code);
fingerprint.Keys.Add(IssueKey.FromFile("Foo.cpp"));

using (IDisposable scope = Logger.BeginIssueScope(fingerprint))
{
    Logger.LogError("Compile errors in {File}", new FileReference("Foo.cpp"));
}
```

#### Adding Fingerprints via Post-Processing

Post-processing uses `IssueHandler` derived classes with the `[IssueHandler]`
attribute. A new handler instance is created per build step. Log events are passed
to `HandleEvent` in priority order until one returns `true`; then `GetIssues`
returns the matched events and fingerprints.

Handlers can return `true` without generating a fingerprint to mask vague errors
(e.g. a failing exit code) when a more specific handler (e.g. compile error) has
already claimed the root cause.

### Issue Handlers

Built-in handlers:

| Handler | Description | Event Types |
|---------|-------------|-------------|
| `CompileIssueHandler` | Multi-line Clang, MSVC, static analyzer errors; groups by source file. | `Compiler`, `UHT`, `SourceFileLine`, `MSBuild` |
| `ContentIssueHandler` | Parses `UE_ASSET_LOG` output; identifies modified assets. | `Engine_AssetLog` |
| `CopyrightIssueHandler` | `CheckCopyrightNotices` UAT command output. | `AutomationTool_MissingCopyright` |
| `DefaultIssueHandler` | Catch-all; groups unhandled events in the same step. | Any |
| `GauntletIssueHandler` | Parses Gauntlet test failure names. | `Gauntlet_TestEvent`, `Gauntlet_DeviceEvent`, etc. |
| `HashedIssueHandler` | Redacts numeric/timestamp patterns; groups by hash. | Any (>30 chars after redaction) |
| `LocalizationIssueHandler` | UE localization command output. | `Engine_Localization` |
| `PerforceCaseIssueHandler` | `CheckPerforceCase` UAT command output. | `AutomationTool_PerforceCase` |
| `ScopedIssueHandler` | Uses scoped fingerprints in log events. Enable via `"Scoped"` tag. | Any (tagged) |
| `ShaderIssueHandler` | Shader compilation output. | `Engine_ShaderCompiler` |
| `SymbolIssueHandler` | Matches linker symbols to modified source files. | `Linker`, `Linker_DuplicateSymbol`, `Linker_UndefinedSymbol` |
| `SystemicIssueHandler` | Infrastructure/flaky test issues. | `Systemic` through `Systemic_Max` |
| `UnacceptableWordsIssueHandler` | `CheckUnacceptableWords` UAT command output. | `AutomationTool_UnacceptableWords` |
| `SanitizerIssueHandler` | ASAN, TSAN, etc. output. | Sanitizer events |

Workflows can opt into specific handlers via the `issueHandlers`
[workflow property](#other).

### Summary Templates

| Variable | Description |
|----------|-------------|
| `{Severity}` | `Warnings` or `Errors` based on highest severity. |
| `{Files}` | Up to 3 filenames from `File`-type keys. |
| `{Meta:Key}` | Metadata values for the given key. |

---

## Workflows

Workflows define how and where developers are notified about build health issues.
They are configured in the `workflows` array of the
[stream configuration](Schema/Streams.md) and assigned to jobs via the template's
`workflowId` property. Individual BuildGraph nodes can override the workflow via
[annotations](#annotations).

Settings cascade: **stream** > **workflow** (including default `annotations`) >
**template** (`workflowId`, `triageChannel`, `updateIssues`) > **node**
(BuildGraph annotations).

### Notification Lifecycle

1. **New issue** — Thread created in `triageChannel` with `triagePrefix`.
   `triageAlias` is @mentioned; `triageInstructions` posted as follow-up.
2. **Suspects** — If `allowMentions` is true, up to `maxMentions` developers
   are @mentioned.
3. **Escalation** — After `escalateTimes` minutes, `escalateAlias` is pinged.
4. **Reports** — At each `reportTimes`, a summary posts to `reportChannel`.
5. **Resolution** — Thread updated when all spans resolve.

Slack parameters require **IDs** (channel ID, user/group ID), not display names.

### Workflow Property Reference

#### Identity

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `id` | `string` | (required) | Unique identifier referenced by templates via `workflowId`. |
| `summaryTab` | `string` | `null` | Dashboard tab name in the Build Health view. |

#### Triage

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `triageChannel` | `string` | `null` | Slack channel ID for new issue threads. |
| `triagePrefix` | `string` | `"*[NEW]* "` | Text prepended to triage messages. |
| `triageSuffix` | `string` | `null` | Text appended to triage messages. |
| `triageInstructions` | `string` | `null` | Developer guidance posted in the thread. Supports Slack markup. |
| `triageAlias` | `string` | `null` | Slack user/group ID to @mention on new issues. |
| `triageTypeAliases` | `object` | `null` | Map of issue type to Slack ID for type-specific routing (e.g. `{"Systemic": "SYYYYYYYYYY"}`). |
| `triageErrors` | `bool` | `true` | Create triage threads for errors. |
| `triageWarnings` | `bool` | `true` | Create triage threads for warnings. |

#### Reports

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `reportTimes` | `string[]` | `[]` | UTC times for summary reports (`"HH:mm:ss"`). |
| `reportChannel` | `string` | `null` | Slack channel ID for reports (can differ from triage). |
| `reportWarnings` | `bool` | `true` | Include warnings in reports. |
| `groupIssuesByTemplate` | `bool` | `true` | Group issues by job template in reports. |
| `skipWhenEmpty` | `bool` | `false` | Skip report when no active issues. |

#### Escalation

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `escalateAlias` | `string` | `null` | Slack user/group ID for escalation. |
| `escalateTimes` | `int[]` | `[120]` | Minutes after creation to escalate. Multiple values = tiered escalation (e.g. `[120, 1440]`). |
| `escalateRootCauseAnalysisTimes` | `int[]` | `[1440]` | Minutes to escalate issues missing an RCA. |

#### Mentions

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `allowMentions` | `bool` | `true` | @mention suspected developers. Set `false` for info-only channels. |
| `maxMentions` | `int` | `5` | Max users to @mention per issue. |
| `inviteRestrictedUsers` | `bool` | `false` | Invite external/contractor Slack users to channels. |

#### External Issue Tracking

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `externalIssues.projectKey` | `string` | `null` | Project key in external tracker (e.g. Jira). `""` disables creation. |
| `externalIssues.defaultComponentId` | `string` | `""` | Default component for created issues. |
| `externalIssues.defaultIssueTypeId` | `string` | `""` | Default issue type for created issues. |

#### Other

| Property | Type | Default | Description |
|----------|------|---------|-------------|
| `issueHandlers` | `string[]` | `null` | Handler tags to enable (e.g. `["Scoped"]`). `null` = all defaults. |
| `autoRcaEnabled` | `bool` | `false` | Enable automatic root cause analysis. |
| `showMergeWarnings` | `bool` | `false` | Show warnings for issues introduced via merges. |
| `annotations` | `object` | `{}` | Default [node annotations](#annotations) for all nodes in this workflow. |

---

## Workflow Examples

### Minimal

Posts triage notifications to a single channel:

```json
{
    "workflows": [
        {
            "id": "default",
            "triageChannel": "CXXXXXXXXXX",
            "triagePrefix": "*[CI]*"
        }
    ]
}
```

### CI with Reports and External Issues

Adds scheduled reports and external tracker integration:

```json
{
    "workflows": [
        {
            "id": "ci-build-health",
            "summaryTab": "CI Build Health",
            "reportTimes": ["09:00:00", "21:00:00"],
            "reportChannel": "CXXXXXXXXXX",
            "triageChannel": "CXXXXXXXXXX",
            "triagePrefix": "*[CI]*",
            "externalIssues": { "projectKey": "MYPROJECT" },
            "allowMentions": false
        }
    ]
}
```

### High-Priority with Escalation and Auto-RCA

For the main development branch where breakages must be fixed immediately:

```json
{
    "workflows": [
        {
            "id": "high-priority-ci",
            "summaryTab": "High Priority CIS",
            "reportTimes": ["03:00:00", "09:00:00", "15:00:00", "21:00:00"],
            "reportChannel": "CXXXXXXXXXX",
            "triageChannel": "CYYYYYYYYYY",
            "triagePrefix": "*[Main CIS]*",
            "triageInstructions": "If you caused this issue, it needs to be backed out immediately. Please acknowledge and fix as soon as possible.",
            "triageAlias": "SXXXXXXXXXX",
            "triageTypeAliases": {
                "Systemic": "SYYYYYYYYYY"
            },
            "escalateAlias": "UYYYYYYYYYY",
            "escalateTimes": [120, 1440],
            "externalIssues": { "projectKey": "MYPROJECT" },
            "allowMentions": true,
            "inviteRestrictedUsers": true,
            "autoRcaEnabled": true
        }
    ]
}
```

Key features demonstrated:
* `triageTypeAliases` routes `Systemic` issues to a dedicated infra team.
* Two-tier escalation: 2 hours, then 24 hours.
* `autoRcaEnabled` activates automatic root cause analysis.

### Low-Noise with Skip-When-Empty

For secondary streams where breakages are infrequent:

```json
{
    "workflows": [
        {
            "id": "dev-stream-health",
            "summaryTab": "Dev Stream",
            "reportTimes": ["06:00:00"],
            "reportChannel": "CXXXXXXXXXX",
            "triageChannel": "CXXXXXXXXXX",
            "externalIssues": { "projectKey": "MYPROJECT" },
            "allowMentions": true,
            "skipWhenEmpty": true
        }
    ]
}
```

### Multiple Workflows per Stream

A real-world pattern: different job categories route to different channels and teams.

```json
{
    "workflows": [
        {
            "id": "ci-high-priority",
            "summaryTab": "High Priority CIS",
            "reportTimes": ["09:00:00", "15:00:00", "21:00:00"],
            "reportChannel": "CXXXXXXXXXX",
            "triageChannel": "CYYYYYYYYYY",
            "triagePrefix": "*[Main CIS]*",
            "triageInstructions": "Back out immediately if you caused this.",
            "escalateAlias": "SXXXXXXXXXX",
            "escalateTimes": [120],
            "externalIssues": { "projectKey": "MYPROJECT" },
            "allowMentions": true,
            "inviteRestrictedUsers": true
        },
        {
            "id": "scheduled-builds",
            "summaryTab": "Scheduled Builds",
            "reportTimes": ["08:00:00", "14:00:00"],
            "reportChannel": "CZZZZZZZZZZ",
            "triageChannel": "CWWWWWWWWWW",
            "triagePrefix": "*[Scheduled Build]*",
            "escalateTimes": [480, 1440],
            "externalIssues": { "projectKey": "MYPROJECT" },
            "allowMentions": false
        },
        {
            "id": "qa-tracking",
            "summaryTab": "QA Tracking",
            "reportTimes": ["09:00:00", "18:00:00"],
            "triageChannel": "CVVVVVVVVVV",
            "triagePrefix": "*[QA]*",
            "triageWarnings": false,
            "reportWarnings": false,
            "issueHandlers": ["Scoped"],
            "allowMentions": false
        }
    ]
}
```

Templates reference workflows by ID. Set `updateIssues: true` on templates that
are not regular CI but should still participate in build health tracking:

```json
{
    "templates": [
        {
            "id": "incremental-ci",
            "name": "Incremental CI",
            "workflowId": "ci-high-priority",
            "arguments": ["-Target=CI Targets", "-Script=Build/CI.xml"]
        },
        {
            "id": "nightly-build",
            "name": "Nightly Build",
            "workflowId": "scheduled-builds",
            "updateIssues": true,
            "arguments": ["-Target=Full Build", "-Script=Build/FullBuild.xml"]
        }
    ]
}
```

---

## Annotations

Issue handling for individual job steps can be configured via **node annotations**
in BuildGraph scripts:

```xml
<Node Name="Compile Editor Win64" Annotations="Workflow=my-workflow;BuildBlocker=true">
```

| Annotation | Type | Description |
|------------|------|-------------|
| `Workflow` | `string` | Workflow ID for this node. Overrides template-level `workflowId`. |
| `CreateIssues` | `bool` | Set `false` to disable issue creation. |
| `CreateWarningIssues` | `bool` | Set `false` to disable warning-level issues. |
| `AutoAssign` | `bool` | Auto-assign when only one suspect or a clear file correlation exists. |
| `AutoAssignToUser` | `string` | Always assign issues from this step to the given user. |
| `NotifySubmitters` | `bool` | Notify all submitters between success and failure. |
| `IssueGroup` | `string` | Suffix for the issue type, preventing merging with other issues. |
| `BuildBlocker` | `bool` | Flag failures as build blockers (shown prominently in Slack). |

Multiple annotations are semicolon-separated. Annotations can also be set as
defaults in a workflow's `annotations` property.

---

## Ignore Patterns

**Ignore patterns** suppress known, non-actionable warnings or errors from Build
Health. Matched lines are demoted to informational severity — they still appear in
logs but won't create issues or notifications.

Each suppressed line is tagged with:

| Property | Description |
|----------|-------------|
| `IgnoredLine` | Original text of the suppressed line |
| `IgnorePatternFile` | Path to the pattern file that matched |
| `IgnorePattern` | The regex that matched |

### File Format

One .NET regex per line. `#` lines are comments; blank lines are ignored.

```
# Suppress known third-party deprecation warnings
warning C4996: 'ThirdPartyLib::DeprecatedFunc'.*was declared deprecated

# Intermittent linker note on Mac
ld: warning: object file.*was built for newer macOS version

# Experimental shader feature behind a flag
LogShaderCompiler: Warning:.*EXPERIMENTAL_FEATURE
```

A 200ms timeout is enforced per match to guard against catastrophic backtracking.

### File Discovery

Agents scan for `Build/Horde/IgnorePatterns.txt` in standard locations:

| Location | Example |
|----------|---------|
| Engine | `Engine/Build/Horde/IgnorePatterns.txt` |
| Project directories | `MyProject/Build/Horde/IgnorePatterns.txt` |
| Platform subdirectories | `Engine/Platforms/Linux/Build/Horde/IgnorePatterns.txt` |

All discovered files are merged. Additional files can be referenced via
`ignorePatternFiles` in project or stream config:

```json
{
    "ignorePatternFiles": [
        "MyProject/Config/Horde/SuppressedWarnings.txt"
    ]
}
```

See [ProjectConfig](Schema/Projects.md) and [StreamConfig](Schema/Streams.md)
for the full schema.
