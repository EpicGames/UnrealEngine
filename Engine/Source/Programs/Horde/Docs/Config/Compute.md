[Horde](../../README.md) > [Configuration](../Config.md) > Compute

# Compute

Compute is a core plugin in Horde that manages compute resources in the form of agents.
These agents are essential for two primary functions:
powering build automation jobs and running remote execution tasks.

To structure management of these agents, Horde allows the creation of pools, which can be configured to include different sets of agents.
In addition to this, compute clusters serve a similar purpose but are specifically designed for handling remote compute tasks.

More details about pools can be found on the [Agents](Agents.md) page.

## Configuring
The two most important points of configuration are pools and compute clusters. 

### Pools

Below is a pool configured for AWS using the job queue for appropriate sizing.
`ScaleOutFactor`/`ScaleInFactor` are parameters for the `JobQueueStrategy` and
describe how aggressive scaling should be (at which rate/how many EC2 instances to add or remove).
```json
{
  "plugins": {
    "compute": {
      "pools": [
        {
          "id": "win-ue5",
          "name": "Win-UE5", // Human-readable name
          "enableAutoscaling": true,
          "conformInterval": "1.00:00:00", // Automatically conform every 24 hours
          "sizeStrategy": "JobQueue",
          "sizeStrategies": [
            {
              "type": "JobQueue",
              "condition": "true",
              "config": "{\"ScaleOutFactor\": 1, \"ScaleInFactor\": 0.1}",
              "extraAgentCount": 1
            }
          ],
          "fleetManagers": [{ "type": "AwsRecycle", "config": "{}" }]
        }
      ]
    }
  }
}
```
For a complete of fields a pool can be configured with, see [PoolConfig](Schema/Globals.md#poolconfig) schema.

### Compute Clusters

A compute cluster in Horde is a logical grouping of agents designed to help organize remote execution,
manage permissions, and storage of compute tasks.

Compute tasks are the core units of work within a cluster. Each task describes an OS process to be executed remotely on an agent.
These descriptions include specific resource requirements such as CPU or RAM allocation, files that need to be transferred,
the executable entrypoint, arguments, and any necessary environment variables.
Typically, compute tasks are designed to be short-lived, with most completing within a 1-15 minute timeframe.

The Unreal Build Tool together with Unreal Build Accelerator leverages compute tasks in Horde to distribute CPU-intensive operations.
This includes C++ and shader compilation allowing for more efficient use of available resources beyond what's available on single a local machine.
This distributed computing approach is one of the cornerstones for accelerating build processes,
whether initiated through Horde jobs or from individual developer workstations.
By leveraging the power of the cluster, teams can significantly reduce build times and improve overall development efficiency.

When setting up a compute cluster, it's important to note that permissions are not granted by default.
Each user or service that requires access to the cluster must be explicitly granted permissions.
This ensures a secure environment and allows for fine-grained control over who can utilize the cluster's resources.

An example compute cluster may be configured as follows in the global config:

```json
{
  "plugins": {
    "compute": {
      "clusters": [
        {
          // Arbitrary name for referencing this cluster
          "id": "uba",
          
          // Use condition to select which agents to include
          "condition": "pool == 'UBA-HQ' && osfamily == 'Linux'",

          // Each user or service that needs access must be explicitly granted permission
          // In almost all cases, this means the "AddComputeTasks" action
          "acl": {
            "entries": [
              { "claim": {"type": "http://epicgames.com/ue/horde/user", "value": "jane.doe"}, "actions": ["AddComputeTasks"] },
              { "claim": {"type": "http://schemas.microsoft.com/ws/2008/06/identity/claims/role", "value": "programmers"}, "actions": ["AddComputeTasks"] }
            ]
          }
        }
      ]
    }
  }
}
```

## Running remote compute tasks

For those looking to understand the basic functionality of compute tasks,
the Horde CLI tool provides a simple example through its `compute run` command.
This command demonstrates how to initiate and manage a basic compute task,
serving as a starting point for developers new to Horde.

Unreal Build Tool implements Horde's compute client in C# for allocating tasks for UBA.
See the [remote C++ tutorial](../Tutorials/RemoteCompilation.md) for getting started.
