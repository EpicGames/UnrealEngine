[Horde](../../README.md) > [Configuration](../Config.md) > Conditions

# Conditions

## Introduction

In several parts of Horde, you can specify a query condition to select particular agents for an operation. These
condition strings are stored in JSON configuration flies as a string and have a C-like syntax.

Agents report many properties collected from their characteristics, which you can view from the
Agents page on the Horde dashboard. Agents may report multiple values for a particular key, such as
`PlatformGroup: Desktop` and `PlatformGroup: Unix`. Conditions evaluate to true if there is any value for a particular
key that satisfies the expression.

## Values

Values in condition strings are dynamically typed and coerced into the correct format for the operation performed on
them. For example, the integer `0`, the strings `'0'` and `'false'`, and the boolean `false` are functionally
equivalent.

Integers may be given the following binary order-of-magnitude suffixes as a shorthand:

| Suffix | Size |
| ------ | ---- |
| Kb | 2^10 (1,024) |
| Mb | 2^20 (1,048,576) |
| Gb | 2^30 (1,073,741,824) |
| Tb | 2^40 (1,099,511,627,776) |

## Operators

Operators follow standard C rules of precedence. Sub-expressions may be enclosed in `(` parentheses `)`.

### Boolean Operators

| Operator | Description |
| -------- | ----------- |
| `true` | Boolean literal |
| `false` | Boolean literal |
| `!A` | Negation |
| A `&&` B | Logical AND |
| A `\|\|` B | Logical OR |
| A `==` B | Equality |
| A `!=` B | Inequality |

### Integer Operators

| Operator | Description |
| -------- | ----------- |
| `1234` | Integer literal |
| `1234kb` | Integer literal with binary size suffix (evaluates to 1,263,616). |
| A `==` B | Test for equality |
| A `!=` B | Test for inequality |
| A `<` B | Test if one value is less than another |
| A `<=` B | Test if one value is less than or equal to another |
| A `>` B | Test if one value is greater than another |
| A `>=` B | Test if one value is greater than or equal to another |
| A `==` B | Test for equality |

### String Operators

| Operator | Description |
| -------- | ----------- |
| `'value'` | String literal |
| `"value"` | String literal |
| A `==` B | Test if two strings compare equal for equality. Comparison is case-insensitive. |
| A `!=` B | Test if two strings are different. Comparison is case-insensitive. |
| A `~=` B | Test if a string (A) matches a regex (B). The regex comparison is case-insensitive. |
