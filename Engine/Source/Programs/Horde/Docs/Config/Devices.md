[Horde](../../README.md) > [Configuration](../Config.md) > Devices

# Device Manager

The Horde device manager administers mobile and console development kit
resources. The service is used extensively at Epic and is quite mature.
It features:

* Configurable device platforms and pools
* Shared remote device resources with user checkouts
* Automation device reservations with issue reporting and recovery
* Dashboard support for managing hardware
* Device usage history, telemetry, and pool health reporting
* Integration with the Gauntlet automation framework

## Shared Pools

Horde users use shared pools to check out remote device resources for
development and testing. Users check out devices through the dashboard,
with devices being returned to the pool either by an explicit check-in or
when a configurable duration has been exceeded.

There is a notification sink that notifies users 24 hours before the
check-in will expire so they can renew if desired. There is a subsequent
notification once the checkout has expired and the device is returned to
the pool.

It is also possible to set up automation jobs targeting the checked-out devices to install builds, for example.

## Automation Pools

The device manager supports automation job device reservations, which can be constrained by pool, platform, and model.

We recommend that you use the [Gauntlet integration](#gauntlet-integration) with the reservation system.
However, you can also drive custom solutions using the same REST API.

## Platform and Pool Configuration

Device platform hardware is partitioned into pools for use by automated
tests and users. Device platforms and pools are configured using the
`Devices` section of the globals.json file (see
[DeviceConfig](Schema/Globals.md#deviceconfig)).

### Example

The following config fragment declares a device manager configuration that:

* Adds an `Android` device platform specifying several models
* Adds two device pools, an `Automation` pool which automated tests can
use, and a `Shared` pool from which users can check out remote device
hardware

```json
    "devices": {
        "platforms": [
            {
                "id": "android",
                "name": "Android",
                "models": [
                    "Pixel4",
                    "Pixel5",
                    "Pixel8"
                ]
            }
        ],
        "pools": [
            {
                "id": "ue5",
                "name": "UE5",
                "poolType": "Automation"
            },
            {
                "id": "remote-ue5",
                "name": "Remote UE5",
                "poolType": "Shared"
            }
        ]
    }
```

## Device Configuration

Shared and automation devices are added and managed through the Horde
Dashboard by navigating to `SERVER => Resources => Devices`.

This includes support for:

* Adding and editing devices
* Putting devices into maintenance mode 
* Adding inline notes about a device
* Moving devices between pools
* Viewing pool health and usage telemetry
* Viewing job history and the last user to modify the device

## Gauntlet Integration

[Gauntlet](https://docs.unrealengine.com/en-US/gauntlet-automation-framework-overview-in-unreal-engine)
tests can reserve hardware from the Horde device manager. This integration
includes features such as reporting problems with devices and recovery
with new devices. It also supports reservation blocks, which can be used
to reuse a device with an installed build across a series of automation
tests.  

### Example

The following [**BuildGraph**](https://docs.unrealengine.com/en-US/buildgraph-for-unreal-engine/) fragment declares:

* `HordeDeviceManager` and `HordeDevicePool` properties that specify your Horde server and which pool to use.
* Adds a `BootTest Android` node, which will reserve an Android Pixel 8 for the test

  ```xml
  <Property Name="HordeDeviceManager" Value="https://horde.yourdomain.com" />
  <Property Name="HordeDevicePool" Value="UE5" />

  <Node Name="BootTest Android">
      <Command Name="RunUnreal" Arguments="-test=UE.BootTest -platform=Android " -deviceurl=&quot;$(HordeDeviceManager)&quot; -devicepool=&quot;$(HordeDevicePool)&quot; -PerfModel=Pixel8/>
  </Node>
  ```

---
