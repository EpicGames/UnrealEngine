[Horde](../../README.md) > [Configuration](../Config.md) > Experimental

# Experimental

Experimental is a plugin used to research, implement, and test functionality within Horde without directly making changes or impacting the
core functionality. Use of the functionality within this plugin does not guarantee backward compatibility, the APIs for these features
are subject to change, and we may remove entire Experimental features or specific functionality at our discretion. The plugin is
Disabled by default and can be enabled by specifying the following snippet in the appsettings.json file within the `Horde` section:

```
"Plugins": {
	"Experimental": {
		"Enabled": true
	}
}
```

The [Getting Started > Experimental](../Tutorials/Experimental.md) guide explains the functionality currently for use within Horde.

## Notifications

Horde already provides support for various Slack notifications. The notification work within this plugin is testing a new grouped
layout which is based on a set of defined rules. The notifications can be configured through the `Notifications` object within
the global.json file (see [NotificationConfig](Schema/Notification.md)).
