[Horde](../../README.md) > [Configuration](../Config.md) > Orientation

# Orientation

Horde is almost exclusively configured through JSON config files. Despite the initial learning curve for some of the
conventions and concepts involved, we strongly believe this is a feature over UI-driven configuration. By
storing configuration in text files, it's easier to diff and version changes, track down changes causing issues, and
provide a clear specification of the feature set.

There are two areas related to configuring Horde:

* The [server configuration](../Deployment/ServerSettings.md) configures the server to talk to other servers, defines
  static parameters, and so on. It is driven by the `Server.json` file deployed alongside the server.
* The [global configuration](Schema/Globals.md) controls all the user-facing elements of the system once deployed and
  is in a file named `Globals.json`. Most configuration after setting up deployment parameters is done here.

The `Server.json` file references a location to read `Globals.json` from via the
[`ConfigPath`](../Deployment/ServerSettings.md) property and may be a path on disk or to a file in a version control
system (see [below](#revision-control)).

Horde will detect changes to the referenced file and automatically update from it without downtime. Errors encountered
while trying to update configuration at runtime can be reported via Slack notifications, and the server will continue
running with a cached version of the previous configuration until fixed.

Horde configuration files may include other configuration files by path. For CI use cases, for example, it can be
convenient to configure each stream within the stream itself.

## Revision Control

Horde supports reading configuration files from Perforce.

Perforce servers and accounts to use for reading configuration data are listed in the `Perforce` section of the
`Server.json` file. Once configured, files can be included from source control using either of the following
forms:

Perforce Syntax (uses the Perforce server configured with the "default" id):

    //Foo/Bar/globals.json

Explicit URI Syntax (using the Perforce server configured with the "some-name" id):

    perforce://some-name//Foo/Bar/globals.json

Relative paths may be used to specify the location of config files in relation to the current file being parsed,
regardless of the current storage backend providing it.

## Schema Server

There are a lot of settings in Horde config files, and it can take time to get used to them. To make editing
easier, Horde implements a JSON schema server that can allow IDEs to perform context highlighting, autocomplete, and
validation functionality.

To set up Horde as a schema server in Microsoft Visual Studio, go to `Tools` > `Options...` and navigate to
`Text Editor` > `JSON` > `Schema`.

Add the path to your Horde server as `{{ SERVER-URL }}/api/v1/schema/catalog.json`, substituting
`{{ SERVER-URL }}` as appropriate.

The extension given to the included files indicates the root element which should be expected by the schema.
Files with a `.project.json` extension start at the project element, files with a `.stream.json` extension
start at the stream element, and so on.
