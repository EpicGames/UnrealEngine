[Horde](../../README.md) > [Configuration](../Config.md) > Permissions

# Permissions

## Authentication

Horde offers three modes of authentication and authorization of users:

* Anonymous
* OpenID Connect
* Built-in user accounts

You can configure the mode via the `AuthMethod` setting.

### Anonymous

Horde ships with authorization disabled by default for demonstration purposes and to get started.

> **NOTE:** For production deployments, proper authentication should be
configured using either OpenID Connect or built-in user accounts.

### OpenID Connect

Horde can use an external OpenID Connect (OIDC) provider for authorization.
See the [server deployment](../Deployment/Server.md) documentation for
information about configuring an OIDC provider.
OIDC is recommended for studios where a central authentication provider is
already in use, such as Google Workspaces, Okta, or Azure AD/Entra ID.

After an OIDC provider is configured, a user's claims may be viewed by
navigating to `http://{{ server_url }}/account` page in a browser.

### Built-in User Accounts

If you are a smaller studio or don't see the need to use the OpenID
Connect method, Horde's built-in user accounts are an option.
These accounts are managed by Horde itself and stored in the local
database. With the server in anonymous mode, you can set up user accounts
via the web UI (Server dropdown in the top right).
Configure these with at least one administrator user and set `AuthMethod` to `Horde`.

## Access Control Lists

**Access control lists (ACLs)** control access to entities in Horde. Each item in the list grants the ability
to perform certain actions to any users with specific OIDC claims. Each claim is a key/value pair returned by
the OIDC provider or synthesized by Horde at login.
See [ACL Actions](../Config/Schema/AclActions.md) page for a complete list of actions available.

Many objects that users can query or manipulate have an attached ACL within a hierarchy of other
ACL-controlled objects. A stream is part of a project, for example. Users can be granted entitlements to view
that specific Perforce stream (via the ACL on that stream's configuration), for all streams within the project (via the ACL on
the project's configuration), or for all streams on the server (via the ACL on the global configuration).

### Administrators

Admin users are permitted to perform any operations regardless of any configured ACLs. Users are granted admin status
if they contain a particular claim configured in the server's [Server.json](../Deployment/ServerSettings.md) file
via the `AdminClaimType` and `AdminClaimValue` properties.

### Synthesized Claims

Horde adds several claims to the configured claims returned through the OIDC provider:

| Name | Description |
| ---- | ----------- |
| `http://epicgames.com/ue/horde/user` | Real name of the user. This is extracted from claims returned by the OIDC provider according to the `OidcClaimNameMapping` [server setting](../Deployment/Server.md). |
| `http://epicgames.com/ue/horde/user-id-v3` | Identifier for the user. This is a 24-character unique ID assigned by Horde. |
| `http://epicgames.com/ue/horde/agent` | Identifies a particular agent (with the value being the agent ID). |
| `http://epicgames.com/ue/horde/perforce-user` | Gives the Perforce username corresponding to the user | 

### Example

The following config fragment declares an ACL which:

* Grants the `ViewJob` and `CreateJob` entitlement to a user by the name of `Tim Sweeney`. 
* Grants the `ViewJob` entitlement to any users with the role claim of `app-horde-users`.

---

    "acl":
    {
        "entries": [
            {
                "claim": {
                    "type": "http://epicgames.com/ue/horde/user",
                    "value": "Tim Sweeney"
                },
                "actions": [
                    "ViewJob",
                    "CreateJob"
                ]
            },
            {
                "claim": {
                    "type": "http://schemas.microsoft.com/ws/2008/06/identity/claims/role",
                    "value": "app-horde-viewers"
                },
                "actions": [
                    "ViewJob"
                ]
            }
        ],
        "inherit": true
    }

---
