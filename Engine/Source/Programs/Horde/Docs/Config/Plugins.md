[Horde](../../README.md) > [Configuration](../Config.md) > Plugins

# Plugins

Horde is implemented as a core host process and set of plugins. Most
plugins correspond to a feature area can be enabled or disabled
individually, though some plugins build on functionality provided by
others. The [Storage](../Config.md#storage) and
[Compute](../Config.md#compute) plugins are notable foundational components
of Horde that other systems rely on.

## Configuration

Like the server itself, plugins can have a static boot-time configuration
(through the [server.json](../Deployment/ServerSettings.md) file) and dynamic
runtime configuration (through the [globals.json](Orientation.md) file).

In each case, settings for each plugin is stored under the `plugins`
property. All plugins have an `enabled` property which can be set to false
to disable them.

* [Plugins (server.json)](../Deployment/ServerSettings.md#serverpluginsconfig)
* [Plugins (globals.json)](Schema/Globals.md#globalpluginsconfig)
