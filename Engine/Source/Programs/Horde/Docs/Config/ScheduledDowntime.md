[Horde](../../README.md) > [Configuration](../Config.md) > Scheduled Downtime

# Scheduled Downtime

Horde supports a set of scheduled downtimes where no new jobs are started. This window can be used pause new jobs from
starting when external services, such as Perforce, are offline.

## Configuring Scheduled Downtime

A scheduled downtime requires `startTime` and `finishTime` properties, which are a [point in time relative to UTC](https://learn.microsoft.com/en-us/dotnet/api/system.datetimeoffset?view=net-8.0), to define the duration of the downtime.

The format of the time properties can be either:

* [ISO 8601](https://learn.microsoft.com/en-us/dotnet/standard/base-types/standard-date-and-time-format-strings#Roundtrip), for example `2025-03-01T02:00:00-05:00` to represent the 1st March 2025 at 2AM and 5 hours behind UTC.

* An abbreviated form, for example `Sat Mar 1, 2025, 2AM -5`. The supported abbreviated forms are the strings supported by the [DateTimeOffset.Parse](https://learn.microsoft.com/en-us/dotnet/api/system.datetimeoffset.parse?view=net-8.0) method.

The `finishTime` property can be omitted and replaced with a `duration` property that is the number of hours and minutes of the downtime. The supported values for `duration` are the strings supported by the [TimeSpan.Parse](https://learn.microsoft.com/en-us/dotnet/api/System.TimeSpan.Parse?view=net-8.0) method.

The UTC offset can be omitted from the `startTime` and `finishTime` properties and replaced with the `timeZone` property that will calculate the offset. For example, for North Carolina the time zone can be either `Eastern Standard Time` or `America/New_York`. The supported time zone values are the values returned by the [TimeZoneInfo.FindSystemTimeZoneById](https://learn.microsoft.com/en-us/dotnet/api/system.timezoneinfo.findsystemtimezonebyid?view=net-8.0) method, which includes the canonical names from the operating system and the aliases provided by .NET.

The canonical names from the operating system can be found using [TimeZoneInfo.GetSystemTimeZones](https://learn.microsoft.com/en-us/dotnet/api/system.timezoneinfo.getsystemtimezones?view=net-8.0). The names of aliases can be found using [TimeZoneInfo.TryConvertWindowsIdToIanaId](https://learn.microsoft.com/en-us/dotnet/api/system.timezoneinfo.tryconvertwindowsidtoianaid?view=net-8.0) and
[TimeZoneInfo.TryConvertIanaIdToWindowsId](https://learn.microsoft.com/en-us/dotnet/api/system.timezoneinfo.tryconvertianaidtowindowsid?view=net-8.0).

### 1. Using ISO 8601 format and absolute date and times
The following example creates a weekly maintenance window using the ISO 8601 format on the 6th September at 3AM and 4 hours behind UTC, for 30 minutes and repeats every 7 days.

  ```json
    "downtime": [
        {
            "startTime": "2020-09-06T03:00:00-04:00",
            "finishTime": "2020-09-06T03:30:00-04:00",
            "frequency": "Weekly"
        }
    ]
  ```

### 2. Using abbreviated format and duration property
Changing the above example to use the abbreviated format and `duration` property instead of `finishTime`. The day of the week name is optional.

   ```json
    "downtime": [
        {
            "startTime": "Sun Sep 6, 2020, 3AM -4",
            "duration": "0:30",
            "frequency": "Weekly"
        }
    ]
   ```

### 3. Using a timezone
The following example creates a daily maintenance window using the abbreviated format and `timeZone` property. Using a time zone the explicit offset component can be omitted.

   ```json
    "downtime": [
        {
            "startTime": "Sun Sep 6, 2020, 3AM",
            "timeZone": "Eastern Standard Time",
            "duration": "0:30",
            "frequency": "Daily"
        }
    ]
   ```

### 4. One-off downtime
The following example creates a one-off maintenance window of 1 hour and 15 minutes. The `frequency` property defaults to `Once` and can be omitted. Because no `timeZone` property and no offset component are set the start time is in UTC.

   ```json
    "downtime": [
        {
            "startTime": "Dec 10, 2024, 1AM",
            "duration": "1:15"
        }
    ]
   ```
