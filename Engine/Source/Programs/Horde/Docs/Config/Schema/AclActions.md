[Horde](../../../README.md) > [Configuration](../../Config.md) > ACL Actions

# ACL Actions

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` | Ability to create new accounts |
| `UpdateAccount` | Update an account settings |
| `DeleteAccount` | Delete an account from the server |
| `ViewAccount` | Ability to view account information |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` | Ability to create new notices |
| `UpdateNotice` | Ability to modify notices on the server |
| `DeleteNotice` | Ability to delete notices |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` | Ability to create new accounts |
| `UpdateAccount` | Update an account settings |
| `DeleteAccount` | Delete an account from the server |
| `ViewAccount` | Ability to view account information |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` | Permission to read from an artifact |
| `WriteArtifact` | Permission to write to an artifact |
| `DeleteArtifact` | Permission to delete to an artifact |
| `UploadArtifact` | Ability to create an artifact. Typically just for debugging; agents have this access for a particular session. |
| `DownloadArtifact` | Ability to download an artifact |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` | Ability to start new bisect tasks |
| `UpdateBisectTask` | Ability to update a bisect task |
| `ViewBisectTask` | Ability to view a bisect task |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` | Ability to add, update, and delete build health filters. |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` | Ability to read devices |
| `DeviceWrite` | Ability to write devices |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` | Ability to start new jobs |
| `UpdateJob` | Rename a job, modify its priority, etc... |
| `DeleteJob` | Delete a job properties |
| `ExecuteJob` | Allows updating a job metadata (name, changelist number, step properties, new groups, job states, etc...). Typically granted to agents. Not user facing. |
| `RetryJobStep` | Ability to retry a failed job step |
| `ViewJob` | Ability to view a job |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` | Ability to subscribe to notifications |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` | Allows the creation of new projects |
| `DeleteProject` | Allows deletion of projects. |
| `UpdateProject` | Modify attributes of a project (name, categories, etc...) |
| `ViewProject` | View information about a project |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` | Allows deletion of projects. |
| `ViewReplicator` | Allows the creation of new projects |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` | Allows the creation of new streams within a project |
| `UpdateStream` | Allows updating a stream (agent types, templates, schedules) |
| `DeleteStream` | Allows deleting a stream |
| `ViewStream` | Ability to view a stream |
| `ViewChanges` | View changes submitted to a stream. NOTE: this returns responses from the server's Perforce account, which may be a priviledged user. |
| `ViewTemplate` | View template associated with a stream |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` | Ability to read test health |
| `TestHealthWrite` | Ability to write test health |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` | Ability to create an agent. This may be done explicitly, or granted to agents to allow them to self-register. |
| `CreateWorkstationAgent` | Ability to create a workstation agent. Granted to users with hardware to offer. |
| `UpdateAgent` | Update an agent's name, pools, etc... |
| `DeleteAgent` | Soft-delete an agent |
| `ViewAgent` | View an agent |
| `ListAgents` | List the available agents |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` | User can add tasks to the compute cluster |
| `GetComputeTasks` | User can get and list tasks from the compute cluster |
| `UbaCacheRead` | User can read from Unreal Build Accelerator cache |
| `UbaCacheWrite` | User can write to Unreal Build Accelerator cache |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` | View all the leases that an agent has worked on |
| `ViewLeaseTasks` | View the task data for a lease |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` | Ability to create a log. Implicitly granted to agents. |
| `UpdateLog` | Ability to update log metadata |
| `ViewLog` | Ability to view a log contents |
| `WriteLogData` | Ability to write log data |
| `CreateEvent` | Ability to create events |
| `ViewEvent` | Ability to view events |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` | Create a global pool of agents |
| `UpdatePool` | Modify an agent pool |
| `DeletePool` | Delete an agent pool |
| `ViewPool` | Ability to view a pool |
| `ListPools` | View all the available agent pools |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` | Granted to agents to call CreateSession, which returns a bearer token identifying themselves valid to call UpdateSesssion via gRPC. |
| `ViewSession` | Allows viewing information about an agent session |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` | Ability to upload new versions of the agent software |
| `DownloadSoftware` | Ability to download the agent software |
| `DeleteSoftware` | Ability to delete agent software |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## Secrets

| Name | Description |
| ---- | ----------- |
| `ViewSecret` | View a credential |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## Secrets

| Name | Description |
| ---- | ----------- |
| `ViewSecret` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Storage

| Name | Description |
| ---- | ----------- |
| `ReadBlobs` | Ability to read blobs from the storage service |
| `WriteBlobs` | Ability to write blobs to the storage service |
| `ReadAliases` | Ability to read aliases from the storage service |
| `WriteAliases` | Ability to write aliases to the storage service |
| `DeleteAliases` | Ability to write aliases to the storage service |
| `ReadRefs` | Ability to read refs from the storage service |
| `WriteRefs` | Ability to write refs to the storage service |
| `DeleteRefs` | Ability to delete refs |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## Secrets

| Name | Description |
| ---- | ----------- |
| `ViewSecret` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Storage

| Name | Description |
| ---- | ----------- |
| `ReadBlobs` |  |
| `WriteBlobs` |  |
| `ReadAliases` |  |
| `WriteAliases` |  |
| `DeleteAliases` |  |
| `ReadRefs` |  |
| `WriteRefs` |  |
| `DeleteRefs` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## Symbols

| Name | Description |
| ---- | ----------- |
| `ReadSymbols` | Ability to download symbols |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## Secrets

| Name | Description |
| ---- | ----------- |
| `ViewSecret` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Storage

| Name | Description |
| ---- | ----------- |
| `ReadBlobs` |  |
| `WriteBlobs` |  |
| `ReadAliases` |  |
| `WriteAliases` |  |
| `DeleteAliases` |  |
| `ReadRefs` |  |
| `WriteRefs` |  |
| `DeleteRefs` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## Symbols

| Name | Description |
| ---- | ----------- |
| `ReadSymbols` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Tools

| Name | Description |
| ---- | ----------- |
| `DownloadTool` | Ability to download a tool |
| `UploadTool` | Ability to upload new tool versions |

## Accounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Agents

| Name | Description |
| ---- | ----------- |
| `CreateAgent` |  |
| `CreateWorkstationAgent` |  |
| `UpdateAgent` |  |
| `DeleteAgent` |  |
| `ViewAgent` |  |
| `ListAgents` |  |

## Artifacts

| Name | Description |
| ---- | ----------- |
| `ReadArtifact` |  |
| `WriteArtifact` |  |
| `DeleteArtifact` |  |
| `UploadArtifact` |  |
| `DownloadArtifact` |  |

## Bisect

| Name | Description |
| ---- | ----------- |
| `CreateBisectTask` |  |
| `UpdateBisectTask` |  |
| `ViewBisectTask` |  |

## BuildHealth

| Name | Description |
| ---- | ----------- |
| `AddUpdateDeleteBuildHealthFilter` |  |

## Compute

| Name | Description |
| ---- | ----------- |
| `AddComputeTasks` |  |
| `GetComputeTasks` |  |
| `UbaCacheRead` |  |
| `UbaCacheWrite` |  |

## Devices

| Name | Description |
| ---- | ----------- |
| `DeviceRead` |  |
| `DeviceWrite` |  |

## Jobs

| Name | Description |
| ---- | ----------- |
| `CreateJob` |  |
| `UpdateJob` |  |
| `DeleteJob` |  |
| `ExecuteJob` |  |
| `RetryJobStep` |  |
| `ViewJob` |  |

## Leases

| Name | Description |
| ---- | ----------- |
| `ViewLeases` |  |
| `ViewLeaseTasks` |  |

## Logs

| Name | Description |
| ---- | ----------- |
| `CreateLog` |  |
| `UpdateLog` |  |
| `ViewLog` |  |
| `WriteLogData` |  |
| `CreateEvent` |  |
| `ViewEvent` |  |

## Notices

| Name | Description |
| ---- | ----------- |
| `CreateNotice` |  |
| `UpdateNotice` |  |
| `DeleteNotice` |  |

## Notifications

| Name | Description |
| ---- | ----------- |
| `CreateSubscription` |  |

## Pools

| Name | Description |
| ---- | ----------- |
| `CreatePool` |  |
| `UpdatePool` |  |
| `DeletePool` |  |
| `ViewPool` |  |
| `ListPools` |  |

## Projects

| Name | Description |
| ---- | ----------- |
| `CreateProject` |  |
| `DeleteProject` |  |
| `UpdateProject` |  |
| `ViewProject` |  |

## Replicators

| Name | Description |
| ---- | ----------- |
| `UpdateReplicator` |  |
| `ViewReplicator` |  |

## Secrets

| Name | Description |
| ---- | ----------- |
| `ViewSecret` |  |

## ServiceAccounts

| Name | Description |
| ---- | ----------- |
| `CreateAccount` |  |
| `UpdateAccount` |  |
| `DeleteAccount` |  |
| `ViewAccount` |  |

## Sessions

| Name | Description |
| ---- | ----------- |
| `CreateSession` |  |
| `ViewSession` |  |

## Software

| Name | Description |
| ---- | ----------- |
| `UploadSoftware` |  |
| `DownloadSoftware` |  |
| `DeleteSoftware` |  |

## Storage

| Name | Description |
| ---- | ----------- |
| `ReadBlobs` |  |
| `WriteBlobs` |  |
| `ReadAliases` |  |
| `WriteAliases` |  |
| `DeleteAliases` |  |
| `ReadRefs` |  |
| `WriteRefs` |  |
| `DeleteRefs` |  |

## Streams

| Name | Description |
| ---- | ----------- |
| `CreateStream` |  |
| `UpdateStream` |  |
| `DeleteStream` |  |
| `ViewStream` |  |
| `ViewChanges` |  |
| `ViewTemplate` |  |

## Symbols

| Name | Description |
| ---- | ----------- |
| `ReadSymbols` |  |

## TestData

| Name | Description |
| ---- | ----------- |
| `TestHealthRead` |  |
| `TestHealthWrite` |  |

## Tools

| Name | Description |
| ---- | ----------- |
| `DownloadTool` |  |
| `UploadTool` |  |
