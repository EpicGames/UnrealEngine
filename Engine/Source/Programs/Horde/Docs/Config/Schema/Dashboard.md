[Horde](../../../README.md) > [Configuration](../../Config.md) > *.dashboard.json

# *.dashboard.json

Configuration for dashboard features

Name | Description
---- | -----------
`showLandingPage` | `boolean`<br>Navigate to the landing page by default
`landingPageRoute` | `string`<br>Custom landing page route to direct users to
`showCI` | `boolean`<br>Enable CI functionality
`showAgents` | `boolean`<br>Whether to show functionality related to agents, pools, and utilization on the dashboard.
`showAgentRegistration` | `boolean`<br>Whether to show the agent registration page. When using registration tokens from elsewhere this is not needed.
`showPerforceServers` | `boolean`<br>Show the Perforce server option on the server menu
`showDeviceManager` | `boolean`<br>Show the device manager on the server menu
`showTests` | `boolean`<br>Show automated tests (legacy) on the server menu
`agentCategories` | [DashboardAgentCategoryConfig](#dashboardagentcategoryconfig)`[]`<br>Configuration for different agent pages
`poolCategories` | [DashboardPoolCategoryConfig](#dashboardpoolcategoryconfig)`[]`<br>Configuration for different pool pages
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within this configuration

## DashboardAgentCategoryConfig

Configuration for a category of agents

Name | Description
---- | -----------
`name` | `string`<br>Name of the category
`condition` | `string`<br>Condition string to be evaluated for this page

## DashboardPoolCategoryConfig

Configuration for a category of pools

Name | Description
---- | -----------
`name` | `string`<br>Name of the category
`condition` | `string`<br>Condition string to be evaluated for this page

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## ConfigMacro

Declares a config macro

Name | Description
---- | -----------
`name` | `string`<br>Name of the macro property
`value` | `string`<br>Value for the macro property
