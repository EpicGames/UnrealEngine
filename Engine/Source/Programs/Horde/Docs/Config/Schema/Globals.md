[Horde](../../../README.md) > [Configuration](../../Config.md) > Globals.json

# Globals.json

Global configuration

Name | Description
---- | -----------
`version` | `integer`<br>Version number for the server. Values are indicated by the .
`include` | [ConfigInclude](#configinclude)`[]`<br>Other paths to include
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within the global scope
`dashboard` | [DashboardConfig](Dashboard.md)<br>Settings for the dashboard
`downtime` | [ScheduledDowntime](#scheduleddowntime)`[]`<br>List of scheduled downtime
`plugins` | [GlobalPluginsConfig](#globalpluginsconfig)<br>Plugin config objects
`parameters` | `object`<br>General parameters for other tools. Can be queried through the api/v1/parameters endpoint.
`resolveSecrets` | `boolean`<br>Whether to enable resolving secrets in config properties
`acl` | [AclConfig](#aclconfig)<br>Access control list

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## ConfigMacro

Declares a config macro

Name | Description
---- | -----------
`name` | `string`<br>Name of the macro property
`value` | `string`<br>Value for the macro property

## ScheduledDowntime

Settings for the maintenance window

Name | Description
---- | -----------
`startTime` | `string`<br>Start time
`finishTime` | `string`<br>Finish time
`duration` | `string`<br>Duration of the maintenance window. An alternative to FinishTime.
`timeZone` | `string`<br>The name of time zone to set the Coordinated Universal Time (UTC) offset for StartTime and FinishTime.
`frequency` | [ScheduledDowntimeFrequency](#scheduleddowntimefrequency-enum)<br>Frequency that the window repeats

## ScheduledDowntimeFrequency (Enum)

How frequently the maintenance window repeats

Name | Description
---- | -----------
`Once` | Once
`Daily` | Every day
`Weekly` | Every week

## GlobalPluginsConfig

Name | Description
---- | -----------
`analytics` | [AnalyticsConfig](#analyticsconfig)<br>Configuration for the analytics plugin
`build` | [BuildConfig](#buildconfig)<br>Configuration for the build plugin
`compute` | [ComputeConfig](#computeconfig)<br>Configuration for the compute plugin
`exampleplugin` | [ExamplePluginConfig](#examplepluginconfig)<br>Configuration for the exampleplugin plugin
`experimental` | [ExperimentalConfig](#experimentalconfig)<br>Configuration for the experimental plugin
`secrets` | [SecretsConfig](#secretsconfig)<br>Configuration for the secrets plugin
`storage` | [StorageConfig](#storageconfig)<br>Configuration for the storage plugin
`symbols` | [SymbolsConfig](#symbolsconfig)<br>Configuration for the symbols plugin
`tools` | [ToolsConfig](#toolsconfig)<br>Configuration for the tools plugin

## AnalyticsConfig

Config settings for analytics

Name | Description
---- | -----------
`stores` | [TelemetryStoreConfig](Telemetry.md)`[]`<br>Metrics to aggregate on the Horde server

## BuildConfig

Configuration for the build plugin

Name | Description
---- | -----------
`perforceClusters` | [PerforceCluster](#perforcecluster)`[]`<br>List of Perforce clusters
`devices` | [DeviceConfig](#deviceconfig)<br>Device configuration
`maxConformCount` | `integer`<br>Maximum number of conforms to run at once
`agentShutdownIfDisabledGracePeriod` | `string`<br>Time to wait before shutting down an agent that has been disabled Used if no value is set on the actual pool.
`robomerge` | [RoboMergeConfig](#robomergeconfig)<br>RoboMerge branchmap file configuration for the merge graph service. When null or empty, the service is inert (no Perforce connections, no API data).
`artifactTypes` | [ArtifactTypeConfig](#artifacttypeconfig)`[]`<br>Configuration for different artifact types
`projects` | [ProjectConfig](Projects.md)`[]`<br>List of projects
`enableConformTasks` | `boolean`<br>Whether to allow conform tasks to run
`enableJobTaskConformCheck` | `boolean`<br>Whether to check if a conform is required on an agent before assigning a lease for a job task source.
`issueFixedTag` | `string`<br>Commit tag to use for marking issues as fixed
`issues` | [IssueConfig](Issues.md)<br>Build health issue configurations
`enableIssueRootCauseAnalysisNotifications` | `boolean`<br>Whether to allow notifications for Root Cause Analysis to be sent
`enableAutoRcaSlackNotifications` | `boolean`<br>Global kill switch for posting auto-RCA results to Slack triage threads. When false (default), no auto-RCA Slack notifications are sent regardless of per-workflow AutoRcaSlackEnabled settings.
`autoRcaUseMultiStage` | `boolean`<br>Whether to use multi-stage LLM analysis for auto-RCA. Multi-stage is slower but potentially more accurate. Set to false for faster processing with higher throughput.
`maxConcurrentRcaRequests` | `integer`<br>Maximum number of concurrent auto-RCA requests. Limits load on the external LLM API. Set to 0 to disable auto-RCA globally (acts as a kill switch).
`issueFindSuspectsMaxSuccessDays` | `integer`<br>The maximum days since node success to scan for suspects
`issueFindSuspectsMaxScoringSeconds` | `integer`<br>The maximum number of seconds to spend scording suspects
`issueFindSuspectsMaxChanges` | `integer`<br>Maximum number of commits to examine for suspects. Previously hardcoded as const MaxChanges = 500.
`issueFindSuspectsEnableParallelScoring` | `boolean`<br>Enable parallel scoring of commits for suspect finding. When enabled, commits are scored concurrently up to MaxParallelism.
`issueFindSuspectsMaxParallelism` | `integer`<br>Maximum concurrent commit file lookups when parallel scoring is enabled. Higher values improve speed but increase Perforce server load.
`issueFindSuspectsEnableEarlyExit` | `boolean`<br>Enable early exit when theoretical maximum score is achieved. Only exits early if the best possible score has been found.
`issueFindSuspectsEnableBidirectionalQuery` | `boolean`<br>Enable bidirectional commit query to capture both recent and old commits. Queries half the limit from each end of the commit range.
`issueFindSuspectsEnableFingerprintFallback` | `boolean`<br>Enable fingerprint-based fallback when no suspects are found. Helps prevent duplicate issues when commit range exceeds MaxChanges.
`issueFingerprintFallbackMinScore` | `integer`<br>Minimum score required for fingerprint fallback matching. Higher values are more conservative (fewer false matches).
`issueFingerprintFallbackMaxAgeDays` | `integer`<br>Maximum age in days for issues to be considered in fingerprint fallback.
`issueFingerprintFallbackExcludedTypes` | `string[]`<br>Fingerprint types excluded from fallback matching (too generic).
`enableExpandedIgnorePatterns` | `boolean`<br>Whether to enable expanded IgnorePatterns discovery (workspace-wide scanning + config-specified files). When false (default), only Engine directory is scanned for ignore patterns.
`lowContextFingerprintTypes` | `string[]`<br>Fingerprint type prefixes considered "low context" for suspect mention suppression. When an issue's fingerprint type starts with any of these prefixes AND all suspects are tied at rank 0, @ mentions are suppressed in Slack. Suspects are still listed without mentions. Set to an empty array to disable suppression entirely.

## PerforceCluster

Information about a cluster of Perforce servers.

Name | Description
---- | -----------
`name` | `string`<br>Name of the cluster
`serviceAccount` | `string`<br>Username for Horde to log in to this server. Will use the first account specified below if not overridden.
`canImpersonate` | `boolean`<br>Whether the service account can impersonate other users
`supportsPartitionedWorkspaces` | `boolean`<br>Whether to use partitioned workspaces on this server
`servers` | [PerforceServer](#perforceserver)`[]`<br>List of servers
`credentials` | [PerforceCredentials](#perforcecredentials)`[]`<br>List of server credentials
`autoSdk` | [AutoSdkWorkspace](#autosdkworkspace)`[]`<br>List of autosdk streams

## PerforceServer

Information about an individual Perforce server

Name | Description
---- | -----------
`serverAndPort` | `string`<br>The server and port. The server may be a DNS entry with multiple records, in which case it will be actively load balanced. If "ssl:" prefix is used, ensure P4 server's fingerprint/certificate is trusted. See Horde's documentation on connecting to SSL-enabled Perforce servers.
`healthCheck` | `boolean`<br>Whether to query the healthcheck address under each server
`resolveDns` | `boolean`<br>Whether to resolve the DNS entries and load balance between different hosts
`maxConformCount` | `integer`<br>Maximum number of simultaneous conforms on this server
`condition` | `string`<br>Optional condition for a machine to be eligible to use this server
`properties` | `string[]`<br>List of properties for an agent to be eligible to use this server

## PerforceCredentials

Credentials for a Perforce user

Name | Description
---- | -----------
`userName` | `string`<br>The username
`password` | `string`<br>Password for the user
`ticket` | `string`<br>Login ticket for the user (will be used instead of password if set)

## AutoSdkWorkspace

Path to a platform and stream to use for syncing AutoSDK

Name | Description
---- | -----------
`name` | `string`<br>Name of this workspace
`properties` | `string[]`<br>The agent properties to check (eg. "OSFamily=Windows")
`userName` | `string`<br>Username for logging in to the server
`stream` | `string`<br>Stream to use

## DeviceConfig

Configuration for devices

Name | Description
---- | -----------
`platforms` | [DevicePlatformConfig](#deviceplatformconfig)`[]`<br>List of device platforms
`pools` | [DevicePoolConfig](#devicepoolconfig)`[]`<br>List of device pools

## DevicePlatformConfig

Configuration for a device platform

Name | Description
---- | -----------
`id` | `string`<br>The id for this platform
`name` | `string`<br>Name of the platform
`models` | `string[]`<br>A list of platform models
`legacyNames` | `string[]`<br>Legacy names which older versions of Gauntlet may be using
`legacyPerfSpecHighModel` | `string`<br>Model name for the high perf spec, which may be requested by Gauntlet

## DevicePoolConfig

Configuration for a device pool

Name | Description
---- | -----------
`id` | `string`<br>The id for this platform
`name` | `string`<br>The name of the pool
`poolType` | [DevicePoolType](#devicepooltype-enum)<br>The type of the pool
`projectIds` | `string[]`<br>List of project ids associated with pool

## DevicePoolType (Enum)

The type of device pool

Name | Description
---- | -----------
`Automation` | Available to CIS jobs
`Shared` | Shared by users with remote checking and checkouts

## RoboMergeConfig

Configuration for the RoboMerge branchmap service

Name | Description
---- | -----------
`branchmapFiles` | `string[]`<br>Perforce depot paths to branchmap JSON files. Plain depot paths (e.g. "//MyProject/Main/Build/Streams/myproject.branchmap.json"). All files are read from the Perforce cluster specified by .
`perforceCluster` | `string`<br>Perforce cluster name to connect to for reading branchmap files. Maps to a connection configured in BuildServerConfig.Perforce[]. Default: "default" (uses the default Perforce connection).
`tickInterval` | `string`<br>How often to poll Perforce for branchmap file changes. Default: 5 minutes. Changing this requires a server restart (ticker interval is set at construction time).

## ArtifactTypeConfig

Configuration for an artifact

Name | Description
---- | -----------
`name` | `string`<br>Legacy 'Name' property
`type` | `string`<br>Name of the artifact type
`acl` | [AclConfig](#aclconfig)<br>Acl for the artifact type
`keepCount` | `integer`<br>Number of artifacts to retain
`keepDays` | `integer`<br>Number of days to retain artifacts of this type
`namespaceId` | `string`<br>Storage namespace to use for this artifact types

## AclConfig

Parameters to update an ACL

Name | Description
---- | -----------
`entries` | [AclEntryConfig](#aclentryconfig)`[]`<br>Entries to replace the existing ACL
`profiles` | [AclProfileConfig](#aclprofileconfig)`[]`<br>Defines profiles which allow grouping sets of actions into named collections
`inherit` | `boolean`<br>Whether to inherit permissions from the parent ACL
`exceptions` | `string[]`<br>List of exceptions to the inherited setting

## AclEntryConfig

Individual entry in an ACL

Name | Description
---- | -----------
`claim` | [AclClaimConfig](#aclclaimconfig)<br>Name of the user or group
`actions` | `string[]`<br>Array of actions to allow
`profiles` | `string[]`<br>List of profiles to grant

## AclClaimConfig

New claim to create

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## AclProfileConfig

Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this profile
`actions` | `string[]`<br>Actions to include
`excludeActions` | `string[]`<br>Actions to exclude from the inherited actions
`extends` | `string[]`<br>Other profiles to extend from

## ComputeConfig

Configuration for the compute system

Name | Description
---- | -----------
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within the compute config
`acl` | [AclConfig](#aclconfig)<br>Inherited root acl
`versionEnum` | [ConfigVersion](#configversion-enum)<br>Config version number
`telemetryStoreId` | `string`<br>Telemetry store for lease and compute telemetry data
`rates` | [AgentRateConfig](#agentrateconfig)`[]`<br>List of costs of a particular agent type
`clusters` | [ComputeClusterConfig](#computeclusterconfig)`[]`<br>List of compute profiles
`pools` | [PoolConfig](#poolconfig)`[]`<br>List of pools
`software` | [AgentSoftwareConfig](#agentsoftwareconfig)`[]`<br>List of costs of a particular agent type
`networks` | [NetworkConfig](#networkconfig)`[]`<br>List of networks

## ConfigVersion (Enum)

Global version number for running the server. As new features are introduced that require data migrations, this version number indicates the backwards compatibility functionality that must be enabled. When adding a new version here, also add a message to ConfigService.CreateSnapshotAsync describing the steps that need to be taken to upgrade the deployment.

Name | Description
---- | -----------
`None` | Not specified
`Initial` | Initial version number
`PoolsInConfigFiles` | All pools are managed through config files. Database pool reads are disabled.
`Latest` | Latest version number
`LatestPlusOne` | One after the last defined version number

## AgentRateConfig

Describes the monetary cost of agents matching a particular criteria

Name | Description
---- | -----------
`condition` | `string`<br>Condition string
`rate` | `number`<br>Rate for this agent

## ComputeClusterConfig

Profile for executing compute requests

Name | Description
---- | -----------
`id` | `string`<br>Name of the partition
`namespaceId` | `string`<br>Name of the namespace to use
`requestBucketId` | `string`<br>Name of the input bucket
`responseBucketId` | `string`<br>Name of the output bucket
`condition` | `string`<br>Filter for agents to include
`acl` | [AclConfig](#aclconfig)<br>Access control list
`uba` | [UbaComputeClusterConfig](#ubacomputeclusterconfig)<br>Configuration settings for Unreal Build Accelerator (UBA) for a compute cluster

## UbaComputeClusterConfig

Configuration settings for Unreal Build Accelerator (UBA) for a compute cluster Contains cache server connectivity and other UBA-related settings. Ensure all cache related fields are non-null to activate use of cache.

Name | Description
---- | -----------
`cacheEndpoint` | `string`<br>Host and port for connecting to the UBA cache server (host:port format) As specified with `-port=[PORT]` to UBA cache process. Serves all cache requests and must be accessible to both agents and initiators running UBA / Unreal Build Tool.
`cacheHttpEndpoint` | `string`<br>Host and port for HTTP management of UBA cache server (host:port format) As specified with `-http=[PORT]` to UBA cache process. This port is used by Horde to manage the UBA cache, such as adding new cache session keys for initiators For improved security, only allow Horde server to access this port
`cacheHttpCrypto` | `string`<br>AES encryption key used for secure Horde-to-cache-server communication As specified with `-httpcrypto=[KEY]` to UBA cache process.

## PoolConfig

Mutable configuration for a pool

Name | Description
---- | -----------
`id` | `string`<br>Unique id for this pool
`base` | `string`<br>Base pool config to copy settings from
`name` | `string`<br>Name of the pool
`condition` | `string`<br>Condition for agents to automatically be included in this pool
`properties` | `string` `->` `string`<br>Arbitrary properties related to this pool
`color` | [PoolColor](#poolcolor-enum)<br>Color to use for this pool on the dashboard
`enableAutoscaling` | `boolean`<br>Whether to enable autoscaling for this pool
`minAgents` | `integer`<br>The minimum number of agents to keep in the pool
`numReserveAgents` | `integer`<br>The minimum number of idle agents to hold in reserve
`conformInterval` | `string`<br>Interval between conforms. If zero, the pool will not conform on a schedule.
`scaleOutCooldown` | `string`<br>Cooldown time between scale-out events
`scaleInCooldown` | `string`<br>Cooldown time between scale-in events
`shutdownIfDisabledGracePeriod` | `string`<br>Time to wait before shutting down an agent that has been disabled
`sizeStrategy` | [PoolSizeStrategy](#poolsizestrategy-enum)<br>
`sizeStrategies` | [PoolSizeStrategyInfo](#poolsizestrategyinfo)`[]`<br>List of pool sizing strategies for this pool. The first strategy with a matching condition will be picked.
`fleetManagers` | [FleetManagerInfo](#fleetmanagerinfo)`[]`<br>List of fleet managers for this pool. The first strategy with a matching condition will be picked. If empty or no conditions match, a default fleet manager will be used.
`leaseUtilizationSettings` | [LeaseUtilizationSettings](#leaseutilizationsettings)<br>Settings for lease utilization pool sizing strategy (if used)
`jobQueueSettings` | [JobQueueSettings](#jobqueuesettings)<br>Settings for job queue pool sizing strategy (if used)

## PoolColor (Enum)

Color to use for labels of this pool

Name | Description
---- | -----------
`Default` | 
`Blue` | 
`Orange` | 
`Green` | 
`Gray` | 

## PoolSizeStrategy (Enum)

Available pool sizing strategies

Name | Description
---- | -----------
`LeaseUtilization` | Strategy based on lease utilization
`JobQueue` | Strategy based on size of job build queue
`NoOp` | No-op strategy used as fallback/default behavior
`ComputeQueueAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet
`LeaseUtilizationAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet

## PoolSizeStrategyInfo

Metadata for configuring and picking a pool sizing strategy

Name | Description
---- | -----------
`type` | [PoolSizeStrategy](#poolsizestrategy-enum)<br>Strategy implementation to use
`condition` | `string`<br>Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
`config` | `object`<br>Configuration for the strategy, serialized as JSON
`extraAgentCount` | `integer`<br>Integer to add after pool size has been calculated. Can also be negative.

## FleetManagerInfo

Metadata for configuring and picking a fleet manager

Name | Description
---- | -----------
`type` | [FleetManagerType](#fleetmanagertype-enum)<br>Fleet manager type implementation to use
`condition` | `string`<br>Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
`config` | `object`<br>Configuration for the strategy, serialized as JSON

## FleetManagerType (Enum)

Available fleet managers

Name | Description
---- | -----------
`Default` | Default fleet manager
`NoOp` | No-op fleet manager.
`Aws` | Fleet manager for handling AWS EC2 instances. Will create and/or terminate instances from scratch.
`AwsReuse` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsRecycle` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsAsg` | Fleet manager for handling AWS EC2 instances. Uses an EC2 auto-scaling group for controlling the number of running instances.

## LeaseUtilizationSettings

Lease utilization sizing settings for a pool

Name | Description
---- | -----------
`sampleTimeSec` | `integer`<br>Time period for each sample
`numSamples` | `integer`<br>Number of samples to collect for calculating lease utilization
`numSamplesForResult` | `integer`<br>Min number of samples for a valid result
`minAgents` | `integer`<br>The minimum number of agents to keep in the pool
`numReserveAgents` | `integer`<br>The minimum number of idle agents to hold in reserve

## JobQueueSettings

Job queue sizing settings for a pool

Name | Description
---- | -----------
`scaleOutFactor` | `number`<br>Factor translating queue size to additional agents to grow the pool with The result is always rounded up to nearest integer. Example: if there are 20 jobs in queue, a factor 0.25 will result in 5 new agents being added (20 * 0.25)
`scaleInFactor` | `number`<br>Factor by which to shrink the pool size with when queue is empty The result is always rounded up to nearest integer. Example: when the queue size is zero, a default value of 0.9 will shrink the pool by 10% (current agent count * 0.9)
`samplePeriodMin` | `integer`<br>How far back in time to look for job batches (that potentially are in the queue)
`readyTimeThresholdSec` | `integer`<br>Time spent in ready state before considered truly waiting for an agent<br>A job batch can be in ready state before getting picked up and executed. This threshold will help ensure only batches that have been waiting longer than this value will be considered.

## ComputeQueueAwsMetricSettings

Settings for

Name | Description
---- | -----------
`computeClusterId` | `string`<br>Compute cluster ID to observe
`namespace` | `string`<br>AWS CloudWatch namespace to write metrics in

## AgentSoftwareConfig

Selects different agent software versions by evaluating a condition

Name | Description
---- | -----------
`toolId` | `string`<br>Tool identifier
`condition` | `string`<br>Condition for using this channel

## NetworkConfig

Describes a network The ID describes any logical grouping, such as region, availability zone, rack or office location.

Name | Description
---- | -----------
`id` | `string`<br>ID for this network
`cidrBlock` | `string`<br>CIDR block
`description` | `string`<br>Human-readable description
`computeId` | `string`<br>Compute ID for this network (used when allocating compute resources)

## ExamplePluginConfig

Configuration for the ExamplePlugin plugin

Name | Description
---- | -----------
`greetingMessage` | `string`<br>The Greeting provided by the example plugin

## ExperimentalConfig

Configuration for the experimental plugin

Name | Description
---- | -----------
`notifications` | [NotificationConfig](Notification.md)`[]`<br>Notifications to be sent on the Horde server

## SecretsConfig

Configuration for the secrets system

Name | Description
---- | -----------
`secrets` | [SecretConfig](#secretconfig)`[]`<br>List of secrets
`providerConfigs` | [SecretProviderConfig](#secretproviderconfig)`[]`<br>List of configurations for secret providers

## SecretConfig

Configuration for a secret value

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this secret
`data` | `string` `->` `string`<br>Key/value pairs associated with this secret
`sources` | [ExternalSecretConfig](#externalsecretconfig)`[]`<br>Providers to source key/value pairs from
`acl` | [AclConfig](#aclconfig)<br>Defines access to this particular secret

## ExternalSecretConfig

Configuration for an external secret provider

Name | Description
---- | -----------
`providerConfig` | `string`<br>The name to look up a secret provider config. A config is mutually exclusive to and to be used when a secret requires additional options than just the name of the provider
`provider` | `string`<br>Name of the provider to use
`format` | [ExternalSecretFormat](#externalsecretformat-enum)<br>Format of the secret
`key` | `string`<br>Optional key indicating the parameter to set in the resulting data array. Required if  is .
`path` | `string`<br>Optional value indicating what to fetch from the provider

## ExternalSecretFormat (Enum)

Format describing how to parse external secret values

Name | Description
---- | -----------
`Text` | Secret is a plain text value which will be stored using the external secret key
`Json` | Secret is a JSON formatted string containing key/value pairs

## SecretProviderConfig

Common settings object for different secret providers

Name | Description
---- | -----------
`name` | `string`<br>Name of the secret provider config
`provider` | `string`<br>Name of the provider
`hcpVault` | [HcpVaultConfig](#hcpvaultconfig)<br>Configuration for HCP Vault
`azureKeyVault` | [AzureKeyVaultConfig](#azurekeyvaultconfig)<br>Configuration for Azure Key Vault

## HcpVaultConfig

Configuration for HCP Vault secrets.

Name | Description
---- | -----------
`preSharedKey` | `string`<br>The Vault authentication token to use when the type of credentials is .<br>More information on this token can be found in the API documentation for X-Vault-Token
`endPoint` | `string`<br>The address of the Vault server.
`awsArnRole` | `string`<br>The optional AWS ARN to assume a role when logging into Vault. To be used when the type of credentials is . When not set the AWS SDK default credential search will be used.
`awsIamServerId` | `string`<br>The value for the X-Vault-AWS-IAM-Server-ID header when the Vault server is configured with iam_server_id_header_value. Most likely to be the Vault server's DNS name. To be used when the type of credentials is .
`role` | `string`<br>The name of the role in the Vault configuration for the login. This role name can be different to the name of the AWS IAM principal. To be used when the type of credentials is . When not set the name of IAM principal will be used.
`credentials` | [HcpVaultCredentialsType](#hcpvaultcredentialstype-enum)<br>Type of credentials to use.

## HcpVaultCredentialsType (Enum)

Credentials to use to log into HCP Vault.

Name | Description
---- | -----------
`PreSharedKey` | Use a pre-shared token provided by Vault for authentication.
`AwsAuth` | Use AWS IAM credentials to authenticate against the Vault server.

## AzureKeyVaultConfig

Configuration for Azure Key Vault secrets.

Name | Description
---- | -----------
`vaultUri` | `string`<br>The Uri to the Azure Key Vault.

## StorageConfig

Configuration for storage

Name | Description
---- | -----------
`enableGc` | `boolean`<br>Whether to enable garbage collection
`enableGcVerification` | `boolean`<br>Whether to enable garbage collection in verification mode (nothing deleted, just logging on access to deleted blobs)
`gcWorkerCount` | `integer`<br>Number of concurrent GC reachability-check workers. Default 8. Reachability checks are I/O-bound (MongoDB queries), so higher values scale throughput roughly linearly up to the database's capacity.
`gcQueueLimit` | `integer`<br>Maximum number of entries in a namespace's Redis GC queue before the blob scanner pauses. Default 50,000.
`enableReliableGcEnqueue` | `boolean`<br>Whether to await Redis writes when enqueuing blobs for GC checking. When false (default), uses fire-and-forget which can silently drop entries on Redis failures. Set true for reliable enqueue.
`enablePerNamespaceQueueIsolation` | `boolean`<br>Whether to isolate GC queue limits per-namespace. When false (default), one backed-up namespace pauses the scanner for ALL namespaces. When true, only the backed-up namespace is paused and its blobs are deferred.
`gcBlobIngestionBatchSize` | `integer`<br>Number of blobs to process per blob-ingestion tick (TickBlobsAsync). Default 500. Raise to increase scan throughput when catching up on a backlog of unscanned blobs.
`gcBatchSize` | `integer`<br>Number of blob IDs to batch together for the import-reachability check. Default 1 (current per-blob behavior). When greater than 1, uses $in query to check multiple blobs in a single MongoDB round-trip. Values of 50-200 recommended.
`backends` | [BackendConfig](#backendconfig)`[]`<br>List of storage backends
`namespaces` | [NamespaceConfig](#namespaceconfig)`[]`<br>List of namespaces for storage

## BackendConfig

Common settings object for different providers

Name | Description
---- | -----------
`id` | `string`<br>The storage backend ID
`base` | `string`<br>Base backend to copy default settings from
`secondary` | `string`<br>Specifies another backend to read from if an object is not found in this one. Can be used when migrating data from one backend to another.
`type` | [StorageBackendType](#storagebackendtype-enum)<br>
`baseDir` | `string`<br>
`awsBucketName` | `string`<br>Name of the bucket to use
`awsBucketPath` | `string`<br>Base path within the bucket
`awsCredentials` | [AwsCredentialsType](#awscredentialstype-enum)<br>Type of credentials to use
`awsRole` | `string`<br>ARN of a role to assume
`awsProfile` | `string`<br>The AWS profile to read credentials form
`awsRegion` | `string`<br>Region to connect to
`azureConnectionString` | `string`<br>Connection string for Azure
`azureContainerName` | `string`<br>Name of the container
`relayServer` | `string`<br>
`relayToken` | `string`<br>
`gcsBucketName` | `string`<br>Name of the GCS bucket to use
`gcsBucketPath` | `string`<br>Base path within the bucket

## StorageBackendType (Enum)

Types of storage backend to use

Name | Description
---- | -----------
`FileSystem` | Local filesystem
`Aws` | AWS S3
`Azure` | Azure blob store
`Gcs` | Google Cloud Storage
`Memory` | In-memory only (for testing)

## AwsCredentialsType (Enum)

Credentials to use for AWS

Name | Description
---- | -----------
`Default` | Use default credentials from the AWS SDK
`Profile` | Read credentials from the  profile in the AWS config file
`AssumeRole` | Assume a particular role. Should specify ARN in
`AssumeRoleWebIdentity` | Assume a particular role using the current environment variables.

## NamespaceConfig

Configuration of a particular namespace

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this namespace
`backend` | `string`<br>Backend to use for this namespace
`prefix` | `string`<br>Prefix for items within this namespace
`gcFrequencyHrs` | `number`<br>How frequently to run garbage collection, in hours.
`gcDelayHrs` | `number`<br>How long to keep newly uploaded orphanned objects before allowing them to be deleted, in hours.
`enableAliases` | `boolean`<br>Support querying exports by their aliases
`acl` | [AclConfig](#aclconfig)<br>Access list for this namespace

## SymbolsConfig

Configuration for the tools system

Name | Description
---- | -----------
`stores` | [SymbolStoreConfig](#symbolstoreconfig)`[]`<br>List of symbol stores

## SymbolStoreConfig

Configuration for a symbol store

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this store
`namespaceId` | `string`<br>Configuration for the symbol store backend
`public` | `boolean`<br>Whether to make this store available without auth
`acl` | [AclConfig](#aclconfig)<br>Access to the symbol store

## ToolsConfig

Configuration for the tools system

Name | Description
---- | -----------
`tools` | [ToolConfig](#toolconfig)`[]`<br>Tool configurations

## ToolConfig

Options for configuring a tool

Name | Description
---- | -----------
`id` | `string`<br>Unique identifier for the tool
`name` | `string`<br>Name of the tool
`description` | `string`<br>Description for the tool
`category` | `string`<br>Category for the tool. Will cause the tool to be shown in a different tab in the dashboard.
`group` | `string`<br>Grouping key for different variations of the same tool. The dashboard will show these together.
`platforms` | `string[]`<br>Platforms for this tool. Takes the form of a NET RID (https://learn.microsoft.com/en-us/dotnet/core/rid-catalog).
`public` | `boolean`<br>Whether this tool should be exposed for download on a public endpoint without authentication
`showInUgs` | `boolean`<br>Whether to show this tool for download in the UGS tools menu
`showInDashboard` | `boolean`<br>Whether to show this tool for download in the dashboard
`showInToolbox` | `boolean`<br>Whether to show this tool for download in Unreal Toolbox
`metadata` | `string` `->` `string`<br>Metadata for this tool
`namespaceId` | `string`<br>Default namespace for new deployments of this tool
`maxDeployments` | `integer`<br>Maximum number of deployments to retain for this tool. When exceeded, the oldest deployments are removed.
`acl` | [AclConfig](#aclconfig)<br>Permissions for the tool
