[Horde](../../../README.md) > [Configuration](../../Config.md) > *.issues.json

# *.issues.json

Stores configuration for build health issues

Name | Description
---- | -----------
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`channelGroups` | [IssueChannelGroupConfig](#issuechannelgroupconfig)`[]`<br>Issue grouping channels

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## IssueChannelGroupConfig

Configuration for grouping to apply based on logging channels

Name | Description
---- | -----------
`channels` | `string[]`<br>A list of channel logging boundaries to group issues on
