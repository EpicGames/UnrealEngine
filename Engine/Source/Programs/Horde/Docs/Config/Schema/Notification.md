[Horde](../../../README.md) > [Configuration](../../Config.md) > *.notification.json

# *.notification.json

Config for notifications

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this store
`notificationStreams` | [NotificationStreamConfig](#notificationstreamconfig)`[]`<br>Streams for the notifications to be sent on the Horde server
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files

## NotificationStreamConfig

Config for the notifications based on the streams

Name | Description
---- | -----------
`streams` | `string[]`<br>List of streams for this notification
`enabledStreamTags` | `string[]`<br>List of enabled tags for a stream for this notification
`jobNotifications` | [JobNotificationConfig](#jobnotificationconfig)`[]`<br>Configuration for sending experimental Job Summary Slack notifications

## JobNotificationConfig

Configuration for the job notification

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether or not the current configuration is enabled
`template` | `string`<br>Identifier of the template this notification applies to
`namePattern` | `string`<br>Regex pattern used to extract name of the step this notification will use for display
`notificationFormats` | [NotificationFormatConfig](#notificationformatconfig)`[]`<br>List of formats to be applied for this notification
`channels` | `string[]`<br>List of Slack channel or user identifiers the notification will be sent to

## NotificationFormatConfig

Configuration for the format of the notification group

Name | Description
---- | -----------
`group` | `string`<br>Name of the group
`stepPattern` | `string`<br>Regex pattern to determine if a job step belongs within the group
`alternativeNamePattern` | `string`<br>Optional regex pattern to override the

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.
