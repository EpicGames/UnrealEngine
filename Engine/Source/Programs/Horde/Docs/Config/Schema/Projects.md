[Horde](../../../README.md) > [Configuration](../../Config.md) > *.project.json

# *.project.json

Stores configuration for a project

Name | Description
---- | -----------
`id` | `string`<br>The project id
`name` | `string`<br>Name for the new project
`path` | `string`<br>Direct include path for the project config. For backwards compatibility with old config files when including from a GlobalConfig object.
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within the global scope
`order` | `integer`<br>Order of this project on the dashboard
`logo` | `string`<br>Path to the project logo
`logoDarkTheme` | `string`<br>Optional path to the project logo for the dark theme
`pools` | [PoolConfig](#poolconfig)`[]`<br>List of pools for this project
`categories` | [ProjectCategoryConfig](#projectcategoryconfig)`[]`<br>Categories to include in this project
`jobOptions` | [JobOptions](#joboptions)<br>Default settings for executing jobs
`retentionOptions` | [RetentionOptions](#retentionoptions)<br>Default storage retention options for all streams in this project
`workspaceTypes` | `string` `->` [WorkspaceConfig](#workspaceconfig)<br>Default workspace types for streams These are added to the list of each stream's workspace types.
`ignorePatternFiles` | `string[]`<br>Additional ignore pattern files applied to all streams in this project. Streams can append additional files via their own IgnorePatternFiles property. Paths are relative to workspace root.
`telemetryStoreId` | `string`<br>Telemetry store for Horde data for this project
`streams` | [StreamConfig](Streams.md)`[]`<br>List of streams
`artifactTypes` | [ArtifactTypeConfig](#artifacttypeconfig)`[]`<br>Permissions for artifact types
`acl` | [AclConfig](#aclconfig)<br>Acl entries

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## ConfigMacro

Declares a config macro

Name | Description
---- | -----------
`name` | `string`<br>Name of the macro property
`value` | `string`<br>Value for the macro property

## PoolConfig

Mutable configuration for a pool

Name | Description
---- | -----------
`id` | `string`<br>Unique id for this pool
`base` | `string`<br>Base pool config to copy settings from
`name` | `string`<br>Name of the pool
`condition` | `string`<br>Condition for agents to automatically be included in this pool
`properties` | `string` `->` `string`<br>Arbitrary properties related to this pool
`color` | [PoolColor](#poolcolor-enum)<br>Color to use for this pool on the dashboard
`enableAutoscaling` | `boolean`<br>Whether to enable autoscaling for this pool
`minAgents` | `integer`<br>The minimum number of agents to keep in the pool
`numReserveAgents` | `integer`<br>The minimum number of idle agents to hold in reserve
`conformInterval` | `string`<br>Interval between conforms. If zero, the pool will not conform on a schedule.
`scaleOutCooldown` | `string`<br>Cooldown time between scale-out events
`scaleInCooldown` | `string`<br>Cooldown time between scale-in events
`shutdownIfDisabledGracePeriod` | `string`<br>Time to wait before shutting down an agent that has been disabled
`sizeStrategy` | [PoolSizeStrategy](#poolsizestrategy-enum)<br>
`sizeStrategies` | [PoolSizeStrategyInfo](#poolsizestrategyinfo)`[]`<br>List of pool sizing strategies for this pool. The first strategy with a matching condition will be picked.
`fleetManagers` | [FleetManagerInfo](#fleetmanagerinfo)`[]`<br>List of fleet managers for this pool. The first strategy with a matching condition will be picked. If empty or no conditions match, a default fleet manager will be used.
`leaseUtilizationSettings` | [LeaseUtilizationSettings](#leaseutilizationsettings)<br>Settings for lease utilization pool sizing strategy (if used)
`jobQueueSettings` | [JobQueueSettings](#jobqueuesettings)<br>Settings for job queue pool sizing strategy (if used)
`computeQueueAwsMetricSettings` | [ComputeQueueAwsMetricSettings](#computequeueawsmetricsettings)<br>

## PoolColor (Enum)

Color to use for labels of this pool

Name | Description
---- | -----------
`Default` | 
`Blue` | 
`Orange` | 
`Green` | 
`Gray` | 

## PoolSizeStrategy (Enum)

Available pool sizing strategies

Name | Description
---- | -----------
`LeaseUtilization` | Strategy based on lease utilization
`JobQueue` | Strategy based on size of job build queue
`NoOp` | No-op strategy used as fallback/default behavior
`ComputeQueueAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet
`LeaseUtilizationAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet

## PoolSizeStrategyInfo

Metadata for configuring and picking a pool sizing strategy

Name | Description
---- | -----------
`type` | [PoolSizeStrategy](#poolsizestrategy-enum)<br>Strategy implementation to use
`condition` | `string`<br>Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
`config` | `object`<br>Configuration for the strategy, serialized as JSON
`extraAgentCount` | `integer`<br>Integer to add after pool size has been calculated. Can also be negative.

## FleetManagerInfo

Metadata for configuring and picking a fleet manager

Name | Description
---- | -----------
`type` | [FleetManagerType](#fleetmanagertype-enum)<br>Fleet manager type implementation to use
`condition` | `string`<br>Condition if this strategy should be enabled (right now, using date/time as a distinguishing factor)
`config` | `object`<br>Configuration for the strategy, serialized as JSON

## FleetManagerType (Enum)

Available fleet managers

Name | Description
---- | -----------
`Default` | Default fleet manager
`NoOp` | No-op fleet manager.
`Aws` | Fleet manager for handling AWS EC2 instances. Will create and/or terminate instances from scratch.
`AwsReuse` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsRecycle` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsAsg` | Fleet manager for handling AWS EC2 instances. Uses an EC2 auto-scaling group for controlling the number of running instances.

## LeaseUtilizationSettings

Lease utilization sizing settings for a pool

Name | Description
---- | -----------
`sampleTimeSec` | `integer`<br>Time period for each sample
`numSamples` | `integer`<br>Number of samples to collect for calculating lease utilization
`numSamplesForResult` | `integer`<br>Min number of samples for a valid result
`minAgents` | `integer`<br>The minimum number of agents to keep in the pool
`numReserveAgents` | `integer`<br>The minimum number of idle agents to hold in reserve

## JobQueueSettings

Job queue sizing settings for a pool

Name | Description
---- | -----------
`scaleOutFactor` | `number`<br>Factor translating queue size to additional agents to grow the pool with The result is always rounded up to nearest integer. Example: if there are 20 jobs in queue, a factor 0.25 will result in 5 new agents being added (20 * 0.25)
`scaleInFactor` | `number`<br>Factor by which to shrink the pool size with when queue is empty The result is always rounded up to nearest integer. Example: when the queue size is zero, a default value of 0.9 will shrink the pool by 10% (current agent count * 0.9)
`samplePeriodMin` | `integer`<br>How far back in time to look for job batches (that potentially are in the queue)
`readyTimeThresholdSec` | `integer`<br>Time spent in ready state before considered truly waiting for an agent<br>A job batch can be in ready state before getting picked up and executed. This threshold will help ensure only batches that have been waiting longer than this value will be considered.

## ComputeQueueAwsMetricSettings

Settings for

Name | Description
---- | -----------
`computeClusterId` | `string`<br>Compute cluster ID to observe
`namespace` | `string`<br>AWS CloudWatch namespace to write metrics in

## ProjectCategoryConfig

Information about a category to display for a stream

Name | Description
---- | -----------
`name` | `string`<br>Name of this category
`row` | `integer`<br>Index of the row to display this category on
`showOnNavMenu` | `boolean`<br>Whether to show this category on the nav menu
`includePatterns` | `string[]`<br>Patterns for stream names to include
`excludePatterns` | `string[]`<br>Patterns for stream names to exclude

## JobOptions

Options for executing a job

Name | Description
---- | -----------
`executor` | `string`<br>Name of the executor to use
`useWine` | `boolean`<br>Whether to execute using Wine emulation on Linux
`runInSeparateProcess` | `boolean`<br>Executes the job lease in a separate process
`workspaceMaterializer` | `string`<br>What workspace materializer to use in WorkspaceExecutor. Will override any value from workspace config.
`container` | [JobContainerOptions](#jobcontaineroptions)<br>Options for executing a job inside a container
`expireAfterDays` | `integer`<br>Number of days after which to expire jobs
`driver` | `string`<br>Name of the driver to use

## JobContainerOptions

Options for executing a job inside a container

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether to execute job inside a container
`imageUrl` | `string`<br>Image URL to container, such as "quay.io/podman/hello"
`containerEngineExecutable` | `string`<br>Container engine executable (docker or with full path like /usr/bin/podman)
`extraArguments` | `string`<br>Additional arguments to pass to container engine

## RetentionOptions

Options controlling storage retention and GC behavior for a stream. Boolean flags are safety valves: only false is meaningful (disables behavior). null means "not disabled" (default). Setting true is identical to null.

Name | Description
---- | -----------
`enableJobExpiration` | `boolean`<br>Whether job expiration is enabled for this stream. When null (default), expiration runs normally if ExpireAfterDays is greater than 0. When explicitly false, disables expiration regardless of ExpireAfterDays (safety valve).
`enableArtifactExpiration` | `boolean`<br>Override for artifact retention behavior. When false, artifact expiry is paused for this stream (useful during investigations).
`maxJobDeletionsPerTick` | `integer`<br>Maximum number of jobs to delete per hourly expiration tick, per stream. Prevents thundering-herd behavior when ExpireAfterDays is first enabled on a stream with a large backlog. Default null, treated as 1000 by consumers via null-coalescing (safe-by-default cap). Set to 0 for unlimited.

## WorkspaceConfig

Information about a workspace type

Name | Description
---- | -----------
`base` | `string`<br>
`cluster` | `string`<br>Name of the Perforce server cluster to use
`serverAndPort` | `string`<br>The Perforce server and port (eg. perforce:1666)
`userName` | `string`<br>User to log into Perforce with (defaults to buildmachine)
`identifier` | `string`<br>Identifier to distinguish this workspace from other workspaces. Defaults to the workspace type name.
`stream` | `string`<br>Override for the stream to sync
`view` | `string[]`<br>Custom view for the workspace
`incremental` | `boolean`<br>Whether to use an incrementally synced workspace
`useAutoSdk` | `boolean`<br>Whether to use the AutoSDK
`autoSdkView` | `string[]`<br>View for the AutoSDK paths to sync. If null, the whole thing will be synced.
`method` | `string`<br>Method to use when syncing/materializing data from Perforce
`minScratchSpace` | `integer`<br>Minimum disk space that must be available *after* syncing this workspace (in megabytes) If not available, the job will be aborted.
`conformDiskFreeSpace` | `integer`<br>Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk. Set to null or 0 to disable.

## ArtifactTypeConfig

Configuration for an artifact

Name | Description
---- | -----------
`name` | `string`<br>Legacy 'Name' property
`type` | `string`<br>Name of the artifact type
`acl` | [AclConfig](#aclconfig)<br>Acl for the artifact type
`keepCount` | `integer`<br>Number of artifacts to retain
`keepDays` | `integer`<br>Number of days to retain artifacts of this type
`namespaceId` | `string`<br>Storage namespace to use for this artifact types

## AclConfig

Parameters to update an ACL

Name | Description
---- | -----------
`entries` | [AclEntryConfig](#aclentryconfig)`[]`<br>Entries to replace the existing ACL
`profiles` | [AclProfileConfig](#aclprofileconfig)`[]`<br>Defines profiles which allow grouping sets of actions into named collections
`inherit` | `boolean`<br>Whether to inherit permissions from the parent ACL
`exceptions` | `string[]`<br>List of exceptions to the inherited setting

## AclEntryConfig

Individual entry in an ACL

Name | Description
---- | -----------
`claim` | [AclClaimConfig](#aclclaimconfig)<br>Name of the user or group
`actions` | `string[]`<br>Array of actions to allow
`profiles` | `string[]`<br>List of profiles to grant

## AclClaimConfig

New claim to create

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## AclProfileConfig

Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this profile
`actions` | `string[]`<br>Actions to include
`excludeActions` | `string[]`<br>Actions to exclude from the inherited actions
`extends` | `string[]`<br>Other profiles to extend from
