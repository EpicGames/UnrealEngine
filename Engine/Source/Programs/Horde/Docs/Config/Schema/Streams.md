[Horde](../../../README.md) > [Configuration](../../Config.md) > *.stream.json

# *.stream.json

Config for a stream

Name | Description
---- | -----------
`id` | `string`<br>Identifier for the stream
`vCS` | `string`<br>Version control service for stream commits
`repositoryName` | `string`<br>Repository name for version control systems that support multiple repositories
`defaultBranchName` | `string`<br>Default branch name for version control systems that use branches (e.g., "main"). If not set, defaults to "main".
`path` | `string`<br>Direct include path for the stream config. For backwards compatibility with old config files when including from a ProjectConfig object.
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within this stream
`name` | `string`<br>Name of the stream
`enginePath` | `string`<br>Path to the engine directory within the workspace. Used for launching UAT.
`ignorePatternFiles` | `string[]`<br>Additional file paths (relative to workspace root) containing ignore patterns for the log parser. Each file should contain one regex per line, with lines starting with '#' treated as comments. Inherits from ProjectConfig and appends stream-level entries.
`clusterName` | `string`<br>The perforce cluster containing the stream
`order` | `integer`<br>Order for this stream
`initialAgentType` | `string`<br>Default initial agent type for templates
`notificationChannel` | `string`<br>Notification channel for all jobs in this stream
`notificationChannelFilter` | `string`<br>Notification channel filter for this template. Can be Success, Failure, or Warnings.
`triageChannel` | `string`<br>Channel to post issue triage notifications
`jobOptions` | [JobOptions](#joboptions)<br>Default settings for executing jobs
`retentionOptions` | [RetentionOptions](#retentionoptions)<br>Storage retention options for this stream
`telemetryStoreId` | `string`<br>Telemetry store for Horde data for this stream
`autoSdkView` | `string[]`<br>View for the AutoSDK paths to sync. If null, the whole thing will be synced.
`defaultPreflightTemplate` | `string`<br>Legacy name for the default preflight template
`defaultPreflight` | [DefaultPreflightConfig](#defaultpreflightconfig)<br>Default template for running preflights
`commitTags` | [CommitTagConfig](#committagconfig)`[]`<br>List of tags to apply to commits. Allows fast searching and classification of different commit types (eg. code vs content).
`tabs` | [TabConfig](#tabconfig)`[]`<br>List of tabs to show for the new stream
`environment` | `string` `->` `string`<br>Global environment variables for all agents in this stream
`agentTypes` | `string` `->` [AgentConfig](#agentconfig)<br>Map of agent name to type
`workspaceTypes` | `string` `->` [WorkspaceConfig](#workspaceconfig)<br>Map of workspace name to type
`templates` | [TemplateRefConfig](#templaterefconfig)`[]`<br>List of templates to create
`aliases` | [StreamAliasConfig](#streamaliasconfig)`[]`<br>Stream aliases configured for this stream. Each alias provides a named subset of the stream's templates.
`acl` | [AclConfig](#aclconfig)<br>Custom permissions for this object
`pausedUntil` | `string`<br>Pause stream builds until specified date
`pauseComment` | `string`<br>Reason for pausing builds of the stream
`replicators` | [ReplicatorConfig](#replicatorconfig)`[]`<br>Configuration for workers to replicate commit data into Horde Storage.
`workflows` | [WorkflowConfig](#workflowconfig)`[]`<br>Workflows for dealing with new issues
`tokens` | [TokenConfig](#tokenconfig)`[]`<br>Tokens to create for each job step
`artifactTypes` | [ArtifactTypeConfig](#artifacttypeconfig)`[]`<br>Permissions for artifact types
`streamTags` | [StreamTag](#streamtag)`[]`<br>Tags for the stream

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## ConfigMacro

Declares a config macro

Name | Description
---- | -----------
`name` | `string`<br>Name of the macro property
`value` | `string`<br>Value for the macro property

## JobOptions

Options for executing a job

Name | Description
---- | -----------
`executor` | `string`<br>Name of the executor to use
`useWine` | `boolean`<br>Whether to execute using Wine emulation on Linux
`runInSeparateProcess` | `boolean`<br>Executes the job lease in a separate process
`workspaceMaterializer` | `string`<br>What workspace materializer to use in WorkspaceExecutor. Will override any value from workspace config.
`container` | [JobContainerOptions](#jobcontaineroptions)<br>Options for executing a job inside a container
`expireAfterDays` | `integer`<br>Number of days after which to expire jobs
`driver` | `string`<br>Name of the driver to use

## JobContainerOptions

Options for executing a job inside a container

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether to execute job inside a container
`imageUrl` | `string`<br>Image URL to container, such as "quay.io/podman/hello"
`containerEngineExecutable` | `string`<br>Container engine executable (docker or with full path like /usr/bin/podman)
`extraArguments` | `string`<br>Additional arguments to pass to container engine

## RetentionOptions

Options controlling storage retention and GC behavior for a stream. Boolean flags are safety valves: only false is meaningful (disables behavior). null means "not disabled" (default). Setting true is identical to null.

Name | Description
---- | -----------
`enableJobExpiration` | `boolean`<br>Whether job expiration is enabled for this stream. When null (default), expiration runs normally if ExpireAfterDays is greater than 0. When explicitly false, disables expiration regardless of ExpireAfterDays (safety valve).
`enableArtifactExpiration` | `boolean`<br>Override for artifact retention behavior. When false, artifact expiry is paused for this stream (useful during investigations).
`maxJobDeletionsPerTick` | `integer`<br>Maximum number of jobs to delete per hourly expiration tick, per stream. Prevents thundering-herd behavior when ExpireAfterDays is first enabled on a stream with a large backlog. Default null, treated as 1000 by consumers via null-coalescing (safe-by-default cap). Set to 0 for unlimited.

## DefaultPreflightConfig

Specifies defaults for running a preflight

Name | Description
---- | -----------
`templateId` | `string`<br>The template id to query
`change` | [ChangeQueryConfig](#changequeryconfig)<br>Query for the change to use

## ChangeQueryConfig

Query selecting the base changelist to use

Name | Description
---- | -----------
`name` | `string`<br>Name of this query, for display on the dashboard.
`condition` | `string`<br>Condition to evaluate before deciding to use this query. May query tags in a preflight.
`templateId` | `string`<br>The template id to query
`target` | `string`<br>The target to query
`outcomes` | [JobStepOutcome](#jobstepoutcome-enum)`[]`<br>Whether to match a job that produced warnings
`commitTag` | `string`<br>Finds the last commit with this tag

## JobStepOutcome (Enum)

Outcome for a jobstep

Name | Description
---- | -----------
`Unspecified` | Outcome is not known
`Failure` | Step failed
`Warnings` | Step completed with warnings
`Success` | Step succeeded

## CommitTagConfig

Configuration for custom commit filters

Name | Description
---- | -----------
`name` | `string`<br>Name of the tag
`base` | `string`<br>Base tag to copy settings from
`filter` | `string[]`<br>List of files to be included in this filter

## TabConfig

Information about a page to display in the dashboard for a stream

Name | Description
---- | -----------
`title` | `string`<br>Title of this page
`type` | `string`<br>Type of this tab
`style` | [TabStyle](#tabstyle-enum)<br>Presentation style for this page
`showNames` | `boolean`<br>Whether to show job names on this page
`showPreflights` | `boolean`<br>Whether to show all user preflights
`jobNames` | `string[]`<br>Names of jobs to include on this page. If there is only one name specified, the name column does not need to be displayed.
`templates` | `string[]`<br>List of job template names to show on this page.
`columns` | [TabColumnConfig](#tabcolumnconfig)`[]`<br>Columns to display for different types of aggregates

## TabStyle (Enum)

Style for rendering a tab

Name | Description
---- | -----------
`Normal` | Regular job list
`Compact` | Omit job names, show condensed view

## TabColumnConfig

Describes a column to display on the jobs page

Name | Description
---- | -----------
`type` | [TabColumnType](#tabcolumntype-enum)<br>The type of column
`heading` | `string`<br>Heading for this column
`category` | `string`<br>Category of aggregates to display in this column. If null, includes any aggregate not matched by another column.
`parameter` | `string`<br>Parameter to show in this column
`relativeWidth` | `integer`<br>Relative width of this column.

## TabColumnType (Enum)

Type of a column in a jobs tab

Name | Description
---- | -----------
`Labels` | Contains labels
`Parameter` | Contains parameters

## AgentConfig

Mapping from a BuildGraph agent type to a set of machines on the farm

Name | Description
---- | -----------
`base` | `string`<br>Base agent config to inherit settings from
`pool` | `string`<br>Pool of agents to use for this agent type
`workspace` | `string`<br>Name of the workspace to sync
`tempStorageDir` | `string`<br>Path to the temporary storage dir
`environment` | `string` `->` `string`<br>Environment variables to be set when executing the job
`tokens` | [TokenConfig](#tokenconfig)`[]`<br>Tokens to allocate for this agent type

## TokenConfig

Configuration for allocating access tokens for each job

Name | Description
---- | -----------
`url` | `string`<br>URL to request tokens from
`clientId` | `string`<br>Client id to use to request a new token
`clientSecret` | `string`<br>Client secret to request a new access token
`envVar` | `string`<br>Environment variable to set with the access token

## WorkspaceConfig

Information about a workspace type

Name | Description
---- | -----------
`base` | `string`<br>
`cluster` | `string`<br>Name of the Perforce server cluster to use
`serverAndPort` | `string`<br>The Perforce server and port (eg. perforce:1666)
`userName` | `string`<br>User to log into Perforce with (defaults to buildmachine)
`identifier` | `string`<br>Identifier to distinguish this workspace from other workspaces. Defaults to the workspace type name.
`stream` | `string`<br>Override for the stream to sync
`view` | `string[]`<br>Custom view for the workspace
`incremental` | `boolean`<br>Whether to use an incrementally synced workspace
`useAutoSdk` | `boolean`<br>Whether to use the AutoSDK
`autoSdkView` | `string[]`<br>View for the AutoSDK paths to sync. If null, the whole thing will be synced.
`method` | `string`<br>Method to use when syncing/materializing data from Perforce
`minScratchSpace` | `integer`<br>Minimum disk space that must be available *after* syncing this workspace (in megabytes) If not available, the job will be aborted.
`conformDiskFreeSpace` | `integer`<br>Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk. Set to null or 0 to disable.

## TemplateRefConfig

Parameters to create a template within a stream

Name | Description
---- | -----------
`id` | `string`<br>Optional identifier for this ref. If not specified, an id will be generated from the name.
`base` | `string`<br>Base template id to copy from
`showUgsBadges` | `boolean`<br>Whether to show badges in UGS for these jobs
`showUgsAlerts` | `boolean`<br>Whether to show alerts in UGS for these jobs
`notificationChannel` | `string`<br>Notification channel for this template. Overrides the stream channel if set.
`notificationChannelFilter` | `string`<br>Notification channel filter for this template. Can be a combination of "Success", "Failure" and "Warnings" separated by pipe characters.
`triageChannel` | `string`<br>Triage channel for this template. Overrides the stream channel if set.
`workflowId` | `string`<br>Workflow to user for this stream
`annotations` | `string` `->` `string`<br>Default annotations to apply to nodes in this template
`schedule` | [ScheduleConfig](#scheduleconfig)<br>Schedule to execute this template
`chainedJobs` | [ChainedJobTemplateConfig](#chainedjobtemplateconfig)`[]`<br>List of chained job triggers
`acl` | [AclConfig](#aclconfig)<br>The ACL for this template
`name` | `string`<br>Name for the new template
`description` | `string`<br>Description for the template
`priority` | [Priority](#priority-enum)<br>Default priority for this job
`allowPreflights` | `boolean`<br>Whether to allow preflights of this template
`updateIssues` | `boolean`<br>Whether issues should be updated for all jobs using this template
`promoteIssuesByDefault` | `boolean`<br>Whether issues should be promoted by default for this template, promoted issues will generate user notifications
`initialAgentType` | `string`<br>Initial agent type to parse the buildgraph script on
`submitNewChange` | `string`<br>Path to a file within the stream to submit to generate a new changelist for jobs
`submitDescription` | `string`<br>Description for new changelists
`defaultChange` | [ChangeQueryConfig](#changequeryconfig)`[]`<br>Default change to build at. Each object has a condition parameter which can evaluated by the server to determine which change to use.
`arguments` | `string[]`<br>Fixed arguments for the new job
`parameters` | [TemplateTextParameterConfig](#templatetextparameterconfig)/[TemplateListParameterConfig](#templatelistparameterconfig)/[TemplateBoolParameterConfig](#templateboolparameterconfig)`[]`<br>Parameters for this template
`jobOptions` | [JobOptions](#joboptions)<br>Default settings for jobs

## ScheduleConfig

Parameters to create a new schedule

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether the schedule should be enabled
`condition` | `string`<br>Condition for whether the schedule should be enabled. When a condition is provided the result of the expression will be logically AND with the  property to decide if the schedule should be enabled.
`maxActive` | `integer`<br>Maximum number of builds that can be active at once
`maxChanges` | `integer`<br>Maximum number of changes the schedule can fall behind head revision. If greater than zero, builds will be triggered for every submitted changelist until the backlog is this size.
`requireSubmittedChange` | `boolean`<br>Whether the build requires a change to be submitted
`gate` | [ScheduleGateConfig](#schedulegateconfig)<br>Gate allowing the schedule to trigger
`commits` | `string[]`<br>Commit tags for this schedule
`claims` | [ScheduleClaimConfig](#scheduleclaimconfig)`[]`<br>Roles to impersonate for this schedule
`filter` | [ChangeContentFlags](#changecontentflags-enum)`[]`<br>The types of changes to run for
`files` | `string[]`<br>Files that should cause the job to trigger
`templateParameters` | `string` `->` `string`<br>Parameters for the template
`patterns` | [SchedulePatternConfig](#schedulepatternconfig)`[]`<br>New patterns for the schedule

## ScheduleGateConfig

Gate allowing a schedule to trigger.

Name | Description
---- | -----------
`templateId` | `string`<br>The template containing the dependency
`target` | `string`<br>Target to wait for

## ScheduleClaimConfig

Claim to grant to leases running a scheduled job

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## ChangeContentFlags (Enum)

Flags identifying content of a changelist

Name | Description
---- | -----------
`ContainsCode` | The change contains code
`ContainsContent` | The change contains content

## SchedulePatternConfig

Parameters to create a new schedule

Name | Description
---- | -----------
`daysOfWeek` | [DayOfWeek](#dayofweek-enum)`[]`<br>Days of the week to run this schedule on. If null, the schedule will run every day.
`minTime` | `string`<br>Time during the day for the first schedule to trigger. Measured in minutes from midnight.
`maxTime` | `string`<br>Time during the day for the last schedule to trigger. Measured in minutes from midnight.
`interval` | `string`<br>Interval between each schedule triggering

## DayOfWeek (Enum)

Name | Description
---- | -----------
`Sunday` | 
`Monday` | 
`Tuesday` | 
`Wednesday` | 
`Thursday` | 
`Friday` | 
`Saturday` | 

## ChainedJobTemplateConfig

Trigger for another template

Name | Description
---- | -----------
`trigger` | `string`<br>Name of the target that needs to complete before starting the other template
`templateId` | `string`<br>Id of the template to trigger
`useDefaultChangeForTemplate` | `boolean`<br>Whether to use the default change for the template rather than the change for the parent job.

## AclConfig

Parameters to update an ACL

Name | Description
---- | -----------
`entries` | [AclEntryConfig](#aclentryconfig)`[]`<br>Entries to replace the existing ACL
`profiles` | [AclProfileConfig](#aclprofileconfig)`[]`<br>Defines profiles which allow grouping sets of actions into named collections
`inherit` | `boolean`<br>Whether to inherit permissions from the parent ACL
`exceptions` | `string[]`<br>List of exceptions to the inherited setting

## AclEntryConfig

Individual entry in an ACL

Name | Description
---- | -----------
`claim` | [AclClaimConfig](#aclclaimconfig)<br>Name of the user or group
`actions` | `string[]`<br>Array of actions to allow
`profiles` | `string[]`<br>List of profiles to grant

## AclClaimConfig

New claim to create

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## AclProfileConfig

Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this profile
`actions` | `string[]`<br>Actions to include
`excludeActions` | `string[]`<br>Actions to exclude from the inherited actions
`extends` | `string[]`<br>Other profiles to extend from

## Priority (Enum)

Priority of a job or step

Name | Description
---- | -----------
`Unspecified` | Not specified
`Lowest` | Lowest priority
`BelowNormal` | Below normal priority
`Normal` | Normal priority
`AboveNormal` | Above normal priority
`High` | High priority
`Highest` | Highest priority

## TemplateTextParameterConfig

Free-form text entry parameter

Name | Description
---- | -----------
`type` | Text<br>Type discriminator
`id` | `string`<br>Identifier for this parameter
`label` | `string`<br>Name of the parameter associated with this parameter.
`argument` | `string`<br>Argument to pass to the executor
`default` | `string`<br>Default value for this argument
`scheduleOverride` | `string`<br>Override for the default value for this parameter when running a scheduled build
`hint` | `string`<br>Hint text for this parameter
`validation` | `string`<br>Regex used to validate this parameter
`validationError` | `string`<br>Message displayed if validation fails, informing user of valid values.
`toolTip` | `string`<br>Tool-tip text to display

## TemplateListParameterConfig

Allows the user to select a value from a constrained list of choices

Name | Description
---- | -----------
`type` | List<br>Type discriminator
`label` | `string`<br>Label to display next to this parameter. Defaults to the parameter name.
`style` | [TemplateListParameterStyle](#templatelistparameterstyle-enum)<br>The type of list parameter
`items` | [TemplateListParameterItemConfig](#templatelistparameteritemconfig)`[]`<br>List of values to display in the list
`toolTip` | `string`<br>Tool tip text to display

## TemplateListParameterStyle (Enum)

Style of list parameter

Name | Description
---- | -----------
`List` | Regular drop-down list. One item is always selected.
`MultiList` | Drop-down list with checkboxes
`TagPicker` | Tag picker from list of options

## TemplateListParameterItemConfig

Possible option for a list parameter

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this parameter
`group` | `string`<br>Optional group heading to display this entry under, if the picker style supports it.
`text` | `string`<br>Name of the parameter associated with this list.
`argumentIfEnabled` | `string`<br>Argument to pass with this parameter.
`argumentsIfEnabled` | `string[]`<br>Arguments to pass with this parameter.
`argumentIfDisabled` | `string`<br>Argument to pass with this parameter.
`argumentsIfDisabled` | `string[]`<br>Arguments to pass if this parameter is disabled.
`default` | `boolean`<br>Whether this item is selected by default
`scheduleOverride` | `boolean`<br>Overridden value for this property in schedule builds

## TemplateBoolParameterConfig

Allows the user to toggle an option on or off

Name | Description
---- | -----------
`type` | Bool<br>Type discriminator
`id` | `string`<br>Identifier for this parameter
`label` | `string`<br>Name of the parameter associated with this parameter.
`argumentIfEnabled` | `string`<br>Argument to add if this parameter is enabled
`argumentsIfEnabled` | `string[]`<br>Argument to add if this parameter is enabled
`argumentIfDisabled` | `string`<br>Argument to add if this parameter is enabled
`argumentsIfDisabled` | `string[]`<br>Arguments to add if this parameter is disabled
`default` | `boolean`<br>Whether this argument is enabled by default
`scheduleOverride` | `boolean`<br>Override for this parameter in scheduled builds
`toolTip` | `string`<br>Tool tip text to display

## StreamAliasConfig

Configuration for a stream alias, which represents a named subset of templates

Name | Description
---- | -----------
`name` | `string`<br>The unique name/identifier for this alias
`templates` | `string[]`<br>The subset of template IDs available through this alias. These must be a subset of the enclosing IStream's Templates.

## ReplicatorConfig

Configuration for a stream replicator

Name | Description
---- | -----------
`id` | `string`<br>Identifier for the replicator within the current stream
`enabled` | `boolean`<br>Whether the replicator is enabled
`minChange` | `integer`<br>Minimum change number to replicate
`maxChange` | `integer`<br>Maximum change number to replicate
`includeContent` | `boolean`<br>Whether to include content in the replication, or just metadata
`namespaceId` | `string`<br>Namespace to replicate data to

## WorkflowConfig

Configuration for an issue workflow

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this workflow
`reportTimes` | `string[]`<br>Times of day at which to send a report
`summaryTab` | `string`<br>Name of the tab to post summary data to
`reportChannel` | `string`<br>Channel to post summary information for these templates.
`reportWarnings` | `boolean`<br>Whether to include issues with a warning status in the summary
`groupIssuesByTemplate` | `boolean`<br>Whether to group issues by template in the report
`triageChannel` | `string`<br>Channel to post threads for triaging new issues
`triagePrefix` | `string`<br>Prefix for all triage messages
`triageSuffix` | `string`<br>Suffix for all triage messages
`triageInstructions` | `string`<br>Instructions posted to triage threads
`triageAlias` | `string`<br>User id of a Slack user/alias to ping if there is nobody assigned to an issue by default.
`triageErrors` | `boolean`<br>Whether to include issues with an error status in the triage
`triageWarnings` | `boolean`<br>Whether to include issues with a warning status in the triage
`triageTypeAliases` | `string` `->` `string`<br>Slack user/alias to ping for specific issue types (such as Systemic), if there is nobody assigned to an issue by default.
`escalateAlias` | `string`<br>Alias to ping if an issue has not been resolved for a certain amount of time
`escalateTimes` | `integer[]`<br>Times after an issue has been opened to escalate to the alias above, in minutes. Continues to notify on the last interval once reaching the end of the list.
`escalateRootCauseAnalysisTimes` | `integer[]`<br>Times after a manual root cause analysis has been opened to escalate to the owner, in minutes. Continues to notify on the last interval once reaching the end of the list.
`autoRcaEnabled` | `boolean`<br>Whether to automatically run RCA processing for issues in this workflow
`autoRcaSlackEnabled` | `boolean`<br>Whether to post auto-RCA results to the issue's Slack triage thread
`maxMentions` | `integer`<br>Maximum number of people to mention on a triage thread
`allowMentions` | `boolean`<br>Whether to mention people on this thread. Useful to disable for testing.
`inviteRestrictedUsers` | `boolean`<br>Uses the admin.conversations.invite API to invite users to the channel
`skipWhenEmpty` | `boolean`<br>Skips sending reports when there are no active issues.
`showMergeWarnings` | `boolean`<br>Whether to show warnings about merging changes into the origin stream.
`annotations` | `string` `->` `string`<br>Additional node annotations implicit in this workflow
`externalIssues` | [WorkflowExternalIssuesConfig](#workflowexternalissuesconfig)<br>External issue tracking configuration for this workflow
`issueHandlers` | `string[]`<br>Additional issue handlers enabled for this workflow

## WorkflowExternalIssuesConfig

External issue tracking configuration for a workflow

Name | Description
---- | -----------
`projectKey` | `string`<br>Project key in external issue tracker
`defaultComponentId` | `string`<br>Default component id for issues using workflow
`defaultIssueTypeId` | `string`<br>Default issue type id for issues using workflow

## ArtifactTypeConfig

Configuration for an artifact

Name | Description
---- | -----------
`name` | `string`<br>Legacy 'Name' property
`type` | `string`<br>Name of the artifact type
`acl` | [AclConfig](#aclconfig)<br>Acl for the artifact type
`keepCount` | `integer`<br>Number of artifacts to retain
`keepDays` | `integer`<br>Number of days to retain artifacts of this type
`namespaceId` | `string`<br>Storage namespace to use for this artifact types

## StreamTag

Tags for a stream

Name | Description
---- | -----------
`name` | `string`<br>Name of the tag
`enabled` | `boolean`<br>Whether the tag should be enabled
