[Horde](../../../README.md) > [Configuration](../../Config.md) > *.telemetry.json

# *.telemetry.json

Config for metrics

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this store
`acl` | [AclConfig](#aclconfig)<br>Permissions for this store
`metrics` | [MetricConfig](#metricconfig)`[]`<br>Metrics to aggregate on the Horde server
`views` | [TelemetryViewConfig](#telemetryviewconfig)`[]`<br>Configuration for telemetry views
`include` | [ConfigInclude](#configinclude)`[]`<br>Includes for other configuration files
`macros` | [ConfigMacro](#configmacro)`[]`<br>Macros within this configuration

## AclConfig

Parameters to update an ACL

Name | Description
---- | -----------
`entries` | [AclEntryConfig](#aclentryconfig)`[]`<br>Entries to replace the existing ACL
`profiles` | [AclProfileConfig](#aclprofileconfig)`[]`<br>Defines profiles which allow grouping sets of actions into named collections
`inherit` | `boolean`<br>Whether to inherit permissions from the parent ACL
`exceptions` | `string[]`<br>List of exceptions to the inherited setting

## AclEntryConfig

Individual entry in an ACL

Name | Description
---- | -----------
`claim` | [AclClaimConfig](#aclclaimconfig)<br>Name of the user or group
`actions` | `string[]`<br>Array of actions to allow
`profiles` | `string[]`<br>List of profiles to grant

## AclClaimConfig

New claim to create

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## AclProfileConfig

Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this profile
`actions` | `string[]`<br>Actions to include
`excludeActions` | `string[]`<br>Actions to exclude from the inherited actions
`extends` | `string[]`<br>Other profiles to extend from

## MetricConfig

Configures a metric to aggregate on the server

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this metric
`filter` | `string`<br>Filter expression to evaluate to determine which events to include. This query is evaluated against an array.
`property` | `string`<br>Property to aggregate
`groupBy` | `string`<br>Property to group by. Specified as a comma-separated list of JSON path expressions.
`function` | [AggregationFunction](#aggregationfunction-enum)<br>How to aggregate samples for this metric
`percentile` | `integer`<br>For the percentile function, specifies the percentile to measure
`topN` | `integer`<br>TopN aggregation
`bottomN` | `integer`<br>BottomN aggregation
`interval` | `string`<br>Interval for each metric. Supports times such as "2d", "1h", "1h30m", "20s".

## AggregationFunction (Enum)

Method for aggregating samples into a metric

Name | Description
---- | -----------
`Count` | Count the number of matching elements
`Min` | Take the minimum value of all samples
`Max` | Take the maximum value of all samples
`Sum` | Sum all the reported values
`Average` | Average all the samples
`Percentile` | Estimates the value at a certain percentile

## TelemetryViewConfig

A telemetry view of related metrics, divided into categofies

Name | Description
---- | -----------
`id` | `string`<br>Identifier for the view
`name` | `string`<br>The name of the view
`telemetryStoreId` | `string`<br>The telemetry store this view uses
`variables` | [TelemetryVariableConfig](#telemetryvariableconfig)`[]`<br>The variables used to filter the view data
`categories` | [TelemetryCategoryConfig](#telemetrycategoryconfig)`[]`<br>The categories contained within the view

## TelemetryVariableConfig

A telemetry view variable used for filtering the charting data

Name | Description
---- | -----------
`name` | `string`<br>The name of the variable for display purposes
`group` | `string`<br>The associated data group attached to the variable
`defaults` | `string[]`<br>The default values to select

## TelemetryCategoryConfig

A chart categody, will be displayed on the dashbord under an associated pivot

Name | Description
---- | -----------
`name` | `string`<br>The name of the category
`charts` | [TelemetryChartConfig](#telemetrychartconfig)`[]`<br>The charts contained within the category

## TelemetryChartConfig

Telemetry chart configuraton

Name | Description
---- | -----------
`name` | `string`<br>The name of the chart, will be displayed on the dashboard
`display` | [TelemetryMetricUnitType](#telemetrymetricunittype-enum)<br>The unit to display
`graph` | [TelemetryMetricGraphType](#telemetrymetricgraphtype-enum)<br>The graph type
`metrics` | [TelemetryChartMetricConfig](#telemetrychartmetricconfig)`[]`<br>List of configured metrics
`min` | `integer`<br>The min unit value for clamping chart
`max` | `integer`<br>The max unit value for clamping chart

## TelemetryMetricUnitType (Enum)

The units used to present the telemetry

Name | Description
---- | -----------
`Time` | Time duration
`Ratio` | Ratio 0-100%
`Value` | Artbitrary numeric value

## TelemetryMetricGraphType (Enum)

The type of

Name | Description
---- | -----------
`Line` | A line graph
`Indicator` | Key performance indicator (KPI) chart with thrasholds

## TelemetryChartMetricConfig

Metric attached to a telemetry chart

Name | Description
---- | -----------
`id` | `string`<br>Associated metric id
`threshold` | `integer`<br>The threshold for KPI values
`alias` | `string`<br>The metric alias for display purposes

## ConfigInclude

Directive to merge config data from another source

Name | Description
---- | -----------
`path` | `string`<br>Path to the config data to be included. May be relative to the including file's location.

## ConfigMacro

Declares a config macro

Name | Description
---- | -----------
`name` | `string`<br>Name of the macro property
`value` | `string`<br>Value for the macro property
