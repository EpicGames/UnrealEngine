[Horde](../../README.md) > [Configuration](../Config.md) > Secrets

# Secrets

Horde implements an API for retrieving secrets that can be stored in its own configuration file or obtained from
an external source. Marshalling data through Horde allows access to be controlled using Horde's permissions model,
and for automated processes to impersonate the user that requested them.

At the moment, Horde only supports the AWS Parameter Store and HashiCorp Vault as external secret providers out of the box, though 
other implementations can be added through the `ISecretProvider` interface.

## Configuring secrets

Secrets are defined in the `secrets` list of the [`globals.json`](Schema/Globals.md) file. Each 
[entry](Schema/Globals.md#secretconfig) includes an identifier for the secret (`id`), a set of key value
pairs (`data`) and an [ACL](Permissions.md) controlling who can access it.

Additional keys and values may be merged from external providers by adding entries to the `sources` array.
Each entry here contains the `name` of the provider to obtain the secret from, and a `path` used to locate
the secret in a provider-specific syntax.

Secrets from external providers may take two forms, determined by the `format` property.

* `text` secrets are simple string values which are added to the secret's set of key/value pairs using
  the specified `key` property.
* `json` secrets are parsed as JSON objects and merged into the secret's key/value pairs using property
  names as keys.

An example secret may be configured as follows:

   ```json
    "secrets": [
        {
            "id": "horde-secrets",

            // Some hard-coded property values        
            "data": {
                "aws-region": "us-east-1"
            },

            // Some values read from the AWS parameter store
            "sources": [

                // Read a single secret from the AWS parameter store and assign it to "aws-secret-access-key"
                {
                    "provider": "AwsParameterStore",
                    "key": "aws-secret-access-key",
                    "path": "name-of-secret-in-parameter-store"
                },

                // Read a JSON object from the AWS parameter store and merge all the key/value pairs into this secret.
                {
                    "provider": "AwsParameterStore",
                    "format": "json",
                    "path": "name-of-secret-in-parameter-store"
                },
            ],

            // Only allow Horde agents to access this
            "acl": {
                "entries": [
                    {
                        "claim": {
                            "type": "http://epicgames.com/ue/horde/role",
                            "value": "agent"
                        },
                        "actions": [
                            "ViewSecret"
                        ]
                    }
                ]
            }
        }
    ]
 ```

Secrets are queried from the external provider when requested by a user, and are not cached by Horde.

Secrets from external providers may require additional configuration.

### AWS

To use AWS Parameter Store in the [`server.json`](../Deployment/ServerSettings.md) file add the following to enable AWS
features for this provider.

   ```json
    "plugins": {
        "secrets": {
            "withAws": true
        }
    }
   ```

### HashiCorp Vault

To set the endpoint and credentials for HashiCorp Vault use the `providerConfig` property instead of the `provider` property.
For example to fetch three secrets from HashiCorp Vault using different authentication methods:

   * Use an AWS ARN to pass to AWS Security Token Service AssumeRole
   * Use AWS default credential search
   * A pre-shared token provided by the Vault server

   ```json
    "secrets": [
        {
            "id": "horde-secrets",

            // Some values read from HashiCorp Vault
            "sources": [
                {
                    "providerConfig": "hcp-vault-aws-assume-role",
                    // The Vault API path to the secret
                    "path": "/v1/secret/data/first-secret",
                    // Format must be json for Vault because a path in Vault returns a JSON document
                    "format": "json"
                },

                {
                    "providerConfig": "hcp-vault-aws",
                    "format": "json",
                    "path": "/v1/secret/data/second-secret",
                },

                {
                    "providerConfig": "hcp-vault-pre-shared-key",
                    "path": "/v1/secret/data/third-secret",
                    "format": "json"
                }
            ],

            // Only allow Horde agents to access this
            "acl": {
                "entries": [
                    {
                        "claim": {
                            "type": "http://epicgames.com/ue/horde/role",
                            "value": "agent"
                        },
                        "actions": [
                            "ViewSecret"
                        ]
                    }
                ]
            }
        }
    ],
    "plugins": {
        "secrets": {
            "providerConfigs": [
                {
                    "name": "hcp-vault-aws-assume-role",
                    "provider": "HcpVault",
                    "hcpVault": {
                        "credentials": "awsauth",
                        "endpoint": "https://vault.example.com",
                        "awsIamServerId": "vault.example.com",
                        "awsArnRole": "arn:aws:iam::1234567890:role/vault-example-role",
                        "role": "vault-role-name"
                    }
                },
                {
                    "name": "hcp-vault-aws",
                    "provider": "HcpVault",
                    "hcpVault": {
                        "credentials": "awsauth",
                        "endpoint": "https://vault.example.com",
                        "awsIamServerId": "vault.example.com",
                        "role": "vault-role-name"
                    }
                },
                {
                    "name": "hcp-vault-pre-shared-key",
                    "provider": "HcpVault",
                    "hcpVault": {
                        "credentials": "presharedkey",
                        "endpoint": "https://vault.example.com",
                        "preSharedKey": "the-key-provided-by-vault"
                    }
                }
            ]
        }
    }
 ```

### Azure

> The Azure Key Vault Secrets Provider is provided as experimental and an example of how to implement a secret provider from another cloud provider.

To set the URI of the Azure Key Vault use the `providerConfig` property instead of the `provider` property.
For authenication to Azure the implementation uses [DefaultAzureCredential](https://learn.microsoft.com/en-us/dotnet/api/azure.identity.defaultazurecredential?view=azure-dotnet).

   ```json
    "secrets": [
        {
            "id": "secret-from-key-vault",
 
            "sources": [
                {
                    "providerConfig": "azure-key-vault",
                    "path": "name-of-the-secret"
                }
            ]
        }
    ],
    "plugins": {
        "secrets": {
            "providerConfigs": [
                {
                    "name": "azure-key-vault",
                    "provider": "AzureKeyVault",
                    "azureKeyVault": {
                        "vaultUri": "https://vault-name.vault.azure.net/"
                    }
                }
            ]
        }
    }
   ```

## Using secrets

The most common use case for secrets is during build automation pipelines. In this scenario, the Horde Server
URL and credentials are taken from environment variables injected automatically by the Horde Agent, and allow
the pipeline to request secrets on behalf of the user starting the job with little additional configuration.

There are three common ways to use secrets:

### 1. Using the **Horde-GetSecrets** BuildGraph task

This task takes a file as a parameter, and will read it in, expand any secrets in the form {{ secret-name.property }}
with their values from Horde, and write it back out again. Rather than updating an existing file, you can also put the
template in a BuildGraph property and expand that instead, as follows:

   ```xml
   <Property Name="AwsEnvironmentText" Multiline="true">
      AWS_REGION={{horde-secrets.aws-region}}
      AWS_ACCESS_KEY_ID={{horde-secrets.aws-access-key-id}}
      AWS_SECRET_ACCESS_KEY={{horde-secrets.aws-secret-access-key}}
   </Property>
   <Horde-GetSecrets File="credentials.txt" Text="$(AwsEnvironmentText)"/>
   ```

### 2. Using the **Horde-SetSecretEnvVar** BuildGraph task

This task sets an environment variable to the value of a secret at runtime. Environment variables are inherited 
by child processes but not set at the system level, so the environment variable will contain that secret until
the end of the current step.

   ```xml
   <Horde-SetSecretEnvVar Name="AWS_SECRET_ACCESS_KEY" Secret="horde-secrets.aws-secret-access-key"/>
   ```

### 3. Using the Horde API

The HTTP secrets endpoint is listed in Horde's API documentation, and `AutomationTool` includes the following
utility methods for querying them at runtime:

   ```csharp
   IReadOnlyDictionary<string, string> secret = await CommandUtils.GetHordeSecretAsync(new SecretId("my-secret-name"));
   ```

   ```csharp
   string propertyValue = await CommandUtils.GetHordeSecretAsync(new SecretId("my-secret-name"), "propertyName")
   ```

## Implement a custom external provider

To implement a custom external provider instead of using AWS or HCP Vault. Create a file in the folder `Engine\Source\Programs\Horde\Plugins\Secrets\HordeServer.Secrets\Secrets\Providers` with a suitable name for the provider e.g. `MySecretProvider.cs` that provides an implementation of `HordeServer.Secrets.ISecretProvider`.

   ```csharp
    namespace HordeServer.Secrets.Providers
    {
        public class MySecretProvider : ISecretProvider
        {
            // The name must be unique across all providers
            public string Name => "MySecretStore";

            // If access to the provider requires a HTTP client
            private readonly IHttpClientFactory _httpClientFactory;

            private readonly ILogger _logger;

            public MySecretProvider(IHttpClientFactory httpClientFactory, ILogger<HcpVaultSecretProvider> logger)
            {
                 _httpClientFactory = httpClientFactory;
                 _logger = logger;
            }

            private HttpClient GetHttpClient()
            {
                return _httpClientFactory.CreateClient("HordeSecretMySecretProvider");
            }

            public async Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
            {
                string secretValue;
                // Implement how to get the secret for the give 'path' from your provider
                return secretValue;
            }
        }
    }
   ```

Add the provider to the service collection in the `ConfigureServices` method found in `Engine\Source\Programs\Horde\Plugins\Secrets\HordeServer.Secrets\SecretsPlugin.cs`.

   ```csharp
    public void ConfigureServices(IServiceCollection services)
    {
        // ...

        services.AddSingleton<ISecretProvider, MySecretProvider>();
    }
   ```

To test on a local workstation unit tests can be added to `Engine\Source\Programs\Horde\Plugins\Secrets\HordeServer.Secrets.Tests\Secrets\SecretProviderTests.cs`.

Edit the `globals.json` file to add a secret to use the new provider as shown in the earlier examples. In the examples where there is `"provider": "AwsParameterStore"` use `"provider": "MySecretStore"` instead.

Rebuild and deploy the Horde server. Once deployed can use [Swagger](../Internals/RestApi.md) and the endpoint `/api/v1/secrets/{secretId}` to get a secret declared in `globals.json`.

### Add additional configuration for the external provider

If the external provider requires additional configuration for example different secrets are stored at different domains or an access token is required then a provider configuarion can be implemented.

Create a file in the folder `Engine\Source\Programs\Horde\Plugins\Secrets\HordeServer.Secrets\Secrets\Providers` with a suitable name for the provider configuration e.g. `MySecretProviderConfig.cs`.

   ```csharp
    namespace HordeServer.Secrets.Providers
    {
        public class MySecretProviderConfig
        {
            public string? DomainName { get; set; }
            public string? LoginToken { get; set; }
        }
    }
   ```

Extend the class `HordeServer.Secrets.SecretProviderConfig` in `Engine\Source\Programs\Horde\Plugins\Secrets\HordeServer.Secrets\Secrets\SecretProviderConfig.cs` to
have a `MySecretProviderConfig` member.

   ```csharp
    public class SecretProviderConfig
    {
        // ...

        public MySecretProviderConfig? MyConfig { get; set; }
    }
   ```

Edit the `GetSecretAsync` method created earlier to use this new configuration to get the secret. For example if the configuraion was to be used in a HTTP GET request it might look something like this

   ```csharp
    public async Task<string> GetSecretAsync(string path, SecretProviderConfig? config, CancellationToken cancellationToken)
    {
        if (config?.MyConfig == null)
        {
            throw new InvalidOperationException($"Unable to fetch secret {path} from My Secret Provider (No Config)");
        }
        if (config.MyConfig.DomainName == null)
        {
            throw new InvalidOperationException($"Unable to fetch secret {path} from My Secret Provider (No Domain Name)");
        }
        if (config.MyConfig.LoginToken == null)
        {
            throw new InvalidOperationException($"Unable to fetch secret {path} from My Secret Provider (No Login Token)");
        }

        Uri uri = new("https://" + config.MyConfig.DomainName + ".example.com", path);
        using HttpRequestMessage request = new(HttpMethod.Get, uri);
        HttpClient httpClient = GetHttpClient();
        httpClient.DefaultRequestHeaders.Add("X-Token", config.MyConfig.LoginToken);
        using HttpResponseMessage httpResponseMessage = await httpClient.SendAsync(request, cancellationToken);

        string secretValue;
        // Implement how to get the secret from the repsone message
        return secretValue;
    }
   ```

Edit the `globals.json` file and add the configuration options to `plugins.secrets.providerConfigs` with an object for each combination of configuration options.
For the secrets replace `"provider": "MySecretStore"` with the key `"providerConfig"` and its value the unique name of the object in `plugins.secrets.providerConfigs`. See the earlier examples of configuring HCP Vault for more details.
