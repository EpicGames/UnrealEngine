[Horde](../../README.md) > [Configuration](../Config.md) > Storage

# Storage

## Introduction

Many features use Horde's storage system within the server. In many respects, the storage layer functions as an
immutable content-addressable store (i.e. items are stored and retrieved via a hash of their content), though the on-disk
representation consists of data packed into larger bundles stored with unique identifiers in a more familiar
location-based storage backend.

An important feature of this system is the ability to reason about massive data structures; that is, individual nodes
in the storage system (some arbitrary block of data) are characterized by being immutable and having zero or more
outward references to other nodes. These tree-like data structures can be traversed one node at a time, with the
underlying storage system retrieving and storing individual nodes efficiently.

The entry point to any such data structure is a user-defined name that keeps a reference to the root of the
tree. Any nodes not directly or indirectly referenced by a ref are subject to garbage collection.

This design intends to support a massive range of payload sizes by splitting larger data blocks into smaller chunks
and has advantages in caching and incremental client data patching. With this, it becomes as reasonable to reference
a single 4kb payload as it is to point to a multi-gigabyte payload partitioned into 128kb chunks.

The [Internals > Storage Architecture](../Internals/StorageArchitecture.md) page
goes into technical details around implementation of the storage system, and
describes how to interface with it programmatically.

## Namespaces and Backends

Storage is partitioned into several non-overlapping namespaces, each of which can be configured to use a different
backend. Permissions to access content in storage is normally controlled at the namespace level, though some
higher-level systems (e.g. logs, artifacts) expose custom endpoints over underlying storage namespaces which provide
their own permissions model that does not require direct access to the backing namespace.

Namespaces and backends are configured through the `Storage` section of the [globals.json](Schema/Globals.md)
file.

### Namespaces

Namespaces configure permissions and garbage collection policies for a logical storage partition. They also specify a
[backend](#backends) to use for storing the actual data.

Multiple namespaces may use the same storage backend by setting the `Prefix` property to append a prefix to any stored
files.

Settings for namespaces are described in the [globals.json](Schema/Globals.md#namespaceconfig) schema.

### Backends

Horde supports several storage backends, as well as local on-disk and in-memory caches:

* Local disk
* Network shares
* AWS S3 Object Store
* Azure Blob Store

Settings for backends are described in the [globals.json](../Config/Schema/Globals.md#backendconfig) schema.

### Well-known Namespaces

Certain subsystems in Horde use well-known namespace names, which can be configured as you see fit. These are defined in
`StorageConfig.cs` and currently consist of the following:

| Name | Description |
| ---- | ----------- |
| `horde-artifacts` | Stores artifacts produced by job steps in the CI system |
| `horde-perforce` | Replicated Perforce data used by the CI system |
| `horde-logs` | Logs generated by steps in the CI system |
| `horde-tools` | Standalone tools that can be downloaded from the Horde dashboard |

## References

* [Glossary](../Glossary.md)
