[Horde](../../README.md) > [Configuration](../Config.md) > Symbols

# Symbols

Horde implements a Windows symbol server compatible with Visual Studio, WinDbg, and Microsoft's DIA library.

Symbols are exposed to the symbol store by uploading them as part of an
[artifact](Artifacts.md). As such, any build produced by Horde's build
automation system can reuse uploaded symbols without any additional
storage requirements, and the lifetime of the uploaded symbols is tied to
that of the artifact.

Internally, this mapping is implemented using _aliases_ in Horde's storage
system which map from symbol server hashes to files in uploaded artifacts.
See [Internals > Storage Architecture](../Internals/StorageArchitecture.md) for more information on aliasing.

## Uploading Symbols

To add the metadata used for indexing symbols as part of the artifact
upload process, set the `Symbols="True"` attribute on the `CreateArtifact`
task from BuildGraph.

## Adding Source Information

In order to allow source-level debugging, it can be useful to embed
information about the source files that are built in the PDB itself. When
configured in this way, Visual Studio supports downloading the matching
source file for a PDB automatically when stepping through a program or
debugging a crash dump.

The functionality is referred to as _source server_ support in Microsoft
literature. This terminology can be a little misleading; the source server
being referred to is that of an existing source control server, and not a
new server deployment. When embedding source information in a PDB file,
you effectively embed a small script that Visual Studio can execute to
fetch the matching source file if needed - and that script can execute
arbitrary commands on the host machine (for example, launching p4.exe with
particular arguments).

The `SrcSrv` BuildGraph task can be used to embed Perforce source server information into PDB files. The `Engine/Build/InstalledEngineBuild.xml` demonstrates adding source information into installed builds as follows:

  ```xml
  <Tag
    Files="Engine/Source/...;Engine/Plugins/..."
    Filter="*.c;*.h;*.cpp;*.hpp;*.inl"
    Except="Engine/Source/ThirdParty/..."
    With="#SourceFiles"/>

  <SrcSrv
    BinaryFiles="#UnrealEditor Win64"
    SourceFiles="#SourceFiles"
    Branch="$(Branch)"
    Change="$(Change)"/>
  ```

Source server support needs to be manually enabled in Visual Studio
through the `Tools > Options` window, then checking the
`Debugging > General > Enable Source Server Support` option.

## Configuring a Symbol Store

Configuration for the symbols plugin is through the `SymbolsConfig` type
in the [globals.json](Schema/Globals.md#symbolsconfig) file.

Each configured symbol store will index an entire storage namespace.
Permission to access a particular store will grant access to any indexed
data, regardless of the user's ability to access that storage namespace
directly.

  ```json
  {
    "stores": [
        {
            "id": "my-store-id",
            "namespaceId": "horde-artifacts",
            "public": true
        }
    ]
  }

  ```

You can configure Visual Studio to use a symbol store through the
`Tools > Options` dialog, selecting the `Debugging > Symbols` tab.

Add Horde as a symbol server using a HTTP URL with a base path containing the symbol store id:

  ```
  http://my-horde-server/api/v1/symbols/my-store-id
  ```

## Accessing a Symbol Store with Authentication

Setting the `"public": true` property on a symbol store makes it accessible to anyone via unauthenticated endpoints.

Microsoft's symbol server API does not directly support authentication, but **Unreal Toolbox** implements a proxy server that can provide an unauthenticated bridge to access the Horde server over an authenticated connection via a port bound to `localhost`.

To use this, install Unreal Toolbox from the Tools > Download page in the Horde dashboard, and enable the proxy server plugin from the settings menu. After enabling the plugin, you can configure Visual Studio to search for symbols on http://localhost:13344.
