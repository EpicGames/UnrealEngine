[Horde](../../README.md) > [Configuration](../Config.md) > Tools

# Tools

Unreal Engine includes a number of tools to streamline certain
workflows. Horde implements functionality for hosting tools for
distribution to your team, and allows versioning and incrementally
rolling out new builds to users.

## Unreal Toolbox

**Unreal Toolbox** is an application which can manage tools
downloaded from Horde. It provides a simple interface for
downloading, installing, configuring, and upgrading Unreal Engine
tools, as well as managing access tokens for connecting to the Horde
Server that can be used by other tools (such as UnrealBuildTool or
the Unreal Editor).

By default, a Windows installer for Unreal Toolbox is distributed
with the Horde Server, which can be downloaded from the Tools > Download
page on the dashboard.

## Adding tools

Tools are configured through the [Tools](Schema/Globals.md#toolsconfig)
list in the [global.json](Schema/Globals.md) file. Licensees are encouraged
to extend the list of tools hosted by Horde with their own tools, and to
use Unreal Toolbox for distributing them.

New tool deployments can be created through the `DeployTool` BuildGraph
task:

  ```xml
  <DeployTool Id="horde-agent" Version="1.0" Directory="$(RootDir)/ToolDirectory"/>
  ```

Placing a Toolbox.json file in the root directory of a tool deployment
can configure commands to run on install and uninstall of the tool by
Unreal Toolbox, as well as adding additional commands to be made available
through the Toolbox popup menu. See
`Engine/Source/Programs/UnrealToolbox/ToolConfig.cs` for the syntax of
this file.

## Staggered Deployments

Each tool can have a number of deployments, and the Tools > Dashboard page
on the Horde dashboard can be used to start, pause, and revert revisions.
The system is designed to support staggered deployments; clients
choose a pseudo-random 'phase' value from 0-1 indicating when during a
rollout they should receive the deployment, and the server returns the
appropriate build information based on the start time and configured duration.

## Bundled tools

Tools bundled with the Horde installer by default are configured
separately, via the
[BundledTools](../Deployment/ServerSettings.md#toolsserverconfig) section
of the server's config file. Data for these tools is installed alongside
the server.
