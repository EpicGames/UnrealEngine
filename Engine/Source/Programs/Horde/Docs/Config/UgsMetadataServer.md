[Horde](../../README.md) > [Configuration](../Config.md) > UnrealGameSync Metadata Server

# UnrealGameSync Metadata Server

UnrealGameSync (**UGS**) is a tool designed to simplify syncing from Perforce, supporting retrieval of pre-built editor
binaries for artists or correctly versioning the local build so engineers can modify content. It is a
convenient hub for surfacing build health, flagging issues, and scripting common workflow tasks outside Unreal
Editor.

For more information on UGS, see
[the UE docs site](https://docs.unrealengine.com/en-US/unreal-game-sync-ugs-for-unreal-engine/).

Horde includes an updated version of the legacy MetadataServer IIS web app that ships alongside UGS, integrating
seamlessly with Horde's CI functionality.

## Configuration

To configure UnrealGameSync to source data from Horde, add the following lines in the `UnrealGameSync.ini` config file:

    [Default]
    ApiUrl=https://{{ HORDE_SERVER_URL }}/ugs

This config file can be in a project-specific location (e.g. `{{ PROJECT_DIR }}/Build/UnrealGameSync.ini`) or in a
location that applies to all projects in a stream (e.g. `{{ ENGINE_DIR }}/Programs/UnrealGameSync/UnrealGameSync.ini`).
