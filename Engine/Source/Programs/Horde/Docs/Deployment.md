[Horde](../README.md) > Deployment

# Deployment

Horde is designed to be scalable and can be deployed in many different configurations depending on need. This section
describes the system architecture of Horde, the various components that make it up, and the external services that it
interfaces with.

## Components

Horde requires the following components:

* **Horde Dashboard**
  * A web frontend for Horde written in Typescript using React. The dashboard can be hosted by the regular Horde
    Server or a separate web server.
* [**Horde Server**](Deployment/Server.md)
  * The core server code, written in C# / ASP.NET. It can be deployed in isolation or as a horizontally scaled service
    with multiple instances behind a load balancer. Supported on Windows and Linux.
  * See: [Deployment > Server](Deployment/Server.md)
* [**Horde Agent**](Deployment/Agent.md)
  * Software that runs on worker machines and executes tasks assigned to it by the Horde server. Written in C#
    / NET. Supported on Windows, Mac, and Linux.
  * See: [Deployment > Agent](Deployment/Agent.md)
* **MongoDB** database (or compatible)
  * A document database used to track and persist Horde state, such as jobs, users, agents, and so on. Unless
    configured otherwise, Horde will automatically create a local MongoDB database on startup.
  * Standalone or hosted instances of MongoDB are available from [MongoDB, Inc](https://www.mongodb.com/). Compatible
    hosted products are also available from Amazon ([DocumentDB](https://aws.amazon.com/documentdb/)) and Microsoft
    ([Azure CosmosDB](https://azure.microsoft.com/en-us/products/cosmos-db/)).
* **Redis** database (or compatible)
  * An in-memory database used to cache frequently accessed states between server instances, and provide pub/sub
    services for intra-server communication. Unless configured otherwise, Horde will automatically start a local
    Redis instance on startup.
  * Standalone (or hosted) instances of Redis are available from [Redis Labs](https://redis.io/). Compatible hosted
    products are also available from Amazon ([ElastiCache](https://aws.amazon.com/elasticache/redis/)) and
    Microsoft ([Azure Cache for Redis](https://azure.microsoft.com/en-us/products/cache/)).
* **Storage** 
  * Horde can utilize a local disk, network share, or cloud-based object store (eg.
    [AWS S3](https://aws.amazon.com/s3/), [Azure Blob Store](https://azure.microsoft.com/en-us/products/storage/blobs/))
    for storing intermediate and output artifacts, log files, and cache data. You can implement other backends through the 
    `IStorageBackend` interface. By default, Horde stores data locally on the server machine.

## Integrations

* **Slack**
  * Horde supports sending notifications to [Slack](https://www.slack.com/) out of the box, though other backends can
    be supported through the `INotificationSink` interface.
  * See: [Deployment > Integrations > Slack](Deployment/Integrations/Slack.md)
* **Perforce**
  * Horde supports Perforce for revision control in a CI setting and supports reading configuration data
    directly from it.
  * See: [Deployment > Integrations > Perforce](Deployment/Integrations/Perforce.md)
* **Jira**
  * Horde has a system for tracking and triaging build health issues but can interface with an external issue
    service if desired. Horde ships with support for [Jira](https://www.atlassian.com/software/jira), though you can implement other backends through the the `IExternalIssueService` interface.

Support for services mentioned above is the result of what Epic uses interally. Support may change in
the future and should not be taken as an endorsement or non-endorsement of particular products.

## Epic's Horde Deployment

Epic has one large deployment of Horde hosted on AWS and is described at a high level below.
Whether this setup should be mimicked depends on the size of your deployment.
We recommend starting with small instance types and adjust as needed.
Separation of servers into runmodes is also an advanced use-case and is likely not needed from the start.

### Server
* Application Load Balancer for TLS termination, certificate handling and routing
  * gRPC using `Http2Port` for unencrypted internal communication between ALB and ECS
* Amazon ECS with Fargate for hosting the C# processes (called _tasks_ in ECS)
  * 6 tasks for Horde Server configured with the `Server` [runmode](Deployment/ServerSettings.md#runmode-enum)
    * 8 vCPU, 16 GB RAM each
    * These handle short-lived, front-facing requests from browsers, agents and REST API
  * 2 tasks for Horde Server configured with the `Worker` [runmode](Deployment/ServerSettings.md#runmode-enum)
    * 2 vCPU, 8 GB RAM each
    * These handle heavier scheduled tasks, such as replicating Perforce metadata, reading and updating
      config state, and starting scheduled jobs.
    * Separating these has the benefit of background tasks never affecting request serving
* Amazon DocumentDB (compatible with MongoDB)
* Amazon ElastiCache (compatible with Redis)

### Agents

* Several hundred EC2 instances running AMIs containing the Horde agent and supporting toolchains
* Agents for remote execution (Unreal Build Accelerator) run exclusively as EC2 spot instances using Linux and Wine
* Around 100 machines hosted on-premise to provide access to mobile devices, consoles, and unhosted platforms
  (eg. Mac).
