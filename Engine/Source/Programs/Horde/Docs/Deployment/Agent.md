[Horde](../../README.md) > [Deployment](../Deployment.md) > Agents

# Agents

## Installation

### MSI Installer (Windows)

Installing the agent in this manner also installs a background application that shows the agent status in the Windows
notification area. You can also configure the agent to only run when the machine is idle.

> **NOTE:** Agents installed in this manner must be upgraded manually for new Horde versions.

### Download from Server

The Horde server can be used as a download source for the Horde agent. Distributing the agent in this way allows the
agent to upgrade automatically to new versions added to the server.

To install from a browser, go to the Horde dashboard, navigate to the `Server > Agents` menu item, and click
on the `Download Agent` link.

Alternatively, you can download the agent via the command line using the following commands. The `AUTH-TOKEN`
parameter referenced here can be obtained by having an admin user log into the `http://[HORDE-SERVER-URL]/account`
page and clicking on the **Get agent software download token** link.

#### Windows (PowerShell)

```ps1
    Invoke-WebRequest -Uri https://[HORDE-SERVER-URL]/api/v1/agentsoftware/default/zip -OutFile C:\Horde\HordeAgent.zip -Headers @{ 'Authorization' = 'Bearer [AUTH-TOKEN]' }
    Expand-Archive -LiteralPath C:\Horde\HordeAgent.zip -DestinationPath C:\Horde -Force
```

The -Headers parameter and value are not required when using an unauthenticated server.

#### Mac & Linux

```sh
    curl https://[HORDE-SERVER-URL]/api/v1/agentsoftware/default/zip --output ~/Horde/HordeAgent.zip -H "Authorization: Bearer [AUTH-TOKEN]"
    unzip -o ~/Horde/HordeAgent.zip -d ~/Horde/
```

The -H parameter and value are not required when using an unauthenticated server.

## Setup

### General

Agent settings are configured through the [`Agent.json`](AgentSettings.md) file. On Windows, this file is stored
at `C:\ProgramData\Epic\Horde\Agent\Agent.json`. On other platforms, it is stored in the `Data` folder under the
application directory by default. Settings in this file are applied on top of the `appsettings.json` file distributed
alongside the agent executable.

All Horde-specific settings are stored under the `horde` top-level key, with middleware and standard .NET settings
under other root keys.

### Server Profiles

The agent's [`Agent.json`](AgentSettings.md) file can contain settings for connecting to multiple Horde servers
through the `ServerProfiles` property. Setting up multiple profiles can be useful when running Horde in multiple
environments (eg. dev vs production), and each [server profile](AgentSettings.md#serverprofile) contains a name,
server URL and authentication token.

Server profiles are referenced by name. The default profile is configured through the `Server` property, or via the
`-Server=..` command line argument when launching the agent.

When scripting agent deployment, you can either modify the build hosted by the server to include the desired
configuration by default, or use the agent's `SetServer` command to modify the configuration file after downloading it.
This command can be invoked as:

```bat
    dotnet HordeAgent.dll SetServer -Name=.. -Url=.. -Token=...
```

Adding the `-Default` argument will configure this server to be used by default. Run with the `-Help` argument for a
full list of available options.

### Registration

Navigating to the `http://[HORDE-SERVER-URL]/account` page with an admin user logged in will include a **Get agent
registration token** link. This token can be embedded into the default agent config file or passed to the `SetServer`
command (see above).

The first time an agent connects to the server, it will generate a unique connection token for itself.

On Windows, connection tokens are stored in:

	C:\Users\[User]\AppData\Local\Epic Games\Horde\Agent\servers.json
    (if running under a normal user account)

    C:\Windows\system32\config\systemprofile\AppData\Local\Epic Games\Horde\Agent\servers.json
	(if running as a service)

On Mac/Linux, connection tokens are stored in:

    ~/.local/share/Horde.Agent/servers.json

### Running as a service

#### Windows

Running the MSI installer will configure the Horde Agent to run as a background service by default. When downloading
the agent directly from the server and configuring it manually, a service can be registered by running the following
command:

```bat
    dotnet HordeAgent.dll service install [-UserName=..] [-Password=..]
```

Where `-UserName` and `-Password` specify credentials for the account to run the service under.

The service may be uninstalled using the following command:

```bat
    dotnet HordeAgent.dll service uninstall
```

#### Mac

Create a `/Library/LaunchAgents/epic.hordeagent.plist` file describing the daemon configuration (substituting the
`{{ HORDE_SERVICE_ACCOUNT }}` variables as appropriate).

```xml
    <?xml version="1.0" encoding="UTF-8"?>
    <!DOCTYPE plist PUBLIC "-//Apple//DTD PLIST 1.0//EN" "http://www.apple.com/DTDs/PropertyList-1.0.dtd">
    <plist version="1.0">
    <dict>
        <key>RunAtLoad</key>
        <true/>

        <key>ProgramArguments</key>
        <array>
            <string>/usr/local/bin/dotnet</string>
            <string>HordeAgent.dll</string>
            <string>service</string>
            <string>run</string>
            <string>-server=Prod</string>
            <string>-workingdir=/Users/{{ HORDE_SERVICE_ACCOUNT }}/Build</string>
        </array>

        <key>EnvironmentVariables</key>
        <dict>
            <key>PATH</key>
            <string>/usr/bin:/bin:/usr/sbin:/sbin:/usr/local/bin</string>
        </dict>

        <key>GroupName</key>
        <string>admin</string>

        <key>WorkingDirectory</key>
        <string>/Users/{{ HORDE_SERVICE_ACCOUNT }}/Horde</string>

        <key>UserName</key>
        <string>{{ HORDE_SERVICE_ACCOUNT }}</string>

        <key>KeepAlive</key>
        <true/>

        <key>Label</key>
        <string>epic.hordeagent</string>

        <key>StandardErrorPath</key>
        <string>/Users/{{ HORDE_SERVICE_ACCOUNT }}/Library/Logs/hordeagent_error.log</string>

        <key>ExitTimeOut</key>
        <integer>10</integer>
    </dict>
    </plist>
```

Adjust `/etc/newsyslog.conf` Log file out size limit (optional):

```sh
    append '/Library/Logs/hordeagent_error.log 700 2 1000 * J'
```

Set any Horde agent environment variables you want to be defined outside of the plist (optional):

```sh
    launchctl setenv Horde:WorkingDirectory {horde_working_directory}
```

Launch the daemon:

```sh
    launchctl load -w /Library/LaunchAgents/epic.hordeagent.plist
```

#### Linux

Create a user to run the agent service. Unreal Editor cannot be run as root on Linux, so the horde-agent service
needs to run as a non-root user. The working directory for the agent needs to be recursively owned by that user. The
user must have `sudo` access to restart/shutdown/autoscale Horde agents.

Create a service descriptor file in `/etc/systemd/system/horde-agent.service` (substitute the `{{ HORDE_PATH }}`,
`{{ HORDE_WORKING_DIRECTORY }}` and `{{ HORDE_SERVICE_ACCOUNT }}` variables as appropriate):

```ini
    [Unit]
    Description=Horde Agent

    [Service]
    ExecStart=dotnet {{ HORDE_PATH }} -WorkingDir={{ HORDE_WORKING_DIRECTORY }}
    WorkingDirectory={{ HORDE_WORKING_DIRECTORY }}

    Restart=always
    RestartSec=5
    SyslogIdentifier=horde-agent

    StandardOutput=append:{{ HORDE_WORKING_DIRECTORY }}/log.txt
    StandardError=append:{{ HORDE_WORKING_DIRECTORY }}/err-log.txt

    User={{ HORDE_SERVICE_ACCOUNT }}

    [Install]
    WantedBy=multi-user.target
```

Launch the daemon:

```sh
    systemctl daemon-reload
```

### Working Directory

The default location for data used by the agent (Perforce workspaces, caches, scratch space) is
`C:\ProgramData\HordeAgent` on Windows and the application directory on Mac/Linux.

This path can be overridden using the `WorkingDir` property in the agent's [`Agent.json`](AgentSettings.md) file.

ITo prevent agents from being affected by runaway jobs filling up the disk with data, 
having the agent store data on a drive other than the system disk is a good idea.  
Setting the `%TEMP%` and `%TMP%` environment variables to this drive on Windows is also recommended.

### Mounting Network Shares

The agent can be configured to mount certain network shares at startup before taking on any work using the `Shares`
property in the agent's [`Agent.json`](AgentSettings.md) file.

### GPU Access

Certain jobs running in Horde, particularly tests, require access to the GPU. However, running the
agent as a Windows service prevents accessing the GPU. To work around this issue, the agent must run
as a normal Windows user in an interactive session. This can be achieved in different ways,
such as running the agent from a PowerShell script on the desktop or using an external tool such as [NSSM](https://nssm.cc/).
It's important to ensure the agent process is restarted in case of an unexpected exit.

Since most agents won't require GPU access, it's recommended to create a separate pool
specifically for interactive agents that need this support.

### Linux and Wine

Wine is a compatibility layer that enables Windows applications to run on Unix-like systems,
allowing agents on Linux to execute remote compute tasks under Wine.
Running workloads on Linux instead of Windows offers several advantages, such as faster OS boot times,
improved disk I/O performance, and the elimination of license costs.

To enable Wine, the path to `wine64` must be set in the [wineExecutablePath](../Deployment/AgentSettings.md) configuration option,
as described in the agent settings documentation.
A wrapper script in Bash can be used to modify any parameters sent to Wine, in this case configuring necessary file paths.
When running a build, the `-UBAHordeAllowWine` flag for UnrealBuildTool must be set to true, which also is the default value.
Once these prerequisites are met, agents will attempt to utilize Wine when executing compute tasks,
leveraging the benefits of running on Linux while maintaining compatibility with Windows applications, such as `cl.exe` and `link.exe`.

```bash
#!/bin/bash
# Wrapper script for Wine providing a hook point to modify incoming command-line arguments from the Windows application to run
# Right now, this script is mostly called by remote execution tasks, which run UnrealBuildAccelerator (UBA)
export WINEDEBUG=-all
export WINEARCH=win64
export WINEPREFIX=/opt/horde/wine-data

# Overwrite UE_HORDE_SHARED_DIR to point to C:\ which will exist inside Wine.
# This in turn is mounted under WINEPREFIX specified above
export UE_HORDE_SHARED_DIR="C:\\Uba"

if [ -n "$${UE_HORDE_TERMINATION_SIGNAL_FILE}" ]; then
  # Rewrite Linux path to be a Windows path under Z:\ which maps to / in Wine (replacing slashes with backslashes)
  export UE_HORDE_TERMINATION_SIGNAL_FILE="Z:$${UE_HORDE_TERMINATION_SIGNAL_FILE//\//\\}"
fi

/usr/local/bin/wine64 "$@"
```
