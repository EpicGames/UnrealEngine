[Horde](../../README.md) > [Deployment](../Deployment.md) > [Agent](Agent.md) > Agent.json (Agent)

# Agent.json (Agent)

All Horde-specific settings are stored in a root object called `Horde`. Other .NET functionality may be configured using properties in the root of this file.

Name | Description
---- | -----------
`serverProfiles` | `string` `->` [ServerProfile](#serverprofile)<br>Known servers to connect to
`server` | `string`<br>The default server, unless overridden from the command line
`name` | `string`<br>Name of agent to report as when connecting to server. By default, the computer's hostname will be used.
`mode` | [AgentMode](#agentmode-enum)<br>Mode of operation for the agent - For trusted agents in controlled environments (e.g., build farms). These agents handle all lease types and run exclusively Horde workloads.<br>- For low-trust workstations, uses interactive authentication (human logs in). These agents yield to non-Horde workloads and only support compute leases for remote execution.
`installed` | `boolean`<br>Whether the server is running in 'installed' mode. In this mode, on Windows, the default data directory will use the common application data folder (C:\ProgramData\Epic\Horde), and configuration data will be read from here and the registry. This setting is overridden to false for local builds from appsettings.Local.json.
`ephemeral` | `boolean`<br>Whether agent should register as being ephemeral. Doing so will not persist any long-lived data on the server and once disconnected it's assumed to have been deleted permanently. Ideal for short-lived agents, such as spot instances on AWS EC2.
`workingDir` | [DirectoryReference](#directoryreference)<br>Working directory for leases and jobs (i.e where files from Perforce will be checked out)
`logsDir` | [DirectoryReference](#directoryreference)<br>Directory where agent and lease logs are written
`shareMountingEnabled` | `boolean`<br>Whether to mount the specified list of network shares
`shares` | [MountNetworkShare](#mountnetworkshare)`[]`<br>List of network shares to mount
`wineExecutablePath` | `string`<br>Path to Wine executable. If null, execution under Wine is disabled
`containerEngineExecutablePath` | `string`<br>Path to container engine executable, such as /usr/bin/podman. If null, execution of compute workloads inside a container is disabled
`writeStepOutputToLogger` | `boolean`<br>Whether to write step output to the logging device
`enableAwsEc2Support` | `boolean`<br>Queries information about the current agent through the AWS EC2 interface
`useLocalStorageClient` | `boolean`<br>Option to use a local storage client rather than connecting through the server. Primarily for convenience when debugging / iterating locally.
`computeIp` | `string`<br>Incoming IP for listening for compute work. If not set, it will be automatically resolved.
`computePort` | `integer`<br>Incoming port for listening for compute work. Needs to be tied with a lease. Set port to 0 to disable incoming compute requests.
`openTelemetry` | [OpenTelemetrySettings](#opentelemetrysettings)<br>Options for OpenTelemetry
`enableTelemetry` | `boolean`<br>Whether to send telemetry back to Horde server
`telemetryReportInterval` | `integer`<br>How often to report telemetry events to server in milliseconds
`bundleCacheSize` | `integer`<br>Maximum size of the bundle cache, in megabytes.
`cpuCount` | `integer`<br>Maximum number of logical CPU cores workloads should use Currently this is only provided as a hint and requires leases to respect this value as it's set via an env variable (UE_HORDE_CPU_COUNT).
`cpuMultiplier` | `number`<br>CPU core multiplier applied to CPU core count setting For example, 32 CPU cores and a multiplier of 0.5 results in max 16 CPU usage.
`ramGb` | `integer`<br>Maximum available RAM in gigabytes. Defaults to available RAM.
`properties` | `string` `->` `string`<br>Key/value properties in addition to those set internally by the agent
`adminEndpoints` | `string[]`<br>Listen addresses for the built-in HTTP admin/management server. Disabled when empty. If activated, it's recommended to bind only to localhost for security reasons. Example: localhost:7008 to listen on localhost, port 7008
`healthCheckEndpoints` | `string[]`<br>Listen addresses for the built-in HTTP health check server. Disabled when empty. If activated, it's recommended to bind only to localhost for security reasons. Example: *:7009 to listen on all interfaces/IPs, port 7009 If all interfaces are bound with *, make sure to run process as administrator.
`tempStorageMaxBatchSize` | `integer`<br>Maximum number of temp storage exports to write in a single request
`logOptionalInputFailureAsWarning` | `boolean`<br>Whether to log optional input retrieval failures as warnings instead of information messages. Defaults to false (information level).

## ServerProfile

Information about a server to use

Name | Description
---- | -----------
`name` | `string`<br>Name of this server profile
`environment` | `string`<br>Name of the environment (currently just used for tracing)
`url` | `string`<br>Url of the server
`token` | `string`<br>Bearer token to use to initiate the connection
`useInteractiveAuth` | `boolean`<br>Whether to authenticate interactively in a desktop environment (for example, when agent is running on a user's workstation)
`thumbprint` | `string`<br>Thumbprint of a certificate to trust. Allows using self-signed certs for the server.
`thumbprints` | `string[]`<br>Thumbprints of certificates to trust. Allows using self-signed certs for the server.

## AgentMode (Enum)

Defines the operation mode of the agent Duplicated to prevent depending on Protobuf structures (weak name reference when deserializing JSON)

Name | Description
---- | -----------
`Dedicated` | 
`Workstation` | 

## DirectoryReference

Representation of an absolute directory path. Allows fast hashing and comparisons.

Name | Description
---- | -----------
`parentDirectory` | [DirectoryReference](#directoryreference)<br>Gets the directory containing this object
`fullName` | `string`<br>The path to this object. Stored as an absolute path, with O/S preferred separator characters, and no trailing slash for directories.

## MountNetworkShare

Describes a network share to mount

Name | Description
---- | -----------
`mountPoint` | `string`<br>Where the share should be mounted on the local machine. Must be a drive letter for Windows.
`remotePath` | `string`<br>Path to the remote resource

## OpenTelemetrySettings

OpenTelemetry configuration for collection and sending of traces and metrics.

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether OpenTelemetry exporting is enabled
`serviceName` | `string`<br>Service name
`serviceNamespace` | `string`<br>Service namespace
`serviceVersion` | `string`<br>Service version
`enableDatadogCompatibility` | `boolean`<br>Whether to enrich and format telemetry to fit presentation in Datadog
`attributes` | `string` `->` `string`<br>Extra attributes to set
`enableConsoleExporter` | `boolean`<br>Whether to enable the console exporter (for debugging purposes)
`protocolExporters` | `string` `->` [OpenTelemetryProtocolExporterSettings](#opentelemetryprotocolexportersettings)<br>Protocol exporters (key is a unique and arbitrary name)

## OpenTelemetryProtocolExporterSettings

Configuration for an OpenTelemetry exporter

Name | Description
---- | -----------
`endpoint` | `string`<br>Endpoint URL. Usually differs depending on protocol used.
`protocol` | `string`<br>Protocol for the exporter ('grpc' or 'httpprotobuf')
