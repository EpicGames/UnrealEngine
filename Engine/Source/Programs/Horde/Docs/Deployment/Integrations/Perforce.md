[Horde](../../../README.md) > [Deployment](../../Deployment.md) > Integrations > Perforce

# Perforce

## General

Horde uses Perforce primarily for CI functionality but also supports reading configuration data directly from a
Perforce server (see [Configuration > Orientation](../../Config/Orientation.md)). 

The Perforce connection to use for reading configuration files is configured alongside the server deployment via the
`perforce` property in the server [Server.json](../ServerSettings.md) file.

> **Note:** Support for other version control systems may be added in the future.

## Clusters

Epic's Perforce deployment is quite elaborate, and the desire to scale our CI build infrastructure across multiple
regions and data centers has resulted in bolstering Horde with a lot of custom functionality for interacting with
Perforce edge servers. Horde implements a load balancer for connecting build agents to Perforce servers, for example,
which uses health check data provided by server instances to roll over to new server instances when a server reports
a problem.

Horde supports the use of multiple independent Perforce installations and load balancing across multiple
mirrors within each installation. A collection of Perforce servers that mirror the same data is called a cluster.
If desired, you can configure each stream in the CI system to use a different cluster.

Clusters are configured through the `perforceClusters` property in the [globals.json](../../Config/Schema/Globals.md)
config file.

There are several configurables for each cluster:

* `Name`: Used to reference the cluster from a stream in the CI system.
* `Servers`: Each server supports several settings of its own:
  * `ServerAndPort`: The server part may be a DNS entry with multiple records, in which case it will be actively load balanced.
  * `ResolveDns`: If true, the given DNS name is resolved to find a concrete list of servers to be used. This allows
  IT/infrastructure teams to add and remove servers to a cluster without reconfiguring Horde.
  * `Properties`: Specifies properties that the agent must have to select this server.
  * `HealthCheck`: If true, the Horde server will periodically poll the server for its health on a well-known endpoint.
  See [#health-checks] for more information.
* `Credentials`: A list of usernames/passwords/tickets for different accounts on this server. CI jobs can request these
  credentials.
* `ServiceAccount`: Sets the account username that Horde should use for internal operations, such as querying
  commits from a stream, submitting on behalf of another user, and so on.
* `CanImpersonate`: Indicates whether Horde should attempt to impersonate other users when submitting changes after a
  successful preflight-and-submit operation. Typically requires an administrator account.

## Servers with SSL

When the Perforce server uses SSL and the `ssl:` prefix is specified,
it is essential to ensure that the server's fingerprint or certificate is trusted.
For more information, refer to the [p4 trust](https://www.perforce.com/manuals/cmdref/Content/CmdRef/p4_trust.html) documentation.
Depending on the configuration, it may be necessary to explicitly set the `P4TRUST` environment variable,
which should point to a valid `p4trust.txt` file. Again, see the P4 documentation for further details.

For example, on Windows, the Horde server runs as a service by default.
In this scenario, the default location of the trust file resolves to `%SystemRoot%\System32\config\systemprofile\p4trust.txt`.

## Health Checks

If enabled, health checks for Perforce servers are performed by performing an HTTP `GET` request to
`http://{{ PERFORCE_SERVER_URL }}:5000/healthcheck`. The endpoint is expected to return a JSON document with the
following structure:

    {
        "results": [
            {
                "checker": "edge_traffic_lights"
                "output": "green"
            }
        ]
    }

Where valid values for `output` are:

* `green`: Server is healthy.
* `yellow`: Server peformance is degraded.
* `red`: Server is draining existing connections and should not be used.

This functionality is implemented in `PerforceLoadBalancer.GetServerHealthAsync()`.

## P4CONFIG

Horde creates a file called `p4.ini` in a parent directory of workspaces created for CI use, containing the appropriate Perforce server, port, and username. 

Running the following command will allow Perforce to automatically detect the correct settings for the workspace in the current directory:

    p4 set P4CONFIG=p4.ini
