[Horde](../../../README.md) > [Deployment](../../Deployment.md) > Integrations > Slack

# Slack

Horde uses Slack to:
* broadcast notifications on configuration errors and CI failures.
* provide avatars for users logged in to Horde.

## Manifest

The Horde Slack app can be configured using the following manifest. Note the `{{ SERVER_URL }}` placeholder below.

    {
        "display_information": {
            "name": "Horde",
            "description": "Allow for interaction with the Horde build system.",
            "background_color": "#000000"
        },
        "features": {
            "bot_user": {
                "display_name": "Horde",
                "always_online": false
            }
        },
        "oauth_config": {
            "scopes": {
                "user": [
                    "admin.conversations:write"
                ],
                "bot": [
                    "chat:write",
                    "chat:write.public",
                    "reactions:read",
                    "reactions:write",
                    "users.profile:read",
                    "users:read",
                    "users:read.email",
                    "channels:manage"
                ]
            }
        },
        "settings": {
            "interactivity": {
                "is_enabled": true,
                "request_url": "{{ SERVER URL }}/api/v1/slack"
            },
            "org_deploy_enabled": true,
            "socket_mode_enabled": true,
            "token_rotation_enabled": false
        }
    }

You can find a suitable application icon in the source tree under `Horde/Horde.Server/Slack`, along with icons that you
can use for build health notification prompts.

Horde requires two tokens to be configured in the server's [Server.json](../ServerSettings.md) file to operate
fully:

* `SlackToken`: Bot token used to post messages to channels (has an `xoxb-` prefix). The Horde bot user must also
  be explicitly invited to any channels where it needs to post.
* `SlackSocketToken`: Token used to open a WebSocket connection to Slack and provide interactive functionality (has
  an `xapp-` prefix), responding to button presses, and so on.

### User mapping

Horde users are mapped to Slack users by correlating the email address in the user's
[OIDC profile](../Server.md#authentication) with their Slack user profile. Horde will use avatars configured through
Slack in the dashboard for any successfully mapped email address.
