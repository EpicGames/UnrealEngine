[Horde](../../README.md) > [Deployment](../Deployment.md) > Server

# Server

## Installation

### MSI Installer (Windows)

Windows builds of MongoDB and Redis are included in the installer and launched by Horde at startup (Horde will also
close them when it terminates). This installation is fine for small-scale installations and testing Horde, though
hosting databases separately would be preferred in production scenarios.

### Docker (Linux)

Images for hosting Horde through Docker are available through the EpicGames organization on
[GitHub](https://www.unrealengine.com/en-US/ue-on-github). Note that you must be signed into GitHub with an account
associated with an EpicGames account to follow these links.

* [Horde Server](https://github.com/orgs/EpicGames/packages/container/package/horde-server)

To download an image, first create a [GitHub personal access token (classic)](https://docs.github.com/en/packages/working-with-a-github-packages-registry/working-with-the-container-registry#authenticating-with-a-personal-access-token-classic) from the developer section
of your account settings page and use that to login with GitHub via `docker`.
Follow the linked GitHub documentation for more details.

Once authenticated, you can verify by downloading an image with:

```bash
docker pull ghcr.io/epicgames/horde-server:latest 
```

Note that in this form, an external MongoDB and Redis instance must be configured through a configuration file or
environment variable (see below).

Running multiple Horde servers behind a load balancer is possible but remember to set `ServerUrl` in settings.
That setting should reflect the URL your users and agents use to connect with.

Using Docker containers on Linux is [Epic's preferred way of running Horde](../Deployment.md).

### Docker Compose (Linux)

[Docker Compose](https://docs.docker.com/compose/) simplifies the setup of a Docker-based installation
by providing a preconfigured group of Docker containers, which includes instances of MongoDB and Redis.

Similar to the MSI installer, this method is suitable for testing Horde or deploying it in small-scale environments.
To access the prebuilt images, refer to the Docker section above.
The necessary Docker Compose configuration can be found within the `Engine/Source/Programs/Horde/HordeServer/docker-compose.yml` file.

From the same directory, start the containers with:
```bash
docker compose up
````
For additional guidance, see the comments within the YAML file.

### Homebrew (Mac)

We don't provide any prebuilt binaries for running the server on Mac, though it's relatively straightforward to
install all the prerequisites using [Homebrew](https://brew.sh/).

1. Install the .NET 10 SDK

    ```bash
    brew install dotnet-sdk
    ```

2. Install MongoDB

    ```bash
    brew tap mongodb/brew
    brew update
    brew install mongodb-community
    brew services start mongodb-community
    ```

3. Install Redis

    ```bash
    brew install redis
    brew services start redis
    ```

4. Launch Horde. The environment variables below use standard ASP.NET syntax; you can modify values in server.json
   instead if you prefer.

    ```bash
    export Horde__MongoConnectionString=mongodb://localhost:27017
    export Horde__HttpPort=37107
    export Horde__Http2Port=37109

    cd Engine/Source/Programs/Horde/Horde.Server
    dotnet run
    ```

### Building from Source

The source code for the Horde server is located under `Engine/Source/Programs/Horde/Horde.Server/...`.

You can build and run the server from Visual Studio using the solution at `Engine/Source/Programs/Horde/Horde.sln`,
or from the command line via the `dotnet build` or `dotnet publish` commands.

Docker images can be built through the BuildGraph script at `Engine/Source/Programs/Horde/BuildHorde.xml`, using
the Dockerfile in `Engine/Source/Programs/Horde.Server/Dockerfile`.

Using the BuildGraph script is recommended over
running the Dockerfile directly because it stages the relevant files to a temporary directory before running
`docker build`, which prevents the Docker daemon from copying the entire UE source tree to the containerized environment
before building.

The command line for building Docker images using BuildGraph is:

```cmd
RunUAT.bat BuildGraph -Script=Engine/Source/Programs/Horde/BuildHorde.xml -Target="Build HordeServer"
```

The Windows installer can be built from the same BuildGraph script with a similar command line:

```cmd
RunUAT.bat BuildGraph -Script=Engine/Source/Programs/Horde/BuildHorde.xml -Target="Build Horde Installer"
```

## Settings

### General

Server settings are configured through the [`Server.json`](ServerSettings.md) file. On Windows, this file is stored
at `C:\ProgramData\Epic\Horde\Server\Server.json`. On other platforms, it is stored in the `Data` folder under the
application directory by default. Settings in this file are applied on top of the `appsettings.json` file distributed
alongside the server executable.

All Horde-specific settings are stored under the `Horde` top-level key, with middleware and standard .NET settings
under other root keys.

As an ASP.NET application, Horde's application configuration supports the following features:

* Individual properties can be overridden through **environment variables** using standard ASP.NET syntax (see
  [MSDN](https://learn.microsoft.com/en-us/aspnet/core/fundamentals/configuration/?view=aspnetcore-7.0#naming-of-environment-variables)).
  For example, the database connection string can be passed in using the `Horde__MongoConnectionString` environment variable.
* The deployment environment can be configured using the ASPNETCORE_ENVIRONMENT environment variable. Standard values
  for Horde are `Production`, `Development`, and `Local`.
* A deployment-specific configuration file can be created called `appsettings.{Environment}.json` (e.g.
  `appsettings.Local.json`), which will be merged with other settings.

Note that the server configuration files (`Server.json`, `appsettings.json`, etc.) differ from the global
configuration file (`globals.json`). The server configuration file is deployed alongside the server. It contains
deployment/infrastructure settings, whereas the global configuration file can be stored in revision control and
updated dynamically during the server's lifetime. See [Config > Orientation](../Config/Orientation.md) for
more information.

### MongoDB

The MongoDB connection string can be specified via the `MongoConnectionString` property in the
[Server.json](ServerSettings.md) file or the `Horde__MongoConnectionString` environment variable. The
connection string should be in standard
[MongoDB syntax](https://www.mongodb.com/docs/manual/reference/connection-string/), e.g.:

```text
mongodb://username:password@host:27017?replicaSet=rs0&readPreference=primary
```

Horde implements many operations as compare-and-swap operations, so it is important that all reads are configured
to use the primary database instance using the `readPreference=primary` argument when using a replica set. Using a
secondary instance for reads can cause deadlocks because the server gets out-of-date documents in a read-modify-write
cycle.

The MongoDB connection can be configured to use a trusted set of certificates via the `MongoPublicCertificate` property.
For example, when running on AWS using DocumentDB, this property can be set to use Amazon's
[combined certificate bundle](https://docs.aws.amazon.com/AmazonRDS/latest/UserGuide/UsingWithRDS.SSL.html) by
placing the `global-bundle.pem` file into the server's application directory.

### Redis

The Redis server is configured through the RedisConnectionConfig property in the [Server.json](ServerSettings.md)
file or via the `Horde__MongoConnectionString` environment variable. This string is formatted as a plain server
and port, e.g.:

```text
127.0.0.1:6379
```

### Ports

By default, Horde is configured to serve data over unencrypted HTTP using port 5000. Agents communicate with the Horde
server using gRPC over unencrypted HTTP/2 on port 5002 by default. These settings are echoed to the console
during server startup.

A separate port is used for gRPC since Kestrel (the .NET web server) does not support unencrypted HTTP/2 traffic over
the same port as HTTP/1 traffic. This separate port for non-TLS HTTP/2 traffic can be useful when putting
Horde behind a reverse proxy. If an HTTPS port is configured, all traffic can use that port.

Settings for port usage are defined in [Server.json](ServerSettings.md):

* To disable serving data over HTTP, set the `HttpPort` property to zero.
* To configure the secondary HTTP/2 port used, set the `Http2Port` property (or set it to zero to disable it).

### HTTPS

To serve data over HTTPS, set the `HttpsPort` property in the [Server.json](ServerSettings.md) file.

Configure the certificate for [Kestrel](https://learn.microsoft.com/en-us/aspnet/core/fundamentals/servers/kestrel?view=aspnetcore-8.0)
(the NET Core web server) by setting the default certificate in the same file.

Cross-platform:

   ```json
    "Kestrel": {
        "Certificates": {
            "Default": {
                "Path": "C:\\cert\\test.pfx",
                "Password": "my-password"
            }
        }
    }
   ```

Windows (using the system certificate store):

   ```json
    "Kestrel": {
        "Certificates": {
            "Default":
            {
                "Subject": "my-domain.com",

                // Use the 'Personal' certificate store on the local machine
                "Store": "My",
                "Location": "LocalMachine"
            }
        }
    }
   ```

> **Note**: The `Kestrel` object must be added at the root scope of the file, not within the `Horde` object.

Other ways to configure certificates for Kestrel are listed on
[MSDN](https://learn.microsoft.com/en-us/aspnet/core/fundamentals/servers/kestrel/endpoints?view=aspnetcore-8.0#configure-https).

Both HTTP/1.1 and HTTP/2.0 traffic can be served over the `HttpsPort`. Unencrypted traffic can be disabled by
setting `HttpPort` and `Http2Port` to zero.

There are occasions where the server provides links back to itself (the OIDC discovery document used when using
Horde's internal account system, for example), and it's important that these URLs match the HTTPS certificate.
By default, this URL is derived from the server's reported DNS name, but this can be overridden through the
`ServerUrl` property.

To set up a self-signed certificate for testing see [Tutorials > Self Signed Certs](../Tutorials/SelfSignedCerts.md).

### Monitoring

Horde uses [Serilog](https://serilog.net/) for logging and is configured to generate plain text and JSON log files
to the application directory on Linux and the `C:\ProgramData\HordeServer` folder on Windows. Plain text output
is written to stdout by default, though Json output can be enabled using the `LogJsonToStdOut` property in
[Server.json](ServerSettings.md).

Profiling and telemetry data for the server is routed through [OpenTelemetry](https://opentelemetry.io/). Settings for
telemetry capture are [listed here](ServerSettings.md#opentelemetrysettings).

### RunModes

In order to separate lighter request traffic from heavier background operations, the Horde server can be configured to
run in different RunModes. You can configure these via the [RunMode](ServerSettings.md) setting.

### Authentication

Horde supports [OpenID Connect (OIDC)](https://openid.net/developers/how-connect-works/) for authentication using
an external identity provider. OIDC is a widely used auth standard, and Okta, AWS, Azure, Google, Facebook, and
many others implement identity providers compatible with it.

The [Getting Started > Authentication](../Tutorials/Authentication.md) page explains how to configure Horde's internal
account system and OIDC provider.

The following settings in [Server.json](ServerSettings.md) are required to configure an external OIDC provider:

* `AuthMethod`: Set this to `OpenIdConnect`.
* `OidcAuthority`: URL of the OIDC authority. You can check the URL specified here is correct by navigating to
  `{{Url}}/.well-known/openid-configuration` in a browser, which should return the OIDC discovery document.
* `OidcClientId`: Identifies the application (Horde) to the OIDC provider. This is generated by the OIDC provider.
* `OidcClientSecret`: A secret value provided by the OIDC provider to identify the application requesting authorization.

In addition, the following settings can be specified:

* `OidcRequestedScopes`: Specifies the scopes requested from the OIDC provider. Scopes determine the access that Horde
  requests from the OIDC provider and the claims that will be returned and available for checking against in Horde
  ACLs. The meaning of these values is specific to your OIDC provider configuration.
* `OidcClaimNameMapping`: Specifies a list of claims to check, in order of preference, when trying to show a user's
  real name.
* `OidcClaimEmailMapping`: Specifies a list of claims to check, in order of preference, when trying to show a user's
  email address.
* `OidcClaimHordePerforceUserMapping`: Specifies a list of claims to check, in order of preference, when trying to
  determine a user's Perforce username.

See [Config > Permissions](../Config/Permissions.md) for other authentication options.

### Reference

For a full list of valid properties in the server configuration file, see
[**Server.json (Server)**](ServerSettings.md).
