[Horde](../../README.md) > [Deployment](../Deployment.md) > [Server](Server.md) > Server.json

# Server.json

All Horde-specific settings are stored in a root object called `Horde`. Other .NET functionality may be configured using properties in the root of this file.

Name | Description
---- | -----------
`runModes` | [RunMode](#runmode-enum)`[]`<br>Modes that the server should run in. Runmodes can be used in a multi-server deployment to limit the operations that a particular instance will try to perform.
`dataDir` | `string`<br>Override the data directory used by Horde. Defaults to C:\ProgramData\HordeServer on Windows, {AppDir}/Data on other platforms.
`installed` | `boolean`<br>Whether the server is running in 'installed' mode. In this mode, on Windows, the default data directory will use the common application data folder (C:\ProgramData\Epic\Horde), and configuration data will be read from here and the registry. This setting is overridden to false for local builds from appsettings.Local.json.
`httpPort` | `integer`<br>Main port for serving HTTP.
`httpsPort` | `integer`<br>Port for serving HTTP with TLS enabled. Disabled by default.
`http2Port` | `integer`<br>Dedicated port for serving only HTTP/2.
`mongoConnectionString` | `string`<br>Connection string for the Mongo database
`databaseConnectionString` | `string`<br>MongoDB connection string
`mongoDatabaseName` | `string`<br>MongoDB database name
`databaseName` | `string`<br>
`mongoPublicCertificate` | `string`<br>Optional certificate to trust in order to access the database (eg. AWS public cert for TLS)
`databasePublicCert` | `string`<br>
`mongoReadOnlyMode` | `boolean`<br>Access the database in read-only mode (avoids creating indices or updating content) Useful for debugging a local instance of HordeServer against a production database.
`databaseReadOnlyMode` | `boolean`<br>
`mongoMigrationsEnabled` | `boolean`<br>Whether database schema migrations are enabled
`mongoMigrationsAutoUpgrade` | `boolean`<br>Whether database schema should automatically be applied Only recommended for dev or test environments
`shutdownMemoryThreshold` | `integer`<br>Shutdown the current server process if memory usage reaches this threshold (specified in MB)<br>Usually set to 80-90% of available memory to avoid CLR heap using all of it. If a memory leak was to occur, it's usually better to restart the process rather than to let the GC work harder and harder trying to recoup memory.<br>Should only be used when multiple server processes are running behind a load balancer and one can be safely restarted automatically by the underlying process handler (Docker, Kubernetes, AWS ECS, Supervisor etc). The shutdown behaves similar to receiving a SIGTERM and will wait for outstanding requests to finish.
`serverPrivateCert` | `string`<br>Optional PFX certificate to use for encrypting agent SSL traffic. This can be a self-signed certificate, as long as it's trusted by agents.
`authMethod` | [AuthMethod](#authmethod-enum)<br>Type of authentication (e.g anonymous, OIDC, built-in Horde accounts) If "Horde" auth mode is used, be sure to configure "ServerUrl" as well.
`oidcProfileName` | `string`<br>Optional profile name to report through the /api/v1/server/auth endpoint. Allows sharing auth tokens between providers configured through the same profile name in OidcToken.exe config files.
`oidcAuthority` | `string`<br>OpenID Connect (OIDC) authority URL (required when OIDC is enabled)
`oidcAudience` | `string`<br>Audience for validating externally issued tokens (required when OIDC is enabled)
`oidcClientId` | `string`<br>Client ID for the OIDC authority (required when OIDC is enabled)
`oidcClientSecret` | `string`<br>Client secret for authenticating with the OIDC provider. Note: If you need authentication support in Unreal Build Tool or Unreal Game Sync, configure your OIDC client as a public client (using PKCE flow without a client secret) instead of a confidential client. These tools utilize the EpicGames.OIDC library which only supports public clients with authorization code flow + PKCE.
`oidcSigninRedirect` | `string`<br>Optional redirect url provided to OIDC login
`oidcLocalRedirectUrls` | `string[]`<br>Optional redirect url provided to OIDC login for external tools (typically to a local server) Default value is the local web server started during signin by EpicGames.OIDC library
`oidcDebugMode` | `boolean`<br>Debug mode for OIDC which logs reasons for why JWT tokens fail to authenticate Also turns off HTTPS requirement for OIDC metadata fetching. NOT FOR PRODUCTION USE!
`oidcRequestedScopes` | `string[]`<br>OpenID Connect scopes to request when signing in
`oidcClaimNameMapping` | `string[]`<br>List of fields in /userinfo endpoint to try map to the standard name claim (see System.Security.Claims.ClaimTypes.Name)
`oidcClaimEmailMapping` | `string[]`<br>List of fields in /userinfo endpoint to try map to the standard email claim (see System.Security.Claims.ClaimTypes.Email)
`oidcClaimHordeUserMapping` | `string[]`<br>List of fields in /userinfo endpoint to try map to the Horde user claim (see HordeClaimTypes.User)
`oidcClaimHordePerforceUserMapping` | `string[]`<br>List of fields in /userinfo endpoint to try map to the Horde Perforce user claim (see HordeClaimTypes.PerforceUser)
`oidcApiRequestedScopes` | `string[]`<br>API scopes to request when acquiring OIDC access tokens
`oidcAddDefaultScopesAndMappings` | `boolean`<br>Add common scopes and mappings to above OIDC config fields Provided as a workaround since .NET config will only *merge* array entries when combining multiple config sources. Due to this unwanted behavior, having hard-coded defaults makes such fields unchangeable. See https://github.com/dotnet/runtime/issues/36569
`serverUrl` | `string`<br>Base URL this Horde server is accessible from For example https://horde.mystudio.com. If not set, a default is used based on current hostname. It's important this URL matches where users and agents access the server as it's used for signing auth tokens etc. Must be configured manually when running behind a reverse proxy or load balancer
`jwtIssuer` | `string`<br>Name of the issuer in bearer tokens from the server
`jwtExpiryTimeHours` | `integer`<br>Length of time before JWT tokens expire, in hours
`extendedJwtExpiryTimeHours` | `integer`<br>Length of time before extended JWT tokens expire, in hours, defaults to 1 week
`adminClaimType` | `string`<br>The claim type for administrators
`adminClaimValue` | `string`<br>Value of the claim type for administrators
`corsEnabled` | `boolean`<br>Whether to enable Cors, generally for development purposes
`corsOrigin` | `string`<br>Allowed Cors origin
`enableDebugEndpoints` | `boolean`<br>Whether to enable debug/administrative REST API endpoints
`enableNewAgentsByDefault` | `boolean`<br>Whether to automatically enable new agents by default. If false, new agents must manually be enabled before they can take on work.
`schedulePollingInterval` | `string`<br>Interval between rebuilding the schedule queue with a DB query.
`noResourceBackOffTime` | `string`<br>Interval between polling for new jobs
`initiateJobBackOffTime` | `string`<br>Interval between attempting to assign agents to take on jobs
`unknownErrorBackOffTime` | `string`<br>Interval between scheduling jobs when an unknown error occurs
`redisConnectionString` | `string`<br>Config for connecting to Redis server(s). Setting it to null will disable Redis use and connection See format at https://stackexchange.github.io/StackExchange.Redis/Configuration.html
`redisConnectionConfig` | `string`<br>
`redisReadOnlyMode` | `boolean`<br>Whether to disable writes to Redis.
`logServiceWriteCacheType` | `string`<br>Overridden settings for storage backends. Useful for running against a production server with custom backends.
`logJsonToStdOut` | `boolean`<br>Whether to log json to stdout
`logSessionRequests` | `boolean`<br>Whether to log requests to the UpdateSession and QueryServerState RPC endpoints
`scheduleTimeZone` | `string`<br>Timezone for evaluating schedules
`dashboardUrl` | `string`<br>The URl to use for generating links back to the dashboard.
`helpEmailAddress` | `string`<br>Help email address that users can contact with issues
`helpSlackChannel` | `string`<br>Help slack channel that users can use for issues
`globalThreadPoolMinSize` | `integer`<br>Set the minimum size of the global thread pool This value has been found in need of tweaking to avoid timeouts with the Redis client during bursts of traffic. Default is 16 for .NET Core CLR. The correct value is dependent on the traffic the Horde Server is receiving. For Epic's internal deployment, this is set to 40.
`configPath` | `string`<br>Path to the root config file. Relative to the server.json file by default.
`forceConfigUpdateOnStartup` | `boolean`<br>Forces configuration data to be read and updated as part of appplication startup, rather than on a schedule. Useful when running locally.
`openBrowser` | `boolean`<br>Whether to open a browser on startup
`featureFlags` | [FeatureFlagSettings](#featureflagsettings)<br>Experimental features to enable on the server.
`openTelemetry` | [OpenTelemetrySettings](#opentelemetrysettings)<br>Options for OpenTelemetry
`plugins` | [ServerPluginsConfig](#serverpluginsconfig)<br>Configuration for plugins

## RunMode (Enum)

Type of run mode this process should use. Each carry different types of workloads. More than one mode can be active. But not all modes are not guaranteed to be compatible with each other and will raise an error if combined in such a way.

Name | Description
---- | -----------
`None` | Default no-op value (ASP.NET config will default to this for enums that cannot be parsed)
`Server` | Handle and respond to incoming external requests, such as HTTP REST and gRPC calls. These requests are time-sensitive and short-lived, typically less than 5 secs. If processes handling requests are unavailable, it will be very visible for users.
`Worker` | Run non-request facing workloads. Such as background services, processing queues, running work based on timers etc. Short periods of downtime or high CPU usage due to bursts are fine for this mode. No user requests will be impacted directly. If auto-scaling is used, a much more aggressive policy can be applied (tighter process packing, higher avg CPU usage).

## AuthMethod (Enum)

Authentication method used for logging users in

Name | Description
---- | -----------
`Anonymous` | No authentication enabled. *Only* for demo and testing purposes.
`Okta` | OpenID Connect authentication, tailored for Okta
`OpenIdConnect` | Generic OpenID Connect authentication, recommended for most
`Horde` | Authenticate using username and password credentials stored in Horde OpenID Connect (OIDC) is first and foremost recommended. But if you have a small installation (less than ~10 users) or lacking an OIDC provider, this is an option.

## FeatureFlagSettings

Feature flags to aid rollout of new features.
Once a feature is running in its intended state and is stable, the flag should be removed. A name and date of when the flag was created is noted next to it to help encourage this behavior. Try having them be just a flag, a boolean.


## OpenTelemetrySettings

OpenTelemetry configuration for collection and sending of traces and metrics.

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether OpenTelemetry exporting is enabled
`serviceName` | `string`<br>Service name
`serviceNamespace` | `string`<br>Service namespace
`serviceVersion` | `string`<br>Service version
`enableDatadogCompatibility` | `boolean`<br>Whether to enrich and format telemetry to fit presentation in Datadog
`attributes` | `string` `->` `string`<br>Extra attributes to set
`enableConsoleExporter` | `boolean`<br>Whether to enable the console exporter (for debugging purposes)
`protocolExporters` | `string` `->` [OpenTelemetryProtocolExporterSettings](#opentelemetryprotocolexportersettings)<br>Protocol exporters (key is a unique and arbitrary name)

## OpenTelemetryProtocolExporterSettings

Configuration for an OpenTelemetry exporter

Name | Description
---- | -----------
`endpoint` | `string`<br>Endpoint URL. Usually differs depending on protocol used.
`protocol` | `string`<br>Protocol for the exporter ('grpc' or 'httpprotobuf')

## ServerPluginsConfig

Name | Description
---- | -----------
`analytics` | [AnalyticsServerConfig](#analyticsserverconfig)<br>Configuration for the analytics plugin
`build` | [BuildServerConfig](#buildserverconfig)<br>Configuration for the build plugin
`compute` | [ComputeServerConfig](#computeserverconfig)<br>Configuration for the compute plugin
`exampleplugin` | [ExamplePluginServerConfig](#examplepluginserverconfig)<br>Configuration for the exampleplugin plugin
`experimental` | [ExperimentalServerConfig](#experimentalserverconfig)<br>Configuration for the experimental plugin
`secrets` | [SecretsServerConfig](#secretsserverconfig)<br>Configuration for the secrets plugin
`storage` | [StorageServerConfig](#storageserverconfig)<br>Configuration for the storage plugin
`symbols` | [PluginServerConfig](#pluginserverconfig)<br>Configuration for the symbols plugin
`tools` | [ToolsServerConfig](#toolsserverconfig)<br>Configuration for the tools plugin

## AnalyticsServerConfig

Server configuration for the analytics system

Name | Description
---- | -----------
`sinks` | [TelemetrySinkConfig](#telemetrysinkconfig)<br>Settings for the various telemetry sinks
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## TelemetrySinkConfig

Telemetry sinks

Name | Description
---- | -----------
`epic` | [EpicTelemetryConfig](#epictelemetryconfig)<br>Settings for the Epic telemetry sink
`mongo` | [MongoTelemetryConfig](#mongotelemetryconfig)<br>Settings for the MongoDB telemetry sink

## EpicTelemetryConfig

Configuration for the telemetry sink

Name | Description
---- | -----------
`url` | `string`<br>Base URL for the telemetry server
`appId` | `string`<br>Application name to send in the event messages
`enabled` | `boolean`<br>Whether to enable this sink

## MongoTelemetryConfig

Configuration for the telemetry sink

Name | Description
---- | -----------
`retainDays` | `number`<br>Number of days worth of telmetry events to keep
`enabled` | `boolean`<br>Whether to enable this sink

## BuildServerConfig

Static configuration for the build plugin

Name | Description
---- | -----------
`perforce` | [PerforceConnectionSettings](#perforceconnectionsettings)`[]`<br>Perforce connections for use by the Horde server (not agents)
`useLocalPerforceEnv` | `boolean`<br>Whether to use the local Perforce environment
`perforceConnectionPoolSize` | `integer`<br>Number of pooled perforce connections to keep
`enableConformTasks` | `boolean`<br>Whether to enable the conform task source.
`p4SwarmUrl` | `string`<br>Url of P4 Swarm installation
`robomergeUrl` | `string`<br>Url of Robomerge installation
`commitsViewerUrl` | `string`<br>Url of Commits Viewer
`jiraUsername` | `string`<br>The Jira service account user name
`jiraApiToken` | `string`<br>The Jira service account API token
`jiraUrl` | `string`<br>The Uri for the Jira installation
`sharedDeviceCheckoutDays` | `integer`<br>The number of days shared device checkouts are held
`deviceProblemCooldownMinutes` | `integer`<br>The number of cooldown minutes for device problems
`deviceReportChannel` | `string`<br>Channel to send device reports to
`disableBisection` | `boolean`<br>Whether to disable bisection tasks
`disableSchedules` | `boolean`<br>Whether to run scheduled jobs.
`slackToken` | `string`<br>Bot token for interacting with Slack (xoxb-*)
`slackSocketToken` | `string`<br>Token for opening a socket to slack (xapp-*)
`slackAdminToken` | `string`<br>Admin user token for Slack (xoxp-*). This is only required when using the admin endpoints to invite users.
`slackUsers` | `string`<br>Filtered list of slack users to send notifications to. Should be Slack user ids, separated by commas.
`slackErrorPrefix` | `string`<br>Prefix to use when reporting errors
`slackWarningPrefix` | `string`<br>Prefix to use when reporting warnings
`configNotificationChannel` | `string`<br>Channel for sending messages related to config update failures
`updateStreamsNotificationChannel` | `string`<br>Channel to send stream notification update failures to
`jobNotificationChannel` | `string`<br>Slack channel to send job related notifications to. Multiple channels can be specified, separated by ;
`agentNotificationChannel` | `string`<br>Slack channel to send agent related notifications to.
`testDataRetainMonths` | `integer`<br>The number of months to retain test data
`testHealthHistoryEvaluationDays` | `integer`<br>The number of days to go back in historical data to evaluate test health
`testHealthReportRemindDays` | `integer`<br>The number of days after which a test health report remind for improvement
`blockCacheDir` | `string`<br>Directory to store the fine-grained block cache. This caches individual exports embedded in bundles.
`blockCacheSize` | `string`<br>Maximum size of the block cache. Accepts standard binary suffixes. Currently only allocates in multiples of 1024mb.
`blockCacheSizeBytes` | `integer`<br>Accessor for the block cache size in bytes
`commits` | [CommitSettings](#commitsettings)<br>Options for the commit service
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## PerforceConnectionSettings

Perforce connection information for use by the Horde server (for reading config files, etc...)

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this server
`serverAndPort` | `string`<br>Server and port
`credentials` | [PerforceCredentials](#perforcecredentials)<br>Credentials for the server

## PerforceCredentials

Credentials for a Perforce user

Name | Description
---- | -----------
`userName` | `string`<br>The username
`password` | `string`<br>Password for the user
`ticket` | `string`<br>Login ticket for the user (will be used instead of password if set)

## CommitSettings

Options for the commit service

Name | Description
---- | -----------
`replicateMetadata` | `boolean`<br>Whether to mirror commit metadata to the database
`replicateContent` | `boolean`<br>Whether to mirror commit data to storage
`bundle` | [BundleOptions](#bundleoptions)<br>Options for how objects are packed together
`chunking` | [ChunkingOptions](#chunkingoptions)<br>Options for how objects are sliced

## BundleOptions

Options for configuring a bundle serializer

Name | Description
---- | -----------
`maxVersion` | [BundleVersion](#bundleversion-enum)<br>Maximum version number of bundles to write
`maxBlobSize` | `integer`<br>Maximum payload size fo a blob
`compressionFormat` | [BundleCompressionFormat](#bundlecompressionformat-enum)<br>Compression format to use
`minCompressionPacketSize` | `integer`<br>Minimum size of a block to be compressed
`maxWriteQueueLength` | `integer`<br>Maximum amount of data to store in memory. This includes any background writes as well as bundles being built.

## BundleVersion (Enum)

Bundle version number

Name | Description
---- | -----------
`Initial` | Initial version number
`ExportAliases` | Added the BundleExport.Alias property
`RemoveAliases` | Back out change to include aliases. Will likely do this through an API rather than baked into the data.
`InPlace` | Use data structures which support in-place reading and writing.
`ImportHashes` | Add import hashes to imported nodes
`LatestV1` | Last version using the V1 pipeline
`PacketSequence` | Structure bundles as a sequence of self-contained packets (uses V2 code)
`Latest` | The current version number
`LatestV2` | Last version using the V2 pipeline
`LatestPlusOne` | Last item in the enum. Used for

## BundleCompressionFormat (Enum)

Indicates the compression format in the bundle

Name | Description
---- | -----------
`None` | Packets are uncompressed
`LZ4` | LZ4 compression
`Gzip` | Gzip compression
`Oodle` | Oodle compression (Selkie)
`Brotli` | Brotli compression
`Zstd` | ZStandard compression

## ChunkingOptions

Options for creating file nodes

Name | Description
---- | -----------
`leafOptions` | [LeafChunkedDataNodeOptions](#leafchunkeddatanodeoptions)<br>Options for creating leaf nodes
`interiorOptions` | [InteriorChunkedDataNodeOptions](#interiorchunkeddatanodeoptions)<br>Options for creating interior nodes

## LeafChunkedDataNodeOptions

Options for creating a specific type of file nodes

Name | Description
---- | -----------
`minSize` | `integer`<br>Minimum chunk size
`maxSize` | `integer`<br>Maximum chunk size. Chunks will be split on this boundary if another match is not found.
`targetSize` | `integer`<br>Target chunk size for content-slicing
`windowSize` | `integer`<br>Window size to use when scanning for split points
`threshold` | `integer`<br>Accessor for the BuzHash chunking threshold

## InteriorChunkedDataNodeOptions

Options for creating interior nodes

Name | Description
---- | -----------
`minChildCount` | `integer`<br>Minimum number of children in each node
`targetChildCount` | `integer`<br>Target number of children in each node
`maxChildCount` | `integer`<br>Maximum number of children in each node
`sliceThreshold` | `integer`<br>Threshold hash value for splitting interior nodes

## ComputeServerConfig

Static configuration for the compute plugin

Name | Description
---- | -----------
`enableUpgradeTasks` | `boolean`<br>Whether to enable the upgrade task source, always upgrading agents to the latest version
`withAws` | `boolean`<br>Whether to enable Amazon Web Services (AWS) specific features
`awsRegions` | `string[]`<br>List of AWS regions for Horde to be aware of (e.g. us-east-1 or eu-central-1) Right now, this is only used for replicating CloudWatch metrics to multiple regions
`awsAutoScalingQueueUrls` | `string[]`<br>AWS SQS queue URLs where lifecycle events from EC2 auto-scaling are received
`fleetManagerV2` | [FleetManagerType](#fleetmanagertype-enum)<br>Default fleet manager to use (when not specified by pool)
`fleetManagerV2Config` | `object`<br>Config for the fleet manager (serialized JSON)
`autoEnrollAgents` | `boolean`<br>Whether to automatically enroll agents in the farm
`defaultAgentPoolSizeStrategy` | [PoolSizeStrategy](#poolsizestrategy-enum)<br>Default agent pool sizing strategy for pools that doesn't have one explicitly configured
`agentPoolScaleOutCooldownSeconds` | `integer`<br>Scale-out cooldown for auto-scaling agent pools (in seconds). Can be overridden by per-pool settings.
`agentPoolScaleInCooldownSeconds` | `integer`<br>Scale-in cooldown for auto-scaling agent pools (in seconds). Can be overridden by per-pool settings.
`computeTunnelPort` | `integer`<br>Port to listen on for tunneling compute sockets to agents
`computeTunnelAddress` | `string`<br>What address (host:port) clients should connect to for compute socket tunneling Port may differ from  if Horde server is behind a reverse proxy/firewall
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## FleetManagerType (Enum)

Available fleet managers

Name | Description
---- | -----------
`Default` | Default fleet manager
`NoOp` | No-op fleet manager.
`Aws` | Fleet manager for handling AWS EC2 instances. Will create and/or terminate instances from scratch.
`AwsReuse` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsRecycle` | Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
`AwsAsg` | Fleet manager for handling AWS EC2 instances. Uses an EC2 auto-scaling group for controlling the number of running instances.

## PoolSizeStrategy (Enum)

Available pool sizing strategies

Name | Description
---- | -----------
`LeaseUtilization` | Strategy based on lease utilization
`JobQueue` | Strategy based on size of job build queue
`NoOp` | No-op strategy used as fallback/default behavior
`ComputeQueueAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet
`LeaseUtilizationAwsMetric` | A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet

## ExamplePluginServerConfig

Server configuration for the ExamplePlugin

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## ExperimentalServerConfig

Server configuration for the experimental plugin

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## SecretsServerConfig

Static configuration for the secrets plugin

Name | Description
---- | -----------
`withAws` | `boolean`<br>Whether to enable Amazon Web Services (AWS) specific features
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## StorageServerConfig

Static settings for the storage system

Name | Description
---- | -----------
`bundleCacheDir` | `string`<br>Directory to use for the coarse-grained backend cache. This caches full bundles downloaded from the upstream object store.
`bundleCacheSize` | `string`<br>Maximum size of the storage cache on disk. Accepts standard binary suffixes (kb, mb, gb, tb, etc...)
`bundleCacheSizeBytes` | `integer`<br>Accessor for the bundle cache size in bytes
`backends` | [BackendConfig](#backendconfig)`[]`<br>Overridden settings for storage backends. Useful for running against a production server with custom backends.
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## BackendConfig

Common settings object for different providers

Name | Description
---- | -----------
`id` | `string`<br>The storage backend ID
`base` | `string`<br>Base backend to copy default settings from
`secondary` | `string`<br>Specifies another backend to read from if an object is not found in this one. Can be used when migrating data from one backend to another.
`type` | [StorageBackendType](#storagebackendtype-enum)<br>
`baseDir` | `string`<br>
`awsBucketName` | `string`<br>Name of the bucket to use
`awsBucketPath` | `string`<br>Base path within the bucket
`awsCredentials` | [AwsCredentialsType](#awscredentialstype-enum)<br>Type of credentials to use
`awsRole` | `string`<br>ARN of a role to assume
`awsProfile` | `string`<br>The AWS profile to read credentials form
`awsRegion` | `string`<br>Region to connect to
`azureConnectionString` | `string`<br>Connection string for Azure
`azureContainerName` | `string`<br>Name of the container
`relayServer` | `string`<br>
`relayToken` | `string`<br>
`gcsBucketName` | `string`<br>Name of the GCS bucket to use
`gcsBucketPath` | `string`<br>Base path within the bucket

## StorageBackendType (Enum)

Types of storage backend to use

Name | Description
---- | -----------
`FileSystem` | Local filesystem
`Aws` | AWS S3
`Azure` | Azure blob store
`Gcs` | Google Cloud Storage
`Memory` | In-memory only (for testing)

## AwsCredentialsType (Enum)

Credentials to use for AWS

Name | Description
---- | -----------
`Default` | Use default credentials from the AWS SDK
`Profile` | Read credentials from the  profile in the AWS config file
`AssumeRole` | Assume a particular role. Should specify ARN in
`AssumeRoleWebIdentity` | Assume a particular role using the current environment variables.

## PluginServerConfig

Base class for plugin server config objects

Name | Description
---- | -----------
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## ToolsServerConfig

Server configuration for bundled tools

Name | Description
---- | -----------
`bundledTools` | [BundledToolConfig](#bundledtoolconfig)`[]`<br>Tools bundled along with the server. Data for each tool can be produced using the 'bundle create' command, and should be stored in the Tools directory.
`enabled` | `boolean`<br>Whether the plugin should be enabled or not

## BundledToolConfig

Configuration for a tool bundled alongsize the server

Name | Description
---- | -----------
`version` | `string`<br>Version string for the current tool data
`refName` | `string`<br>Ref name in the tools directory
`dataDir` | `string`<br>Directory containing blob data for this tool. If empty, the tools/{id} folder next to the server will be used.
`id` | `string`<br>Unique identifier for the tool
`name` | `string`<br>Name of the tool
`description` | `string`<br>Description for the tool
`category` | `string`<br>Category for the tool. Will cause the tool to be shown in a different tab in the dashboard.
`group` | `string`<br>Grouping key for different variations of the same tool. The dashboard will show these together.
`platforms` | `string[]`<br>Platforms for this tool. Takes the form of a NET RID (https://learn.microsoft.com/en-us/dotnet/core/rid-catalog).
`public` | `boolean`<br>Whether this tool should be exposed for download on a public endpoint without authentication
`showInUgs` | `boolean`<br>Whether to show this tool for download in the UGS tools menu
`showInDashboard` | `boolean`<br>Whether to show this tool for download in the dashboard
`showInToolbox` | `boolean`<br>Whether to show this tool for download in Unreal Toolbox
`metadata` | `string` `->` `string`<br>Metadata for this tool
`namespaceId` | `string`<br>Default namespace for new deployments of this tool
`maxDeployments` | `integer`<br>Maximum number of deployments to retain for this tool. When exceeded, the oldest deployments are removed.
`acl` | [AclConfig](#aclconfig)<br>Permissions for the tool

## AclConfig

Parameters to update an ACL

Name | Description
---- | -----------
`entries` | [AclEntryConfig](#aclentryconfig)`[]`<br>Entries to replace the existing ACL
`profiles` | [AclProfileConfig](#aclprofileconfig)`[]`<br>Defines profiles which allow grouping sets of actions into named collections
`inherit` | `boolean`<br>Whether to inherit permissions from the parent ACL
`exceptions` | `string[]`<br>List of exceptions to the inherited setting

## AclEntryConfig

Individual entry in an ACL

Name | Description
---- | -----------
`claim` | [AclClaimConfig](#aclclaimconfig)<br>Name of the user or group
`actions` | `string[]`<br>Array of actions to allow
`profiles` | `string[]`<br>List of profiles to grant

## AclClaimConfig

New claim to create

Name | Description
---- | -----------
`type` | `string`<br>The claim type
`value` | `string`<br>The claim value

## AclProfileConfig

Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.

Name | Description
---- | -----------
`id` | `string`<br>Identifier for this profile
`actions` | `string[]`<br>Actions to include
`excludeActions` | `string[]`<br>Actions to exclude from the inherited actions
`extends` | `string[]`<br>Other profiles to extend from
