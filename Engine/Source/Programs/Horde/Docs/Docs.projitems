<?xml version="1.0" encoding="utf-8"?>
<Project xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup>
    <MSBuildAllProjects Condition="'$(MSBuildVersion)' == '' Or '$(MSBuildVersion)' &lt; '16.0'">$(MSBuildAllProjects);$(MSBuildThisFileFullPath)</MSBuildAllProjects>
    <HasSharedItems>true</HasSharedItems>
    <SharedGUID>0f085df2-a0bb-405a-ba24-9ed745f7089a</SharedGUID>
  </PropertyGroup>
  <PropertyGroup Label="Configuration">
    <Import_RootNamespace>Docs</Import_RootNamespace>
  </PropertyGroup>

  <ItemGroup>
	<Content Include="./**/*.md"/>
  </ItemGroup>
</Project>
