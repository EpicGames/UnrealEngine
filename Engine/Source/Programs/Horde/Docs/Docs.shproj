<?xml version="1.0" encoding="utf-8"?>
<Project ToolsVersion="15.0" xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <PropertyGroup Label="Globals">
    <ProjectGuid>{791298F6-091F-478B-874B-568D18F54E97}</ProjectGuid>
    <MinimumVisualStudioVersion>14.0</MinimumVisualStudioVersion>
  </PropertyGroup>
  <Import Project="$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props" Condition="Exists('$(MSBuildExtensionsPath)\$(MSBuildToolsVersion)\Microsoft.Common.props')" />
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v$(VisualStudioVersion)\CodeSharing\Microsoft.CodeSharing.Common.Default.props" />
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v$(VisualStudioVersion)\CodeSharing\Microsoft.CodeSharing.Common.props" />
  <PropertyGroup />
  <Import Project="Docs.projitems" Label="Shared" />
  <Import Project="$(MSBuildExtensionsPath32)\Microsoft\VisualStudio\v$(VisualStudioVersion)\CodeSharing\Microsoft.CodeSharing.CSharp.targets" />
</Project>