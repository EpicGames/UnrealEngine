[Horde](../README.md) > FAQ

# FAQ

## Why are all these use cases being muddled together?

Most of our target use cases are traditionally handled as distinct components, but bringing them all together lets us
take a lot of shortcuts and optimize for common use cases. For example:

* Storage is a key component of any data pipeline for caching.
* Remote execution needs data close to compute nodes where it can be retrieved quickly.
* Scalable build automation systems use the same scheduling, management tools, and auto-scaling
  functionality as a remote execution platform and require a storage backend for intermediate and final
  artifacts.

By sharing the infrastructure that supports our internal dev teams, we hope to remove more of the busy work that is
needed for teams to collaborate successfully.

## Will I need to deploy Horde to use Unreal Engine?

No. Horde is not a requirement to use Unreal Engine, but it has been valuable for Epic, and
we hope it can be valuable for other teams too.

## Do I have to deploy Horde to the cloud?

No. Horde runs well in local deployments using off-the-shelf hardware, though some applications may benefit from
cloud storage even if you don't host any other infrastructure in the cloud.

## Do I have to use the CI system / remote execution functionality / test framework / etc.?

No. Each feature is optional, and any disabled features do not incur any costs.

## Why would I use Horde for build automation rather than Jenkins or TeamCity?

Most build automation systems are purposefully generic, so you can run any workload on them.

That leaves a lot of plumbing for build ops teams to do: writing build scripts, managing artifact transfers between
agents to implement parallelism, setting up stores for final artifacts, implementing a way to manage the allocation of
network-connected devkits and mobile devices for build agents and so on.

What's more, the resulting system is not very smart. You may have notifications for build failures, but it requires
a human to diagnose what and when a failure began and who needs to fix it. You need a way to clean up old build
artifacts and manage permissions for them. If you're using network shares for artifacts, you need to deal
with lots of overlapping artifacts that are almost the same. Then, if you're a large organization, you need to figure
out a way to distribute these artifacts to developers in different locations - and, of course, you probably want
some tooling so devs can find and fetch these artifacts.

Functionality such as servers for tracking results of automated tests between builds is another layer; it may run on
the build automation system, but you might have a myriad of tests generating data over lots of different changes that
needs to be stored somewhere, and want to go spelunking through that data to find when, say, framerate dropped below a
certain threshold in a certain map, or when the size of a particular level exceeded a certain point.

A lot of these problems are orthogonal to the problems that build automation systems typically concern themselves
with solving. Still, by thinking about them together, we can make much smarter implementation decisions that understand
their operating context.

That said, Horde's CI functionality is not enabled by default. Other functionality in Horde can still be used without
migrating to a new build automation system.

## Does the build automation system support Git / Subversion / etc...?

Not at the moment. We make a number of assumptions that are fairly Perforce-centric, such as being able to stronglyorder
changes in a single, linear branch history. We also have a lot of custom logic for provisioning Perforce workspaces in
an efficient way.

We may explore support for other version control systems in future.

## Does Epic get telemetry about our project if we use Horde?

No. Horde does not send any data to Epic.
