[Horde](../README.md) > Feature Status

# Feature Status

| Feature | Status | Notes/Roadmap |
| ------- | ------ | ----- |
| Analytics | 🟧&nbsp;Experimental | - |
| Artifact&nbsp;Distribution | 🟨&nbsp;Beta | Needs better integration with other tooling.<br>Improved chunking and upload performance. |
| Test Automation Hub | 🟩&nbsp;Production Ready | - |
| Build Automation | 🟩&nbsp;Production Ready | Migrate jobs to a more lightweight data model.<br>Programmable state machine for jobs. |
| Build Health | 🟩&nbsp;Production Ready | - |
| Experimental | 🟧&nbsp;Experimental | - |
| Remote Execution | 🟩&nbsp;Production Ready | - |
| UnrealGameSync&nbsp;Server | 🟩&nbsp;Production Ready | - |

## Key

* 🟩 **Production Ready**: Recommended for use in production environments.
  Epic will maintain an upgrade path for this feature between releases.
* 🟨 **Beta**: Feature is stable and ready for evaluation but may have bugs and minor missing features.
  Epic will attempt to maintain an upgrade path for this feature.
* 🟧 **Experimental**: Feature is functional but may undergo significant modifications or be removed entirely
  in future releases.
* 🟥 **Early**: Feature is still early in development and is not recommended for use.
