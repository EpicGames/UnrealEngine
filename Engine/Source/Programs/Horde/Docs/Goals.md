[Horde](../README.md) > [Goals & Philosophy](Goals.md)

# Goals & Philosophy

## Opinionated

Horde was borne out of workflows and best practices that have arisen for Epic over time. They aren't the only way
to work and may not be for everyone. Being completely generic is a non-goal for Horde - we believe it is in
the interaction between systems; the context in which tools are being used, that provides the richest opportunity
for creating smooth and slick workflows for creators.

## Simple to Deploy

We've tried to build Horde in a way that requires little setup to run. While you can have fairly elaborate,
multi-machine, distributed deployments (and [we do!](Deployment.md#epics-horde-deployment)), it's just as easy to
run and debug locally on all our supported desktop platforms with few prerequisites installed. A database
will be created locally if you don't have one set up, and all the required services will automatically start and
stop with the server's lifetime.

## Easy to Manage

Having control over the source for a project like Horde, while also using it aggressively in a high-velocity
environment, allows us to optimize for our own ease of use in managing it as well as end-user facing features.
We have a tight feedback loop with our ops teams and try to make their life as easy as possible. Most
configuration data can be stored in source control, and we provide built-in profiling and performance
tooling.

## Scalable

There are other tools developed by large engineering companies that provide (some) similar services
ingrained into their development culture; Bazel, Pants, and BuildXL for remote execution, for example - though
they frequently focus on delivering public client tools without the matching server backends to make them shine
in practice. We distribute full source code for all client and server parts of Horde.

## Private

Epic does not host any of your data with Horde nor receive any telemetry from user deployments of it.
You can host it on a private network, as befits your IT policies, and integrate it with your own OIDC
authentication provider to allow access to it.
