![Horde](Images/ue-horde-black.png#gh-light-mode-only)![Horde](Images/ue-horde-white.png#gh-dark-mode-only)

## Getting Started

* **[Install the Horde agent](Tutorials/InstallHorde.md)**
* **[Enable remote C++ compilation](Tutorials/RemoteCompilation.md)**
* **[Enable remote shader compilation](Tutorials/RemoteShaderCompilation.md)**
* **[Set up build automation](Tutorials/BuildAutomation.md)**
* **[Enable test automation with Gauntlet](Tutorials/TestAutomation.md)**
* **[Get telemetry and analytics for your team](Tutorials/Analytics.md)**
* **[Work with mobile and console devices](Tutorials/DeviceManager.md)**
* **[Install UnrealGameSync and distribute Unreal Editor to your team](Tutorials/UnrealGameSync.md)**
* **[Enable authentication](Tutorials/Authentication.md)**

## Reference

Horde's reference documentation is divided into sections by target audience:

* [**Deploying Horde**](Deployment.md)
  * Information on the architecture and components making up Horde, and best practices for deploying them.
  * **Target audience:** IT, sysadmins, coders intending to modify Horde.
* [**Configuring and Operating Horde**](Config.md)
  * Describes how to set up and administer Horde.
  * **Target audience:** Build/dev ops teams, admins.
* [**Horde Internals**](Internals.md)
  * Describes how to build and modify Horde, and its architecture.
  * **Target audience:** Developers wishing to extend Horde.

## Status

Horde is considered production ready for Unreal Engine licensees.

See: [Feature Status](Features.md)

## Further Reading

* [Glossary](Glossary.md)
