[Horde](../README.md) > Internals

# Internals

## Getting Started

* [Building Horde](Internals/Building.md)

## Interfacing with Horde

* [C# API](Internals/CsApi.md)
* [REST API](Internals/RestApi.md)

## Features

* [Compute Architecture](Internals/ComputeArchitecture.md)
* [Storage Architecture](Internals/StorageArchitecture.md)
* [Logs](Internals/Logs.md)
* [Structured Logging](Internals/StructuredLogging.md)
