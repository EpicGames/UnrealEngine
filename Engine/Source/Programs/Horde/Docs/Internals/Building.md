[Horde](../../README.md) > [Internals](../Internals.md) > Building Horde

# Building Horde

## Server

The source code for the Horde Server is in `Engine/Source/Programs/Horde/Horde.Server`. It is written in C#, using ASP.NET.

Horde uses the standard
[C# coding conventions](https://learn.microsoft.com/en-us/dotnet/csharp/fundamentals/coding-style/coding-conventions)
published by Microsoft, though we use tabs rather than spaces for legacy reasons. We enable most static analyzer warnings
that ship with the NET SDK, but you may disable some through `.editorconfig` files.

Horde is configured to support local development by default. You can launch it by opening
`Engine/Source/Programs/Horde/Horde.sln` and setting Horde.Server as the default project. 
By default, you can access the server at `http://localhost:5000/account`.

When debugging a local Horde server against a live deployment, setting the `DatabaseReadOnlyMode` property in
[`Server.json`](../Deployment/ServerSettings.md) prevents the server from attempting any operation that modifies the
server state. Using a read-only DB account in addition is recommended for safety.

## Dashboard

The Horde dashboard is a frontend client developed in [TypeScript](https://www.typescriptlang.org) using
[React](https://react.dev/). To set up your machine for developing the dashboard:

1. Install [Node.js](https://nodejs.org/en/download).
2. Navigate to the dashboard folder at `Engine\Source\Programs\Horde\HordeDashboard`.
3. Run `npm install --legacy-peer-deps` to install the package dependencies.
4. Edit vite.config.ts setting the proxyTarget variable to point at your server URL, for example: `http://localhost:13340`.
5. Navigate to the admin token endpoint of your server to get an expiring access token, for example:
   `http://localhost:13340/api/v1/admin/token`.
6. Create a file called `.env.local` in the root HordeDashboard folder and paste the access token in like
   so: `VITE_HORDE_DEBUG_TOKEN=eyFhbGciziJIUz`.
7. Run `npm run dev` to start the development web server which should open a tab to the local dashboard at
   `http://localhost:5173`.

## Docker Image

Horde includes a `Dockerfile` for creating Docker images. However, its position in Unreal Engine source tree requires
files to be staged beforehand to reduce the size of data copied into build images.

A BuildGraph script to perform these operations is included under `Engine\Source\Programs\Horde\BuildHorde.xml`, which
can be run as follows:

    RunUAT.bat BuildGraph -Script=Engine/Source/Programs/Horde/BuildHorde.xml -Target="Build HordeServer"
