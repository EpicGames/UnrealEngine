[Horde](../../README.md) > [Internals](../Internals.md) > Compute Architecture

# Compute Architecture

The mechanism by which Horde communicates with its agents is based heavily
on Google's Remote Worker API. Work items assigned to agents are known as
*leases*.

Communication between the agent and server is done through streaming gRPC calls initiated by the agent. The ends of the
connection exchange copies of what they believe the current state of leases owned by the agent should be, and a state
machine determines which field is authoritative at different times. The server and agent keep exchanging state objects
and reconciling differences until they are in sync; at that point, the agent has acknowledged that it is taking any
new leases added by the server, and the server has acknowledged the completion of any removed leases by the agent.

Within the Horde server, leases are assigned to agents through classes implementing `ITaskSource`.

## Lease Types

Each lease type consists of the following key classes:

* a message defining the lease itself (i.e. what the agent needs to do),
* a task source on the server that decides to assign a lease to an agent (`ITaskSource`),
* a lease handler on the agent which does the work (`LeaseHandler`).

Horde ships with the following lease types:

| Message | Task source (server) | Lease handler (agent) | Description |
| ---------------- | --- | --- | --- |
| `job_task.proto` | `JobTaskSource` | `JobHandler` | Executes a batch as part of a CI job. |
| `upgrade_task.proto` | `UpgradeTaskSource` | `UpgradeHandler` | Upgrades the Horde agent software to a newer version. |
| `conform_task.proto` | `ConformTaskSource` | `ConformHandler` | Syncs all the workspaces on a machine to latest and optionally removes any untracked files. |
| `restart_task.proto` | `RestartTaskSource` | `RestartHandler` | Restarts the machine. |
| `shutdown_task.proto` | `ShutdownTaskSource` | `ShutdownHandler` | Powers down the machine. |

## Adding New Lease Types

To add a new lease type, each of the key classes above must be added.

* After adding a task source to the server, register it as a singleton implementing `ITaskSource` in `Startup.cs`.
* After adding a lease handler to the agent, register it as a singleton implementing `LeaseHandler` in `Program.cs`.
