[Horde](../../README.md) > [Internals](../Internals.md) > C# API

# C# API

Horde is implemented in C#, so the C# API is able to share a lot
of implementation boilerplate with the server. That makes it an ideal
choice for scripting, and makes the integration with other Unreal
Engine tools such as AutomationTool, UnrealBuildTool,
UnrealGameSync and Unreal Toolbox more convenient.

## Overview

There are two primary mechanisms for interacting with the server;
through the [HTTP REST API](RestApi.md) (`HordeHttpClient`), which
provides typed request and response classes, or via the native
C# API (`IHordeClient`). The `IHordeClient` interface is the main
entry point, but provides a method for getting a HTTP client if necessary (`IHordeClient.CreateHttpClient`).

While the HTTP API is more complete, we intend to move all functionality
into the native C# API over time. Doing so allows client tooling to write
idiomatic C# code without having to worry about marshalling and transport,
and allows clients to write code against the same public APIs
exposed by plugins on the server.

This approach gives more flexibility for prototyping code outside
the server then moving it into the server as it makes sense, and
gives flexibility if we need to split the server monolith into
microservices in the future. Where possible, use functionality exposed
through `IHordeClient` directly rather than using `HordeHttpClient`.

## Configuring Horde

There is a per-user setting for the preferred Horde server.

On Windows, this is stored in the registry under the following keys:

  ```
  HKEY_CURRENT_USER\SOFTWARE\Epic Games\Horde\Url (preferred)
  HKEY_LOCAL_MACHINE\SOFTWARE\Epic Games\Horde\Url
  ```

On MacOS and Linux, this is stored in the following JSON file:

  ```
  ~/.horde.json
  ```

The `HordeOptions.GetDefaultServerUrl()` method will retrieve the user's
preferred server for the current platform.

## Connecting to Horde

Horde is typically configured using .NET's standard dependency injection
library.

AutomationTool creates a service provider including Horde by default,
allowing you to get a Horde client interface as follows:

  ```cs
  IHordeClient hordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>();
  ```

From other C# tools, it may be necessary to construct the required
service container manually.

  ```cs
  ServiceCollection serviceCollection = new ServiceCollection();
  serviceCollection.AddHorde();

  await using ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider();
  IHordeClient hordeClient = 
     serviceProvider.GetRequiredService<IHordeClient>();
  ```

The server URL can be overridden for a particular tool using the
standard [.NET options pattern](https://learn.microsoft.com/en-us/aspnet/core/fundamentals/configuration/options?view=aspnetcore-8.0):

  ```cs
  serviceCollection.Configure<HordeOptions>(options => 
  {
    options.ServerUrl = new Uri("http://my-horde-server.com");
  });
  ```

Or through a delegate passed to the `AddHorde()` method:

  ```cs
  serviceCollection.AddHorde(options => 
  {
    options.ServerUrl = new Uri("http://my-horde-server.com");
  });
  ```
