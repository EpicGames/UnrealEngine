[Horde](../../README.md) > [Internals](../Internals.md) > Logs

# Logs

Historically, working with large log files in CI systems has been difficult. Across several previous incarnations of
our build infrastructure, we became used to plain-text logs, browser crashes when opening large cook logs, poor
search performance, and so on.

Logs are one of the things we wanted to get right in Horde from day one. At the top of our list of requirements were the abilities to:

* Perform indexed text searches through large logs quickly.
* Scrub through log files without fully downloading the log to the client.
* Provide much richer context-aware functionality to events in logs - cross-referencing them with build health issues, Perforce history, and external sites explaining the meaning of error codes.

## Storage

Horde logs are stored in bundles using several node types. All these classes are implemented in
`Engine/Source/Programs/Shared/EpicGames.Horde/Logs`.

* `LogNode` objects are the main entry point for the log data and contain metadata about the log as a whole (how
many lines it is, how long it is, whether it is still being appended to), as well as references to chunk and index
nodes containing the log data.
* `LogChunkNode` objects contain raw, UTF-8 encoded structured log data for a span of lines, plus offsets to quickly index lines within it. Each reference to a chunk from the root `LogNode` object contains its starting line
number, which enables quick identifcation of which node contains a particular line.
* `LogIndexNode` objects contain chunks of data used to determine which chunks contain particular search terms and a
plain text rendering of the chunk. The index/search algorithm is described in the section below.

Agents generating logs usually upload data to the storage backend directly, appending data by uploading a new root node
and any appended chunks and index nodes.

## Indexing

An index is generated from the plain text rendering of a log message in UTF-8. ANSI characters `A-Z` are converted into
their lowercase form `a-z`.

First, the message is split into tokens, each containing only characters of the following types:

* 0: Anything not in a category below
* 1: Alphabetic characters `[a-zA-Z]`
* 2: Numeric characters `[0-9]`
* 3: Whitespace characters `[ \t]`
* 4: Newline characters `\n`

For example, `hello world123` would be split into four tokens: `hello`, ` `, `world` and `123`.

Each token is then further decomposed into 32-bit ngrams; a 32-bit integer value containing successive 4-character
sequences, with partial ngrams shifted into left into more significant bits.

    "hell" = ('h' << 24) | ('e' << 16) | ('l' << 8) | 'l' = 0x68656c6c
    "o"    = ('o' << 24)                                  = 0x6f000000
    " "    = (' ' << 24)                                  = 0x20000000
    "worl" = ('w' << 24) | ('o' << 16) | ('r' << 8) | 'l' = 0x776f726c
    "d"    = ('d' << 24)                                  = 0x64000000
    "123"  = ('1' << 24) | ('2' << 16) | ('3' << 8)       = 0x31323300

These values are inserted into a sparse trie (`NgramSet`) over the 64-bit integer space, with the ngram value in the
top 32 bits, and an index of the block containing the ngram in the lower 32 bits. This forms a very space-efficient
data structure for performing a coarse search of where ngrams may be found in the log file.

When searching for a particular string, we perform the same transformation into ngrams on it and find all block
indexes which contain **all** ngrams of the search term. Special handling is taken for the first and last token in the
search term since it may match within a source token that had a longer or shorter prefix, changing the alignment
before it was decomposed into ngrams.

Once we have a potential match for the search term, we can perform the more expensive operation of fetching the chunk
and searching for the exact term using a simplified Knuth–Morris–Pratt algorithm.

## Tailing

Log tailing is enabled the first time a log is requested if it has not already been marked as complete. A request for log
tailing is stored in Redis with a TTL of 30 seconds, a value that is reset on subsequent calls.

Agents uploading logs poll the server to see whether tailing is desired. Once enabled, the agent starts uploading data
that has not yet been flushed to persistent storage directly to the server, where it is stored in Redis and returned
to any clients on subsequent reads of the log.

## JSON Syntax

Logs generated by Horde agents consist of one JSON object per line. Properties are as follows:

| Name         | Type      | Description |
| ------------ | --------- | ----------- |
| `time`       | `String`  | Timestamp that the event occurred, in UTC. |
| `level`      | `String`  | The event level. Valid values are listed [here](https://learn.microsoft.com/en-us/dotnet/api/microsoft.extensions.logging.loglevel). |
| `id`         | `Integer` | An event identifier. Known event IDs are listed in `Engine/Source/Progams/Shared/EpicGames.Core/KnownLogEvents.cs.` |
| `message`    | `String`  | The rendered event message. |
| `format`     | `String`  | Formatting string for rendering the message using standard [message template](https://messagetemplates.org/) syntax. |
| `properties` | `Object`  | Dictionary of properties for the format string. See below for more information. |
| `lineIndex`  | `Integer` | For multi-line messages, indicates the zero-based index of this log event within the message. |
| `lineCount`  | `Integer` | For multi-line messages, indicates the total number of lines in this message. |

Properties may be any regular JSON types or may be an object containing the `$type` property, indicating that the
dashboard may treat it as a particular type (and render it accordingly). Valid values for `$type` are defined in the
`LogValueType` class in `Engine/Source/Programs/Shared/EpicGames.Core/LogValue.cs`, and include:

| Name | Description | Additional Fields |
| ---- | ----------- | ----------------- |
| `Asset` | Path to an asset | `file`: Local file containing the asset<br>`depotPath`: Perforce depot path of the file containing the asset |
| `SourceFile` | Path to a file containing source code | `file`: Local file containing the asset<br>`depotPath`: Perforce depot path of the file containing the asset |
| `Channel` | An Unreal Engine channel name | |
| `Severity` | An Unreal Engine channel severity | |
| `LineNumber` | Line number for a compiler error message | |
| `ColumnNumber` | Column number for a compiler error message | |
| `Symbol` | A C++ linker symbol | `identifier`: Undecorated symbol name |
| `ToolName` | For standard Microsoft errors, indicates the name of the tool producing the the error. | |
| `ScreenshotTest` | The name of a screenshot test that has failed. | |

When a `$type` field is specified, a `$text` field may also be present. This indicates the intended rendering of the
field determined at the source, regardless of any navigation or context-sensitive functionality Horde may add on top.
