[Horde](../../README.md) > [Internals](../Internals.md) > REST API

# REST API

Horde's REST API is exposed through paths under `/api` on the
server.

Requests and responses support JSON and UE Compact Binary
(by using the `application/x-ue-cb` MIME type), and responses support
compression using Gzip, Zstd and Brotli.

## Documentation

A list of endpoints, and their parameters, is available
on the server under:

  ```
  http://my-horde-server.com/swagger
  ```

This page also supports sending requests to the server directly.

## Filtered responses

Many Horde HTTP methods support a filter parameter for filtering
responses. The syntax for filters includes simple wildcard expressions, such as:

  ```
  http://my-horde-server.com/api/v1/jobs?filter=id,batches.steps.id
  ```
