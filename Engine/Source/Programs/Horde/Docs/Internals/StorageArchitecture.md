[Horde](../../README.md) > [Internals](../Internals.md) > Storage

# Storage Architecture

Horde's storage platform is designed to support manipulating massive data structures consisting of
interlinked blobs. Blobs are **immutable**, and consist of an arbitrary block of data and zero
or more outward references to other blobs.

The entry point to any such data structure is a named reference, which maps a
user-defined name to a blob at the root of the data structure. Any nodes that are
not directly or indirectly referenced by a named reference are subject to garbage
collection.

## History

Horde's storage system can be seen as
an evolution of the GitSync tool used to download binaries for Epic's GitHub repo;  for each change
committed to Epic's Perforce server, we upload matching binaries to AWS S3 and commit a manifest to those
files to the Git repository that can be used to download those files. The GitSync tool uses the manifest to retrieve and
unpack those files whenever a new commit is checked out via a Git hook that is installed by running the Setup.bat file
in the root of the repo.

One of the main design goals for GitSync was to offload the hosting of binary data to a proven, scalable third-party
storage service (AWS S3) without having to maintain an active server capable of supporting many Unreal Engine developers. 
As such, we kept the ideas of content addressing used by Git but
packed the content-addressed payloads into non-deterministic packages for more efficient downloads. At upload
time, we use a heuristic to decide whether to reuse existing packages to download data versus re-packing
them into new packages to avoid expensive gather operations.

While clients can still model the data as a Git-like Merkle tree - using any locally cached data they may
have available using a uniquely identifying SHA1 hash - we reduce the chattiness and server-side
compute load in negotiating data to transfer to clients by putting the data into pre-made, static
download packages that are arranged to optimize for coherency between blobs we anticipate to be requested together.

This model optimizes for streaming reads and writes while still accomodating point reads when necessary.

## Blobs

A blob in Horde has the following attributes (see `BlobData` class):

* **Type**: Represented by a GUID and integer version number and used to distinguish between payloads that
may have a particular serialization format but the same data.

* **Payload**: A byte array. Blobs are meant to be fully read into memory, so payloads are typically limited
to a few hundred kb. Larger payloads can be split into smaller blobs using static or
content-defined chunking using utility libraries.

* **References**: A set of references to other blobs.

References to blobs are typically manipulated in memory using `IBlobHandle` instances. After flushing to
storage, blob handles can be converted to and from a `BlobLocator`, an opaque implementation-defined
string identifier assigned by the storage system.

Horde abstracts the way that blobs are serialized to the underlying storage backend through the `IBlobWriter`
interface; the programmer requests a buffer to serialize a blob to, writes the data and its references, and
gets a `IBlobHandle` back to allow retrieving it at any point in the future. The implementation is left to
decide how to store the data, implementing compression, packing, buffering, and uploading the data as
necessary.

Multiple `IBlobWriter` instances can be created to write separate streams of related blobs.

## Refs and Aliases

Refs and aliases provide entry points into the storage system. With either, you can assign a user-defined name to a particular blob and retrieve it later.

* **Refs** are strong references into the blob store and act as roots for the garbage collector. Refs can be
set to expire at fixed times or after not being retrieved for a specific period of time - which can be useful
for implementing caches.

* **Aliases** are weak references to blobs. Multiple aliases with the same name may exist, and users can
query for one or more blobs with a particular alias. Aliases have an associated user-specified rank, and
consumers can query aliases ordered by rank.

## Content Addressing

Horde models blobs in a way that supports content addressing. While hashes are not exposed
directly through `BlobLocator` strings, hashes can be encoded into a blob's payload, which has a matching
entry in the references array.

Since references are stored separately from the blob payload, the unique identifier stored through a `BlobLocator`
does not affect the payload's hash.

The implementation primarily uses `IoHash` for hashing blob data (a truncated 20-byte Blake 3 hash), but
decoupling the encoding of hashes in the payload from references in the storage system allows any other
hashing algorithm to be used instead. The underlying storage system can reason about the topology of blob
trees while still supporting a variety of hashing algorithms.

The `IBlobRef` interface extends the basic `IBlobHandle` interface with an `IoHash` of the target blob.

## Implementation

> **Note:** This section describes the storage system's current implementation details, which may change in future releases.

### Layers

The storage system is implemented through several layers:

* The C# serialization library (`BlobSerializer`, `BlobConverter`, and so on).

* The logical storage model is declared through the `IStorageClient` interface, which is the primary method for
  interacting with the storage system.
  At this layer, blobs are manipulated via `IBlobHandle` objects.

  * `BundleStorageClient` is the standard implementation of `IStorageClient`
  and packs blobs into [bundles](#bundles).

  * `KeyValueStorageClient` implements a client that passes individual
  blobs to the underlying `IStorageBackend`.

* The physical storage model is declared through the `IStorageBackend` interface, which deals with sending data over
  the wire to storage service implementations.
  
  * `HttpStorageBackend` uploads data to the Horde Server over HTTP.

  * `FileStorageBackend` writes data directly to files on disk.

  * `MemoryStorageBackend` stores data in memory.

* The bulk data store is declared through the `IObjectStore` interface, which interacts with low-level
  storage services.
  
  * `FileObjectStore` writes data to files on disk.

  * `AwsObjectStore` reads and writes data from AWS S3.

  * `AzureObjectStore` reads and writes data from the Azure blob store.

### Bundles

Blobs are designed to be used as a general-purpose storage primitive, so we try to
efficiently accommodate blob types
ranging from a handful of bytes to several hundred kilobytes (larger streams of data can be split up into smaller chunks
along fixed boundaries or using content-defined slicing).

Blobs are packed together into bundles for storage in the underlying object store.

The implementation of bundles and their use within the storage system is mostly hidden from user code, though
knowing how a stream of blobs will be written to storage may help reasoning about access patterns.

Each bundle consists of a sequence of compressed packets, each of which may contain several blobs. Each packet is
self-contained, so it may be decoded from a single contiguous ranged read of the bundle data.

### Locators

Locators typically have the following form:

    [path]#pkt=[offset],[length]&exp=[index]

* `[path]`: Path to an object within the underlying object store.
* `[offset]` and `[length]`: Byte range of the compressed packet data within a bundle.
* `[index]`: The index of an exported blob within the packet.
