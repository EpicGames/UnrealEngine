[Horde](../../README.md) > [Internals](../Internals.md) > Structured Logging

# Structured Logging

Horde heavily uses structured logging output from Unreal Engine and tools, which provides more context-specific
information than is typically available in plain-text logs.

To understand how Horde uses structured logging, it's helpful to consider the sort of information we'd like to get from
diagnostics in our automated processes:

* Human-readable message
* Source of diagnostic (compiler, linker, etc.)
* File(s) triggering the error (local path, path in version control)
* Line number
* Severity
* Error code
* Other context-specific information

Ideally, we would tag errors at source and add as much context-specific information as possible. For example, the
following log fragment:

    NVENC_EncoderH264.cpp (0:05.95 at +16:18)
    d:\build\AutoSDK\Sync\HostWin64\Win64\Windows Kits\10\include\10.0.18362.0\um\winnt.h(603): error C2220: the following warning is treated as an error
    d:\build\AutoSDK\Sync\HostWin64\Win64\Windows Kits\10\include\10.0.18362.0\um\winnt.h(603): warning C4005: 'TEXT': macro redefinition
    Engine\Source\Runtime\Core\Public\HAL\Platform.h(1081): note: see previous definition of 'TEXT'

...provides the following information:

* There has been a **compile** error.
* It occurred while compiling `NVENC_EncoderH264.cpp`, due to a conflict between macros defined in `winnt.h` (line 603)
  and `Platform.h` (line 1081).
* We can map `NVENC_EncoderH264.cpp` and `Platform.h` to files in source control and see their revision history.
* The warning number is `C4005`, which is treated as error `C2220`.
* It took 5.95 seconds to compile the file.

Rather than outputting plain text for log events, we preserve the format string and arguments and render them later.
That allows us to render those arguments differently for types we understand and enables us to index and search
logs based on those fields. 

Horde natively supports structured log events, which we use to do things like render source
files as links to the P4V timelapse view and error codes as links to MSDN. We can also map paths back to their history
in source control, which we use to figure out who broke the build via Horde's build health system.

Anyone who's bumped up against the build system treating any log line containing the string "error:" as an error can
rejoice; if you output a structured log event directly, Horde will no longer need to guess whether it's an error or not.

## Formatting

Unreal Engine uses standard [message templates](https://messagetemplates.org) syntax to write errors, for example:

    Logger.LogInformation("Hello {Text}", "world");

Notably:

* All parameters in a format string should be named rather than using numeric placeholders (ie. {Text} rather than {0},
  {1} etc.). These identifiers are used to name properties in the structured log event and can be indexed and
  searched through tools like Splunk and Datadog.
* Format strings should be constants rather than using interpolated or concatenated strings. This allows the logger
  implementation to cache and reuse parsed format strings between messages.

The log events may be rendered into a plain-text log for display in the console immediately or preserved in a
structured form in a [JSONL](https://jsonlines.org/) file.

## Writing events from C#

AutomationTool and UnrealBuildTool have support for writing log events using the NET ILogger API.

All logging should be done through an
[ILogger](https://learn.microsoft.com/en-us/dotnet/api/microsoft.extensions.logging.ilogger)
instance (defined in the Microsoft.Extensions.Logging namespace) rather than passed through the legacy 
`Log.TraceInformation` and other static methods.

## Writing events from C++

Unreal Engine runtime supports writing structured log events using the `UE_LOGFMT` macro.

## Capturing Output

Horde sets a UE_LOG_JSON_TO_STDOUT environment variable, which instructs tools such as AutomationTool to output JSON
directly to stdout, which it ingests and stores for rendering on the dashboard.

## Legacy Log Output

For external tools that don't support structured logs (e.g. compilers, etc.), we have a library of regexes that run
over plain text output and construct structured log events from them. Some are in UBT
(`Engine/Source/Programs/UnrealBuildTool/Matchers/...`) and some in UAT (
`Engine/Source/Programs/AutomationTool/AutomationUtils/Matchers/...`).
These are used by the `LogEventParser` class in EpicGames.Core.

To implement a new matcher for plain-text log output, create a class that implements the `ILogEventMatcher` interface
from `EpicGames.Core`, and ensure it's registered 

When adding or modifying log parsers, we strongly recommend running (and writing) tests in the `UnrealBuildTool.Tests`
and `HordeAgent.Tests` projects to check interaction with other handlers.
