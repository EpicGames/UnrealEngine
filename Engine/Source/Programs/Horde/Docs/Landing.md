![Horde](Images/ue-horde-black.png#gh-light-mode-only)![Horde](Images/ue-horde-white.png#gh-dark-mode-only)

## Getting Started

**Horde** has been installed successfully. To continue setting up your deployment, check out the following guides:

* **[Install the Horde agent](Tutorials/InstallHorde.md)**
* **[Enable remote C++ compilation](Tutorials/RemoteCompilation.md)**
* **[Enable remote shader compilation](Tutorials/RemoteShaderCompilation.md)**
* **[Set up build automation](Tutorials/BuildAutomation.md)**
* **[Enable test automation with Gauntlet](Tutorials/TestAutomation.md)**
* **[Get telemetry and analytics for your team](Tutorials/Analytics.md)**
* **[Work with mobile and console devices](Tutorials/DeviceManager.md)**
* **[Install UnrealGameSync and distribute Unreal Editor to your team](Tutorials/UnrealGameSync.md)**
* **[Enable authentication](Tutorials/Authentication.md)**

Alternatively, open the full documentation through the `Help` menu at the top right of the page.
