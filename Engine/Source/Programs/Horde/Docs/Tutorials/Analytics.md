[Horde](../../README.md) > Getting Started: Analytics

# Getting Started: Analytics

## Introduction

**Horde** implements a telemetry collector which can receive
and process events sent by the Unreal Editor.

Horde aggregates telemetry events into **metrics** for discrete time intervals, which can then be charted by the Horde
dashboard to give valuable insights into the bottlenecks experienced by your team.

![Analytics](../Images/Analytics-Main.png)

## Prerequisites

* Horde Server installation (see [Getting Started: Install Horde](InstallHorde.md)).
* An Unreal Engine project targetting Unreal Engine 5.4 or later.

## Steps

1. Open your project in the Unreal Editor and go to the `Edit > Plugins` menu. Search for the `Studio Telemetry` plugin and
make sure it is enabled. It should be enabled by default.
2. Open your project's `DefaultEngine.ini` file (in the `Config` folder next to your `.uproject` file) and add
the following lines:

    ```ini
    [StudioTelemetry.Provider.Horde]
    Name=HordeAnalytics
    ProviderModule=AnalyticsET
    UsageType=EditorAndClient
    APIKeyET=HordeAnalytics.Dev
    APIServerET="http://localhost:13340/"
    APIEndpointET="api/v1/telemetry/engine"
    ```

    Make sure to replace the value for `APIServerET` with the address of your Horde Server.

3. Configure a telemetry store to aggregate metrics from your
   telemetry events. There are some default metrics and charts included
   with the Horde installation, which can be added by adding the following snippet to your
   [globals.json](../Config/Orientation.md) file:

    ```json
    // Define the 'Engine' telemetry store and create some standard metrics & views with it.
    "analytics": {
        "stores": [
            {
                "id": "engine",
                "include": [
                    {
                        "path": "$(HordeDir)/Defaults/default-metrics.telemetry.json"
                    },
                    {
                        "path": "$(HordeDir)/Defaults/default-views.telemetry.json"
                    }
                ]
            }
        ]
    },
    ```

## See Also

* [Configuration > Analytics](../Config/Analytics.md)
