[Horde](../../README.md) > Getting Started: Authentication

# Getting Started: Authentication

## Introduction

Horde ships with auth disabled by default to make it easier to demonstrate and experiment with. Most production
deployments will probably want users to log in, and restrict the actions they can perform based on their role.

To do this, Horde supports **[OAuth2](https://oauth.net/2/)** and **[OIDC](https://openid.net/developers/how-connect-works/)**,
which is supported by most third party identity providers - including Okta, AWS, Azure, and Google.
Configuring an external identity provider is out of scope for this documentation, though the relevant configuration
points are touched on in the [Deployment > Server](../Deployment/Server.md#authentication) page.

If you don't have an existing OIDC-compatible identity provider, Horde includes it's own - which this guide covers.

## Prerequisites

* Horde Server installation (see [Getting Started: Install Horde](InstallHorde.md)).
* A valid certificate, and [HTTPS support](../Deployment/Server.md#https) enabled on your server.

## Steps

1. In your [server.json](../Config/Orientation.md) file set the `AuthMode` property to `Horde`, and restart the server.
2. The first time you launch the server, you'll be prompted to enter an administrator password.
3. After logging in, there will be an `Accounts` menu item in the `Server` menu. From here, you can manage the users
allowed to log in to the server, and the [**claims**](../Glossary.md#authorization) that they have. Horde's account
system uses the `http://epicgames.com/ue/horde/group` claim for groups that a user belongs to, and the dashboard will
suggest and autocomplete any groups found in the deployment's configuration files.

There are two standard groups defined in the server's `default.globals.json` file, which is included from the standard
`globals.json` file by default: `View` and `Run`.

   ```json
    "acl": {
        "entries": [
            {
                "claim": {
                    "type": "http://epicgames.com/ue/horde/group", 
                    "value": "View"
                },
                "profiles": [
                    "default-read"
                ]
            },
            {
                "claim": {
                    "type": "http://epicgames.com/ue/horde/group", 
                    "value": "Run"
                },
                "profiles": [
                    "default-run"
                ]
            }
        ]
    }
   ```

The `default-read` and `default-run` profiles are defined in code (`AclConfig.cs`). You can define your own profiles within
the `profiles` element of each [AclConfig](../Config/Schema/Globals.md#aclconfig) object.
