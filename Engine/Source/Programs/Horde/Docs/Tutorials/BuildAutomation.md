[Horde](../../README.md) > Getting Started: Build Automation

# Getting Started: Build Automation

## Introduction

Horde implements a build automation system using BuildGraph as a scripting language, which supports Windows, Mac, and
Linux. It integrates closely with Horde's remote execution capabilities, UnrealGameSync, and other tools
in the Unreal Engine ecosystem.

The terms Continuous Integration (CI) and Continuous Delivery (CD) are common monikers for build automation,
ensuring that the state of a project is continuously being monitored and that builds are produced regularly.

## Prerequisites

* Horde Server and one or more Horde Agents (see [Getting Started: Install Horde](InstallHorde.md))
* A configured Perforce server with a stream containing Unreal Engine 5.4 or later.
  * Legacy Perforce branches are not currently supported.
  * Horde may support other revision control systems in the future.

## Setup

1. Locate the `C:\ProgramData\Epic\Horde\Server` folder containing configuration files for the Horde Server.
   * The `C:\ProgramData` folder is hidden by default. You may have to enter it into the Windows Explorer address bar manually.

2. Open the `globals.json` file.
   * This file is one of two main configuration files for the server. The `server.json` file configures settings
     specific to this machine (such as other servers to communicate with, logging settings, etc.), while the
     `globals.json` file configures settings shared by all server instances in a multi-machine installation.
   * See [Configuration > Orientation](../Config/Orientation.md) for more information.
   * See [Configuration > Schema > Globals.json](../Config/Schema/Globals.md) for detailed information on all
     settings.

3. Configure your Perforce server in the `perforceClusters` section of the config file.
   * A default configuration is included but commented out - configure the `serverAndPort` and
     `credentials` entries as appropriate for your deployment.
   * The server will reload any changes to `globals.json` automatically when the file is saved.
   * You may wish to go to the Horde Dashboard ([http://localhost:13340](http://localhost:13340) by default),
     open the **Server** menu, and select **Status**. This page shows the status of various server subsystems,
     including the Perforce connection status - which should be confirmed as working.
   * Consider setting `supportsPartitionedWorkspaces` to true, in order to reduce Perforce journal load 
    ([Perforce documentation](https://portal.perforce.com/s/article/17297)) 

4. Enable the example project by uncommenting the following lines in `globals.json`:

   ```json
   "projects": [
      {
         "id": "ue5",
         "path": "ue5.project.json"
      }
   ]
   ```

   * A project in Horde parlance is typically a game or project, analagous to a stream depot in
   Perforce. Epic has several streams under the `UE5` project on our internal Horde instance, such as `//UE5/Main`,
   `//UE5/Release-5.8`, `//UE5/Dev-Main-HordeDemo` and so on.
   * The referenced config file, `ue5.project.json` exists in the same directory and references a stream configured
   in `ue5-release-5.8.stream.json`. The name of this file is not important for this tutorial.

5. Open the `ue5-release-5.8.stream.json` file and update the `name` property to a stream on your Perforce server.
   The default is `//UE5/Release-5.8-HordeSync`.
   * You should update the `Project` and `ProjectPath` macros below to reference your project. By
     default, these are set to build Epic's **Lyra** sample.

6. At this point, you should see the **UE5** project listed in the menu bar on the Horde Dashboard. Click on this
   button and select the stream you configured above.
   * You may need to refresh the dashboard in your web browser for the project to appear.

## Default Jobs

The example `ue5-release-5.8.stream.json` file configures the appearance of its page in the Horde Dashboard, as
well as job templates and agent types.

* A **job template** defines a set of parameters that are used to construct a
  [BuildGraph](https://docs.unrealengine.com/en-US/buildgraph-for-unreal-engine/) command line. Job templates
  are used to start jobs.
* An **agent type** defines a mapping from agents listed in BuildGraph script to a pool of machines that can
  execute it and settings for what those machines should sync from Perforce to execute the job.

After enabling the example stream, any agents connected to the server will start syncing Perforce workspaces
necessary to support agent types that they match.

The default page for the example stream shows tabs across the top of the page, which can be used
to group related job types. There are several predefined jobs on different tabs.

### Incremental

* **Incremental Build** - Builds the editor for your project and uploads editor builds to Horde that can be synced with
  UnrealGameSync. These jobs are designed to be fast, run frequently during the day, and use incremental workspaces that
  are not cleaned between runs. This allows them to start quickly and use intermediate artifacts produced on previous runs.

### Packaged Builds

* **Packaged Project Build** - Compiles and cooks a standalone game/client/server build for different target platforms
  and runs standard tests on it. These jobs run on non-incremental workspaces, restoring them to a
  pristine state before the job starts.

### Presubmit

* **Presubmit Tests** - Performs a quick editor compilation on a shelved changelist before submitting the change on
  behalf of the initiating user. The **P4VUtils** tool, which can be enabled through UnrealGameSync, provides extensions
  to Perforce P4V so you can initiate a presubmit build on a shelf through the UI.

### Utility

* **Installed Engine Build** - Creates an installed engine build similar to that which can be downloaded from the Epic
  Games Store. With Installed Engine Builds, you can use the engine like an SDK. This is designed for small teams that don't
  expect to modify the engine heavily.
* **Parameter Demo** - Shows the different types of parameters that can be configured through the Horde Dashboard and
  how to use them from a corresponding BuildGraph script (`Engine/Build/Graph/Examples/Parameters.xml`).
* **Remote Execution Test** - Tests compilation using UnrealBuildAccelerator. Horde passes settings to jobs through
  environment variables that allow UnrealBuildTool to connect to the server without any additional configuration.
* **Test Executor** - Runs a mock job with simulated errors or warnings, which is useful for testing connectivity to agents
  without syncing a Perforce workspace.

## See Also

* [Configuration > Build Automation](../Config/BuildAutomation.md)
