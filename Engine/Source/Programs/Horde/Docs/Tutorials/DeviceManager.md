[Horde](../../README.md) > Getting Started: Device Manager

# Getting Started: Device Manager

## Introduction

Horde includes a [device manager](../Config/Devices.md) for maintaining mobile and console development kit resources.  Managed devices are reserved by automated tests using a simple REST api.  Additionally, devices may be partitioned into user pools that support checkouts of shared remote devices for manual testing and development.  

The device manager includes a dashboard UI which makes it easy to administer and monitor devices.  Device usage generates telemetry which can be viewed to help allocate and schedule tests.  The Horde device manager is used extensively at Epic and is battle tested with features such as automatic problem reporting, kit rollover on errors, and integration with Slack.

While the device manager is integreated with Horde [build automation](BuildAutomation.md), it is possible to use it in isolation via the REST api.  

## Prerequisites

* Horde Server and an Android phone or tablet (see [Getting Started: Install Horde](InstallHorde.md)).

## Steps

1. Navigate to the Device management view on the Horde dashboard
  ![Device Navigation](../Images/Tutorial-DeviceManager-Devices.png)

2. The example Horde configuration includes a definition for the Android platform.  It also defines a shared and automation pool which devices can be partioned into for user checkouts and automated testing respectively.

  ```
  "devices": {
    "platforms": [
      {
        "id": "android",
        "name": "Android"
      }
    ],
    "pools": [
      {
        "id": "ue5",
        "name": "UE5",
        "poolType": "Automation"
      },
      {
        "id": "remote-ue5",
        "name": "Remote UE5",
        "poolType": "Shared"
      }
    ]
  }
  ```
  Click Add Device and fill out the new device form including the devices IP.  Select the example UE5 automation pool to assign the device for this workload.

  ![New Device](../Images/Tutorial-DeviceManager-NewDevice.png)

  Once saved, the Android device will be added and available for work.  The device can also be edited, maintenance notes applied, and historical job details viewed.    

  ![Device Added](../Images/Tutorial-DeviceManager-DeviceAdded.png)

3.  Optionally repeat step 2 selecting a shared pool instead for the device.  This will populate the Shared device pivot, making that device available for users to check out.  

