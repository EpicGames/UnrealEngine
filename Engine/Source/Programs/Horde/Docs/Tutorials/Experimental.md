[Horde](../../README.md) > Getting Started: Experimental

# Getting Started: Experimental

## Introduction

**Horde** provides an Experimental plugin used to research, develop and test new fucntionality. The functionality
specified within this plugin, while functionally stable, should not be used in a production environment.

The information specified below shows how to configure the Experimental plugin for use with the current
features and functionality.

## Prerequisites

* Horde Server installation (see [Getting Started: Install Horde](InstallHorde.md)).
* A working Slack token which will be used to send out notifications (see [Slack Integration](../Deployment/Integrations/Slack.md)).

## Steps

1. Configure the plugin to provide the path where the notifications will reside, which can be added by
   the following snippet to your [globals.json](../Config/Orientation.md) file:

    ```json
    // Define the location to the available notification configurations.
    "experimental": {
        "notifications": [
            {
                "id": "default",
                "include": [
                    {
                        "path": "$(HordeDir)/Defaults/default.notification.json"
                    }
                ]
            }
        ]
    },
    ```

2. Create the configuration file as specified by the path in the previous step. This config
   will include information regarding the streams the notifications will be applied to as well
   as the template and formats which will define the style and job information of the notification.

## See Also

* [Configuration > Experimental](../Config/Experimental.md)
