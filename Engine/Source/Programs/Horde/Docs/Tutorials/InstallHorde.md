[Horde](../../README.md) > Getting Started: Installing Horde

# Getting Started: Installing Horde

## Introduction

This guide describes a simple local Horde server installation on Windows.

Horde can also be installed via [Docker on Linux](../Deployment/Server.md#docker-linux), both as a single
instance and horizontally scaled service via a container orchestration system such as Kubernetes.

For a more detailed discussion of these advanced deployment scenarios, see [Horde > Deployment](../Deployment.md).

## Prerequisites

* A machine to function as the Horde Server.
* One or more machines to function as Horde Agents. We currently recommend dedicated machines for this purpose.

## Steps

### Horde Server

1. Install the Horde Server by running `Engine\Extras\Horde\UnrealHordeServer.msi`.
   * The Horde Server can also be deployed on [Linux using Docker](../Deployment/Server.md#docker-linux).
   * By default, Horde is configured to use [ports 13340 (HTTP) and 13342 (HTTP/2)](../Deployment/Server.md#ports).
     We recommend setting up [HTTPS](../Deployment/Server.md#https) for production deployments.
   * See also: [Deployment > Server](../Deployment/Server.md)

### Horde Agents

1. Navigate to the installed [Horde Server](#horde-server) in a web browser on the agent machine.
   * This is typically `http://{{ HOST_NAME_OR_IP_ADDRESS }}:13340` with a default installation.
   * Note that the Horde Server defaults to HTTP hosting by default (not HTTPS), so you may need
     to enter `http://` manually as part of the address.
2. Open the **Tools** menu at the top of the dashboard and select **Downloads**.
3. Download and run **Horde Agent (Windows Installer)**.
   * Enter the same server address you used above when prompted, and choose an empty working directory for
     the remote execution sandbox.
   * We recommend choosing a drive with at least 100GB of free space for C++ compilation.
4. Leave the `Enroll with Server` option checked at the end of the installation, or locate the Unreal icon
   in the system notification area, right-click on it, and select `Enroll with Server`.
5. Choose your agent from the list, and select **Enroll**. This process validates that you trust the agent
   and permits it to take on work.
