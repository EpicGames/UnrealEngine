[Horde](../../README.md) > Getting Started: Remote Compilation

# Getting Started: Remote Compilation

## Introduction

Horde implements a platform for generic remote execution workloads, allowing clients to leverage idle CPU cycles on
other machines to accelerate workloads that would otherwise be executed locally. With Horde's remote execution platform,
you can issue explicit commands to remote agents sequentially, such as "upload these files", "run this process",
"send these files back", and so on.

**Unreal Build Accelerator** is a tool that implements lightweight virtualization for
third-party programs (such as C++ compilers), allowing it to run on a remote machine - requesting information from the
initiating machine as required. The remotely executed process behaves as if it's executing
on the local machine, seeing the same view of the file system and so on, and files are transferred to and from the
remote machine behind the scenes as necessary.

**Unreal Build Tool** can use Unreal Build Accelerator with Horde to offload compilation tasks to connected agents,
spreading the workload over multiple machines.

## Prerequisites

* Horde Server and one or more Horde Agents (see [Getting Started: Install Horde](InstallHorde.md)).
* A workstation with a UE project under development.
* Network connectivity between your workstation and Horde Agents on port range 7000-7010.
* The default `Anonymous` authentication method enabled (see below for more details)

## Steps

1. On the machine initiating the build, ensure your UE project is synced and builds locally.

2. Configure UnrealBuildTool

  First setup your Uba.Provider settings, we recommend you do this in a file within perforce (so they can be shared for all users):
    * `Engine/Config/BaseEngine.ini` for settings to be used globally
    * `Config/DefaultEngine.ini` (within a project directory) for settings to be used for a specific project

  An example of how to configure the Uba.Provider settings can be found here:

    ```ini
    [Uba.Provider.Horde.Example]
	ServerUrl="[YOUR HORDE URL]"
	Pool=[YOUR HORDE POOL, COULD BE UNSET]
	Enabled=True
    ```

  Secondly configure the UbaControllers, this is done in a local file you need to manually create:
    * For Windows => `%LocalAppData%/Unreal Engine/Engine/Config/UserEngine.ini`
    * For Linux => `$HOME/.config/Epic/Unreal Engine/Engine/Config/UserEngine.ini`
    * For Mac => `$HOME/Library/Application Support/Epic/Unreal Engine/Engine/Config/UserEngine.ini`

  An example of how to configure this file can be found below:

	```ini
    [UbaController]
	+Providers=Uba.Provider.Horde.Example
	```

  Note, you can alternatively configure UnrealBuildTool within `Engine/Saved/UnrealBuildTool/BuildConfiguration.xml` but this is not recommended and is a legacy approach.

  An example of how to configure this file:

    ```xml
    <?xml version="1.0" encoding="utf-8" ?>
    <Configuration xmlns="https://www.unrealengine.com/BuildConfiguration">

        <BuildConfiguration>
            <!-- Enable support for UnrealBuildAccelerator -->
            <bAllowUBAExecutor>true</bAllowUBAExecutor>
        </BuildConfiguration>

        <Horde>
            <!-- Address of the Horde server -->
            <Server>http://{{ SERVER_HOST_NAME }}:13340</Server>

            <!-- Pool of machines to offload work to. Horde configures Win-UE5 by default. -->
            <WindowsPool>Win-UE5</WindowsPool>
        </Horde>

        <UnrealBuildAccelerator>
            <!-- Enable for visualizing UBA's progress (optional) -->
            <bLaunchVisualizer>true</bLaunchVisualizer>
        </UnrealBuildAccelerator>

    </Configuration>
    ```

   Replace `SERVER_HOST_NAME` with the address associated with your Horde server installation.

   * `BuildConfiguration.xml` can be sourced from many locations in the filesystem, depending on your preference,
     including locations typically under source control. See
     [Build Configuration](https://docs.unrealengine.com/en-US/build-configuration-for-unreal-engine/).
     in the UnrealBuildTool documentation for more details.

3. Compile your project through your IDE as normal. You should observe log lines such as:

   ```text
   [Worker0] Connected to AGENT-1 (10.0.10.172) under lease 65d48fe1eb6ff84c8197a9b0
   ...
   [17/5759] Compile [x64] Module.CoreUObject.2.cpp [RemoteExecutor: AGENT-1]
   ```

   This indicates work is being spread to multiple agents. If you enabled the UBA visualizer, you can also see
   a graphical overview of how the build progresses over multiple machines.

   For debugging and tuning purposes, it can be useful to force remote execution for all compile workloads. To do
   so, enable the following option in your `BuildConfiguration.xml` file or pass `-UBAForceRemote` on the
   UnrealBuildTool command line:

   ```xml
   <UnrealBuildAccelerator>
       <bForceBuildAllRemote>true</bForceBuildAllRemote>
   </UnrealBuildAccelerator>
   ```

> **Note:** It is not recommended to run a Horde Agent on the same machine as the Horde Server for performance reasons.

> **Note:** When using Horde's build automation functionality, be mindful of mixing pools of agents for UBA and 
  pools of agents for build automation. Agents used for build automation typically have higher requirements 
  and are a more scarce resource than compute helpers.

## Enabling authentication

When using the anonymous authentication mode, there's no authentication and every user has full access
to perform remote compilation. This is only recommended during a testing phase and you should switch to either
`Horde` or `OpenIdConnect` as soon as possible. Once you do that, additional permissions must be granted to the
so called compute cluster. By default, a cluster called `default` is already defined.

Horde jobs utilizing remote compilation are automatically granted access via an injected token that's set as
an environment variable for each job step (`UE_HORDE_TOKEN`).

Below is an updated global config with a `default` cluster and `AddComputeTasks` granted for letting UBT and UBA schedule remote compilation.
To see what claims are available to your users, open `/api/v1/user/claims` as a logged in user.

    ```json
    {
      // ...
      "plugins": {
        // ...
        "compute": {
          // ...
          "clusters": [
            {
              "id": "default",
              "namespaceid": "horde.compute",
              "acl": {
                "entries": [
                  {
                    "claim": { "type": "http://epicgames.com/ue/horde/user", "value": "jane.smith" },
                    "actions": ["AddComputeTasks"]
                  }
                ]
              }
            }
          ]
        }
      }
    }
    ```