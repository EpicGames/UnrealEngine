[Horde](../../README.md) > Getting Started: Remote Shader Compilation

# Getting Started: Remote Shader Compilation

## Introduction

In addition to remote C++ compilation, Unreal Engine supports compiling shaders remotely on Horde agents.
Since shader compilation is highly CPU-intensive, this capability can significantly accelerate the cooking process.

The following steps demonstrate how to configure Horde and Unreal Build Accelerator (UBA) functionality from the command line.
Once you've verified that it's working, this same functionality will be available from within the editor as well.

## Prerequisites

* Completed the guide described in [Getting Started: Remote Compilation](RemoteCompilation.md)
* A Windows workstation
* Lyra sample game for UE 5.5+ installed ([instructions](https://dev.epicgames.com/documentation/en-us/unreal-engine/lyra-sample-game-in-unreal-engine))
  * Note that the GitHub version requires additional files to be installed

## Steps

1. If building Lyra from source, ensure the editor is built:

   ```
   Engine\Build\BatchFiles\RunUBT.bat LyraEditor ShaderCompileWorker Win64 Development -Project=..\..\Samples\Games\Lyra\Lyra.uproject
   ```

2. Configure the Uba settings

   First setup your Uba.Provider settings, we recommend you do this in a file within perforce (so they can be shared for all users):
     * `Engine\Config\BaseEngine.ini` for settings to be used globally
     * `Samples\Games\Lyra\Config\DefaultEngine.ini` for settings to be used for a specific project (in this case Lyra)
  
   An example of how to configure the Uba.Provider settings can be found here:

   ```ini
   [Uba.Provider.Horde.Example]
   ServerUrl="http://{{ SERVER_HOST_NAME }}:13340/"
   Enabled=true
   Pool=[YOUR HORDE POOL, COULD BE UNSET]
   MaxCores=500
   ```

   `MaxCores=500` describes how many CPU cores this UBA session should try to allocate for work.
   Adjust as needed for fair use and size of your Horde setup.

   Secondly configure the UbaControllers, this is done in a local file you need to manually create:
     * For Windows => `%LocalAppData%/Unreal Engine/Engine/Config/UserEngine.ini`
     * For Linux => `$HOME/.config/Epic/Unreal Engine/Engine/Config/UserEngine.ini`
     * For Mac => `$HOME/Library/Application Support/Epic/Unreal Engine/Engine/Config/UserEngine.ini`

   An example of how to configure this file can be found below:

   ```ini
   [UbaController]
   +Providers=Uba.Provider.Horde.Example
   ```

3. Cook the data, which will compile shaders among many other asset transformations

   ```
   Engine\Binaries\Win64\UnrealEditor-Cmd.exe Samples\Games\Lyra\Lyra.uproject -run=cook -targetplatform=Windows -noshaderddc
   ```

   The last argument `-noshaderddc` ensures any shader cache lookups are skipped which is useful during testing or benchmarking UBA.

4. Following the log you should see shader compilation being started with Horde and UBA lines appearing:

   ```
   LogUbaHorde: Display: Created Horde bundle for: ../../../Engine/Binaries/Win64/UnrealBuildAccelerator/x64/UbaAgent.exe
   LogUbaHorde: Display: Getting Horde server URL succeeded [URL: http://{{ SERVER_HOST_NAME }}:13340/, Source: Engine INI configuration]
   LogDesktopPlatform: Display: Launching OidcToken... [Engine/Binaries/DotNET/OidcToken/win-x64/OidcToken.exe --HordeUrl="http://{{ SERVER_HOST_NAME }}:13340/" --OutFile="<some-path>" --Unattended=true]
   LogUbaHorde: Display: UBA Horde machine assigned (Linux) [192.168.2.156:7000]: http://{{ SERVER_HOST_NAME }}:13340/lease/670ff187af783212b5c15b6f
   LogUbaHorde: Display: UBA Horde machine assigned (Linux) [192.168.2.247:7000]: http://{{ SERVER_HOST_NAME }}:13340/lease/670ff187542cd63730a0a23b
   LogUbaHorde: Display: UBA Horde machine assigned (Linux) [192.168.2.132:7000]: http://{{ SERVER_HOST_NAME }}:13340/lease/670ff1886673800f993f56ac
   ```

   This indicates work is being sent to remote Horde agents and you can observe them from Horde server as well.
   <br><br>
   By default, authentication via OidcToken and a browser is performed the first time a developer triggers use of Horde and UBA.
   The [guide for remote C++ compilation](RemoteCompilation.md) covers this in more detail.
   When cooking through a Horde job, the `-Unattended` flag should be passed to avoid triggering any interactive dialogs or blocking UI.
   Every job includes an environment variable `UE_HORDE_TOKEN` which, when present, takes precedence over the default interactive authentication method. 

5. For debugging and tuning purposes, it can be useful to force remote execution for all compile workloads.
   To do so, add `bForceRemote=true` under `[UbaController]`


> **Tip:** Start UbaVisualizer tool and leave it running. It will automatically attach to new UBA sessions and
> display an overview of how compilations are scheduled. Located in `Engine\Binaries\Win64\UnrealBuildAccelerator\x64\UbaVisualizer.exe`.