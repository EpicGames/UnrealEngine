[Horde](../../README.md) > Getting Started: Self-Signed Certs

# Getting Started: Self-Signed Certs

## Introduction

When deploying Horde in production environments, it is recommended
to use a verified signing certificate.

For testing scenarios, if can be useful to install a self-signed
certificate.

> **Note:** Using self-signed certificates circumvents basic security
measures. Do not use this technique in production environments.

## Server

1. From an administrator PowerShell prompt, add a certificate to the "Personal" store for the local machine by running:

   ```pwsh
   New-SelfSignedCertificate -CertStoreLocation 'Cert:\LocalMachine\My' -DnsName 'my-domain.com'
   ```

2. Open the Certificate Manager MMC snap-in by running `certmgr.msc` from the Windows 'Run' menu. You should see the
certificate created above in the `Personal\Certificates` section.

   Select the certificate and press Ctrl+C. Navigate to the
   'Trusted Root Authorities\Certificates' section, and press Ctrl+V to create a copy.

3. Open the [server.json](../Config/Orientation.md) file, and uncomment the `HttpsPort` line:

    ```json
     "HttpsPort": 13341,
    ```

   ...as well as the certificate section at the bottom of the file -
   updating the subject name to the DNS name on the certificate created above.

   ```json
    "Kestrel":
    {
        "Certificates":
        {
            "Default":
            {
                "Subject": "my-domain.com",
                "Store": "My",
                "Location": "LocalMachine"
            }
        }
    }
    ```

4. Restart the server. You should be able to connect over HTTPS from a browser on the same machine on port 13341.

## Clients

1. Browse to the server specified above on the HTTPS URL. On the warning
   dialog about the server having an invalid certificate, choose to export
   it to a file.

   On **Google Chrome**, this can be accessed by clicking on the "Not Secure" button in the address bar, selecting
   "Certificate is not valid", switching to the "Details" tab in the certificate browse, and chosing "Export". Select
   'Base-64 Encoded ASCII' as the file type, and save the file.

   The certificate may also be exported directly from the Certificate
   Manager MMC snap-in.

2. Locate the exported certificate file in Windows Explorer, right click on it, and choose "Install Certificate". When
   prompted, choose to import the certificate in to the "Trusted Root Certificates" store.
