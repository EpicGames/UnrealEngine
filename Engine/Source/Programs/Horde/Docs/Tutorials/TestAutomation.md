[Horde](../../README.md) > Getting Started: Test Automation

# Getting Started: Test Automation

## Introduction

The Horde Automation Hub surfaces individual and suite [Gauntlet](https://docs.unrealengine.com/5.3/en-US/gauntlet-automation-framework-overview-in-unreal-engine) test results.  Horde efficiently generates searchable metadata for streams, platforms, configurations, rendering apis, etc.  The automation hub is used at Epic by QA, release managers, and code owners to quickly view and investigate the latest test results across platforms and streams.  It provides historical data and views which drill into specific test events which can include screenshots, logging, and callstacks.   

## Prerequisites

* The Horde Server installed (see [Getting Started: Install Horde](InstallHorde.md)).
* A Horde Agent with AutoSDK configured (see Engine\Extras\AutoSDK\README.md in the Engine sources)
* The Horde Build example project configured (see [Getting Started: Build Automation](BuildAutomation.md)).
* An Android phone or tablet added to the Device Manager (see [Getting Started: Device Manager](DeviceManager.md)).

## Steps

1. The Horde example UE5 project includes a reference template for building, packaging, and testing the Lyra example game project.  The build graph for this example is intended to be general purpose and extensible and is a good source of a real world graph used at Epic for automated testing.

    ![New Build](../Images/Tutorial-TestAutomation-NewBuild.png)

    From the UE5 project stream view, select the Packaged Build category, and click `New Build`, and then select `Packaged Lyra Build`. Add the Android target platform and click `Start Job`

    ![Select Android](../Images/Tutorial-TestAutomation-Android.png)

    Please note that the Device Manager URL field is for example purposes and you would generally set this in the related Gauntlet build graph configuration.

2. The job will build, cook, and run a Lyra Boot Test on the Android device, reserving it from the device manager during the process.

    ![Automation Labels](../Images/Tutorial-TestAutomation-Labels.png)

3. Once completed the test results will be availble in the [Automation Hub](../Config/TestAutomationHub.md) which features finely grained filters and views that can cross compare platforms and streams.

    ![Test Results](../Images/Tutorial-TestAutomation-TestResult.png)