// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using HordeCommon.Rpc.Tasks;
using JobDriver.Execution;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace JobDriver.Tests
{
	class AbortTestExecutor : JobExecutor
	{
		public const string Name = "Abort";

		private readonly Func<ILogger, CancellationToken, Task> _func;

		public AbortTestExecutor(Func<ILogger, CancellationToken, Task> func)
			: base(new JobExecutorOptions(null!, null!, [], default, default, default, new RpcJobOptions { Executor = Name, Container = new RpcJobContainerOptions()}), SimpleTestExecutor.NoOpTracer, NullLogger.Instance)
		{
			_func = func;
		}

		protected override Task ExecuteInternalAsync(ILogger logger, CancellationToken cancellationToken)
		{
			return _func(logger, cancellationToken);
		}

		protected override Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		protected override Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}
	}
}
