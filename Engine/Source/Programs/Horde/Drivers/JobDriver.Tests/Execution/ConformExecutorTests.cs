// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Perforce.Fixture;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using JobDriver.Execution;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;
using SyncOptions = JobDriver.Execution.SyncOptions;

namespace JobDriver.Tests.Execution;

[TestClass]
public class ConformTest : BasePerforceFixtureTest
{
	private static Tracer NoOpTracer { get; } = TracerProvider.Default.GetTracer("NoOp");

	private readonly ILogger _logger;
	private readonly IServiceProvider _serviceProvider;
	private readonly ManagedWorkspaceMaterializerFactory _mwMatFactory;
	private readonly PerforceMaterializerFactory _p4MatFactory;
	private readonly List<IWorkspaceMaterializerFactory> _materializerFactories;

	public ConformTest()
	{
		_logger = LoggerFactory.CreateLogger("ConformExecutor");
		ServiceCollection services = [];
		services.AddSingleton<Tracer>(NoOpTracer);
		services.AddLogging(builder => 
		{
			builder.AddConsole();
			builder.AddDebug();
			builder.SetMinimumLevel(LogLevel.Debug);
		});
		_serviceProvider = services.BuildServiceProvider();
		_mwMatFactory = new ManagedWorkspaceMaterializerFactory(_serviceProvider);
		_p4MatFactory = new PerforceMaterializerFactory(_serviceProvider);
		_materializerFactories = [_mwMatFactory, _p4MatFactory];
	}

	[TestMethod]
	public async Task ConformAsync()
	{
		RpcAgentWorkspace rawMw = CreateAgentWorkspace("mw");
		RpcAgentWorkspace rawPm = CreateAgentWorkspace("pm", "name=perforce");

		IWorkspaceMaterializer? mwm = await _mwMatFactory.CreateMaterializerAsync(ManagedWorkspaceMaterializer.TypeName, rawMw, TempDir, false, CancellationToken.None);
		IWorkspaceMaterializer? pm = await _p4MatFactory.CreateMaterializerAsync(PerforceMaterializer.TypeName, rawPm, TempDir, false, CancellationToken.None);
		Assert.IsNotNull(mwm);
		Assert.IsNotNull(pm);

		// Sync and verify workspaces to a random CL
		ChangelistFixture syncedCl = Fixture.StreamFooMain.GetChangelist(5);
		await mwm.SyncAsync(syncedCl.Number, -1, new SyncOptions(), CancellationToken.None);
		await pm.SyncAsync(syncedCl.Number, -1, new SyncOptions(), CancellationToken.None);
		syncedCl.AssertDepotFiles(mwm.SyncDir.FullName);
		syncedCl.AssertDepotFiles(pm.SyncDir.FullName);

		// Conform the workspaces, which also results in them syncing to latest CL of the stream
		List<RpcAgentWorkspace> workspaces = [rawMw, rawPm];
		ConformExecutor executor = CreateConformExecutor(workspaces);
		await executor.ExecuteAsync(CancellationToken.None);
		
		// Verify synced files (nothing missing/added, except what's at latest CL)
		ChangelistFixture conformedCl = Fixture.StreamFooMain.Head;
		conformedCl.AssertDepotFiles(mwm.SyncDir.FullName);
		conformedCl.AssertDepotFiles(pm.SyncDir.FullName);
	}

	private RpcAgentWorkspace CreateAgentWorkspace(string identifer, string method = "")
	{
		return new RpcAgentWorkspace
		{
			ServerAndPort = PerforceConnection.ServerAndPort,
			UserName = PerforceConnection.UserName,
			Password = PerforceConnection.Settings.Password,
			Identifier = identifer,
			Stream = Fixture.StreamFooMain.Root,
			MinScratchSpace = 1,
			Method = method
		};
	}

	private ConformExecutor CreateConformExecutor(List<RpcAgentWorkspace> workspaces)
	{
		ConformTask conformTask = new () { LogId = "222222222222222222222222", RemoveUntrackedFiles = false };
		conformTask.Workspaces.AddRange(workspaces);

		LeaseId leaseId = LeaseId.Parse("111111111111111111111111");
		JobRpcClientStub jobRpcClientStub = new (_logger);
		FakeHordeClient hordeClient = new (jobRpcClientStub);
		return new ConformExecutor(hordeClient, _materializerFactories, TempDir, new AgentId("agent1"), leaseId, conformTask,
			new DriverSettings(),
			NoOpTracer, _logger);
	}
}
