// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Metrics;

namespace JobDriver.Tests.Execution;

internal class FakeSystemMetrics : ISystemMetrics
{
	public long FreeSpace { get; init; }
	public long TotalSize { get; init; }

	/// <inheritdoc/>
	public CpuMetrics? GetCpu() => null;

	/// <inheritdoc/>
	public CpuTimeMetrics? GetCpuTime() => null;

	/// <inheritdoc/>
	public MemoryMetrics? GetMemory() => null;

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk()
	{
		return new DiskMetrics { FreeSpace = FreeSpace, TotalSize = TotalSize };
	}

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk()
	{
		return new DiskMetrics { FreeSpace = FreeSpace, TotalSize = TotalSize };
	}
}
