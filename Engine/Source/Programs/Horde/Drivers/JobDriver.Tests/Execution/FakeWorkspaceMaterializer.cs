// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using JobDriver.Execution;
using Microsoft.Extensions.Logging;

namespace JobDriver.Tests.Execution;

class FakeWorkspaceMaterializer : IWorkspaceMaterializer
{
	private readonly DirectoryReference _baseDir;
	private readonly DirectoryReference _syncDir;
	private readonly Dictionary<int, Dictionary<string, string>> _changeToFiles = new();

	public DirectoryReference SyncDir => _baseDir;
	public DirectoryReference BaseDir => _syncDir;

	public string Identifier => "fakeWorkspaceIdentifier";
	public string Name => "FakeWorkspaceMaterializer";
	public IReadOnlyDictionary<string, string> EnvironmentVariables { get; } = new Dictionary<string, string>();

	public bool IsPerforceWorkspace => false;
	public ILogger GetLogger(ILogger logger) => logger;

	public FakeWorkspaceMaterializer()
	{
		_baseDir = new DirectoryReference(Path.Join(Path.GetTempPath(), "horde-fakeworkspace-" + Guid.NewGuid().ToString()[..8]));
		_syncDir = DirectoryReference.Combine(_baseDir,"Sync");
		Directory.CreateDirectory(_syncDir.FullName);
	}

	public void SetFile(int changeNum, string path, string content)
	{
		if (path.Contains("..", StringComparison.Ordinal))
		{
			throw new ArgumentException("Cannot contain '..'");
		}
		if (path.Contains(':', StringComparison.Ordinal))
		{
			throw new ArgumentException("Cannot contain ':'");
		}
		path = path.Replace("\\", "/", StringComparison.Ordinal);

		if (!_changeToFiles.TryGetValue(changeNum, out Dictionary<string, string>? pathToContent))
		{
			pathToContent = new();
			_changeToFiles[changeNum] = pathToContent;
		}

		pathToContent[path] = content;
	}

	/// <inheritdoc/>
	public void Dispose()
	{
	}

	/// <inheritdoc/>
	public Task FinalizeAsync(CancellationToken cancellationToken)
	{
		if (Directory.Exists(_baseDir.FullName))
		{
			Directory.Delete(_baseDir.FullName, true);
		}

		return Task.CompletedTask;
	}

	public Task ConformAsync(bool removeUntrackedFiles, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	/// <inheritdoc/>
	public Task SyncAsync(int changeNum, int preflightChangeNum, SyncOptions options, CancellationToken cancellationToken)
	{
		if (changeNum == IWorkspaceMaterializer.LatestChangeNumber)
		{
			changeNum = _changeToFiles.Keys.Max();
		}

		if (!_changeToFiles.ContainsKey(changeNum))
		{
			throw new WorkspaceMaterializationException($"Change {changeNum} could not be found");
		}

		if (options.RemoveUntracked)
		{
			DeleteAllFiles(_baseDir.FullName);
		}

		WriteChangesToDisk(changeNum);

		if (preflightChangeNum > 0)
		{
			WriteChangesToDisk(preflightChangeNum);
		}

		return Task.CompletedTask;
	}

	private void WriteChangesToDisk(int changeNum)
	{
		foreach ((string relativeFilePath, string content) in _changeToFiles[changeNum])
		{
			string absFilePath = Path.GetFullPath(relativeFilePath, _baseDir!.FullName);
			string absDirPath = Path.GetDirectoryName(absFilePath)!;
			Directory.CreateDirectory(absDirPath);
			File.WriteAllText(absFilePath, content);
		}
	}

	private static void DeleteAllFiles(string dirPath)
	{
		DirectoryInfo di = new(dirPath);

		foreach (FileInfo file in di.EnumerateFiles())
		{
			file.Delete();
		}
		foreach (DirectoryInfo dir in di.EnumerateDirectories())
		{
			dir.Delete(true);
		}
	}
}