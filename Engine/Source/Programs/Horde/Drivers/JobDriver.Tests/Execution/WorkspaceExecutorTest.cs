// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using Grpc.Core;
using Grpc.Net.Client;
using JobDriver.Execution;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Tools;

namespace JobDriver.Tests.Execution
{
	internal class FakeHordeClient : IHordeClient
	{
		class FakeJobRpc : JobRpc.JobRpcClient
		{
		}

		private readonly JobRpc.JobRpcClient? _jobRpcClient = null;
		private readonly Dictionary<StreamId, RpcGetStreamResponse> _streamIdToStreamResponse = new();
		private readonly Dictionary<JobId, RpcGetJobResponse> _jobIdToJobResponse = new();

		public event Action? OnAccessTokenStateChanged
		{
			add { }
			remove { }
		}

		public void AddStream(StreamId streamId, string streamName)
		{
			if (_streamIdToStreamResponse.ContainsKey(streamId))
			{
				throw new Exception($"Stream ID {streamId} already added");
			}

			_streamIdToStreamResponse[streamId] = new RpcGetStreamResponse
			{
				Name = streamName,
			};
		}

		public void AddAgentType(StreamId streamId, string agentType)
		{
			if (!_streamIdToStreamResponse.TryGetValue(streamId, out RpcGetStreamResponse? streamResponse))
			{
				throw new Exception($"Stream ID {streamId} not found");
			}

			string tempDir = Path.Join(Path.GetTempPath(), $"horde-agent-type-{agentType}-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);

			streamResponse.AgentTypes[agentType] = new RpcGetAgentTypeResponse
			{
				TempStorageDir = tempDir
			};
		}

		public void AddJob(JobId jobId, StreamId streamId, int change, int preflightChange)
		{
			if (!_streamIdToStreamResponse.ContainsKey(streamId))
			{
				throw new Exception($"Stream ID {streamId} not found");
			}

			_jobIdToJobResponse[jobId] = new RpcGetJobResponse
			{
				StreamId = streamId.ToString(),
				Change = change,
				PreflightChange = preflightChange
			};
		}

		public FakeHordeClient(JobRpc.JobRpcClient? jobRpcClient = null)
		{
			_jobRpcClient = jobRpcClient;
		}

		public Uri ServerUrl => new ("http://fake-horde-server");
		public IArtifactCollection Artifacts => throw new NotImplementedException();
		public IComputeClient Compute => throw new NotImplementedException();
		public IProjectCollection Projects => throw new NotImplementedException();
		public ISecretCollection Secrets => throw new NotImplementedException();
		public IToolCollection Tools => throw new NotImplementedException();

		public IComputeClient CreateComputeClient() => throw new NotImplementedException();
		public Task<GrpcChannel> CreateGrpcChannelAsync(CancellationToken cancellationToken = default) => throw new NotImplementedException();
		public Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>
		{
			if (typeof(TClient) == typeof(JobRpc.JobRpcClient))
			{
				return _jobRpcClient != null
					? Task.FromResult((TClient)(object)_jobRpcClient)
					: Task.FromResult((TClient)(object)new FakeJobRpc());
			}
			throw new NotImplementedException();
		}

		public HordeHttpClient CreateHttpClient() => throw new NotImplementedException();
		public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information) => throw new NotImplementedException();
		public IStorageNamespace GetStorageNamespace(string relativePath, string? accessToken = null) => throw new NotImplementedException();
		public Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default) => Task.FromResult<string?>("access-token");
		public bool HasValidAccessToken() => true;
		public Task<bool> LoginAsync(bool allowPrompt, CancellationToken cancellationToken) => throw new NotImplementedException();
	}
	
	[TestClass]
	public sealed class WorkspaceExecutorTest : IDisposable
	{
		

		private readonly StreamId _streamId = new StreamId("foo-main");
		private readonly JobId _jobId = JobId.Parse("65bd0655591b5d5d7d047b58");
		private readonly JobStepBatchId _batchId = new JobStepBatchId(0x1234);
		private const string AgentType = "BogusAgentType";

		private readonly ILogger<WorkspaceExecutorTest> _logger;
		private readonly FakeHordeClient _hordeClient;
		private readonly DirectoryReference _workingDir = new DirectoryReference("workspace-test");
		private readonly FakeWorkspaceMaterializer _autoSdkWorkspace = new();
		private readonly FakeWorkspaceMaterializer _workspace = new();
		private readonly WorkspaceExecutor _executor;

		public WorkspaceExecutorTest()
		{
			using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
			{
				builder.SetMinimumLevel(LogLevel.Debug);
				builder.AddSimpleConsole(options => { options.SingleLine = true; });
			});

			_logger = loggerFactory.CreateLogger<WorkspaceExecutorTest>();

			_hordeClient = new FakeHordeClient();
			_hordeClient.AddStream(_streamId, "//Foo/Main");
			_hordeClient.AddAgentType(_streamId, AgentType);
			_hordeClient.AddJob(_jobId, _streamId, 1, 0);

			_autoSdkWorkspace.SetFile(1, "HostWin64/Android/base.h", "base");
			_workspace.SetFile(1, "main.cpp", "main");
			_workspace.SetFile(1, "foo/bar/baz.h", "baz");

			JobExecutorOptions executorOptions = new JobExecutorOptions(_hordeClient, _workingDir, null, _jobId, _batchId, default, new RpcJobOptions());
			_executor = new(executorOptions, _workspace, null, SimpleTestExecutor.NoOpTracer, NullLogger.Instance);
		}

		public void Dispose()
		{
			_executor.Dispose();
			_workspace.Dispose();
			_autoSdkWorkspace.Dispose();
		}

		[TestMethod]
		public async Task RegularWorkspaceAsync()
		{
			RpcBeginBatchResponse batch = new RpcBeginBatchResponse { Change = 1 };
			await _executor.InitializeAsync(batch, _logger, CancellationToken.None);
			AssertWorkspaceFile(_workspace, "main.cpp", "main");
			AssertWorkspaceFile(_workspace, "foo/bar/baz.h", "baz");
			await _executor.FinalizeAsync(_logger, CancellationToken.None);
		}

		[TestMethod]
		public async Task RegularAndAutoSdkWorkspaceAsync()
		{
			JobExecutorOptions executorOptions = new JobExecutorOptions(_hordeClient, _workingDir, Array.Empty<ProcessToTerminate>(), _jobId, _batchId, default, new RpcJobOptions());
			using WorkspaceExecutor executor = new(executorOptions, _workspace, _autoSdkWorkspace, SimpleTestExecutor.NoOpTracer, NullLogger.Instance);

			RpcBeginBatchResponse batch = new RpcBeginBatchResponse { Change = 1 };
			await executor.InitializeAsync(batch, _logger, CancellationToken.None);
			AssertWorkspaceFile(_autoSdkWorkspace, "HostWin64/Android/base.h", "base");
			AssertWorkspaceFile(_workspace, "main.cpp", "main");
			AssertWorkspaceFile(_workspace, "foo/bar/baz.h", "baz");
			await executor.FinalizeAsync(_logger, CancellationToken.None);
		}

		[TestMethod]
		public async Task EnvVarsAsync()
		{
			JobExecutorOptions executorOptions = new JobExecutorOptions(_hordeClient, _workingDir, Array.Empty<ProcessToTerminate>(), _jobId, _batchId, default, new RpcJobOptions());
			using WorkspaceExecutor executor = new(executorOptions, _workspace, _autoSdkWorkspace, SimpleTestExecutor.NoOpTracer, NullLogger.Instance);

			RpcBeginBatchResponse batch = new RpcBeginBatchResponse { Change = 1, StreamName = "//UE5/Main" };
			await executor.InitializeAsync(batch, _logger, CancellationToken.None);

			IReadOnlyDictionary<string, string> envVars = executor.GetEnvVars();
			Assert.AreEqual("1", envVars["IsBuildMachine"]);
			Assert.AreEqual(_workspace.SyncDir.FullName, envVars["uebp_LOCAL_ROOT"]);
			Assert.AreEqual(batch.StreamName, envVars["uebp_BuildRoot_P4"]);
			Assert.AreEqual("++UE5+Main", envVars["uebp_BuildRoot_Escaped"]);
			Assert.AreEqual("1", envVars["uebp_CL"]);
			Assert.AreEqual("0", envVars["uebp_CodeCL"]);
			Assert.AreEqual(_autoSdkWorkspace.SyncDir.FullName, envVars["UE_SDKS_ROOT"]);

			await executor.FinalizeAsync(_logger, CancellationToken.None);
		}

		[TestMethod]
		public async Task JobWithPreflightAsync()
		{
			JobId preflightJobId = JobId.Parse("65bd0655591b5d5d7d047b59");

			_hordeClient.AddJob(preflightJobId, _streamId, 1, 1000);
			_workspace.SetFile(1000, "New/Feature/Foo.cs", "foo");

			JobExecutorOptions executorOptions = new JobExecutorOptions(_hordeClient, _workingDir, Array.Empty<ProcessToTerminate>(), preflightJobId, _batchId, default, new RpcJobOptions());
			using WorkspaceExecutor executor = new(executorOptions, _workspace, null, SimpleTestExecutor.NoOpTracer, NullLogger.Instance);

			RpcBeginBatchResponse batch = new RpcBeginBatchResponse { Change = 1, PreflightChange = 1000 };
			await executor.InitializeAsync(batch, _logger, CancellationToken.None);
			AssertWorkspaceFile(_workspace, "main.cpp", "main");
			AssertWorkspaceFile(_workspace, "foo/bar/baz.h", "baz");
			AssertWorkspaceFile(_workspace, "New/Feature/Foo.cs", "foo");
			await executor.FinalizeAsync(_logger, CancellationToken.None);
		}

		[TestMethod]
		public async Task JobWithNoChangeAsync()
		{
			JobId noChangeJobId = JobId.Parse("65bd0655591b5d5d7d047b5a");

			_hordeClient.AddJob(noChangeJobId, _streamId, 0, 0);

			JobExecutorOptions executorOptions = new JobExecutorOptions(_hordeClient, _workingDir, Array.Empty<ProcessToTerminate>(), noChangeJobId, _batchId, default, new RpcJobOptions());
			using WorkspaceExecutor executor = new(executorOptions, _workspace, null, SimpleTestExecutor.NoOpTracer, NullLogger.Instance);

			RpcBeginBatchResponse batch = new RpcBeginBatchResponse { };
			await Assert.ThrowsExactlyAsync<WorkspaceMaterializationException>(() => executor.InitializeAsync(batch, _logger, CancellationToken.None));
		}

		private static void AssertWorkspaceFile(IWorkspaceMaterializer workspace, string relativePath, string expectedContent)
		{
			DirectoryReference workspaceDir = workspace.SyncDir;
			Assert.AreEqual(expectedContent, File.ReadAllText(Path.Join(workspaceDir.FullName, relativePath)));
		}
	}
}
