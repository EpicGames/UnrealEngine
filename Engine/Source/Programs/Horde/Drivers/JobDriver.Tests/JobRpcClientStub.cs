// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;

namespace JobDriver.Tests
{
	class JobRpcClientStub : JobRpc.JobRpcClient
	{
		public readonly Queue<RpcBeginStepResponse> BeginStepResponses = new Queue<RpcBeginStepResponse>();
		public readonly List<RpcUpdateStepRequest> UpdateStepRequests = new List<RpcUpdateStepRequest>();
		public readonly Dictionary<RpcGetStepRequest, RpcGetStepResponse> GetStepResponses = new Dictionary<RpcGetStepRequest, RpcGetStepResponse>();
		public Func<RpcGetStepRequest, RpcGetStepResponse>? _getStepFunc = null;
		private readonly ILogger _logger;

		public JobRpcClientStub(ILogger logger)
		{
			_logger = logger;
		}

		public override AsyncUnaryCall<RpcBeginBatchResponse> BeginBatchAsync(RpcBeginBatchRequest request,
			CallOptions options)
		{
			_logger.LogDebug("HordeRpcClientStub.BeginBatchAsync()");
			RpcBeginBatchResponse res = new RpcBeginBatchResponse();

			res.AgentType = "agentType1";
			res.LogId = "logId1";
			res.Change = 1;

			return Wrap(res);
		}

		public override AsyncUnaryCall<Empty> FinishBatchAsync(RpcFinishBatchRequest request, CallOptions options)
		{
			Empty res = new Empty();
			return Wrap(res);
		}

		public override AsyncUnaryCall<RpcGetJobResponse> GetJobAsync(RpcGetJobRequest request, CallOptions options)
		{
			RpcGetJobResponse res = new RpcGetJobResponse();
			return Wrap(res);
		}

		public override AsyncUnaryCall<RpcBeginStepResponse> BeginStepAsync(RpcBeginStepRequest request, CallOptions options)
		{
			if (BeginStepResponses.Count == 0)
			{
				RpcBeginStepResponse completeRes = new RpcBeginStepResponse();
				completeRes.State = RpcBeginStepResponse.Types.Result.Complete;
				return Wrap(completeRes);
			}

			RpcBeginStepResponse res = BeginStepResponses.Dequeue();
			res.State = RpcBeginStepResponse.Types.Result.Ready;
			return Wrap(res);
		}

		public override AsyncUnaryCall<Empty> UpdateStepAsync(RpcUpdateStepRequest request, CallOptions options)
		{
			_logger.LogDebug("UpdateStepAsync(Request: {Request})", request);
			UpdateStepRequests.Add(request);
			Empty res = new Empty();
			return Wrap(res);
		}

		[Obsolete("Use LogRpc.CreateLogEvents instead")]
		public override AsyncUnaryCall<Empty> CreateEventsAsync(RpcCreateLogEventsRequest request, CallOptions options)
		{
			_logger.LogDebug("CreateEventsAsync: {Request}", request);
			Empty res = new Empty();
			return Wrap(res);
		}

		public override AsyncUnaryCall<RpcGetStepResponse> GetStepAsync(RpcGetStepRequest request, CallOptions options)
		{
			if (_getStepFunc != null)
			{
				return Wrap(_getStepFunc(request));
			}

			if (GetStepResponses.TryGetValue(request, out RpcGetStepResponse? res))
			{
				return Wrap(res);
			}

			return Wrap(new RpcGetStepResponse());
		}

		public override AsyncUnaryCall<RpcUpdateAgentWorkspacesResponse> UpdateAgentWorkspacesAsync(RpcUpdateAgentWorkspacesRequest request, CallOptions options)
		{
			RpcUpdateAgentWorkspacesResponse res = new () { Retry = false };
			return Wrap(res);
		}

		public static AsyncUnaryCall<T> Wrap<T>(T res)
		{
			return new AsyncUnaryCall<T>(Task.FromResult(res), Task.FromResult(Metadata.Empty),
				() => Status.DefaultSuccess, () => Metadata.Empty, null!);
		}
	}
}
