// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce.Fixture;
using HordeCommon.Rpc.Messages;
using JobDriver.Execution;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;
using SyncOptions = JobDriver.Execution.SyncOptions;

namespace JobDriver.Tests;

[TestClass]
public class ManagedWorkspaceMaterializerTests : BasePerforceFixtureTest
{
	private static Tracer NoOpTracer { get; } = TracerProvider.Default.GetTracer("NoOp");

	[TestMethod]
	[DataRow(true, DisplayName = "Get logger before sync")]
	[DataRow(false, DisplayName = "Get logger after sync")]
	public async Task Logger_RelativeDepotPath_Async(bool getLoggerBeforeSync)
	{
		using ManagedWorkspaceMaterializer pm = await CreateMwmAsync(PerforceSnapshot.FooMain);
		TestLogger baseLogger = new();
		ILogger materializerLogger;
		
		if (getLoggerBeforeSync)
		{
			materializerLogger = pm.GetLogger(baseLogger);
			await pm.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, new SyncOptions(), CancellationToken.None);
		}
		else
		{
			await pm.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, new SyncOptions(), CancellationToken.None);
			materializerLogger = pm.GetLogger(baseLogger);
		}

		Assert.AreNotEqual(baseLogger, materializerLogger);
		FileReference source = FileReference.Combine(pm.SyncDir, "Sub1", "Sub2", "MyFile.txt");
		LogValue fileValue = LogValue.SourceFile(source, source.GetFileName());
		materializerLogger.LogError(KnownLogEvents.Compiler, "Testing path resolution {File}", fileValue);
		Assert.AreEqual(1, baseLogger.Entries.Count);

		if (baseLogger.Entries[0].State is JsonLogEvent jle)
		{
			JsonDocument document = JsonDocument.Parse(jle.Data);
			JsonElement fileElement = document.RootElement.GetProperty("properties").GetProperty("File");
			Assert.AreEqual("MyFile.txt", fileElement.GetProperty("$text").GetString());
			Assert.AreEqual("Sub1/Sub2/MyFile.txt", fileElement.GetProperty("relativePath").GetString());
			Assert.AreEqual("//Foo/Main/Sub1/Sub2/MyFile.txt@9", fileElement.GetProperty("depotPath").GetString());
		}
		else
		{
			Assert.Fail("Unable to find JsonLogEvent");
		}
	}

	private async Task<ManagedWorkspaceMaterializer> CreateMwmAsync(string stream, string? dataDir = null, string? identifier = "mwmTest", string[]? view = null)
	{
		ILogger logger = LoggerFactory.CreateLogger("MWM");
		RpcAgentWorkspace raw = new ()
		{
			ServerAndPort = PerforceConnection.ServerAndPort,
			UserName = PerforceConnection.UserName,
			Password = PerforceConnection.Settings.Password,
			Identifier = $"{TestGuid}-{identifier}",
			Stream = stream,
			MinScratchSpace = 1
		};
		raw.View.AddRange(view ?? []);
		return await ManagedWorkspaceMaterializer.CreateAsync(raw, dataDir is null ? TempDir : new DirectoryReference(dataDir), false, true, NoOpTracer, logger, CancellationToken.None);
	}
}

