// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Fixture;
using HordeCommon.Rpc.Messages;
using JobDriver.Execution;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;
using SyncOptions = JobDriver.Execution.SyncOptions;

namespace JobDriver.Tests;

public record TestLoggerEntry(LogLevel LogLevel, EventId EventId, object? State);
public class TestLogger : ILogger
{
	public List<TestLoggerEntry> Entries { get; } = [];
	public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
	{
		Entries.Add(new TestLoggerEntry(logLevel, eventId, state));
	}

	public bool IsEnabled(LogLevel logLevel) => true;
	public IDisposable? BeginScope<TState>(TState state) where TState : notnull => null;
}

/// <summary>
/// Logger wrapper that captures formatted messages while delegating to an inner logger.
/// </summary>
public class LogCapture : ILogger
{
	private readonly ILogger _inner;
	public List<string> Messages { get; } = [];

	public LogCapture(ILogger inner) => _inner = inner;

	public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
	{
		Messages.Add(formatter(state, exception));
		_inner.Log(logLevel, eventId, state, exception, formatter);
	}

	public bool IsEnabled(LogLevel logLevel) => _inner.IsEnabled(logLevel);
	public IDisposable? BeginScope<TState>(TState state) where TState : notnull => _inner.BeginScope(state);
}

[TestClass]
public class PerforceMaterializerTests : BasePerforceFixtureTest
{
	private static Tracer NoOpTracer { get; } = TracerProvider.Default.GetTracer("NoOp");
	private static int s_idCounter = 0;
	private readonly (string clientFile, long size, string digest, string content) _untrackedFile = ("random.txt", 11, "3E25960A79DBC69B674CD4EC67A72C62", "Hello world");

	[TestMethod]
	public async Task WalkChangeListsWithReusedMaterializerAsync()
	{
		CancellationToken ct = CancellationToken.None;
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		List<WorkspaceState> states = new(PerforceSnapshot.FooMainStates);
		states.AddRange(states.SkipLast(1).Reverse());
		foreach (WorkspaceState ss in states)
		{
			await pm.SyncAsync(ss.Cl, ss.ShelvedCl, new SyncOptions() { RemoveUntracked = false }, ct);
			PerforceSnapshot.GetSnapshot().GetStream(ss.Stream).GetWorkspace(ss.Cl, ss.ShelvedCl).AssertDepotFiles(pm.SyncDir.FullName);
			await pm.FinalizeAsync(ct);
		}
	}

	[TestMethod]
	[Ignore] // Disabled for now as it mutates the stream. Need to obliterate after submit, or similar kind of restore
	public async Task Submit_Async()
	{
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		
		await pm.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, new SyncOptions(), CancellationToken.None);
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(7).AssertDepotFiles(pm.SyncDir.FullName);
		await SubmitFileAsync(pm, "Adding new file", "hello.txt", "Hello world!");
		await pm.FinalizeAsync(CancellationToken.None);
	}

	[TestMethod]
	[DataRow(true, DisplayName = "Get logger before sync")]
	[DataRow(false, DisplayName = "Get logger after sync")]
	public async Task Logger_RelativeDepotPath_Async(bool getLoggerBeforeSync)
	{
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		TestLogger baseLogger = new();
		ILogger materializerLogger;
		
		if (getLoggerBeforeSync)
		{
			materializerLogger = pm.GetLogger(baseLogger);
			await pm.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, new SyncOptions(), CancellationToken.None);
		}
		else
		{
			await pm.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, new SyncOptions(), CancellationToken.None);
			materializerLogger = pm.GetLogger(baseLogger);
		}

		Assert.AreNotEqual(baseLogger, materializerLogger);
		FileReference source = FileReference.Combine(pm.SyncDir, "Sub1", "Sub2", "MyFile.txt");
		LogValue fileValue = LogValue.SourceFile(source, source.GetFileName());
		materializerLogger.LogError(KnownLogEvents.Compiler, "Testing path resolution {File}", fileValue);
		Assert.AreEqual(1, baseLogger.Entries.Count);

		if (baseLogger.Entries[0].State is JsonLogEvent jle)
		{
			JsonDocument document = JsonDocument.Parse(jle.Data);
			JsonElement fileElement = document.RootElement.GetProperty("properties").GetProperty("File");
			Assert.AreEqual("MyFile.txt", fileElement.GetProperty("$text").GetString());
			Assert.AreEqual("Sub1/Sub2/MyFile.txt", fileElement.GetProperty("relativePath").GetString());
			Assert.AreEqual("//Foo/Main/Sub1/Sub2/MyFile.txt@9", fileElement.GetProperty("depotPath").GetString());
		}
		else
		{
			Assert.Fail("Unable to find JsonLogEvent");
		}
	}

	[TestMethod]
	[DataRow(PerforceSnapshot.FooMain, IWorkspaceMaterializer.LatestChangeNumber, 9)]
	[DataRow(PerforceSnapshot.FooMain, IWorkspaceMaterializer.LatestChangeNumber, 9, true)]
	[DataRow(PerforceSnapshot.FooMainSparseDev, IWorkspaceMaterializer.LatestChangeNumber, 9)]
	[DataRow(PerforceSnapshot.FooMainSparseDev, 13, 13)]
	public async Task Sync_Async(string streamName, int changeNum, int expectedChangeNum, bool syncTwice = false)
	{
		using PerforceMaterializer pm = CreatePm(streamName);
		
		await pm.SyncAsync(changeNum, -1, new SyncOptions(), CancellationToken.None);
		if (syncTwice)
		{
			await pm.SyncAsync(changeNum, -1, new SyncOptions(), CancellationToken.None);
		}
		PerforceSnapshot.GetSnapshot().GetStream(streamName).GetWorkspace(expectedChangeNum).AssertDepotFiles(pm.SyncDir.FullName);
		await pm.FinalizeAsync(CancellationToken.None);

		PerforceMaterializer.State? state = await pm.LoadStateForTestAsync(CancellationToken.None);
		Assert.AreEqual(expectedChangeNum, state!.Changelist);
	}

	[TestMethod]
	public async Task Sync_RemoveUntracked_Async()
	{
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		WorkspaceState ws = PerforceSnapshot.Main(2) with
		{
			RemoveUntracked = true,
			OnPostSync = () => SimulateWorkspaceChanges(pm.SyncDir.FullName)
		};
		await SyncToStateAsync(pm, ws, CancellationToken.None);
	}
	
	[TestMethod]
	public async Task Sync_KeepUntracked_Async()
	{
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		string addedFile = $"{pm.SyncDir.FullName}/Data/added.txt";
		WorkspaceState ws = PerforceSnapshot.Main(2) with
		{
			RemoveUntracked = false,
			OnPostSync = () => { File.WriteAllText(addedFile, "New file"); }
		};
		await SyncToStateAsync(pm, ws, CancellationToken.None);
		
		// Verify file is *not* cleaned up by materializer
		Assert.AreEqual("New file", await File.ReadAllTextAsync(addedFile));
	}
	
	[TestMethod]
	public async Task Conform_RemoveUntracked_Async()
	{
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		await pm.SyncAsync(2, -1, new SyncOptions(), CancellationToken.None);
		SimulateWorkspaceChanges(pm.SyncDir.FullName);
		
		await pm.ConformAsync(true, CancellationToken.None);
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(9).AssertDepotFiles(pm.SyncDir.FullName);
	}
	
	[TestMethod]
	public async Task Conform_KeepUntracked_Async()
	{
		// Arrange
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		await pm.SyncAsync(5, -1, new SyncOptions(), CancellationToken.None);
		await File.WriteAllTextAsync(Path.Join(pm.SyncDir.FullName, _untrackedFile.clientFile), _untrackedFile.content);
		
		// Act
		await pm.ConformAsync(false, CancellationToken.None);
		
		// Assert
		HashSet<WorkspaceFile> expectedWorkspaceFiles = new (PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(9).Files); // Copy set
		HashSet<WorkspaceFile> actualLocalFiles = PerforceSnapshot.GetLocalFileSet(pm.SyncDir.FullName);
		expectedWorkspaceFiles.Add(WorkspaceFile.Create(_untrackedFile.clientFile, _untrackedFile.content));
		WorkspaceSnapshot.AssertFileSets(actualLocalFiles, expectedWorkspaceFiles);
	}
	
	[TestMethod]
	[DataRow(true, DisplayName = "Remove untracked files")]
	[DataRow(false, DisplayName = "Keep untracked files")]
	public async Task Conform_EndToEnd_Async(bool removeUntrackedFiles )
	{
		string identifier = "end-to-end";
		RpcAgentWorkspace raw = new ()
		{
			ServerAndPort = PerforceConnection.ServerAndPort,
			UserName = PerforceConnection.UserName,
			Password = PerforceConnection.Settings.Password,
			Identifier = $"{TestGuid}-{identifier}",
			Stream = PerforceSnapshot.FooMain,
			Method = "name=perforce"
		};
		
		using PerforceMaterializer pm = CreatePm(raw.Stream, dataDir: Path.Join(TempDir.FullName, raw.Identifier), identifier: raw.Identifier);
		await pm.SyncAsync(2, -1, new SyncOptions(), CancellationToken.None);
		SimulateWorkspaceChanges(pm.SyncDir.FullName);
		
		ILogger logger = LoggerFactory.CreateLogger("PM");
		ServiceCollection services = [];
		services.AddSingleton<ILogger<PerforceMaterializer>>(x => LoggerFactory.CreateLogger<PerforceMaterializer>());
		services.AddSingleton<Tracer>(x => NoOpTracer);
		IServiceProvider serviceProvider = services.BuildServiceProvider();
		List<IWorkspaceMaterializerFactory> factories = [new PerforceMaterializerFactory(serviceProvider)];
		
		await PerforceExecutor.ConformMaterializersAsync(factories, TempDir, [raw], removeUntrackedFiles, [], NoOpTracer, logger, CancellationToken.None);
	}

	[TestMethod]
	public async Task Conform_WithDeletedClient_ShouldStillClean_Async()
	{
		// Arrange: Sync first, then delete the client, simulating a stale/deleted client
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);
		await pm.SyncAsync(2, -1, new SyncOptions(), CancellationToken.None);
		await pm.FinalizeAsync(CancellationToken.None);

		// Delete the client to simulate a stale client scenario
		await pm.DeleteClientForTestAsync(CancellationToken.None);
		
		// Pollute the workspace with an untracked file
		await File.WriteAllTextAsync(Path.Join(pm.SyncDir.FullName, _untrackedFile.clientFile), _untrackedFile.content);

		// Act: Conform with removeUntrackedFiles=true (which triggers CleanAsync)
		// This should recreate the client and successfully clean
		await pm.ConformAsync(true, CancellationToken.None);

		// Assert: Workspace should be valid
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(9).AssertDepotFiles(pm.SyncDir.FullName);
	}

	[TestMethod]
	public async Task Sync_WithChangedClientName_ShouldFlushNotWipe_Async()
	{
		CancellationToken ct = CancellationToken.None;
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);

		// Step 1: Sync to CL 2 and finalize so state is saved to disk
		await pm.SyncAsync(2, -1, new SyncOptions { RemoveUntracked = false }, ct);
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(2).AssertDepotFiles(pm.SyncDir.FullName);
		await pm.FinalizeAsync(ct);

		// Step 2: Write a marker file into the workspace directory
		string markerPath = Path.Join(pm.SyncDir.FullName, "_edge_test_marker.txt");
		await File.WriteAllTextAsync(markerPath, "marker", ct);
		Assert.IsTrue(File.Exists(markerPath));

		// Step 3: Tamper with the client name in State.json to simulate an edge server switch
		PerforceMaterializer.State? state = await pm.LoadStateForTestAsync(ct);
		Assert.IsNotNull(state);
		state = state with { Client = "Horde+PM+FAKE+id+fakeEdge" };
		string stateFilePath = Path.Join(pm.BaseDir.FullName, "State.json");
		await File.WriteAllTextAsync(stateFilePath, JsonSerializer.Serialize(state, new JsonSerializerOptions { WriteIndented = true }), ct);

		// Step 4: Sync to CL 5  - edge server switch should flush, not wipe
		await pm.SyncAsync(5, -1, new SyncOptions { RemoveUntracked = false }, ct);

		// Step 5: Assert the marker file still exists (workspace was NOT wiped, only flushed)
		Assert.IsTrue(File.Exists(markerPath));
	}

	[TestMethod]
	public async Task Sync_WithChangedClientNameAndDirtyFiles_ShouldRevertAndSync_Async()
	{
		CancellationToken ct = CancellationToken.None;
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);

		// Sync to CL 2 and finalize so state is saved to disk
		await pm.SyncAsync(2, -1, new SyncOptions { RemoveUntracked = false }, ct);
		await pm.FinalizeAsync(ct);

		// Write a marker file to later verify the workspace was not wiped
		string markerPath = Path.Join(pm.SyncDir.FullName, "_edge_dirty_marker.txt");
		await File.WriteAllTextAsync(markerPath, "marker", ct);

		// Tamper with State.json to simulate an edge server switch with leftover DirtyFiles
		PerforceMaterializer.State? state = await pm.LoadStateForTestAsync(ct);
		Assert.IsNotNull(state);
		string oldClientName = "Horde+PM+FAKE+id+fakeEdge";
		string relativePath = PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(2).Files.First().Path;
		state = state with
		{
			Client = oldClientName,
			Status = PerforceMaterializer.TransactionStatus.Clean,
			DirtyFiles = [new PerforceMaterializer.WorkspaceFile($"//{oldClientName}/{relativePath}", Path.Join(pm.SyncDir.FullName, relativePath))]
		};
		string stateFilePath = Path.Join(pm.BaseDir.FullName, "State.json");
		await File.WriteAllTextAsync(stateFilePath, JsonSerializer.Serialize(state, new JsonSerializerOptions { WriteIndented = true }), ct);

		// Sync to CL 5 - should not crash; the fix remaps DirtyFiles to the current client
		await pm.SyncAsync(5, -1, new SyncOptions { RemoveUntracked = false }, ct);

		// Marker file should still exist (workspace was not wiped)
		Assert.IsTrue(File.Exists(markerPath));
	}

	[TestMethod]
	public async Task Sync_WithChangedClientNameAndStream_ShouldWipe_Async()
	{
		CancellationToken ct = CancellationToken.None;
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain);

		// Step 1: Sync to CL 2 and finalize so state is saved to disk
		await pm.SyncAsync(2, -1, new SyncOptions { RemoveUntracked = false }, ct);
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(2).AssertDepotFiles(pm.SyncDir.FullName);
		await pm.FinalizeAsync(ct);

		// Step 2: Write a marker file into the workspace directory
		string markerPath = Path.Join(pm.SyncDir.FullName, "_edge_test_marker.txt");
		await File.WriteAllTextAsync(markerPath, "marker", ct);
		Assert.IsTrue(File.Exists(markerPath));

		// Step 3: Tamper with both client name AND stream to simulate a genuine workspace change
		PerforceMaterializer.State? state = await pm.LoadStateForTestAsync(ct);
		Assert.IsNotNull(state);
		state = state with { Client = "Horde+PM+FAKE+id+fakeEdge", Stream = "//Foo/OtherStream" };
		string stateFilePath = Path.Join(pm.BaseDir.FullName, "State.json");
		await File.WriteAllTextAsync(stateFilePath, JsonSerializer.Serialize(state, new JsonSerializerOptions { WriteIndented = true }), ct);

		// Step 4: Sync  - the combined changes should trigger isDirty and wipe the workspace
		await pm.SyncAsync(5, -1, new SyncOptions { RemoveUntracked = false }, ct);
		PerforceSnapshot.GetSnapshot().GetStream(PerforceSnapshot.FooMain).GetWorkspace(5).AssertDepotFiles(pm.SyncDir.FullName);

		// Step 5: Assert the marker file was deleted (workspace was wiped because isDirty was true)
		Assert.IsFalse(File.Exists(markerPath));
	}

	[TestMethod]
	[DataRow(true, DisplayName = "Fresh sync")]
	[DataRow(false, DisplayName = "Subsequent sync")]
	public async Task Finalize_AlwaysRunsClean_Async(bool isFreshSync)
	{
		// FinalizeAsync should always run CleanAsync when RemoveUntracked=true,
		// even on a fresh sync, because the job/build step runs between SyncAsync and FinalizeAsync and may modify the workspace.
		LogCapture log = new(LoggerFactory.CreateLogger("PM"));
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain, logger: log);

		if (!isFreshSync)
		{
			await pm.SyncAsync(2, -1, new SyncOptions { RemoveUntracked = true }, CancellationToken.None);
			await pm.FinalizeAsync(CancellationToken.None);
		}

		await pm.SyncAsync(5, -1, new SyncOptions { RemoveUntracked = true }, CancellationToken.None);
		log.Messages.Clear();

		await pm.FinalizeAsync(CancellationToken.None);

		Assert.IsTrue(log.Messages.Any(m => m.Contains("Cleaning files", StringComparison.Ordinal)));
	}

	[TestMethod]
	public async Task Conform_FreshState_SkipsClean_Async()
	{
		// ConformAsync with no prior state (fresh) should skip CleanAsync even when removeUntrackedFiles=true.
		LogCapture log = new(LoggerFactory.CreateLogger("PM"));
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain, logger: log);
		log.Messages.Clear();

		await pm.ConformAsync(true, CancellationToken.None);

		Assert.IsFalse(log.Messages.Any(m => m.Contains("Cleaning files", StringComparison.Ordinal)));
	}

	[TestMethod]
	public async Task Conform_ExistingState_RunsClean_Async()
	{
		// ConformAsync with existing state should run CleanAsync when removeUntrackedFiles=true.
		LogCapture log = new(LoggerFactory.CreateLogger("PM"));
		using PerforceMaterializer pm = CreatePm(PerforceSnapshot.FooMain, logger: log);

		// Establish state with a first sync+finalize
		await pm.SyncAsync(2, -1, new SyncOptions(), CancellationToken.None);
		await pm.FinalizeAsync(CancellationToken.None);
		log.Messages.Clear();

		await pm.ConformAsync(true, CancellationToken.None);

		Assert.IsTrue(log.Messages.Any(m => m.Contains("Cleaning files", StringComparison.Ordinal)));
	}

	[TestMethod]
	[DataRow("Platforms/...", "//Foo/Main/Platforms/... //MyComputer/Platforms/...")]
	[DataRow(" Platforms/...", "//Foo/Main/Platforms/... //MyComputer/Platforms/...")]
	[DataRow("/Engine/Build/...", "//Foo/Main/Engine/Build/... //MyComputer/Engine/Build/...")]
	[DataRow("-/Engine/Docs/...", "-//Foo/Main/Engine/Docs/... //MyComputer/Engine/Docs/...")]
	[DataRow("   -/Engine/Docs/...", "-//Foo/Main/Engine/Docs/... //MyComputer/Engine/Docs/...")]
	public void ConvertHordeViews(string hordeView, string expected)
	{
		string actual = PerforceMaterializer.ConvertHordeViewToClientView(hordeView, "//Foo/Main", "MyComputer");
		Assert.AreEqual(expected, actual);
	}

	[TestMethod]
	public async Task TestAllStatesAsync()
	{
		List<WorkspaceState> mainStates = new(PerforceSnapshot.FooMainStates);
		List<WorkspaceState> releaseStates = new(PerforceSnapshot.FooReleaseStates);
		
		mainStates = mainStates.Select(x => x with { RemoveUntracked = true }).ToList();
		releaseStates = releaseStates.Select(x => x with { RemoveUntracked = true }).ToList();
		
		WorkspaceState?[] localStates = [null, ..mainStates, ..releaseStates];
		WorkspaceState[] targetStates = [..mainStates, ..releaseStates];

		foreach (WorkspaceState? localState in localStates)
		{
			// Server state can only be empty or match the local state for now
			// This could theoretically differ, so checking as "serverStates = [..localStates]" would be more robust
			// But requires more (slightly expensive) checks 
			WorkspaceState?[] serverStates = [null, localState];
			foreach (WorkspaceState? serverState in serverStates)
			{
				foreach (WorkspaceState targetState in targetStates)
				{
					await SyncWithScenarioAsync(localState, serverState, targetState, CancellationToken.None);
				}
			}
		}
	}

	private record SyncScenario(WorkspaceState? LocalState, WorkspaceState? ServerState, WorkspaceState TargetState);

	private Task SyncWithScenarioAsync(WorkspaceState? localState, WorkspaceState? serverState, WorkspaceState targetState, CancellationToken cancellationToken)
	{
		return SyncWithScenarioAsync(new SyncScenario(localState, serverState, targetState), cancellationToken);
	}

	private async Task SyncWithScenarioAsync(SyncScenario scenario, CancellationToken cancellationToken)
	{
		RemoveTempDir();
		CreateTempDir();
		string id = $"pmTest-{s_idCounter++}";
        
		if (scenario.LocalState != null)
		{
			// Initialize a local state with files corresponding to given stream state, but delete the client
			using PerforceMaterializer pm = CreatePm(scenario.LocalState.Stream, identifier: id);
			await SyncToStateAsync(pm, scenario.LocalState, cancellationToken);
			await pm.DeleteClientForTestAsync(cancellationToken);
		}
		
		if (scenario.ServerState != null)
		{
			string stateDir = TempDir.FullName;
			string backupDir = TempDir.FullName + "-backup";
			if (Directory.Exists(stateDir))
			{
				// Save the original local state so the sync below cannot overwrite
				Directory.Move(stateDir, backupDir);
			}
			
			using PerforceMaterializer pm = CreatePm(scenario.ServerState.Stream, identifier: id);
			await SyncToStateAsync(pm, scenario.ServerState, cancellationToken);

			// Now that the workspace has been synced, reset the local files back
			DeletePerforceDir(stateDir);
			if (Directory.Exists(backupDir))
			{
				Directory.Move(backupDir, stateDir);
			}
		}

		{
			Console.WriteLine($" Local state: {scenario.LocalState}");
			Console.WriteLine($"Server state: {scenario.ServerState}");
			Console.WriteLine($"Target state: {scenario.TargetState}");
			Console.WriteLine("Syncing ...");
			using PerforceMaterializer pm = CreatePm(scenario.TargetState.Stream, identifier: id);
			await SyncToStateAsync(pm, scenario.TargetState, cancellationToken);
		}
	}

	private static async Task SyncToStateAsync(PerforceMaterializer pm, WorkspaceState state, CancellationToken cancellationToken)
	{
		state.OnPreSync?.Invoke();
		await pm.SyncAsync(state.Cl, state.ShelvedCl, new SyncOptions { RemoveUntracked = state.RemoveUntracked }, cancellationToken);
		PerforceSnapshot.GetSnapshot().GetStream(state.Stream).GetWorkspace(state.Cl, state.ShelvedCl).AssertDepotFiles(pm.SyncDir.FullName);

		state.OnPostSync?.Invoke(); // Perform imaginary file operations to workspace
		await pm.FinalizeAsync(cancellationToken);

		if (state.RemoveUntracked)
		{
			// Only verify files on disk if they are expected to be cleaned up.
			// Note that shelved CL is not checked for as state file/have-list does not represent that.
			// Materializer simply p4 prints those. 
			PerforceSnapshot.GetSnapshot().GetStream(state.Stream).GetWorkspace(state.Cl).AssertDepotFiles(pm.SyncDir.FullName);	
		}
		
		PerforceMaterializer.State? pmState = await pm.LoadStateForTestAsync(cancellationToken);
		Assert.AreEqual(state.Cl, pmState!.Changelist);
		Assert.AreEqual(state.ShelvedCl, pmState!.ShelvedChangelist);
		Assert.AreEqual(state.Stream, pmState.Stream);
		Assert.AreEqual(PerforceMaterializer.TransactionStatus.Clean, pmState.Status);
	}

	private PerforceMaterializer CreatePm(string stream, string? dataDir = null, string? identifier = "pmTest", string[]? view = null, ILogger? logger = null)
	{
		logger ??= LoggerFactory.CreateLogger("PM");
		RpcAgentWorkspace raw = new ()
		{
			ServerAndPort = PerforceConnection.ServerAndPort,
			UserName = PerforceConnection.UserName,
			Password = PerforceConnection.Settings.Password,
			Identifier = $"{TestGuid}-{identifier}",
			Stream = stream,
		};
		raw.View.AddRange(view ?? []);
		PerforceMaterializerOptions options = new(dataDir ?? TempDir.FullName, raw);
		return new PerforceMaterializer(options, NoOpTracer, logger);
	}
	
	private static async Task SubmitFileAsync(PerforceMaterializer pm, string clDescription, string filename, string fileContent)
	{
		string tempFile = Path.Join(pm.SyncDir.FullName, filename);
		await File.WriteAllTextAsync(tempFile, fileContent);

		IPerforceConnection perforce = await pm.GetPerforceWithClientAsync();
		List<AddRecord> addRecords = await perforce.AddAsync(-1, tempFile);
		ChangeRecord change = new () { Description = clDescription };
		change.Files.Add(addRecords[0].DepotFile);
		change = await perforce.CreateChangeAsync(change);
		await perforce.SubmitAsync(change.Number, SubmitOptions.None);
	}

	private static void SimulateWorkspaceChanges(string rootDir)
	{
		// With CleanOptions.ModifiedTimes, last modified time of a file is too close in time for Perforce client to detect a change.
		// We nudge it just by a second to simulate real-world behavior.
		// Using digest-based change detection is simply unfeasible for the size of streams Horde handles.
		_ = new FileInfo($"{rootDir}/main.cpp") { IsReadOnly = false };
		File.AppendAllText($"{rootDir}/main.cpp", " appended text!");
		_ = new FileInfo($"{rootDir}/main.cpp") { LastWriteTime = DateTime.Now - TimeSpan.FromSeconds(1) };

		File.Move($"{rootDir}/main.h", $"{rootDir}/main-moved.h");
		File.WriteAllText($"{rootDir}/Data/added.txt", "New file");
				
		_ = new FileInfo($"{rootDir}/common.h") { IsReadOnly = false };
		File.Delete($"{rootDir}/common.h");
	}
}