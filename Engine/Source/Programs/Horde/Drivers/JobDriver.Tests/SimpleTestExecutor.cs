// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs;
using JobDriver.Execution;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using OpenTelemetry.Trace;

namespace JobDriver.Tests
{
	class SimpleTestExecutor : JobExecutor
	{
		public const string Name = "Simple";

		private readonly Func<JobStepInfo, ILogger, CancellationToken, Task<JobStepOutcome>> _func;
		public static Tracer NoOpTracer { get;  }= TracerProvider.Default.GetTracer("NoOp");

		public SimpleTestExecutor(Func<JobStepInfo, ILogger, CancellationToken, Task<JobStepOutcome>> func)
			: base(new JobExecutorOptions(null!, null!, null, default, default, default, null!), NoOpTracer, NullLogger.Instance)
		{
			_func = func;
		}

		public override Task InitializeAsync(RpcBeginBatchResponse batch, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogDebug("SimpleTestExecutor.InitializeAsync()");
			return Task.CompletedTask;
		}

		public override Task<JobStepOutcome> RunAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogDebug("SimpleTestExecutor.RunAsync(Step: {Step})", step);
			return _func(step, logger, cancellationToken);
		}

		public override Task FinalizeAsync(ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogDebug("SimpleTestExecutor.FinalizeAsync()");
			return Task.CompletedTask;
		}

		protected override Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		protected override Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}
	}

	class SimpleTestExecutorFactory : IJobExecutorFactory
	{
		readonly JobExecutor _executor;

		public string Name => SimpleTestExecutor.Name;

		public SimpleTestExecutorFactory(JobExecutor executor)
		{
			_executor = executor;
		}

		public Task<JobExecutor> CreateExecutorAsync(RpcAgentWorkspace? workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, CancellationToken cancellationToken) 
			=> Task.FromResult<JobExecutor>(_executor);
	}
}
