// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using JobDriver.Execution;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace JobDriver.Tests
{
	[TestClass]
	public class WorkerServiceTest
	{
		private readonly ServiceCollection _serviceCollection;

		private readonly JobId _jobId = JobId.Parse("65bd0655591b5d5d7d047b58");
		private readonly JobStepBatchId _batchId = new JobStepBatchId(0x1234);
		private readonly JobStepId _stepId1 = new JobStepId(1);
		private readonly JobStepId _stepId2 = new JobStepId(2);
		private readonly JobStepId _stepId3 = new JobStepId(3);
		private readonly LogId _logId = LogId.Parse("65bd0655591b5d5d7d047b00");

		class FakeServerLogger : IServerLogger
		{
			public IDisposable? BeginScope<TState>(TState state) where TState : notnull => NullLogger.Instance.BeginScope<TState>(state);

			public ValueTask DisposeAsync() => new ValueTask();

			public bool IsEnabled(LogLevel logLevel) => true;

			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter) { }

			public Task StopAsync() => Task.CompletedTask;
		}

		internal static JobExecutor NullExecutor = new SimpleTestExecutor(async (step, logger, cancellationToken) =>
		{
			await Task.Delay(1, cancellationToken);
			return JobStepOutcome.Success;
		});

		public WorkerServiceTest()
		{
			_serviceCollection = new ServiceCollection();
			_serviceCollection.AddLogging();
			_serviceCollection.AddHorde();
			_serviceCollection.AddSingleton<BundleCache>();
			_serviceCollection.AddSingleton<StorageBackendCache>();
			_serviceCollection.AddSingleton<HttpStorageBackendFactory>();
			_serviceCollection.AddSingleton<HttpStorageClient>();

			_serviceCollection.Configure<DriverSettings>(settings =>
			{
				settings.Executor = TestExecutor.Name; // Not really used since the executor is overridden in the tests
			});
		}

		[TestMethod]
		public async Task AbortExecuteStepTestAsync()
		{
			{
				using CancellationTokenSource cancelSource = new CancellationTokenSource();
				using CancellationTokenSource stepCancelSource = new CancellationTokenSource();

				using JobExecutor executor = new SimpleTestExecutor(async (stepResponse, logger, cancelToken) =>
				{
					cancelSource.CancelAfter(10);
					await Task.Delay(5000, cancelToken);
					return JobStepOutcome.Success;
				});

				await Assert.ThrowsExactlyAsync<TaskCanceledException>(() => executor.ExecuteStepAsync(
					null!, NullLogger.Instance, cancelSource.Token, stepCancelSource.Token));
			}

			{
				using CancellationTokenSource cancelSource = new CancellationTokenSource();
				using CancellationTokenSource stepCancelSource = new CancellationTokenSource();

				using JobExecutor executor = new SimpleTestExecutor(async (stepResponse, logger, cancelToken) =>
				{
					stepCancelSource.CancelAfter(10);
					await Task.Delay(5000, cancelToken);
					return JobStepOutcome.Success;
				});
				(JobStepOutcome stepOutcome, JobStepState stepState) = await executor.ExecuteStepAsync(null!, NullLogger.Instance,
					cancelSource.Token, stepCancelSource.Token);
				Assert.AreEqual(JobStepOutcome.Failure, stepOutcome);
				Assert.AreEqual(JobStepState.Aborted, stepState);
			}
		}

		[Ignore]
		[TestMethod]
		public async Task AbortExecuteJobTestAsync()
		{
			using CancellationTokenSource source = new CancellationTokenSource();
			CancellationToken token = source.Token;

			ExecuteJobTask executeJobTask = new ExecuteJobTask();
			executeJobTask.JobId = _jobId.ToString();
			executeJobTask.BatchId = _batchId.ToString();
			executeJobTask.LogId = _logId.ToString();
			executeJobTask.JobName = "jobName1";
			executeJobTask.JobOptions = new RpcJobOptions { Executor = SimpleTestExecutor.Name };
			executeJobTask.AutoSdkWorkspace = new RpcAgentWorkspace();
			executeJobTask.Workspace = new RpcAgentWorkspace();

			JobRpcClientStub client = new JobRpcClientStub(NullLogger.Instance);

			//			await using FakeHordeRpcServer fakeServer = new();
			//			await using ISession session = FakeServerSessionFactory.CreateSession(null!);

			client.BeginStepResponses.Enqueue(new RpcBeginStepResponse { Name = "stepName1", StepId = _stepId1.ToString() });
			client.BeginStepResponses.Enqueue(new RpcBeginStepResponse { Name = "stepName2", StepId = _stepId2.ToString() });
			client.BeginStepResponses.Enqueue(new RpcBeginStepResponse { Name = "stepName3", StepId = _stepId3.ToString() });

			RpcGetStepRequest step2Req = new RpcGetStepRequest(_jobId, _batchId, _stepId2);
			RpcGetStepResponse step2Res = new RpcGetStepResponse(JobStepOutcome.Unspecified, JobStepState.Unspecified, true);
			client.GetStepResponses[step2Req] = step2Res;

			using SimpleTestExecutor executor = new SimpleTestExecutor(async (step, logger, cancelToken) =>
			{
				await Task.Delay(50, cancelToken);
				return JobStepOutcome.Success;
			});

			_serviceCollection.AddSingleton<IJobExecutorFactory>(x => new SimpleTestExecutorFactory(executor));
			await using ServiceProvider serviceProvider = _serviceCollection.BuildServiceProvider();

			executor._stepAbortPollInterval = TimeSpan.FromMilliseconds(1);

			await executor.ExecuteAsync(NullLogger.Instance, token);

			Assert.AreEqual(3, client.UpdateStepRequests.Count);
			Assert.AreEqual(JobStepOutcome.Success, (JobStepOutcome)client.UpdateStepRequests[0].Outcome);
			Assert.AreEqual(JobStepState.Completed, (JobStepState)client.UpdateStepRequests[0].State);
			Assert.AreEqual(JobStepOutcome.Failure, (JobStepOutcome)client.UpdateStepRequests[1].Outcome);
			Assert.AreEqual(JobStepState.Aborted, (JobStepState)client.UpdateStepRequests[1].State);
			Assert.AreEqual(JobStepOutcome.Success, (JobStepOutcome)client.UpdateStepRequests[2].Outcome);
			Assert.AreEqual(JobStepState.Completed, (JobStepState)client.UpdateStepRequests[2].State);
		}

		[TestMethod]
		[DataRow(true, DisplayName = "Cancel job executor when abort is signaled")]
		[DataRow(false, DisplayName = "Complete job executor when abort is not signaled")]
		public async Task AbortRequestedTestAsync(bool abort)
		{
			using AbortTestExecutor executor = new((logger, cancelToken) => Task.CompletedTask);

			if (abort)
			{
				executor._abortHandle?.Set();
			}

			await executor.ExecuteAsync(NullLogger.Instance, CancellationToken.None);

			if (abort)
			{
				Assert.IsTrue(executor._abortCancellationSource.IsCancellationRequested);
			}
			else
			{
				Assert.IsFalse(executor._abortCancellationSource.IsCancellationRequested);
			}
		}

		[Ignore]
		[TestMethod]
		public async Task PollForStepAbortFailureTestAsync()
		{
			using JobExecutor executor = new SimpleTestExecutor(async (step, logger, cancelToken) =>
			{
				await Task.Delay(50, cancelToken);
				return JobStepOutcome.Success;
			});

			_serviceCollection.AddSingleton<IJobExecutorFactory>(x => new SimpleTestExecutorFactory(executor));
			await using ServiceProvider serviceProvider = _serviceCollection.BuildServiceProvider();

			//			JobHandler jobHandler = serviceProvider.GetRequiredService<JobHandler>();
			executor._stepAbortPollInterval = TimeSpan.FromMilliseconds(5);

			JobRpcClientStub client = new JobRpcClientStub(NullLogger.Instance);

			int c = 0;
			client._getStepFunc = (request) =>
			{
				return ++c switch
				{
					1 => new RpcGetStepResponse { AbortRequested = false },
					2 => throw new RpcException(new Status(StatusCode.Cancelled, "Fake cancel from test")),
					3 => new RpcGetStepResponse { AbortRequested = true },
					_ => throw new Exception("Should never reach here")
				};
			};

			using CancellationTokenSource stepPollCancelSource = new CancellationTokenSource();
			using CancellationTokenSource stepCancelSource = new CancellationTokenSource();
			TaskCompletionSource<bool> stepFinishedSource = new TaskCompletionSource<bool>();

			await executor.PollForStepAbortAsync(null!, _jobId, _batchId, _stepId2, stepCancelSource, stepFinishedSource.Task, NullLogger.Instance, stepPollCancelSource.Token);
			Assert.IsTrue(stepCancelSource.IsCancellationRequested);
		}
	}
	/*
	internal class FakeServerSessionFactory : ISessionFactory
	{
		readonly FakeHordeRpcServer _fakeServer;

		public FakeServerSessionFactory(FakeHordeRpcServer fakeServer) => _fakeServer = fakeServer;

		public Task<ISession> CreateAsync(CancellationToken cancellationToken)
		{
			return Task.FromResult(CreateSession(_fakeServer.GetHordeClient()));
		}

		public static ISession CreateSession(IHordeClient hordeClient)
		{
			Mock<ISession> fakeSession = new Mock<ISession>(MockBehavior.Strict);
			fakeSession.Setup(x => x.HordeClient).Returns(hordeClient);
			fakeSession.Setup(x => x.AgentId).Returns(new EpicGames.Horde.Agents.AgentId("LocalAgent"));
			fakeSession.Setup(x => x.SessionId).Returns(new EpicGames.Horde.Agents.Sessions.SessionId(default));
			fakeSession.Setup(x => x.DisposeAsync()).Returns(new ValueTask());
			fakeSession.Setup(x => x.WorkingDir).Returns(DirectoryReference.Combine(DirectoryReference.GetCurrentDirectory(), Guid.NewGuid().ToString()));
			return fakeSession.Object;
		}
	}
	*/
#if false
	/// <summary>
	/// Fake implementation of a HordeRpc gRPC server.
	/// Provides a corresponding gRPC client class that can be used with the WorkerService
	/// to test client-server interactions.
	/// </summary>
	internal class FakeHordeRpcServer : IAsyncDisposable
	{
		private readonly string _serverName;
		private readonly bool _isStopping = false;
		private readonly Dictionary<string, RpcLease> _leases = new();

		private readonly Dictionary<StreamId, RpcGetStreamResponse> _streamIdToStreamResponse = new();
		private readonly Dictionary<JobId, RpcGetJobResponse> _jobIdToJobResponse = new();
		private readonly ILogger<FakeHordeRpcServer> _logger;
		public readonly TaskCompletionSource<bool> CreateSessionReceived = new();
		public readonly TaskCompletionSource<bool> UpdateSessionReceived = new();

//		private readonly FakeHordeClient _hordeClient;
/*
		private class FakeHordeClient : IHordeClient
		{
			readonly FakeHordeRpcServer _server;

			public Uri ServerUrl => new Uri("http://horde-server");

			public FakeHordeClient(FakeHordeRpcServer server)
				=> _server = server;

			public Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken)
				=> throw new NotImplementedException();

			public HordeHttpClient CreateHttpClient()
				=> throw new NotImplementedException();

			public IComputeClient CreateComputeClient()
				=> throw new NotImplementedException();

			public IStorageClient CreateStorageClient(string relativePath, string? accessToken = null)
				=> throw new NotImplementedException();

			public ValueTask DisposeAsync()
				=> default;

			public Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default)
				=> Task.FromResult<string?>(null);

			public Task<GrpcChannel> CreateGrpcChannelAsync(CancellationToken cancellationToken = default)
				=> throw new NotImplementedException();

			public Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>
			{
				if (typeof(TClient) == typeof(HordeRpc.HordeRpcClient))
				{
					return Task.FromResult<TClient>((TClient)(object)new FakeHordeRpcClient(_server));
				}
				if (typeof(TClient) == typeof(JobRpc.JobRpcClient))
				{
					return Task.FromResult<TClient>((TClient)(object)new FakeJobRpcClient(_server));
				}
				throw new NotImplementedException();
			}

			public bool HasValidAccessToken()
				=> throw new NotImplementedException();

			public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information)
				=> throw new NotImplementedException();
		}

		private class FakeHordeRpcClient : HordeRpc.HordeRpcClient
		{
			private readonly FakeHordeRpcServer _outer;

			public FakeHordeRpcClient(FakeHordeRpcServer outer)
			{
				_outer = outer;
			}

			public override AsyncDuplexStreamingCall<RpcUpdateSessionRequest, RpcUpdateSessionResponse> UpdateSession(Metadata headers = null!, DateTime? deadline = null, CancellationToken cancellationToken = default)
			{
				return _outer.GetUpdateSessionCall(CancellationToken.None);
			}
		}
*/
		private class FakeJobRpcClient : JobRpc.JobRpcClient
		{
			private readonly FakeHordeRpcServer _outer;

			public FakeJobRpcClient(FakeHordeRpcServer outer)
			{
				_outer = outer;
			}

			public override AsyncUnaryCall<RpcGetJobResponse> GetJobAsync(RpcGetJobRequest request, CallOptions options)
			{
				if (_outer._jobIdToJobResponse.TryGetValue(JobId.Parse(request.JobId), out RpcGetJobResponse? jobResponse))
				{
					return JobRpcClientStub.Wrap(jobResponse);
				}

				throw new RpcException(new Status(StatusCode.NotFound, $"Job ID {request.JobId} not found"));
			}
		}

		public FakeHordeRpcServer()
		{
			_serverName = "FakeServer";
			_logger = NullLogger<FakeHordeRpcServer>.Instance;
			_hordeClient = new FakeHordeClient(this);
		}

		public void AddTestLease(string leaseId)
		{
			if (_leases.ContainsKey(leaseId))
			{
				throw new ArgumentException($"Lease ID {leaseId} already exists");
			}

			TestTask testTask = new();
			_leases[leaseId] = new RpcLease
			{
				Id = leaseId,
				State = RpcLeaseState.Pending,
				Payload = Any.Pack(testTask)
			};
		}

		public RpcLease GetLease(string leaseId)
		{
			return _leases[leaseId];
		}

		public void AddStream(StreamId streamId, string streamName)
		{
			if (_streamIdToStreamResponse.ContainsKey(streamId))
			{
				throw new Exception($"Stream ID {streamId} already added");
			}

			_streamIdToStreamResponse[streamId] = new RpcGetStreamResponse
			{
				Name = streamName,
			};
		}

		public void AddAgentType(StreamId streamId, string agentType)
		{
			if (!_streamIdToStreamResponse.TryGetValue(streamId, out RpcGetStreamResponse? streamResponse))
			{
				throw new Exception($"Stream ID {streamId} not found");
			}

			string tempDir = Path.Join(Path.GetTempPath(), $"horde-agent-type-{agentType}-" + Guid.NewGuid().ToString()[..8]);
			Directory.CreateDirectory(tempDir);

			streamResponse.AgentTypes[agentType] = new RpcGetAgentTypeResponse
			{
				TempStorageDir = tempDir
			};
		}

		public void AddJob(JobId jobId, StreamId streamId, int change, int preflightChange)
		{
			if (!_streamIdToStreamResponse.ContainsKey(streamId))
			{
				throw new Exception($"Stream ID {streamId} not found");
			}

			_jobIdToJobResponse[jobId] = new RpcGetJobResponse
			{
				StreamId = streamId.ToString(),
				Change = change,
				PreflightChange = preflightChange
			};
		}

		public IHordeClient GetHordeClient()
		{
			return _hordeClient;
		}

		public RpcCreateSessionResponse OnCreateSessionRequest(RpcCreateSessionRequest request)
		{
			CreateSessionReceived.TrySetResult(true);
			_logger.LogInformation("OnCreateSessionRequest: {AgentId} {Status}", request.Id, request.Status);
			RpcCreateSessionResponse response = new()
			{
				AgentId = "bogusAgentId",
				Token = "bogusToken",
				SessionId = "bogusSessionId",
				ExpiryTime = Timestamp.FromDateTime(DateTime.UtcNow.AddHours(3)),
			};

			return response;
		}

		public AsyncDuplexStreamingCall<RpcQueryServerStateRequest, RpcQueryServerStateResponse> GetQueryServerStateCall(CancellationToken cancellationToken)
		{
			FakeAsyncStreamReader<RpcQueryServerStateResponse> responseStream = new(cancellationToken);
			FakeClientStreamWriter<RpcQueryServerStateRequest> requestStream = new(onComplete: () =>
			{
				responseStream.Complete();
				return Task.CompletedTask;
			});

			responseStream.Write(new RpcQueryServerStateResponse { Name = _serverName, Stopping = _isStopping });

			return new(
				requestStream,
				responseStream,
				Task.FromResult(new Metadata()),
				() => Status.DefaultSuccess,
				() => new Metadata(),
				() => { /*isDisposed = true;*/ });
		}

		public AsyncDuplexStreamingCall<RpcUpdateSessionRequest, RpcUpdateSessionResponse> GetUpdateSessionCall(CancellationToken cancellationToken)
		{
			FakeAsyncStreamReader<RpcUpdateSessionResponse> responseStream = new(cancellationToken);

			async Task OnRequest(RpcUpdateSessionRequest request)
			{
				UpdateSessionReceived.TrySetResult(true);

				foreach (RpcLease agentLease in request.Leases)
				{
					RpcLease serverLease = _leases[agentLease.Id];
					serverLease.State = agentLease.State;
					serverLease.Outcome = agentLease.Outcome;
					serverLease.Output = agentLease.Output;
				}

				_logger.LogInformation("OnUpdateSessionRequest: {AgentId} {SessionId} {Status}", request.AgentId, request.SessionId, request.Status);
				await Task.Delay(100, cancellationToken);
				RpcUpdateSessionResponse response = new() { ExpiryTime = Timestamp.FromDateTime(DateTime.UtcNow + TimeSpan.FromMinutes(120)) };
				response.Leases.AddRange(_leases.Values.Where(x => x.State != RpcLeaseState.Completed));
				await responseStream.Write(response);
			}

			FakeClientStreamWriter<RpcUpdateSessionRequest> requestStream = new(OnRequest, () =>
			{
				responseStream.Complete();
				return Task.CompletedTask;
			});

			return new(
				requestStream,
				responseStream,
				Task.FromResult(new Metadata()),
				() => Status.DefaultSuccess,
				() => new Metadata(),
				() => { });
		}

		public static AsyncUnaryCall<TResponse> CreateAsyncUnaryCall<TResponse>(TResponse response)
		{
			return new AsyncUnaryCall<TResponse>(
				Task.FromResult(response),
				Task.FromResult(new Metadata()),
				() => Status.DefaultSuccess,
				() => new Metadata(),
				() => { });
		}

		public async ValueTask DisposeAsync()
		{
			await _hordeClient.DisposeAsync();

			foreach (RpcGetStreamResponse stream in _streamIdToStreamResponse.Values)
			{
				foreach (RpcGetAgentTypeResponse agentType in stream.AgentTypes.Values)
				{
					if (Directory.Exists(agentType.TempStorageDir))
					{
						Directory.Delete(agentType.TempStorageDir, true);
					}
				}
			}
		}
	}

	/// <summary>
	/// Fake stream reader used for testing gRPC clients
	/// </summary>
	/// <typeparam name="T">Message type reader will handle</typeparam>
	internal class FakeAsyncStreamReader<T> : IAsyncStreamReader<T> where T : class
	{
		private readonly Channel<T> _channel = System.Threading.Channels.Channel.CreateUnbounded<T>();
		private T? _current;
		private readonly CancellationToken? _cancellationTokenOverride;

		public FakeAsyncStreamReader(CancellationToken? cancellationTokenOverride = null)
		{
			_cancellationTokenOverride = cancellationTokenOverride;
		}

		public Task Write(T message)
		{
			if (!_channel.Writer.TryWrite(message))
			{
				throw new InvalidOperationException("Unable to write message.");
			}

			return Task.CompletedTask;
		}

		public void Complete()
		{
			_channel.Writer.Complete();
		}

		/// <inheritdoc/>
		public async Task<bool> MoveNext(CancellationToken cancellationToken)
		{
			if (_cancellationTokenOverride != null)
			{
				cancellationToken = _cancellationTokenOverride.Value;
			}

			if (await _channel.Reader.WaitToReadAsync(cancellationToken))
			{
				if (_channel.Reader.TryRead(out T? message))
				{
					_current = message;
					return true;
				}
			}

			_current = null!;
			return false;
		}

		/// <inheritdoc/>
		public T Current
		{
			get
			{
				if (_current == null)
				{
					throw new InvalidOperationException("No current element is available.");
				}
				return _current;
			}
		}
	}

	/// <summary>
	/// Fake stream writer used for testing gRPC clients
	/// </summary>
	/// <typeparam name="T">Message type writer will handle</typeparam>
	internal class FakeClientStreamWriter<T> : IClientStreamWriter<T> where T : class
	{
		private readonly Func<T, Task>? _onWrite;
		private readonly Func<Task>? _onComplete;
		private bool _isCompleted;

		public FakeClientStreamWriter(Func<T, Task>? onWrite = null, Func<Task>? onComplete = null)
		{
			_onWrite = onWrite;
			_onComplete = onComplete;
		}

		/// <inheritdoc/>
		public async Task WriteAsync(T message)
		{
			if (_isCompleted)
			{
				throw new InvalidOperationException("Stream is marked as complete");
			}
			if (_onWrite != null)
			{
				await _onWrite(message);
			}
		}

		/// <inheritdoc/>
		public WriteOptions? WriteOptions { get; set; }

		/// <inheritdoc/>
		public async Task CompleteAsync()
		{
			_isCompleted = true;
			if (_onComplete != null)
			{
				await _onComplete();
			}
		}
	}
#endif
}
