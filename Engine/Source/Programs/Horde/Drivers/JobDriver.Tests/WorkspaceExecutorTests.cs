// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using JobDriver.Execution;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;

namespace JobDriver.Tests;

internal class MaterializerStub(string name) : IWorkspaceMaterializer
{
	public void Dispose() { } 
	public string Name { get; } = name;
	public DirectoryReference SyncDir { get; } = new("not-set");
	public DirectoryReference BaseDir { get; } = new("not-set");
	public string Identifier { get; } = "not-set";
	public IReadOnlyDictionary<string, string> EnvironmentVariables { get; } = new Dictionary<string, string>();
	public bool IsPerforceWorkspace { get; } = false;
	public ILogger GetLogger(ILogger logger) => logger;

	public Task SyncAsync(int changeNum, int shelveChangeNum, SyncOptions options, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}
	public Task FinalizeAsync(CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task ConformAsync(bool removeUntrackedFiles, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}
}

internal class MaterializerFactoryStub(IWorkspaceMaterializer wm) : IWorkspaceMaterializerFactory
{
	public Task<IWorkspaceMaterializer?> CreateMaterializerAsync(string name, RpcAgentWorkspace raw, DirectoryReference dir, bool forAutoSdk = false, CancellationToken ct = default)
	{
		return Task.FromResult(name.Equals(wm.Name, StringComparison.OrdinalIgnoreCase) ? wm : null);
	}
}

[TestClass]
public class WorkspaceExecutorTests
{
	private static Tracer NoOpTracer { get; } = TracerProvider.Default.GetTracer("NoOp");
	private readonly DirectoryReference _tempDir = new ("unused-temp-dir");
	private const string ManagedWorkspace = ManagedWorkspaceMaterializer.TypeName;
	private const string Perforce = PerforceMaterializer.TypeName;
	private const string FooBar = "FooBar";
	
	[TestMethod]
	public void Factory()
	{
		Assert.ThrowsExactly<ArgumentException>(() => GetMaterializer(new RpcJobOptions(), new RpcAgentWorkspace(), []));
		
		AssertWm(ManagedWorkspace, "", "");
		AssertWm(ManagedWorkspace, "mAnaGedWorkSpace", "");
		AssertWm(FooBar, "fOObAr", "");
		AssertWm(ManagedWorkspace, "", "fOObAr");
		AssertWm(FooBar, "", "name=fOObAr");
		AssertWm(ManagedWorkspace, "", "name=mAnaGedWorkSpace");
		AssertWm(Perforce, "perforce", "name=fOObAr");
	}
	
	private void AssertWm(string expected, string jobMaterializer, string method)
	{
		RpcJobOptions rjo = new () { WorkspaceMaterializer = jobMaterializer };
		RpcAgentWorkspace raw = new () { Method = method };
		IWorkspaceMaterializer wm = GetMaterializer(rjo, raw, [FooBar, ManagedWorkspace, Perforce]);
		Assert.AreEqual(expected, wm.Name);
	}
	
	private IWorkspaceMaterializer GetMaterializer(RpcJobOptions rjo, RpcAgentWorkspace raw, string[] names)
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});
		
		List<IWorkspaceMaterializerFactory> factories = [..names.Select(name => new MaterializerFactoryStub(new MaterializerStub(name)))];
		WorkspaceExecutorFactory wef = new (factories, NoOpTracer, loggerFactory);
		return wef.CreateMaterializerAsync(rjo, raw, _tempDir, false, CancellationToken.None).GetAwaiter().GetResult();
	}
}
