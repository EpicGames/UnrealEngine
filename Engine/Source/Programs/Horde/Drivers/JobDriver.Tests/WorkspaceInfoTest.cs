// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Perforce.Managed;
using JobDriver.Utility;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace JobDriver.Tests
{
	using MWO = ManagedWorkspaceOptions;

	[TestClass]
	public class WorkspaceInfoTest
	{
		[TestMethod]
		public void ShouldUseHaveTable()
		{
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable(null));
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable(""));
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable("name=managedWorkspace"));
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable("name=managedWorkspace$#@!@#"));
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable("name=managedWorkspace$#@!@#"));
			Assert.IsTrue(WorkspaceInfo.ShouldUseHaveTable("name=managedWorkspace&useHaveTable=true"));

			Assert.IsFalse(WorkspaceInfo.ShouldUseHaveTable("name=managedWorkspace&useHaveTable=false"));
			Assert.IsFalse(WorkspaceInfo.ShouldUseHaveTable("name=ManagedWorkspace&useHaveTable=FalsE"));
		}

		[TestMethod]
		public void GetManagedWorkspaceOptions()
		{
			ManagedWorkspaceOptions defaultOptions = new();
			Assert.AreEqual(defaultOptions, GetMwOptions(null));
			Assert.AreEqual(defaultOptions, GetMwOptions(""));
			Assert.AreEqual(defaultOptions, GetMwOptions("name=somethingElse&numParallelSyncThreads=111"));
			Assert.AreEqual(new MWO { MinScratchSpace = 444 }, GetMwOptions("name=managedWorkspace&minScratchSpace=444", 10));
			Assert.AreEqual(new MWO { NumParallelSyncThreads = 111 }, GetMwOptions("name=managedWorkspace&numParallelSyncThreads=111"));
			Assert.AreEqual(new MWO { MaxFileConcurrency = 222 }, GetMwOptions("name=managedWorkspace&maxFileConcurrency=222"));
			Assert.AreEqual(new MWO { MinScratchSpace = 333 }, GetMwOptions("name=managedWorkspace&minScratchSpace=333"));
			Assert.AreEqual(new MWO { UseHaveTable = false }, GetMwOptions("name=managedWorkspace&useHaveTable=false"));
			Assert.AreEqual(new MWO { UseHaveTable = false }, GetMwOptions("name=managedWorkspace&useHaveTable=FalSe"));
			Assert.AreEqual(new MWO { UseHaveTable = true }, GetMwOptions("name=managedWorkspace&useHaveTable=true"));
			Assert.AreEqual(new MWO { UseHaveTable = true }, GetMwOptions("name=managedWorkspace&useHaveTable=TrUE"));
			Assert.AreEqual(new MWO { UseModTime = true }, GetMwOptions("name=managedWorkspace&UseModTime=true"));
		}

		static ManagedWorkspaceOptions GetMwOptions(string? method, long? minScratchSpace = null)
		{
			HordeCommon.Rpc.Messages.RpcAgentWorkspace workspace = new() { Method = method ?? "", MinScratchSpace = minScratchSpace ?? 0 };
			return WorkspaceInfo.GetMwOptions(workspace);
		}
	}
}

