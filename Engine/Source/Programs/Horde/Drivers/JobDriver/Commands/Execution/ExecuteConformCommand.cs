// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using JobDriver.Execution;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace JobDriver.Commands.Execution
{
	[Command("Execute", "Conform", "Executes a conform")]
	class ExecuteConformCommand : Command
	{
		[CommandLine("-AgentId=", Required = true)]
		public AgentId AgentId { get; set; }

		[CommandLine("-SessionId=", Required = true)]
		public SessionId SessionId { get; set; }

		[CommandLine("-LeaseId", Required = true)]
		public LeaseId LeaseId { get; set; }

		[CommandLine("-Task=", Required = true)]
		public string Task { get; set; } = null!;

		[CommandLine("-WorkingDir=", Required = true)]
		public DirectoryReference WorkingDir { get; set; } = null!;

		readonly HordeClientFactory _hordeClientFactory;
		readonly IEnumerable<IWorkspaceMaterializerFactory> _materializerFactories;
		readonly IOptions<DriverSettings> _driverSettings;
		readonly Tracer _tracer;

		public ExecuteConformCommand(HordeClientFactory hordeClientFactory, IEnumerable<IWorkspaceMaterializerFactory> materializerFactories, IOptions<DriverSettings> driverSettings, Tracer tracer)
		{
			_hordeClientFactory = hordeClientFactory;
			_materializerFactories = materializerFactories;
			_driverSettings = driverSettings;
			_tracer = tracer;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			logger.LogInformation("Running conform command in driver");

			ConformTask conformTask = ConformTask.Parser.ParseFrom(Convert.FromBase64String(Task));

			await using HordeClient hordeClient = _hordeClientFactory.Create();

			ConformExecutor conformExecutor = new ConformExecutor(hordeClient, _materializerFactories, WorkingDir, AgentId, LeaseId, conformTask, _driverSettings.Value, _tracer, logger);
			await conformExecutor.ExecuteAsync(CancellationToken.None);

			return 0;
		}
	}
}
