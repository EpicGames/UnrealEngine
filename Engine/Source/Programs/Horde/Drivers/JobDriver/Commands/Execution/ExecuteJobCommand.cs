// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using JobDriver.Execution;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace JobDriver.Commands.Execution
{
	[Command("Execute", "Job", "Executes a job")]
	class ExecuteJobCommand : Command
	{
		[CommandLine("-AgentId=", Required = true)]
		public AgentId AgentId { get; set; }

		[CommandLine("-SessionId=", Required = true)]
		public SessionId SessionId { get; set; }

		[CommandLine("-LeaseId", Required = true)]
		public LeaseId LeaseId { get; set; }

		[CommandLine("-Task=", Required = true)]
		public string Task { get; set; } = null!;

		[CommandLine("-WorkingDir=", Required = true)]
		public DirectoryReference WorkingDir { get; set; } = null!;

		readonly HordeClientFactory _hordeClientFactory;
		readonly IEnumerable<IJobExecutorFactory> _jobExecutorFactories;
		readonly IOptions<DriverSettings> _driverSettings;

		public ExecuteJobCommand(HordeClientFactory hordeClientFactory, IEnumerable<IJobExecutorFactory> jobExecutorFactories, IOptions<DriverSettings> driverSettings)
		{
			_hordeClientFactory = hordeClientFactory;
			_jobExecutorFactories = jobExecutorFactories;
			_driverSettings = driverSettings;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			logger.LogInformation("Running job command in driver");

			ExecuteJobTask executeTask = ExecuteJobTask.Parser.ParseFrom(Convert.FromBase64String(Task));

			HordeClient hordeClient = _hordeClientFactory.Create(accessToken: executeTask.Token);

			await JobExecutorHelpers.ExecuteAsync(hordeClient, WorkingDir, LeaseId, executeTask, _jobExecutorFactories, _driverSettings.Value, logger, CancellationToken.None);
			return 0;
		}
	}
}
