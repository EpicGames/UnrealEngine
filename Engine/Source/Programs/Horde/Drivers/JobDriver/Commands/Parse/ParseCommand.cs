// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using JobDriver.Parser;
using Microsoft.Extensions.Logging;

namespace JobDriver.Commands.Parse
{
	/// <summary>
	/// Installs the agent as a service
	/// </summary>
	[Command("parse", "Parses a file into structured logging output")]
	class ParseCommand : Command
	{
		[CommandLine("-File=", Required = true)]
		[Description("Log file to parse rather than executing an external program.")]
		FileReference InputFile { get; set; } = null!;

		[CommandLine("-Ignore=")]
		[Description("Path to a file containing error patterns to ignore, one regex per line.")]
		FileReference? IgnorePatternsFile { get; set; } = null;

		[CommandLine("-WorkspaceDir=")]
		[Description("The root workspace directory")]
		DirectoryReference? WorkspaceDir { get; set; } = null;

		[CommandLine("-Stream=")]
		[Description("The stream synced to the workspace")]
		string? Stream { get; set; } = null;

		[CommandLine("-Change=")]
		[Description("The changelist number that has been synced")]
		int? Change { get; set; } = null;

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			// Read all the ignore patterns
			List<string> ignorePatterns = new List<string>();
			if (IgnorePatternsFile != null)
			{
				if (!FileReference.Exists(IgnorePatternsFile))
				{
					throw new FatalErrorException("Unable to read '{0}", IgnorePatternsFile);
				}

				// Read all the ignore patterns
				string[] lines = await FileReference.ReadAllLinesAsync(IgnorePatternsFile);
				foreach (string line in lines)
				{
					string trimLine = line.Trim();
					if (trimLine.Length > 0 && !trimLine.StartsWith("#", StringComparison.Ordinal))
					{
						ignorePatterns.Add(trimLine);
					}
				}
			}

			// Read the file and pipe it through the event parser
			using (FileStream inputStream = FileReference.Open(InputFile, FileMode.Open, FileAccess.Read))
			{
				PerforceMetadataLogger jsonLogger = new PerforceMetadataLogger(logger);
				jsonLogger.AddClientView(WorkspaceDir ?? DirectoryReference.GetCurrentDirectory(), $"{Stream}/...", Change ?? 1);

				using (LogParser parser = new LogParser(jsonLogger, ignorePatterns))
				{
					byte[] data = new byte[1024];
					for (; ; )
					{
						int length = await inputStream.ReadAsync(data);
						if (length == 0)
						{
							parser.Flush();
							break;
						}
						parser.WriteData(data.AsMemory(0, length));
					}
				}
			}
			return 0;
		}
	}
}
