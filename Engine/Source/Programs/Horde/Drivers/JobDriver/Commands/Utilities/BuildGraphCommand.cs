// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using JobDriver.Parser;
using Microsoft.Extensions.Logging;

namespace JobDriver.Commands.Utilities
{
	/// <summary>
	/// Runs a BuildGraph script and captures the processed log output
	/// </summary>
	[Command("buildgraph", "Executes a BuildGraph script with the given arguments using a build of UAT within the current branch, and runs the output through the log processor")]
	class BuildGraphCommand : Command
	{
		readonly List<string> _arguments = new List<string>();

		/// <inheritdoc/>
		public override void Configure(CommandLineArguments arguments, ILogger logger)
		{
			for (int idx = 0; idx < arguments.Count; idx++)
			{
				if (!arguments.HasBeenUsed(idx))
				{
					_arguments.Add(arguments[idx]);
					arguments.MarkAsUsed(idx);
				}
			}
		}

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference baseDir = DirectoryReference.GetCurrentDirectory();

			FileReference runUatBat;
			for (; ; )
			{
				runUatBat = FileReference.Combine(baseDir, "RunUAT.bat");
				if (FileReference.Exists(runUatBat))
				{
					break;
				}

				DirectoryReference? nextDir = baseDir.ParentDirectory;
				if (nextDir == null)
				{
					logger.LogError("Unable to find RunUAT.bat in the current path");
					return 1;
				}

				baseDir = nextDir;
			}

			using (LogParser filter = new LogParser(logger, new List<string>()))
			{
				using (ManagedProcessGroup processGroup = new ManagedProcessGroup())
				{
					Dictionary<string, string> newEnvironment = ManagedProcess.GetCurrentEnvVars();
					newEnvironment["UE_STDOUT_JSON"] = "1";

					List<string> allArguments = new List<string>();
					allArguments.Add(runUatBat.FullName);
					allArguments.Add("BuildGraph");
					allArguments.AddRange(_arguments);

					string fileName = Environment.GetEnvironmentVariable("COMSPEC") ?? "cmd.exe";
					string commandLine = $"/C \"{CommandLineArguments.Join(allArguments)}\"";
					using (ManagedProcess process = new ManagedProcess(processGroup, fileName, commandLine, runUatBat.Directory.FullName, newEnvironment, null, ProcessPriorityClass.Normal))
					{
						await process.CopyToAsync((buffer, offset, length) => filter.WriteData(buffer.AsMemory(offset, length)), 4096, CancellationToken.None);
						await process.WaitForExitAsync(CancellationToken.None);
					}
				}
				filter.Flush();
			}

			return 0;
		}
	}
}
