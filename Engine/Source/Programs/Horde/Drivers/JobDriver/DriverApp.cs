// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Utilities;
using JobDriver.Execution;
using JobDriver.Utility;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using OpenTelemetry.Trace;

namespace JobDriver
{
	/// <summary>
	/// Application class for the job driver
	/// </summary>
	public static class DriverApp
	{
		/// <summary>
		/// Main entry point
		/// </summary>
		public static async Task<int> Main(string[] args)
		{
			string? taskArg = Environment.GetEnvironmentVariable("UE_HORDE_JOB_DRIVER_TASK_ARG");
			if (!String.IsNullOrEmpty(taskArg))
			{
				Array.Resize(ref args, args.Length + 1);
				args[^1] = taskArg;
				Environment.SetEnvironmentVariable("UE_HORDE_JOB_DRIVER_TASK_ARG", null);
			}

			CommandLineArguments arguments = new CommandLineArguments(args);

			// Create the services
			IServiceCollection services = new ServiceCollection();
			RegisterServices(services);

			// Run the host
			await using ServiceProvider serviceProvider = services.BuildServiceProvider();
			int exitCode = await CommandHost.RunAsync(arguments, serviceProvider, null);
			
			// JobDriver execution is often short-lived so ensure any outstanding traces are flushed
			TracerProvider? tracerProvider = serviceProvider.GetService<TracerProvider>();
			tracerProvider?.ForceFlush(5000);
			
			return exitCode;
		}

		/// <summary>
		/// Helper method to register services for this app
		/// </summary>
		/// <param name="services"></param>
		public static void RegisterServices(IServiceCollection services)
		{
			// Read the driver config
			IConfiguration configuration = new ConfigurationBuilder()
				.AddJsonFile("appsettings.json", optional: false)
				.AddJsonFile("appsettings.Epic.json", optional: true)
				.AddEnvironmentVariables()
				.Build();

			// Register the services
			services.AddOptions<DriverSettings>().Configure(options => configuration.GetSection("Driver").Bind(options)).ValidateDataAnnotations();
			services.AddLogging(builder => builder.AddEpicDefault());
			
			// Read any OpenTelemetry settings as an env var, so we can bootstrap it during dependency injection setup
			// Passing it as an argument is too late in startup
			OpenTelemetrySettings openTelemetrySettings = new();
			string? otelSettingsJson = Environment.GetEnvironmentVariable("UE_HORDE_OTEL_SETTINGS");
			if (otelSettingsJson != null)
			{
				try
				{
					openTelemetrySettings = OpenTelemetrySettingsExtensions.Deserialize(otelSettingsJson, true);
				}
				catch (Exception e)
				{
					Console.WriteLine("Unable to enable OpenTelemetry: " + e.Message);
				}
			}
			OpenTelemetryHelper.Configure(services, openTelemetrySettings);

			services.AddHorde(options => options.AllowAuthPrompt = false);

			services.AddSingleton<IJobExecutorFactory, PerforceExecutorFactory>();
			services.AddSingleton<IJobExecutorFactory, WorkspaceExecutorFactory>();
			services.AddSingleton<IJobExecutorFactory, LocalExecutorFactory>();
			services.AddSingleton<IJobExecutorFactory, TestExecutorFactory>();

			services.AddSingleton<IWorkspaceMaterializerFactory, ManagedWorkspaceMaterializerFactory>();
			services.AddSingleton<IWorkspaceMaterializerFactory, PerforceMaterializerFactory>();

			services.AddCommandsFromAssembly(Assembly.GetExecutingAssembly());
		}
	}
}
