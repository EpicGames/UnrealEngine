// Copyright Epic Games, Inc. All Rights Reserved.

using HordeCommon.Rpc.Tasks;
using JobDriver.Execution;
using JobDriver.Utility;

namespace JobDriver
{
	/// <summary>
	/// Settings for the driver
	/// </summary>
	class DriverSettings
	{
		/// <summary>
		/// The executor to use for jobs
		/// </summary>
		public string Executor { get; set; } = WorkspaceExecutor.Name;
		
		/// <summary>
		/// Whether to use Wine for executing the job. This normally is configured via <see cref="RpcJobOptions"/>
		/// </summary>
		public bool UseWine { get; set; } = false;

		/// <summary>
		/// Settings for the local executor
		/// </summary>
		public LocalExecutorSettings LocalExecutor { get; set; } = new LocalExecutorSettings();

		/// <summary>
		/// Settings for the perforce executor
		/// </summary>
		public PerforceExecutorSettings PerforceExecutor { get; set; } = new PerforceExecutorSettings();

		/// <summary>
		/// List of process names to terminate after a lease completes, but not after a job step
		/// </summary>
		public List<ProcessToTerminate> ProcessesToTerminate { get; } = new List<ProcessToTerminate>();
	}

	/// <summary>
	/// Specifies a process to terminate
	/// </summary>
	public class ProcessToTerminate
	{
		/// <summary>
		/// Name of the process
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// When to terminate this process
		/// </summary>
		public List<TerminateCondition>? When { get; init; }
	}
}
