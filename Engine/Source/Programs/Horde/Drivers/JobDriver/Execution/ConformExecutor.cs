// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using JobDriver.Utility;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Execution
{
	class ConformExecutor
	{
		readonly IHordeClient _hordeClient;
		readonly DirectoryReference _workingDir;
		readonly AgentId _agentId;
		readonly LeaseId _leaseId;
		readonly IEnumerable<IWorkspaceMaterializerFactory> _materializerFactories;
		readonly ConformTask _conformTask;
		readonly DriverSettings _driverSettings;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		public ConformExecutor(IHordeClient hordeClient, IEnumerable<IWorkspaceMaterializerFactory> materializerFactories,
			DirectoryReference workingDir, AgentId agentId, LeaseId leaseId, ConformTask conformTask, DriverSettings driverSettings, Tracer tracer, ILogger logger)
		{
			_hordeClient = hordeClient;
			_workingDir = workingDir;
			_agentId = agentId;
			_leaseId = leaseId;
			_materializerFactories = materializerFactories;
			_conformTask = conformTask;
			_driverSettings = driverSettings;
			_tracer = tracer;
			_logger = logger;
		}

		public async Task ExecuteAsync(CancellationToken cancellationToken)
		{
			try
			{
				await ExecuteInternalAsync(cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unhandled exception while running conform: {Message}", ex.Message);
				throw;
			}
		}

		async Task ExecuteInternalAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Conforming, lease {LeaseId}", _leaseId);
			await TerminateProcessHelper.TerminateProcessesAsync(TerminateCondition.BeforeConform, _workingDir, _driverSettings.ProcessesToTerminate, _logger, cancellationToken);

			bool removeUntrackedFiles = _conformTask.RemoveUntrackedFiles;
			IList<RpcAgentWorkspace> pendingWorkspaces = _conformTask.Workspaces;
			for (; ; )
			{
				bool isPerforceExecutor = _driverSettings.Executor.Equals(PerforceExecutor.Name, StringComparison.OrdinalIgnoreCase);
				bool isWorkspaceExecutor = _driverSettings.Executor.Equals(WorkspaceExecutor.Name, StringComparison.OrdinalIgnoreCase);

				List<RpcAgentWorkspace> managedWorkspaces = pendingWorkspaces
					.Where(x => !String.Equals(PerforceExecutor.GetMaterializerName(x.Method), PerforceExecutor.Name, StringComparison.OrdinalIgnoreCase))
					.ToList();

				List<RpcAgentWorkspace> perforceWorkspaces = pendingWorkspaces
					.Where(x => String.Equals(PerforceExecutor.GetMaterializerName(x.Method), PerforceExecutor.Name, StringComparison.OrdinalIgnoreCase))
					.ToList();
				
				// When using WorkspaceExecutor, only job options can override exact materializer to use
				// It will default to ManagedWorkspaceMaterializer, which is compatible with the conform call below
				// Therefore, compatibility is assumed for now. Exact materializer to use should be changed to a per workspace setting.
				// See WorkspaceExecutorFactory.CreateExecutor
				bool isExecutorConformCompatible = isPerforceExecutor || isWorkspaceExecutor;

				if (managedWorkspaces.Count > 0 && perforceWorkspaces.Count > 0)
				{
					_logger.LogInformation("Agent contains a mix of workspace materializers");
				}

				foreach (RpcAgentWorkspace raw in managedWorkspaces)
				{
					_logger.LogInformation("ManagedWorkspace: Stream={Stream} Identifier={Identifier} Incremental={Incremental} Method={Method}",
						raw.Stream, raw.Identifier, raw.Incremental, raw.Method);
				}
				
				foreach (RpcAgentWorkspace raw in perforceWorkspaces)
				{
					_logger.LogInformation("PerforceMaterializer: Stream={Stream} Identifier={Identifier} Incremental={Incremental} Method={Method}",
						raw.Stream, raw.Identifier, raw.Incremental, raw.Method);
				}

				// Get the workspace metadata dirs to protect from deletion when running the MW-based vs PerforceMaterialized-based conform
				// This is cheating a bit since we infer the metadata dir without asking the implementations for it
				// Conform handling should be rewritten to be strictly materializer-based instead
				List<DirectoryReference> workspaceMetadataDirs = pendingWorkspaces
					.Select(x => DirectoryReference.Combine(_workingDir, x.Identifier)).ToList();

				// Run the conform task
				if (isExecutorConformCompatible && _driverSettings.PerforceExecutor.RunConform)
				{
					if (managedWorkspaces.Count > 0)
					{
						_logger.LogInformation("Conforming ManagedWorkspaces...");
						await PerforceExecutor.ConformAsync(_workingDir, managedWorkspaces, removeUntrackedFiles, workspaceMetadataDirs, _tracer, _logger, cancellationToken);	
					}
					if (perforceWorkspaces.Count > 0)
					{
						_logger.LogInformation("Conforming PerforceMaterializers...");
						await PerforceExecutor.ConformMaterializersAsync(_materializerFactories, _workingDir, perforceWorkspaces, removeUntrackedFiles, workspaceMetadataDirs, _tracer, _logger, cancellationToken);	
					}
				}
				else
				{
					_logger.LogInformation("Skipping conform. Executor={Executor} RunConform={RunConform}", _driverSettings.Executor, _driverSettings.PerforceExecutor.RunConform);
				}

				// Update the new set of workspaces
				RpcUpdateAgentWorkspacesRequest request = new RpcUpdateAgentWorkspacesRequest();
				request.AgentId = _agentId.ToString();
				request.Workspaces.AddRange(pendingWorkspaces);
				request.RemoveUntrackedFiles = removeUntrackedFiles;

				JobRpc.JobRpcClient hordeRpc = await _hordeClient.CreateGrpcClientAsync<JobRpc.JobRpcClient>(cancellationToken);

				RpcUpdateAgentWorkspacesResponse response = await hordeRpc.UpdateAgentWorkspacesAsync(request, cancellationToken: cancellationToken);
				if (!response.Retry)
				{
					_logger.LogInformation("Conform finished");
					break;
				}

				_logger.LogInformation("Pending workspaces have changed - running conform again...");
				pendingWorkspaces = response.PendingWorkspaces;
				removeUntrackedFiles = response.RemoveUntrackedFiles;
			}
		}
	}
}
