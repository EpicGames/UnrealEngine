// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using EpicGames.Core;
using EpicGames.Horde.Agents.Metrics;
using Microsoft.Extensions.Logging;

namespace JobDriver.Execution;

/// <summary>
/// Implementation of <see cref="ISystemMetrics"/>
/// </summary>
public class DefaultSystemMetrics : ISystemMetrics
{
	private readonly DirectoryReference? _workingDir;
	private readonly ILogger _logger;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="workingDir"></param>
	/// <param name="logger"></param>
	public DefaultSystemMetrics(DirectoryReference? workingDir, ILogger logger)
	{
		_workingDir = workingDir;
		_logger = logger;
	}

	/// <inheritdoc/>
	public CpuMetrics? GetCpu() => null;

	/// <inheritdoc/>
	public CpuTimeMetrics? GetCpuTime() => null;

	/// <inheritdoc/>
	public MemoryMetrics? GetMemory() => null;

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk() => GetDisk(EpicGames.Horde.Agents.Metrics.DriveType.Working);

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk() => GetDisk(EpicGames.Horde.Agents.Metrics.DriveType.System);

	/// <summary>
	/// Fetches root directory metrics based off drive classification
	/// </summary>
	/// <param name="type">Classification of the drive we're fetching telemetry for</param>
	/// <returns>DiskMetrics object representing total size and free space</returns>
	private DiskMetrics? GetDisk(EpicGames.Horde.Agents.Metrics.DriveType type)
	{
		string? rootDir;
		if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
		{
			rootDir = Path.GetPathRoot(type == EpicGames.Horde.Agents.Metrics.DriveType.Working ? _workingDir?.FullName : Environment.SystemDirectory);
		}
		else
		{
			rootDir = type == EpicGames.Horde.Agents.Metrics.DriveType.Working ? _workingDir?.FullName : "/";
		}

		if (rootDir == null)
		{
			_logger.LogError("Cannot get disk metrics: no root directory");
			return null;
		}

		DriveInfo driveInfo = new(rootDir);
		return new DiskMetrics
		{
			FreeSpace = driveInfo.AvailableFreeSpace,
			TotalSize = driveInfo.TotalSize
		};
	}
}
