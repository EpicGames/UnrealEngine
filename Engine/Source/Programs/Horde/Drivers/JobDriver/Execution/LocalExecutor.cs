// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace JobDriver.Execution
{
	/// <summary>
	/// Settings for the local executor
	/// </summary>
	public class LocalExecutorSettings
	{
		/// <summary>
		/// Path to the local workspace to use with the local executor
		/// </summary>
		public string? WorkspaceDir { get; set; }

		/// <summary>
		/// Whether to actually execute steps, or just do job setup
		/// </summary>
		public bool RunSteps { get; set; } = true;
	}

	/// <summary>
	/// Job executor which runs out of a local workspace
	/// </summary>
	class LocalExecutor : JobExecutor
	{
		private readonly LocalExecutorSettings _settings;
		private readonly DirectoryReference _localWorkspaceDir;

		public LocalExecutor(JobExecutorOptions options, LocalExecutorSettings settings, Tracer tracer, ILogger logger)
			: base(options, tracer, logger)
		{
			_settings = settings;
			if (settings.WorkspaceDir == null)
			{
				_localWorkspaceDir = FindWorkspaceRoot();
			}
			else
			{
				_localWorkspaceDir = new DirectoryReference(settings.WorkspaceDir);
			}
		}

		static DirectoryReference FindWorkspaceRoot()
		{
			const string HordeSlnRelativePath = "Engine/Source/Programs/Horde/Horde.sln";

			DirectoryReference executableFileDir = new(AppContext.BaseDirectory);
			for (DirectoryReference? directory = executableFileDir; directory != null; directory = directory.ParentDirectory)
			{
				FileReference hordeSln = FileReference.Combine(directory, HordeSlnRelativePath);
				if (FileReference.Exists(hordeSln))
				{
					return directory;
				}
			}

			throw new Exception($"Unable to find workspace root directory (looking for '{HordeSlnRelativePath}' in a parent directory of '{executableFileDir}'");
		}

		protected override Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			return SetupAsync(step, _localWorkspaceDir, null, logger, cancellationToken);
		}

		protected override Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			if (_settings.RunSteps)
			{
				return ExecuteAsync(step, _localWorkspaceDir, null, logger, cancellationToken);
			}
			else
			{
				logger.LogInformation("**** SKIPPING NODE {StepName} ****", step.Name);
				return Task.FromResult(true);
			}
		}
	}

	class LocalExecutorFactory : IJobExecutorFactory
	{
		readonly LocalExecutorSettings _settings;
		readonly Tracer _tracer;
		readonly ILogger<LocalExecutor> _logger;

		public string Name => "Local";

		public LocalExecutorFactory(IOptions<LocalExecutorSettings> settings, Tracer tracer, ILogger<LocalExecutor> logger)
		{
			_settings = settings.Value;
			_tracer = tracer;
			_logger = logger;
		}

		public Task<JobExecutor> CreateExecutorAsync(RpcAgentWorkspace workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, CancellationToken cancellationToken)
		{
			return Task.FromResult<JobExecutor>(new LocalExecutor(options, _settings, _tracer, _logger));
		}
	}
}
