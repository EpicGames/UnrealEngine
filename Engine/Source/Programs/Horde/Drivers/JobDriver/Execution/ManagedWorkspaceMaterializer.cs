// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Perforce.Managed;
using JobDriver.Utility;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using EpicGames.Perforce;
using Microsoft.Extensions.DependencyInjection;
using OpenTelemetry.Trace;

namespace JobDriver.Execution;

/// <summary>
/// Workspace materializer wrapping ManagedWorkspace and WorkspaceInfo
/// </summary>
public sealed class ManagedWorkspaceMaterializer : IWorkspaceMaterializer
{
	/// <summary>
	/// Name of this materializer
	/// </summary>
	public const string TypeName = "ManagedWorkspace";

	private readonly RpcAgentWorkspace _agentWorkspace;
	private readonly bool _useCacheFile;
	private readonly bool _cleanDuringFinalize;
	private readonly WorkspaceInfo _workspace;
	private readonly Tracer _tracer;
	private readonly ILogger _logger;
	private PerforceMetadataLogger? _perforceLogger;
	private int? _lastSyncedChangeNum;

	/// <inheritdoc/>
	public DirectoryReference SyncDir => _workspace.WorkspaceDir;
	
	/// <inheritdoc/>
	public DirectoryReference BaseDir => _workspace.MetadataDir;
	
	/// <inheritdoc/>
	public string Name => TypeName;

	/// <inheritdoc/>
	public string Identifier => _agentWorkspace.Identifier;

	/// <inheritdoc/>
	public IReadOnlyDictionary<string, string> EnvironmentVariables { get; }

	/// <inheritdoc/>
	public bool IsPerforceWorkspace => true;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="agentWorkspace">Workspace configuration</param>
	/// <param name="useCacheFile">Whether to use a cache file during syncs</param>
	/// <param name="cleanDuringFinalize">Whether to clean and revert files during finalize</param>
	/// <param name="workspace"></param>
	/// <param name="tracer"></param>
	/// <param name="logger"></param>
	private ManagedWorkspaceMaterializer(
		RpcAgentWorkspace agentWorkspace,
		bool useCacheFile,
		bool cleanDuringFinalize,
		WorkspaceInfo workspace,
		Tracer tracer,
		ILogger logger)
	{
		_agentWorkspace = agentWorkspace;
		_useCacheFile = useCacheFile;
		_cleanDuringFinalize = cleanDuringFinalize;
		_workspace = workspace;
		_tracer = tracer;
		_logger = logger;

		// Variables expected to be set for UAT/BuildGraph when Perforce is enabled (-P4 flag is set) 
		EnvironmentVariables = new Dictionary<string, string>()
		{
			["uebp_PORT"] = _workspace.ServerAndPort,
			["uebp_USER"] = _workspace.UserName,
			["uebp_CLIENT"] = _workspace.ClientName,
			["uebp_CLIENT_ROOT"] = $"//{_workspace.ClientName}",
			["P4USER"] = _workspace.UserName,
			["P4CLIENT"] = _workspace.ClientName
		};
	}

	/// <inheritdoc/>
	public void Dispose()
	{
	}

	/// <inheritdoc/>
	public static async Task<ManagedWorkspaceMaterializer> CreateAsync(RpcAgentWorkspace agentWorkspace, DirectoryReference workingDir, bool useCacheFile, bool cleanDuringFinalize, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
	{
		ManagedWorkspaceOptions options = WorkspaceInfo.GetMwOptions(agentWorkspace);
		WorkspaceInfo workspace = await WorkspaceInfo.CreateWorkspaceInfoAsync(agentWorkspace, workingDir, options, tracer, logger, cancellationToken);
		return new ManagedWorkspaceMaterializer(agentWorkspace, useCacheFile, cleanDuringFinalize, workspace, tracer, logger);
	}

	/// <inheritdoc/>
	public ILogger GetLogger(ILogger logger)
	{
		_perforceLogger = new PerforceMetadataLogger(logger);
		AddClientViewToLogger();
		return _perforceLogger;
	}

	/// <inheritdoc/>
	public async Task SyncAsync(int changeNum, int preflightChangeNum, SyncOptions options, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = CreateTraceSpan($"{nameof(ManagedWorkspaceMaterializer)}.{nameof(SyncAsync)}");
		span.SetAttribute("horde.change_num", changeNum);
		span.SetAttribute("horde.remove_untracked", options.RemoveUntracked);

		using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_workspace.PerforceSettings, _logger);
		await _workspace.SetupWorkspaceAsync(perforce, cancellationToken);

		if (changeNum == IWorkspaceMaterializer.LatestChangeNumber)
		{
			int latestChangeNum = await _workspace.GetLatestChangeAsync(perforce, cancellationToken);
			span.SetAttribute("horde.change_num_latest", latestChangeNum);
			changeNum = latestChangeNum;
		}

		_lastSyncedChangeNum = changeNum;
		AddClientViewToLogger();

		FileReference cacheFile = FileReference.Combine(_workspace.MetadataDir, "Contents.dat");
		if (_useCacheFile)
		{
			bool isSyncedDataDirty = await _workspace.UpdateLocalCacheMarkerAsync(cacheFile, changeNum, preflightChangeNum);
			span.SetAttribute("horde.is_dirty", isSyncedDataDirty);
			if (!isSyncedDataDirty)
			{
				return;
			}
		}
		else
		{
			WorkspaceInfo.RemoveLocalCacheMarker(cacheFile);
		}

		await _workspace.SyncAsync(perforce, changeNum, preflightChangeNum, cacheFile, cancellationToken);
	}

	/// <inheritdoc/>
	public async Task FinalizeAsync(CancellationToken cancellationToken)
	{
		using TelemetrySpan span = CreateTraceSpan($"{nameof(ManagedWorkspaceMaterializer)}.{nameof(FinalizeAsync)}");

		if (_workspace != null && _cleanDuringFinalize)
		{
			using IPerforceConnection perforceClient = await PerforceConnection.CreateAsync(_workspace.PerforceSettings, _logger);
			await _workspace.CleanAsync(perforceClient, cancellationToken);
		}
	}

	/// <inheritdoc/>
	public Task ConformAsync(bool removeUntrackedFiles, CancellationToken cancellationToken)
	{
		// Implemented in PerforceExecutor.ConformAsync
		return Task.CompletedTask;
	}

	/// <summary>
	/// Get info for Perforce workspace
	/// </summary>
	/// <returns>Workspace info</returns>
	public WorkspaceInfo? GetWorkspaceInfo()
	{
		return _workspace;
	}

	private void AddClientViewToLogger()
	{
		if (_lastSyncedChangeNum is not null)
		{
			_perforceLogger?.AddClientView(SyncDir, _workspace.StreamView, _lastSyncedChangeNum.Value);
		}
	}

	private TelemetrySpan CreateTraceSpan(string operationName)
	{
		TelemetrySpan span = _tracer.StartActiveSpan(operationName);
		span.SetAttribute("horde.use_have_table", WorkspaceInfo.ShouldUseHaveTable(_agentWorkspace.Method));
		span.SetAttribute("horde.cluster", _agentWorkspace.Cluster);
		span.SetAttribute("horde.identifier", _agentWorkspace.Identifier);
		span.SetAttribute("horde.incremental", _agentWorkspace.Incremental);
		span.SetAttribute("horde.method", _agentWorkspace.Method);
		span.SetAttribute("horde.stream", _agentWorkspace.Stream);
		span.SetAttribute("horde.partitioned", _agentWorkspace.Partitioned);
		span.SetAttribute("horde.use_cache_file", _useCacheFile);
		span.SetAttribute("horde.clean_during_finalize", _cleanDuringFinalize);
		return span;
	}
}

class ManagedWorkspaceMaterializerFactory : IWorkspaceMaterializerFactory
{
	readonly IServiceProvider _serviceProvider;

	public ManagedWorkspaceMaterializerFactory(IServiceProvider serviceProvider) => _serviceProvider = serviceProvider;

	/// <inheritdoc/>
	public async Task<IWorkspaceMaterializer?> CreateMaterializerAsync(string name, RpcAgentWorkspace workspaceInfo, DirectoryReference workspaceDir, bool forAutoSdk, CancellationToken cancellationToken)
	{
		if (name.Equals(ManagedWorkspaceMaterializer.TypeName, StringComparison.OrdinalIgnoreCase))
		{
			Tracer tracer = _serviceProvider.GetRequiredService<Tracer>();
			ILogger<ManagedWorkspaceMaterializer> logger = _serviceProvider.GetRequiredService<ILogger<ManagedWorkspaceMaterializer>>();
			
			if (forAutoSdk)
			{
				return await ManagedWorkspaceMaterializer.CreateAsync(workspaceInfo, workspaceDir, true, false, tracer, logger, cancellationToken);
			}
			else
			{
				return await ManagedWorkspaceMaterializer.CreateAsync(workspaceInfo, workspaceDir, false, true, tracer, logger, cancellationToken);
			}
		}
		return null;
	}
}
