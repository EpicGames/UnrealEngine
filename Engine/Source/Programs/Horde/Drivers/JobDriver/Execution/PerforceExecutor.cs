// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using System.Text;
using System.Web;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using JobDriver.Utility;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Execution
{
	/// <summary>
	/// Settings for the perforce executor
	/// </summary>
	public class PerforceExecutorSettings
	{
		/// <summary>
		/// Whether to run conform jobs
		/// </summary>
		public bool RunConform { get; set; } = true;
	}

	/// <summary>
	/// Executor which syncs workspaces from Perforce
	/// </summary>
	class PerforceExecutor : JobExecutor
	{
		public const string Name = "Perforce";

		protected RpcAgentWorkspace _workspaceInfo;
		protected RpcAgentWorkspace? _autoSdkWorkspaceInfo;
		protected DirectoryReference _rootDir;

		protected WorkspaceInfo? _autoSdkWorkspace;
		protected WorkspaceInfo _workspace;

		public PerforceExecutor(RpcAgentWorkspace workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, Tracer tracer, ILogger logger)
			: base(options, tracer, logger)
		{
			_workspaceInfo = workspaceInfo;
			_autoSdkWorkspaceInfo = autoSdkWorkspaceInfo;
			_rootDir = options.WorkingDir;
			_workspace = null!;
		}

		public override async Task InitializeAsync(RpcBeginBatchResponse batch, ILogger logger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = Tracer.StartActiveSpan($"{nameof(PerforceExecutor)}.{nameof(InitializeAsync)}");
			await base.InitializeAsync(batch, logger, cancellationToken);

			// Setup and sync the AutoSDK workspace
			if (_autoSdkWorkspaceInfo != null)
			{
				using TelemetrySpan autoSdkSpan = Tracer.StartActiveSpan("Workspace");
				autoSdkSpan.SetAttribute("horde.workspace", "AutoSDK");

				ManagedWorkspaceOptions options = WorkspaceInfo.GetMwOptions(_autoSdkWorkspaceInfo);
				_autoSdkWorkspace = await WorkspaceInfo.CreateWorkspaceInfoAsync(_autoSdkWorkspaceInfo, _rootDir, options, Tracer, logger, cancellationToken);

				using IPerforceConnection autoSdkPerforce = await PerforceConnection.CreateAsync(_autoSdkWorkspace.PerforceSettings, logger);
				await _autoSdkWorkspace.SetupWorkspaceAsync(autoSdkPerforce, cancellationToken);

				DirectoryReference legacyDir = DirectoryReference.Combine(_autoSdkWorkspace.MetadataDir, "HostWin64");
				if (DirectoryReference.Exists(legacyDir))
				{
					try
					{
						FileUtils.ForceDeleteDirectory(legacyDir);
					}
					catch (Exception ex)
					{
						logger.LogInformation(ex, "Unable to delete {Dir}", legacyDir);
					}
				}

				int autoSdkChangeNumber = await _autoSdkWorkspace.GetLatestChangeAsync(autoSdkPerforce, cancellationToken);
				autoSdkSpan.SetAttribute("horde.change_number", autoSdkChangeNumber);

				string syncText = $"Synced to CL {autoSdkChangeNumber}";
				if (_autoSdkWorkspaceInfo.View.Count > 0)
				{
					StringBuilder syncTextBuilder = new StringBuilder(syncText);
					foreach (string line in _autoSdkWorkspaceInfo.View)
					{
						syncTextBuilder.Append($"\nView: {line}");
					}
					syncText = syncTextBuilder.ToString();
				}

				FileReference syncFile = FileReference.Combine(_autoSdkWorkspace.MetadataDir, "Synced.txt");
				if (!FileReference.Exists(syncFile) || (await FileReference.ReadAllTextAsync(syncFile, cancellationToken)) != syncText)
				{
					FileReference.Delete(syncFile);

					FileReference autoSdkCacheFile = FileReference.Combine(_autoSdkWorkspace.MetadataDir, "Contents.dat");
					await _autoSdkWorkspace.UpdateLocalCacheMarkerAsync(autoSdkCacheFile, autoSdkChangeNumber, -1);
					await _autoSdkWorkspace.SyncAsync(autoSdkPerforce, autoSdkChangeNumber, -1, autoSdkCacheFile, cancellationToken);

					await FileReference.WriteAllTextAsync(syncFile, syncText, cancellationToken);
				}
			}

			{
				using TelemetrySpan workspaceSpan = Tracer.StartActiveSpan("Workspace");
				workspaceSpan.SetAttribute("horde.workspace", _workspaceInfo.Identifier);
				
				// Sync the regular workspace
				ManagedWorkspaceOptions options = WorkspaceInfo.GetMwOptions(_workspaceInfo);
				_workspace = await WorkspaceInfo.CreateWorkspaceInfoAsync(_workspaceInfo, _rootDir, options, Tracer, logger, cancellationToken);

				using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_workspace.PerforceSettings, logger);
				await _workspace.SetupWorkspaceAsync(perforce, cancellationToken);

				// Figure out the change to build
				if (Batch.Change == 0)
				{
					List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.None, 1, ChangeStatus.Submitted, new[] { Batch.StreamName + "/..." }, cancellationToken);
					Batch.Change = changes[0].Number;

					RpcUpdateJobRequest updateJobRequest = new RpcUpdateJobRequest();
					updateJobRequest.JobId = JobId.ToString();
					updateJobRequest.Change = Batch.Change;
					await JobRpc.UpdateJobAsync(updateJobRequest, null, null, cancellationToken);
				}

				// Sync the workspace
				await _workspace.SyncAsync(perforce, Batch.Change, Batch.PreflightChange, null, cancellationToken);

				// Remove any cached BuildGraph manifests
				DirectoryReference manifestDir = DirectoryReference.Combine(_workspace.WorkspaceDir, $"{EnginePath}/Saved/BuildGraph");
				if (DirectoryReference.Exists(manifestDir))
				{
					try
					{
						FileUtils.ForceDeleteDirectoryContents(manifestDir);
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Unable to delete contents of {ManifestDir}", manifestDir);
					}
				}
			}

			// Remove all the local settings directories
			DeleteEngineUserSettings(logger);

			// Get all the environment variables for jobs
			EnvVars["IsBuildMachine"] = "1";
			EnvVars["uebp_LOCAL_ROOT"] = GetUnrealRootDirectoryFromWorkspace(_workspace.WorkspaceDir).FullName;
			EnvVars["uebp_PORT"] = _workspace.ServerAndPort;
			EnvVars["uebp_USER"] = _workspace.UserName;
			EnvVars["uebp_CLIENT"] = _workspace.ClientName;
			EnvVars["uebp_BuildRoot_P4"] = Batch.StreamName;
			EnvVars["uebp_BuildRoot_Escaped"] = Batch.StreamName.Replace('/', '+');
			EnvVars["uebp_CLIENT_ROOT"] = $"//{_workspace.ClientName}";
			EnvVars["uebp_CL"] = Batch.Change.ToString();
			EnvVars["uebp_CodeCL"] = Batch.CodeChange.ToString();
			EnvVars["P4USER"] = _workspace.UserName;
			EnvVars["P4CLIENT"] = _workspace.ClientName;

			if (_autoSdkWorkspace != null)
			{
				EnvVars["UE_SDKS_ROOT"] = _autoSdkWorkspace.WorkspaceDir.FullName;
			}
		}

		internal static void DeleteEngineUserSettings(ILogger logger)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				DirectoryReference? appDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
				if (appDataDir != null)
				{
					string[] dirNames = { "Unreal Engine", "UnrealEngine", "UnrealEngineLauncher", "UnrealHeaderTool", "UnrealPak" };
					DeleteEngineUserSettings(appDataDir, dirNames, logger);
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				string? homeDir = Environment.GetEnvironmentVariable("HOME");
				if (!String.IsNullOrEmpty(homeDir))
				{
					string[] dirNames = { "Library/Preferences/Unreal Engine", "Library/Application Support/Epic" };
					DeleteEngineUserSettings(new DirectoryReference(homeDir), dirNames, logger);
				}
			}
		}

		private static void DeleteEngineUserSettings(DirectoryReference baseDir, string[] subDirNames, ILogger logger)
		{
			foreach (string subDirName in subDirNames)
			{
				DirectoryReference settingsDir = DirectoryReference.Combine(baseDir, subDirName);
				if (DirectoryReference.Exists(settingsDir))
				{
					logger.LogInformation("Removing local settings directory ({SettingsDir})...", settingsDir);
					try
					{
						FileUtils.ForceDeleteDirectory(settingsDir);
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Error while deleting directory.");
					}
				}
			}
		}

		public static PerforceMetadataLogger CreatePerforceLogger(ILogger logger, int changeNum, WorkspaceInfo workspace, WorkspaceInfo? autoSdkWorkspace)
		{
			PerforceMetadataLogger perforceLogger = new PerforceMetadataLogger(logger);
			perforceLogger.AddClientView(workspace.WorkspaceDir, workspace.StreamView, changeNum);
			if (autoSdkWorkspace != null)
			{
				perforceLogger.AddClientView(autoSdkWorkspace.WorkspaceDir, autoSdkWorkspace.StreamView, changeNum);
			}
			return perforceLogger;
		}

		protected override async Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			PerforceMetadataLogger perforceLogger = CreatePerforceLogger(logger, Batch.Change, _workspace, _autoSdkWorkspace);
			bool useP4 = WorkspaceInfo.ShouldUseHaveTable(_workspaceInfo.Method);
			return await SetupAsync(step, _workspace.WorkspaceDir, useP4, perforceLogger, cancellationToken);
		}

		protected override async Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			PerforceMetadataLogger perforceLogger = CreatePerforceLogger(logger, Batch.Change, _workspace, _autoSdkWorkspace);
			bool useP4 = WorkspaceInfo.ShouldUseHaveTable(_workspaceInfo.Method);
			return await ExecuteAsync(step, _workspace.WorkspaceDir, useP4, perforceLogger, cancellationToken);
		}

		public override async Task FinalizeAsync(ILogger logger, CancellationToken cancellationToken)
		{
			await ExecuteLeaseCleanupScriptAsync(_workspace.WorkspaceDir, logger);
			await TerminateProcessesAsync(TerminateCondition.AfterBatch, logger);

			IPerforceConnection perforce = await PerforceConnection.CreateAsync(_workspace.PerforceSettings, logger);
			await _workspace.CleanAsync(perforce, cancellationToken);

			await base.FinalizeAsync(logger, cancellationToken);
		}

		public static async Task ConformAsync(
			DirectoryReference rootDir,
			IList<RpcAgentWorkspace> pendingWorkspaces,
			bool removeUntrackedFiles,
			List<DirectoryReference> ignoreDirs,
			Tracer tracer,
			ILogger logger,
			CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan("Conform");
			span.SetAttribute("horde.perforce.workspaces", String.Join(',', pendingWorkspaces.Select(x => x.Identifier)));
			span.SetAttribute("horde.perforce.remove_untracked", removeUntrackedFiles);

			// Print out all the workspaces we're going to sync
			logger.LogInformation("Workspaces to conform using ManagedWorkspace:");
			foreach (RpcAgentWorkspace pendingWorkspace in pendingWorkspaces)
			{
				logger.LogInformation("  Identifier={Identifier}, Stream={StreamName}, Incremental={Incremental} Method={Method} Partitioned={Partitioned}",
					pendingWorkspace.Identifier, pendingWorkspace.Stream, pendingWorkspace.Incremental, pendingWorkspace.Method, pendingWorkspace.Partitioned);
			}

			// Make workspaces for all the unique configurations on this agent
			List<WorkspaceInfo> workspaces = new List<WorkspaceInfo>();
			List<IPerforceConnection> perforceConnections = new List<IPerforceConnection>();
			try
			{
				// Set up all the workspaces
				int maxFileConcurrency = 1;
				foreach (RpcAgentWorkspace pendingWorkspace in pendingWorkspaces)
				{
					ManagedWorkspaceOptions options = WorkspaceInfo.GetMwOptions(pendingWorkspace);

					WorkspaceInfo workspace = await WorkspaceInfo.CreateWorkspaceInfoAsync(pendingWorkspace, rootDir, options, tracer, logger, cancellationToken);
					workspaces.Add(workspace);

					// Also collect the max set of files to delete in parallel..
					maxFileConcurrency = Int32.Max(maxFileConcurrency, options.MaxFileConcurrency);
				}

				// Find all the unique Perforce servers
				foreach (WorkspaceInfo workspace in workspaces)
				{
					if (!perforceConnections.Any(x => x.Settings.ServerAndPort!.Equals(workspace.ServerAndPort, StringComparison.OrdinalIgnoreCase) && x.Settings.UserName!.Equals(workspace.PerforceSettings.UserName, StringComparison.Ordinal)))
					{
						IPerforceConnection connection = await PerforceConnection.CreateAsync(workspace.PerforceSettings, logger);
						perforceConnections.Add(connection);
					}
				}

				// Enumerate all the workspaces
				foreach (IPerforceConnection perforce in perforceConnections)
				{
					// Get the server info
					InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, CancellationToken.None);

					// Enumerate all the clients on the server
					List<ClientsRecord> clients = await perforce.GetClientsAsync(ClientsOptions.None, null, -1, null, perforce.Settings.UserName, cancellationToken);
					foreach (ClientsRecord client in clients)
					{
						// Check the host matches
						if (!String.Equals(client.Host, info.ClientHost, StringComparison.OrdinalIgnoreCase))
						{
							continue;
						}

						// Check the edge server id matches
						if (!String.IsNullOrEmpty(client.ServerId) && !String.Equals(client.ServerId, info.ServerId, StringComparison.OrdinalIgnoreCase))
						{
							continue;
						}

						// Check it's under the managed root directory
						DirectoryReference? clientRoot;
						try
						{
							clientRoot = new DirectoryReference(client.Root);
						}
						catch
						{
							clientRoot = null;
						}

						if (clientRoot == null || !clientRoot.IsUnderDirectory(rootDir))
						{
							continue;
						}

						// Check it doesn't match one of the workspaces we want to keep
						if (workspaces.Any(x => String.Equals(client.Name, x.ClientName, StringComparison.OrdinalIgnoreCase)))
						{
							continue;
						}

						// Revert all the files in this clientspec and delete it
						logger.LogInformation("Deleting client {ClientName}...", client.Name);
						using IPerforceConnection perforceClient = await perforce.WithClientAsync(client.Name);
						await WorkspaceInfo.RevertAllChangesAsync(perforceClient, logger, cancellationToken);
						await perforce.DeleteClientAsync(DeleteClientOptions.None, client.Name, cancellationToken);
					}
				}

				// Delete all the directories that aren't a workspace root
				if (DirectoryReference.Exists(rootDir))
				{
					// Delete all the files in the root
					logger.LogInformation("Deleting files at the root of {RootDir}", rootDir);
					Parallel.ForEach(rootDir.ToDirectoryInfo().EnumerateFiles(), new() { MaxDegreeOfParallelism = maxFileConcurrency }, FileUtils.ForceDeleteFile);

					// Build a set of directories to protect
					HashSet<DirectoryReference> protectDirs = [DirectoryReference.Combine(rootDir, "Leases")];// Current lease may be writing a log here
					if (!removeUntrackedFiles)
					{
						protectDirs.Add(DirectoryReference.Combine(rootDir, "Temp"));
						protectDirs.Add(DirectoryReference.Combine(rootDir, "Saved"));
					}
					protectDirs.UnionWith(workspaces.Select(x => x.MetadataDir));
					protectDirs.UnionWith(ignoreDirs);

					// Delete all the directories which aren't a workspace root
					foreach (DirectoryReference dir in DirectoryReference.EnumerateDirectories(rootDir))
					{
						if (protectDirs.Contains(dir))
						{
							logger.LogInformation("Keeping directory {KeepDir}", dir);
						}
						else
						{
							logger.LogInformation("Deleting directory {DeleteDir}", dir);
							Parallel.ForEach(dir.ToDirectoryInfo().EnumerateFiles("*", SearchOption.AllDirectories), new() { MaxDegreeOfParallelism = maxFileConcurrency }, FileUtils.ForceDeleteFile);
							FileUtils.ForceDeleteDirectory(dir);
						}
					}
					logger.LogInformation("");
				}

				// Revert any open files in any workspace
				HashSet<string> revertedClientNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
				foreach (WorkspaceInfo workspace in workspaces)
				{
					if (revertedClientNames.Add(workspace.ClientName))
					{
						using IPerforceConnection connection = await PerforceConnection.CreateAsync(workspace.PerforceSettings, logger);
						await workspace.Repository.RevertAsync(connection, cancellationToken);
					}
				}

				// Sync all the branches.
				List<Func<Task>> syncFuncs = new List<Func<Task>>();
				foreach (IGrouping<DirectoryReference, WorkspaceInfo> workspaceGroup in workspaces.GroupBy(x => x.MetadataDir).OrderBy(x => x.Key.FullName))
				{
					logger.LogInformation("Queuing workspaces for sync/populate:");

					List<PopulateRequest> populateRequests = new List<PopulateRequest>();
					foreach (WorkspaceInfo workspace in workspaceGroup)
					{
						logger.LogInformation("  Stream={StreamName} RemoveUntrackedFiles={RemoveUntrackedFiles} MetadataDir={MetadataDir} WorkspaceDir={WorkspaceDir} ClientName={ClientName}",
							workspace.StreamName, workspace.RemoveUntrackedFiles, workspace.MetadataDir.FullName, workspace.WorkspaceDir.FullName, workspace.ClientName);

						IPerforceConnection perforceClient = await PerforceConnection.CreateAsync(workspace.PerforceSettings, logger);
						perforceConnections.Add(perforceClient);

						populateRequests.Add(new PopulateRequest(perforceClient, workspace.StreamName, workspace.View));
					}

					WorkspaceInfo? firstWorkspace = workspaceGroup.First();
					if (populateRequests.Count == 1 && !firstWorkspace.RemoveUntrackedFiles && !removeUntrackedFiles)
					{
						PopulateRequest populateRequest = populateRequests[0];
						await firstWorkspace.CleanAsync(populateRequest.PerforceClient, cancellationToken);
						syncFuncs.Add(() => firstWorkspace.SyncAsync(populateRequest.PerforceClient, -1, -1, null, cancellationToken));
					}
					else
					{
						ManagedWorkspace repository = firstWorkspace.Repository;
						Tuple<int, StreamSnapshot>[] streamStates = await repository.PopulateCleanAsync(populateRequests, cancellationToken);
						syncFuncs.Add(() => repository.PopulateSyncAsync(populateRequests, streamStates, false, cancellationToken));
					}
				}
				foreach (Func<Task> syncFunc in syncFuncs)
				{
					await syncFunc();
				}
			}
			finally
			{
				foreach (IPerforceConnection perforceConnection in perforceConnections)
				{
					perforceConnection.Dispose();
				}
			}
		}

		internal static string? GetMaterializerName(string method)
		{
			if (!String.IsNullOrEmpty(method))
			{
				string? materializerName = HttpUtility.ParseQueryString(method)["name"];
				if (materializerName != null)
				{
					return materializerName;
				}
			}

			return null;
		}

		public static async Task ConformMaterializersAsync(
			IEnumerable<IWorkspaceMaterializerFactory> factories,
			DirectoryReference rootDir,
			IList<RpcAgentWorkspace> workspaces,
			bool removeUntrackedFiles,
			List<DirectoryReference> ignoreDirs,
			Tracer tracer,
			ILogger logger,
			CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(PerforceExecutor)}.{nameof(ConformMaterializersAsync)}");
			span.SetAttribute("horde.perforce.workspaces", String.Join(',', workspaces.Select(x => x.Identifier)));
			span.SetAttribute("horde.perforce.remove_untracked", removeUntrackedFiles);

			WorkspaceMaterializerFactory factory = new(factories);
			
			// Make workspaces for all the unique configurations on this agent
			List<IWorkspaceMaterializer> materializers = [];
			
			// Set up all the materializers
			logger.LogInformation("Workspaces to conform as materializers:");
			foreach (RpcAgentWorkspace workspace in workspaces)
			{
				logger.LogInformation("  Identifier={Identifier}, Stream={StreamName}, Incremental={Incremental} Method={Method} Partitioned={Partitioned}",
					workspace.Identifier, workspace.Stream, workspace.Incremental, workspace.Method, workspace.Partitioned);
				
				string? materializerName = GetMaterializerName(workspace.Method);
				if (materializerName != null)
				{
					materializers.Add(await factory.CreateMaterializerOrThrowAsync(materializerName, workspace, rootDir, false, cancellationToken));	
				}
			}
			
			// TODO: Remove unused clients

			// Delete all the directories that aren't a workspace root
			if (DirectoryReference.Exists(rootDir))
			{
				// Delete all the files in the root
				foreach (FileInfo file in rootDir.ToDirectoryInfo().EnumerateFiles())
				{
					FileUtils.ForceDeleteFile(file);
				}

				// Build a set of directories to protect
				HashSet<DirectoryReference> protectDirs = [];
				protectDirs.Add(DirectoryReference.Combine(rootDir, "Leases")); // Current lease may be writing a log here
				if (!removeUntrackedFiles)
				{
					protectDirs.Add(DirectoryReference.Combine(rootDir, "Temp"));
					protectDirs.Add(DirectoryReference.Combine(rootDir, "Saved"));
				}
				protectDirs.UnionWith(materializers.Select(x => x.BaseDir));
				protectDirs.UnionWith(ignoreDirs);

				// Delete all the directories which aren't a workspace root
				foreach (DirectoryReference dir in DirectoryReference.EnumerateDirectories(rootDir))
				{
					if (protectDirs.Contains(dir))
					{
						logger.LogInformation("Keeping directory {KeepDir}", dir);
					}
					else
					{
						logger.LogInformation("Deleting directory {DeleteDir}", dir);
						FileUtils.ForceDeleteDirectory(dir);
						logger.LogInformation("Finished deleting directory");
					}
				}
				logger.LogInformation("");
			}

			// Revert any open files in any workspace
			foreach (IWorkspaceMaterializer materializer in materializers)
			{
				await materializer.ConformAsync(removeUntrackedFiles, cancellationToken);
			}
		}

		/// <summary>
		/// Deletes Perforce client workspaces that are no longer needed,
		/// cleaning up old or orphaned clients on the current machine under the specified root directory.
		/// </summary>
		/// <param name="perforce">The Perforce connection to use for querying and deleting clients</param>
		/// <param name="clientsToKeep">List of client names that should be preserved and not deleted</param>
		/// <param name="rootDir">The root directory under which clients will be considered for deletion</param>
		/// <param name="logger">Logger</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public static async Task DeleteUnusedClientsAsync(IPerforceConnection perforce, List<string> clientsToKeep, DirectoryReference rootDir, ILogger logger, CancellationToken cancellationToken)
		{
			// Get the server info
			InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);

			// Enumerate all the clients on the server
			List<ClientsRecord> clients = await perforce.GetClientsAsync(ClientsOptions.None, null, -1, null, perforce.Settings.UserName, cancellationToken);
			foreach (ClientsRecord client in clients)
			{
				// Check the host matches
				if (!String.Equals(client.Host, info.ClientHost, StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				// Check the edge server id matches
				if (!String.IsNullOrEmpty(client.ServerId) && !String.Equals(client.ServerId, info.ServerId, StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				// Check it's under the managed root directory
				DirectoryReference? clientRoot;
				try
				{
					clientRoot = new DirectoryReference(client.Root);
				}
				catch
				{
					clientRoot = null;
				}

				if (clientRoot == null || !clientRoot.IsUnderDirectory(rootDir))
				{
					continue;
				}

				// Check it doesn't match one of the clients we want to keep
				if (clientsToKeep.Exists(x => String.Equals(client.Name, x, StringComparison.OrdinalIgnoreCase)))
				{
					continue;
				}

				// Revert all the files in this clientspec and delete it
				logger.LogInformation("Deleting client {ClientName}...", client.Name);
				using IPerforceConnection perforceClient = await perforce.WithClientAsync(client.Name);
				await WorkspaceInfo.RevertAllChangesAsync(perforceClient, logger, cancellationToken);
				await perforce.DeleteClientAsync(DeleteClientOptions.None, client.Name, cancellationToken);
			}
		}
	}

	class PerforceExecutorFactory : IJobExecutorFactory
	{
		readonly ILogger<PerforceExecutor> _logger;
		readonly Tracer _tracer;

		public string Name => PerforceExecutor.Name;

		public PerforceExecutorFactory(Tracer tracer, ILogger<PerforceExecutor> logger)
		{
			_tracer = tracer;
			_logger = logger;
		}

		public Task<JobExecutor> CreateExecutorAsync(RpcAgentWorkspace workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, CancellationToken cancellationToken)
		{
			return Task.FromResult<JobExecutor>(new PerforceExecutor(workspaceInfo, autoSdkWorkspaceInfo, options, _tracer, _logger));
		}
	}
}
