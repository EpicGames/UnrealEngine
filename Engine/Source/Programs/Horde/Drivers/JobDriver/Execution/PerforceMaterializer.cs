// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Specialized;
using System.Diagnostics;
using System.Text.Json;
using System.Web;
using EpicGames.Core;
using EpicGames.Perforce;
using HordeCommon.Rpc.Messages;
using JobDriver.Utility;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Execution;

/// <summary>
/// Options for <see cref="PerforceMaterializer" />
/// </summary>
/// <param name="DirPath">Base directory for this workspace</param>
/// <param name="AgentWorkspace">Workspace options</param>
/// <param name="HostName">Optional hostname override</param>
/// <param name="EnvVars">Optional environment variables to specify</param>
/// <param name="SyncThreads">Number of threads to use when syncing a Perforce workspace</param>
/// <param name="SyncBatch">How many files in each sync batch</param>
/// <param name="SyncBatchSize">Size of each sync batch</param>
/// <param name="MaxFileConcurrency">Maximum number of concurrent file system operations (copying, moving, deleting)</param>
/// <param name="MaxFileScanConcurrency">Maximum number of current file system scaning operations (file system discovery, traversing)</param>
/// <param name="UseModTime">Whether to use ModTime flag on the workspace</param>
public record PerforceMaterializerOptions(
	string DirPath,
	RpcAgentWorkspace AgentWorkspace,
	string? HostName,
	IReadOnlyDictionary<string, string>? EnvVars,
	int SyncThreads,
	int SyncBatch,
	int SyncBatchSize,
	int MaxFileConcurrency,
	int MaxFileScanConcurrency,
	bool UseModTime = false)
{
	/// <inheritdoc cref="PerforceMaterializerOptions(String, RpcAgentWorkspace, String, IReadOnlyDictionary{String, String}, Int32, Int32, Int32, Int32, Int32, Boolean)"/>
	public PerforceMaterializerOptions(string dirPath, RpcAgentWorkspace agentWorkspace, string? hostName = null, IReadOnlyDictionary<string, string>? envVars = null, int syncThreads = -1, int syncBatch = -1, int syncBatchSize = -1, int? maxFileConcurrency = null, int? maxFileScanConcurrency = null, bool bUseModTime = false)
		: this(
			  dirPath,
			  agentWorkspace,
		      hostName,
			  envVars,
			  syncThreads,
			  syncBatch,
			  syncBatchSize,
			  maxFileConcurrency ?? GetDefaultThreadCount(4),
			  maxFileScanConcurrency ?? GetDefaultThreadCount(4),
			  bUseModTime)
	{ }

	static int GetDefaultThreadCount(int defaultValue)
			=> Math.Min(defaultValue, Math.Max(Environment.ProcessorCount - 1, 1));

	/// <summary>
	/// Parse options from <see cref="RpcAgentWorkspace.Method" /> as HTTP query string
	/// </summary>
	/// <param name="dirPath">Base directory for this workspace</param>
	/// <param name="agentWorkspace">Workspace options</param>
	public static PerforceMaterializerOptions FromMethodString(string dirPath, RpcAgentWorkspace agentWorkspace)
	{
		const string NameKey = "name";
		const string PerforceMaterializerValue = "perforce";
		const string SyncThreadsKey = "syncThreads";
		const string SyncBatchKey = "syncBatch";
		const string SyncBatchSizeKey = "syncBatchSize";
		const string MaxFileConcurrencyKey = "maxFileConcurrency";
		const string MaxFileScanConcurrencyKey = "maxFileScanConcurrency";
		const string UseModTimeKey = "useModTime";
		
		PerforceMaterializerOptions options = new (dirPath, agentWorkspace);
		if (!String.IsNullOrEmpty(agentWorkspace.Method))
		{
			NameValueCollection nameValues = HttpUtility.ParseQueryString(agentWorkspace.Method);
			if (String.Equals(nameValues[NameKey], PerforceMaterializerValue, StringComparison.OrdinalIgnoreCase))
			{
				if (Int32.TryParse(nameValues[SyncThreadsKey], out int threads))
				{
					options = options with { SyncThreads = threads };
				}
				if (Int32.TryParse(nameValues[SyncBatchKey], out int batch))
				{
					options = options with { SyncBatch = batch };
				}
				if (Int32.TryParse(nameValues[SyncBatchSizeKey], out int batchSize))
				{
					options = options with { SyncBatchSize = batchSize };
				}
				if (String.Equals(nameValues[UseModTimeKey], "true", StringComparison.OrdinalIgnoreCase))
				{
					options = options with { UseModTime = true };
				}
				if (String.Equals(nameValues[UseModTimeKey], "false", StringComparison.OrdinalIgnoreCase))
				{
					options = options with { UseModTime = false };
				}
				if (Int32.TryParse(nameValues[MaxFileConcurrencyKey], out int maxConcurrency))
				{
					options = options with { MaxFileConcurrency = maxConcurrency };
				}
				if (Int32.TryParse(nameValues[MaxFileScanConcurrencyKey], out int maxScanConcurrency))
				{
					options = options with { MaxFileScanConcurrency = maxScanConcurrency };
				}
			}
		}

		return options;
	}
}

/// <summary>
/// Materializer using standard Perforce client for syncing files
/// Main use-case are for agents using only a single stream.
/// For example, incremental build agents, as stream switching has not been optimized.
/// </summary>
public sealed class PerforceMaterializer : IWorkspaceMaterializer
{
	/// <summary>
	/// Name of this materializer
	/// </summary>
	public const string TypeName = "Perforce";

	internal const int CurrentStateVersion = 2;
	internal enum TransactionStatus { Clean = 0, Dirty = 1 }

	internal record WorkspaceFile(string ClientFile, string LocalFile);
	internal record State(
		int Version,
		TransactionStatus Status,
		string Client,
		string Identifier,
		string Stream,
		int Changelist,
		int ShelvedChangelist,
		List<WorkspaceFile> DirtyFiles);
	
	/// <inheritdoc/>
	public DirectoryReference SyncDir { get; }

	/// <inheritdoc/>
	public DirectoryReference BaseDir { get; }

	/// <inheritdoc/>
	public string Name => TypeName;

	/// <inheritdoc/>
	public string Identifier { get; }

	/// <inheritdoc/>
	public IReadOnlyDictionary<string, string> EnvironmentVariables { get; private set; }
	
	/// <inheritdoc/>
	public bool IsPerforceWorkspace { get; } = true;
	
	private static readonly JsonSerializerOptions s_jsonOptions = new() { WriteIndented = true };
	private readonly PerforceMaterializerOptions _options;
	private readonly Tracer _tracer;
	private readonly ILogger _logger;
	private readonly DirectoryReference _metadataDir;
	private readonly DirectoryReference _workspaceDir;
	private PerforceSettings? _perforceSettings;
	private IPerforceConnection? _perforceWithoutClient;
	private IPerforceConnection? _perforceWithClient;
	private List<string>? _streamView;
	private int? _lastSyncedChangeNum;
	private SyncOptions? _lastSyncOptions;
    private bool _isFreshSync;
	private const string StateFilename = "State.json";
	private const bool PreferNativePerforceClient = true;
	private const int MaxFilesLogged = 500;
	private readonly string _stateFile;
	private readonly string _stateTempFile;
	private PerforceMetadataLogger? _perforceLogger;

	/// <summary>
	/// Constructor
	/// </summary>
	public PerforceMaterializer(PerforceMaterializerOptions options, Tracer tracer, ILogger logger)
	{
		_options = options.SyncThreads == -1 ? options with { SyncThreads = GetDefaultThreadCount(6) } : options;
		Identifier = options.AgentWorkspace.Identifier;
		EnvironmentVariables = options.EnvVars ?? new Dictionary<string, string>();
		_tracer = tracer;
		_logger = logger;
		_metadataDir = DirectoryReference.Combine(new DirectoryReference(_options.DirPath), Identifier);
		_workspaceDir = DirectoryReference.Combine(_metadataDir, "Sync");
		_stateFile = Path.Join(_metadataDir.FullName, StateFilename);
		_stateTempFile = Path.Join(_metadataDir.FullName, StateFilename + ".tmp");
		SyncDir = _workspaceDir;
		BaseDir = _metadataDir;
	}
	
	/// <inheritdoc/>
	public void Dispose()
	{
		_perforceWithClient?.Dispose();
		_perforceWithoutClient?.Dispose();
	}
	
	private async Task InitializeAsync(CancellationToken cancellationToken)
	{
		if (_perforceSettings != null)
		{
			return;
		}
		
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(InitializeAsync)}");
		using ILoggerProgress status = _logger.BeginProgressScope("Initializing...");
		Stopwatch timer = Stopwatch.StartNew();
		
		const bool PreferNative = true;
		RpcAgentWorkspace raw = _options.AgentWorkspace;
		PerforceSettings tempSettings = new(raw.ServerAndPort, raw.UserName) { PreferNativeClient = PreferNative, Password = raw.Ticket };
		_perforceWithoutClient = await PerforceConnection.CreateAsync(tempSettings, _logger);
		if (String.IsNullOrEmpty(raw.Password))
		{
			_logger.LogInformation("Perforce password not set - skipping login");
		}
		else
		{
			await _perforceWithoutClient.LoginAsync(raw.Password, cancellationToken);
		}
		
		
		// Get the host name, and fill in any missing metadata about the connection
		InfoRecord info = await _perforceWithoutClient.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);
		
		string? hostName = info.ClientHost;
		if (hostName == null)
		{
			throw new Exception("Unable to determine Perforce host name");
		}
		
		// Replace invalid characters in the workspace identifier with a '+' character
		// Append the slot index, if it's non-zero
		//			my $slot_idx = $optional_arguments->{'slot_idx'} || 0;
		//			$workspace_identifier .= sprintf("+%02d", $slot_idx) if $slot_idx;
		// If running on an edge server, append the server ID to the client name
		string edgeSuffix = String.Empty;
		if (info is { Services: not null, ServerId: not null })
		{
			string[] services = info.Services.Split(' ', StringSplitOptions.RemoveEmptyEntries);
			if (services.Any(x => x.Equals("edge-server", StringComparison.OrdinalIgnoreCase)))
			{
				edgeSuffix = $"+{info.ServerId}";
			}
		}
		
		StreamRecord stream = await _perforceWithoutClient.GetStreamAsync(_options.AgentWorkspace.Stream, true, cancellationToken);
		_streamView = stream.View;
		
		string clientName = $"Horde+PM+{WorkspaceInfo.GetNormalizedHostName(hostName)}+{raw.Identifier}{edgeSuffix}";
		_perforceSettings = new PerforceSettings(_perforceWithoutClient.Settings) { ClientName = clientName, PreferNativeClient = PreferNative };
		status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
	}

	/// <inheritdoc/>
	public ILogger GetLogger(ILogger logger)
	{
		_perforceLogger = new PerforceMetadataLogger(logger);
		AddClientViewToLogger();
		return _perforceLogger;
	}

	/// <inheritdoc/>
	public async Task SyncAsync(int changeNum, int shelveChangeNum, SyncOptions options, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(SyncAsync)}");
		
		await InitializeAsync(cancellationToken);
		if (_perforceSettings == null || _perforceWithoutClient == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}

		_lastSyncOptions = options;
		State? state = await LoadStateAsync(cancellationToken);
		_isFreshSync = state == null;

		// Check if only the client name changed (edge server switch scenario)
		// In this case, don't mark as dirty - the existing flush + sync will handle it correctly
		bool isClientNameChange = state != null
			&& state.Version == CurrentStateVersion
			&& state.Status != TransactionStatus.Dirty
			&& state.Identifier == Identifier
			&& state.Stream == _options.AgentWorkspace.Stream
			&& state.Client != _perforceSettings!.ClientName
			&& state.Changelist != -1;
 
		bool isDirty = state == null
			|| state.Version != CurrentStateVersion
			|| state.Status == TransactionStatus.Dirty
			|| state.Identifier != Identifier
			|| state.Stream != _options.AgentWorkspace.Stream
			|| (!isClientNameChange && state.Client != _perforceSettings!.ClientName)
			|| state.Changelist == -1;
		
		_logger.LogInformation("Status: {Status}\nVersion: {PrevVersion} -> {Version}\nClient: {PrevClient} -> {Client}\nIdentifier: {PrevId} -> {Id}\nStream: {PrevStream} -> {Stream}\nCL: {PrevCl} -> {Cl}\nShelve: {PrevShelve} -> {Shelve}\nDirty: {Dirty}",
			state?.Status,
			state?.Version,
			CurrentStateVersion,
			state?.Client, _perforceSettings.ClientName,
			state?.Identifier, Identifier,
			state?.Stream, _options.AgentWorkspace.Stream,
			state?.Changelist, changeNum,
			state?.ShelvedChangelist, shelveChangeNum,
			isDirty);
		
		if (isDirty)
		{
			using TelemetrySpan deleteSpan = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(SyncAsync)}.Delete");
			using ILoggerProgress status = _logger.BeginProgressScope("Deleting local workspace files...");
			Stopwatch deleteTimer = Stopwatch.StartNew();

			if (Directory.Exists(_workspaceDir.FullName))
			{
				Parallel.ForEach(DirectoryReference.EnumerateFiles(_workspaceDir, "*", SearchOption.AllDirectories), new() { MaxDegreeOfParallelism = _options.MaxFileConcurrency }, FileUtils.ForceDeleteFile);
				FileUtils.ForceDeleteDirectory(_workspaceDir);
			}
			Directory.CreateDirectory(_workspaceDir.FullName);
			status.Progress = $"({deleteTimer.Elapsed.TotalSeconds:0.0}s)";
		}

		(ClientRecord client, bool clientWasCreated) = await CreateOrUpdateClientAsync(isDirty, cancellationToken);
		
		_logger.LogInformation("Using client {ClientName} (Host: {HostName}, Stream: {StreamName}, Type: {Type}, Root: {Path}, Recreated: {Recreated})",
			client.Name, client.Host, client.Stream, client.Type, client.Root, clientWasCreated);

		// Variables expected to be set for UAT/BuildGraph when Perforce is enabled (-P4 flag is set) 
		EnvironmentVariables = new Dictionary<string, string>()
		{
			["uebp_PORT"] = _options.AgentWorkspace.ServerAndPort,
			["uebp_USER"] = _options.AgentWorkspace.UserName,
			["uebp_CLIENT"] = client.Name,
			["uebp_CLIENT_ROOT"] = $"//{client.Name}",
			["P4CLIENT"] = client.Name,
			["P4USER"] = _options.AgentWorkspace.UserName,
		};
		
		_perforceWithClient = await PerforceConnection.CreateAsync(_perforceSettings, _logger);
		await RevertInternalAsync(_perforceWithClient, cancellationToken);

		if ((clientWasCreated || isClientNameChange) && !isDirty && state != null)
		{
			using TelemetrySpan deleteSpan = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(SyncAsync)}.Flush");
			using ILoggerProgress status = _logger.BeginProgressScope($"Flushing client to CL {state.Changelist} (p4 sync -k)");
			Stopwatch flushTimer = Stopwatch.StartNew();
			await _perforceWithClient.SyncQuietAsync(EpicGames.Perforce.SyncOptions.KeepWorkspaceFiles, -1, $"@{state.Changelist}", cancellationToken);
			status.Progress = $"({flushTimer.Elapsed.TotalSeconds:0.0}s)";
		}

		// If syncing to latest we need find the actual change number
        if (changeNum == IWorkspaceMaterializer.LatestChangeNumber)
        {
			List<ChangesRecord> changesRecords = await _perforceWithClient.GetChangesAsync(ChangesOptions.None,  1, ChangeStatus.Submitted, new List<string> {$"//{client.Name}/..."}, cancellationToken);
			if (changesRecords.Count == 0)
			{
				throw new Exception("No change records returned when looking up latest change number");
			}
			
			changeNum = changesRecords[0].Number;
		}

		_lastSyncedChangeNum = changeNum;
		AddClientViewToLogger();

		await SaveStateAsync(TransactionStatus.Dirty, client.Name, changeNum, shelveChangeNum, [], cancellationToken);
		
		{
			using TelemetrySpan syncSpan = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(SyncAsync)}.Sync");
			using ILoggerProgress status = _logger.BeginProgressScope("Syncing files...");
			Stopwatch syncTimer = Stopwatch.StartNew();
			
			string changeNumStr = "@" + changeNum;
			string shelveStr = shelveChangeNum > 0 ? $" with shelved CL {shelveChangeNum}" : "";
			_logger.LogInformation("Syncing CL {ChangeNum}{Shelve} ... (Threads: {Threads} Batch: {Batch} BatchSize: {BatchSize})",
				changeNumStr, shelveStr, _options.SyncThreads, _options.SyncBatch, _options.SyncBatchSize);

			List<SyncSummaryRecord> syncSummaries = await _perforceWithClient.SyncQuietAsync(
				EpicGames.Perforce.SyncOptions.None, -1, _options.SyncThreads, _options.SyncBatch, _options.SyncBatchSize,
				-1, -1, changeNumStr, cancellationToken);

			long totalFileCount = syncSummaries.Sum(x => x.TotalFileCount);
			long totalFileSize = syncSummaries.Sum(x => x.TotalFileSize);
			double totalFileSizeMb = Math.Round(totalFileSize / (1024.0 * 1024.0), 1);
			double filesPerSecond = Math.Round(totalFileCount / syncTimer.Elapsed.TotalSeconds, 1);
			double dataPerSecond = Math.Round(totalFileSizeMb / syncTimer.Elapsed.TotalSeconds, 1);
			_logger.LogInformation("Synced {FileCount} files. {FileSizeMb:F2} MB ({FileSize} bytes). {FilesPerSecond} files/s. {DataPerSecond} MB/s",
				totalFileCount, totalFileSizeMb, totalFileSize, filesPerSecond, dataPerSecond);
			syncSpan.SetAttribute("horde.pm.file_count", totalFileCount);
			syncSpan.SetAttribute("horde.pm.file_size", totalFileSize);
			status.Progress = $"({syncTimer.Elapsed.TotalSeconds:0.0}s)";
		}

		if (shelveChangeNum > 0)
		{
			await UnshelveAsync(_perforceWithClient, client, changeNum, shelveChangeNum, cancellationToken);
		}
		else
		{
			await SaveStateAsync(TransactionStatus.Clean, client.Name, changeNum, shelveChangeNum, [], cancellationToken);
		}
	}
	
	/// <inheritdoc/>
	public async Task FinalizeAsync(CancellationToken cancellationToken)
	{
		if (_perforceWithClient != null)
		{
			await RevertInternalAsync(_perforceWithClient, cancellationToken);
			if (_lastSyncOptions is { RemoveUntracked: true })
			{
				await CleanAsync(_perforceWithClient, cancellationToken);
			}
		}
	}

	/// <inheritdoc/>
	public async Task ConformAsync(bool removeUntrackedFiles, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(ConformAsync)}");
		using ILoggerProgress status = _logger.BeginProgressScope("Conforming...");
		
		await InitializeAsync(cancellationToken);

		// Sync from scratch again. Any files are reverted by SyncAsync
		// TODO: Call ResetState() to ensure a complete sync is performed?
		SyncOptions syncOptions = new () { RemoveUntracked = removeUntrackedFiles };
		await SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, 0, syncOptions, cancellationToken);
		
		if (removeUntrackedFiles && !_isFreshSync)
		{
			if (_perforceWithClient != null)
			{
				await CleanAsync(_perforceWithClient, cancellationToken);
			}
			else
			{
				_logger.LogWarning("Cannot clean: Perforce client not initialized after sync");
			}
		}
	}
	
		/// <summary>
		/// Replays the effects of unshelving a changelist
		/// But clobbering files in the workspace rather than actually unshelving them.
		/// This to prevent problems with multiple machines locking them.
		/// </summary>
		public async Task UnshelveAsync(IPerforceConnection perforce, ClientRecord client, int changeNum, int shelveChangeNum, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(UnshelveAsync)}");
			
			// Need to mark those files as dirty - update the workspace with those files 
			// Delete is fine, but need to flag anything added
			Stopwatch timer = Stopwatch.StartNew();
			_logger.LogInformation("Unshelving changelist {Change}...", shelveChangeNum);

			// Query the contents of the shelved changelist
			List<DescribeRecord> records = await perforce.DescribeAsync(DescribeOptions.Shelved, -1, [shelveChangeNum], cancellationToken);
			if (records.Count != 1)
			{
				throw new WorkspaceMaterializationException($"Changelist {shelveChangeNum} is not shelved");
			}
			DescribeRecord lastRecord = records[0];
			if (lastRecord.Files.Count == 0)
			{
				throw new WorkspaceMaterializationException($"Changelist {shelveChangeNum} does not contain any shelved files");
			}

			// Query the location of each file
			List<PerforceResponse<WhereRecord>> whereResponseList = await perforce.TryWhereAsync(lastRecord.Files.Select(x => x.DepotFile).ToArray(), cancellationToken).ToListAsync(cancellationToken);
			Dictionary<string, WhereRecord> whereRecords = whereResponseList.Where(x => x.Succeeded).Select(x => x.Data).ToDictionary(x => x.DepotFile, x => x, StringComparer.OrdinalIgnoreCase);

			// Parse out all the list of deleted and modified files
			List<WhereRecord> deleteFiles = [];
			List<WhereRecord> writeFiles = [];
			foreach (DescribeFileRecord fileRecord in lastRecord.Files)
			{
				if (!whereRecords.TryGetValue(fileRecord.DepotFile, out WhereRecord? whereRecord))
				{
					_logger.LogInformation("Unable to get location of {File} in current workspace; ignoring.", fileRecord.DepotFile);
					continue;
				}

				switch (fileRecord.Action)
				{
					case FileAction.Delete:
					case FileAction.MoveDelete:
						deleteFiles.Add(whereRecord);
						break;
					case FileAction.Add:
					case FileAction.Edit:
					case FileAction.MoveAdd:
					case FileAction.Branch:
					case FileAction.Integrate:
						writeFiles.Add(whereRecord);
						break;
					case FileAction.Unknown:
						_logger.LogWarning("Ignoring file {File} with unknown action in shelved CL {Change}", fileRecord.DepotFile, shelveChangeNum);
						break;
					case FileAction.None:
					case FileAction.Purge:
					case FileAction.Archive:
					case FileAction.Import:
					default:
						throw new Exception($"Unknown action '{fileRecord.Action}' for shelved file {fileRecord.DepotFile}");
				}
			}

			List<WorkspaceFile> dirtyFiles = writeFiles
				.Concat(deleteFiles)
				.Select(x => new WorkspaceFile(x.ClientFile, x.Path))
				.ToList();
			
			await SaveStateAsync(TransactionStatus.Dirty, client.Name, changeNum, shelveChangeNum, dirtyFiles, cancellationToken);

			// Delete all the files
			foreach (WhereRecord deleteFile in deleteFiles)
			{
				string localPath = deleteFile.Path;
				if (File.Exists(localPath))
				{
					_logger.LogInformation("  Deleting {LocalPath}", localPath);
					FileUtils.ForceDeleteFile(localPath);
				}
			}

			// Use common paths with wild cards speed up the print operation with one call instead of many calls to print.
			_logger.LogInformation("Writing files from shelved changelist {Change}", shelveChangeNum);
			PerforceResponseList<PrintRecord> printResponse = await perforce.TryPrintAsync($"{SyncDir.FullName}{Path.DirectorySeparatorChar}...", $"//{perforce.Settings.ClientName}/...@={shelveChangeNum}", cancellationToken);
			if (!printResponse.Succeeded)
			{
				_logger.LogWarning("Unable to print shelved changelist: {Error}", printResponse.ToString());
			}

			foreach (PerforceResponse<PrintRecord> pr in printResponse.Take(MaxFilesLogged))
			{
				_logger.LogInformation("Unshelved {Action} {File}", pr.Data.Action, pr.Data.DepotFile);
			}
			if (printResponse.Count > MaxFilesLogged)
			{
				_logger.LogInformation(".. with {MoreCount} more files", printResponse.Count - MaxFilesLogged);
			}

			_logger.LogInformation("Completed in {TimeSeconds}s", $"{timer.Elapsed.TotalSeconds:0.0}");
			await SaveStateAsync(TransactionStatus.Clean, client.Name, changeNum, shelveChangeNum, dirtyFiles, cancellationToken);
		}
	
	internal static async Task<IPerforceConnection> CreatePerforceWithoutClientAsync(RpcAgentWorkspace agentWorkspace, ILogger logger, CancellationToken cancellationToken)
	{
		RpcAgentWorkspace raw = agentWorkspace;
		PerforceSettings tempSettings = new(raw.ServerAndPort, raw.UserName) { PreferNativeClient = PreferNativePerforceClient, Password = raw.Ticket };
		IPerforceConnection perforceWithoutClient = await PerforceConnection.CreateAsync(tempSettings, logger);
		
		if (raw.UserName != null)
		{
			if (!String.IsNullOrEmpty(raw.Ticket))
			{
				Environment.SetEnvironmentVariable("P4PASSWD", raw.Ticket);
			}
			if (!String.IsNullOrEmpty(raw.Password))
			{
				await perforceWithoutClient.LoginAsync(raw.Password, cancellationToken);
			}
			else
			{
				logger.LogInformation("Using locally logged in session for {UserName}", raw.UserName);
			}
		}

		return perforceWithoutClient;
	}
	
	internal static async Task<string> GetClientNameAsync(IPerforceConnection perforce, string prefix, string workspaceIdentifier, CancellationToken cancellationToken)
	{
		// Get the host name, and fill in any missing metadata about the connection
		InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);
		
		string? hostName = info.ClientHost;
		if (hostName == null)
		{
			throw new Exception("Unable to determine Perforce host name");
		}
		
		// Replace invalid characters in the workspace identifier with a '+' character
		// Append the slot index, if it's non-zero
		//			my $slot_idx = $optional_arguments->{'slot_idx'} || 0;
		//			$workspace_identifier .= sprintf("+%02d", $slot_idx) if $slot_idx;
		// If running on an edge server, append the server ID to the client name
		string edgeSuffix = String.Empty;
		if (info is { Services: not null, ServerId: not null })
		{
			string[] services = info.Services.Split(' ', StringSplitOptions.RemoveEmptyEntries);
			if (services.Any(x => x.Equals("edge-server", StringComparison.OrdinalIgnoreCase)))
			{
				edgeSuffix = $"+{info.ServerId}";
			}
		}
		
		string clientName = $"{prefix}{WorkspaceInfo.GetNormalizedHostName(hostName)}+{workspaceIdentifier}{edgeSuffix}";
		return clientName;
	}

	internal static async Task<IPerforceConnection> CreatePerforceWithClientAsync(RpcAgentWorkspace agentWorkspace, ILogger logger, CancellationToken cancellationToken)
	{
		IPerforceConnection perforceWithoutClient = await CreatePerforceWithoutClientAsync(agentWorkspace, logger, cancellationToken);
		string clientName = await GetClientNameAsync(perforceWithoutClient, "Horde+PM+", agentWorkspace.Identifier, cancellationToken);
		PerforceSettings perforceSettings = new (perforceWithoutClient.Settings) { ClientName = clientName, PreferNativeClient = PreferNativePerforceClient };
		return await PerforceConnection.CreateAsync(perforceSettings, logger);
	}

	internal async Task<IPerforceConnection> GetPerforceWithClientAsync()
	{
		if (_perforceWithClient != null)
		{
			return _perforceWithClient;
		}
		if (_perforceSettings == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}
		_perforceWithClient = await PerforceConnection.CreateAsync(_perforceSettings, _logger);
		return _perforceWithClient;
	}

	private async Task RevertInternalAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
	{
		if (_perforceSettings == null || _perforceWithClient == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}
		
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(RevertInternalAsync)}");
		using ILoggerProgress status = _logger.BeginProgressScope("Reverting files...");
		Stopwatch timer = Stopwatch.StartNew();

		State? state = await LoadStateAsync(cancellationToken);
		if (state is { DirtyFiles.Count: > 0 })
		{
			foreach (WorkspaceFile wf in state.DirtyFiles)
			{
				FileUtils.ForceDeleteFile(wf.LocalFile);
			}
			
			// When an agent is reassigned to a different edge server between jobs, DirtyFiles
			// persisted in State.json still carry the old client name prefix (e.g. //buildedge_01/...).
			// Remap them to the current client so the force-sync targets the right workspace.
			string oldClientPrefix = $"//{state.Client}/";
			string newClientPrefix = $"//{_perforceSettings!.ClientName}/";
			List<string> dirtyClientFiles = state.DirtyFiles
				.Select(x =>
				{
					string clientFile = x.ClientFile.StartsWith(oldClientPrefix, StringComparison.Ordinal)
						? newClientPrefix + x.ClientFile[oldClientPrefix.Length..]
						: x.ClientFile;
					return $"{clientFile}@{state.Changelist}";
				})
				.ToList();
			List<SyncRecord> syncedFiles = await _perforceWithClient.SyncAsync(
				EpicGames.Perforce.SyncOptions.Force, -1, _options.SyncThreads, _options.SyncBatch, _options.SyncBatchSize,
				-1, -1, dirtyClientFiles, cancellationToken).ToListAsync(cancellationToken);
			
			foreach (SyncRecord sr in syncedFiles.Take(MaxFilesLogged))
			{
				_logger.LogInformation("Restored dirty file: {File}", sr.DepotFile);
			}
			if (syncedFiles.Count > MaxFilesLogged)
			{
				_logger.LogInformation(".. with {MoreCount} more files", syncedFiles.Count - MaxFilesLogged);
			}
			
			await SaveStateAsync(TransactionStatus.Clean, state.Client, state.Changelist, state.ShelvedChangelist, [], cancellationToken);
		}

		List<RevertRecord> revertRecords = await perforce.RevertAsync(-1, null, RevertOptions.DeleteAddedFiles, "//...", cancellationToken);
		_logger.LogInformation("Reverted {FileCount} files.", revertRecords.Count);
		
		span.SetAttribute("horde.pm.num_files", revertRecords.Count);
		status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
	}

	private async Task CleanAsync(IPerforceConnection perforce, CancellationToken cancellationToken)
	{
		if (_perforceSettings == null || _perforceWithClient == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}
		
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(CleanAsync)}");
		using ILoggerProgress status = _logger.BeginProgressScope("Cleaning files...");
		Stopwatch timer = Stopwatch.StartNew();

		await SaveStateAsync(TransactionStatus.Dirty, cancellationToken);
		PerforceResponseList<CleanRecord> res = await perforce.TryCleanAsync(CleanOptions.ModifiedTimes, "//...", cancellationToken);
		if (res.Succeeded)
		{
			List<CleanRecord> cleanedFiles = res.Data;
			foreach (CleanRecord cr in cleanedFiles.Take(MaxFilesLogged))
			{
				_logger.LogInformation("Cleaned file: {File} {Action}", cr.DepotFile, cr.Action);
			}
			if (cleanedFiles.Count > MaxFilesLogged)
			{
				_logger.LogInformation(".. with {MoreCount} more files", cleanedFiles.Count - MaxFilesLogged);
			}
			span.SetAttribute("horde.pm.num_files", cleanedFiles.Count);
		}
		else
		{
			PerforceError? error = res.FirstError;
			if (error != null)
			{
				_logger.LogWarning("Clean failed: {Error}", error.ToString());
				span.SetAttribute("horde.pm.clean_error", error.Data ?? "Unknown error");
			}
			else
			{
				_logger.LogWarning("Clean failed with no error details");
			}
		}
		
		await SaveStateAsync(TransactionStatus.Clean, cancellationToken);
		status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
	}
	
	private async Task<(ClientRecord client, bool wasRecreated)> CreateOrUpdateClientAsync(bool forceCreate, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceMaterializer)}.{nameof(CreateOrUpdateClientAsync)}");
		if (_perforceSettings == null || _perforceWithoutClient == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}
		
		string clientName = _perforceSettings.ClientName!;
		string streamName = _options.AgentWorkspace.Stream;
		
		ClientRecord newClient = new (clientName, _perforceSettings.UserName, _workspaceDir.FullName)
		{
			Host = _options.HostName ?? Environment.MachineName,
			Type = "partitioned",
			Stream = streamName,
			Root = _workspaceDir.FullName,
			Options = _options.UseModTime ? ClientOptions.Clobber | ClientOptions.ModTime : ClientOptions.Clobber,
		};
		Directory.CreateDirectory(newClient.Root);
		
		using ILoggerProgress status = _logger.BeginProgressScope(forceCreate ? "Force creating client ..." : "Updating client ...");
		Stopwatch timer = Stopwatch.StartNew();
		
		bool wasRecreated = false;
		if (forceCreate)
		{
			await _perforceWithoutClient.TryDeleteClientAsync(DeleteClientOptions.None, clientName, cancellationToken);
			await _perforceWithoutClient.CreateClientAsync(newClient, cancellationToken);
			wasRecreated = true;
		}
		else
		{
			// Check if an existing and matching client exists.
			// There's no good way of doing this except checking if client's "Update" timestamp is unset or not
			ClientRecord existingClient = await _perforceWithoutClient.GetClientAsync(clientName, cancellationToken);
			bool clientExists = existingClient.Update != DateTime.MinValue;
			if (clientExists &&
				existingClient.Owner == newClient.Owner &&
				existingClient.Stream == newClient.Stream &&
				existingClient.Root == newClient.Root &&
				existingClient.Host == newClient.Host &&
				existingClient.Type == newClient.Type)
			{
				_logger.LogInformation("Found existing and matching client!");
				newClient = existingClient;
			}
			else
			{
				wasRecreated = true;
				_logger.LogInformation("Creating new client...");
				PerforceResponse createRes = await _perforceWithoutClient.TryCreateClientAsync(newClient, cancellationToken);
				if (!createRes.Succeeded)
				{
					_logger.LogInformation("Deleting and creating client...");
					await _perforceWithoutClient.TryDeleteClientAsync(DeleteClientOptions.None, clientName, cancellationToken);
					await _perforceWithoutClient.CreateClientAsync(newClient, cancellationToken);
				}
			}
		}
		
		status.Progress = $"({timer.Elapsed.TotalSeconds:0.0}s)";
		return (newClient, wasRecreated);
	}
	
	internal static string ConvertHordeViewToClientView(string hordeView, string stream, string clientName)
	{
		string exclude = "";
		string path = hordeView.TrimStart();
		bool isExclude = path[0] == '-';
		if (isExclude)
		{
			exclude = "-";
			path = path[1..]; // Strip minus
		}

		path = path[0] == '/' ? path : '/' + path;
		return $"{exclude}{stream}{path} //{clientName}{path}";
	}

	private Task SaveStateAsync(TransactionStatus status, string client, int changelist, int shelvedChangelist, List<WorkspaceFile> dirtyFiles, CancellationToken cancellationToken)
	{
		State pms = new (CurrentStateVersion, status, client, Identifier, _options.AgentWorkspace.Stream, changelist, shelvedChangelist, dirtyFiles);
		return SaveStateAsync(pms, cancellationToken);
	}
	
	private async Task SaveStateAsync(TransactionStatus status, CancellationToken cancellationToken)
	{
		State state = await LoadStateAsync(cancellationToken) ?? new State(CurrentStateVersion, status, "<not initialized>", Identifier, _options.AgentWorkspace.Stream, -1, -1, []);
		await SaveStateAsync(state with { Status = status }, cancellationToken);
	}
	
	private async Task SaveStateAsync(State state, CancellationToken cancellationToken)
	{
		try
		{
			Directory.CreateDirectory(_metadataDir.FullName); // Ensure metadata directory exists
			string json = JsonSerializer.Serialize(state, s_jsonOptions);
			await File.WriteAllTextAsync(_stateTempFile, json, cancellationToken);
			File.Move(_stateTempFile, _stateFile, overwrite: true);
		}
		catch (Exception e)
		{
			throw new WorkspaceMaterializationException("Failed to save local workspace state", e);
		}
	}
	
	private async Task<State?> LoadStateAsync(CancellationToken cancellationToken)
	{
		try
		{
			if (!File.Exists(_stateFile))
			{
				return null;
			}

			string json = await File.ReadAllTextAsync(_stateFile, cancellationToken);
			return JsonSerializer.Deserialize<State>(json, s_jsonOptions);
		}
		catch (Exception e)
		{
			_logger.LogWarning(e, "Failed to load local workspace state");
			return null;
		}
	}

	private void AddClientViewToLogger()
	{
		if (_streamView is not null && _lastSyncedChangeNum is not null)
		{
			_perforceLogger?.AddClientView(SyncDir, PerforceViewMap.Parse(_streamView), _lastSyncedChangeNum.Value);
		}
	}

	private static int GetDefaultThreadCount(int defaultValue) => Math.Min(defaultValue, Math.Max(Environment.ProcessorCount - 1, 1));
	
	internal async Task DeleteClientForTestAsync(CancellationToken cancellationToken)
	{
		if (_perforceSettings == null || _perforceWithoutClient == null)
		{
			throw new WorkspaceMaterializationException("Perforce settings/client not initialized!");
		}
		await _perforceWithoutClient.DeleteClientAsync(DeleteClientOptions.None, _perforceSettings.ClientName!, cancellationToken);
	}
	
	internal Task<State?> LoadStateForTestAsync(CancellationToken cancellationToken)
	{
		return LoadStateAsync(cancellationToken);
	}
}

class PerforceMaterializerFactory(IServiceProvider serviceProvider) : IWorkspaceMaterializerFactory
{
	/// <inheritdoc/>
	public async Task<IWorkspaceMaterializer?> CreateMaterializerAsync(string name, RpcAgentWorkspace workspaceInfo, DirectoryReference workspaceDir, bool forAutoSdk, CancellationToken cancellationToken)
	{
		if (name.Equals(PerforceMaterializer.TypeName, StringComparison.OrdinalIgnoreCase))
		{
			Tracer tracer = serviceProvider.GetRequiredService<Tracer>();
			if (forAutoSdk)
			{
				// Use ManagedWorkspace for AutoSDK until Horde-based views are implemented in PerforceMaterializer
				ILogger<ManagedWorkspaceMaterializer> mwLogger = serviceProvider.GetRequiredService<ILogger<ManagedWorkspaceMaterializer>>();
				return await ManagedWorkspaceMaterializer.CreateAsync(workspaceInfo, workspaceDir, true, false, tracer, mwLogger, cancellationToken);
			}
			
			ILogger<PerforceMaterializer> logger = serviceProvider.GetRequiredService<ILogger<PerforceMaterializer>>();
			PerforceMaterializerOptions options = PerforceMaterializerOptions.FromMethodString(workspaceDir.FullName, workspaceInfo);
			return new PerforceMaterializer(options, tracer, logger);
		}
		return null;
	}
}