// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Perforce;
using JobDriver.Parser;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Execution
{
	class TestExecutor : JobExecutor
	{
		public const string Name = "Test";

		readonly Dictionary<string, string> _arguments = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

		public TestExecutor(JobExecutorOptions options, Tracer tracer, ILogger logger)
			: base(options, tracer, logger)
		{
		}

		[return: NotNullIfNotNull("defaultValue")]
		string? GetArgument(string name, string? defaultValue)
		{
			string? result;
			if (!_arguments.TryGetValue(name, out result))
			{
				result = defaultValue;
			}
			return result;
		}

		bool GetArgument(string name, bool defaultValue)
		{
			bool result;
			if (!_arguments.TryGetValue(name, out string? value) || !Boolean.TryParse(value, out result))
			{
				result = defaultValue;
			}
			return result;
		}

		public override async Task InitializeAsync(RpcBeginBatchResponse batch, ILogger logger, CancellationToken cancellationToken)
		{
			await base.InitializeAsync(batch, logger, cancellationToken);

			logger.LogInformation("Initializing");

			foreach (string argument in batch.Arguments)
			{
				const string ArgumentPrefix = "-set:";
				if (argument.StartsWith(ArgumentPrefix, StringComparison.OrdinalIgnoreCase))
				{
					int keyIdx = ArgumentPrefix.Length;
					int valueIdx = argument.IndexOf('=', keyIdx);

					if (valueIdx != -1)
					{
						string key = argument.Substring(keyIdx, valueIdx - keyIdx);
						string value = argument.Substring(valueIdx + 1);
						_arguments[key] = value;
					}
				}
			}
		}

		protected override async Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("**** BEGIN JOB SETUP ****");

			await Task.Delay(5000, cancellationToken);

			JobStepOutcome warningOutcome = GetArgument("SimulateWarning", false) ? JobStepOutcome.Warnings : JobStepOutcome.Success;
			JobStepOutcome failureOutcome = GetArgument("SimulateError", false) ? JobStepOutcome.Failure : JobStepOutcome.Success;

			string projectName = GetArgument("Project", "UnknownProject");

			RpcUpdateGraphRequest updateGraph = new RpcUpdateGraphRequest();
			updateGraph.JobId = JobId.ToString();

			RpcCreateGroupRequest winEditorGroup = CreateGroup("Win64");
			winEditorGroup.Nodes.Add(CreateNode("Update Version Files", Array.Empty<string>(), JobStepOutcome.Success));
			winEditorGroup.Nodes.Add(CreateNode("Compile UnrealHeaderTool Win64", new string[] { "Update Version Files" }, JobStepOutcome.Success));
			winEditorGroup.Nodes.Add(CreateNode("Compile UnrealEditor Win64", new string[] { "Compile UnrealHeaderTool Win64" }, JobStepOutcome.Success));
			winEditorGroup.Nodes.Add(CreateNode($"Compile {projectName}Editor Win64", new string[] { "Compile UnrealHeaderTool Win64", "Compile UnrealEditor Win64" }, JobStepOutcome.Success));
			updateGraph.Groups.Add(winEditorGroup);

			RpcCreateGroupRequest winToolsGroup = CreateGroup("Win64");
			winToolsGroup.Nodes.Add(CreateNode("Compile Tools Win64", new string[] { "Compile UnrealHeaderTool Win64" }, warningOutcome));
			updateGraph.Groups.Add(winToolsGroup);

			RpcCreateGroupRequest winClientsGroup = CreateGroup("Win64");
			winClientsGroup.Nodes.Add(CreateNode($"Compile {projectName}Client Win64", new string[] { "Compile UnrealHeaderTool Win64" }, JobStepOutcome.Success));
			updateGraph.Groups.Add(winClientsGroup);

			RpcCreateGroupRequest winCooksGroup = CreateGroup("Win64");
			winCooksGroup.Nodes.Add(CreateNode($"Cook {projectName}Client Win64", new string[] { $"Compile {projectName}Editor Win64", "Compile Tools Win64" }, warningOutcome));
			winCooksGroup.Nodes.Add(CreateNode($"Stage {projectName}Client Win64", new string[] { $"Cook {projectName}Client Win64", "Compile Tools Win64" }, failureOutcome));
			winCooksGroup.Nodes.Add(CreateNode($"Publish {projectName}Client Win64", new string[] { $"Stage {projectName}Client Win64" }, JobStepOutcome.Success));
			updateGraph.Groups.Add(winCooksGroup);

			RpcCreateAggregateRequest aggregate = new RpcCreateAggregateRequest();
			aggregate.Name = "Full Build";
			aggregate.Nodes.Add($"Publish {projectName}Client Win64");
			updateGraph.Aggregates.Add(aggregate);

			Dictionary<string, string[]> dependencyMap = CreateDependencyMap(updateGraph.Groups);
			updateGraph.Labels.Add(CreateLabel("Editors", "Engine", new string[] { "Compile UnrealEditor Win64" }, Array.Empty<string>(), dependencyMap));
			updateGraph.Labels.Add(CreateLabel("Editors", "Project", new string[] { $"Compile {projectName}Editor Win64" }, Array.Empty<string>(), dependencyMap));
			updateGraph.Labels.Add(CreateLabel("Clients", "Project", new string[] { $"Cook {projectName}Client Win64" }, new string[] { $"Publish {projectName}Client Win64" }, dependencyMap));

			await JobRpc.UpdateGraphAsync(updateGraph, null, null, cancellationToken);

			logger.LogInformation("**** FINISH JOB SETUP ****");
			return true;
		}

		static RpcCreateGroupRequest CreateGroup(string agentType)
		{
			RpcCreateGroupRequest request = new RpcCreateGroupRequest();
			request.AgentType = agentType;
			return request;
		}

		static RpcCreateNodeRequest CreateNode(string name, string[] inputDependencies, JobStepOutcome outcome)
		{
			RpcCreateNodeRequest request = new RpcCreateNodeRequest();
			request.Name = name;
			request.InputDependencies.AddRange(inputDependencies);
			request.Properties.Add("Action", "Build");
			request.Properties.Add("Outcome", outcome.ToString());
			return request;
		}

		static RpcCreateLabelRequest CreateLabel(string category, string name, string[] requiredNodes, string[] includedNodes, Dictionary<string, string[]> dependencyMap)
		{
			RpcCreateLabelRequest request = new RpcCreateLabelRequest();
			request.DashboardName = name;
			request.DashboardCategory = category;
			request.RequiredNodes.AddRange(requiredNodes);
			request.IncludedNodes.AddRange(Enumerable.Union(requiredNodes, includedNodes).SelectMany(x => dependencyMap[x]).Distinct());
			return request;
		}

		static Dictionary<string, string[]> CreateDependencyMap(IEnumerable<RpcCreateGroupRequest> groups)
		{
			Dictionary<string, string[]> nameToDependencyNames = new Dictionary<string, string[]>();
			foreach (RpcCreateGroupRequest group in groups)
			{
				foreach (RpcCreateNodeRequest node in group.Nodes)
				{
					HashSet<string> dependencyNames = new HashSet<string> { node.Name };

					foreach (string inputDependency in node.InputDependencies)
					{
						dependencyNames.UnionWith(nameToDependencyNames[inputDependency]);
					}
					foreach (string orderDependency in node.OrderDependencies)
					{
						dependencyNames.UnionWith(nameToDependencyNames[orderDependency]);
					}

					nameToDependencyNames[node.Name] = dependencyNames.ToArray();
				}
			}
			return nameToDependencyNames;
		}

		protected override async Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("**** BEGIN NODE {StepName} ****", step.Name);

			await Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken);
			cancellationToken.ThrowIfCancellationRequested();

			JobStepOutcome outcome = Enum.Parse<JobStepOutcome>(step.Properties["Outcome"]);

			Dictionary<string, object> items = new Dictionary<string, object>
			{
				["hello"] = new { prop = 12345, prop2 = "world" },
				["world"] = new { prop = 123 }
			};
			await UploadTestDataAsync(step.StepId, items);

			foreach (KeyValuePair<string, string> credential in step.Credentials)
			{
				logger.LogInformation("Credential: {CredentialName}={CredentialValue}", credential.Key, credential.Value);
			}

			PerforceMetadataLogger perforceLogger = new PerforceMetadataLogger(logger);
			perforceLogger.AddClientView(new DirectoryReference("D:\\Test"), "//UE4/Main/...", 12345);

			using (LogParser filter = new LogParser(perforceLogger, new List<string>()))
			{
				if (outcome == JobStepOutcome.Warnings)
				{
					filter.WriteLine("D:\\Test\\Path\\To\\Source\\File.cpp(234): warning: This is a compilation warning");
					logger.LogWarning("This is a warning!");
					filter.WriteLine("warning: this is a test");
				}
				if (outcome == JobStepOutcome.Failure)
				{
					filter.WriteLine("D:\\Test\\Path\\To\\Source\\File.cpp(234): error: This is a compilation error");
					logger.LogError("This is an error!");
					filter.WriteLine("error: this is a test");
				}
			}

			logger.LogInformation("**** FINISH NODE {StepName} ****", step.Name);

			return true;
		}

		public override Task FinalizeAsync(ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("Finalizing");
			return Task.CompletedTask;
		}
	}

	class TestExecutorFactory : IJobExecutorFactory
	{
		readonly Tracer _tracer;
		readonly ILogger<TestExecutor> _logger;

		public string Name => TestExecutor.Name;

		public TestExecutorFactory(Tracer tracer, ILogger<TestExecutor> logger)
		{
			_tracer = tracer;
			_logger = logger;
		}

		public Task<JobExecutor> CreateExecutorAsync(RpcAgentWorkspace workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, CancellationToken cancellationToken)
		{
			return Task.FromResult<JobExecutor>(new TestExecutor(options, _tracer, _logger));
		}
	}
}
