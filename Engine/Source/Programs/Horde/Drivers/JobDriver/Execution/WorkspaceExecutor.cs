// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Specialized;
using System.Text.RegularExpressions;
using System.Web;
using EpicGames.Core;
using JobDriver.Utility;
using Horde.Common.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Execution
{
	class WorkspaceExecutor : JobExecutor
	{
		public const string Name = "Workspace";

		private DirectoryReference? _sharedStorageDir;
		private readonly IWorkspaceMaterializer _workspace;
		private readonly IWorkspaceMaterializer? _autoSdkWorkspace;

		public WorkspaceExecutor(JobExecutorOptions options, IWorkspaceMaterializer workspace, IWorkspaceMaterializer? autoSdkWorkspace, Tracer tracer, ILogger logger)
			: base(options, tracer, logger)
		{
			_workspace = workspace;
			_autoSdkWorkspace = autoSdkWorkspace;
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				_workspace.Dispose();
				_autoSdkWorkspace?.Dispose();
			}

			base.Dispose(disposing);
		}

		public override async Task InitializeAsync(RpcBeginBatchResponse batch, ILogger logger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = Tracer.StartActiveSpan($"{nameof(WorkspaceExecutor)}.{nameof(InitializeAsync)}");
			await base.InitializeAsync(batch, logger, cancellationToken);

			if (Batch.Change == 0)
			{
				throw new WorkspaceMaterializationException("Jobs with an empty change number are not supported");
			}

			// Setup and sync the AutoSDK workspace
			if (_autoSdkWorkspace != null)
			{
				SyncOptions syncOptions = new();
				await _autoSdkWorkspace.SyncAsync(IWorkspaceMaterializer.LatestChangeNumber, -1, syncOptions, cancellationToken);
			}

			// Sync the regular workspace
			{
				await _workspace.SyncAsync(Batch.Change, Batch.PreflightChange, new SyncOptions(), cancellationToken);

				// TODO: Purging of cache for ManagedWorkspace did happen here in WorkspaceInfo
				DeleteCachedBuildGraphManifests(_workspace.SyncDir, logger);
			}

			// Remove all the local settings directories
			PerforceExecutor.DeleteEngineUserSettings(logger);

			// Get the temp storage directory
			if (!String.IsNullOrEmpty(Batch.TempStorageDir))
			{
				string escapedStreamName = Regex.Replace(Batch.StreamName, "[^a-zA-Z0-9_-]", "+");
				_sharedStorageDir = DirectoryReference.Combine(new DirectoryReference(Batch.TempStorageDir), escapedStreamName, $"CL {Batch.Change} - Job {JobId}");
				CopyAutomationTool(_sharedStorageDir, _workspace.SyncDir, logger);
			}

			// Set any non-materializer specific environment variables for jobs
			EnvVars["IsBuildMachine"] = "1";
			EnvVars["uebp_LOCAL_ROOT"] = GetUnrealRootDirectoryFromWorkspace(_workspace.SyncDir).FullName;
			EnvVars["uebp_BuildRoot_P4"] = Batch.StreamName;
			EnvVars["uebp_BuildRoot_Escaped"] = Batch.StreamName.Replace('/', '+');
			EnvVars["uebp_CL"] = Batch.Change.ToString();
			EnvVars["uebp_CodeCL"] = Batch.CodeChange.ToString();

			foreach ((string key, string value) in _workspace.EnvironmentVariables)
			{
				EnvVars[key] = value;
			}

			if (_autoSdkWorkspace != null)
			{
				EnvVars["UE_SDKS_ROOT"] = _autoSdkWorkspace.SyncDir.FullName;
			}
		}

		/// <inheritdoc/>
		protected override async Task<bool> SetupAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			// Loop back to JobExecutor's SetupAsync again, but with workspace and shared storage dir set
			DirectoryReference workspaceDir = _workspace.SyncDir;
			return await SetupAsync(step, workspaceDir, _workspace.IsPerforceWorkspace, GetLogger(logger), cancellationToken);
		}

		/// <inheritdoc/>
		protected override async Task<bool> ExecuteAsync(JobStepInfo step, ILogger logger, CancellationToken cancellationToken)
		{
			// Loop back to JobExecutor's ExecuteAsync again, but with workspace and shared storage dir set
			DirectoryReference workspaceDir = _workspace.SyncDir;
			return await ExecuteAsync(step, workspaceDir, _workspace.IsPerforceWorkspace, GetLogger(logger), cancellationToken);
		}

		/// <inheritdoc/>
		public override async Task FinalizeAsync(ILogger logger, CancellationToken cancellationToken)
		{
			DirectoryReference workspaceDir = _workspace.SyncDir;
			await ExecuteLeaseCleanupScriptAsync(workspaceDir, logger);
			await TerminateProcessesAsync(TerminateCondition.AfterBatch, logger);

			if (_autoSdkWorkspace != null)
			{
				await _autoSdkWorkspace.FinalizeAsync(cancellationToken);
			}

			await _workspace.FinalizeAsync(cancellationToken);
		}

		private ILogger GetLogger(ILogger logger)
		{
			return _workspace.GetLogger(logger);
		}
	}

	class WorkspaceExecutorFactory : IJobExecutorFactory
	{
		private readonly IEnumerable<IWorkspaceMaterializerFactory> _materializerFactories;
		private readonly Tracer _tracer;
		private readonly ILoggerFactory _loggerFactory;

		public string Name => WorkspaceExecutor.Name;

		public WorkspaceExecutorFactory(IEnumerable<IWorkspaceMaterializerFactory> materializerFactories, Tracer tracer, ILoggerFactory loggerFactory)
		{
			_materializerFactories = materializerFactories;
			_tracer = tracer;
			_loggerFactory = loggerFactory;
		}

		public async Task<JobExecutor> CreateExecutorAsync(RpcAgentWorkspace workspaceInfo, RpcAgentWorkspace? autoSdkWorkspaceInfo, JobExecutorOptions options, CancellationToken cancellationToken)
		{
			IWorkspaceMaterializer? workspaceMaterializer = null;
			IWorkspaceMaterializer? autoSdkMaterializer = null;
			try
			{
				workspaceMaterializer = await CreateMaterializerAsync(options.JobOptions, workspaceInfo, options.WorkingDir, forAutoSdk: false, cancellationToken);
				if (autoSdkWorkspaceInfo != null)
				{
					autoSdkMaterializer = await CreateMaterializerAsync(options.JobOptions, autoSdkWorkspaceInfo, options.WorkingDir, forAutoSdk: true, cancellationToken);
				}

				return new WorkspaceExecutor(options, workspaceMaterializer, autoSdkMaterializer, _tracer, _loggerFactory.CreateLogger<WorkspaceExecutor>());
			}
			catch
			{
				autoSdkMaterializer?.Dispose();
				workspaceMaterializer?.Dispose();
				throw;
			}
		}

		internal async Task<IWorkspaceMaterializer> CreateMaterializerAsync(RpcJobOptions jobOptions, RpcAgentWorkspace workspaceInfo, DirectoryReference workspaceDir, bool forAutoSdk, CancellationToken cancellationToken)
		{
			const string DefaultMaterializer = ManagedWorkspaceMaterializer.TypeName;
			string name = DefaultMaterializer;
			
			if (!String.IsNullOrEmpty(workspaceInfo.Method))
			{
				NameValueCollection nameValues = HttpUtility.ParseQueryString(workspaceInfo.Method);
				name = nameValues["name"] ?? name;
			}
			
			if (!String.IsNullOrEmpty(jobOptions.WorkspaceMaterializer))
			{
				name = jobOptions.WorkspaceMaterializer;
			}
			
			foreach (IWorkspaceMaterializerFactory materializerFactory in _materializerFactories)
			{
				IWorkspaceMaterializer? materializer = await materializerFactory.CreateMaterializerAsync(name, workspaceInfo, workspaceDir, forAutoSdk, cancellationToken);
				if (materializer != null)
				{
					return materializer;
				}
			}
			throw new ArgumentException($"Unable to find materializer type '{name}'");
		}
	}
}
