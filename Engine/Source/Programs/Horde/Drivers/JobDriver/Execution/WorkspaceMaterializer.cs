// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;

namespace JobDriver.Execution;

/// <summary>
/// Exception for workspace materializer
/// </summary>
public class WorkspaceMaterializationException : Exception
{
	/// <summary>
	/// Constructor
	/// </summary>
	public WorkspaceMaterializationException(string? message) : base(message)
	{
	}

	/// <summary>
	/// Constructor
	/// </summary>
	public WorkspaceMaterializationException(string? message, Exception? innerException) : base(message, innerException)
	{
	}
}

/// <summary>
/// Options passed to SyncAsync
/// </summary>
public class SyncOptions
{
	/// <summary>
	/// Remove any files not referenced by changelist
	/// </summary>
	public bool RemoveUntracked { get; set; }

	/// <summary>
	/// If true, skip syncing actual file data and instead create empty placeholder files.
	/// Used for testing.
	/// </summary>
	public bool FakeSync { get; set; }
}

/// <summary>
/// Interface for materializing a file tree to the local file system.
/// One instance roughly equals one Perforce stream.
/// </summary>
public interface IWorkspaceMaterializer : IDisposable
{
	/// <summary>
	/// Placeholder for resolving the latest available change number of stream during sync
	/// </summary>
	public const int LatestChangeNumber = -2;

	/// <summary>
	/// Name of this materializer
	/// </summary>
	string Name { get; }
	
	/// <summary>
	/// Path to local file system directory where files from changelist are materialized
	/// </summary>
	DirectoryReference SyncDir { get; }
	
	/// <summary>
	/// Path to local file system directory where materializer contain all its metadata, state and synchronized files
	/// </summary>
	DirectoryReference BaseDir { get; }

	/// <summary>
	/// Identifier for this workspace
	/// </summary>
	string Identifier { get; }

	/// <summary>
	/// Environment variables expected to be set for applications executing inside the workspace
	/// Mostly intended for Perforce-specific variables when <see cref="IsPerforceWorkspace" /> is set to true
	/// </summary>
	IReadOnlyDictionary<string, string> EnvironmentVariables { get; }

	/// <summary>
	/// Whether the materialized workspace is a true Perforce workspace
	/// This flag is provided as a stop-gap solution to allow replacing ManagedWorkspace with WorkspaceMaterializer.
	/// It's *highly* recommended to set this to false for any new implementations of IWorkspaceMaterializer.
	/// </summary>
	bool IsPerforceWorkspace { get; }

	/// <summary>
	/// Get a logger capable of decorating log lines with materializer specific hints (resolving of paths or change numbers)
	/// </summary>
	/// <param name="logger">Base logger to wrap</param>
	/// <returns>A wrapped logger</returns>
	ILogger GetLogger(ILogger logger);

	/// <summary>
	/// Materialize (or sync) a Perforce stream at a given change number
	/// Once method has completed, file tree is available on disk.
	/// </summary>
	/// <param name="changeNum">Change number to materialize</param>
	/// <param name="shelveChangeNum">Preflight change number to add</param>
	/// <param name="options">Additional options</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	/// <exception cref="WorkspaceMaterializationException">Thrown if syncing fails</exception>
	/// <returns>Async task</returns>
	Task SyncAsync(int changeNum, int shelveChangeNum, SyncOptions options, CancellationToken cancellationToken);

	/// <summary>
	/// Finalize and clean file system.
	/// Usually called after the workspace has been used (e.g a build operation completed)
	/// </summary>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	/// <returns>Async task</returns>
	Task FinalizeAsync(CancellationToken cancellationToken);
	
	/// <summary>
	/// Resets a workspace
	/// Removes unneeded files from workspace as well as cleaning up resources or state in VCS server (e.g workspaces/clients)
	/// and syncs
	/// </summary>
	/// <param name="removeUntrackedFiles">Whether to remove local files not found on or tracked by VCS server</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	/// <returns>Async task</returns>
	Task ConformAsync(bool removeUntrackedFiles, CancellationToken cancellationToken);
}

/// <summary>
/// Factory for creating new workspace materializers
/// </summary>
public interface IWorkspaceMaterializerFactory
{
	/// <summary>
	/// Creates a new workspace materializer instance
	/// </summary>
	/// <param name="name">Name of the materializer to create</param>
	/// <param name="workspaceInfo">Agent workspace</param>
	/// <param name="workingDir">Working directory for the agent (ie. the root directory, not the workspace location)</param>
	/// <param name="forAutoSdk">Whether intended for AutoSDK materialization</param>
	/// <param name="cancellationToken">Cancellation token for the operation</param>
	/// <returns>A new workspace materializer instance</returns>
	Task<IWorkspaceMaterializer?> CreateMaterializerAsync(string name, RpcAgentWorkspace workspaceInfo, DirectoryReference workingDir, bool forAutoSdk = false, CancellationToken cancellationToken = default);
}

/// <summary>
/// Helper class for creating workspace materializer
/// </summary>
/// <param name="factories">List of materializer factories</param>
public class WorkspaceMaterializerFactory(IEnumerable<IWorkspaceMaterializerFactory> factories) : IWorkspaceMaterializerFactory
{
	/// <inheritdoc/>
	public async Task<IWorkspaceMaterializer?> CreateMaterializerAsync(string name, RpcAgentWorkspace agentWorkspace, DirectoryReference workingDir, bool forAutoSdk = false, CancellationToken cancellationToken = default)
	{
		foreach (IWorkspaceMaterializerFactory factory in factories)
		{
			IWorkspaceMaterializer? materializer = await factory.CreateMaterializerAsync(name, agentWorkspace, workingDir, forAutoSdk, cancellationToken);
			if (materializer != null)
			{
				return materializer;
			}
		}
		return null;
	}
	
	/// <summary>
	/// Creates a new workspace materializer instance, throws an exception if not found
	/// </summary>
	/// <param name="name">Name of the materializer to create</param>
	/// <param name="workspaceInfo">Agent workspace</param>
	/// <param name="workingDir">Working directory for the agent (ie. the root directory, not the workspace location)</param>
	/// <param name="forAutoSdk">Whether intended for AutoSDK materialization</param>
	/// <param name="cancellationToken">Cancellation token for the operation</param>
	/// <returns>A new workspace materializer instance</returns>
	public async Task<IWorkspaceMaterializer> CreateMaterializerOrThrowAsync(string name, RpcAgentWorkspace workspaceInfo, DirectoryReference workingDir, bool forAutoSdk = false, CancellationToken cancellationToken = default)
	{
		return await CreateMaterializerAsync(name, workspaceInfo, workingDir, forAutoSdk, cancellationToken) ?? throw new ArgumentException($"Unable to find materializer type '{name}'");
	}
}

