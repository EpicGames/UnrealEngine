<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <ImplicitUsings>enable</ImplicitUsings>
    <AssemblyName>JobDriver</AssemblyName>
    <PublishRepositoryUrl>true</PublishRepositoryUrl>
    <EmbedUntrackedSources>false</EmbedUntrackedSources>
    <NoWarn>$(NoWarn);NU3003</NoWarn>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Debug|AnyCPU'">
    <DocumentationFile>HordeAgentDriver.xml</DocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <None Remove="HordeAgentDriver.xml" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="..\..\HordeServer\Protos\horde\*.proto">
      <Link>Protos\%(FileName)%(Extension)</Link>
    </Content>
    <Content Include="..\..\HordeServer\Protos\horde\common\**\*.proto">
      <Link>Protos\horde\common\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
    <Content Include="..\..\HordeServer\Protos\horde\messages\**\*.proto">
      <Link>Protos\horde\messages\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
    <Content Include="..\..\HordeServer\Protos\horde\tasks\**\*.proto">
      <Link>Protos\horde\tasks\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.Core" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.EC2" Version="3.7.306.6" />
    <PackageReference Include="AWSSDK.SimpleSystemsManagement" Version="3.7.301.8" />
    <PackageReference Include="Google.Protobuf" Version="3.25.8" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Meziantou.Analyzer" Version="2.0.220">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.Extensions.Caching.Memory" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting.Abstractions" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting.WindowsServices" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http.Polly" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging.Abstractions" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Options.DataAnnotations" Version="10.0.6" />
    <PackageReference Include="Microsoft.VisualStudio.Threading.Analyzers" Version="17.14.15">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="OpenTelemetry" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.Console" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.OpenTelemetryProtocol" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Extensions.Hosting" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.GrpcNetClient" Version="1.8.0-beta.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.Http" Version="1.8.1" />
    <PackageReference Include="Serilog" Version="3.1.1" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="Serilog.Extensions.Logging" Version="8.0.0" />
    <PackageReference Include="Serilog.Settings.Configuration" Version="8.0.4" />
    <PackageReference Include="Serilog.Sinks.Console" Version="5.0.1" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
    <PackageReference Include="System.Diagnostics.PerformanceCounter" Version="10.0.6" />
    <PackageReference Include="System.ServiceProcess.ServiceController" Version="10.0.6" />
  </ItemGroup>
  
  <ItemGroup>
    <Protobuf Include="..\..\Plugins\Build\HordeServer.Build\Protos\horde\job_rpc.proto" ProtoRoot="..\..\Plugins\Build\HordeServer.Build\Protos" GrpcServices="Client" Access="Public" AdditionalImportDirs="..\..\..\Shared\EpicGames.Horde\Protos;..\..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\..\Plugins\Build\HordeServer.Build\Protos\horde\job_rpc_messages.proto" ProtoRoot="..\..\Plugins\Build\HordeServer.Build\Protos" Access="Public" AdditionalImportDirs="..\..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\..\Plugins\Build\HordeServer.Build\Protos\horde\common\label_change.proto" ProtoRoot="..\..\Plugins\Build\HordeServer.Build\Protos" Access="Public" />
    <Protobuf Include="..\..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\agent_workspace.proto" ProtoRoot="..\..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
    <Protobuf Include="..\..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\conform_task.proto" ProtoRoot="..\..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
    <Protobuf Include="..\..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\job_task.proto" ProtoRoot="..\..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.LogMatchers\EpicGames.LogMatchers.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
    <ProjectReference Include="..\..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
  </ItemGroup>

  <Target Name="DeleteUnusedFiles-Win64" AfterTargets="Publish" Condition="'$(Configuration)' == 'Release' AND '$(RuntimeIdentifier)' == 'win-x64'">
    <ItemGroup>
      <FilesToDelete Include="$(PublishDir)**/*.dylib" />
      <FilesToDelete Include="$(PublishDir)**/*.so.9" />
      <FilesToDelete Include="$(PublishDir)**/*.so" />
      <FilesToDelete Include="$(PublishDir)**/*.xml" />
      <FilesToDelete Include="$(PublishDir)**/oo2core*win32.dll" />
      <FilesToDelete Include="$(PublishDir)**/oo2core*winuwp*.dll" />
    </ItemGroup>
    <Delete Files="@(FilesToDelete)">
      <Output TaskParameter="DeletedFiles" ItemName="DeletedList" />
    </Delete>
    <Message Text="Deleted files:%0A@(DeletedList->'  %(Identity)','%0A')" Importance="high" />
  </Target>

  <ItemGroup>
    <Content Include="appsettings.json" Condition="Exists('appsettings.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
    </Content>
    <Content Include="..\..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\Drivers\JobDriver\appsettings.Epic.json" Condition="Exists('..\..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\Drivers\JobDriver\appsettings.Epic.json')">
      <Link>appsettings.Epic.json</Link>
      <DependentUpon>appsettings.json</DependentUpon>
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
    </Content>
  </ItemGroup>

</Project>
