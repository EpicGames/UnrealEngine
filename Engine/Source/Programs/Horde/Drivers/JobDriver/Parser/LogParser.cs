// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;

namespace JobDriver.Parser
{
	/// <summary>
	/// Turns raw text output into structured logging events
	/// </summary>
	public class LogParser : LogEventParser
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">The logger to receive parsed output messages</param>
		/// <param name="logEventSinks">Additional sinks to receive log events</param>
		public LogParser(ILogger logger, List<ILogEventSink>? logEventSinks = null)
			: base(logger, logEventSinks)
		{
			AddMatchers(LogMatcherCollections.AutomationUtilsMatchers);
			AddMatchers(LogMatcherCollections.UBTMatchers);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">The logger to receive parsed output messages</param>
		/// <param name="ignorePatterns">List of patterns to ignore</param>
		/// <param name="logEventSinks">Additional sinks to receive log events</param>
		public LogParser(ILogger logger, List<string> ignorePatterns, List<ILogEventSink>? logEventSinks = null)
			: base(logger, logEventSinks)
		{
			foreach (string ignorePattern in ignorePatterns)
			{
				try
				{
					IgnorePatterns.Add(new IgnorePatternEntry(new Regex(ignorePattern, RegexOptions.Compiled, IgnorePatternTimeout), "(inline)"));
				}
				catch (ArgumentException ex)
				{
					logger.LogWarning("Invalid ignore pattern: {Pattern} ({Message})", ignorePattern, ex.Message);
				}
			}

			AddMatchers(LogMatcherCollections.AutomationUtilsMatchers);
			AddMatchers(LogMatcherCollections.UBTMatchers);
		}

		/// <summary>
		/// Static constructor
		/// </summary>
		static LogParser()
		{
			System.Text.RegularExpressions.Regex.CacheSize = 1000;
		}
	}
}
