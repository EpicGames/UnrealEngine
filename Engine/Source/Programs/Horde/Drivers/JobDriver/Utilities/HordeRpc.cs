// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Jobs;
using OpenTelemetry.Trace;

// These partial classes/extensions operate on generated gRPC and Protobuf code.
// Warnings below are disabled to avoid documenting every class touched.
#pragma warning disable CS1591
#pragma warning disable CA1716
namespace Horde.Common.Rpc
{
	partial class RpcGetJobRequest
	{
		public RpcGetJobRequest(JobId jobId)
		{
			JobId = jobId.ToString();
		}
	}

	partial class RpcBeginBatchRequest
	{
		public RpcBeginBatchRequest(JobId jobId, JobStepBatchId batchId, LeaseId leaseId)
		{
			JobId = jobId.ToString();
			BatchId = batchId.ToString();
			LeaseId = leaseId.ToString();
		}
	}

	partial class RpcFinishBatchRequest
	{
		public RpcFinishBatchRequest(JobId jobId, JobStepBatchId batchId, LeaseId leaseId)
		{
			JobId = jobId.ToString();
			BatchId = batchId.ToString();
			LeaseId = leaseId.ToString();
		}
	}

	partial class RpcBeginStepRequest
	{
		public RpcBeginStepRequest(JobId jobId, JobStepBatchId batchId, LeaseId leaseId)
		{
			JobId = jobId.ToString();
			BatchId = batchId.ToString();
			LeaseId = leaseId.ToString();
		}
	}

	partial class RpcUpdateStepRequest
	{
		public RpcUpdateStepRequest(JobId jobId, JobStepBatchId batchId, JobStepId stepId, JobStepState state, JobStepOutcome outcome)
		{
			JobId = jobId.ToString();
			BatchId = batchId.ToString();
			StepId = stepId.ToString();
			State = (int)state;
			Outcome = (int)outcome;
		}
	}

	partial class RpcGetStepRequest
	{
		public RpcGetStepRequest(JobId jobId, JobStepBatchId batchId, JobStepId stepId)
		{
			JobId = jobId.ToString();
			BatchId = batchId.ToString();
			StepId = stepId.ToString();
		}
	}

	partial class RpcGetStepResponse
	{
		public RpcGetStepResponse(JobStepOutcome outcome, JobStepState state, bool abortRequested)
		{
			Outcome = (int)outcome;
			State = (int)state;
			AbortRequested = abortRequested;
		}
	}
}

namespace HordeCommon.Rpc.Tasks
{
	partial class RpcJobOptions
	{
		public void DecorateSpan(TelemetrySpan span)
		{
			const string Prefix = "horde.job.options.";
			span.SetAttribute(Prefix + "executor", Executor);
			span.SetAttribute(Prefix + "materializer", WorkspaceMaterializer);
			span.SetAttribute(Prefix + "driver", Driver);
			
			if (UseWine != null)
			{
				span.SetAttribute(Prefix + "use_wine", UseWine.Value);	
			}
			
			if (RunInSeparateProcess != null)
			{
				span.SetAttribute(Prefix + "separate_process", RunInSeparateProcess.Value);
			}
			
			if (Container.Enabled != null)
			{
				span.SetAttribute(Prefix + "container.enabled", Container.Enabled.Value);
				span.SetAttribute(Prefix + "container.image_url", Container.ImageUrl);
				span.SetAttribute(Prefix + "container.engine", Container.ContainerEngineExecutable);
				span.SetAttribute(Prefix + "container.extra_args", Container.ExtraArguments);
			}
		}
	}
}

#pragma warning restore CA1716
#pragma warning restore CS1591