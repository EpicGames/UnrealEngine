// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;

namespace JobDriver.Utility
{
	/// <summary>
	/// Logger which updates the job step outcome whenever a warning or error is detected.
	/// </summary>
	class JobStepLogger : IServerLogger
	{
		/// <summary>
		/// The current outcome for this step. Updated to reflect any errors and warnings that occurred.
		/// </summary>
		public JobStepOutcome Outcome => _outcome;

		readonly IHordeClient _hordeClient;
		readonly JobId _jobId;
		readonly JobStepBatchId _jobBatchId;
		readonly JobStepId _jobStepId;
		readonly bool _warnings;

		readonly IServerLogger _serverLogger;
		readonly ILogger _localLogger;
		JobStepOutcome _outcome;
		Task _updateOutcomeTask;
		readonly CancellationTokenSource _cancellationSource;
		readonly ILogger _internalLogger;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobStepLogger(IHordeClient hordeClient, LogId logId, ILogger localLogger, JobId jobId, JobStepBatchId jobBatchId, JobStepId jobStepId, bool? warnings, LogLevel outputLevel, ILogger internalLogger)
		{
			_hordeClient = hordeClient;
			_jobId = jobId;
			_jobBatchId = jobBatchId;
			_jobStepId = jobStepId;
			_warnings = warnings ?? true;
			_serverLogger = hordeClient.CreateServerLogger(logId, outputLevel);
			_localLogger = localLogger;
			_outcome = JobStepOutcome.Success;
			_updateOutcomeTask = Task.CompletedTask;
			_cancellationSource = new CancellationTokenSource();
			_internalLogger = internalLogger;
		}

		/// <inheritdoc/>
		public IDisposable? BeginScope<TState>(TState state) where TState : notnull
			=> _localLogger.BeginScope(state);

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			try
			{
				if (!_updateOutcomeTask.IsCompleted)
				{
					await _cancellationSource.CancelAsync();
					await _updateOutcomeTask;
					_cancellationSource.Dispose();
				}

				await _serverLogger.DisposeAsync();
			}
			catch (Exception e)
			{
				_internalLogger.LogError(e, "Error while disposing job step logger");
			}
		}

		/// <inheritdoc/>
		public bool IsEnabled(LogLevel logLevel)
			=> _serverLogger.IsEnabled(logLevel) || _localLogger.IsEnabled(logLevel);

		/// <inheritdoc/>
		public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
		{
			// Downgrade warnings to information if not required
			if (logLevel == LogLevel.Warning && !_warnings)
			{
				logLevel = LogLevel.Information;

				// If this is a json log event, we need to re-encode it to pick up the new log level.
				if (state is JsonLogEvent jsonEvent)
				{
					LogEvent logEvent = LogEvent.FromState(logLevel, eventId, state, exception, formatter);
					state = (TState)(object)new JsonLogEvent(logEvent);
				}
			}

			// Write to the local logger
			_localLogger.Log<TState>(logLevel, eventId, state, exception, formatter);

			// Forward the log data to the inner writer
			_serverLogger.Log<TState>(logLevel, eventId, state, exception, formatter);

			// Update the state of this job if this is an error status
			JobStepOutcome newOutcome = _outcome;
			if (logLevel == LogLevel.Error || logLevel == LogLevel.Critical)
			{
				newOutcome = JobStepOutcome.Failure;
			}
			else if (logLevel == LogLevel.Warning && Outcome != JobStepOutcome.Failure)
			{
				newOutcome = JobStepOutcome.Warnings;
			}

			// If it changed, create a new task to update the job state on the server
			if (newOutcome != _outcome)
			{
				Task prevUpdateTask = _updateOutcomeTask;
				_updateOutcomeTask = Task.Run(() => UpdateOutcomeAsync(prevUpdateTask, newOutcome, _cancellationSource.Token), _cancellationSource.Token);
				_outcome = newOutcome;
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync()
		{
			await _serverLogger.StopAsync();
			await _updateOutcomeTask;
		}

		async Task UpdateOutcomeAsync(Task prevTask, JobStepOutcome newOutcome, CancellationToken cancellationToken)
		{
			// Wait for the last task to complete
			await prevTask;

			// Update the outcome of this jobstep
			try
			{
				JobRpc.JobRpcClient jobRpc = await _hordeClient.CreateGrpcClientAsync<JobRpc.JobRpcClient>(cancellationToken);
				await jobRpc.UpdateStepAsync(new RpcUpdateStepRequest(_jobId, _jobBatchId, _jobStepId, JobStepState.Unspecified, newOutcome), cancellationToken: cancellationToken);
			}
			catch (Exception ex)
			{
				_internalLogger.LogWarning(ex, "Unable to update step outcome to {NewOutcome}", Outcome);
			}
		}
	}
}
