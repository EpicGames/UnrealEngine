// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;

namespace JobDriver.Utility
{
	/// <summary>
	/// Linux specific P/Invoke calls
	/// </summary>
	static class LinuxInterop
	{
		/// <summary>
		/// Get user identity
		/// </summary>
		/// <returns>Real user ID of the calling process</returns>
		[DllImport("libc", SetLastError = true)]
		internal static extern uint getuid();

		/// <summary>
		/// Get group identity
		/// </summary>
		/// <returns>Real group ID of the calling process</returns>
		[DllImport("libc", SetLastError = true)]
		internal static extern uint getgid();
	}
}
