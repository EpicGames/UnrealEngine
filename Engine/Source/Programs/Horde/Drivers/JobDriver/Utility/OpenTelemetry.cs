// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.Metrics;
using EpicGames.Core;
using EpicGames.Horde.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Exporter;
using OpenTelemetry.Logs;
using OpenTelemetry.Metrics;
using OpenTelemetry.Resources;
using OpenTelemetry.Trace;
using Serilog.Core;

namespace JobDriver.Utility;

/// <summary>
/// Serilog event enricher attaching trace and span ID for Datadog using current System.Diagnostics.Activity
/// </summary>
public class OpenTelemetryDatadogLogEnricher : ILogEventEnricher
{
	/// <inheritdoc />
	public void Enrich(Serilog.Events.LogEvent logEvent, ILogEventPropertyFactory propertyFactory)
	{
		if (Activity.Current != null)
		{
			string stringTraceId = Activity.Current.TraceId.ToString();
			string stringSpanId = Activity.Current.SpanId.ToString();
			string ddTraceId = Convert.ToUInt64(stringTraceId.Substring(16), 16).ToString();
			string ddSpanId = Convert.ToUInt64(stringSpanId, 16).ToString();

			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.trace_id", ddTraceId));
			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.span_id", ddSpanId));
		}
	}
}

/// <summary>
/// Helper for configuring OpenTelemetry
/// </summary>
public static class OpenTelemetryHelper
{
	private static ResourceBuilder? s_resourceBuilder;

	private static readonly Tracer s_noOpTracer = TracerProvider.Default.GetTracer("NoOp");
	private static readonly Meter s_noOpMeter = new("NoOp");

	/// <summary>
	/// Configure OpenTelemetry in Horde and ASP.NET
	/// </summary>
	/// <param name="services">Service collection for DI</param>
	/// <param name="settings">Current server settings</param>
	public static void Configure(IServiceCollection services, OpenTelemetrySettings settings)
	{
		if (settings.Enabled)
		{
			services.AddOpenTelemetry()
				.WithTracing(builder => ConfigureTracing(builder, settings))
				.WithMetrics(builder => ConfigureMetrics(builder, settings));
			
			services.AddSingleton<Tracer>(sp => sp.GetRequiredService<TracerProvider>().GetTracer(settings.ServiceName));
			services.AddSingleton<Meter>(sp => sp.GetRequiredService<Meter>());
		}
		else
		{
			// Configure a no-op tracer and meters when OpenTelemetry is disabled as they are still used in the codebase
			services.AddSingleton(s_noOpTracer);
			services.AddSingleton(s_noOpMeter);
		}
	}

	private static void ConfigureTracing(TracerProviderBuilder builder, OpenTelemetrySettings settings)
	{
		void DatadogHttpRequestEnricher(Activity activity, HttpRequestMessage message)
		{
			activity.SetTag("service.name", settings.ServiceName + "-http-client");
			activity.SetTag("operation.name", "http.request");
			string url = $"{message.Method} {message.Headers.Host}{message.RedactedRequestUri()?.LocalPath}";
			activity.DisplayName = url;
			activity.SetTag("resource.name", url);
		}

		builder
			.AddSource(settings.ServiceName)
			.AddHttpClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			.AddGrpcClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			.SetResourceBuilder(GetResourceBuilder(settings));

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	private static void ConfigureMetrics(MeterProviderBuilder builder, OpenTelemetrySettings settings)
	{
		builder.AddMeter(settings.ServiceName);
		builder.AddHttpClientInstrumentation();

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	/// <summary>
	/// Configure .NET logging with OpenTelemetry
	/// </summary>
	/// <param name="builder">Logging builder to modify</param>
	/// <param name="settings">Current server settings</param>
	public static void ConfigureLogging(ILoggingBuilder builder, OpenTelemetrySettings settings)
	{
		if (!settings.Enabled)
		{
			return;
		}

		builder.AddOpenTelemetry(options =>
		{
			options.IncludeScopes = true;
			options.IncludeFormattedMessage = true;
			options.ParseStateValues = true;
			options.SetResourceBuilder(GetResourceBuilder(settings));

			if (settings.EnableConsoleExporter)
			{
				options.AddConsoleExporter();
			}
		});
	}

	private static ResourceBuilder GetResourceBuilder(OpenTelemetrySettings settings)
	{
		if (s_resourceBuilder != null)
		{
			return s_resourceBuilder;
		}

		List<KeyValuePair<string, object>> attributes = settings.Attributes.Select(x => new KeyValuePair<string, object>(x.Key, x.Value)).ToList();
		s_resourceBuilder = ResourceBuilder.CreateDefault()
			.AddService(settings.ServiceName, serviceNamespace: settings.ServiceNamespace, serviceVersion: settings.ServiceVersion)
			.AddAttributes(attributes)
			.AddTelemetrySdk()
			.AddEnvironmentVariableDetector();

		return s_resourceBuilder;
	}
}