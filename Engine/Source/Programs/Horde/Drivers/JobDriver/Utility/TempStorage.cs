// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Text;
using System.Xml;
using System.Xml.Serialization;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Horde.Common.Rpc;
using Microsoft.Extensions.Logging;

#pragma warning disable CA1819 // Properties should not return arrays

namespace JobDriver.Utility
{
	/// <summary>
	/// Exception thrown by the temp storage system
	/// </summary>
	public sealed class TempStorageException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public TempStorageException(string message, Exception? innerException = null)
			: base(message, innerException)
		{
		}
	}

	/// <summary>
	/// Stores the name of a temp storage block
	/// </summary>
	public class TempStorageBlockRef
	{
		/// <summary>
		/// Name of the node
		/// </summary>
		[XmlAttribute]
		public string NodeName { get; set; }

		/// <summary>
		/// Name of the output from this node
		/// </summary>
		[XmlAttribute]
		public string OutputName { get; set; }

		/// <summary>
		/// Default constructor, for XML serialization.
		/// </summary>
		private TempStorageBlockRef()
		{
			NodeName = String.Empty;
			OutputName = String.Empty;
		}

		/// <summary>
		/// Construct a temp storage block
		/// </summary>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="outputName">Name of the node's output</param>
		public TempStorageBlockRef(string nodeName, string outputName)
		{
			NodeName = nodeName;
			OutputName = outputName;
		}

		/// <summary>
		/// Tests whether two temp storage blocks are equal
		/// </summary>
		/// <param name="other">The object to compare against</param>
		/// <returns>True if the blocks are equivalent</returns>
		public override bool Equals(object? other) => other is TempStorageBlockRef otherBlock && NodeName.Equals(otherBlock.NodeName, StringComparison.OrdinalIgnoreCase) && OutputName.Equals(otherBlock.OutputName, StringComparison.OrdinalIgnoreCase);

		/// <summary>
		/// Returns a hash code for this block name
		/// </summary>
		/// <returns>Hash code for the block</returns>
		public override int GetHashCode() => HashCode.Combine(NodeName, OutputName);

		/// <summary>
		/// Returns the name of this block for debugging purposes
		/// </summary>
		/// <returns>Name of this block as a string</returns>
		public override string ToString() => $"{NodeName}/{OutputName}";
	}

	/// <summary>
	/// Information about a single file in temp storage
	/// </summary>
	[DebuggerDisplay("{RelativePath}")]
	public class TempStorageFile : IEquatable<TempStorageFile>
	{
		/// <summary>
		/// The path of the file, relative to the engine root. Stored using forward slashes.
		/// </summary>
		[XmlAttribute]
		public string RelativePath { get; set; }

		/// <summary>
		/// The last modified time of the file, in UTC ticks since the Epoch.
		/// </summary>
		[XmlAttribute]
		public long LastWriteTimeUtcTicks { get; set; }

		/// <summary>
		/// Length of the file
		/// </summary>
		[XmlAttribute]
		public long Length { get; set; }

		/// <summary>
		/// Digest for the file. Not all files are hashed.
		/// </summary>
		[XmlAttribute]
		public string? Digest { get; set; }

		/// <summary>
		/// Default constructor, for XML serialization.
		/// </summary>
		private TempStorageFile()
		{
			RelativePath = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public TempStorageFile(string relativePath, long lastWriteTimeUtcTicks, long length, string? digest)
		{
			RelativePath = relativePath;
			LastWriteTimeUtcTicks = lastWriteTimeUtcTicks;
			Length = length;
			Digest = digest;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileInfo">File to be added</param>
		/// <param name="rootDir">Root directory to store paths relative to</param>
		public TempStorageFile(FileInfo fileInfo, DirectoryReference rootDir)
		{
			// Check the file exists and is in the right location
			FileReference file = new FileReference(fileInfo);
			if (!file.IsUnderDirectory(rootDir))
			{
				throw new TempStorageException($"Attempt to add file to temp storage manifest that is outside the root directory ({file.FullName})");
			}
			if (!fileInfo.Exists)
			{
				throw new TempStorageException($"Attempt to add file to temp storage manifest that does not exist ({file.FullName})");
			}

			RelativePath = file.MakeRelativeTo(rootDir).Replace(Path.DirectorySeparatorChar, '/');
			LastWriteTimeUtcTicks = fileInfo.LastWriteTimeUtc.Ticks;
			Length = fileInfo.Length;

			if (GenerateDigest())
			{
				Digest = ContentHash.MD5(file).ToString();
			}
		}

		/// <summary>
		/// Compare stored for this file with the one on disk, and output an error if they differ.
		/// </summary>
		/// <param name="rootDir">Root directory for this branch</param>
		/// <param name="logger">Logger for output</param>
		/// <returns>True if the files are identical, false otherwise</returns>
		public bool Compare(DirectoryReference rootDir, ILogger logger)
		{
			string? message;
			if (Compare(rootDir, out message))
			{
				if (message != null)
				{
					logger.LogInformation("{Message}", message);
				}
				return true;
			}
			else
			{
				if (message != null)
				{
					logger.LogError("{Message}", message);
				}
				return false;
			}
		}

		/// <summary>
		/// Compare stored for this file with the one on disk, and output an error if they differ.
		/// </summary>
		/// <param name="rootDir">Root directory for this branch</param>
		/// <param name="message">Message describing the difference</param>
		/// <returns>True if the files are identical, false otherwise</returns>
		public bool Compare(DirectoryReference rootDir, [NotNullWhen(false)] out string? message)
		{
			FileReference localFile = ToFileReference(rootDir);

			// Get the local file info, and check it exists
			FileInfo info = new FileInfo(localFile.FullName);
			if (!info.Exists)
			{
				message = String.Format("Missing file from manifest - {0}", RelativePath);
				return false;
			}

			// Check the size matches
			if (info.Length != Length)
			{
				if (TempStorage.IsDuplicateBuildProduct(localFile))
				{
					message = String.Format("Ignored file size mismatch for {0} - was {1} bytes, expected {2} bytes", RelativePath, info.Length, Length);
					return true;
				}
				else
				{
					message = String.Format("File size differs from manifest - {0} is {1} bytes, expected {2} bytes", RelativePath, info.Length, Length);
					return false;
				}
			}

			// Check the timestamp of the file matches. On FAT filesystems writetime has a two seconds resolution (see http://msdn.microsoft.com/en-us/library/windows/desktop/ms724290%28v=vs.85%29.aspx)
			TimeSpan timeDifference = new TimeSpan(info.LastWriteTimeUtc.Ticks - LastWriteTimeUtcTicks);
			if (timeDifference.TotalSeconds >= -2 && timeDifference.TotalSeconds <= +2)
			{
				message = null;
				return true;
			}

			// Check if the files have been modified
			DateTime expectedLocal = new DateTime(LastWriteTimeUtcTicks, DateTimeKind.Utc).ToLocalTime();
			if (Digest != null)
			{
				string localDigest = ContentHash.MD5(localFile).ToString();
				if (Digest.Equals(localDigest, StringComparison.Ordinal))
				{
					message = null;
					return true;
				}
				else
				{
					message = String.Format("Digest mismatch for {0} - was {1} ({2}), expected {3} ({4}), TimeDifference {5}", RelativePath, localDigest, info.LastWriteTime, Digest, expectedLocal, timeDifference);
					return false;
				}
			}
			else
			{
				if (RequireMatchingTimestamps() && !TempStorage.IsDuplicateBuildProduct(localFile))
				{
					message = String.Format("File date/time mismatch for {0} - was {1}, expected {2}, TimeDifference {3}", RelativePath, info.LastWriteTime, expectedLocal, timeDifference);
					return false;
				}
				else
				{
					message = String.Format("Ignored file date/time mismatch for {0} - was {1}, expected {2}, TimeDifference {3}", RelativePath, info.LastWriteTime, expectedLocal, timeDifference);
					return true;
				}
			}
		}

		/// <summary>
		/// Whether we should compare timestamps for this file. Some build products are harmlessly overwritten as part of the build process, so we flag those here.
		/// </summary>
		/// <returns>True if we should compare the file's timestamp, false otherwise</returns>
		bool RequireMatchingTimestamps()
		{
			return !RelativePath.Contains("/Binaries/DotNET/", StringComparison.OrdinalIgnoreCase) && !RelativePath.Contains("/Binaries/Mac/", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Determines whether to generate a digest for the current file
		/// </summary>
		/// <returns>True to generate a digest for this file, rather than relying on timestamps</returns>
		bool GenerateDigest()
		{
			return RelativePath.EndsWith(".version", StringComparison.OrdinalIgnoreCase) || RelativePath.EndsWith(".modules", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Determine whether to serialize the digest property
		/// </summary>
		/// <returns></returns>
		public bool ShouldSerializeDigest()
		{
			return Digest != null;
		}

		/// <summary>
		/// Gets a local file reference for this file, given a root directory to base it from.
		/// </summary>
		/// <param name="rootDir">The local root directory</param>
		/// <returns>Reference to the file</returns>
		public FileReference ToFileReference(DirectoryReference rootDir)
		{
			return FileReference.Combine(rootDir, RelativePath.Replace('/', Path.DirectorySeparatorChar));
		}

		/// <inheritdoc/>
		public override bool Equals(object? other)
			=> Equals(other as TempStorageFile);

		/// <inheritdoc/>
		public bool Equals(TempStorageFile? other)
		{
			return other != null
				&& String.Equals(RelativePath, other.RelativePath, StringComparison.Ordinal)
				&& LastWriteTimeUtcTicks == other.LastWriteTimeUtcTicks
				&& Length == other.Length
				&& (Digest == null || other.Digest == null || String.Equals(Digest, other.Digest, StringComparison.Ordinal));
		}

		/// <inheritdoc/>
		public override int GetHashCode()
			=> HashCode.Combine(RelativePath, LastWriteTimeUtcTicks, Length);
	}

	/// <summary>
	/// A manifest storing information about build products for a node's output
	/// </summary>
	[XmlRoot(ElementName = "TempStorageManifest")]
	public class TempStorageBlockManifest
	{
		/// <summary>
		/// List of output files
		/// </summary>
		[XmlArray]
		[XmlArrayItem("File")]
		public TempStorageFile[] Files { get; set; }

		/// <summary>
		/// Construct a static Xml serializer to avoid throwing an exception searching for the reflection info at runtime
		/// </summary>
		static readonly XmlSerializer s_serializer = XmlSerializer.FromTypes(new Type[] { typeof(TempStorageBlockManifest) })[0]!;

		/// <summary>
		/// Construct an empty temp storage manifest
		/// </summary>
		private TempStorageBlockManifest()
		{
			Files = Array.Empty<TempStorageFile>();
		}

		/// <summary>
		/// Construct a temp storage manifest
		/// </summary>
		public TempStorageBlockManifest(TempStorageFile[] files)
		{
			Files = files;
		}

		/// <summary>
		/// Creates a manifest from a flat list of files (in many folders) and a BaseFolder from which they are rooted.
		/// </summary>
		/// <param name="files">List of full file paths</param>
		/// <param name="rootDir">Root folder for all the files. All files must be relative to this RootDir.</param>
		public TempStorageBlockManifest(FileInfo[] files, DirectoryReference rootDir)
		{
			Files = files.Select(x => new TempStorageFile(x, rootDir)).ToArray();
		}

		/// <summary>
		/// Gets the total size of the files stored in this manifest
		/// </summary>
		/// <returns>The total size of all files</returns>
		public long GetTotalSize()
		{
			long result = 0;
			foreach (TempStorageFile file in Files)
			{
				result += file.Length;
			}
			return result;
		}

		/// <summary>
		/// Load a manifest from disk
		/// </summary>
		/// <param name="file">File to load</param>
		public static TempStorageBlockManifest Load(FileReference file)
		{
			using (StreamReader reader = new(file.FullName))
			{
				XmlReaderSettings settings = new XmlReaderSettings();
				using (XmlReader xmlReader = XmlReader.Create(reader, settings))
				{
					return (TempStorageBlockManifest)s_serializer.Deserialize(xmlReader)!;
				}
			}
		}

		/// <summary>
		/// Saves a manifest to disk
		/// </summary>
		/// <param name="file">File to save</param>
		public void Save(FileReference file)
		{
			using (StreamWriter writer = new StreamWriter(file.FullName))
			{
				XmlWriterSettings writerSettings = new() { Indent = true };
				using (XmlWriter xmlWriter = XmlWriter.Create(writer, writerSettings))
				{
					s_serializer.Serialize(xmlWriter, this);
				}
			}
		}
	}

	/// <summary>
	/// Stores the contents of a tagged file set
	/// </summary>
	[XmlRoot(ElementName = "TempStorageFileList")]
	public class TempStorageTagManifest
	{
		/// <summary>
		/// List of files that are in this tag set, relative to the root directory
		/// </summary>
		[XmlArray]
		[XmlArrayItem("LocalFile")]
		public string[] LocalFiles { get; set; }

		/// <summary>
		/// List of files that are in this tag set, but not relative to the root directory
		/// </summary>
		[XmlArray]
		[XmlArrayItem("LocalFile")]
		public string[] ExternalFiles { get; set; }

		/// <summary>
		/// List of referenced storage blocks
		/// </summary>
		[XmlArray]
		[XmlArrayItem("Block")]
		public TempStorageBlockRef[] Blocks { get; set; }

		/// <summary>
		/// Construct a static Xml serializer to avoid throwing an exception searching for the reflection info at runtime
		/// </summary>
		static readonly XmlSerializer s_serializer = XmlSerializer.FromTypes(new Type[] { typeof(TempStorageTagManifest) })[0]!;

		/// <summary>
		/// Construct an empty file list for deserialization
		/// </summary>
		private TempStorageTagManifest()
		{
			LocalFiles = Array.Empty<string>();
			ExternalFiles = Array.Empty<string>();
			Blocks = Array.Empty<TempStorageBlockRef>();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="localFiles"></param>
		/// <param name="externalFiles"></param>
		/// <param name="blocks"></param>
		public TempStorageTagManifest(string[] localFiles, string[] externalFiles, TempStorageBlockRef[] blocks)
		{
			LocalFiles = localFiles;
			ExternalFiles = externalFiles;
			Blocks = blocks;
		}

		/// <summary>
		/// Creates a manifest from a flat list of files (in many folders) and a BaseFolder from which they are rooted.
		/// </summary>
		/// <param name="files">List of full file paths</param>
		/// <param name="rootDir">Root folder for all the files. All files must be relative to this RootDir.</param>
		/// <param name="blocks">Referenced storage blocks required for these files</param>
		public TempStorageTagManifest(IEnumerable<FileReference> files, DirectoryReference rootDir, IEnumerable<TempStorageBlockRef> blocks)
		{
			List<string> newLocalFiles = new List<string>();
			List<string> newExternalFiles = new List<string>();
			foreach (FileReference file in files)
			{
				if (file.IsUnderDirectory(rootDir))
				{
					newLocalFiles.Add(file.MakeRelativeTo(rootDir).Replace(Path.DirectorySeparatorChar, '/'));
				}
				else
				{
					newExternalFiles.Add(file.FullName.Replace(Path.DirectorySeparatorChar, '/'));
				}
			}
			LocalFiles = newLocalFiles.ToArray();
			ExternalFiles = newExternalFiles.ToArray();

			Blocks = blocks.ToArray();
		}

		/// <summary>
		/// Load this list of files from disk
		/// </summary>
		/// <param name="file">File to load</param>
		public static TempStorageTagManifest Load(FileReference file)
		{
			using (StreamReader reader = new StreamReader(file.FullName))
			{
				XmlReaderSettings settings = new XmlReaderSettings();
				using (XmlReader xmlReader = XmlReader.Create(reader, settings))
				{
					return (TempStorageTagManifest)s_serializer.Deserialize(xmlReader)!;
				}
			}
		}

		/// <summary>
		/// Saves this list of files to disk
		/// </summary>
		/// <param name="file">File to save</param>
		public void Save(FileReference file)
		{
			using (StreamWriter writer = new StreamWriter(file.FullName))
			{
				XmlWriterSettings writerSettings = new() { Indent = true };
				using (XmlWriter xmlWriter = XmlWriter.Create(writer, writerSettings))
				{
					s_serializer.Serialize(xmlWriter, this);
				}
			}
		}

		/// <summary>
		/// Converts this file list into a set of FileReference objects
		/// </summary>
		/// <param name="rootDir">The root directory to rebase local files</param>
		/// <returns>Set of files</returns>
		public HashSet<FileReference> ToFileSet(DirectoryReference rootDir)
		{
			HashSet<FileReference> files = new HashSet<FileReference>();
			files.UnionWith(LocalFiles.Select(x => FileReference.Combine(rootDir, x)));
			files.UnionWith(ExternalFiles.Select(x => new FileReference(x)));
			return files;
		}
	}

	/// <summary>
	/// Tracks the state of the current build job using the filesystem, allowing jobs to be restarted after a failure or expanded to include larger targets, and 
	/// providing a proxy for different machines executing parts of the build in parallel to transfer build products and share state as part of a build system.
	/// 
	/// If a shared temp storage directory is provided - typically a mounted path on a network share - all build products potentially needed as inputs by another node
	/// are compressed and copied over, along with metadata for them (see TempStorageFile) and flags for build events that have occurred (see TempStorageEvent).
	/// 
	/// The local temp storage directory contains the same information, with the exception of the archived build products. Metadata is still kept to detect modified 
	/// build products between runs. If data is not present in local temp storage, it's retrieved from shared temp storage and cached in local storage.
	/// </summary>
	static class TempStorage
	{
		/// <summary>
		/// Gets the ref name for a particular node
		/// </summary>
		/// <param name="name">Name of the node producing the artifact</param>
		public static ArtifactName GetArtifactNameForNode(string name)
		{
			StringBuilder builder = new StringBuilder();

			int prefixLength = Math.Min(name.Length, StringId.MaxLength - 10);
			bool appendHash = name.Length > prefixLength;

			for (int idx = 0; idx < prefixLength; idx++)
			{
				if (name[idx] >= 'A' && name[idx] <= 'Z')
				{
					builder.Append((char)(name[idx] + 'a' - 'A'));
				}
				else if ((name[idx] >= 'a' && name[idx] <= 'z') || (name[idx] >= '0' && name[idx] <= '9'))
				{
					builder.Append(name[idx]);
				}
				else if (name[idx] == ' ')
				{
					builder.Append('-');
				}
				else
				{
					appendHash = true;
				}
			}

			if (appendHash)
			{
				IoHash hash = IoHash.Compute(Encoding.UTF8.GetBytes(name.ToUpperInvariant()));
				builder.Append('-');
				builder.Append(hash.ToString(), 0, 8);
			}

			return new ArtifactName(builder.ToString());
		}

		/// <summary>
		/// Reads a set of tagged files from disk
		/// </summary>
		/// <param name="hordeClient">The job rpc interface</param>
		/// <param name="jobId"></param>
		/// <param name="stepId"></param>
		/// <param name="nodeName">Name of the node which produced the tag set</param>
		/// <param name="tagName">Name of the tag, with a '#' prefix</param>
		/// <param name="manifestDir">The local directory containing manifests</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken"></param>
		/// <returns>The set of files</returns>
		public static async Task<TempStorageTagManifest> RetrieveTagAsync(IHordeClient hordeClient, JobId jobId, JobStepId stepId, string nodeName, string tagName, DirectoryReference manifestDir, ILogger logger, CancellationToken cancellationToken)
		{
			// Try to read the tag set from the local directory
			FileReference localFileListLocation = GetTagManifestLocation(manifestDir, nodeName, tagName);
			if (FileReference.Exists(localFileListLocation))
			{
				logger.LogInformation("Reading tag \"{NodeName}\":\"{TagName}\" from {File}", nodeName, tagName, localFileListLocation.FullName);
			}
			else
			{
				ArtifactName artifactName = GetArtifactNameForNode(nodeName);
				ArtifactType artifactType = ArtifactType.StepOutput;

				RpcGetJobArtifactRequest artifactRequest = new RpcGetJobArtifactRequest();
				artifactRequest.JobId = jobId.ToString();
				artifactRequest.StepId = stepId.ToString();
				artifactRequest.Name = artifactName.ToString();
				artifactRequest.Type = artifactType.ToString();

				JobRpc.JobRpcClient jobRpc = await hordeClient.CreateGrpcClientAsync<JobRpc.JobRpcClient>(cancellationToken);
				RpcGetJobArtifactResponse artifact = await jobRpc.GetArtifactAsync(artifactRequest, cancellationToken: cancellationToken);

				NamespaceId namespaceId = new NamespaceId(artifact.NamespaceId);
				RefName refName = new RefName(artifact.RefName);

				logger.LogInformation("Reading tag \"{NodeName}\":\"{TagName}\" from temp storage (artifact: {ArtifactId} '{ArtifactName}' ({ArtifactType}), ns: {NamespaceId}, ref: {RefName}, localFile: {LocalFile})", nodeName, tagName, ArtifactId.Parse(artifact.Id), artifactName, artifactType, namespaceId, refName, localFileListLocation);

				IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(namespaceId, artifact.Token);
				DirectoryNode node = await storageNamespace.ReadRefTargetAsync<DirectoryNode>(artifact.RefName, cancellationToken: cancellationToken);

				FileEntry fileEntry = node.GetFileEntry(localFileListLocation.GetFileName());
				DirectoryReference.CreateDirectory(localFileListLocation.Directory);
				await fileEntry.CopyToFileAsync(localFileListLocation.ToFileInfo(), cancellationToken);
			}
			return TempStorageTagManifest.Load(localFileListLocation);
		}

		/// <summary>
		/// Saves a tag to temp storage
		/// </summary>
		/// <param name="manifestDir">Directory containing manifests</param>
		/// <param name="nodeName">Name of the node to store</param>
		/// <param name="tagName">Name of the output tag</param>
		/// <param name="workspaceDir">Root directory for the build products</param>
		/// <param name="files">Files in the tag</param>
		/// <param name="blocks"></param>
		/// <param name="writer">Writer for output</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<FileEntry> ArchiveTagAsync(DirectoryReference manifestDir, string nodeName, string tagName, DirectoryReference workspaceDir, IEnumerable<FileReference> files, TempStorageBlockRef[] blocks, IBlobWriter writer, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("Creating output tag \"{TagName}\"", tagName);

			TempStorageTagManifest fileList = new TempStorageTagManifest(files, workspaceDir, blocks);

			FileReference localFileListLocation = GetTagManifestLocation(manifestDir, nodeName, tagName);
			fileList.Save(localFileListLocation);

			using ChunkedDataWriter fileNodeWriter = new ChunkedDataWriter(writer, new ChunkingOptions());
			ChunkedData fileNodeData = await fileNodeWriter.CreateAsync(localFileListLocation.ToFileInfo(), cancellationToken);

			return new FileEntry(localFileListLocation.GetFileName(), FileEntryFlags.None, fileNodeData);
		}

		/// <summary>
		/// Gets the directory name for storing data from a block
		/// </summary>
		/// <param name="blockName"></param>
		/// <returns></returns>
		static string GetBlockDirectoryName(string blockName)
		{
			if (String.IsNullOrEmpty(blockName))
			{
				return "block";
			}
			else
			{
				return $"block-{blockName}";
			}
		}

		/// <summary>
		/// Saves the given files (that should be rooted at the branch root) to a shared temp storage manifest with the given temp storage node and game.
		/// </summary>
		/// <param name="manifestDir">Directory containing manifests</param>
		/// <param name="nodeName">Name of the node producing the block</param>
		/// <param name="blockName">Name of the output block</param>
		/// <param name="workspaceDir">Root directory for the build products</param>
		/// <param name="files">Files to add to the block</param>
		/// <param name="writer">Writer for output</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The created manifest instance (which has already been saved to disk).</returns>
		public static async Task<DirectoryEntry> ArchiveBlockAsync(DirectoryReference manifestDir, string nodeName, string blockName, DirectoryReference workspaceDir, IEnumerable<FileReference> files, IBlobWriter writer, ILogger logger, CancellationToken cancellationToken)
		{
			string blockDirectoryName = GetBlockDirectoryName(blockName);
			logger.LogInformation("Creating output block \"{BlockName}\" (as {DirectoryName})", blockName, blockDirectoryName);

			// Create a manifest for the given build products
			FileInfo[] fileInfos = files.Select(x => new FileInfo(x.FullName)).ToArray();
			TempStorageBlockManifest manifest = new TempStorageBlockManifest(fileInfos, workspaceDir);

			FileReference manifestLocation = GetBlockManifestLocation(manifestDir, nodeName, blockName);
			manifest.Save(manifestLocation);

			List<FileInfo> archiveFiles = new List<FileInfo>(fileInfos);
			archiveFiles.Add(manifestLocation.ToFileInfo());

			// Create the file tree
			DirectoryNode rootNode = new DirectoryNode();
			await rootNode.AddFilesAsync(workspaceDir, archiveFiles, writer, progress: new UpdateStatsLogger(logger), cancellationToken: cancellationToken);

			IHashedBlobRef<DirectoryNode> rootNodeRef = await writer.WriteBlobAsync(rootNode, cancellationToken: cancellationToken);
			return new DirectoryEntry(blockDirectoryName, rootNode.Length, rootNodeRef);
		}

		/// <summary>
		/// Retrieve an output of the given node. Fetches and decompresses the files from shared storage if necessary, or validates the local files.
		/// </summary>
		/// <param name="hordeClient"></param>
		/// <param name="jobId"></param>
		/// <param name="stepId"></param>
		/// <param name="nodeName">The node which created the storage block</param>
		/// <param name="blockName">Name of the block to retrieve.</param>
		/// <param name="rootDir">Local directory for extracting data to</param>
		/// <param name="manifestDir">Local directory containing manifests</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Manifest of the files retrieved</returns>
		public static async Task<TempStorageBlockManifest> RetrieveBlockAsync(IHordeClient hordeClient, JobId jobId, JobStepId stepId, string nodeName, string blockName, DirectoryReference rootDir, DirectoryReference manifestDir, ILogger logger, CancellationToken cancellationToken)
		{
			// Get the path to the local manifest
			FileReference localManifestFile = GetBlockManifestLocation(manifestDir, nodeName, blockName);
			bool local = FileReference.Exists(localManifestFile);

			// Read the manifest, either from local storage or shared storage
			TempStorageBlockManifest? manifest;
			if (local)
			{
				logger.LogInformation("Reading block \"{NodeName}\":\"{BlockName}\" from {File}", nodeName, blockName, localManifestFile);
				manifest = TempStorageBlockManifest.Load(localManifestFile);
			}
			else
			{
				string blockDirectoryName = GetBlockDirectoryName(blockName);

				// Read the shared manifest
				ArtifactName artifactName = GetArtifactNameForNode(nodeName);
				ArtifactType artifactType = ArtifactType.StepOutput;

				RpcGetJobArtifactRequest artifactRequest = new RpcGetJobArtifactRequest();
				artifactRequest.JobId = jobId.ToString();
				artifactRequest.StepId = stepId.ToString();
				artifactRequest.Name = artifactName.ToString();
				artifactRequest.Type = artifactType.ToString();

				JobRpc.JobRpcClient jobRpc = await hordeClient.CreateGrpcClientAsync<JobRpc.JobRpcClient>(cancellationToken);

				RpcGetJobArtifactResponse artifact = await jobRpc.GetArtifactAsync(artifactRequest, cancellationToken: cancellationToken);
				NamespaceId namespaceId = new NamespaceId(artifact.NamespaceId);
				RefName refName = new RefName(artifact.RefName);

				logger.LogInformation("Reading block \"{NodeName}\":\"{BlockName}\" from temp storage (artifact: {ArtifactId} '{ArtifactName}' ({ArtifactType}), ns: {NamespaceId}, ref: {RefName}, local: {LocalFile}, blockdir: {BlockDir})", nodeName, blockName, ArtifactId.Parse(artifact.Id), artifactName, artifactType, namespaceId, refName, localManifestFile, blockDirectoryName);

				IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(namespaceId, artifact.Token);
				DirectoryNode node = await storageNamespace.ReadRefTargetAsync<DirectoryNode>(refName, cancellationToken: cancellationToken);

				DirectoryEntry? rootDirEntry;
				if (!node.TryGetDirectoryEntry(blockDirectoryName, out rootDirEntry))
				{
					throw new TempStorageException($"Missing block \"{blockName}\" from node \"{nodeName}\". If running Horde against a 5.4 or earlier version of Unreal Engine, please make sure you have integrated //UE5/Main/Engine/Source/Programs/AutomationTool/BuildGraph/BuildGraph.cs#70.");
				}

				_ = Int32.TryParse(Environment.GetEnvironmentVariable("UE_HORDE_TEMP_STORAGE_MAX_BATCH_SIZE"), out int maxBatchSize);

				for (int attempt = 0; ; attempt++)
				{
					StorageStats initialStats = storageNamespace.GetStats();
					Stopwatch timer = Stopwatch.StartNew();

					// Add all the files and flush the ref
					ExtractOptions options = new ExtractOptions();
					options.Progress = new ExtractStatsLogger(logger);
					options.VerifyOutput = (attempt > 0);
					options.MaxBatchSize = maxBatchSize;

					await rootDirEntry.Handle.ExtractAsync(rootDir.ToDirectoryInfo(), options, logger, cancellationToken);

					StorageStats deltaStats = StorageStats.GetDelta(initialStats, storageNamespace.GetStats());
					logger.LogInformation("{Stats}", $"Elapsed: {(int)timer.Elapsed.TotalSeconds}s, {String.Join(", ", deltaStats.Values.Select(x => $"{x.Item1}: {x.Item2:n0}"))}");

					try
					{
						// Read the manifest in
						manifest = TempStorageBlockManifest.Load(localManifestFile);

						// Update the timestamps to match the manifest.
						foreach (TempStorageFile manifestFile in manifest.Files)
						{
							FileInfo fileInfo = manifestFile.ToFileReference(rootDir).ToFileInfo();
							fileInfo.LastWriteTimeUtc = new DateTime(manifestFile.LastWriteTimeUtcTicks, DateTimeKind.Utc);

							if (fileInfo.Length != manifestFile.Length)
							{
								logger.LogError("File {File} extracted from temp storage has different size to file in manifest (manifest: {ManifestLength}, local: {LocalLength})", manifestFile.RelativePath, manifestFile.Length, fileInfo.Length);
							}
						}

						break;
					}
					catch (FileNotFoundException ex) when (attempt == 0)
					{
						logger.LogWarning(ex, "File not found: {Message}", ex.Message);
						logger.LogInformation("Running extract again with VerifyOutput enabled to check hashes.");
					}
				}
			}

			return manifest;
		}

		/// <summary>
		/// Gets the path to the manifest created for a node's output.
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node to get the file for</param>
		/// <param name="blockName">Name of the output block to get the manifest for</param>
		public static FileReference GetBlockManifestLocation(DirectoryReference baseDir, string nodeName, string? blockName)
		{
			return FileReference.Combine(baseDir, nodeName, String.IsNullOrEmpty(blockName) ? "Manifest.xml" : String.Format("Manifest-{0}.xml", blockName));
		}

		/// <summary>
		/// Gets the path to the file created to store a tag manifest for a node
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node to get the file for</param>
		/// <param name="tagName">Name of the tag to get the manifest for</param>
		public static FileReference GetTagManifestLocation(DirectoryReference baseDir, string nodeName, string tagName)
		{
			Debug.Assert(tagName.StartsWith("#", StringComparison.Ordinal));
			return FileReference.Combine(baseDir, nodeName, String.Format("Tag-{0}.xml", tagName.Substring(1)));
		}

		/// <summary>
		/// Checks whether the given path is allowed as a build product that can be produced by more than one node (timestamps may be modified, etc..). Used to suppress
		/// warnings about build products being overwritten.
		/// </summary>
		/// <param name="localFile">File name to check</param>
		/// <returns>True if the given path may be output by multiple build products</returns>
		public static bool IsDuplicateBuildProduct(FileReference localFile)
		{
			string fileName = localFile.GetFileName();
			if (fileName.Equals("AgentInterface.dll", StringComparison.OrdinalIgnoreCase) || fileName.Equals("AgentInterface.pdb", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.Equals("dxil.dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.Equals("dxcompiler.dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("tbb", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".pdb", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.EndsWith(".dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".dylib", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".so", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.Contains(".so.", StringComparison.OrdinalIgnoreCase))
			{
				// e.g. a Unix shared library with a version number suffix.
				return true;
			}
			if (fileName.Equals("plugInfo.json", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if ((fileName.Equals("info.plist", StringComparison.OrdinalIgnoreCase) || fileName.Equals("coderesources", StringComparison.OrdinalIgnoreCase)) && localFile.FullName.Contains(".app/", StringComparison.OrdinalIgnoreCase))
			{
				// xcode can generate plist files and coderesources differently in different stages of compile/cook/stage/package/etc. only allow ones inside a .app bundle
				return true;
			}
			return false;
		}
	}
}
