// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace JobDriver.Utility
{
	/// <summary>
	/// Flags for processes to terminate
	/// </summary>
	[Flags]
	public enum TerminateCondition
	{
		/// <summary>
		/// Not specified; terminate in all circumstances
		/// </summary>
		None = 0,

		/// <summary>
		/// Before running a conform
		/// </summary>
		BeforeConform = 2,

		/// <summary>
		/// Before executing a batch
		/// </summary>
		BeforeBatch = 4,

		/// <summary>
		/// Terminate at the end of a batch
		/// </summary>
		AfterBatch = 8,

		/// <summary>
		/// After a step completes
		/// </summary>
		AfterStep = 16,
	}

	/// <summary>
	/// Utility methods for terminating processes
	/// </summary>
	public static class TerminateProcessHelper
	{
		/// <summary>
		/// Terminate processes matching certain criteria
		/// </summary>
		public static Task TerminateProcessesAsync(TerminateCondition condition, DirectoryReference workingDir, IReadOnlyList<ProcessToTerminate>? processesToTerminate, ILogger logger, CancellationToken cancellationToken)
		{
			// Terminate child processes from any previous runs
			ProcessUtils.TerminateProcesses(x => ShouldTerminateProcess(x, condition, workingDir, processesToTerminate), logger, cancellationToken);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Callback for determining whether a process should be terminated
		/// </summary>
		static bool ShouldTerminateProcess(FileReference imageFile, TerminateCondition condition, DirectoryReference workingDir, IReadOnlyList<ProcessToTerminate>? processesToTerminate)
		{
			if (imageFile.IsUnderDirectory(workingDir))
			{
				return true;
			}

			if (processesToTerminate != null)
			{
				string fileName = imageFile.GetFileName();
				foreach (ProcessToTerminate processToTerminate in processesToTerminate)
				{
					if (String.Equals(processToTerminate.Name, fileName, StringComparison.OrdinalIgnoreCase))
					{
						TerminateCondition terminateFlags = TerminateCondition.None;
						foreach (TerminateCondition when in processToTerminate.When ?? Enumerable.Empty<TerminateCondition>())
						{
							terminateFlags |= when;
						}
						if (terminateFlags == TerminateCondition.None || (terminateFlags & condition) != 0)
						{
							return true;
						}
					}
				}
			}

			return false;
		}
	}
}
