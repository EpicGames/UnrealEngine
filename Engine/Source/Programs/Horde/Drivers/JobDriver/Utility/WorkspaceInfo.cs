// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Specialized;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.RegularExpressions;
using System.Web;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace JobDriver.Utility
{
	/// <summary>
	/// Stores information about a managed Perforce workspace
	/// </summary>
	public sealed class WorkspaceInfo
	{
		/// <summary>
		/// The perforce settings
		/// </summary>
		public PerforceSettings PerforceSettings { get; }

		/// <summary>
		/// The Perforce server and port. This is checked to not be null in the constructor.
		/// </summary>
		public string ServerAndPort => PerforceSettings.ServerAndPort!;

		/// <summary>
		/// The Perforce client name. This is checked to not be null in the constructor.
		/// </summary>
		public string ClientName => PerforceSettings.ClientName!;

		/// <summary>
		/// The Perforce user name. This is checked to not be null in the constructor.
		/// </summary>
		public string UserName => PerforceSettings.UserName!;

		/// <summary>
		/// The hostname
		/// </summary>
		public string HostName { get; }

		/// <summary>
		/// The current stream
		/// </summary>
		public string StreamName { get; }

		/// <summary>
		/// View for the stream
		/// </summary>
		public PerforceViewMap StreamView { get; }

		/// <summary>
		/// The directory containing metadata for this workspace
		/// </summary>
		public DirectoryReference MetadataDir { get; }

		/// <summary>
		/// The directory containing workspace data
		/// </summary>
		public DirectoryReference WorkspaceDir { get; }

		/// <summary>
		/// The view for files to sync
		/// </summary>
		public IReadOnlyList<string> View { get; }

		/// <summary>
		/// Whether untracked files should be removed from this workspace
		/// </summary>
		public bool RemoveUntrackedFiles { get; }

		/// <summary>
		/// The managed repository instance
		/// </summary>
		public ManagedWorkspace Repository { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="perforceSettings">The perforce connection</param>
		/// <param name="hostName">Name of this host</param>
		/// <param name="streamName">Name of the stream to sync</param>
		/// <param name="streamView">Stream view onto the depot</param>
		/// <param name="metadataDir">Path to the metadata directory</param>
		/// <param name="workspaceDir">Path to the workspace directory</param>
		/// <param name="view">View for files to be synced</param>
		/// <param name="removeUntrackedFiles">Whether to remove untracked files when syncing</param>
		/// <param name="repository">The repository instance</param>
		public WorkspaceInfo(PerforceSettings perforceSettings, string hostName, string streamName, PerforceViewMap streamView, DirectoryReference metadataDir, DirectoryReference workspaceDir, IList<string>? view, bool removeUntrackedFiles, ManagedWorkspace repository)
		{
			PerforceSettings = perforceSettings;

			if (perforceSettings.ClientName == null)
			{
				throw new ArgumentException("PerforceConnection does not have valid client name");
			}

			HostName = hostName;
			StreamName = streamName;
			StreamView = streamView;
			MetadataDir = metadataDir;
			WorkspaceDir = workspaceDir;
			View = (view == null) ? new List<string>() : new List<string>(view);
			RemoveUntrackedFiles = removeUntrackedFiles;
			Repository = repository;
		}

		/// <summary>
		/// Creates a new managed workspace
		/// </summary>
		/// <param name="workspace">The workspace definition</param>
		/// <param name="rootDir">Root directory for storing the workspace</param>
		/// <param name="options">Extra options for ManagedWorkspace</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="logger">Logger output</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>New workspace info</returns>
		public static async Task<WorkspaceInfo> CreateWorkspaceInfoAsync(RpcAgentWorkspace workspace, DirectoryReference rootDir, ManagedWorkspaceOptions options, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			// Fill in the default credentials iff they are not set
			string? serverAndPort = String.IsNullOrEmpty(workspace.ServerAndPort) ? null : workspace.ServerAndPort;
			string? userName = String.IsNullOrEmpty(workspace.UserName) ? null : workspace.UserName;
			string? password = String.IsNullOrEmpty(workspace.Password) ? null : workspace.Password;
			string? ticket = String.IsNullOrEmpty(workspace.Ticket) ? null : workspace.Ticket;

			if (serverAndPort == null)
			{
				serverAndPort = PerforceSettings.Default.ServerAndPort;
				logger.LogInformation("Using locally configured Perforce server: '{ServerAndPort}'", serverAndPort);
			}

			if (userName == null)
			{
				userName = PerforceSettings.Default.UserName;
				logger.LogInformation("Using locally configured and logged in Perforce user: '{UserName}'", userName);
			}

			if (options.PreferNativeClient)
			{
				logger.LogInformation("Using native P4 client for {ServerAndPort}", serverAndPort);
			}
			else
			{
				logger.LogInformation("Using command-line P4 client for {ServerAndPort}", serverAndPort);
			}

			// Create the connection
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(new PerforceSettings(serverAndPort, userName) { PreferNativeClient = options.PreferNativeClient, Password = ticket }, logger);
			if (userName != null)
			{
				if (ticket != null)
				{
					Environment.SetEnvironmentVariable("P4PASSWD", ticket);
				}
				else if (password != null)
				{
					await perforce.LoginAsync(password, cancellationToken);
				}
				else
				{
					logger.LogInformation("Using locally logged in session for {UserName}", userName);
				}
			}

			// Get the host name, and fill in any missing metadata about the connection
			InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);

			string? hostName = info.ClientHost;
			if (hostName == null)
			{
				throw new Exception("Unable to determine Perforce host name");
			}

			if (!options.UseHaveTable)
			{
				logger.LogInformation("Skipping use of have table");
			}

			// replace invalid characters in the workspace identifier with a '+' character

			// append the slot index, if it's non-zero
			//			my $slot_idx = $optional_arguments->{'slot_idx'} || 0;
			//			$workspace_identifier .= sprintf("+%02d", $slot_idx) if $slot_idx;

			// if running on an edge server, append the server id to the client name
			string edgeSuffix = String.Empty;
			if (info.Services != null && info.ServerId != null)
			{
				string[] services = info.Services.Split(' ', StringSplitOptions.RemoveEmptyEntries);
				if (services.Any(x => x.Equals("edge-server", StringComparison.OrdinalIgnoreCase)))
				{
					edgeSuffix = $"+{info.ServerId}";
				}
			}

			// get all the workspace settings
			string clientName = $"Horde+{GetNormalizedHostName(hostName)}+{workspace.Identifier}{edgeSuffix}";
			PerforceSettings perforceClientSettings = new PerforceSettings(perforce.Settings) { ClientName = clientName, PreferNativeClient = options.PreferNativeClient };

			// Get the view for this stream
			StreamRecord stream = await perforce.GetStreamAsync(workspace.Stream, true, cancellationToken);
			PerforceViewMap streamView = PerforceViewMap.Parse(stream.View);

			// get the workspace names
			DirectoryReference metadataDir = DirectoryReference.Combine(rootDir, workspace.Identifier);
			DirectoryReference workspaceDir = DirectoryReference.Combine(metadataDir, "Sync");

			// Create the repository
			ManagedWorkspace newRepository = await ManagedWorkspace.LoadOrCreateAsync(hostName, metadataDir, true, options, tracer, logger, cancellationToken);
			return new WorkspaceInfo(perforceClientSettings, hostName, workspace.Stream, streamView, metadataDir, workspaceDir, workspace.View, !workspace.Incremental, newRepository);
		}
		
		/// <summary>
		/// Creates a new managed workspace
		/// </summary>
		/// <param name="perforceClient">The perforce connection to use</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>New workspace info</returns>
		public async Task SetupWorkspaceAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			// Create the workspace info
			perforceClient.Logger.LogInformation("Syncing {ClientName} to {BaseDir} from {Server}, using stream {Stream} and view: {View}", PerforceSettings.ClientName, WorkspaceDir, PerforceSettings.ServerAndPort, StreamName, String.Join("", View.Select(x => $"\n  {x}")));

			// Create the repository
			if (RemoveUntrackedFiles)
			{
				await Repository.DeleteClientAsync(perforceClient, cancellationToken);
			}
			await Repository.SetupAsync(perforceClient, StreamName, cancellationToken);

			// Revert any open files
			await Repository.RevertAsync(perforceClient, cancellationToken);
		}

		/// <summary>
		/// Parse a text string as a query string to determine use of have table
		/// </summary>
		/// <param name="method">Method text string from workspace config</param>
		/// <returns>True if have table should be enabled</returns>
		public static bool ShouldUseHaveTable(string? method)
		{
			const string NameKey = "name";
			const string ManagedWorkspaceValue = "managedWorkspace";
			const string UseHaveTableKey = "useHaveTable";

			if (String.IsNullOrEmpty(method))
			{
				return true;
			}

			NameValueCollection nameValues = HttpUtility.ParseQueryString(method);
			string? name = nameValues[NameKey];
			string? useHaveTable = nameValues[UseHaveTableKey];

			if (name != null && name.Equals(ManagedWorkspaceValue, StringComparison.OrdinalIgnoreCase))
			{
				if (useHaveTable != null && useHaveTable.Equals("false", StringComparison.OrdinalIgnoreCase))
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Create manage workspace options
		/// </summary>
		/// <param name="workspace">Agent workspace RPC message</param>
		/// <returns>Options for a managed workspace</returns>
		public static ManagedWorkspaceOptions GetMwOptions(RpcAgentWorkspace workspace)
		{
			const string NameKey = "name";
			const string ManagedWorkspaceValue = "managedWorkspace";
			const string NumParallelSyncThreadsKey = "numParallelSyncThreads";
			const string MaxFileConcurrencyKey = "maxFileConcurrency";
			const string MaxFileScanConcurrencyKey = "maxFileScanConcurrency";
			const string MinScratchSpaceKey = "minScratchSpace";
			const string UseHaveTableKey = "useHaveTable";
			const string UseModTimeKey = "useModTime";
			const string PreferNativeClientKey = "preferNativeClient";

			ManagedWorkspaceOptions options = new ManagedWorkspaceOptions();
			options = options with { Partitioned = workspace.Partitioned };

			if (workspace.MinScratchSpace > 0)
			{
				options = options with { MinScratchSpace = workspace.MinScratchSpace };
			}

			string? method = workspace.Method;
			if (!String.IsNullOrEmpty(method))
			{
				NameValueCollection nameValues = HttpUtility.ParseQueryString(method);
				if (String.Equals(nameValues[NameKey], ManagedWorkspaceValue, StringComparison.OrdinalIgnoreCase))
				{
					if (Int32.TryParse(nameValues[NumParallelSyncThreadsKey], out int v))
					{
						options = options with { NumParallelSyncThreads = v };
					}
					if (Int32.TryParse(nameValues[MaxFileConcurrencyKey], out v))
					{
						options = options with { MaxFileConcurrency = v };
					}
					if (Int32.TryParse(MaxFileScanConcurrencyKey, out v))
					{
						options = options with { MaxFileScanConcurrency = v };
					}
					if (Int32.TryParse(nameValues[MinScratchSpaceKey], out v))
					{
						options = options with { MinScratchSpace = v };
					}
					if (String.Equals(nameValues[UseHaveTableKey], "false", StringComparison.OrdinalIgnoreCase))
					{
						options = options with { UseHaveTable = false };
					}
					if (String.Equals(nameValues[PreferNativeClientKey], "true", StringComparison.OrdinalIgnoreCase) && RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
					{
						options = options with { PreferNativeClient = true };
					}
					if (String.Equals(nameValues[UseModTimeKey], "true", StringComparison.OrdinalIgnoreCase))
					{
						options = options with { UseModTime = true };
					}
				}
			}

			return options;
		}

		/// <summary>
		/// Gets the latest change in the stream
		/// </summary>
		/// <param name="perforceClient">The Perforce client instance</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Latest changelist number</returns>
		public Task<int> GetLatestChangeAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			return Repository.GetLatestChangeAsync(perforceClient, StreamName, cancellationToken);
		}

		/// <summary>
		/// Revert any open files in the workspace and clean it
		/// </summary>
		/// <param name="perforceClient">The Perforce client instance</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		public async Task CleanAsync(IPerforceConnection perforceClient, CancellationToken cancellationToken)
		{
			await Repository.RevertAsync(perforceClient, cancellationToken);
			await Repository.CleanAsync(perforceClient, RemoveUntrackedFiles, cancellationToken);
		}

		/// <summary>
		/// Removes the given cache file if it's invalid, and updates the metadata to reflect the version about to be synced
		/// </summary>
		/// <param name="cacheFile">Path to the cache file</param>
		/// <returns>Async task</returns>
		public static void RemoveLocalCacheMarker(FileReference cacheFile)
		{
			FileReference.Delete(cacheFile);
			FileReference.Delete(cacheFile.ChangeExtension(".txt"));
		}

		/// <summary>
		/// Removes the given cache file if it's invalid, and updates the metadata to reflect the version about to be synced
		/// </summary>
		/// <param name="cacheFile">Path to the cache file</param>
		/// <param name="change">The current change being built</param>
		/// <param name="preflightChange">The preflight changelist number</param>
		/// <returns>True if cache file was replaced. That is if changelist or stream view is different from what is already synced</returns>
		public async Task<bool> UpdateLocalCacheMarkerAsync(FileReference cacheFile, int change, int preflightChange)
		{
			// Create the new cache file descriptor
			string newDescriptor = GetCacheMarkerDescriptor(change, preflightChange);

			// Remove the cache file if the current descriptor doesn't match
			FileReference descriptorFile = cacheFile.ChangeExtension(".txt");
			if (FileReference.Exists(cacheFile))
			{
				if (FileReference.Exists(descriptorFile))
				{
					string oldDescriptor = await FileReference.ReadAllTextAsync(descriptorFile);
					if (oldDescriptor.Equals(newDescriptor, StringComparison.Ordinal))
					{
						return false;
					}
					else
					{
						FileReference.Delete(descriptorFile);
					}
				}
				FileReference.Delete(cacheFile);
			}

			// Write the new descriptor file
			await FileReference.WriteAllTextAsync(descriptorFile, newDescriptor);
			return true;
		}

		string GetCacheMarkerDescriptor(int change, int preflightChange)
		{
			StringBuilder descriptor = new StringBuilder();
			if (preflightChange <= 0)
			{
				descriptor.AppendLine($"CL {change}");
			}
			else
			{
				descriptor.AppendLine($"CL {preflightChange} with base CL {change}");
			}

			descriptor.AppendLine();
			foreach (PerforceViewMapEntry streamViewEntry in StreamView.Entries)
			{
				descriptor.AppendLine($"StreamView: {streamViewEntry}");
			}

			descriptor.AppendLine();
			foreach (string viewLine in View)
			{
				descriptor.AppendLine($"View: {viewLine}");
			}

			return descriptor.ToString();
		}

		/// <summary>
		/// Sync the workspace to a given changelist, and capture it so we can quickly clean in the future
		/// </summary>
		/// <param name="perforceClient">Perforce client connection</param>
		/// <param name="change">The changelist to sync to</param>
		/// <param name="preflightChange">Change to preflight, or 0</param>
		/// <param name="cacheFile">Path to the cache file to use</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		public async Task SyncAsync(IPerforceConnection perforceClient, int change, int preflightChange, FileReference? cacheFile, CancellationToken cancellationToken)
		{
			await Repository.SyncAsync(perforceClient, StreamName, change, View, RemoveUntrackedFiles, false, cacheFile, cancellationToken);

			// Purge the cache for incremental workspaces
			if (!RemoveUntrackedFiles)
			{
				await Repository.PurgeAsync(0, cancellationToken);
			}

			await UnshelveAsync(perforceClient, preflightChange, cancellationToken);
		}

		/// <summary>
		/// Unshelves a changelist
		/// Assumes base changelist already has been synced.
		/// </summary>
		/// <param name="perforceClient">Perforce client connection</param>
		/// <param name="change">Change number to unshelve</param>
		/// <param name="cancellationToken">Cancellation token</param>
		async Task UnshelveAsync(IPerforceConnection perforceClient, int change, CancellationToken cancellationToken)
		{
			if (change > 0)
			{
				await Repository.UnshelveAsync(perforceClient, change, cancellationToken);
			}
		}

		/// <summary>
		/// Strip the domain suffix to get the host name
		/// </summary>
		/// <param name="hostName">Hostname with optional domain</param>
		/// <returns>Normalized host name</returns>
		public static string GetNormalizedHostName(string hostName)
		{
			return Regex.Replace(hostName, @"\..*$", "").ToUpperInvariant();
		}

		/// <summary>
		/// Revert all files in a workspace
		/// </summary>
		public static async Task RevertAllChangesAsync(IPerforceConnection perforce, ILogger logger, CancellationToken cancellationToken)
		{
			// Make sure the client name is set
			if (perforce.Settings.ClientName == null)
			{
				throw new ArgumentException("RevertAllChangesAsync() requires PerforceConnection with client");
			}

			// check if there are any open files, and revert them
			List<OpenedRecord> files = await perforce.OpenedAsync(OpenedOptions.None, -1, null, null, 1, new[] { "//..." }, cancellationToken).ToListAsync(cancellationToken);
			if (files.Count > 0)
			{
				await perforce.RevertAsync(-1, null, RevertOptions.KeepWorkspaceFiles, new[] { "//..." }, cancellationToken);
			}

			// enumerate all the pending changelists
			List<ChangesRecord> pendingChanges = await perforce.GetChangesAsync(ChangesOptions.None, perforce.Settings.ClientName, -1, ChangeStatus.Pending, null, FileSpecList.Empty, cancellationToken);
			foreach (ChangesRecord pendingChange in pendingChanges)
			{
				// delete any shelved files if there are any
				List<DescribeRecord> records = await perforce.DescribeAsync(DescribeOptions.Shelved, -1, new int[] { pendingChange.Number }, cancellationToken);
				if (records.Count > 0 && records[0].Files.Count > 0)
				{
					logger.LogInformation("Deleting shelved files in changelist {Change}", pendingChange.Number);
					await perforce.DeleteShelvedFilesAsync(pendingChange.Number, FileSpecList.Any, cancellationToken);
				}

				// delete the changelist
				logger.LogInformation("Deleting changelist {Change}", pendingChange.Number);
				await perforce.DeleteChangeAsync(DeleteChangeOptions.None, pendingChange.Number, cancellationToken);
			}
		}
	}
}
