// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;
using HordeAgent.Commands.Service;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeAgent.Tests.Commands;

[TestClass]
public sealed class ServiceUpgradeTests : IDisposable
{
	private readonly string _testDir = Path.Combine(Path.GetTempPath(), $"upgrade-test-{Guid.NewGuid()}");

	public ServiceUpgradeTests() => Directory.CreateDirectory(_testDir);

	public void Dispose() => Try(() => Directory.Delete(_testDir, true));

	[TestMethod]
	[Timeout(30000)]
	public async Task DeleteFileWithRetry_LockedFile_RetriesAndSucceedsAsync()
	{
		string testFile = Path.Combine(_testDir, "locked.dll");
		await File.WriteAllTextAsync(testFile, "test content");

		await WithLockedFileAsync(testFile, 1500, () =>
			UpgradeCommand.DeleteFileWithRetry(NullLogger.Instance, testFile, [500, 1000, 2000]));

		Assert.IsFalse(File.Exists(testFile));
	}

	[TestMethod]
	public void DeleteFileWithRetry_FileAlreadyDeleted_Succeeds()
	{
		string testFile = Path.Combine(_testDir, "nonexistent.dll");
		UpgradeCommand.DeleteFileWithRetry(NullLogger.Instance, testFile, [100]);
	}

	[TestMethod]
	public void MoveFileWithRetry_AlreadyMoved_Succeeds()
	{
		string targetFile = Path.Combine(_testDir, "source.dll");
		File.WriteAllText(targetFile, "already moved content");

		UpgradeCommand.MoveFileWithRetry(NullLogger.Instance, targetFile + ".new", targetFile, [100]);

		Assert.AreEqual("already moved content", File.ReadAllText(targetFile));
	}

	[TestMethod]
	[Timeout(30000)]
	public async Task MoveFileWithRetry_LockedTarget_RetriesAndSucceedsAsync()
	{
		string sourceFile = Path.Combine(_testDir, "new.dll.new");
		string targetFile = Path.Combine(_testDir, "new.dll");
		await File.WriteAllTextAsync(sourceFile, "new content");
		await File.WriteAllTextAsync(targetFile, "old content");

		await WithLockedFileAsync(targetFile, 1500, () =>
			UpgradeCommand.MoveFileWithRetry(NullLogger.Instance, sourceFile, targetFile, [500, 1000, 2000]));

		Assert.IsFalse(File.Exists(sourceFile));
		Assert.AreEqual("new content", await File.ReadAllTextAsync(targetFile));
	}

	[TestMethod]
	public void MoveFileWithRetry_NormalMove_Succeeds()
	{
		string sourceFile = Path.Combine(_testDir, "normal.dll.new");
		string targetFile = Path.Combine(_testDir, "normal.dll");
		File.WriteAllText(sourceFile, "content to move");

		UpgradeCommand.MoveFileWithRetry(NullLogger.Instance, sourceFile, targetFile, [100]);

		Assert.IsFalse(File.Exists(sourceFile));
		Assert.AreEqual("content to move", File.ReadAllText(targetFile));
	}

	private static async Task WithLockedFileAsync(string path, int lockDurationMs, Action action)
	{
		using ManualResetEventSlim lockAcquired = new();
		Task lockTask = Task.Run(async () =>
		{
			await using FileStream fs = File.Open(path, FileMode.Open, FileAccess.Read, FileShare.None);
			lockAcquired.Set();
			await Task.Delay(lockDurationMs);
		});

		lockAcquired.Wait();
		action();
		await TryAsync(() => lockTask);
	}

	private static void Try(Action action)
	{
		try
		{
			action();
		}
		catch
		{
		}
	}

	private static async Task TryAsync(Func<Task> action)
	{
		try
		{
			await action();
		}
		catch
		{
		}
	}
}
