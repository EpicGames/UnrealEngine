// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Tools;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Tests;

internal class FakeHordeRpcClient(FakeHordeRpcServer outer) : HordeRpc.HordeRpcClient
{
	public override AsyncDuplexStreamingCall<RpcUpdateSessionRequest, RpcUpdateSessionResponse> UpdateSession(Metadata headers = null!,
		DateTime? deadline = null, CancellationToken cancellationToken = default)
	{
		return outer.GetUpdateSessionCall(CancellationToken.None);
	}
}

internal class FakeHordeClient(FakeHordeRpcServer server) : IHordeClient
{
	public Uri ServerUrl => new ("http://fake-horde-server");
	public IArtifactCollection Artifacts => throw new NotImplementedException();
	public IComputeClient Compute => throw new NotImplementedException();
	public IProjectCollection Projects => throw new NotImplementedException();
	public ISecretCollection Secrets => throw new NotImplementedException();
	public IToolCollection Tools => throw new NotImplementedException();
	public event Action? OnAccessTokenStateChanged
	{
		add { }
		remove { }
	}
	public Task<bool> LoginAsync(bool allowLogin, CancellationToken cancellationToken) => throw new NotImplementedException();
	public bool HasValidAccessToken() => true;
	public Task<string?> GetAccessTokenAsync(bool interactive, CancellationToken cancellationToken = default) => Task.FromResult<string?>(null);
	public Task<TClient> CreateGrpcClientAsync<TClient>(CancellationToken cancellationToken = default) where TClient : ClientBase<TClient>
	{
		if (typeof(TClient) == typeof(HordeRpc.HordeRpcClient))
		{
			return Task.FromResult((TClient)(object)new FakeHordeRpcClient(server));
		}
		throw new NotImplementedException($"No support for gRPC client {typeof(TClient)}");
	}
	
	public HordeHttpClient CreateHttpClient() => throw new NotImplementedException();
	public IStorageNamespace GetStorageNamespace(string relativePath, string? accessToken = null) => throw new NotImplementedException();
	public IServerLogger CreateServerLogger(LogId logId, LogLevel minimumLevel = LogLevel.Information) => throw new NotImplementedException();
}

/// <summary>
/// Fake implementation of a HordeRpc gRPC server.
/// Provides a corresponding gRPC client class that can be used with to test client-server interactions inside a single process.
/// Can only handle a single agent per instance.
/// </summary>
internal class FakeHordeRpcServer
{
	public readonly TaskCompletionSource<bool> CreateSessionReceived = new();
	public readonly Channel<RpcUpdateSessionRequest> UpdateSessionRequests = Channel.CreateUnbounded<RpcUpdateSessionRequest>();
	
	/// <summary>
	/// Last status reported by the agent during a session update
	/// </summary>
	public RpcAgentStatus? LastReportedStatus { get; private set; } = null;
	
	/// <summary>
	/// Agent status decided by server. Returned back to agent in session update. 
	/// </summary>
	public RpcAgentStatus AgentStatus { get; private set; } = RpcAgentStatus.Ok;
	
	private readonly Dictionary<LeaseId, RpcLease> _leases = new();
	private readonly FakeHordeClient _hordeClient;
	private readonly ILogger<FakeHordeRpcServer> _logger;
	
	public FakeHordeRpcServer(ILogger<FakeHordeRpcServer> logger)
	{
		_logger = logger;
		_hordeClient = new FakeHordeClient(this);
	}
	
	public void ScheduleTestLease(LeaseId? leaseId = null)
	{
		if (leaseId == null)
		{
			Span<byte> randomBytes = stackalloc byte[12];
			Random.Shared.NextBytes(randomBytes);
			leaseId = new LeaseId(new BinaryId(randomBytes));
		}
		
		if (_leases.ContainsKey(leaseId.Value))
		{
			throw new ArgumentException($"Lease ID {leaseId.Value} already exists");
		}
		
		TestTask testTask = new();
		_leases[leaseId.Value] = new RpcLease { Id = leaseId.Value, State = RpcLeaseState.Pending, Payload = Any.Pack(testTask) };
	}
	
	public void SetAgentStatus(RpcAgentStatus newStatus)
	{
		AgentStatus = newStatus;
	}
	
	public RpcLease GetLease(LeaseId leaseId)
	{
		return _leases[leaseId];
	}
	
	public IHordeClient GetHordeClient()
	{
		return _hordeClient;
	}
	
	public RpcCreateSessionResponse OnCreateSessionRequest(RpcCreateSessionRequest request)
	{
		CreateSessionReceived.TrySetResult(true);
		_logger.LogInformation("OnCreateSessionRequest: {AgentId}", request.Id);
		RpcCreateSessionResponse response = new()
		{
			AgentId = "bogusAgentId",
			Token = "bogusToken",
			SessionId = "bogusSessionId",
			ExpiryTime = Timestamp.FromDateTime(DateTime.UtcNow.AddHours(3)),
		};
		
		return response;
	}
	
	public AsyncDuplexStreamingCall<RpcUpdateSessionRequest, RpcUpdateSessionResponse> GetUpdateSessionCall(CancellationToken cancellationToken)
	{
		FakeAsyncStreamReader<RpcUpdateSessionResponse> responseStream = new(cancellationToken);
		
		async Task OnRequest(RpcUpdateSessionRequest request)
		{
			await UpdateSessionRequests.Writer.WriteAsync(request, cancellationToken);
			LastReportedStatus = request.Status;
			
			switch (request.Status)
			{
				case RpcAgentStatus.Ok: break;
				case RpcAgentStatus.Stopping: SetAgentStatus(RpcAgentStatus.Stopped); break;
				case RpcAgentStatus.Stopped: break;
				default: throw new Exception($"Unhandled agent status {request.Status}");
			}
			
			foreach (RpcLease agentLease in request.Leases)
			{
				RpcLease serverLease = _leases[agentLease.Id];
				serverLease.State = agentLease.State;
				serverLease.Outcome = agentLease.Outcome;
				serverLease.Output = agentLease.Output;
			}
			
			_logger.LogInformation("OnUpdateSessionRequest: {AgentId} {SessionId} {Status}", request.AgentId, request.SessionId, request.Status);
			await Task.Delay(100, cancellationToken);
			
			RpcUpdateSessionResponse response = new()
			{
				Status = AgentStatus,
				ExpiryTime = Timestamp.FromDateTime(DateTime.UtcNow + TimeSpan.FromMinutes(120))
			};
			response.Leases.AddRange(_leases.Values.Where(x => x.State != RpcLeaseState.Completed));
			await responseStream.Write(response);
		}
		
		FakeClientStreamWriter<RpcUpdateSessionRequest> requestStream = new(OnRequest, () =>
		{
			responseStream.Complete();
			return Task.CompletedTask;
		});
		
		return new(
			requestStream,
			responseStream,
			Task.FromResult(new Metadata()),
			() => Status.DefaultSuccess,
			() => new Metadata(),
			() => { });
	}
}

/// <summary>
/// Fake stream reader used for testing gRPC clients
/// </summary>
/// <typeparam name="T">Message type reader will handle</typeparam>
internal class FakeAsyncStreamReader<T> : IAsyncStreamReader<T> where T : class
{
	private readonly Channel<T> _channel = System.Threading.Channels.Channel.CreateUnbounded<T>();
	private T? _current;
	private readonly CancellationToken? _cancellationTokenOverride;
	
	public FakeAsyncStreamReader(CancellationToken? cancellationTokenOverride = null)
	{
		_cancellationTokenOverride = cancellationTokenOverride;
	}
	
	public Task Write(T message)
	{
		if (!_channel.Writer.TryWrite(message))
		{
			throw new InvalidOperationException("Unable to write message.");
		}
		
		return Task.CompletedTask;
	}
	
	public void Complete()
	{
		_channel.Writer.Complete();
	}
	
	/// <inheritdoc/>
	public async Task<bool> MoveNext(CancellationToken cancellationToken)
	{
		if (_cancellationTokenOverride != null)
		{
			cancellationToken = _cancellationTokenOverride.Value;
		}
		
		if (await _channel.Reader.WaitToReadAsync(cancellationToken))
		{
			if (_channel.Reader.TryRead(out T? message))
			{
				_current = message;
				return true;
			}
		}
		
		_current = null!;
		return false;
	}
	
	/// <inheritdoc/>
	public T Current
	{
		get
		{
			if (_current == null)
			{
				throw new InvalidOperationException("No current element is available.");
			}
			
			return _current;
		}
	}
}

/// <summary>
/// Fake stream writer used for testing gRPC clients
/// </summary>
/// <typeparam name="T">Message type writer will handle</typeparam>
internal class FakeClientStreamWriter<T> : IClientStreamWriter<T> where T : class
{
	private readonly Func<T, Task>? _onWrite;
	private readonly Func<Task>? _onComplete;
	private bool _isCompleted;
	
	public FakeClientStreamWriter(Func<T, Task>? onWrite = null, Func<Task>? onComplete = null)
	{
		_onWrite = onWrite;
		_onComplete = onComplete;
	}
	
	/// <inheritdoc/>
	public async Task WriteAsync(T message)
	{
		if (_isCompleted)
		{
			throw new InvalidOperationException("Stream is marked as complete");
		}
		
		if (_onWrite != null)
		{
			await _onWrite(message);
		}
	}
	
	/// <inheritdoc/>
	public WriteOptions? WriteOptions { get; set; }
	
	/// <inheritdoc/>
	public async Task CompleteAsync()
	{
		_isCompleted = true;
		if (_onComplete != null)
		{
			await _onComplete();
		}
	}
}