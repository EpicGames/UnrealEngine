<Project Sdk="Microsoft.NET.Sdk">

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Logging.Abstractions" Version="10.0.6" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\HordeAgent\HordeAgent.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="remote-exec-test-bin\*.exe">
      <CopyToOutputDirectory>PreserveNewest</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

</Project>
