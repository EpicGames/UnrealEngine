// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Metrics;
using EpicGames.Horde.Agents.Sessions;
using HordeAgent.Leases;
using HordeAgent.Services;
using HordeAgent.Tests.Services;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using OpenTelemetry.Trace;

namespace HordeAgent.Tests.Leases;

public class FakeCapabilitiesService : ICapabilitiesService
{
	public RpcAgentCapabilities Capabilities { get; set; } = new ();
	public Task<RpcAgentCapabilities> GetCapabilitiesAsync(DirectoryReference? workingDir)
	{
		return Task.FromResult(Capabilities);
	}
}

public class FakeSystemMetrics : ISystemMetrics
{
	public long DiskFreeSpace { get; set; } = 1000;
	public CpuMetrics? GetCpu() => null;
	public CpuTimeMetrics? GetCpuTime() => null;
	public MemoryMetrics? GetMemory() => null;
	public DiskMetrics? GetWorkingDisk() => new() { FreeSpace = DiskFreeSpace, TotalSize = 10000 };
	public DiskMetrics? GetSystemDisk() => null;
}

public class SessionStub(AgentId agentId, SessionId sessionId, DirectoryReference workingDir, IHordeClient hordeClient) : ISession
{
	public AgentId AgentId { get; } = agentId;
	public SessionId SessionId { get; } = sessionId;
	public DirectoryReference WorkingDir { get; } = workingDir;
	public IHordeClient HordeClient { get; } = hordeClient;
	
	public async ValueTask DisposeAsync()
	{
		if (HordeClient is IAsyncDisposable disposableClient)
		{
			await disposableClient.DisposeAsync();
		}
		GC.SuppressFinalize(this);
	}
}

internal class TestHandlerFactory : LeaseHandlerFactory<TestTask>
{
	private readonly LeaseResult _result = new((byte[]?)null);
	
	public TestHandlerFactory(LeaseResult? leaseResult = null)
	{
		_result = leaseResult ?? _result;
	}
	
	public override LeaseHandler<TestTask> CreateHandler(RpcLease lease)
	{
		return new Handler(lease, _result);
	}
	
	private class Handler(RpcLease rpcLease, LeaseResult leaseResult) : LeaseHandler<TestTask>(rpcLease)
	{
		protected override Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, TestTask message, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("Executed TestTask");
			return Task.FromResult(leaseResult);
		}
	}
}

[TestClass]
public sealed class LeaseManagerTests
{
	private readonly FakeHordeRpcServer _hordeServer;
	private readonly LeaseManager _leaseManager;
	
	public LeaseManagerTests()
	{
		_hordeServer = new FakeHordeRpcServer(CreateConsoleLogger<FakeHordeRpcServer>());
		_leaseManager = CreateLeaseManager(_hordeServer.GetHordeClient());
	}
	
	[TestMethod]
	public async Task Run_SingleLease_FinishesSuccessfullyAsync()
	{
		using CancellationTokenSource cts = new(5000);
		_hordeServer.ScheduleTestLease();
		Task<SessionResult> runTask = _leaseManager.RunAsync(cts.Token);
		_leaseManager.OnLeaseFinished += (lease, result) =>
		{
			Assert.AreEqual(RpcAgentStatus.Ok, _hordeServer.LastReportedStatus);
			_hordeServer.SetAgentStatus(RpcAgentStatus.Stopped);
		};
		
		Assert.AreEqual(new SessionResult(SessionOutcome.BackOff, SessionReason.Completed), await runTask);
	}
	
	[TestMethod]
	public async Task UpdateSession_SendsCapabilitiesInFinalUpdate_WhenSessionTerminates_Async()
	{
		using CancellationTokenSource cts = new(5000);
		LeaseResult leaseResult = new (new SessionResult((_, _) => Task.CompletedTask));
		RpcAgentCapabilities fooCaps = new();
		fooCaps.Properties.Add("foo");
		FakeCapabilitiesService capsService = new () { Capabilities = fooCaps };
		
		LeaseManager leaseManager = CreateLeaseManager(_hordeServer.GetHordeClient(), capsService, leaseResult);
		_hordeServer.ScheduleTestLease();
		SessionResult sessionResult = await leaseManager.RunAsync(cts.Token);
		_hordeServer.UpdateSessionRequests.Writer.Complete();
		
		List<RpcUpdateSessionRequest> requests = await _hordeServer.UpdateSessionRequests.Reader.ReadAllAsync(cts.Token).ToListAsync(cts.Token);
		Assert.AreEqual(3, requests.Count);
		Assert.IsTrue(requests[2].Capabilities.Properties.Contains("foo"));
		Assert.AreEqual(new SessionResult(SessionOutcome.RunCallback, SessionReason.Completed), sessionResult);
	}
	
	[TestMethod]
	public async Task TerminateGracefully_LeaseIsActive_Async()
	{
		using CancellationTokenSource cts = new(5000);
		_leaseManager.TerminateSessionAfterLease = true;
		_hordeServer.ScheduleTestLease();
		Assert.AreEqual(new SessionResult(SessionOutcome.Terminate, SessionReason.Completed), await _leaseManager.RunAsync(cts.Token));
		Assert.AreEqual(RpcAgentStatus.Stopped, _hordeServer.AgentStatus);
	}
	
	[TestMethod]
	public async Task TerminateGracefully_NoLeaseActive_Async()
	{
		using CancellationTokenSource cts = new(5000);
		_leaseManager.TerminateSessionAfterLease = true;
		Assert.AreEqual(new SessionResult(SessionOutcome.Terminate, SessionReason.Completed), await _leaseManager.RunAsync(cts.Token));
		Assert.AreEqual(RpcAgentStatus.Stopped, _hordeServer.AgentStatus);
	}

	[TestMethod]
	public async Task UpdateSession_SendsDiskFreeSpaceInCapabilitiesAsync()
	{
		using CancellationTokenSource cts = new(5000);
		FakeSystemMetrics fakeMetrics = new() { DiskFreeSpace = 2000 };
		FakeHordeRpcServer server = new(CreateConsoleLogger<FakeHordeRpcServer>());
		LeaseManager leaseManager = CreateLeaseManager(server.GetHordeClient(), systemMetrics: fakeMetrics);
		leaseManager.TerminateSessionAfterLease = true;

		await leaseManager.RunAsync(cts.Token);
		server.UpdateSessionRequests.Writer.Complete();

		List<RpcUpdateSessionRequest> requests = await server.UpdateSessionRequests.Reader.ReadAllAsync(cts.Token).ToListAsync(cts.Token);
		RpcUpdateSessionRequest request = requests.Last();
		Assert.Contains($"{KnownPropertyNames.WorkingDiskFreeSpace}=2000", request.CapabilityOverrides.Properties, StringComparer.Ordinal);
	}

	[TestMethod]
	public async Task UpdateSession_SendsDiskFreeSpaceInCapabilityOverridesAsync()
	{
		using CancellationTokenSource cts = new(5000);
		FakeSystemMetrics fakeMetrics = new() { DiskFreeSpace = 2000 };
		FakeCapabilitiesService capsService = new()
		{
			Capabilities = new RpcAgentCapabilities { Properties = { $"{KnownPropertyNames.WorkingDiskFreeSpace}=1000" } }
		};

		FakeHordeRpcServer server = new(CreateConsoleLogger<FakeHordeRpcServer>());
		LeaseManager leaseManager = CreateLeaseManager(server.GetHordeClient(), capsService, systemMetrics: fakeMetrics);
		leaseManager.TerminateSessionAfterLease = true;

		await leaseManager.RunAsync(cts.Token);
		server.UpdateSessionRequests.Writer.Complete();

		List<RpcUpdateSessionRequest> requests = await server.UpdateSessionRequests.Reader.ReadAllAsync(cts.Token).ToListAsync(cts.Token);
		RpcUpdateSessionRequest request = requests.Last();
		Assert.Contains($"{KnownPropertyNames.WorkingDiskFreeSpace}=1000", request.Capabilities.Properties, StringComparer.Ordinal);
		Assert.Contains($"{KnownPropertyNames.WorkingDiskFreeSpace}=2000", request.CapabilityOverrides.Properties, StringComparer.Ordinal);
	}

	[TestMethod]
	public async Task UpdateSession_NoDiskMetrics_NoCapabilityOverridesAsync()
	{
		using CancellationTokenSource cts = new(5000);
		FakeHordeRpcServer server = new(CreateConsoleLogger<FakeHordeRpcServer>());
		LeaseManager leaseManager = CreateLeaseManager(server.GetHordeClient());
		leaseManager.TerminateSessionAfterLease = true;

		await leaseManager.RunAsync(cts.Token);
		server.UpdateSessionRequests.Writer.Complete();

		List<RpcUpdateSessionRequest> requests = await server.UpdateSessionRequests.Reader.ReadAllAsync(cts.Token).ToListAsync(cts.Token);
		RpcUpdateSessionRequest request = requests.Last();
		Assert.IsNull(request.CapabilityOverrides);
	}

	/// <summary>
	/// Create a console logger for tests
	/// </summary>
	/// <typeparam name="T">Type to instantiate</typeparam>
	/// <returns>A logger</returns>
	private static ILogger<T> CreateConsoleLogger<T>()
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});
		
		return loggerFactory.CreateLogger<T>();
	}
	
	[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope")]
	private static LeaseManager CreateLeaseManager(IHordeClient hordeClient, FakeCapabilitiesService? capsService = null, LeaseResult? leaseResult = null, ISystemMetrics? systemMetrics = null)
	{
		DirectoryReference tempWorkingDir = new(Path.Join(Path.GetTempPath(), Path.GetRandomFileName()));
		AgentSettings settings = new() { WriteStepOutputToLogger = true };
		ISession session = new SessionStub(new AgentId("testAgent"), SessionId.Parse("aaaaaaaaaaaaaaaaaaaaaaaa"), tempWorkingDir, hordeClient);
		TestOptionsMonitor<AgentSettings> settingsOptions = new (settings);
		StatusService statusService = new(settingsOptions, NullLogger<StatusService>.Instance);

		return new LeaseManager(
			session,
			capsService ?? new FakeCapabilitiesService(),
			statusService,
			systemMetrics ?? new DefaultSystemMetrics(),
			new List<LeaseHandlerFactory>() { new TestHandlerFactory(leaseResult) },
			new LeaseLoggerFactory(settingsOptions, CreateConsoleLogger<LeaseLoggerFactory>()),
			settingsOptions,
			TracerProvider.Default.GetTracer("TestTracer"),
			CreateConsoleLogger<LeaseManager>());
	}
}
