// Copyright Epic Games, Inc. All Rights Reserved.

#region Copyright notice and license

// Copyright 2019 The gRPC Authors
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

#endregion

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using Grpc.Core;

namespace HordeAgent.Tests.Relay
{
	public class TestServerStreamWriter<T> : IServerStreamWriter<T> where T : class
	{
		private readonly ServerCallContext _serverCallContext;
		private readonly Channel<T> _channel;

		public WriteOptions? WriteOptions { get; set; }

		public TestServerStreamWriter(ServerCallContext serverCallContext)
		{
			_channel = Channel.CreateUnbounded<T>();

			_serverCallContext = serverCallContext;
		}

		public void Complete()
		{
			_channel.Writer.Complete();
		}

		public IAsyncEnumerable<T> ReadAllAsync()
		{
			return _channel.Reader.ReadAllAsync();
		}

		public async Task<T?> ReadNextAsync()
		{
			if (await _channel.Reader.WaitToReadAsync())
			{
				_channel.Reader.TryRead(out T? message);
				return message;
			}
			else
			{
				return null;
			}
		}

		public Task WriteAsync(T message)
		{
			if (_serverCallContext.CancellationToken.IsCancellationRequested)
			{
				return Task.FromCanceled(_serverCallContext.CancellationToken);
			}

			if (!_channel.Writer.TryWrite(message))
			{
				throw new InvalidOperationException("Unable to write message.");
			}

			return Task.CompletedTask;
		}
	}

	public class TestAsyncStreamReader<T> : IAsyncStreamReader<T> where T : class
	{
		private readonly Channel<T> _channel;
		private readonly ServerCallContext? _serverCallContext;

		public T Current { get; private set; } = null!;

		public TestAsyncStreamReader(ServerCallContext? serverCallContext = null)
		{
			_channel = Channel.CreateUnbounded<T>();
			_serverCallContext = serverCallContext;
		}

		public void AddMessage(T message)
		{
			if (!_channel.Writer.TryWrite(message))
			{
				throw new InvalidOperationException("Unable to write message.");
			}
		}

		public void Complete()
		{
			_channel.Writer.Complete();
		}

		public async Task<bool> MoveNext(CancellationToken cancellationToken)
		{
			_serverCallContext?.CancellationToken.ThrowIfCancellationRequested();

			if (await _channel.Reader.WaitToReadAsync(cancellationToken))
			{
				if (_channel.Reader.TryRead(out T? message))
				{
					Current = message;
					return true;
				}
			}

			Current = null!;
			return false;
		}
	}
}