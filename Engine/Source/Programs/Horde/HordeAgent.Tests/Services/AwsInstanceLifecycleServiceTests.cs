// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeAgent.Services;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeAgent.Tests.Services;

/// <summary>
/// Testing stub for IWorkerService
/// </summary>
public class WorkerServiceStub : IWorkerService
{
	public bool TerminateSession { get; set; }
	
	public List<RpcLease> GetActiveLeases()
	{
		throw new NotImplementedException();
	}
	
	public void TerminateSessionAfterLease()
	{
		TerminateSession = true;
	}
	
	public bool IsConnected { get; set; }
	public IReadOnlyList<string> PoolIds { get; set; } = [];
}

/// <summary>
/// Fake implementation of the AWS EC2 metadata server (IMDSv2)
/// </summary>
public class FakeAwsImds : HttpMessageHandler
{
	public bool IsSpotInstance { get; set; }
	public bool IsSpotTerminating { get; set; }
	public bool IsAsgTerminating { get; set; }
	private const string FakeTokenPrefix = "fakeToken-";
	private readonly IClock _clock;
	private readonly Dictionary<string, Func<HttpRequestMessage, HttpResponseMessage>> _responses = new();

	public FakeAwsImds(IClock clock)
	{
		_clock = clock;
		_responses["/"] = req =>
		{
			if (!IsValidImdsV2Request(req))
			{
				return new HttpResponseMessage(HttpStatusCode.Unauthorized);
			}
			return Res(HttpStatusCode.OK, "Metadata server");
		};
		_responses[AwsInstanceLifecycleService.ImdsTokenPath] = req =>
		{
			if (req.Method != HttpMethod.Put)
			{
				return Res(HttpStatusCode.MethodNotAllowed);
			}

			if (!req.Headers.TryGetValues(AwsInstanceLifecycleService.ImdsTokenTtlHeader, out IEnumerable<string>? values))
			{
				return Res(HttpStatusCode.BadRequest);
			}
			
			string? tokenTtlSecs = values.FirstOrDefault();
			TimeSpan tokenTtl = TimeSpan.FromSeconds(Convert.ToInt32(tokenTtlSecs));
			long expireTime = new DateTimeOffset(_clock.UtcNow).Add(tokenTtl).ToUnixTimeSeconds();
			return Res(HttpStatusCode.OK, $"{FakeTokenPrefix}{expireTime}");
		};
		
		_responses[AwsInstanceLifecycleService.ImdsInstanceLifeCyclePath] = req => !IsValidImdsV2Request(req)
			? Res(HttpStatusCode.Unauthorized)
			: Res(HttpStatusCode.OK, IsSpotInstance ? "spot" : "on-demand");
		
		_responses[AwsInstanceLifecycleService.ImdsSpotActionPath] = req =>
		{
			if (!IsValidImdsV2Request(req))
			{
				return new HttpResponseMessage(HttpStatusCode.Unauthorized);
			}
			
			HttpStatusCode statusCode = IsSpotTerminating ? HttpStatusCode.OK : HttpStatusCode.NotFound;
			return Res(statusCode, IsSpotTerminating ? "{\"action\": \"terminate\", \"time\": \"2024-11-01T12:00:00Z\"}" : "");
		};
		
		_responses[AwsInstanceLifecycleService.ImdsAsgTargetStatePath] = req => !IsValidImdsV2Request(req)
			? new HttpResponseMessage(HttpStatusCode.Unauthorized)
			: Res(HttpStatusCode.OK, IsAsgTerminating ? "Terminated" : "InService");
	}
	
	public HttpClient CreateHttpClient() => new (this) { BaseAddress = new Uri("http://169.254.169.254") };
	
	private static HttpResponseMessage Res(HttpStatusCode statusCode, string body = "") => new (statusCode) { Content = new StringContent(body) };
	
	private bool IsValidImdsV2Request(HttpRequestMessage request)
	{
		if (!request.Headers.TryGetValues(AwsInstanceLifecycleService.ImdsTokenHeader, out IEnumerable<string>? tokens))
		{
			return false;
		}
		string[] parts = tokens.First().Split(FakeTokenPrefix);
		return parts.Length == 2 && Convert.ToInt64(parts[1]) > new DateTimeOffset(_clock.UtcNow).ToUnixTimeSeconds();
	}
	
	/// <inheritdoc />
	protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
	{
		string path = request.RequestUri!.PathAndQuery;
		if (_responses.TryGetValue(path, out Func<HttpRequestMessage, HttpResponseMessage>? handler))
		{
			return Task.FromResult(handler(request));
		}
		
		return Task.FromResult(Res(HttpStatusCode.NotFound, $"FakeAwsImds: path {path} not found"));
	}
}

// Stub for fulfilling IOptionsMonitor interface during testing
// Copied from HordeServerTests until a good way to share code between these is decided.
internal class TestOptionsMonitor<T>(T value) : IOptionsMonitor<T>
	where T : class, new()
{
	public T CurrentValue { get; } = value;
	public T Get(string? name) => CurrentValue;
	public IDisposable? OnChange(Action<T, string?> listener) => null;
}

[TestClass]
public sealed class AwsInstanceLifecycleServiceTests : IDisposable
{
	private readonly WorkerServiceStub _workerService = new();
	private readonly HttpClient _httpClient;
	private readonly StubClock _clock = new();
	private readonly FakeAwsImds _fakeImds;
	private readonly AwsInstanceLifecycleService _service;
	private readonly FileReference _terminationSignalFile;
	private Ec2TerminationInfo? _info;

	public AwsInstanceLifecycleServiceTests()
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});

		DirectoryInfo tempDir = Directory.CreateDirectory(Path.Combine(Path.GetTempPath(), "horde-agent-test-" + Path.GetRandomFileName()));
		AgentSettings settings = new() { WorkingDir = new DirectoryReference(tempDir) };
		_terminationSignalFile = settings.GetTerminationSignalFile();

		_fakeImds = new(_clock);
		_httpClient = _fakeImds.CreateHttpClient();
		_service = new AwsInstanceLifecycleService(_workerService, _httpClient, _clock, new TestOptionsMonitor<AgentSettings>(settings),
			loggerFactory.CreateLogger<AwsInstanceLifecycleService>());
		_service._timeToLiveAsg = TimeSpan.FromMilliseconds(10);
		_service._timeToLiveSpot = TimeSpan.FromMilliseconds(20);
		_service._terminationBufferTime = TimeSpan.FromMilliseconds(2);
		
		AwsInstanceLifecycleService.TerminationWarningDelegate origWarningCallback = _service._terminationWarningCallback;
		_service._terminationWarningCallback = (info, ct) =>
		{
			_info = info;
			Ec2TerminationInfo newInfo = new(_info.State, _info.IsSpot, _info.TimeToLive, DateTime.UnixEpoch + TimeSpan.FromSeconds(2222), _info.Reason);
			return origWarningCallback(newInfo, ct);
		};
		_service._terminationCallback = (info, _) =>
		{
			_info = info;
			return Task.CompletedTask;
		};
	}
	
	[TestMethod]
	[Timeout(5000)]
	public async Task Terminate_Asg_CallbackHasCorrectParametersAsync()
	{
		_fakeImds.IsAsgTerminating = true;
		await _service.MonitorInstanceLifecycleAsync(CancellationToken.None);
		Assert.AreEqual(Ec2InstanceState.TerminatingAsg, _info!.State);
		Assert.IsFalse(_info!.IsSpot);
		Assert.AreEqual(8, _info!.TimeToLive.TotalMilliseconds); // 10 ms for ASG, minus 2 ms for termination buffer
	}
	
	[TestMethod]
	[Timeout(5000)]
	public async Task Terminate_Spot_CallbackHasCorrectParametersAsync()
	{
		_fakeImds.IsSpotInstance = true;
		_fakeImds.IsSpotTerminating = true;
		await _service.MonitorInstanceLifecycleAsync(CancellationToken.None);
		Assert.AreEqual(Ec2InstanceState.TerminatingSpot, _info!.State);
		Assert.IsTrue(_info!.IsSpot);
		Assert.AreEqual(18, _info!.TimeToLive.TotalMilliseconds); // 20 ms for spot, minus 2 ms for termination buffer
	}
	
	[TestMethod]
	[Timeout(5000)]
	public async Task Terminate_Spot_WritesSignalFileAsync()
	{
		_fakeImds.IsSpotInstance = true;
		_fakeImds.IsSpotTerminating = true;
		Assert.IsFalse(File.Exists(_terminationSignalFile.FullName));
		
		Assert.IsFalse(_workerService.TerminateSession);
		await _service.MonitorInstanceLifecycleAsync(CancellationToken.None);
		Assert.IsTrue(_workerService.TerminateSession);
		
		string data = await File.ReadAllTextAsync(_terminationSignalFile.FullName);
		Assert.AreEqual("v1\n18\n2222000\nAWS EC2 Spot interruption\n", data); // 20 ms for spot, minus 2 ms for termination buffer
	}
	
	[TestMethod]
	public async Task Imds_TokenExpired_RefreshesToken_Async()
	{
		_fakeImds.IsSpotInstance = true;
		Assert.AreEqual(true, await _service.IsSpotInstanceAsync(CancellationToken.None));
		_clock.Advance(TimeSpan.FromHours(12));
		Assert.AreEqual(true, await _service.IsSpotInstanceAsync(CancellationToken.None));
	}
	
	public void Dispose()
	{
		_httpClient.Dispose();
		_service.Dispose();
		_fakeImds.Dispose();
	}
}