// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Net;
using System.Text.Json;
using HordeAgent.Services;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeAgent.Tests.Services;

internal class HttpListenerRequestStub(Uri? url) : IHttpListenerRequest
{
	public Uri? Url { get; } = url;
}

internal class HttpListenerResponseStub : IHttpListenerResponse
{
	public string? ContentType { get; set; }
	public int StatusCode { get; set; }
	public long ContentLength64 { get; set; }
	public WebHeaderCollection Headers { get; set; } = [];
	public CookieCollection Cookies { get; set; } = new();
	public Stream OutputStream { get; } = new MemoryStream();
	public void Close()
	{
	}
}

[TestClass]
public sealed class ManagementServiceTests
{
	private readonly WorkerServiceStub _workerServiceStub = new();
	
	[TestMethod]
	public void HttpRequest_NotFound_Async()
	{
		(ErrorResponse? res, IHttpListenerResponse? httpRes) = RequestAdmin<ErrorResponse>("http://localhost/not-found");
		Assert.AreEqual(404, httpRes.StatusCode);
		Assert.IsFalse(res!.Success);
		Assert.AreEqual("application/json", httpRes.ContentType);
		Assert.AreEqual("Not Found", res.Message);
	}
	
	[TestMethod]
	public void HttpRequest_HealthCheck_Async()
	{
		{
			_workerServiceStub.IsConnected = true;
			(HealthCheckResponse? res, IHttpListenerResponse? httpRes) = RequestHealth<HealthCheckResponse>("http://localhost/");
			Assert.AreEqual(200, httpRes.StatusCode);
			Assert.IsTrue(res!.IsHealthy);
			Assert.IsTrue(res.IsConnected);
		}
		{
			_workerServiceStub.IsConnected = false;
			(HealthCheckResponse? res, IHttpListenerResponse? httpRes) = RequestHealth<HealthCheckResponse>("http://localhost/");
			Assert.AreEqual(503, httpRes.StatusCode);
			Assert.IsFalse(res!.IsHealthy);
			Assert.IsFalse(res.IsConnected);
		}
	}
	
	[TestMethod]
	public void HttpRequest_Pools_Async()
	{
		_workerServiceStub.PoolIds = ["foo", "bar"];
		(PoolsResponse? res, IHttpListenerResponse? httpRes) = RequestHealth<PoolsResponse>("http://localhost/pools");
		Assert.AreEqual(200, httpRes.StatusCode);
		CollectionAssert.AreEquivalent(new List<string> { "bar", "foo" }, res!.PoolIds);
	}
	
	private (T?, IHttpListenerResponse) RequestAdmin<T>(string path)
	{
		return Request<T>(path, (ms, req, res) => ms.TestAdminRoute(req, res));
	}
	
	private (T?, IHttpListenerResponse) RequestHealth<T>(string path)
	{
		return Request<T>(path, (ms, req, res) => ms.TestHealthRoute(req, res));
	}
	
	private (T?, IHttpListenerResponse) Request<T>(string path, Action<ManagementService, IHttpListenerRequest, IHttpListenerResponse> cb)
	{
		using ManagementService ms = GetMs();
		HttpListenerRequestStub req = new(new Uri(path));
		HttpListenerResponseStub res = new();
		cb(ms, req, res);
		res.OutputStream.Position = 0;
		
		T? deserialized = JsonSerializer.Deserialize<T>(res.OutputStream, ManagementService.JsonSerializerOptions);
		return (deserialized, res);
	}
	
	private ManagementService GetMs()
	{
		AgentSettings settings = new ();
		TestOptionsMonitor<AgentSettings> settingsOpt = new(settings);
		return new ManagementService(settingsOpt, _workerServiceStub, NullLogger<ManagementService>.Instance);
	}
}
