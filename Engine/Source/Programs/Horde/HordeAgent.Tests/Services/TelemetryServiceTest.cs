// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using HordeAgent.Services;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeAgent.Tests.Services;

[TestClass]
public sealed class TelemetryServiceTest : IDisposable
{
	private readonly TelemetryService _telemetryService;
	private readonly NullLoggerFactory _loggerFactory = new();

	public TelemetryServiceTest()
	{
		AgentSettings settings = new() { Server = "Test", ServerProfiles = { ["Test"] = new ServerProfile() { Name = "Test", Url = new Uri("http://localhost:1234") } } };
		TestOptionsMonitor<AgentSettings> settingsOpt = new(settings);
		using WorkerService workerService = new WorkerService(null!, null!, null!, null!);
		GrpcService grpcService = new(settingsOpt, NullLogger<GrpcService>.Instance, _loggerFactory);

		_telemetryService = new TelemetryService(NullLogger<TelemetryService>.Instance);
	}

	[TestMethod]
	public async Task NormalEventLoopTimingAsync()
	{
		DateTime now = DateTime.UtcNow;
		_telemetryService.GetUtcNow = () => now;
		Task<(bool onTime, TimeSpan diff)> task = _telemetryService.IsEventLoopOnTimeAsync(TimeSpan.FromSeconds(1), TimeSpan.FromMilliseconds(500), CancellationToken.None);
		_telemetryService.GetUtcNow = () => now + TimeSpan.FromMilliseconds(1000 + 12);
		(bool onTime, TimeSpan diff) = await task;
		Assert.IsTrue(onTime);
	}

	[TestMethod]
	public async Task SlowEventLoopTooEarlyAsync()
	{
		DateTime now = DateTime.UtcNow;
		_telemetryService.GetUtcNow = () => now;
		Task<(bool onTime, TimeSpan diff)> task = _telemetryService.IsEventLoopOnTimeAsync(TimeSpan.FromSeconds(1), TimeSpan.FromMilliseconds(100), CancellationToken.None);
		_telemetryService.GetUtcNow = () => now + TimeSpan.FromMilliseconds(50);
		(bool onTime, TimeSpan diff) = await task;
		Assert.IsFalse(onTime);
	}

	[TestMethod]
	public async Task SlowEventLoopTooLateAsync()
	{
		DateTime now = DateTime.UtcNow;
		_telemetryService.GetUtcNow = () => now;
		Task<(bool onTime, TimeSpan diff)> task = _telemetryService.IsEventLoopOnTimeAsync(TimeSpan.FromSeconds(1), TimeSpan.FromMilliseconds(100), CancellationToken.None);
		_telemetryService.GetUtcNow = () => now + TimeSpan.FromMilliseconds(2500);
		(bool onTime, TimeSpan diff) = await task;
		Assert.IsFalse(onTime);
	}

	[TestMethod]
	public void ParseFltMcOutput()
	{
		string emptyOutput = @"
Filter listing failed with error: 0x80070005
Access is denied.
";

		List<string>? noFilters = TelemetryService.ParseFltMcOutput(emptyOutput);
		Assert.IsNull(noFilters);

		string output = @"
Filter Name                     Num Instances    Altitude    Frame
------------------------------  -------------  ------------  -----
cbfsconnect2017                                             <Legacy>
bindflt                                 1       409800         0
CSAgent                                 9       321410         0
storqosflt                              0       244000         0
wcifs                                   0       189900         0
PrjFlt                                  2       189800         0
CldFlt                                  0       180451         0
FileCrypt                               0       141100         0
luafv                                   1       135000         0
npsvctrig                               1        46000         0
Wof                                     4        40700         0
FileInfo                                7        40500         0

";

		List<string>? filters = TelemetryService.ParseFltMcOutput(output);
		Assert.AreEqual("cbfsconnect2017", filters![0]);
		Assert.AreEqual("bindflt", filters![1]);
		Assert.AreEqual("FileInfo", filters![11]);
	}

	public void Dispose()
	{
		_telemetryService.Dispose();
		_loggerFactory.Dispose();
	}
}
