// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Runtime.Versioning;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeAgent.Tests.Utility;

[TestClass]
[SupportedOSPlatform("windows")]
public class WindowsProcessMemoryTrackerTests
{
	private ILogger _logger = null!;

	[TestInitialize]
	public void Setup()
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});

		_logger = loggerFactory.CreateLogger("WindowsProcessMemoryTrackerTests");
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task TrackerStartsAndStopsCleanlyAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();
		using WindowsProcessMemoryTracker tracker = new(processGroup, TimeSpan.FromMilliseconds(100), _logger);

		tracker.Start();
		await Task.Delay(150); // Let it run for a bit

		// Should dispose cleanly without throwing
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task TrackerReturnsEmptyListWhenNoProcessesAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		using WindowsProcessMemoryTracker tracker = new(processGroup, TimeSpan.FromMilliseconds(100), _logger);
		tracker.Start();
		await Task.Delay(50); // Give it time to sample

		IReadOnlyList<ProcessMemoryInfo> topProcesses = tracker.GetTopMemoryProcesses(5);

		// Should return empty list, not throw
		Assert.IsNotNull(topProcesses);
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task TrackerTracksChildProcessMemoryAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		// Start a simple process that allocates some memory
		using ManagedProcess process = new(
			processGroup,
			"cmd.exe",
			"/c ping -n 2 127.0.0.1",
			null,
			null,
			ProcessPriorityClass.Normal);

		using WindowsProcessMemoryTracker tracker = new(
			processGroup,
			TimeSpan.FromMilliseconds(50),
			_logger);

		tracker.Start();

		// Wait for process to run and be sampled
		await Task.Delay(200);

		IReadOnlyList<ProcessMemoryInfo> topProcesses = tracker.GetTopMemoryProcesses(5);

		// Should have tracked at least the cmd.exe process
		Assert.IsTrue(topProcesses.Count > 0, "Expected at least one tracked process");

		// Verify the tracked process has reasonable data
		ProcessMemoryInfo firstProcess = topProcesses[0];
		Assert.IsTrue(firstProcess.ProcessId > 0, "ProcessId should be positive");
		Assert.IsFalse(string.IsNullOrEmpty(firstProcess.ProcessName), "ProcessName should not be empty");
		Assert.IsTrue(firstProcess.PeakPrivateBytes > 0, "PeakPrivateBytes should be positive");
		Assert.IsTrue(firstProcess.PeakWorkingSetBytes > 0, "PeakWorkingSetBytes should be positive");

		// Wait for process to finish
		await process.WaitForExitAsync(CancellationToken.None);
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task TrackerUpdatesPeakMemoryOverTimeAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		// Start a longer-running process
		using ManagedProcess process = new(
			processGroup,
			"cmd.exe",
			"/c ping -n 3 127.0.0.1",
			null,
			null,
			ProcessPriorityClass.Normal);

		using WindowsProcessMemoryTracker tracker = new(
			processGroup,
			TimeSpan.FromMilliseconds(50),
			_logger);

		tracker.Start();

		// Wait for first sample
		await Task.Delay(100);

		IReadOnlyList<ProcessMemoryInfo> firstSample = tracker.GetTopMemoryProcesses(5);
		Assert.IsTrue(firstSample.Count > 0, "Expected at least one tracked process after first sample");

		// Wait for more samples
		await Task.Delay(200);

		IReadOnlyList<ProcessMemoryInfo> laterSample = tracker.GetTopMemoryProcesses(5);
		Assert.IsTrue(laterSample.Count > 0, "Expected at least one tracked process after later sample");

		// Peak values should be maintained (not reset)
		// Note: We can't guarantee peak increased, but it should at least not be zero
		Assert.IsTrue(laterSample[0].PeakPrivateBytes > 0);
		Assert.IsTrue(laterSample[0].PeakWorkingSetBytes > 0);

		// Wait for process to finish
		await process.WaitForExitAsync(CancellationToken.None);
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task TrackerCapturesCommandLineAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		// Start a process with a distinctive command line
		string uniqueArg = $"test-arg-{Guid.NewGuid():N}";
		using ManagedProcess process = new(
			processGroup,
			"cmd.exe",
			$"/c echo {uniqueArg} && ping -n 2 127.0.0.1",
			null,
			null,
			ProcessPriorityClass.Normal);

		using WindowsProcessMemoryTracker tracker = new(
			processGroup,
			TimeSpan.FromMilliseconds(50),
			_logger);

		tracker.Start();

		// Wait for sampling
		await Task.Delay(150);

		IReadOnlyList<ProcessMemoryInfo> topProcesses = tracker.GetTopMemoryProcesses(10);

		// Find cmd.exe in the list
		ProcessMemoryInfo? cmdProcess = null;
		foreach (ProcessMemoryInfo proc in topProcesses)
		{
			if (proc.ProcessName.Equals("cmd", StringComparison.OrdinalIgnoreCase))
			{
				cmdProcess = proc;
				break;
			}
		}

		Assert.IsNotNull(cmdProcess, "Expected to find cmd.exe in tracked processes");

		// Command line should be captured (may be truncated)
		Assert.IsFalse(string.IsNullOrEmpty(cmdProcess.CommandLine), "CommandLine should not be empty");

		// Wait for process to finish
		await process.WaitForExitAsync(CancellationToken.None);
	}

	[TestMethod]
	[TestCategory("Windows")]
	[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "MSTEST0032:Assertion condition is always true", Justification = "")]
	public void TrackerTruncatesLongCommandLines()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		// Verify the MaxCommandLineLength constant is reasonable
		Assert.AreEqual(500, WindowsProcessMemoryTracker.MaxCommandLineLength);
	}

	[TestMethod]
	[TestCategory("Windows")]
	public async Task GetTopMemoryProcessesReturnsCorrectCountAsync()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		// Start multiple processes
		using ManagedProcess process1 = new(
			processGroup,
			"cmd.exe",
			"/c ping -n 2 127.0.0.1",
			null,
			null,
			ProcessPriorityClass.Normal);

		using ManagedProcess process2 = new(
			processGroup,
			"cmd.exe",
			"/c ping -n 2 127.0.0.1",
			null,
			null,
			ProcessPriorityClass.Normal);

		using WindowsProcessMemoryTracker tracker = new(
			processGroup,
			TimeSpan.FromMilliseconds(50),
			_logger);

		tracker.Start();

		// Wait for sampling
		await Task.Delay(150);

		// Request only 1 process
		IReadOnlyList<ProcessMemoryInfo> top1 = tracker.GetTopMemoryProcesses(1);
		Assert.IsTrue(top1.Count <= 1, "GetTopMemoryProcesses(1) should return at most 1 process");

		// Request 5 processes
		IReadOnlyList<ProcessMemoryInfo> top5 = tracker.GetTopMemoryProcesses(5);
		Assert.IsTrue(top5.Count <= 5, "GetTopMemoryProcesses(5) should return at most 5 processes");

		// Verify processes are sorted by PeakPrivateBytes descending
		for (int i = 1; i < top5.Count; i++)
		{
			Assert.IsTrue(top5[i - 1].PeakPrivateBytes >= top5[i].PeakPrivateBytes,
				"Processes should be sorted by PeakPrivateBytes descending");
		}

		// Wait for processes to finish
		await process1.WaitForExitAsync(CancellationToken.None);
		await process2.WaitForExitAsync(CancellationToken.None);
	}

	[TestMethod]
	[TestCategory("Windows")]
	public void TrackerThrowsWhenStartedTwice()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		using WindowsProcessMemoryTracker tracker = new(
			processGroup,
			TimeSpan.FromMilliseconds(100),
			_logger);

		tracker.Start();

		Assert.ThrowsExactly<InvalidOperationException>(() => tracker.Start());
	}

	[TestMethod]
	[TestCategory("Windows")]
	public void TrackerThrowsOnNullProcessGroup()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		// Constructor throws before any resources are allocated, so no dispose needed
		Assert.ThrowsExactly<ArgumentNullException>(() =>
			_ = new WindowsProcessMemoryTracker(null!, TimeSpan.FromSeconds(30), _logger));
	}

	[TestMethod]
	[TestCategory("Windows")]
	public void TrackerThrowsOnNullLogger()
	{
		if (!OperatingSystem.IsWindows())
		{
			Assert.Inconclusive("This test only runs on Windows");
			return;
		}

		using ManagedProcessGroup processGroup = new();

		// Constructor throws before any resources are allocated, so no dispose needed
		Assert.ThrowsExactly<ArgumentNullException>(() =>
			_ = new WindowsProcessMemoryTracker(processGroup, TimeSpan.FromSeconds(30), null!));
	}
}
