FROM alpine:3.13 as builder

# Install Zig and dependencies
RUN apk update && apk add curl xz libc-dev samurai git cmake
ARG ZIGVER=0.9.1
RUN mkdir -p /deps
WORKDIR /deps
RUN curl https://ziglang.org/deps/zig+llvm+lld+clang-$(uname -m)-linux-musl-$ZIGVER.tar.xz  -O && \
    tar xf zig+llvm+lld+clang-$(uname -m)-linux-musl-$ZIGVER.tar.xz && \
    mv zig+llvm+lld+clang-$(uname -m)-linux-musl-$ZIGVER/ local/

# Build the binary for different platforms
ADD remote-exec-test-bin.c /
RUN /deps/local/bin/zig cc -o /remote-exec-test-x86_64-windows.exe /remote-exec-test-bin.c -target x86_64-windows-gnu
RUN /deps/local/bin/zig cc -o /remote-exec-test-x86_64-linux /remote-exec-test-bin.c -target x86_64-linux-gnu
RUN /deps/local/bin/zig cc -o /remote-exec-test-x86_64-macos /remote-exec-test-bin.c -target x86_64-macos-gnu
RUN /deps/local/bin/zig cc -o /remote-exec-test-aarch64-linux /remote-exec-test-bin.c -target aarch64-linux-gnu
RUN /deps/local/bin/zig cc -o /remote-exec-test-aarch64-macos /remote-exec-test-bin.c -target aarch64-macos-gnu

# Copy binaries to a scratch image so that they can be cleanly be copied back to host system
FROM scratch
COPY --from=builder /remote-exec-test-x86_64-windows.exe /
COPY --from=builder /remote-exec-test-x86_64-linux /
COPY --from=builder /remote-exec-test-x86_64-macos /
COPY --from=builder /remote-exec-test-aarch64-linux /
COPY --from=builder /remote-exec-test-aarch64-macos /