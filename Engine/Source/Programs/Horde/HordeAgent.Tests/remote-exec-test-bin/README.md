# Test binary for remote execution

Provided in this directory are binaries for the different platforms that remote execution in Horde supports.
The binary accepts two arguments: one for sleeping an arbitrary amount of time (for simulating work) and one for specifying a forced exit code.
This is useful for testing and verifying behavior of the different platforms.

## Building
The binaries are built using the Dockerfile together with Zig.
The cross-compilation capability offered by Zig out of the box is great for producing the binaries for Windows, Linux and macOS together with their corresponding processor architectures.

Simply run `build.bat` on Windows with Docker installed to produce new binaries in the same directory (works on Linux or macOS too if the command-line is copied from batch file).