#include <stdio.h>
#include <stdlib.h>

#ifdef _WINDOWS
#include <windows.h>
#else
#include <unistd.h>
#define Sleep(x) usleep((x)*1000)
#endif

int main(int argc, char *argv[]) {
    printf("Horde Remote Execution Test Binary\n");
    int exitCode = 0;
    int sleepMs = -1;
    
    if (argc == 2) {
        sleepMs = atoi(argv[1]);
    } else if (argc == 3) {
        sleepMs = atoi(argv[1]);
        exitCode = atoi(argv[2]);
    }

    printf("    Sleep: %d ms\n", sleepMs);
    printf("Exit code: %d\n", exitCode);

    if (sleepMs > 0) {
        Sleep(sleepMs);
    }

    return exitCode;
}