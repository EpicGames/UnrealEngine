// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Runtime;
using System.Runtime.InteropServices;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Metrics;
using HordeAgent.Leases;
using HordeAgent.Leases.Handlers;
using HordeAgent.Services;
using HordeAgent.Utility;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.FileProviders;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using Polly;
using IConfigurationSource = Microsoft.Extensions.Configuration.IConfigurationSource;
using JsonConfigurationSource = Microsoft.Extensions.Configuration.Json.JsonConfigurationSource;

namespace HordeAgent
{
	/// <summary>
	/// Injectable list of existing services
	/// </summary>
	class DefaultServices
	{
		/// <summary>
		/// The base configuration object
		/// </summary>
		public IConfiguration Configuration { get; }

		/// <summary>
		/// List of service descriptors
		/// </summary>
		public IEnumerable<ServiceDescriptor> Descriptors { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultServices(IConfiguration configuration, IEnumerable<ServiceDescriptor> descriptors)
		{
			Configuration = configuration;
			Descriptors = descriptors;
		}
	}

	/// <summary>
	/// Entry point
	/// </summary>
	public static class AgentApp
	{
		/// <summary>
		/// Name of the http client
		/// </summary>
		public const string HordeServerClientName = "HordeServer";

		/// <summary>
		/// Path to the root application directory
		/// </summary>
		public static DirectoryReference AppDir { get; } = GetAppDir();

		/// <summary>
		/// Path to the default data directory
		/// </summary>
		public static DirectoryReference DataDir { get; private set; } = DirectoryReference.Combine(AppDir, "Data");

		/// <summary>
		/// The launch arguments
		/// </summary>
		public static IReadOnlyList<string> Args { get; private set; } = null!;

#pragma warning disable IL3000 // Avoid accessing Assembly file path when publishing as a single file
		/// <summary>
		/// Whether agent is packaged as a self-contained package where the .NET runtime is included.
		/// </summary>
		public static bool IsSelfContained => String.IsNullOrEmpty(Assembly.GetExecutingAssembly().Location);
#pragma warning restore IL3000 // Avoid accessing Assembly file path when publishing as a single file		

		/// <summary>
		/// The current application version from compile-time generated version
		/// </summary>
		public static string Version { get; } = VersionInfo.Version;

		/// <summary>
		/// Default settings for json serialization
		/// </summary>
		public static JsonSerializerOptions DefaultJsonSerializerOptions { get; } = new JsonSerializerOptions { AllowTrailingCommas = true, PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase, DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull };

		/// <summary>
		/// Entry point
		/// </summary>
		/// <param name="args">Command-line arguments</param>
		/// <returns>Exit code</returns>
		public static async Task<int> Main(string[] args)
		{
			AgentApp.Args = args;

			CommandLineArguments arguments = new CommandLineArguments(args);

			Dictionary<string, string?> configOverrides = new Dictionary<string, string?>();
			if (arguments.TryGetValue("-Server=", out string? serverOverride))
			{
				configOverrides.Add($"{AgentSettings.SectionName}:{nameof(AgentSettings.Server)}", serverOverride);
			}
			if (arguments.TryGetValue("-WorkingDir=", out string? workingDirOverride))
			{
				configOverrides.Add($"{AgentSettings.SectionName}:{nameof(AgentSettings.WorkingDir)}", workingDirOverride);
			}
			
			List<FileReference> configFilesFromArgs = SplitPath(arguments.GetStringOrDefault("-ConfigFiles=", ""));
			List<string> configSourcesInstalled = [];
			
			// Create the base configuration data by just reading from the application directory.
			// We need to check some settings before being able to read user configuration files.
			IConfiguration configuration = CreateConfig(false, [], configOverrides, out _);
			AgentSettings settings = BindSettings(configuration);

			if (settings.Installed)
			{
				if (OperatingSystem.IsWindows())
				{
					DirectoryReference? commonAppDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData);
					if (commonAppDataDir != null)
					{
						DataDir = DirectoryReference.Combine(commonAppDataDir, "Epic", "Horde", "Agent");
						await CopyDefaultConfigFilesAsync(DataDir);
					}
				}
				
				List<FileReference> agentConfigFiles = [FileReference.Combine(DataDir, "agent.json")];
				agentConfigFiles.AddRange(configFilesFromArgs);
				configuration = CreateConfig(true, agentConfigFiles, configOverrides, out configSourcesInstalled);
				settings = BindSettings(configuration);
			}

			using ILoggerFactory loggerFactory = Logging.CreateLoggerFactory(configuration);
			
			{
				ILogger logger = loggerFactory.CreateLogger(typeof(AgentApp));
				foreach (string configSource in configSourcesInstalled)
				{
					logger.LogInformation("Config source: {ConfigSource}", configSource);
				}
				logger.LogInformation("Logs dir: {LogsDir}", settings.LogsDir);
				LogGcStats(logger);
			}
			
			IServiceCollection services = new ServiceCollection();
			services.AddCommandsFromAssembly(Assembly.GetExecutingAssembly());
			services.AddSingleton(loggerFactory);

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				// Prioritize agent execution time over any job its running.
				// We've seen file copying starving the agent communication to the Horde server, causing a disconnect.
				// Increasing the process priority is speculative fix to combat this.
				using (Process process = Process.GetCurrentProcess())
				{
					process.PriorityClass = ProcessPriorityClass.High;
				}
			}

			// Add all the default 
			IConfigurationSection configSection = configuration.GetSection(AgentSettings.SectionName);
			services.AddOptions<AgentSettings>().Configure(options => configSection.Bind(options)).ValidateDataAnnotations();

			ServerProfile serverProfile = settings.GetCurrentServerProfile();
			if (String.IsNullOrEmpty(serverProfile.Url.Scheme) || String.IsNullOrEmpty(serverProfile.Url.Host))
			{
				ILogger logger = loggerFactory.CreateLogger(typeof(AgentApp));
				logger.LogError("\"{Url}\" is an invalid server url. The specified url must have a valid scheme and host name.", serverProfile.Url);
				return 1;
			}

			OpenTelemetryHelper.Configure(services, settings.OpenTelemetry);
			
			services.AddSingleton<IClock, DefaultClock>();
			services.AddHorde(options =>
			{
				options.ServerUrl = serverProfile.Url;
				options.AccessToken = serverProfile.GetAuthToken();
				options.AllowAuthPrompt = serverProfile.UseInteractiveAuth;
				options.BackendCache.CacheDir = DirectoryReference.Combine(settings.WorkingDir, "Saved", "Bundles").FullName;
				options.BackendCache.MaxSize = settings.BundleCacheSize * 1024 * 1024;
			});

			services.AddHttpClient(AwsInstanceLifecycleService.HttpClientName)
				.AddTransientHttpErrorPolicy(builder =>
				{
					return builder.WaitAndRetryAsync(new[] { TimeSpan.FromSeconds(1), TimeSpan.FromSeconds(5), TimeSpan.FromSeconds(5), TimeSpan.FromSeconds(5) });
				});
			services.AddSingleton<AwsInstanceLifecycleService>();
			if (settings.EnableAwsEc2Support)
			{
				services.AddHostedService(sp => sp.GetRequiredService<AwsInstanceLifecycleService>());
			}

			services.AddSingleton<GrpcService>();
			services.AddSingleton<TelemetryService>();
			services.AddHostedService(sp => sp.GetRequiredService<TelemetryService>());
			services.AddSingleton<RuntimeWatchdog>();
			services.AddHostedService(sp => sp.GetRequiredService<RuntimeWatchdog>());

			services.AddSingleton<JobHandlerFactory>();
			services.AddSingleton<StatusService>();
			services.AddHostedService<StatusService>(sp => sp.GetRequiredService<StatusService>());
			services.AddSingleton<ManagementService>();
			services.AddHostedService<ManagementService>(sp => sp.GetRequiredService<ManagementService>());

			services.AddSingleton<LeaseHandlerFactory, ComputeHandlerFactory>();
			services.AddSingleton<LeaseHandlerFactory, ConformHandlerFactory>();
			services.AddSingleton<LeaseHandlerFactory, JobHandlerFactory>(x => x.GetRequiredService<JobHandlerFactory>());
			services.AddSingleton<LeaseHandlerFactory, RestartHandlerFactory>();
			services.AddSingleton<LeaseHandlerFactory, ShutdownHandlerFactory>();
			services.AddSingleton<LeaseHandlerFactory, UpgradeHandlerFactory>();

			services.AddSingleton<CapabilitiesService>();
			services.AddSingleton<ISessionFactory, SessionFactory>();
			services.AddSingleton<WorkerService>();
			services.AddSingleton<IWorkerService>(sp => sp.GetRequiredService<WorkerService>());
			services.AddSingleton<LeaseLoggerFactory>();
			services.AddHostedService(sp => sp.GetRequiredService<WorkerService>());

			services.AddSingleton<ComputeListenerService>();
			services.AddHostedService(sp => sp.GetRequiredService<ComputeListenerService>());

			services.AddMemoryCache();

			services.AddSingleton<ISystemMetrics>(sp => CreateSystemMetrics(settings.WorkingDir, sp.GetRequiredService<ILogger<ISystemMetrics>>()));

			// Allow commands to augment the service collection for their own DI service providers
			services.AddSingleton<DefaultServices>(x => new DefaultServices(configuration, services));

			// Execute all the commands
			await using ServiceProvider serviceProvider = services.BuildServiceProvider();
			return await CommandHost.RunAsync(arguments, serviceProvider, typeof(Commands.Service.RunCommand));
		}

		static ISystemMetrics CreateSystemMetrics(DirectoryReference workingDir, ILogger logger)
		{
			try
			{
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					return new WindowsSystemMetrics(workingDir);
				}
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
				{
					return new LinuxSystemMetrics(workingDir);
				}
				if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
				{
					return new MacSystemMetrics(workingDir);
				}
			}
			catch (Exception e)
			{
				logger.LogError(e, "Unable to initialize system metric collector for telemetry. Disabling. Reason: {Message}", e.Message);
			}
			return new DefaultSystemMetrics();
		}

		static IConfiguration CreateConfig(bool readInstalledConfig, List<FileReference> agentConfigFiles, Dictionary<string, string?> configOverrides, out List<string> configSources)
		{
			configSources = new List<string>();
			string? environment = Environment.GetEnvironmentVariable("DOTNET_ENVIRONMENT");
			if (String.IsNullOrEmpty(environment))
			{
				environment = "Production";
			}

			IConfigurationBuilder builder = new ConfigurationBuilder();
			if (readInstalledConfig && OperatingSystem.IsWindows())
			{
				static bool IncludeRegistrySetting(string name)
					=> !String.Equals(name, $"{AgentSettings.SectionName}:Installed", StringComparison.OrdinalIgnoreCase);

				builder = builder.Add(new RegistryConfigurationSource(Registry.LocalMachine, "SOFTWARE\\Epic Games\\Horde\\Agent", AgentSettings.SectionName, IncludeRegistrySetting));
			}
			
			string basePath = AppDir.FullName;
			builder.SetBasePath(basePath)
				.AddJsonFile("appsettings.json", optional: false)
				.AddJsonFile("appsettings.Build.json", optional: true) // specific settings for builds (installer/dockerfile)
				.AddJsonFile($"appsettings.{environment}.json", optional: true) // environment variable overrides, also used in k8s setups with Helm
				.AddJsonFile("appsettings.User.json", optional: true);
			
			foreach (FileReference configFile in agentConfigFiles.Where(path => Path.Exists(path.FullName)))
			{
				// Adding a JSON file outside the common base path requires adding a completely separate JsonConfigurationSource entry
				builder.Add(new JsonConfigurationSource()
				{
					Path = configFile.GetFileName(),
					Optional = true,
					ReloadOnChange = true,
					FileProvider = new PhysicalFileProvider(configFile.Directory.FullName),
				});
			}
			
			foreach (IConfigurationSource source in builder.Sources)
			{
				switch (source)
				{
					case JsonConfigurationSource jcs:
						string path = jcs.FileProvider is PhysicalFileProvider pfp
							? Path.Join(pfp.Root, jcs.Path)
							: Path.Join(basePath, jcs.Path); // Use shared base path if file provider is missing
						configSources.Add("JSON file: " + path);
						break;
					case RegistryConfigurationSource rcs:
						if (OperatingSystem.IsWindows())
						{
							configSources.Add("Windows Registry key: " + rcs.GetRegistryKey());	
						}
						break;
					case null:
						throw new ArgumentException(nameof(source));
				}
			}

			return builder
				.AddInMemoryCollection(configOverrides)
				.AddEnvironmentVariables()
				.Build();
		}

		internal static AgentSettings BindSettings(IConfiguration configuration)
		{
			AgentSettings settings = new AgentSettings();
			configuration.GetSection(AgentSettings.SectionName).Bind(settings);
			return settings;
		}

		static async Task CopyDefaultConfigFilesAsync(DirectoryReference configDir)
		{
			DirectoryReference.CreateDirectory(configDir);

			DirectoryReference defaultsDir = DirectoryReference.Combine(AppDir, "Defaults");
			if (DirectoryReference.Exists(defaultsDir))
			{
				foreach (FileReference sourceFile in DirectoryReference.EnumerateFiles(defaultsDir, "*.json"))
				{
					FileReference targetFile = FileReference.Combine(configDir, sourceFile.GetFileName());
					if (!FileReference.Exists(targetFile))
					{
						using FileStream targetStream = FileReference.Open(targetFile, FileMode.Create, FileAccess.Write, FileShare.Read);
						using FileStream sourceStream = FileReference.Open(sourceFile, FileMode.Open, FileAccess.Read, FileShare.Read);
						await sourceStream.CopyToAsync(targetStream);
					}
				}
			}
		}

		/// <summary>
		/// Gets the application directory
		/// </summary>
		/// <returns></returns>
		[SuppressMessage("SingleFile", "IL3000:Avoid accessing Assembly file path when publishing as a single file", Justification = "Has fallback handling")]
		static DirectoryReference GetAppDir()
		{
			string? directoryName = Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location);
			if (!String.IsNullOrEmpty(directoryName))
			{
				return new DirectoryReference(directoryName);
			}

			// When C# project is packaged as a single file, GetExecutingAssembly above does not work
			return DirectoryReference.FromFile(new FileReference(Environment.ProcessPath!));
		}
		
		/// <summary>
		/// Splits a string containing multiple file paths
		/// Uses platform-specific path separators: semicolon (;) on Windows, colon (:) on other operating systems.
		/// </summary>
		/// <param name="paths">String containing one or more paths separated by the platform-specific separator</param>
		/// <returns>List of FileReference objects, empty list if input is null or empty</returns>
		internal static List<FileReference> SplitPath(string paths)
		{
			if (String.IsNullOrEmpty(paths))
			{
				return new List<FileReference>();
			}
			
			char separator = OperatingSystem.IsWindows() ? ';' : ':';
			return paths
				.Split(new[] { separator }, StringSplitOptions.RemoveEmptyEntries)
				.Select(path => path.Trim())
				.Where(path => !String.IsNullOrEmpty(path))
				.Select(path => new FileReference(path))
				.ToList();
		}

		internal static void LogGcStats(ILogger logger)
		{
			string concurrentGc = "<unknown>";
			if (GC.GetConfigurationVariables().TryGetValue("ConcurrentGC", out object? concurrentGcObj))
			{
				concurrentGc = concurrentGcObj?.ToString() ?? "null";
			}
			
			logger.LogInformation("GC: ServerGc={ServerGc} LatencyMode={LatencyMode} ConcurrentGc={ConcurrentGc}",
				GCSettings.IsServerGC, GCSettings.LatencyMode, concurrentGc);
		}
	}
}
