// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Utilities;
using HordeCommon.Rpc;
using HordeServer.Utilities;
using Microsoft.Extensions.Configuration;

namespace HordeAgent
{
	/// <summary>
	/// Defines the operation mode of the agent
	/// Duplicated to prevent depending on Protobuf structures (weak name reference when deserializing JSON)
	/// </summary>
	public enum AgentMode
	{
		/// <see cref="Mode.Dedicated" />
		Dedicated,
		/// <see cref="Mode.Workstation" />
		Workstation,
	}
	
	/// <summary>
	/// Describes a network share to mount
	/// </summary>
	public class MountNetworkShare
	{
		/// <summary>
		/// Where the share should be mounted on the local machine. Must be a drive letter for Windows.
		/// </summary>
		public string? MountPoint { get; set; }

		/// <summary>
		/// Path to the remote resource
		/// </summary>
		public string? RemotePath { get; set; }
	}

	/// <summary>
	/// Information about a server to use
	/// </summary>
	public class ServerProfile
	{
		/// <summary>
		/// Name of this server profile
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// Name of the environment (currently just used for tracing)
		/// </summary>
		[Required]
		public string Environment { get; set; } = "prod";

		/// <summary>
		/// Url of the server
		/// </summary>
		[Required]
		public Uri Url { get; set; } = null!;

		/// <summary>
		/// Bearer token to use to initiate the connection
		/// </summary>
		public string? Token { get; set; }
		
		/// <summary>
		/// Whether to authenticate interactively in a desktop environment (for example, when agent is running on a user's workstation)
		/// </summary>
		public bool UseInteractiveAuth { get; set; } = false;

		/// <summary>
		/// Thumbprint of a certificate to trust. Allows using self-signed certs for the server.
		/// </summary>
		public string? Thumbprint { get; set; }

		/// <summary>
		/// Thumbprints of certificates to trust. Allows using self-signed certs for the server.
		/// </summary>
		public List<string> Thumbprints { get; } = new List<string>();
		
		/// <summary>
		/// Returns auth token if interactive auth is disabled. Otherwise, null is returned.
		/// </summary>
		public string? GetAuthToken()
		{
			return UseInteractiveAuth ? null : Token;
		}

		/// <summary>
		/// Checks whether the given certificate thumbprint should be trusted
		/// </summary>
		/// <param name="certificateThumbprint">The cert thumbprint</param>
		/// <returns>True if the cert should be trusted</returns>
		public bool IsTrustedCertificate(string certificateThumbprint)
		{
			if (Thumbprint != null && Thumbprint.Equals(certificateThumbprint, StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (Thumbprints.Any(x => x.Equals(certificateThumbprint, StringComparison.OrdinalIgnoreCase)))
			{
				return true;
			}
			return false;
		}
	}

	/// <summary>
	/// Global settings for the agent
	/// </summary>
	[ConfigDoc("Agent.json (Agent)", "[Horde](../../README.md) > [Deployment](../Deployment.md) > [Agent](Agent.md)", "Deployment/AgentSettings.md",
		Introduction = "All Horde-specific settings are stored in a root object called `Horde`. Other .NET functionality may be configured using properties in the root of this file.")]
	public class AgentSettings
	{
		/// <summary>
		/// Name of the section containing these settings
		/// </summary>
		public const string SectionName = "Horde";

		/// <summary>
		/// Known servers to connect to
		/// </summary>
		public Dictionary<string, ServerProfile> ServerProfiles { get; } = new Dictionary<string, ServerProfile>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// The default server, unless overridden from the command line
		/// </summary>
		public string? Server { get; set; }

		/// <summary>
		/// Name of agent to report as when connecting to server.
		/// By default, the computer's hostname will be used.
		/// </summary>
		public string? Name { get; set; }
		
		/// <summary>
		/// Mode of operation for the agent
		/// <see cref="AgentMode.Dedicated"/> - For trusted agents in controlled environments (e.g., build farms). 
		/// These agents handle all lease types and run exclusively Horde workloads.
		/// 
		/// <see cref="AgentMode.Workstation"/> - For low-trust workstations, uses interactive authentication (human logs in).
		/// These agents yield to non-Horde workloads and only support compute leases for remote execution.
		/// </summary>
		public AgentMode? Mode { get; set; } = AgentMode.Dedicated;

		/// <summary>
		/// Whether the server is running in 'installed' mode. In this mode, on Windows, the default data directory will use the common 
		/// application data folder (C:\ProgramData\Epic\Horde), and configuration data will be read from here and the registry.
		/// This setting is overridden to false for local builds from appsettings.Local.json.
		/// </summary>
		public bool Installed { get; set; } = true;

		/// <summary>
		/// Whether agent should register as being ephemeral.
		/// Doing so will not persist any long-lived data on the server and
		/// once disconnected it's assumed to have been deleted permanently.
		/// Ideal for short-lived agents, such as spot instances on AWS EC2.
		/// </summary>
		public bool Ephemeral { get; set; } = false;

		/// <summary>
		/// Working directory for leases and jobs (i.e where files from Perforce will be checked out) 
		/// </summary>
		public DirectoryReference WorkingDir { get; set; } = DirectoryReference.Combine(AgentApp.DataDir, "Sandbox");

		/// <summary>
		/// Directory where agent and lease logs are written
		/// </summary>
		public DirectoryReference LogsDir { get; set; } = AgentApp.DataDir;

		/// <summary>
		/// Whether to mount the specified list of network shares
		/// </summary>
		public bool ShareMountingEnabled { get; set; } = true;

		/// <summary>
		/// List of network shares to mount
		/// </summary>
		public List<MountNetworkShare> Shares { get; } = new List<MountNetworkShare>();

		/// <summary>
		/// Path to Wine executable. If null, execution under Wine is disabled
		/// </summary>
		public string? WineExecutablePath { get; set; }

		/// <summary>
		/// Path to container engine executable, such as /usr/bin/podman. If null, execution of compute workloads inside a container is disabled
		/// </summary>
		public string? ContainerEngineExecutablePath { get; set; }

		/// <summary>
		/// Whether to write step output to the logging device
		/// </summary>
		public bool WriteStepOutputToLogger { get; set; }

		/// <summary>
		/// Queries information about the current agent through the AWS EC2 interface
		/// </summary>
		public bool EnableAwsEc2Support { get; set; } = false;

		/// <summary>
		/// Option to use a local storage client rather than connecting through the server. Primarily for convenience when debugging / iterating locally.
		/// </summary>
		public bool UseLocalStorageClient { get; set; }
		
		/// <summary>
		/// Incoming IP for listening for compute work. If not set, it will be automatically resolved.
		/// </summary>
		public string? ComputeIp { get; set; } = null;
		
		/// <summary>
		/// Incoming port for listening for compute work. Needs to be tied with a lease. Set port to 0 to disable incoming compute requests.
		/// </summary>
		public int ComputePort { get; set; } = 7000;
		
		/// <summary>
		/// Options for OpenTelemetry
		/// </summary>
		public OpenTelemetrySettings OpenTelemetry { get; set; } = new ();

		/// <summary>
		/// Whether to send telemetry back to Horde server
		/// </summary>
		public bool EnableTelemetry { get; set; } = false;

		/// <summary>
		/// How often to report telemetry events to server in milliseconds
		/// </summary>
		public int TelemetryReportInterval { get; set; } = 30 * 1000;

		/// <summary>
		/// Maximum size of the bundle cache, in megabytes.
		/// </summary>
		public long BundleCacheSize { get; set; } = 1024;

		/// <summary>
		/// Maximum number of logical CPU cores workloads should use
		/// Currently this is only provided as a hint and requires leases to respect this value as it's set via an env variable (UE_HORDE_CPU_COUNT).
		/// </summary>
		public int? CpuCount { get; set; } = null;
		
		/// <summary>
		/// CPU core multiplier applied to CPU core count setting
		/// For example, 32 CPU cores and a multiplier of 0.5 results in max 16 CPU usage.
		/// <see cref="CpuCount" />
		/// </summary>
		public double CpuMultiplier { get; set; } = 1.0;

		/// <summary>
		/// Maximum available RAM in gigabytes. Defaults to available RAM.
		/// </summary>
		public int? RamGb { get; set; } = null;

		/// <summary>
		/// Key/value properties in addition to those set internally by the agent
		/// </summary>
		public Dictionary<string, string> Properties { get; } = new();
		
		/// <summary>
		/// Listen addresses for the built-in HTTP admin/management server. Disabled when empty.
		/// If activated, it's recommended to bind only to localhost for security reasons.
		/// Example: localhost:7008 to listen on localhost, port 7008
		/// </summary>
		public string[] AdminEndpoints { get; set; } = [];
		
		/// <summary>
		/// Listen addresses for the built-in HTTP health check server. Disabled when empty.
		/// If activated, it's recommended to bind only to localhost for security reasons.
		/// Example: *:7009 to listen on all interfaces/IPs, port 7009
		/// If all interfaces are bound with *, make sure to run process as administrator.
		/// </summary>
		public string[] HealthCheckEndpoints { get; set; } = [];

		/// <summary>
		/// Maximum number of temp storage exports to write in a single request
		/// </summary>
		public int? TempStorageMaxBatchSize { get; set; } = null;

		/// <summary>
		/// Whether to log optional input retrieval failures as warnings instead of information messages.
		/// Defaults to false (information level).
		/// </summary>
		public bool LogOptionalInputFailureAsWarning { get; set; } = false;

		/// <summary>
		/// Gets the current server settings
		/// </summary>
		/// <returns>The current server settings</returns>
		public ServerProfile GetServerProfile(string name)
		{
			ServerProfile? serverProfile;
			if (!ServerProfiles.TryGetValue(name, out serverProfile))
			{
				serverProfile = ServerProfiles.Values.FirstOrDefault(x => name.Equals(x.Name, StringComparison.OrdinalIgnoreCase));
				if (serverProfile == null)
				{
					if (ServerProfiles.Count == 0)
					{
						throw new Exception("No server profiles are defined (missing configuration?)");
					}
					else
					{
						throw new Exception($"Unknown server profile name '{name}' (valid profiles: {GetServerProfileNames()})");
					}
				}
			}
			return serverProfile;
		}

		string GetServerProfileNames()
		{
			HashSet<string> names = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach ((string key, ServerProfile profile) in ServerProfiles)
			{
				if (!String.IsNullOrEmpty(profile.Name) && Int32.TryParse(key, out _))
				{
					names.Add(profile.Name);
				}
				else
				{
					names.Add(key);
				}
			}
			return String.Join("/", names);
		}

		/// <summary>
		/// Gets the current server settings
		/// </summary>
		/// <returns>The current server settings</returns>
		public ServerProfile GetCurrentServerProfile()
		{
			if (Server == null)
			{
				Uri? defaultServerUrl = Installed ? HordeOptions.GetDefaultServerUrl() : null;

				ServerProfile defaultServerProfile = new ServerProfile();
				defaultServerProfile.Name = "Default";
				defaultServerProfile.Environment = "Development";
				defaultServerProfile.Url = defaultServerUrl ?? new Uri("http://localhost:5000");
				return defaultServerProfile;
			}

			return GetServerProfile(Server);
		}

		/// <summary>
		/// Path to file used for signaling impending termination and shutdown of the agent
		/// </summary>
		/// <returns>Path to file which may or may not exist</returns>
		public FileReference GetTerminationSignalFile()
		{
			return FileReference.Combine(WorkingDir, ".horde-termination-signal");
		}

		internal string GetAgentName()
		{
			return Name ?? Environment.MachineName;
		}
		
		internal Mode GetMode()
		{
			return Mode switch
			{
				AgentMode.Dedicated => HordeCommon.Rpc.Mode.Dedicated,
				AgentMode.Workstation => HordeCommon.Rpc.Mode.Workstation,
				_ => throw new Exception($"Unknown agent mode: {Mode}")
			};
		}
	}

	/// <summary>
	/// Extension methods for retrieving config settings
	/// </summary>
	public static class AgentSettingsExtensions
	{
		/// <summary>
		/// Gets the configuration section for the active server profile
		/// </summary>
		/// <param name="configSection"></param>
		/// <returns></returns>
		public static IConfigurationSection GetCurrentServerProfile(this IConfigurationSection configSection)
		{
			string? profileName = configSection[nameof(AgentSettings.Server)];
			if (profileName == null)
			{
				throw new Exception("Server is not set");
			}

			return configSection.GetSection(nameof(AgentSettings.ServerProfiles)).GetChildren().First(x => x[nameof(ServerProfile.Name)] == profileName);
		}
	}
}
