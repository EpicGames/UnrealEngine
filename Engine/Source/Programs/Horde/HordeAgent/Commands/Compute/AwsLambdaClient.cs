// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Compute
{
	/// <summary>
	/// Class representing the HTTP response from next invocation request
	/// </summary>
	public class NextInvocationResponse
	{
		/// <summary>
		/// Lambda request ID
		/// </summary>
		public string RequestId { get; }

		/// <summary>
		/// Name of the Lambda function being invoked
		/// </summary>
		public string InvokedFunctionArn { get; }

		/// <summary>
		/// Deadline in milliseconds for completing the function invocation
		/// </summary>
		public long DeadlineMs { get; }

		/// <summary>
		/// Data
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="requestId"></param>
		/// <param name="invokedFunctionArn"></param>
		/// <param name="deadlineMs"></param>
		/// <param name="data"></param>
		public NextInvocationResponse(string requestId, string invokedFunctionArn, long deadlineMs, byte[] data)
		{
			RequestId = requestId;
			InvokedFunctionArn = invokedFunctionArn;
			DeadlineMs = deadlineMs;
			Data = data;
		}
	}

	/// <summary>
	/// Class representing an exception when communicating with the AWS Lambda Runtime API
	/// </summary>
	public class AwsLambdaClientException : Exception
	{
		/// <summary>
		/// Whether the exception is fatal and requires termination of the process (to adhere to Lambda specs)
		/// </summary>
		public bool IsFatal { get; }

		/// <summary>
		/// HTTP status code in response from AWS Lambda API
		/// </summary>
		public HttpStatusCode? StatusCode { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="message"></param>
		/// <param name="statusCode"></param>
		/// <param name="isFatal"></param>
		public AwsLambdaClientException(string message, bool isFatal = false, HttpStatusCode? statusCode = null) : base(message)
		{
			IsFatal = isFatal;
			StatusCode = statusCode;
		}
	}

	/// <summary>
	/// Client interfacing with the AWS Lambda runtime API
	/// See https://docs.aws.amazon.com/lambda/latest/dg/runtimes-api.html
	/// </summary>
	public class AwsLambdaClient
	{
		private const string EnvVarRuntimeApi = "AWS_LAMBDA_RUNTIME_API";
		private const string HeaderNameAwsRequestId = "Lambda-Runtime-Aws-Request-Id";
		private const string HeaderNameDeadlineMs = "Lambda-Runtime-Deadline-Ms";
		private const string HeaderNameInvokedFunctionArn = "Lambda-Runtime-Invoked-Function-Arn";
		private const string HeaderNameFuncErrorType = "Lambda-Runtime-Function-Error-Type";

		private readonly ILogger<AwsLambdaClient> _logger;
		private readonly string _hostPort;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="hostPort">Base URL to the Lambda Runtime API endpoint</param>
		/// <param name="logger"></param>
		public AwsLambdaClient(string hostPort, ILogger<AwsLambdaClient> logger)
		{
			_hostPort = hostPort;
			_logger = logger;
		}

		/// <summary>
		/// Create an AwsLambdaClient from environment variables. Usually how the client is created when running
		/// inside a real AWS Lambda environment.
		/// </summary>
		/// <param name="logger"></param>
		/// <returns>A configured AwsLambdaClient instance</returns>
		/// <exception cref="ArgumentException"></exception>
		public static AwsLambdaClient InitFromEnv(ILogger<AwsLambdaClient> logger)
		{
			string? runtimeApi = Environment.GetEnvironmentVariable(EnvVarRuntimeApi);
			if (runtimeApi == null)
			{
				throw new ArgumentException($"Env var {EnvVarRuntimeApi} is not set");
			}

			return new AwsLambdaClient(runtimeApi, logger);
		}

		private string GetNextInvocationUrl() => $"http://{_hostPort}/2018-06-01/runtime/invocation/next";
		private string GetInvocationResponseUrl(string awsRequestId) => $"http://{_hostPort}/2018-06-01/runtime/invocation/{awsRequestId}/response";
		private string GetInitErrorUrl() => $"http://{_hostPort}/runtime/init/error";
		private string GetInvocationErrorUrl(string awsRequestId) => $"http://{_hostPort}/2018-06-01/runtime/invocation/{awsRequestId}/error";

		private static HttpClient GetHttpClient()
		{
			return new HttpClient();
		}

		/// <summary>
		/// Get the next invocation from the Lambda runtime API
		///
		/// Can block for many minutes waiting for a new invocation request to be received. 
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns>A response containing details for invocation</returns>
		/// <exception cref="AwsLambdaClientException"></exception>
		private async Task<NextInvocationResponse> GetNextInvocationAsync(CancellationToken cancellationToken)
		{
			using HttpClient client = GetHttpClient();
			client.Timeout = TimeSpan.FromHours(1); // Waiting for the next invocation can block for a while
			using HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Get, GetNextInvocationUrl());

			HttpResponseMessage response = await client.SendAsync(request, cancellationToken);

			if (response.StatusCode != HttpStatusCode.OK)
			{
				throw new AwsLambdaClientException("Failed getting next invocation", false, response.StatusCode);
			}

			if (!response.Headers.Contains(HeaderNameAwsRequestId))
			{
				throw new AwsLambdaClientException($"Missing response header {HeaderNameAwsRequestId}");
			}

			if (!response.Headers.Contains(HeaderNameDeadlineMs))
			{
				throw new AwsLambdaClientException($"Missing response header {HeaderNameDeadlineMs}");
			}

			if (!response.Headers.Contains(HeaderNameInvokedFunctionArn))
			{
				throw new AwsLambdaClientException($"Missing response header {HeaderNameInvokedFunctionArn}");
			}

			string requestId = response.Headers.GetValues(HeaderNameAwsRequestId).First();
			long deadlineMs = Convert.ToInt64(response.Headers.GetValues(HeaderNameDeadlineMs).First());
			string invokedFunctionArn = response.Headers.GetValues(HeaderNameInvokedFunctionArn).First();

			byte[] data = await response.Content.ReadAsByteArrayAsync(cancellationToken);
			return new NextInvocationResponse(requestId, invokedFunctionArn, deadlineMs, data);
		}

		private async Task SendInvocationResponseAsync(string awsRequestId, ReadOnlyMemory<byte> data)
		{
			using HttpClient client = GetHttpClient();
			client.Timeout = TimeSpan.FromSeconds(30);
			using HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Post, GetInvocationResponseUrl(awsRequestId));
			request.Content = new ByteArrayContent(data.ToArray());

			HttpResponseMessage response = await client.SendAsync(request);
			if (response.StatusCode != HttpStatusCode.Accepted)
			{
				_logger.LogError("Failed sending invocation response! RequestId={AwsRequestId} DataLen={DataLen}", awsRequestId, data.Length);
				throw new AwsLambdaClientException("Failed sending invocation response!");
			}
		}

		/// <summary>
		/// Notify the Lambda runtime API that an error occurred during initialization. 
		/// </summary>
		/// <param name="errorType">Any string but AWS recommends a format of [category.reason]</param>
		/// <param name="errorMessage">Message describing the error</param>
		/// <param name="stackTrace">Lines of an (optional) stacktrace</param>
		/// <returns></returns>
		public Task SendInitErrorAsync(string errorType, string errorMessage, List<string>? stackTrace = null)
		{
			return SendErrorAsync(GetInitErrorUrl(), errorType, errorMessage, stackTrace);
		}

		private Task SendInvocationErrorAsync(string awsRequestId, string errorType, string errorMessage, List<string>? stackTrace = null)
		{
			return SendErrorAsync(GetInvocationErrorUrl(awsRequestId), errorType, errorMessage, stackTrace);
		}

		/// <summary>
		/// Listen for new Lambda invocations
		///
		/// Call <paramref name="function"/> for each new invocation received.
		/// Will block until cancellation token is triggered.
		/// </summary>
		/// <param name="function"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<bool> ListenForInvocationsAsync(IAwsLambdaFunction function, CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				NextInvocationResponse? nextInvocationResponse = null;
				try
				{
					_logger.LogDebug("Waiting for next invocation...");

					// Block and wait for the next invocation (can be multiple minutes)
					nextInvocationResponse = await GetNextInvocationAsync(cancellationToken);
					ReadOnlyMemory<byte> responseData = await function.OnLambdaInvokeAsync(nextInvocationResponse.RequestId, nextInvocationResponse.Data, cancellationToken);
					await SendInvocationResponseAsync(nextInvocationResponse.RequestId, responseData);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Exception invoking Lambda function from request");

					if (nextInvocationResponse != null)
					{
						try
						{
							List<string>? stacktraceLines = e.StackTrace?.Split("\n").ToList();
							await SendInvocationErrorAsync(nextInvocationResponse.RequestId, "general", $"{e.GetType()}: {e.Message}", stacktraceLines);
						}
						catch (AwsLambdaClientException sendErrorException)
						{
							_logger.LogError(sendErrorException, "Bad response when sending invocation error. isFatal={IsFatal}", sendErrorException.IsFatal);
							return false;
						}
					}

					await Task.Delay(500, cancellationToken); // Cool down before trying another next invocation call
				}
			}

			return true;
		}

		private static async Task SendErrorAsync(string url, string errorType, string errorMessage, List<string>? stackTrace)
		{
			using HttpClient client = new HttpClient();
			client.Timeout = TimeSpan.FromHours(1);
			using HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Post, url);
			request.Headers.Add(HeaderNameFuncErrorType, errorType);
			string bodyStr = JsonSerializer.Serialize(new
			{
				errorMessage = errorMessage,
				errorType = errorType,
				stackTrace = stackTrace != null ? stackTrace.ToArray() : Array.Empty<string>()
			});
			request.Content = new StringContent(bodyStr);

			HttpResponseMessage response = await client.SendAsync(request);
			if (response.StatusCode != HttpStatusCode.Accepted)
			{
				bool isFatal = response.StatusCode == HttpStatusCode.InternalServerError;
				throw new AwsLambdaClientException($"Failed sending initialization error! Status code {response.StatusCode}", isFatal, response.StatusCode);
			}
		}
	}

	/// <summary>
	/// Represents an AWS Lambda function that can be invoked
	/// </summary>
	public interface IAwsLambdaFunction
	{
		/// <summary>
		/// Handles an AWS Lambda function call
		/// </summary>
		/// <param name="requestId">Lambda request ID for this invocation</param>
		/// <param name="requestData">Raw bytes for the request body</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Raw bytes for the response body</returns>
		public Task<ReadOnlyMemory<byte>> OnLambdaInvokeAsync(string requestId, ReadOnlyMemory<byte> requestData, CancellationToken cancellationToken);
	}
}
