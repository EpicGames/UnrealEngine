// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Net;
using System.Net.Sockets;
using EpicGames.Core;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Transports;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Compute
{
	/// <summary>
	/// Helper command for hosting a local compute worker in a separate process
	/// </summary>
	[Command("computeworker", "Runs the agent as a local compute host, accepting incoming connections on the loopback adapter with a given port")]
	class ComputeWorkerCommand : Command
	{
		[CommandLine("-Port=")]
		[Description("Port to listen for connections on.")]
		int Port { get; set; } = 2000;

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			logger.LogInformation("** WORKER **");

			using Socket tcpSocket = new Socket(SocketType.Stream, ProtocolType.IP);
			await tcpSocket.ConnectAsync(IPAddress.Loopback, Port);

			await using TcpTransport transport = new TcpTransport(tcpSocket);
			await using (RemoteComputeSocket socket = new RemoteComputeSocket(transport, ComputeProtocol.Latest, logger))
			{
				logger.LogInformation("Running worker...");
				await RunWorkerAsync(socket, logger, CancellationToken.None);
				logger.LogInformation("Worker complete");
				await socket.CloseAsync(CancellationToken.None);
			}

			logger.LogInformation("Stopping");
			return 0;
		}

		public static async Task RunWorkerAsync(ComputeSocket socket, ILogger logger, CancellationToken cancellationToken)
		{
			DirectoryReference sandboxDir = DirectoryReference.Combine(AgentApp.DataDir, "Sandbox");

			AgentMessageHandler worker = new AgentMessageHandler(sandboxDir, null, false, null, null, logger);
			await worker.RunAsync(socket, cancellationToken);
		}
	}
}
