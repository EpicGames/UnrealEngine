// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Service
{
	internal record InstallServiceOptions(
		string? UserName = null,
		string? Password = null,
		string? ServerProfile = null,
		string? ConfigFiles = null,
		string DotNetExecutable = "dotnet",
		bool StartService = true,
		bool AutoRestartService = true
	);
	
	/// <summary>
	/// Installs the agent as a Windows service
	/// </summary>
	[Command("service", "install", "Installs the agent as a Windows service")]
	class InstallServiceCommand : Command
	{
		/// <summary>
		/// Name of the service
		/// </summary>
		public const string ServiceName = "HordeAgent";

		[CommandLine("-UserName=")]
		[Description("Specifies the username for the service to run under")]
		public string? UserName { get; set; } = null;

		[CommandLine("-Password=")]
		[Description("Password for the service account")]
		public string? Password { get; set; } = null;

		[CommandLine("-Server=")]
		[Description("The server profile to use")]
		public string? Server { get; set; } = null;

		[CommandLine("-DotNetExecutable=")]
		[Description("Path to dotnet executable (dotnet.exe on Windows). When left empty, the value of \"dotnet\" will be used.")]
		public string DotNetExecutable { get; set; } = "dotnet";

		[CommandLine("-Start=")]
		[Description("Whether to start the service after installation (true/false)")]
		public string? Start { get; set; } = "true";
		
		[CommandLine("-ConfigFiles=")]
		[Description("Paths to additional config JSON files to load, separated by colon (Linux/macOS) or semi-colon (Windows)")]
		public string? ConfigFiles { get; set; } = null;

		[CommandLine("-AutoRestartService=")]
		[Description("Configures the Windows Service to autostart when HordeAgent crashes/exits abnormally. (true/false)")]
		public string AutoRestartService { get; set; } = "true";

		/// <summary>
		/// Installs the service
		/// </summary>
		/// <param name="logger">Logger to use</param>
		/// <returns>Exit code</returns>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			if (!OperatingSystem.IsWindows())
			{
				logger.LogError("This command requires Windows");
				return Task.FromResult(1);
			}

			InstallServiceOptions options = new ()
			{
				UserName = UserName,
				Password = Password,
				ServerProfile = Server,
				DotNetExecutable = DotNetExecutable,
				StartService = Start != null && Start.Equals("true", StringComparison.OrdinalIgnoreCase),
				AutoRestartService = AutoRestartService != null && AutoRestartService.Equals("true", StringComparison.OrdinalIgnoreCase)
			};
			return Task.FromResult(InstallService(options, logger));
		}
		
		internal static bool StopAndDeleteWindowsService(ILogger logger)
		{
			using WindowsServiceManager serviceManager = new ();
			using WindowsService service = serviceManager.Open(ServiceName);
			if (service.IsValid)
			{
				logger.LogInformation("Stopping existing service...");
				service.Stop();

				WindowsServiceStatus status = service.WaitForStatusChange(WindowsServiceStatus.Stopping, TimeSpan.FromSeconds(30.0));
				if (status != WindowsServiceStatus.Stopped)
				{
					logger.LogError("Unable to stop service (status = {Status})", status);
					return false;
				}

				logger.LogInformation("Deleting service...");
				service.Delete();
			}
			
			return true;
		}
		
		internal static int InstallService(InstallServiceOptions options, ILogger logger)
		{
			using WindowsServiceManager serviceManager = new ();
			if (!StopAndDeleteWindowsService(logger))
			{
				return 1;
			}
			
			logger.LogInformation("Registering {ServiceName} service", ServiceName);

			StringBuilder commandLine = new();
			if (AgentApp.IsSelfContained)
			{
				if (Environment.ProcessPath == null)
				{
					logger.LogError("Unable to detect current process path");
					return 1;
				}
				commandLine.Append($"\"{Environment.ProcessPath}\" service run");
			}
			else
			{
#pragma warning disable IL3000 // Avoid accessing Assembly file path when publishing as a single file
				commandLine.AppendFormat("{0} \"{1}\" service run", options.DotNetExecutable, Assembly.GetEntryAssembly()!.Location);
#pragma warning restore IL3000 // Avoid accessing Assembly file path when publishing as a single file					
			}

			if (options.ServerProfile != null)
			{
				commandLine.Append($" -Server={options.ServerProfile}");
			}
			
			if (options.ConfigFiles != null)
			{
				commandLine.Append($" -ConfigFiles=\"{options.ConfigFiles}\"");
			}

			using (WindowsService service = serviceManager.Create(ServiceName, "Horde Agent", commandLine.ToString(), options.UserName, options.Password))
			{
				service.SetDescription("Allows this machine to participate in a Horde farm.");

				// Set the service to auto restart if deemed to.
				if (options.AutoRestartService)
				{
					service.ChangeServiceFailureActions([WindowsServiceRecoverAction.Restart, WindowsServiceRecoverAction.Restart, WindowsServiceRecoverAction.Restart]);
				}

				if (options.StartService)
				{
					logger.LogInformation("Starting...");
					service.Start();

					WindowsServiceStatus status = service.WaitForStatusChange(WindowsServiceStatus.Starting, TimeSpan.FromSeconds(30.0));
					if (status != WindowsServiceStatus.Running)
					{
						logger.LogError("Unable to start service (status = {Status})", status);
						return 1;
					}
				}
				logger.LogInformation("Done");
			}

			return 0;
		}
	}
}
