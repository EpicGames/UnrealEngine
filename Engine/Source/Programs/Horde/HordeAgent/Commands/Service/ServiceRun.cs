// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Serilog.Events;

namespace HordeAgent.Commands.Service
{
	/// <summary>
	/// Runs the agent
	/// </summary>
	[Command("service", "run", "Runs the Horde agent")]
	class RunCommand : Command
	{
		[CommandLine("-LogLevel")]
		[Description("Log verbosity level (use normal MS levels such as debug, warning or information)")]
		public string LogLevelStr { get; set; } = "information";
		
		[CommandLine("-ConfigFiles=")]
		[Description("Paths to additional config JSON files to load, separated by colon (Linux/macOS) or semi-colon (Windows)")]
		// This property only provides help text for the CLI interface. The actual config files are processed earlier in AgentApp.cs.
		public string? ConfigFiles { get; set; } = null;

		readonly DefaultServices _defaultServices;

		/// <summary>
		/// Constructor
		/// </summary>
		public RunCommand(DefaultServices defaultServices)
		{
			_defaultServices = defaultServices;
		}

		/// <summary>
		/// Runs the service indefinitely
		/// </summary>
		/// <returns>Exit code</returns>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (Enum.TryParse(LogLevelStr, true, out LogEventLevel logEventLevel))
			{
				Logging.LogLevelSwitch.MinimumLevel = logEventLevel;
			}
			else
			{
				logger.LogError("Unable to parse log level: {Level}", LogLevelStr);
				return 1;
			}

			using SingleInstanceMutex mutex = new SingleInstanceMutex();
			if (!await mutex.HasMutexAsync())
			{
				logger.LogError("Another instance of the Horde Agent is already running.");
				return 1;
			}

			IHostBuilder hostBuilder = Host.CreateDefaultBuilder();

			// Attempt to setup this process as a Windows service. A race condition inside Microsoft.Extensions.Hosting.WindowsServices.WindowsServiceHelpers.IsWindowsService
			// can result in accessing the parent process after it's terminated, so catch any exceptions that it throws.
			try
			{
				hostBuilder = hostBuilder.UseWindowsService();
			}
			catch (InvalidOperationException)
			{
			}

			hostBuilder = hostBuilder
				.ConfigureAppConfiguration(builder =>
				{
					builder.AddConfiguration(_defaultServices.Configuration);
				})
				.ConfigureLogging(builder =>
				{
					// We add our logger through ConfigureServices, inherited from _defaultServices
					builder.ClearProviders();
				})
				.ConfigureServices((hostContext, services) =>
				{
					services.Configure<HostOptions>(options =>
					{
						// Allow the worker service to terminate any before shutting down
						options.ShutdownTimeout = TimeSpan.FromSeconds(30);
					});

					foreach (ServiceDescriptor descriptor in _defaultServices.Descriptors)
					{
						services.Add(descriptor);
					}
				});

			try
			{
				await hostBuilder.Build().RunAsync();
			}
			catch (OperationCanceledException)
			{
				// Need to catch this to prevent it propagating to the awaiter when pressing ctrl-c
			}

			return 0;
		}

		class SingleInstanceMutex : IDisposable
		{
			Thread _backgroundThread;
			readonly ManualResetEventSlim _quitEvent = new ManualResetEventSlim();
			readonly TaskCompletionSource<bool> _waitEvent = new TaskCompletionSource<bool>();

			public SingleInstanceMutex()
			{
				_backgroundThread = new Thread(WaitForMutex) { IsBackground = true };
				_backgroundThread.Start();
			}

			public Task<bool> HasMutexAsync()
				=> _waitEvent.Task;

			public void Dispose()
			{
				if (_backgroundThread != null)
				{
					_quitEvent.Set();
					_quitEvent.Dispose();

					_backgroundThread.Join();
					_backgroundThread = null!;
				}
			}

			void WaitForMutex()
			{
				using Mutex mutex = new Mutex(false, "HordeAgent.{C1F30772-CDD3-41E9-A6CE-42356DE7DEE3}");
				try
				{
					if (!mutex.WaitOne(0))
					{
						_waitEvent.SetResult(false);
						return;
					}
				}
				catch (AbandonedMutexException)
				{
				}
				_waitEvent.SetResult(true);
				_quitEvent.Wait();
			}
		}
	}
}
