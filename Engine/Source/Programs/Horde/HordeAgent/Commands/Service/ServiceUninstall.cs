// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Service
{
	/// <summary>
	/// Uninstalls the Windows service
	/// </summary>
	[Command("service", "uninstall", "Uninstalls the Windows service")]
	class UninstallServiceCommand : Command
	{
		/// <summary>
		/// Name of the service
		/// </summary>
		public const string ServiceName = InstallServiceCommand.ServiceName;

		/// <summary>
		/// Uninstalls the service
		/// </summary>
		/// <param name="logger">Logger to use</param>
		/// <returns>Exit code</returns>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			using (WindowsServiceManager serviceManager = new WindowsServiceManager())
			{
				using (WindowsService service = serviceManager.Open(ServiceName))
				{
					if (service.IsValid)
					{
						logger.LogInformation("Stopping existing service...");
						service.Stop();

						WindowsServiceStatus status = service.WaitForStatusChange(WindowsServiceStatus.Stopping, TimeSpan.FromSeconds(30.0));
						if (status != WindowsServiceStatus.Stopped)
						{
							logger.LogError("Unable to stop service (status = {Status})", status);
							return Task.FromResult(1);
						}

						logger.LogInformation("Deleting service...");
						service.Delete();
					}
					else
					{
						logger.LogInformation("Unable to find service {ServiceName}", ServiceName);
					}
				}
			}
			return Task.FromResult(0);
		}
	}
}
