// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.ServiceProcess;
using System.Text;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Service
{
	/// <summary>
	/// Upgrades a running service to the current application
	/// </summary>
	[Command("service", "upgrade", "Replaces a running service with the application in the current directory", Advertise = false)]
	class UpgradeCommand : Command
	{
		/// <summary>
		/// The process ID to replace (the old but currently running agent process)
		/// </summary>
		[CommandLine("-ProcessId=", Required = true)]
		int ProcessId { get; set; } = -1;

		/// <summary>
		/// The target directory to install to
		/// </summary>
		[CommandLine("-TargetDir=", Required = true)]
		DirectoryReference TargetDir { get; set; } = null!;

		/// <summary>
		/// Arguments to forward to the target executable
		/// </summary>
		[CommandLine("-Arguments=", Required = true)]
		string Arguments { get; set; } = null!;

		/// <summary>
		/// Upgrades the application to a new version
		/// </summary>
		/// <param name="logger">The log output device</param>
		/// <returns>True if the upgrade succeeded</returns>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			// Stop the other process
			logger.LogInformation("Attempting to perform upgrade on process {ProcessId}", ProcessId);
			using (Process otherProcess = Process.GetProcessById(ProcessId))
			{
				// Get the directory containing the target application
				DirectoryInfo targetDir = new DirectoryInfo(TargetDir.FullName);
				HashSet<string> targetFiles = new HashSet<string>(targetDir.EnumerateFiles("*", SearchOption.AllDirectories).Select(x => x.FullName), StringComparer.OrdinalIgnoreCase);

				// Find all the source files
				DirectoryInfo sourceDir = new(AppContext.BaseDirectory);
				HashSet<string> sourceFiles = new HashSet<string>(sourceDir.EnumerateFiles("*", SearchOption.AllDirectories).Select(x => x.FullName), StringComparer.OrdinalIgnoreCase);

				// Exclude all the source files from the list of target files, since we may be in a subdirectory
				targetFiles.ExceptWith(sourceFiles);

				// Ignore any files that are in the saved directory
				string sourceDataDir = Path.Combine(sourceDir.FullName, "Saved") + Path.DirectorySeparatorChar;
				sourceFiles.RemoveWhere(x => x.StartsWith(sourceDataDir, StringComparison.OrdinalIgnoreCase));

				string targetDataDir = Path.Combine(targetDir.FullName, "Saved") + Path.DirectorySeparatorChar;
				targetFiles.RemoveWhere(x => x.StartsWith(targetDataDir, StringComparison.OrdinalIgnoreCase));

				// Copy all the files into the target directory
				List<Tuple<string, string>> renameFiles = new List<Tuple<string, string>>();
				foreach (string sourceFile in sourceFiles)
				{
					if (!sourceFile.StartsWith(sourceDir.FullName, StringComparison.OrdinalIgnoreCase))
					{
						throw new InvalidDataException($"Expected {sourceFile} to be under {sourceDir.FullName}");
					}

					string targetFile = Path.Combine(targetDir.FullName, sourceFile.Substring(sourceDir.FullName.Length));
					Directory.CreateDirectory(Path.GetDirectoryName(targetFile)!);

					string targetFileBeforeRename = targetFile + ".new";
					logger.LogDebug("Copying {SourceFile} to {TargetFileBeforeRename}", sourceFile, targetFileBeforeRename);
					File.Copy(sourceFile, targetFileBeforeRename, true);

					renameFiles.Add(Tuple.Create(targetFileBeforeRename, targetFile));
					targetFiles.Remove(targetFileBeforeRename);
				}

				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					UpgradeWindowsService(logger, otherProcess, targetFiles, renameFiles);
				}
				else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
				{
					UpgradeMacService(logger, otherProcess, targetFiles, renameFiles);
				}
				else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
				{
					UpgradeLinuxService(logger, otherProcess, targetFiles, renameFiles);
				}
				else
				{
					logger.LogError("Agent is not running a platform that supports upgrades. Platform: {Platform}", RuntimeInformation.OSDescription);
					return Task.FromResult(-1);
				}
			}
			logger.LogInformation("Upgrade complete");
			return Task.FromResult(0);
		}

		private static readonly int[] s_retryDelaysMs = { 500, 1000, 2000, 3000, 5000 };

		private static void UpgradeFilesInPlace(ILogger logger, HashSet<string> targetFiles, List<Tuple<string, string>> moveFiles)
		{
			// Remove all the target files
			foreach (string targetFile in targetFiles)
			{
				DeleteFileWithRetry(logger, targetFile, s_retryDelaysMs);
			}

			// Move all the new files into place
			foreach (Tuple<string, string> pair in moveFiles)
			{
				MoveFileWithRetry(logger, pair.Item1, pair.Item2, s_retryDelaysMs);
			}
		}

		internal static void DeleteFileWithRetry(ILogger logger, string filePath, int[] retryDelaysMs)
		{
			for (int attempt = 0; attempt <= retryDelaysMs.Length; attempt++)
			{
				try
				{
					if (!File.Exists(filePath))
					{
						return;
					}
					logger.LogDebug("Deleting {File} (attempt {Attempt})", filePath, attempt + 1);
					FileUtils.ForceDeleteFile(filePath);
					return;
				}
				catch (Exception ex) when (attempt < retryDelaysMs.Length)
				{
					logger.LogWarning(ex, "Failed to delete {File}, retrying in {Delay} ms... ({Attempt}/{Max})",
						filePath, retryDelaysMs[attempt], attempt + 1, retryDelaysMs.Length + 1);
					Thread.Sleep(retryDelaysMs[attempt]);
				}
			}
		}

		internal static void MoveFileWithRetry(ILogger logger, string sourceFile, string targetFile, int[] retryDelaysMs)
		{
			for (int attempt = 0; attempt <= retryDelaysMs.Length; attempt++)
			{
				try
				{
					// Idempotent check: if source is gone and target exists, move was already done
					if (!File.Exists(sourceFile) && File.Exists(targetFile))
					{
						return;
					}
					logger.LogDebug("Moving {SourceFile} to {TargetFile} (attempt {Attempt})", sourceFile, targetFile, attempt + 1);
					FileUtils.ForceMoveFile(sourceFile, targetFile);
					return;
				}
				catch (Exception ex) when (attempt < retryDelaysMs.Length)
				{
					logger.LogWarning(ex, "Failed to move {SourceFile} to {TargetFile}, retrying in {Delay} ms... ({Attempt}/{Max})",
						sourceFile, targetFile, retryDelaysMs[attempt], attempt + 1, retryDelaysMs.Length + 1);
					Thread.Sleep(retryDelaysMs[attempt]);
				}
			}
		}

		static void UpgradeMacService(ILogger logger, Process otherProcess, HashSet<string> targetFiles, List<Tuple<string, string>> renameFiles)
		{
			UpgradeFilesInPlace(logger, targetFiles, renameFiles);
			logger.LogDebug("Upgrade completed, restarting...");
			otherProcess.Kill();
			// Assume agent process is auto-restarted by OS or external daemon process handler (such as launchd)
		}

		static void UpgradeLinuxService(ILogger logger, Process otherProcess, HashSet<string> targetFiles, List<Tuple<string, string>> renameFiles)
		{
			UpgradeFilesInPlace(logger, targetFiles, renameFiles);
			logger.LogDebug("Upgrade completed, restarting...");
			otherProcess.Kill();
			// Assume agent process is auto-restarted by OS or external daemon process handler (such as systemd)
		}

		[SupportedOSPlatform("windows")]
		void UpgradeWindowsService(ILogger logger, Process otherProcess, HashSet<string> targetFiles, List<Tuple<string, string>> renameFiles)
		{
			// Try to get the service associated with the passed-in process id
			using ServiceController? service = NativeWindowsServiceUtils.GetServiceForProcess(ProcessId);

			// Stop the process
			if (service == null)
			{
				logger.LogInformation("Terminating running agent process...");
				otherProcess.Kill();
			}
			else
			{
				logger.LogInformation("Stopping service...");
				service.Stop();
				service.WaitForStatus(ServiceControllerStatus.Stopped, TimeSpan.FromSeconds(30));
			}
			otherProcess.WaitForExit();

			// Additional buffer for Windows to release all file handles (antivirus, indexers, etc.)
			Thread.Sleep(5000);

			UpgradeFilesInPlace(logger, targetFiles, renameFiles);

			// Run the new application
			if (service == null)
			{
				string executable;
				StringBuilder arguments = new();
				if (AgentApp.IsSelfContained)
				{
					if (Environment.ProcessPath == null)
					{
						throw new Exception("Unable to detect current process path");
					}

					executable = Path.Combine(TargetDir.FullName, Path.GetFileName(Environment.ProcessPath));
					if (!File.Exists(executable))
					{
						throw new Exception($"{executable} not found. Is the new agent software packaged as self-contained?");
					}
				}
				else
				{
#pragma warning disable IL3000 // Avoid accessing Assembly file path when publishing as a single file
					executable = "dotnet";
					string assemblyFileName = Path.Combine(TargetDir.FullName, Path.GetFileName(Assembly.GetExecutingAssembly().Location));
					arguments.AppendArgument(assemblyFileName);
#pragma warning restore IL3000 // Avoid accessing Assembly file path when publishing as a single file					
				}
				arguments.Append(' ');
				arguments.Append(Arguments);

				logger.LogInformation("Launching: {Executable} {Arguments}", executable, arguments.ToString());
				using Process newProcess = Process.Start(executable, arguments.ToString());
			}
			else
			{
				// Start the service again
				logger.LogInformation("Restarting service...");
				service.Start();
			}
		}
	}
}
