// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using System.Runtime.Versioning;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using JsonObject = System.Text.Json.Nodes.JsonObject;

namespace HordeAgent.Commands.Setup;

/// <summary>
/// Exception for any installer related errors
/// </summary>
/// <param name="message">Error message</param>
/// <param name="exitCode">Optional error code to return</param>
/// <param name="innerException">Optional inner exception</param>
public class InstallException(string message, int exitCode = 1, Exception? innerException = null) : Exception(message, innerException)
{
	/// <summary>
	/// Exit code
	/// </summary>
	public int ExitCode { get; } = exitCode;
}

/// <summary>
/// Install the agent as a Windows application and service
/// </summary>
[Command("setup", "install", "Installs the agent (on Windows)")]
class SetupInstallCommand : Command
{
	const string HordeAgentDll = "HordeAgent.dll";
	const string HordeAgentExe = "HordeAgent.exe";
	const string HordeAgentIco = "HordeAgent.ico";
	const string UninstallKeyPath = @"SOFTWARE\Microsoft\Windows\CurrentVersion\Uninstall";
	const string UninstallAppName = "Unreal Horde Agent";
	const string PortRange = "7000-7010";
	internal const string FirewallRuleName = "Horde Agent";
	
	[CommandLine("-ShowStackTrace=")]
	[Description("Whether to show full stack trace on errors (true/false)")]
	public string? ShowStackTrace { get; set; } = "false";
	
	[CommandLine("-RegisterForUninstall=")]
	[Description("Whether to register the app for uninstall in Windows (true/false)")]
	public string? RegisterForUninstall { get; set; } = "true";
	
	/// <summary>
	/// Install the 
	/// </summary>
	/// <param name="logger">Logger to use</param>
	/// <returns>Exit code</returns>
	public override Task<int> ExecuteAsync(ILogger logger)
	{
		if (!OperatingSystem.IsWindows())
		{
			logger.LogError("This command requires Windows");
			return Task.FromResult(1);
		}
		
		bool showStackTrace = ShowStackTrace != null && ShowStackTrace.Equals("true", StringComparison.OrdinalIgnoreCase);
		bool registerForUninstall = RegisterForUninstall != null && RegisterForUninstall.Equals("true", StringComparison.OrdinalIgnoreCase);
		
		try
		{
			Install(logger, new InstallParams(RegisterForUninstall: registerForUninstall));
		}
		catch (InstallException e)
		{
			logger.LogError("Install failed: {Reason}", e.Message);
			if (showStackTrace)
			{
				throw;
			}
			
			return Task.FromResult(e.ExitCode);
		}
		
		return Task.FromResult(0);
	}
	
	/// <summary>
	/// Parameters for installing agent on Windows 
	/// </summary>
	/// <param name="SourceDir">Directory containing files to install (HordeAgent.dll etc)</param>
	/// <param name="DestDir">Installation directory (unless specified, defaults to a directory under Program Files)</param>
	/// <param name="ConfigDir">Config directory</param>
	/// <param name="WorkingDir">Sandbox directory for agent (where VCS-synced projects are stored)</param>
	/// <param name="RegisterForUninstall">Whether to register the app for uninstall in Windows</param>
	public record InstallParams(
		string? SourceDir = null,
		string? DestDir = null,
		string? ConfigDir = null,
		string? WorkingDir = null,
		bool RegisterForUninstall = true
	);
	
	/// <summary>
	/// Install agent on Windows 
	/// </summary>
	/// <param name="logger">Logger</param>
	/// <param name="installParams">Parameters</param>
	[SupportedOSPlatform("windows")]
	public static int Install(ILogger logger, InstallParams installParams)
	{
		if (!UserSecurityUtils.IsAdministrator())
		{
			throw new InstallException("Administrator access is required for installing");
		}
		
		DirectoryReference sourceDir = new(installParams.SourceDir ?? AppContext.BaseDirectory);
		DirectoryReference destinationDir = new(installParams.DestDir ?? GetDefaultInstallDir());
		DirectoryReference configDir = new(installParams.ConfigDir ?? Path.Join(Environment.GetFolderPath(Environment.SpecialFolder.CommonApplicationData), "Epic", "Horde", "Agent"));
		DirectoryReference workingDir = new(installParams.WorkingDir ?? Path.Join("c:", "HordeAgent", "Sandbox"));
		logger.LogInformation(" Source dir {SourceDir}", sourceDir);
		logger.LogInformation("   Dest dir {DestinationDir}", destinationDir);
		logger.LogInformation(" Config dir {ConfigDir}", configDir);
		logger.LogInformation("Working dir {WorkingDir}", workingDir);
		
		if (DirectoryReference.Exists(destinationDir) && Directory.GetFiles(destinationDir.FullName).Length > 0)
		{
			throw new InstallException($"Installation destination directory {destinationDir} already exists. Please uninstall any current agent!");
		}
		
		DirectoryReference.CreateDirectory(destinationDir);
		if (!ContainsHordeAgentExecutable(sourceDir))
		{
			throw new InstallException($"Source directory is missing either {HordeAgentDll} or {HordeAgentExe}");
		}
		
		logger.LogInformation("Copying files...");
		try
		{
			DirectoryReference.Copy(sourceDir, destinationDir);
		}
		catch (Exception e)
		{
			throw new InstallException($"Failed copying files to destination directory {destinationDir}", 1, e);
		}
		
		// Remove dir containing agent.json config template as it's installed under ProgramData instead
		DirectoryReference.Delete(DirectoryReference.Combine(destinationDir, "Defaults"), true);
		
		FileReference agentJson = FileReference.Combine(configDir, "agent.json");
		if (!FileReference.Exists(agentJson))
		{
			logger.LogInformation("Copied default template agent.json to {AgentJsonPath}", agentJson);
			FileReference.Copy(FileReference.Combine(sourceDir, "Defaults", "agent.json"), agentJson);
		}
		
		PatchAgentJsonFile(agentJson, "WorkingDir", workingDir.FullName);
		DirectoryReference.CreateDirectory(workingDir);
		
		logger.LogInformation("Configuring Windows firewall...");
		AddFirewallPortRule(FirewallRuleName, PortRange);
		
		logger.LogInformation("Installing Horde Agent as a Windows service...");
		(string executable, string args) = ResolveCommandLine(destinationDir.FullName, "service install");
		if (RunProcessSilently(executable, args) != 0)
		{
			throw new InstallException("Failed installing Horde Agent as a Windows service");
		}
		
		if (installParams.RegisterForUninstall)
		{
			(executable, args) = ResolveCommandLine(destinationDir.FullName, "setup uninstall");
			RegisterAppForUninstall(new AppInfo(
				DisplayName: "Unreal Horde Agent",
				InstallLocation: destinationDir,
				Version: AgentApp.Version,
				Publisher: "Epic Games",
				UninstallCommand: $"\"{executable}\" {args}",
				IconPath: FileReference.Combine(destinationDir, HordeAgentIco),
				EstimatedSizeInKb: DirectoryReference.GetTotalSize(destinationDir) / 1024
			));
		}
		
		return 0;
	}
	
	/// <summary>
	/// Resolves path to executable and arguments to send for invoking the Horde agent executable
	/// If agent is self-contained, the binary will be a standalone executable.
	/// Otherwise, 'dotnet' executable will be used with the .dll and additional arguments appended
	/// </summary>
	/// <param name="hordeAgentDir">Path to directory where Horde Agent is installed</param>
	/// <param name="args">Additional arguments</param>
	/// <returns>Executable and arguments</returns>
	private static (string executable, string args) ResolveCommandLine(string hordeAgentDir, string args)
	{
		return AgentApp.IsSelfContained
			? (Path.Join(hordeAgentDir, HordeAgentExe), args)
			: ("dotnet", $"\"{Path.Join(hordeAgentDir, HordeAgentDll)}\" {args}");
	}
	
	private static bool PatchAgentJsonFile(FileReference jsonFilePath, string key, string value)
	{
		// Patching like this will change formatting and comments of the JSON file. Handling that in a good way is tricky.
		string rawJson = File.ReadAllText(jsonFilePath.FullName);
		JsonNode? jsonNode = JsonNode.Parse(rawJson, new JsonNodeOptions { PropertyNameCaseInsensitive = true },
			new JsonDocumentOptions() { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip });
		if (jsonNode == null)
		{
			return false;
		}
		
		JsonNode hordeNode = jsonNode["Horde"] ?? new JsonObject();
		hordeNode[key] = value;
		File.WriteAllText(jsonFilePath.FullName, jsonNode.ToJsonString(new JsonSerializerOptions() { WriteIndented = true }));
		return true;
	}
	
	internal static string GetDefaultInstallDir()
	{
		return Path.Join(Environment.GetFolderPath(Environment.SpecialFolder.ProgramFiles), "Epic Games", "Horde", "Agent");
	}
	
	internal static void AddFirewallPortRule(string name, string portRange)
	{
		RemoveFirewallPortRule(name);
		string arguments = $"advfirewall firewall add rule name=\"{name}\" dir=in action=allow protocol=TCP localport={portRange}";
		if (RunProcessSilently("netsh", arguments) != 0)
		{
			throw new InstallException($"Failed to add firewall rule '{name}'");
		}
	}
	
	internal static bool ContainsHordeAgentExecutable(DirectoryReference dir)
	{
		bool containsDll = FileReference.Exists(FileReference.Combine(dir, HordeAgentDll));
		bool containsExe = FileReference.Exists(FileReference.Combine(dir, HordeAgentExe));
		return containsDll || containsExe;
	}
	
	internal static void RemoveFirewallPortRule(string name)
	{
		RunProcessSilently("netsh", $"advfirewall firewall delete rule name=\"{name}\"");
	}
	
	internal static int RunProcessSilently(string fileName, string arguments)
	{
		ProcessStartInfo startInfo = new() { FileName = fileName, Arguments = arguments, UseShellExecute = false, CreateNoWindow = true };
		try
		{
			using Process? process = Process.Start(startInfo);
			process?.WaitForExit(10000);
			return process?.ExitCode ?? 100;
		}
		catch (Exception)
		{
			return 100;
		}
	}
	
	internal record AppInfo(
		string DisplayName,
		DirectoryReference InstallLocation,
		string Version,
		string Publisher,
		string UninstallCommand,
		FileReference? IconPath,
		long EstimatedSizeInKb);
	
	[SupportedOSPlatform("windows")]
	public static void RegisterAppForUninstall(AppInfo appInfo)
	{
		string regKeyPath = $"{UninstallKeyPath}\\{UninstallAppName}";
		using RegistryKey? appKey = Registry.LocalMachine.CreateSubKey(regKeyPath, true);
		if (appKey == null)
		{
			throw new InstallException($"Unable to create registry key {regKeyPath}");
		}
		
		appKey.SetValue("DisplayName", appInfo.DisplayName);
		appKey.SetValue("DisplayVersion", appInfo.Version);
		appKey.SetValue("InstallLocation", appInfo.InstallLocation.FullName);
		appKey.SetValue("InstallDate", DateTime.Now.ToString("yyyyMMdd"));
		appKey.SetValue("Publisher", appInfo.Publisher);
		appKey.SetValue("UninstallString", appInfo.UninstallCommand);
		appKey.SetValue("QuietUninstallString", $"{appInfo.UninstallCommand}");
		appKey.SetValue("EstimatedSize", appInfo.EstimatedSizeInKb, RegistryValueKind.DWord);
		appKey.SetValue("NoModify", 1, RegistryValueKind.DWord);
		appKey.SetValue("NoRepair", 1, RegistryValueKind.DWord);
		
		if (appInfo.IconPath != null)
		{
			appKey.SetValue("DisplayIcon", appInfo.IconPath.FullName);
		}
	}
	
	[SupportedOSPlatform("windows")]
	public static bool UnregisterAppForUninstall(string appName = UninstallAppName)
	{
		using RegistryKey? key = Registry.LocalMachine.OpenSubKey(UninstallKeyPath, true);
		if (key == null)
		{
			return false;
		}
		
		key.DeleteSubKeyTree(appName, false);
		return true;
	}
	
	[SupportedOSPlatform("windows")]
	public static DirectoryReference GetUninstallLocation(bool useRegistryPath = true)
	{
		if (!useRegistryPath)
		{
			return new DirectoryReference(GetDefaultInstallDir());
		}
		
		const string RegKeyPath = $"{UninstallKeyPath}\\{UninstallAppName}";
		using RegistryKey? key = Registry.LocalMachine.OpenSubKey(RegKeyPath, true);
		if (key?.GetValue("InstallLocation") is string uninstallString)
		{
			return new DirectoryReference(uninstallString);
		}
		
		throw new InstallException($"Unable to open registry key {RegKeyPath}");
	}
	
	/// <summary>
	/// Runs a Windows batch script that will delete the given directory
	/// The script will be executed in the background and wait for a process to exit before attempting to delete.
	/// This allows the uninstaller (HordeAgent.dll or similar) to remove itself once the main duties are completed.
	/// </summary>
	/// <param name="deleteDir">Directory to delete</param>
	/// <param name="waitPid">Process ID to wait for before attempting a delete</param>
	/// <exception cref="InstallException">Thrown when script can't be created or executed</exception>
	public static void RunSelfDeleteScript(DirectoryReference deleteDir, int waitPid)
	{
		string script = $"""
		                 @echo off
		                 setlocal EnableDelayedExpansion
		                 
		                 :: Configuration
		                 set "dirPath={deleteDir.FullName}"
		                 set "maxWaitSeconds=60"
		                 set "maxRetries=3"
		                 set "retryDelaySeconds=2"
		                 set "currentPid={waitPid}"
		                 
		                 :: Wait for the process to exit
		                 set /A secsPassed=0
		                 :waitloop
		                 tasklist /FI "PID eq %currentPid%" 2>nul | find /I "%currentPid%" >nul
		                 if !ERRORLEVEL! EQU 0 (
		                     echo Process still running, checking again in one second...
		                     timeout /t 1 /nobreak >nul
		                     set /A secsPassed+=1
		                     if !secsPassed! LSS %maxWaitSeconds% goto :waitloop
		                     echo Warning: Process %currentPid% did not exit within %maxWaitSeconds% seconds.
		                 )
		                 
		                 :: Attempt to remove directory
		                 set /A retry=0
		                 :retryloop
		                 rd /s /q "%dirPath%" 2>nul
		                 if exist "%dirPath%" (
		                     set /A retry+=1
		                     if !retry! LSS %maxRetries% (
		                         echo Attempt !retry! failed. Retrying in %retryDelaySeconds% seconds...
		                         timeout /t %retryDelaySeconds% /nobreak >nul
		                         goto :retryloop
		                     ) else (
		                         echo Error: Failed to remove directory after %maxRetries% attempts.
		                         exit /b 1
		                     )
		                 ) else (
		                     echo Successfully removed directory.
		                 )
		                 
		                 :: Clean up the script itself
		                 rem del "%~f0"
		                 exit /b 0
		                 """;
		
		try
		{
			string batFilePath = Path.Combine(Path.GetTempPath(), "horde-agent-uninstall.bat");
			File.WriteAllText(batFilePath, script);
			Process.Start(new ProcessStartInfo
			{
				FileName = batFilePath,
				CreateNoWindow = true,
				UseShellExecute = false,
				WindowStyle = ProcessWindowStyle.Hidden,
			});
		}
		catch (System.Exception ex)
		{
			throw new InstallException("Unable to create and run self-delete script", 1, ex);
		}      
	}
}