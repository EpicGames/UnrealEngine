// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Runtime.Versioning;
using EpicGames.Core;
using HordeAgent.Commands.Service;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Setup;

/// <summary>
/// Uninstall the agent from Windows
/// </summary>
[Command("setup", "uninstall", "Uninstalls the agent (on Windows)")]
class SetupUninstallCommand : Command
{
	[CommandLine("-ShowStackTrace=")]
	[Description("Whether to show full stack trace on errors (true/false)")]
	public string? ShowStackTrace { get; set; } = "false";
	
	[CommandLine("-UseRegistryPath=")]
	[Description("Use uninstall string as specified in registry")]
	public string? UseRegistryPath { get; set; } = "false";
	
	/// <summary>
	/// Uninstall the agent from Windows 
	/// </summary>
	/// <param name="logger">Logger to use</param>
	/// <returns>Exit code</returns>
	public override Task<int> ExecuteAsync(ILogger logger)
	{
		if (!OperatingSystem.IsWindows())
		{
			logger.LogError("This command requires Windows");
			return Task.FromResult(1);
		}
		
		bool useRegistryPath = UseRegistryPath != null && UseRegistryPath.Equals("true", StringComparison.OrdinalIgnoreCase);
		bool showStackTrace = ShowStackTrace != null && ShowStackTrace.Equals("true", StringComparison.OrdinalIgnoreCase);
		
		try
		{
			Uninstall(useRegistryPath, logger);
		}
		catch (InstallException e)
		{
			logger.LogError("Uninstall failed: {Reason}", e.Message);
			if (showStackTrace)
			{
				throw;
			}
			
			return Task.FromResult(e.ExitCode);
		}
		
		return Task.FromResult(0);
	}
	
	[SupportedOSPlatform("windows")]
	private static void Uninstall(bool useRegistryPath, ILogger logger)
	{
		if (!UserSecurityUtils.IsAdministrator())
		{
			throw new InstallException("Administrator access is required for uninstalling");
		}
		
		DirectoryReference installDir = SetupInstallCommand.GetUninstallLocation(useRegistryPath);
		if (!SetupInstallCommand.ContainsHordeAgentExecutable(installDir))
		{
			throw new InstallException($"Directory to uninstall does not contain Horde agent executable");
		}
		
		logger.LogInformation("Uninstalling Horde agent...");
		InstallServiceCommand.StopAndDeleteWindowsService(logger);
		SetupInstallCommand.RemoveFirewallPortRule(SetupInstallCommand.FirewallRuleName);
		SetupInstallCommand.UnregisterAppForUninstall();
		
		// Remove directory containing agent binaries
		logger.LogInformation("Deleting dir {AgentDir}", installDir.FullName);
		SetupInstallCommand.RunSelfDeleteScript(installDir, Environment.ProcessId);
	}
}