// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeAgent.Services;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Utilities
{
	/// <summary>
	/// Shows capabilities of this agent
	/// </summary>
	[Command("caps", "Lists detected capabilities of this agent")]
	class CapsCommand : Command
	{
		readonly CapabilitiesService _capabilitiesService;

		/// <summary>
		/// Constructor
		/// </summary>
		public CapsCommand(CapabilitiesService capabilitiesService)
		{
			_capabilitiesService = capabilitiesService;
		}

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			RpcAgentCapabilities capabilities = await _capabilitiesService.GetCapabilitiesAsync(null);

			logger.LogInformation("Properties:");
			foreach (string property in capabilities.Properties)
			{
				logger.LogInformation("  {Property}", property);
			}

			logger.LogInformation("");

			logger.LogInformation("Resources:");
			foreach ((string name, int value) in capabilities.Resources)
			{
				logger.LogInformation("  {Name}={Value}", name, value);
			}

			return 0;
		}
	}
}
