// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Cryptography.X509Certificates;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Certs
{
	/// <summary>
	/// Creates a certificate that can be used for server/agent SSL connections
	/// </summary>
	[Command("createcert", "Creates a self-signed certificate that can be used for server/agent gRPC connections", Advertise = false)]
	class CreateCertCommand : Command
	{
		[CommandLine("-Server=")]
		public string? Server { get; set; } = null;

		[CommandLine("-DnsName=")]
		public string? DnsName { get; set; }

		[CommandLine("-PrivateCert=")]
		public string? PrivateCertFile { get; set; }

		[CommandLine("-Environment=")]
		public string? Environment { get; set; } = "Development";

		/// <summary>
		/// Main entry point for this command
		/// </summary>
		/// <param name="logger"></param>
		/// <returns>Async task</returns>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (DnsName == null)
			{
				IConfigurationRoot config = new ConfigurationBuilder()
					.SetBasePath(Directory.GetCurrentDirectory())
					.AddJsonFile("appsettings.json", true)
					.AddJsonFile($"appsettings.{Environment}.json", true)
					.Build();

				AgentSettings settings = new AgentSettings();
				config.GetSection("Horde").Bind(settings);

				if (Server != null)
				{
					settings.Server = Server;
				}

				DnsName = settings.GetCurrentServerProfile().Url.ToString();
				DnsName = Regex.Replace(DnsName, @"^[a-zA-Z]+://", "");
				DnsName = Regex.Replace(DnsName, "/.*$", "");
			}

			if (PrivateCertFile == null)
			{
				FileReference solutionFile = FileReference.Combine(new DirectoryReference(AppContext.BaseDirectory), "..", "..", "..", "..", "Horde.sln");
				if (!FileReference.Exists(solutionFile))
				{
					logger.LogError("The -PrivateCertFile=... arguments must be specified when running outside the default build directory");
					return 1;
				}
				PrivateCertFile = FileReference.Combine(solutionFile.Directory, "HordeServer", "Certs", $"ServerToAgent-{Environment}.pfx").FullName;
			}

			logger.LogInformation("Creating certificate for {DnsName}", DnsName);

			byte[] privateCertData = CertificateUtils.CreateSelfSignedCert(DnsName, "Horde Server");

			logger.LogInformation("Writing private cert: {PrivateCert}", new FileReference(PrivateCertFile).FullName);
			await File.WriteAllBytesAsync(PrivateCertFile, privateCertData);

			using X509Certificate2 certificate = X509CertificateLoader.LoadPkcs12(privateCertData, null);
			logger.LogInformation("Certificate thumbprint is {Thumbprint}", certificate.Thumbprint);
			logger.LogInformation("Add this thumbprint to list of trusted servers in appsettings.json to trust this server.");

			return 0;
		}
	}
}
