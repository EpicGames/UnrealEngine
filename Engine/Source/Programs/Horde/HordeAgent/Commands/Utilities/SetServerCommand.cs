// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Text.Json.Nodes;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Utilities
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Shows capabilities of this agent
	/// </summary>
	[Command("setserver", "Configures a server for this agent")]
	class SetServerCommand : Command
	{
		[CommandLine("-Name=")]
		[Description("Name of the server to use")]
		public string Name { get; set; } = "Default";

		[CommandLine("-Url=", Required = true)]
		[Description("URL of the server")]
		public string? Url { get; set; }

		[CommandLine("-Env=")]
		[Description("Environment to configure for (Prod/Dev).")]
		public string? Environment { get; set; }

		[CommandLine("-Token=")]
		[Description("Token to use for initial connection")]
		public string? Token { get; set; }

		[CommandLine("-Thumbprint=")]
		[Description("Optional thumbprint of the server's SSL certificate. Will bypass the CA if matched. Useful for deploying self-signed certs.")]
		public string? Thumbprint { get; set; }

		[CommandLine("-Default")]
		[Description("Makes this server the default")]
		public bool Default { get; set; }

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			// Update the agent to recognize the server certificate, and give it a custom token for being able to connect
			FileReference agentConfigFile = FileReference.Combine(AgentApp.AppDir, "appsettings.json");
			JsonObject agentConfig = await JsonConfig.ReadAsync(agentConfigFile);

			JsonObject hordeConfig = JsonConfig.FindOrAddNode(agentConfig, "Horde", () => new JsonObject());
			if (Default)
			{
				hordeConfig[nameof(AgentSettings.Server)] = Name;
			}

			JsonArray serverProfiles = JsonConfig.FindOrAddNode(hordeConfig, nameof(AgentSettings.ServerProfiles), () => new JsonArray());

			JsonObject serverProfile = JsonConfig.FindOrAddElementByKey(serverProfiles, nameof(ServerProfile.Name), Name);
			serverProfile[nameof(ServerProfile.Name)] = Name;
			serverProfile[nameof(ServerProfile.Url)] = Url;

			if (!String.IsNullOrEmpty(Environment))
			{
				serverProfile[nameof(ServerProfile.Environment)] = Environment;
			}
			if (!String.IsNullOrEmpty(Token))
			{
				serverProfile[nameof(ServerProfile.Token)] = Token;
			}
			if (!String.IsNullOrEmpty(Thumbprint))
			{
				serverProfile[nameof(ServerProfile.Thumbprint)] = new JsonArray(JsonValue.Create(Thumbprint));
			}

			await JsonConfig.WriteAsync(agentConfigFile, agentConfig);
			return 0;
		}
	}
}
