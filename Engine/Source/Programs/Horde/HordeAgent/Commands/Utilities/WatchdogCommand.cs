// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Commands.Utilities;

/// <summary>
/// Test runtime watchdog
/// </summary>
[Command("watchdog", "Test runtime watchdog")]
class RuntimeWatchdogCommand : Command
{
	/// <summary>
	/// Constructor
	/// </summary>
	public RuntimeWatchdogCommand()
	{
	}

	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});
		ILogger<RuntimeWatchdog> rwLogger = loggerFactory.CreateLogger<RuntimeWatchdog>();
		using RuntimeWatchdog watchdog = new (rwLogger);
		await watchdog.StartAsync(CancellationToken.None);

		watchdog.ArtificialThreadDelay = TimeSpan.FromSeconds(10);
		StarveWorkerThreads();
		await Task.Delay(120 * 1000);

		return 0;
	}
	
	internal static void StarveWorkerThreads()
	{
		if (!ThreadPool.SetMinThreads(1, 1))
		{
			throw new Exception("Failed to set min threads");
		}
		if (!ThreadPool.SetMaxThreads(3, 3))
		{
			throw new Exception("Failed to set max threads");
		}

		ThreadPool.GetMaxThreads(out int maxWorker, out int _);
		for (int i = 0; i < maxWorker + 5; i++)
		{
			ThreadPool.QueueUserWorkItem(_ =>
			{
				LogThreadPoolState();
				Thread ct = Thread.CurrentThread;
				Console.WriteLine($"Simulating blocking work in thread {ct.ManagedThreadId} {ct.Name}");
				Thread.Sleep(TimeSpan.FromMinutes(5));
			});
		}
	}
	
	private static void LogThreadPoolState()
	{
		ThreadPool.GetMinThreads(out int workerMin, out int comPortMin);
		ThreadPool.GetMaxThreads(out int workerMax, out int comPortMax);
		ThreadPool.GetAvailableThreads(out int workerFree, out int comPortFree);
		StringBuilder sb = new();
		sb.Append("Thread ");
		sb.AppendFormat("Count={0} ", ThreadPool.ThreadCount);
		sb.AppendFormat("PendingItems={0} ", ThreadPool.PendingWorkItemCount);
		sb.AppendFormat("CompletedItems={0} | ", ThreadPool.CompletedWorkItemCount);
		sb.AppendFormat("Worker min={0} max={1} free={2} | ", workerMin, workerMax, workerFree);
		sb.AppendFormat("ComPort min={0} max={1} free={2} ", comPortMin, comPortMax, comPortFree);
		Console.WriteLine(sb.ToString());
	}
}
