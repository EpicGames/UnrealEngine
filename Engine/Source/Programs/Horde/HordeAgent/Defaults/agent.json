{
	// Settings for your agent deployment go here
}
