<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <RollForward>LatestMinor</RollForward>
    <ImplicitUsings>enable</ImplicitUsings>
    <ApplicationIcon>HordeAgent.ico</ApplicationIcon>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <AssemblyName>HordeAgent</AssemblyName>
    <PublishRepositoryUrl>true</PublishRepositoryUrl>
    <EmbedUntrackedSources>false</EmbedUntrackedSources>
    <NoWarn>$(NoWarn);NU3003</NoWarn>
  </PropertyGroup>

  <PropertyGroup Condition="'$(Configuration)|$(Platform)'=='Debug|AnyCPU'">
    <DocumentationFile>HordeAgent.xml</DocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <None Remove="HordeAgent.xml" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.Core" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.EC2" Version="3.7.306.6" />
    <PackageReference Include="AWSSDK.SimpleSystemsManagement" Version="3.7.301.8" />
    <PackageReference Include="Google.Protobuf" Version="3.25.8" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Meziantou.Analyzer" Version="2.0.220">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.Extensions.Caching.Memory" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting.Abstractions" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting.WindowsServices" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http.Polly" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Logging.Abstractions" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Options.DataAnnotations" Version="10.0.6" />
    <PackageReference Include="Microsoft.VisualStudio.Threading.Analyzers" Version="17.14.15">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="OpenTelemetry" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.Console" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.OpenTelemetryProtocol" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Extensions.Hosting" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.GrpcNetClient" Version="1.8.0-beta.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.Http" Version="1.8.1" />
    <PackageReference Include="Serilog" Version="3.1.1" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="Serilog.Extensions.Logging" Version="8.0.0" />
    <PackageReference Include="Serilog.Settings.Configuration" Version="8.0.4" />
    <PackageReference Include="Serilog.Sinks.Console" Version="5.0.1" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
    <PackageReference Include="System.Diagnostics.PerformanceCounter" Version="10.0.6" />
    <PackageReference Include="System.ServiceProcess.ServiceController" Version="10.0.6" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="appsettings.json" Condition="Exists('appsettings.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
    </Content>
    <Content Include="appsettings.Development.json" Condition="Exists('appsettings.Development.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
      <DependentUpon>appsettings.json</DependentUpon>
    </Content>
    <Content Include="appsettings.Local.json" Condition="Exists('appsettings.Local.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
      <DependentUpon>appsettings.json</DependentUpon>
    </Content>
    <Content Include="appsettings.Production.json" Condition="Exists('appsettings.Production.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
      <DependentUpon>appsettings.json</DependentUpon>
    </Content>
    <Content Include="appsettings.User.json" Condition="Exists('appsettings.User.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
      <DependentUpon>appsettings.json</DependentUpon>
    </Content>
    <Content Include="Defaults\agent.json">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
    </Content>
    <Content Include="HordeAgent.ico">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
    <Content Include="Toolbox.json">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <Content Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\*.proto">
      <Link>Protos\%(FileName)%(Extension)</Link>
    </Content>
    <Content Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\common\**\*.proto">
      <Link>Protos\horde\common\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
    <Compile Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\agent_capabilities.proto.cs">
      <Link>Protos\horde\messages\agent_capabilities.proto.cs</Link>
    </Compile>
    <Compile Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\lease.proto.cs">
      <Link>Protos\horde\messages\lease.proto.cs</Link>
    </Compile>
    <Content Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\**\*.proto">
      <Link>Protos\horde\messages\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
    <Content Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\**\*.proto">
      <Link>Protos\horde\tasks\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
<!--    
    <Compile Include="..\HordeServer\Protos\horde\**\*.cs">
      <Link>Protos\horde\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Compile>
    -->
    <Compile Include="$(IntermediateOutputPath)VersionInfo.cs">
      <Link>Properties/VersionInfo.cs</Link>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\horde_rpc.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" GrpcServices="Client" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\horde_rpc_messages.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\enrollment_rpc.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" GrpcServices="Client" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\enrollment_rpc_messages.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\common\agent_status.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\common\lease_outcome.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\common\lease_state.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\agent_capabilities.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\agent_requirements.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\lease.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\aws_lambda.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\compute_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\restart_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\shutdown_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\test_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\upgrade_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />

    <Protobuf Include="..\Plugins\Build\HordeServer.Build\Protos\horde\job_rpc_messages.proto" ProtoRoot="..\Plugins\Build\HordeServer.Build\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Build\HordeServer.Build\Protos\horde\common\label_change.proto" ProtoRoot="..\Plugins\Build\HordeServer.Build\Protos" Access="Public" AdditionalImportDirs="..\Plugins\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\messages\agent_workspace.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\conform_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
    <Protobuf Include="..\Plugins\Compute\HordeServer.Compute\Protos\horde\tasks\job_task.proto" ProtoRoot="..\Plugins\Compute\HordeServer.Compute\Protos" Access="Public" />
  </ItemGroup>

  <ItemGroup>
    <Compile Update="Properties\Resources.Designer.cs">
      <DesignTime>True</DesignTime>
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <EmbeddedResource Update="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
    </EmbeddedResource>
  </ItemGroup>

  <ItemGroup>
    <Content Update="appsettings.Local.json" CopyToPublishDirectory="Never" />
  </ItemGroup>

  <Target Name="DeleteUnusedFiles-Win64" AfterTargets="Publish" Condition="'$(Configuration)' == 'Release' AND '$(RuntimeIdentifier)' == 'win-x64'">
    <ItemGroup>
      <FilesToDelete Include="$(PublishDir)**/*.dylib" />
      <FilesToDelete Include="$(PublishDir)**/*.so.9" />
      <FilesToDelete Include="$(PublishDir)**/*.so" />
      <FilesToDelete Include="$(PublishDir)**/*.xml" />
      <FilesToDelete Include="$(PublishDir)**/oo2core*win32.dll" />
      <FilesToDelete Include="$(PublishDir)**/oo2core*winuwp*.dll" />
    </ItemGroup>
    <Delete Files="@(FilesToDelete)">
      <Output TaskParameter="DeletedFiles" ItemName="DeletedList" />
    </Delete>
    <Message Text="Deleted files:%0A@(DeletedList->'  %(Identity)','%0A')" Importance="high" />
  </Target>

  <Target Name="PublishSymbols" AfterTargets="AfterBuild" Condition="'$(FullSymbolsDir)' != ''">
    <ItemGroup>
      <OutputFiles Include="$(OutDir)**" />
      <BinaryFilesBase Include="$(OutDir)\*.dll" />
      <BinaryFilesBase Include="$(OutDir)\*.pdb" />
      <BinaryFiles Include="@(BinaryFilesBase)" />
    </ItemGroup>

    <MakeDir Directories="$(FullSymbolsDir)" />
    <Copy SourceFiles="@(Files)" DestinationFolder="$(FullSymbolsDir)" />
  </Target>

  <!-- Encode version in a source file as inferring version from assemblies is unreliable on platform-specific binaries -->
  <Target Name="PrepareVersionInfo" BeforeTargets="BeforeBuild">
    <PropertyGroup>
      <TemplateFile>Utility/VersionInfo.cs.template</TemplateFile>
      <OutputFile>$(IntermediateOutputPath)/VersionInfo.cs</OutputFile>
      <!-- Self-contained versions should set this to match the tool ID for that OS/arch -->
      <ToolId Condition="'$(ToolId)' == ''">unknown</ToolId>
    </PropertyGroup>
    <ReadLinesFromFile File="$(TemplateFile)">
      <Output TaskParameter="Lines" ItemName="Lines" />
    </ReadLinesFromFile>
    <WriteLinesToFile File="$(OutputFile)" Lines="@(Lines-&gt;Replace('$VERSION$', '$(InformationalVersion)')-&gt;Replace('$TOOL_ID$', '$(ToolId)'))" Overwrite="true" Encoding="UTF-8" />
  </Target>
  
</Project>
