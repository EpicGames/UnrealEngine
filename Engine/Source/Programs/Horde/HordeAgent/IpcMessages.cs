// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers.Binary;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace HordeAgent
{
	/// <summary>
	/// Constants
	/// </summary>
	public static class AgentMessagePipe
	{
		/// <summary>
		/// Name of the pipe to communicate over
		/// </summary>
		public const string PipeName = "Horde.Agent";
	}

	/// <summary>
	/// Message 
	/// </summary>
	public enum AgentMessageType
	{
		/// <summary>
		/// Response indicating that the sent message was invalid
		/// </summary>
		InvalidResponse = 0,

		/// <summary>
		/// Request the current agent status.
		/// </summary>
		GetStatusRequest = 1,

		/// <summary>
		/// Returns the current status, as a Json-encoded <see cref="AgentStatusMessage"/> message
		/// </summary>
		GetStatusResponse = 2,

		/// <summary>
		/// Sets the paused state
		/// </summary>
		SetEnabledRequest = 3,

		/// <summary>
		/// Gets information about the server we're connected to.
		/// </summary>
		GetSettingsRequest = 4,

		/// <summary>
		/// Returns information about the server we're connected to.
		/// </summary>
		GetSettingsResponse = 5,
		
		/// <summary>
		/// Set agent settings request
		/// </summary>
		SetSettingsRequest = 6,
		
		/// <summary>
		/// Set agent settings response
		/// </summary>
		SetSettingsResponse = 7,
	}

	/// <summary>
	/// Buffer used to read/write agent messages
	/// </summary>
	public class AgentMessageBuffer
	{
		static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions { PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase, DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull };

		readonly ArrayMemoryWriter _writer = new ArrayMemoryWriter(256);

		/// <summary>
		/// Type of the message currently in the buffer
		/// </summary>
		public AgentMessageType Type { get; private set; }

		/// <summary>
		/// Gets the full message data
		/// </summary>
		public ReadOnlyMemory<byte> Data => _writer.WrittenMemory;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentMessageBuffer()
		{
			_writer.Advance(4);
		}

		/// <summary>
		/// Sets the current message to the given type, with a json serialized body
		/// </summary>
		/// <param name="type">Type of the message</param>
		/// <param name="message">The message body</param>
		public void Set(AgentMessageType type, object? message = null)
		{
			_writer.Clear();
			_writer.Advance(4);

			if (message != null)
			{
				using Utf8JsonWriter writer = new Utf8JsonWriter(_writer);
				JsonSerializer.Serialize(writer, message, s_jsonOptions);
			}

			Type = type;
			BinaryPrimitives.WriteUInt32BigEndian(_writer.WrittenSpan, ((uint)type << 24) | (uint)(_writer.Length - 4));
		}

		/// <summary>
		/// Reads a message from the given stream
		/// </summary>
		/// <param name="stream">Stream to read from</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<bool> TryReadAsync(Stream stream, CancellationToken cancellationToken)
		{
			_writer.Clear();

			Memory<byte> headerData = _writer.GetMemoryAndAdvance(4);

			int readLength = await stream.ReadAsync(headerData, cancellationToken);
			if (readLength == 0)
			{
				return false;
			}
			if (readLength < headerData.Length)
			{
				await stream.ReadFixedLengthBytesAsync(headerData.Slice(readLength), cancellationToken);
			}

			uint header = BinaryPrimitives.ReadUInt32BigEndian(headerData.Span);

			Type = (AgentMessageType)(header >> 24);
			int length = (int)(header & 0xffffff);

			await stream.ReadFixedLengthBytesAsync(_writer.GetMemoryAndAdvance(length), cancellationToken);
			return true;
		}

		/// <summary>
		/// Writes the current message to a stream
		/// </summary>
		/// <param name="stream">Stream to write to</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task SendAsync(Stream stream, CancellationToken cancellationToken)
		{
			await stream.WriteAsync(_writer.WrittenMemory, cancellationToken);
		}

		/// <summary>
		/// Parse the message body as a particular type
		/// </summary>
		public T Parse<T>()
		{
			Utf8JsonReader reader = new Utf8JsonReader(_writer.WrittenSpan.Slice(4));
			return JsonSerializer.Deserialize<T>(ref reader, s_jsonOptions)!;
		}
	}

	/// <summary>
	/// Settings for the agent
	/// </summary>
	/// <param name="ServerUrl">Url of the Horde server</param>
	public record AgentSettingsMessage(Uri? ServerUrl);

	/// <summary>
	/// Sets the enabled state for the agent
	/// </summary>
	/// <param name="IsEnabled">Whether the agent is enabled or not</param>
	public record AgentEnabledMessage(bool IsEnabled);
	
	/// <summary>
	/// Set settings request message
	/// </summary>
	/// <param name="CpuCount">Maximum number of logical CPU cores workloads should use</param>
	/// <param name="CpuMultiplier">CPU core multiplier applied to CPU core count setting</param>
	public record AgentSetSettingsRequest(int CpuCount, double CpuMultiplier);

	/// <summary>
	/// Current status of the agent
	/// </summary>
	/// <param name="Healthy">Whether the agent is healthy</param>
	/// <param name="NumLeases">Number of leases currently being executed</param>
	/// <param name="Detail">Description of the state</param>
	public record class AgentStatusMessage(bool Healthy, int NumLeases, string Detail)
	{
		/// <summary>
		/// Static status object for starting an agent
		/// </summary>
		public const string Starting = "Starting up...";

		/// <summary>
		/// Agent is waiting to be enrolled with the server
		/// </summary>
		public const string WaitingForEnrollment = "Waiting for enrollment...";

		/// <summary>
		/// Agent is connecting to the server
		/// </summary>
		public const string ConnectingToServer = "Connecting to server...";
	}
}
