// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Net.Sockets;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Transports;
using EpicGames.Horde.Logs;
using HordeAgent.Services;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeAgent.Leases.Handlers
{
	/// <summary>
	/// Handler for compute tasks
	/// </summary>
	class ComputeHandler : LeaseHandler<ComputeTask>
	{
		readonly ComputeListenerService _listenerService;
		readonly IOptionsMonitor<AgentSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeHandler(RpcLease lease, ComputeListenerService listenerService, IOptionsMonitor<AgentSettings> settings)
			: base(lease)
		{
			_listenerService = listenerService;
			_settings = settings;
		}

		/// <inheritdoc/>
		protected override async Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, ComputeTask computeTask, Tracer tracer, ILogger localLogger, CancellationToken cancellationToken)
		{
			await using IServerLogger? serverLogger = (computeTask.LogId != null) ? session.HordeClient.CreateServerLogger(LogId.Parse(computeTask.LogId), LogLevel.Trace).WithLocalLogger(localLogger) : null;
			ILogger logger = serverLogger ?? localLogger;

			if (!String.IsNullOrEmpty(computeTask.ParentLeaseId))
			{
				logger.LogInformation("Parent lease: {LeaseId}", computeTask.ParentLeaseId);
			}

			logger.LogInformation("Starting compute task (lease {LeaseId}). Waiting for connection with nonce {Nonce}...", leaseId, StringUtils.FormatHexString(computeTask.Nonce.Span));
			ClearTerminationSignalFile(logger);

			TcpClient? tcpClient = null;
			try
			{
				TimeSpan timeout = TimeSpan.FromMilliseconds(computeTask.InactivityTimeoutMs <= 0 ? 30000 : computeTask.InactivityTimeoutMs);

				tcpClient = await _listenerService.WaitForClientAsync(new ByteString(computeTask.Nonce.Memory), timeout, cancellationToken);
				if (tcpClient == null)
				{
					logger.LogInformation("Timed out waiting for connection after {Time} ms", (int)timeout.TotalMilliseconds);
					return LeaseResult.Success;
				}

				logger.LogInformation("Matched connection for {Nonce}", StringUtils.FormatHexString(computeTask.Nonce.Span));
				
				using CancellationTokenSource cts = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
				await using ComputeTransport innerTransport = await CreateTransportAsync(computeTask, tcpClient.Client, cts.Token);
				await using IdleTimeoutTransport idleTimeoutTransport = new(innerTransport, timeout);
				
				await using BackgroundTask timeoutTask = BackgroundTask.StartNew(ctx => idleTimeoutTransport.StartWatchdogTimerAsync(cts, logger, ctx));
				try
				{
					ComputeProtocol protocol = (ComputeProtocol)computeTask.Protocol;
					logger.LogInformation("Using compute protocol version {Version} with encryption {Encryption}", (int)protocol, computeTask.Encryption);
					
					await using RemoteComputeSocket socket = new (idleTimeoutTransport, protocol, logger);
					DirectoryReference sandboxDir = DirectoryReference.Combine(session.WorkingDir, "Sandbox", leaseId.ToString());
					try
					{
						DirectoryReference.CreateDirectory(sandboxDir);
						DirectoryReference sharedDir = DirectoryReference.Combine(session.WorkingDir, "Saved");
						DirectoryReference.CreateDirectory(sharedDir);
						
						Dictionary<string, string?> newEnvVars = new ()
						{
							["UE_HORDE_SHARED_DIR"] = sharedDir.FullName,
							["UE_HORDE_TERMINATION_SIGNAL_FILE"] = _settings.CurrentValue.GetTerminationSignalFile().FullName,
							["UE_HORDE_CPU_COUNT"] = Convert.ToString(_settings.CurrentValue.CpuCount ?? Environment.ProcessorCount),
							["UE_HORDE_CPU_MULTIPLIER"] = Convert.ToString(_settings.CurrentValue.CpuMultiplier, CultureInfo.InvariantCulture),
						};
						
						AgentMessageHandler worker = new AgentMessageHandler(sandboxDir, newEnvVars, false, _settings.CurrentValue.WineExecutablePath, _settings.CurrentValue.ContainerEngineExecutablePath, logger);
						await worker.RunAsync(socket, cts.Token);
						await socket.CloseAsync(cts.Token);
						return LeaseResult.Success;
					}
					finally
					{
						FileUtils.ForceDeleteDirectory(sandboxDir);
					}
				}
				catch (OperationCanceledException ex) when (cts.IsCancellationRequested && idleTimeoutTransport.TimeSinceActivity > idleTimeoutTransport.NoDataTimeout)
				{
					logger.LogError(ex, "Lease was terminated due to no data being received for {Time} seconds", (int)idleTimeoutTransport.NoDataTimeout.TotalSeconds);
					return new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.TimedOut);
				}
			}
			catch (OperationCanceledException)
			{
				throw;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Exception while executing compute task: {Message}", ex.Message);
				return new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError);
			}
			finally
			{
				tcpClient?.Dispose();
			}
		}

		private static async Task<ComputeTransport> CreateTransportAsync(ComputeTask computeTask, Socket socket, CancellationToken cancellationToken)
		{
			switch (computeTask.Encryption)
			{
				case ComputeEncryption.SslRsa2048:
				case ComputeEncryption.SslEcdsaP256:
					TcpSslTransport sslTransport = new(socket, computeTask.Certificate.ToByteArray(), true);
					await sslTransport.AuthenticateAsync(cancellationToken);
					return sslTransport;

				case ComputeEncryption.Aes:
#pragma warning disable CA2000 // Dispose objects before losing scope
					return new AesTransport(new TcpTransport(socket), computeTask.Key.ToByteArray());
#pragma warning restore CA2000 // Restore CA2000

				case ComputeEncryption.Unspecified:
				case ComputeEncryption.None:
				default:
					return new TcpTransport(socket);
			}
		}

		private void ClearTerminationSignalFile(ILogger logger)
		{
			string path = _settings.CurrentValue.GetTerminationSignalFile().FullName;
			try
			{
				File.Delete(path);
			}
			catch (Exception e)
			{
				// If this file is not removed and lingers on from previous executions,
				// new compute tasks may pick it up and erroneously decide to terminate.
				logger.LogError(e, "Unable to delete termination signal file {Path}", path);
			}
		}
	}

	class ComputeHandlerFactory : LeaseHandlerFactory<ComputeTask>
	{
		readonly ComputeListenerService _listenerService;
		readonly IOptionsMonitor<AgentSettings> _settings;

		public ComputeHandlerFactory(ComputeListenerService listenerService, IOptionsMonitor<AgentSettings> settings)
		{
			_listenerService = listenerService;
			_settings = settings;
		}

		public override LeaseHandler<ComputeTask> CreateHandler(RpcLease lease)
			=> new ComputeHandler(lease, _listenerService, _settings);
	}
}

