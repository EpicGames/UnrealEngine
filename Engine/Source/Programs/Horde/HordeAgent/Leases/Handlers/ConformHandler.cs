// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Logs;
using Google.Protobuf;
using HordeAgent.Services;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeAgent.Leases.Handlers
{
	internal class ConformHandler(RpcLease lease, IOptionsMonitor<AgentSettings> settings) : LeaseHandler<ConformTask>(lease)
	{
		/// <inheritdoc/>
		protected override async Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, ConformTask conformTask, Tracer tracer, ILogger localLogger, CancellationToken cancellationToken)
		{
			await using IServerLogger serverLogger = session.HordeClient.CreateServerLogger(LogId.Parse(conformTask.LogId)).WithLocalLogger(localLogger);
			
			if (settings.CurrentValue.Mode == AgentMode.Workstation)
			{
				serverLogger.LogInformation("Agent is running in workstation mode. Conform lease ignored.");
				return LeaseResult.Success;
			}
			
			List<string> arguments = new List<string>();
			arguments.Add("execute");
			arguments.Add("conform");
			arguments.Add($"-AgentId={session.AgentId}");
			arguments.Add($"-SessionId={session.SessionId}");
			arguments.Add($"-LeaseId={leaseId}");
			arguments.Add($"-WorkingDir={session.WorkingDir}");

			FileReference driverAssembly = FileReference.Combine(new DirectoryReference(AppContext.BaseDirectory), "JobDriver", "JobDriver.dll");

			Dictionary<string, string> environment = ManagedProcess.GetCurrentEnvVars();
			environment["UE_HORDE_JOB_DRIVER_TASK_ARG"] = $"-Task={Convert.ToBase64String(conformTask.ToByteArray())}";
			environment[HordeHttpClient.HordeUrlEnvVarName] = session.HordeClient.ServerUrl.ToString();
			environment[HordeHttpClient.HordeTokenEnvVarName] = await session.HordeClient.GetAccessTokenAsync(false, cancellationToken) ?? String.Empty;
			environment["UE_LOG_JSON_TO_STDOUT"] = "1";

			int exitCode = await RunDotNetProcessAsync(driverAssembly, arguments, environment, AgentApp.IsSelfContained, serverLogger, cancellationToken);
			serverLogger.LogInformation("Driver finished with exit code {ExitCode}", exitCode);

			return (exitCode == 0) ? LeaseResult.Success : new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.NonZeroExitCode);
		}
	}

	internal class ConformHandlerFactory(IOptionsMonitor<AgentSettings> settings) : LeaseHandlerFactory<ConformTask>
	{
		public override LeaseHandler<ConformTask> CreateHandler(RpcLease lease) => new ConformHandler(lease, settings);
	}
}

