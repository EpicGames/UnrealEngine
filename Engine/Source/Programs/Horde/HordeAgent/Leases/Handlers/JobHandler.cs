// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Globalization;
using System.Runtime.InteropServices;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Utilities;
using Google.Protobuf;
using HordeAgent.Services;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using HordeAgent.Utility;

namespace HordeAgent.Leases.Handlers
{
	class JobHandler : LeaseHandler<ExecuteJobTask>
	{
		private readonly IOptionsMonitor<AgentSettings> _settings;

		private ManagedProcess? _driverProcess;
		private ManagedProcessGroup? _driverProcessGroup;
		private readonly string? _driverCancelEventName;
		private readonly EventWaitHandle? _driverCancelEvent;
		private const string DriverCancelEventKeyName = "UE_HORDE_JOB_EXECUTOR_ABORT_EVENT_NAME";
		const double BytesPerMegabyte = 1024.0 * 1024.0;
		private WindowsProcessMemoryTracker? _memoryTracker;

		/// <summary>
		/// The cancellation source to make sure the job driver is always terminated 
		/// </summary>
		private readonly CancellationTokenSource _cancellationSource = new();

		/// <summary>
		/// How long to give the job driver child process to handle the shutdown request before terminating the process ourselves
		/// </summary>
		private readonly TimeSpan _driverCancelGracePeriod = TimeSpan.FromSeconds(30);

		public JobHandler(RpcLease lease, IOptionsMonitor<AgentSettings> settings)
			: base(lease)
		{
			_settings = settings;

			(_driverCancelEvent, _driverCancelEventName) = CreateDriverCancelEvent("Horde.JobDriver.Cancel");
		}

		private static (EventWaitHandle?, string?) CreateDriverCancelEvent(string name)
		{
			// Named events across processes are only supported on Windows
			if (!OperatingSystem.IsWindows())
			{
				return (null, null);
			}

			// Must have a unique name per job to avoid a previous shared signaled event being used and the job driver exiting immediately
			string uniqueName = $"{name}.{Guid.NewGuid()}";

			bool createdNew = false;
			EventWaitHandle? eventObj = null;
			try
			{
				eventObj = new EventWaitHandle(false, EventResetMode.ManualReset, uniqueName, out createdNew);
			}
			catch
			{
				// ignored
			}
			finally
			{
				if (!createdNew)
				{
					eventObj?.Dispose();
					eventObj = null;
				}
			}

			return createdNew ? (eventObj, uniqueName) : (null, null);
		}

		protected override void Dispose(bool disposing)
		{
			if (disposing)
			{
				_driverProcess?.Dispose();
				_driverProcessGroup?.Dispose();
				_driverCancelEvent?.Dispose();
				_cancellationSource.Dispose();
				
				// Normally disposed in ExecuteAsync, but handle edge cases
				if (OperatingSystem.IsWindows())
				{
					_memoryTracker?.Dispose();
				}
			}
			base.Dispose(disposing);
		}

		/// <inheritdoc/>
		protected override async Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, ExecuteJobTask executeTask, Tracer tracer, ILogger localLogger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(JobHandler)}.{nameof(ExecuteAsync)}");
			span.SetAttribute("horde.job.id", executeTask.JobId);
			span.SetAttribute("horde.job.name", executeTask.JobName);
			span.SetAttribute("horde.job.batch_id", executeTask.BatchId);

			await using IServerLogger logger = session.HordeClient.CreateServerLogger(LogId.Parse(executeTask.LogId)).WithLocalLogger(localLogger);

			// Track Horde agent's own memory at job start (Windows only)
			long agentMemoryStart = 0;
			if (OperatingSystem.IsWindows())
			{
				agentMemoryStart = Process.GetCurrentProcess().PrivateMemorySize64;
			}

			int exitCode = 0;
			Exception? exception = null;

			executeTask.JobOptions ??= new RpcJobOptions();

			List<string> arguments = new List<string>();
			arguments.Add("execute");
			arguments.Add("job");
			arguments.Add($"-AgentId={session.AgentId}");
			arguments.Add($"-SessionId={session.SessionId}");
			arguments.Add($"-LeaseId={leaseId}");
			arguments.Add($"-WorkingDir={session.WorkingDir}");

			string driverName = String.IsNullOrEmpty(executeTask.JobOptions.Driver) ? "JobDriver" : executeTask.JobOptions.Driver;
			FileReference driverAssembly = FileReference.Combine(new DirectoryReference(AppContext.BaseDirectory), driverName, $"{driverName}.dll");			
			span.SetAttribute("horde.job.driver_name", driverName);

			Dictionary<string, string> environment = ManagedProcess.GetCurrentEnvVars();
			environment["UE_HORDE_JOB_DRIVER_TASK_ARG"] = $"-Task={Convert.ToBase64String(executeTask.ToByteArray())}";

			try
			{
				environment[HordeHttpClient.HordeUrlEnvVarName] = session.HordeClient.ServerUrl.ToString();
				environment[HordeHttpClient.HordeTokenEnvVarName] = executeTask.Token;
				environment["UE_LOG_JSON_TO_STDOUT"] = "1";
				environment["UE_HORDE_OTEL_SETTINGS"] = OpenTelemetrySettingsExtensions.Serialize(_settings.CurrentValue.OpenTelemetry, true);
				if (_settings.CurrentValue.TempStorageMaxBatchSize.HasValue)
				{
					environment["UE_HORDE_TEMP_STORAGE_MAX_BATCH_SIZE"] = Convert.ToString(_settings.CurrentValue.TempStorageMaxBatchSize.Value, CultureInfo.InvariantCulture);
				}
				environment["UE_HORDE_LOG_OPTIONAL_INPUT_FAILURE_AS_WARNING"] = _settings.CurrentValue.LogOptionalInputFailureAsWarning ? "1" : "0";

				if (_driverCancelEventName != null)
				{
					environment[DriverCancelEventKeyName] = _driverCancelEventName;
				}

				// Need to know the driver process ID for cancellation, use an alternative to RunDotNetProcessAsync
				StartDriverProcess(driverAssembly, arguments, environment, AgentApp.IsSelfContained, logger);

				// Start memory tracking for child processes (Windows only)
				if (OperatingSystem.IsWindows() && _driverProcessGroup != null)
				{
					_memoryTracker = new WindowsProcessMemoryTracker(_driverProcessGroup, TimeSpan.FromSeconds(30), logger);
					_memoryTracker.Start();
				}

				await using (cancellationToken.Register(DriverCancelHandler))
				{
					exitCode = await WaitForDriverProcessExitAsync(logger, _cancellationSource.Token);
				}

				cancellationToken.ThrowIfCancellationRequested();
				logger.LogInformation("Driver finished with exit code {ExitCode}", exitCode);
			}
			catch (OperationCanceledException ex)
			{
				logger.LogError(ex, "Lease was cancelled ({Reason})", CancellationReason);
				exception = ex;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unhandled exception: {Message}", ex.Message);
				exception = ex;
			}
			finally
			{
				_driverCancelEvent?.Reset();
			}

			// Log memory metrics to server (Windows only)
			if (OperatingSystem.IsWindows())
			{
				LogAgentMemory(agentMemoryStart, logger);
				if (_driverProcessGroup != null && _memoryTracker != null)
				{
					LogChildProcessMemory(_driverProcessGroup, _memoryTracker, logger);
				}
				_memoryTracker?.Dispose();
				_memoryTracker = null;
			}

			if (exception != null || exitCode != 0)
			{
				try
				{
					logger.LogInformation("Unexpected exit of job driver. Initiating job batch cleanup... (exitCode={ExitCode} exception={Exception})", exitCode, exception?.Message);
					// Run job batch cleanup
					environment["UE_HORDE_JOB_BATCH_CLEANUP"] = "1";
					int cleanupCode = await RunDotNetProcessAsync(driverAssembly, arguments, environment, AgentApp.IsSelfContained, logger, CancellationToken.None);
					if (cleanupCode != 0)
					{
						logger.LogInformation("Driver job batch cleanup finished with exit code {ExitCode}", cleanupCode);
					}
				}
				catch (Exception ex)
				{
					logger.LogError(ex, "Driver job batch cleanup error ({Message})", ex.Message);
				}

				if (exception != null)
				{
					throw exception;
				}
			}

			return (exitCode == 0) ? LeaseResult.Success : new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.NonZeroExitCode);
		}

		private void DriverCancelHandler()
		{
			bool handled = false;
			if (_driverCancelEvent != null)
			{
				_driverCancelEvent.Set();
				handled = true;
			}
			else if (!OperatingSystem.IsWindows())
			{
				// Cannot use named events on non-Windows OSes, instead send SIGTERM to the job driver process
				if (_driverProcess != null)
				{
					handled = SendSigTerm(_driverProcess.Id);
				}
			}

			// Whether the job driver process was signaled to abort, the job driver must be terminated
			if (handled)
			{
				_cancellationSource.CancelAfter(_driverCancelGracePeriod);
			}
			else
			{
				_cancellationSource.Cancel();
			}
		}

		private void StartDriverProcess(FileReference entryAssembly, IEnumerable<string> arguments, IReadOnlyDictionary<string, string>? environment, bool useNativeHost, ILogger logger)
		{
			if (useNativeHost)
			{
				FileReference nativeHost = entryAssembly.ChangeExtension(OperatingSystem.IsWindows() ? ".exe" : null);
				StartDriverProcess(nativeHost.FullName, arguments, environment, logger);
			}
			else
			{
				IEnumerable<string> allArguments = arguments.Prepend(entryAssembly.FullName);
				StartDriverProcess("dotnet", allArguments, environment, logger);
			}
		}

		private void StartDriverProcess(string executable, IEnumerable<string> arguments, IReadOnlyDictionary<string, string>? environment, ILogger logger)
		{
			string commandLine = CommandLineArguments.Join(arguments);
			logger.LogInformation("Starting child process: {Executable} {CommandLine}", CommandLineArguments.Quote(executable), commandLine);
			try
			{
				_driverProcessGroup = new ManagedProcessGroup();
				_driverProcess = new ManagedProcess(_driverProcessGroup, executable, commandLine, null, environment, ProcessPriorityClass.Normal);
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Failed to start process: {Message}", ex.Message);
				throw;
			}
		}

		private async Task<int> WaitForDriverProcessExitAsync(ILogger logger, CancellationToken cancellationToken)
		{
			if (_driverProcess == null)
			{
				throw new InvalidOperationException("Job Driver process not created");
			}

			try
			{
				for (; ; )
				{
					string? line = await _driverProcess.ReadLineAsync(cancellationToken);
					if (line == null)
					{
						break;
					}

					JsonLogEvent jsonLogEvent;
					if (JsonLogEvent.TryParse(line, out jsonLogEvent))
					{
						logger.LogJsonLogEvent(jsonLogEvent);
					}
					else
					{
						logger.LogInformation("{Line}", line);
					}
				}

				await _driverProcess.WaitForExitAsync(CancellationToken.None);
				return _driverProcess.ExitCode;
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Failed to run process: {Message}", ex.Message);
				throw;
			}
		}

		[DllImport("libc", SetLastError = true)]
		private static extern int kill(int pid, int sig);

		private static bool SendSigTerm(int processId)
		{
			if (!(OperatingSystem.IsLinux() || OperatingSystem.IsMacOS()))
			{
				throw new PlatformNotSupportedException($"{nameof(SendSigTerm)} not supported");
			}

			const int SigTerm = 15;
			int result = kill(processId, SigTerm);
			return result == 0;
		}

		private static void LogAgentMemory(long memoryStart, ILogger logger)
		{
			using Process currentProcess = Process.GetCurrentProcess();
			long memoryEnd = currentProcess.PrivateMemorySize64;
			long peakMemory = currentProcess.PeakWorkingSet64;

			logger.LogInformation(
				"HordeAgent Memory: Start={StartMb:F1} MB End={EndMb:F1} MB Delta={DeltaMb:+0.0;-0.0;0.0} MB Peak={PeakMb:F1} MB",
				memoryStart / BytesPerMegabyte,
				memoryEnd / BytesPerMegabyte,
				(memoryEnd - memoryStart) / BytesPerMegabyte,
				peakMemory / BytesPerMegabyte);
		}

		[System.Runtime.Versioning.SupportedOSPlatform("windows")]
		private static void LogChildProcessMemory(
			ManagedProcessGroup processGroup,
			WindowsProcessMemoryTracker tracker,
			ILogger logger)
		{
			IReadOnlyList<ProcessMemoryInfo> topProcesses = tracker.GetTopMemoryProcesses(5);

			long totalPeakBytes = processGroup.PeakJobMemoryUsed;
			long largestProcessPeakBytes = processGroup.PeakProcessMemoryUsed;

			logger.LogInformation(
				"Child Process Memory Summary: TotalPeak={TotalPeakMb:F1} MB, LargestSingleProcess={LargestMb:F1} MB",
				totalPeakBytes / BytesPerMegabyte,
				largestProcessPeakBytes / BytesPerMegabyte);

			if (topProcesses.Count > 0)
			{
				logger.LogInformation("Top {Count} memory-consuming processes:", topProcesses.Count);
				foreach (ProcessMemoryInfo proc in topProcesses)
				{
					logger.LogInformation(
						"  PID={Pid} {Name}.exe PeakPrivate={PrivateMb:F1} MB PeakWorkingSet={WorkingSetMb:F1} MB Args=\"{Args}\"",
						proc.ProcessId,
						proc.ProcessName,
						proc.PeakPrivateBytes / BytesPerMegabyte,
						proc.PeakWorkingSetBytes / BytesPerMegabyte,
						proc.CommandLine);
				}
			}
		}
	}

	class JobHandlerFactory : LeaseHandlerFactory<ExecuteJobTask>
	{
		private readonly IOptionsMonitor<AgentSettings> _settings;
		
		public JobHandlerFactory(IOptionsMonitor<AgentSettings> settings)
		{
			_settings = settings;
		}
		
		public override LeaseHandler<ExecuteJobTask> CreateHandler(RpcLease lease)
			=> new JobHandler(lease, _settings);
	}
}

