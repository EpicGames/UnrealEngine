// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Logs;
using HordeAgent.Services;
using HordeAgent.Utility;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeAgent.Leases.Handlers
{
	class RestartHandler : LeaseHandler<RestartTask>
	{
		public RestartHandler(RpcLease lease)
			: base(lease)
		{ }

		/// <inheritdoc/>
		protected override async Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, RestartTask task, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			await using IServerLogger serverLogger = session.HordeClient.CreateServerLogger(LogId.Parse(task.LogId)).WithLocalLogger(logger);
			serverLogger.LogInformation("Scheduling restart task for agent {AgentId} when agent shuts down.", session.AgentId);
			SessionResult result = new SessionResult((logger, ctx) => Shutdown.ExecuteAsync(true, logger, ctx));
			return new LeaseResult(result);
		}
	}

	class RestartHandlerFactory : LeaseHandlerFactory<RestartTask>
	{
		public override LeaseHandler<RestartTask> CreateHandler(RpcLease lease)
			=> new RestartHandler(lease);
	}
}
