// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.IO.Compression;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Logs;
using Grpc.Core;
using HordeAgent.Services;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeAgent.Leases.Handlers
{
	class UpgradeHandler : LeaseHandler<UpgradeTask>
	{
		public UpgradeHandler(RpcLease lease)
			: base(lease)
		{ }

		/// <inheritdoc/>
		protected override async Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, UpgradeTask task, Tracer tracer, ILogger localLogger, CancellationToken cancellationToken)
		{
			await using IServerLogger logger = session.HordeClient.CreateServerLogger(LogId.Parse(task.LogId)).WithLocalLogger(localLogger);

			string requiredVersion = task.SoftwareId;

			// Check if we're running the right version
			if (requiredVersion != null && requiredVersion != AgentApp.Version)
			{
				logger.LogInformation("Upgrading from {CurrentVersion} to {TargetVersion}", AgentApp.Version, requiredVersion);

				// Clear out the working directory
				DirectoryReference upgradeDir = DirectoryReference.Combine(session.WorkingDir, "Upgrade");
				DirectoryReference.CreateDirectory(upgradeDir);
				await DeleteDirectoryContentsAsync(new DirectoryInfo(upgradeDir.FullName));

				// Download the new software
				FileInfo outputFile = new FileInfo(Path.Combine(upgradeDir.FullName, "Agent.zip"));

				HordeRpc.HordeRpcClient hordeRpc = await session.HordeClient.CreateGrpcClientAsync<HordeRpc.HordeRpcClient>(cancellationToken);
				using (AsyncServerStreamingCall<RpcDownloadSoftwareResponse> cursor = hordeRpc.DownloadSoftware(new RpcDownloadSoftwareRequest { Version = requiredVersion }, null, null, cancellationToken))
				{
					await using Stream outputStream = outputFile.Open(FileMode.Create);
					while (await cursor.ResponseStream.MoveNext(cancellationToken))
					{
						outputStream.Write(cursor.ResponseStream.Current.Data.Span);
					}
				}

				// Extract it to a temporary directory
				DirectoryReference extractedDir = DirectoryReference.Combine(upgradeDir, "Extracted");
				DirectoryReference.CreateDirectory(extractedDir);
				await ZipFile.ExtractToDirectoryAsync(outputFile.FullName, extractedDir.FullName, cancellationToken);

				//				// Debug code for updating an agent with the local version
				//				foreach (FileInfo SourceFile in new FileInfo(Assembly.GetExecutingAssembly().Location).Directory.EnumerateFiles())
				//				{
				//					SourceFile.CopyTo(Path.Combine(ExtractedDir.FullName, SourceFile.Name), true);
				//				}

				// Get the current process and assembly. This may be different if running through dotnet.exe rather than a native PE image.
				DirectoryReference targetDir = new(AppContext.BaseDirectory);
				StringBuilder currentArguments = new StringBuilder();

				foreach (string arg in AgentApp.Args)
				{
					currentArguments.AppendArgument(arg);
				}

				bool isUpdateSelfContained = !FileReference.Exists(FileReference.Combine(extractedDir, "HordeAgent.dll"));

				StringBuilder arguments = new();
				string executable;
				if (AgentApp.IsSelfContained)
				{
					// Current running agent is packaged as a self-contained .NET application and assume incoming update is the same
					if (!isUpdateSelfContained)
					{
						logger.LogError("Current running agent is packaged as a self-contained app, but incoming update is not");
						return new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError);
					}

					executable = "HordeAgent";
					if (OperatingSystem.IsWindows())
					{
						executable += ".exe";
					}

					executable = FileReference.Combine(extractedDir, executable).FullName;
					if (!File.Exists(executable))
					{
						logger.LogError("Unable to find self-contained {AgentExe} in extracted archive", executable);
						return new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError);
					}
				}
				else
				{
#pragma warning disable IL3000 // Avoid accessing Assembly file path when publishing as a single file
					FileReference assemblyFileName = new(Assembly.GetExecutingAssembly().Location);
#pragma warning restore IL3000 // Avoid accessing Assembly file path when publishing as a single file

					// New unpacked agent is not self-contained, launch the upgrade command via "dotnet" external executable
					executable = "dotnet";
					FileReference newAssemblyFileName = FileReference.Combine(extractedDir, assemblyFileName.MakeRelativeTo(targetDir));
					if (!FileReference.Exists(newAssemblyFileName))
					{
						logger.LogError("Unable to find {AgentExe} in extracted archive", newAssemblyFileName);
						return new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError);
					}

					arguments.AppendArgument(newAssemblyFileName.FullName);
				}

				arguments.AppendArgument("Service");
				arguments.AppendArgument("Upgrade");
				arguments.AppendArgument("-ProcessId=", Environment.ProcessId.ToString());
				arguments.AppendArgument("-TargetDir=", targetDir.FullName);
				arguments.AppendArgument("-Arguments=", currentArguments.ToString());

				// Spawn the other process
				SessionResult result = new SessionResult((logger, ctx) => RunUpgradeAsync(executable, arguments.ToString(), logger, ctx));
				return new LeaseResult(result);
			}

			return LeaseResult.Success;
		}

		static async Task RunUpgradeAsync(string fileName, string arguments, ILogger logger, CancellationToken cancellationToken)
		{
			using (Process process = new Process())
			{
				StringBuilder launchCommand = new StringBuilder();
				launchCommand.AppendArgument(fileName);
				launchCommand.Append(' ');
				launchCommand.Append(arguments);
				logger.LogInformation("Launching: {Launch}", launchCommand.ToString());

				process.StartInfo.FileName = fileName;
				process.StartInfo.Arguments = arguments.ToString();
				process.StartInfo.UseShellExecute = false;
				process.EnableRaisingEvents = true;

				TaskCompletionSource<int> exitCodeSource = new TaskCompletionSource<int>(TaskCreationOptions.RunContinuationsAsynchronously);
				process.Exited += (sender, args) => { exitCodeSource.SetResult(process.ExitCode); };

				process.Start();

				using (cancellationToken.Register(() => { exitCodeSource.SetResult(0); }))
				{
					await exitCodeSource.Task;
				}

				logger.LogInformation("Upgrade did not terminate initiating process; reconnecting to server...");
			}
		}

		/// <summary>
		/// Delete the contents of a directory without deleting it itself
		/// </summary>
		/// <param name="baseDir">Directory to clean</param>
		/// <returns>Async task</returns>
		static async Task DeleteDirectoryContentsAsync(DirectoryInfo baseDir)
		{
			List<Task> childTasks = new List<Task>();
			foreach (DirectoryInfo subDir in baseDir.EnumerateDirectories())
			{
				childTasks.Add(Task.Run(() => DeleteDirectoryAsync(subDir)));
			}
			foreach (FileInfo file in baseDir.EnumerateFiles())
			{
				file.Attributes = FileAttributes.Normal;
				file.Delete();
			}
			foreach (Task childTask in childTasks)
			{
				await childTask;
			}
		}

		/// <summary>
		/// Deletes a directory and its contents
		/// </summary>
		/// <param name="baseDir">Directory to delete</param>
		/// <returns>Async task</returns>
		static async Task DeleteDirectoryAsync(DirectoryInfo baseDir)
		{
			await DeleteDirectoryContentsAsync(baseDir);
			baseDir.Delete();
		}
	}

	class UpgradeHandlerFactory : LeaseHandlerFactory<UpgradeTask>
	{
		public override LeaseHandler<UpgradeTask> CreateHandler(RpcLease lease)
			=> new UpgradeHandler(lease);
	}
}

