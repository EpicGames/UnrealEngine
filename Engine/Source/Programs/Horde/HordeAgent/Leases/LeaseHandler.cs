// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Metrics;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using HordeAgent.Services;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;
using ByteString = Google.Protobuf.ByteString;

namespace HordeAgent.Leases
{
	/// <summary>
	/// Handles execution of a specific lease type
	/// </summary>
	abstract class LeaseHandler : IDisposable
	{
		/// <summary>
		/// Identifier for this lease
		/// </summary>
		public LeaseId Id { get; }

		/// <summary>
		/// The RPC lease state
		/// </summary>
		public RpcLease RpcLease { get; set; }

		/// <summary>
		/// Payload from the lease
		/// </summary>
		public Any RpcPayload { get; }

		/// <summary>
		/// Result from executing the lease
		/// </summary>
		public Task<LeaseResult> Result { get; private set; }

		/// <summary>
		/// Whether the lease has been cancelled
		/// </summary>
		public bool IsCancelled => _cancellationSource.IsCancellationRequested;

		/// <summary>
		/// Reason for cancellation
		/// </summary>
		public string CancellationReason => _outcomeReason.ToString();

		readonly CancellationTokenSource _cancellationSource = new ();
		LeaseOutcomeReason _outcomeReason = LeaseOutcomeReason.None;

		/// <summary>
		/// Constructor
		/// </summary>
		protected LeaseHandler(RpcLease rpcLease)
		{
			Id = rpcLease.Id;
			RpcLease = rpcLease;
			RpcPayload = rpcLease.Payload;
			Result = Task.FromException<LeaseResult>(new InvalidOperationException("Lease has not been started"));
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			Dispose(disposing: true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Overridable dispose method
		/// </summary>
		protected virtual void Dispose(bool disposing)
		{
			if (disposing)
			{
				_cancellationSource.Dispose();
			}
		}

		/// <summary>
		/// Starts executing the lease
		/// </summary>
		public void Start(ISession session, ISystemMetrics systemMetrics, Tracer tracer, ILogger logger, LeaseLoggerFactory leaseLoggerFactory)
		{
			Result = Task.Run(async () =>
			{
				try
				{
					return await HandleLeaseAsync(session, systemMetrics, tracer, logger, leaseLoggerFactory);
				}
				catch (Exception e)
				{
					logger.LogError(e, "Unhandled exception while executing lease: {Message} ", e.Message);
					throw;
				}
			});
		}

		/// <summary>
		/// Cancels the lease
		/// </summary>
		public void Cancel(LeaseOutcomeReason reason)
		{
			_outcomeReason = reason;
			_cancellationSource.Cancel();
		}

		/// <summary>
		/// Handle a lease request
		/// </summary>
		async Task<LeaseResult> HandleLeaseAsync(ISession session, ISystemMetrics systemMetrics, Tracer tracer, ILogger logger, LeaseLoggerFactory leaseLoggerFactory)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(LeaseHandler)}.{nameof(HandleLeaseAsync)}");
			span.SetAttribute("horde.lease.id", RpcLease.Id.ToString());
			span.SetAttribute("horde.agent.id", session.AgentId.ToString());
			span.SetAttribute("horde.agent.session_id", session.SessionId.ToString());
			logger.LogInformation("Handling lease {LeaseId}: {LeaseTypeUrl}", Id, RpcLease.Payload.TypeUrl);

			Stopwatch leaseDuration = Stopwatch.StartNew();
			CpuTimeMetrics? cpuTimeStart = systemMetrics.GetCpuTime();

			// Get the lease outcome
			LeaseResult result = new (LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError);
			try
			{
				span.SetAttribute("horde.lease.task", RpcPayload.TypeUrl);
				result = await HandleLeasePayloadAsync(session, tracer, leaseLoggerFactory);
			}
			catch (OperationCanceledException) when (_cancellationSource.IsCancellationRequested)
			{
				logger.LogInformation("Lease {LeaseId} cancelled", Id);
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unhandled exception while executing lease {LeaseId}: {Message}", Id, ex.Message);
				span.SetStatus(Status.Error);
				span.RecordException(ex);
			}
			CpuTimeMetrics? cpuTimeEnd = systemMetrics.GetCpuTime();

			// Update the state of the lease
			RpcLease newRpcLease = new RpcLease(RpcLease);
			if (_cancellationSource.IsCancellationRequested)
			{
				newRpcLease.State = RpcLeaseState.Cancelled;
				newRpcLease.Outcome = RpcLeaseOutcome.Failed;
				newRpcLease.OutcomeReasons.AddRange(RpcLease.ToProtobufOutcomeReasons(result.OutcomeReason | _outcomeReason));
				newRpcLease.Output = Google.Protobuf.ByteString.Empty;
			}
			else
			{
				LeaseOutcomeReason reason = result.Outcome is LeaseOutcome.Success ? result.OutcomeReason : result.OutcomeReason | _outcomeReason;
				newRpcLease.State = (result.Outcome == LeaseOutcome.Cancelled) ? RpcLeaseState.Cancelled : RpcLeaseState.Completed;
				newRpcLease.Outcome = (RpcLeaseOutcome)result.Outcome;
				newRpcLease.OutcomeReasons.AddRange(RpcLease.ToProtobufOutcomeReasons(reason));
				newRpcLease.Output = (result.Output != null) ? ByteString.CopyFrom(result.Output) : ByteString.Empty;
			}

			newRpcLease.Metrics = CalculateLeaseMetrics(cpuTimeStart, cpuTimeEnd, leaseDuration.Elapsed, span, logger);
			RpcLease = newRpcLease;

			logger.LogInformation("Transitioning lease {LeaseId} to {State}, outcome={Outcome}", Id, RpcLease.State, RpcLease.Outcome);
			span.SetAttribute("horde.lease.state", RpcLease.State.ToString());
			span.SetAttribute("horde.lease.outcome", RpcLease.Outcome.ToString());

			return result;
		}

		/// <summary>
		/// Dispatch a lease payload to the appropriate handler
		/// </summary>
		internal async Task<LeaseResult> HandleLeasePayloadAsync(ISession session, Tracer tracer, LeaseLoggerFactory leaseLoggerFactory)
		{
			using ILoggerFactory loggerFactory = leaseLoggerFactory.CreateLoggerFactory(Id);
			ILogger leaseLogger = loggerFactory.CreateLogger(GetType());
			return await ExecuteAsync(session, tracer, leaseLogger, _cancellationSource.Token);
		}

		/// <summary>
		/// Executes a lease
		/// </summary>
		/// <returns>Result for the lease</returns>
		protected abstract Task<LeaseResult> ExecuteAsync(ISession session, Tracer tracer, ILogger logger, CancellationToken cancellationToken);

		/// <summary>
		/// Runs a child process, piping the output to the given logger
		/// </summary>
		/// <param name="executable">Executable to launch</param>
		/// <param name="arguments">Command line arguments for the new process</param>
		/// <param name="environment">Environment for the new process</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Exit code of the process</returns>
		protected static async Task<int> RunProcessAsync(string executable, IEnumerable<string> arguments, IReadOnlyDictionary<string, string>? environment, ILogger logger, CancellationToken cancellationToken)
		{
			string commandLine = CommandLineArguments.Join(arguments);
			logger.LogInformation("Running child process: {Executable} {CommandLine}", CommandLineArguments.Quote(executable), commandLine);
			try
			{
				using ManagedProcessGroup processGroup = new ManagedProcessGroup();
				using ManagedProcess process = new ManagedProcess(processGroup, executable, commandLine, null, environment, ProcessPriorityClass.Normal);
				return await RunProcessCoreAsync(process, logger, cancellationToken);
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Failed to run process: {Message}", ex.Message);
				throw;
			}
		}

		/// <summary>
		/// Core process execution loop
		/// </summary>
		private static async Task<int> RunProcessCoreAsync(ManagedProcess process, ILogger logger, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				string? line = await process.ReadLineAsync(cancellationToken);
				if (line == null)
				{
					break;
				}

				if (JsonLogEvent.TryParse(line, out JsonLogEvent jsonLogEvent))
				{
					logger.LogJsonLogEvent(jsonLogEvent);
				}
				else
				{
					logger.LogInformation("{Line}", line);
				}
			}

			await process.WaitForExitAsync(CancellationToken.None);
			return process.ExitCode;
		}

		/// <summary>
		/// Runs a .NET assembly as a child process, piping the output to the given logger
		/// </summary>
		/// <param name="entryAssembly">Assembly to launch</param>
		/// <param name="arguments">Command line arguments for the new process</param>
		/// <param name="environment">Environment for the new process</param>
		/// <param name="useNativeHost">Whether to use a native host process</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Exit code of the process</returns>
		protected static async Task<int> RunDotNetProcessAsync(FileReference entryAssembly, IEnumerable<string> arguments, IReadOnlyDictionary<string, string>? environment, bool useNativeHost, ILogger logger, CancellationToken cancellationToken)
		{
			if (useNativeHost)
			{
				FileReference nativeHost = entryAssembly.ChangeExtension(OperatingSystem.IsWindows() ? ".exe" : null);
				return await RunProcessAsync(nativeHost.FullName, arguments, environment, logger, cancellationToken);
			}
			else
			{
				IEnumerable<string> allArguments = arguments.Prepend(entryAssembly.FullName);
				return await RunProcessAsync("dotnet", allArguments, environment, logger, cancellationToken);
			}
		}

		private static RpcLeaseMetrics CalculateLeaseMetrics(CpuTimeMetrics? cpuTimeStart, CpuTimeMetrics? cpuTimeEnd, TimeSpan leaseDuration, TelemetrySpan span, ILogger logger)
		{
			RpcLeaseMetrics metrics = new()
			{
				SetupTimeMs = 0, // TODO: Implement
				TeardownTimeMs = 0, // TODO: Implement
				GlobalCpuUtilization = 0.0f,
				CpuCount = Environment.ProcessorCount,
			};
			try
			{
				if (cpuTimeStart != null && cpuTimeEnd != null)
				{
					CpuTimeMetrics cpuTimeSpent = cpuTimeEnd.Subtract(cpuTimeStart);
					TimeSpan totalCpuTimeSpent = cpuTimeSpent.User + cpuTimeSpent.System;
					TimeSpan totalCpuTimeAvailable = Environment.ProcessorCount * leaseDuration;
				
					// Prevent the unlikely divide by zero below
					totalCpuTimeAvailable = totalCpuTimeAvailable == TimeSpan.Zero ? TimeSpan.FromMilliseconds(1) : totalCpuTimeAvailable;
				
					metrics.GlobalCpuUtilization = totalCpuTimeSpent / totalCpuTimeAvailable;
					metrics.GlobalCpuTime = new RpcLeaseMetricCpuTime
					{
						UserMs = (uint)Math.Min(uint.MaxValue, cpuTimeSpent.User.TotalMilliseconds),
						SystemMs = (uint)Math.Min(uint.MaxValue, cpuTimeSpent.System.TotalMilliseconds),
						IdleMs = (uint)Math.Min(uint.MaxValue, cpuTimeSpent.Idle.TotalMilliseconds),
					};
					span.SetAttribute("horde.lease.cpu.time_spent_ms", (uint)totalCpuTimeSpent.TotalMilliseconds);
					span.SetAttribute("horde.lease.cpu.time_available_ms", (uint)totalCpuTimeAvailable.TotalMilliseconds);
					span.SetAttribute("horde.lease.cpu.utilization", metrics.GlobalCpuUtilization);
					span.SetAttribute("horde.lease.cpu.num_cores", metrics.CpuCount);
				}
			}
			catch (Exception e)
			{
				logger.LogWarning(e, "Failed to calculate lease metrics");
			}
			return metrics;
		}
	}

	/// <summary>
	/// Implementation of <see cref="LeaseHandler"/> for a specific lease type
	/// </summary>
	/// <typeparam name="T">Type of the lease message</typeparam>
	abstract class LeaseHandler<T> : LeaseHandler where T : IMessage<T>, new()
	{
		protected LeaseHandler(RpcLease rpcLease)
			: base(rpcLease)
		{
		}

		/// <inheritdoc/>
		protected override Task<LeaseResult> ExecuteAsync(ISession session, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
		{
			return ExecuteAsync(session, Id, RpcLease.Payload.Unpack<T>(), tracer, logger, cancellationToken);
		}

		/// <inheritdoc cref="LeaseHandler.ExecuteAsync" />
		protected abstract Task<LeaseResult> ExecuteAsync(ISession session, LeaseId leaseId, T message, Tracer tracer, ILogger logger, CancellationToken cancellationToken);
	}

	class DefaultLeaseHandler : LeaseHandler
	{
		readonly LeaseResult _result;

		public DefaultLeaseHandler(RpcLease lease, LeaseResult result) : base(lease)
			=> _result = result;

		/// <inheritdoc/>
		protected override Task<LeaseResult> ExecuteAsync(ISession session, Tracer tracer, ILogger logger, CancellationToken cancellationToken)
			=> Task.FromResult(_result);
	}
}
