// Copyright Epic Games, Inc. All Rights Reserved.

using Google.Protobuf;
using Google.Protobuf.Reflection;
using HordeCommon.Rpc.Messages;

namespace HordeAgent.Leases
{
	abstract class LeaseHandlerFactory
	{
		/// <summary>
		/// Returns protobuf type urls for the handled message types
		/// </summary>
		public abstract string LeaseType { get; }

		/// <summary>
		/// Creates a new lease handler for the given lease definition
		/// </summary>
		/// <param name="lease">The lease to create a handler for</param>
		public abstract LeaseHandler CreateHandler(RpcLease lease);
	}

	/// <summary>
	/// Implementation of <see cref="LeaseHandler"/> for a specific lease type
	/// </summary>
	/// <typeparam name="T">Type of the lease message</typeparam>
	abstract class LeaseHandlerFactory<T> : LeaseHandlerFactory where T : IMessage<T>, new()
	{
		/// <summary>
		/// Static for the message type descriptor
		/// </summary>
		public static MessageDescriptor Descriptor { get; } = new T().Descriptor;

		/// <inheritdoc/>
		public override string LeaseType { get; } = $"type.googleapis.com/{Descriptor.Name}";

		/// <summary>
		/// Creates a new lease handler for the given lease definition
		/// </summary>
		/// <param name="lease">The lease to create a handler for</param>
		public abstract override LeaseHandler<T> CreateHandler(RpcLease lease);
	}
}
