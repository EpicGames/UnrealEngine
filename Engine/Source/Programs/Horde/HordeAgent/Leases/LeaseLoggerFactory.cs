// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Leases
{
	/// <summary>
	/// Creates local log files for leases executed on this agent
	/// </summary>
	sealed class LeaseLoggerFactory : IAsyncDisposable
	{
		static TimeSpan MaxAge { get; } = TimeSpan.FromDays(3.0);

		readonly IOptionsMonitor<AgentSettings> _settings;
		readonly DirectoryReference _logDir;
		readonly BackgroundTask _backgroundTask;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public LeaseLoggerFactory(IOptionsMonitor<AgentSettings> settings, ILogger<LeaseLoggerFactory> logger)
		{
			_settings = settings;
			_logDir = DirectoryReference.Combine(settings.CurrentValue.LogsDir, "Leases");
			_logger = logger;
			_backgroundTask = BackgroundTask.StartNew(BackgroundCleanupAsync);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
		}

		/// <summary>
		/// Remove all log files older than the given time
		/// </summary>
		public void Cleanup(TimeSpan maxAge)
		{
			try
			{
				DirectoryInfo directoryInfo = _logDir.ToDirectoryInfo();
				if (directoryInfo.Exists)
				{
					DateTime cleanTimeUtc = DateTime.UtcNow - maxAge;
					foreach (FileInfo fileInfo in _logDir.ToDirectoryInfo().EnumerateFiles())
					{
						if (fileInfo.LastWriteTimeUtc < cleanTimeUtc)
						{
							try
							{
								fileInfo.Delete();
							}
							catch (Exception ex)
							{
								_logger.LogInformation(ex, "Error deleting log file {File}: {Message}", fileInfo.FullName, ex.Message);
							}
						}
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Error deleting log file in {Dir}: {Message}", _logDir, ex.Message);
			}
		}

		class LoggerProvider : ILoggerProvider
		{
			readonly ILogger _logger;

			public LoggerProvider(ILogger logger)
				=> _logger = logger;

			public ILogger CreateLogger(string categoryName)
				=> _logger;

			public void Dispose() { }
		}

		/// <summary>
		/// Create a new logger factory for the given lease id
		/// </summary>
		public ILoggerFactory CreateLoggerFactory(LeaseId leaseId)
		{
			return LoggerFactory.Create(builder =>
			{
				builder.AddProvider(Logging.CreateFileLoggerProvider(_logDir, leaseId.ToString()));
				if (_settings.CurrentValue.WriteStepOutputToLogger)
				{
					builder.AddProvider(new LoggerProvider(_logger));
				}
			});
		}

		async Task BackgroundCleanupAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Cleanup(MaxAge);
				await Task.Delay(TimeSpan.FromHours(1.0), cancellationToken);
			}
		}
	}
}
