// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Metrics;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using HordeAgent.Services;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeAgent.Leases
{
	/// <summary>
	/// Implements the message handling loop for an agent. Runs asynchronously until disposed.
	/// </summary>
	class LeaseManager
	{
		/// <summary>
		/// Number of leases completed
		/// </summary>
		public int NumLeasesCompleted { get; private set; }
		
		/// <summary>
		/// Delegate for lease active events
		/// </summary>
		public delegate void LeaseEvent(RpcLease lease, LeaseResult? result = null);
		
		/// <summary>
		/// Event triggered when a lease is accepted and started
		/// </summary>
		public event LeaseEvent? OnLeaseStarted;
		
		/// <summary>
		/// Event triggered when a lease is finished with an outcome
		/// </summary>
		public event LeaseEvent? OnLeaseFinished;
		
		/// <summary>
		/// List of pool IDs this session is a member of
		/// </summary>
		public IReadOnlyList<string> PoolIds { get; private set; } = new List<string>();
		
		/// <summary>
		/// Whether to terminate session after at least one lease has finished
		/// Useful for shutting down as gracefully as possible without interrupting the current lease executing.
		/// </summary>
		public bool TerminateSessionAfterLease { get; set; }
		
		/// <summary>
		/// Object used for controlling access to the access tokens and active sessions list
		/// </summary>
		readonly Lock _lockObject = new Lock();

		/// <summary>
		/// The list of active leases.
		/// </summary>
		readonly List<LeaseHandler> _activeLeases = new List<LeaseHandler>();

		/// <summary>
		/// Whether the agent is currently in an unhealthy state
		/// </summary>
		readonly bool _unhealthy = false;

		/// <summary>
		/// Result from executing this session
		/// </summary>
		SessionResult? _sessionResult;

		/// <summary>
		/// Task completion source used to trigger the background thread to update the leases. Must take a lock on LockObject before 
		/// </summary>
		readonly AsyncEvent _updateLeasesEvent = new AsyncEvent();

		/// <summary>
		/// Number of times UpdateSession has failed
		/// </summary>
		int _updateSessionFailures;

		/// <summary>
		/// How long to wait before trying to reacquire a new connection
		/// Exposed as internal to ease testing. Using a lower delay can speed up tests. 
		/// </summary>
		internal TimeSpan _rpcConnectionRetryDelay = TimeSpan.FromSeconds(5);

		readonly ISession _session;
		readonly ICapabilitiesService _capabilitiesService;
		readonly StatusService _statusService;
		readonly ISystemMetrics _systemMetrics;
		readonly Dictionary<string, LeaseHandlerFactory> _typeUrlToLeaseHandler;
		readonly LeaseLoggerFactory _leaseLoggerFactory;
		readonly IOptionsMonitor<AgentSettings> _settings;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		RpcAgentCapabilities? _capabilities;

		public LeaseManager(
			ISession session,
			ICapabilitiesService capabilitiesService,
			StatusService statusService,
			ISystemMetrics systemMetrics,
			IEnumerable<LeaseHandlerFactory> leaseHandlerFactories,
			LeaseLoggerFactory leaseLoggerFactory,
			IOptionsMonitor<AgentSettings> settings,
			Tracer tracer,
			ILogger logger)
		{
			_session = session;
			_capabilitiesService = capabilitiesService;
			_statusService = statusService;
			_systemMetrics = systemMetrics;
			_typeUrlToLeaseHandler = leaseHandlerFactories.ToDictionary(x => x.LeaseType, x => x);
			_leaseLoggerFactory = leaseLoggerFactory;
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
		}

		public LeaseManager(ISession session, IServiceProvider serviceProvider)
			: this(session,
				serviceProvider.GetRequiredService<CapabilitiesService>(),
				serviceProvider.GetRequiredService<StatusService>(),
				serviceProvider.GetRequiredService<ISystemMetrics>(),
				serviceProvider.GetRequiredService<IEnumerable<LeaseHandlerFactory>>(),
				serviceProvider.GetRequiredService<LeaseLoggerFactory>(),
				serviceProvider.GetRequiredService<IOptionsMonitor<AgentSettings>>(),
				serviceProvider.GetRequiredService<Tracer>(),
				serviceProvider.GetRequiredService<ILogger<LeaseManager>>())
		{
		}

		public List<RpcLease> GetActiveLeases()
		{
			lock (_lockObject)
			{
				return _activeLeases.Select(x => x.RpcLease).ToList();
			}
		}
		
		/// <summary>
		/// Write a termination signal file to disk
		/// Used to communicate an impending termination of workload. It does require workload to be aware of the file and also act on it.
		/// </summary>
		/// <param name="filePath">Path to signal file</param>
		/// <param name="reason">Reason for termination</param>
		/// <param name="terminateAt">Timestamp when termination takes place</param>
		/// <param name="timeToLive">Time left to live before termination (relative to above)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		public static Task WriteTerminationSignalFileAsync(string filePath, string reason, DateTime terminateAt, TimeSpan timeToLive, CancellationToken cancellationToken)
		{
			StringBuilder sb = new(100);
			sb.Append("v1\n");
			sb.Append($"{timeToLive.TotalMilliseconds}\n");
			sb.Append($"{new DateTimeOffset(terminateAt).ToUnixTimeMilliseconds()}\n");
			sb.Append($"{reason}\n");
			return File.WriteAllTextAsync(filePath, sb.ToString(), cancellationToken);
		}

		public async Task<SessionResult> RunAsync(CancellationToken stoppingToken)
		{
			SessionResult result;
			try
			{
				result = await HandleSessionAsync(stoppingToken);
			}
			catch (OperationCanceledException ex) when (stoppingToken.IsCancellationRequested)
			{
				_logger.LogInformation(ex, "Execution cancelled");
				result = new SessionResult(SessionOutcome.Terminate, SessionReason.Cancelled);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while executing session. Restarting.");
				result = new SessionResult(SessionOutcome.BackOff, SessionReason.Failed);
			}

			while (_activeLeases.Count > 0)
			{
				try
				{
					_logger.LogInformation("Draining leases... ({NumLeases} remaining)", _activeLeases.Count);
					await DrainLeasesAsync(LeaseOutcomeReason.AgentSessionTerminated);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while draining leases. Agent may be in an inconsistent state.");
					await Task.Delay(TimeSpan.FromSeconds(30.0), CancellationToken.None);
				}
			}
			return result;
		}

		/// <summary>
		/// Drain and terminate any currently running leases
		/// </summary>
		/// <param name="reason">Reason for draining the leases</param>
		/// <param name="graceful">Whether to warn workload about upcoming lease termination via signal file</param>
		async Task DrainLeasesAsync(LeaseOutcomeReason reason, bool graceful = false)
		{
			if (graceful)
			{
				TimeSpan ttl = TimeSpan.FromSeconds(1);
				await WriteTerminationSignalFileAsync(_settings.CurrentValue.GetTerminationSignalFile().FullName, reason.ToString(), DateTime.UtcNow + ttl, ttl, CancellationToken.None);
				await Task.Delay(TimeSpan.FromSeconds(6)); // Allow workload some time to act on signal file
			}
			
			for (int idx = 0; idx < _activeLeases.Count; idx++)
			{
				LeaseHandler activeLease = _activeLeases[idx];
				if (activeLease.Result.IsCompleted)
				{
					_activeLeases.RemoveAt(idx--);
					_logger.LogInformation("Removed lease {LeaseId}", activeLease.Id);
				}
				else
				{
					_logger.LogInformation("Cancelling active lease {LeaseId}", activeLease.Id);
					activeLease.Cancel(reason);
				}
			}

			while (_activeLeases.Count > 0)
			{
				List<Task> tasks = _activeLeases.Select(x => (Task)x.Result).ToList();
				tasks.Add(Task.Delay(TimeSpan.FromMinutes(1.0)));
				await Task.WhenAny(tasks);

				for (int idx = 0; idx < _activeLeases.Count; idx++)
				{
					LeaseHandler activeLease = _activeLeases[idx];
					if (activeLease.Result.IsCompleted)
					{
						_activeLeases.RemoveAt(idx--);
						try
						{
							await activeLease.Result;
						}
						catch (OperationCanceledException)
						{
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Lease {LeaseId} threw an exception while terminating", activeLease.Id);
						}
						_logger.LogInformation("Lease {LeaseId} has completed", activeLease.Id);
						activeLease.Dispose();
					}
					else
					{
						_logger.LogInformation("Still waiting for lease {LeaseId} to terminate...", activeLease.Id);
					}
				}
			}
		}

		async Task<SessionResult> HandleSessionAsync(CancellationToken stoppingToken)
		{
			HordeRpc.HordeRpcClient hordeRpc = await _session.HordeClient.CreateGrpcClientAsync<HordeRpc.HordeRpcClient>(stoppingToken);

			// Terminate any remaining child processes from other instances
			ProcessUtils.TerminateProcesses(x => x.IsUnderDirectory(_session.WorkingDir), _logger, stoppingToken);

			// Track how many updates we get in 10 seconds. We'll start rate limiting this if it looks like we've got a problem that's causing us to spam the server.
			Stopwatch updateTimer = Stopwatch.StartNew();
			Queue<TimeSpan> updateTimes = new Queue<TimeSpan>();

			// Run a background task to update the capabilities of this agent
			await using BackgroundTask updateCapsTask = BackgroundTask.StartNew(ctx => UpdateCapabilitiesBackgroundAsync(_session.WorkingDir, ctx));

			// Run another background task to send telemetry data
			await using BackgroundTask telemetryTask = BackgroundTask.StartNew(ctx => SendTelemetryAsync(hordeRpc, ctx));

			// Loop until we're ready to exit
			Stopwatch updateCapabilitiesTimer = Stopwatch.StartNew();
			for (; ; )
			{
				Task waitTask = Task.WhenAny(_updateLeasesEvent.Task, _statusService.StatusChangedEvent.Task);

				// Flag for whether the service is stopping
				if (stoppingToken.IsCancellationRequested && _sessionResult == null)
				{
					_logger.LogInformation("Cancellation from token requested; setting session result to terminate.");
					_sessionResult = new SessionResult(SessionOutcome.Terminate, SessionReason.Cancelled);
				}

				bool stopping = false;
				if (_sessionResult != null)
				{
					_logger.LogInformation("Session termination requested (result: {Result})", _sessionResult.Outcome);
					stopping = true;
				}

				// Build the next update request
				RpcUpdateSessionRequest updateSessionRequest = new RpcUpdateSessionRequest();
				updateSessionRequest.AgentId = _session.AgentId.ToString();
				updateSessionRequest.SessionId = _session.SessionId.ToString();

				// Get the new the lease states. If a restart is requested and we have no active leases, signal to the server that we're stopping.
				lock (_lockObject)
				{
					foreach (LeaseHandler activeLease in _activeLeases)
					{
						updateSessionRequest.Leases.Add(new RpcLease(activeLease.RpcLease));
					}
					if (_sessionResult != null && _activeLeases.Count == 0)
					{
						stopping = true;
					}
				}

				// Get the new agent status to be reported back to server
				if (stopping)
				{
					// If stopping, ensure capabilities are updated one last time
					_capabilities = await _capabilitiesService.GetCapabilitiesAsync(_session.WorkingDir);
					updateSessionRequest.Status = RpcAgentStatus.Stopping;
				}
				else if (_unhealthy)
				{
					updateSessionRequest.Status = RpcAgentStatus.Unhealthy;
				}
				else if (_statusService.IsBusy)
				{
					updateSessionRequest.Status = RpcAgentStatus.Busy;
				}
				else
				{
					updateSessionRequest.Status = RpcAgentStatus.Ok;
				}

				// Update the capabilities whenever the background task has generated a new instance
				updateSessionRequest.Capabilities = Interlocked.Exchange(ref _capabilities, null);

				// Add working disk free space as a capability override (updated every UpdateSession for fresh data)
				DiskMetrics? workingDiskMetrics = _systemMetrics.GetWorkingDisk();
				if (workingDiskMetrics != null)
				{
					updateSessionRequest.CapabilityOverrides ??= new RpcAgentCapabilities();
					updateSessionRequest.CapabilityOverrides.Properties.Add($"{KnownPropertyNames.WorkingDiskFreeSpace}={workingDiskMetrics.FreeSpace}");
				}

				// Complete the wait task if we subsequently stop
				using (stopping ? (CancellationTokenRegistration?)null : stoppingToken.Register(() => _updateLeasesEvent.Set()))
				{
					// Update the state with the server
					RpcUpdateSessionResponse? updateSessionResponse = await UpdateSessionAsync(hordeRpc, updateSessionRequest, waitTask);

					lock (_lockObject)
					{
						// Now reconcile the local state to match what the server reports
						if (updateSessionResponse != null)
						{
							if (TerminateSessionAfterLease)
							{
								bool atLeastOneLeaseFinished = _activeLeases.Any(x => x.RpcLease.State is RpcLeaseState.Completed or RpcLeaseState.Cancelled);
								if (atLeastOneLeaseFinished || _activeLeases.Count == 0)
								{
									_logger.LogInformation("Session termination requested. At least one lease executed or no lease is active, proceeding...");
									_sessionResult = new SessionResult(SessionOutcome.Terminate, SessionReason.Completed);
								}
							}

							PoolIds = updateSessionResponse.PoolIds;

							// Remove any leases which have completed
							int numRemoved = _activeLeases.RemoveAll(x => (x.RpcLease.State == RpcLeaseState.Completed || x.RpcLease.State == RpcLeaseState.Cancelled) && !updateSessionResponse.Leases.Any(y => y.Id == x.RpcLease.Id && y.State != RpcLeaseState.Cancelled));
							NumLeasesCompleted += numRemoved;

							// Create any new leases and cancel any running leases
							foreach (RpcLease serverLease in updateSessionResponse.Leases)
							{
								if (serverLease.State == RpcLeaseState.Cancelled)
								{
									LeaseHandler? handler = _activeLeases.FirstOrDefault(x => x.RpcLease.Id == serverLease.Id);
									if (handler != null)
									{
										_logger.LogInformation("Cancelling lease {LeaseId}", serverLease.Id);
										handler.Cancel(LeaseOutcomeReason.ServerInitiated);
									}
								}
								if (serverLease.State == RpcLeaseState.Pending && !_activeLeases.Any(x => x.RpcLease.Id == serverLease.Id))
								{
									serverLease.State = RpcLeaseState.Active;

									_logger.LogInformation("Adding lease {LeaseId}", serverLease.Id);

									LeaseHandler leaseHandler = CreateLeaseHandler(serverLease);
									leaseHandler.Start(_session, _systemMetrics, _tracer, _logger, _leaseLoggerFactory);
									leaseHandler.Result.ContinueWith((Task<LeaseResult> task) =>
									{
										LeaseResult result = task.Result;
										if (result.SessionResult != null && _sessionResult == null)
										{
											_logger.LogInformation("Lease {LeaseId} is setting session result to {Result}", leaseHandler.Id, result.SessionResult.Outcome);
											_sessionResult = result.SessionResult;
										}
										OnLeaseFinished?.Invoke(serverLease, task.Result);
										_updateLeasesEvent.Set();
									}, TaskScheduler.Default);

									_activeLeases.Add(leaseHandler);
									OnLeaseStarted?.Invoke(serverLease);
								}
							}

							// Update the session result if we've transitioned to stopped
							if (updateSessionResponse.Status == RpcAgentStatus.Stopped)
							{
								SessionResult result = _sessionResult ?? new SessionResult(SessionOutcome.BackOff, SessionReason.Completed);
								_logger.LogInformation("Agent status is stopped; returning from session update loop with result {Result}", result);
								return result;
							}
						}
					}

					// Update the current status
					if (!_session.HordeClient.HasValidAccessToken())
					{
						_statusService.Set(false, _activeLeases.Count, "Attempting to connect to server...");
					}
					else if (_statusService.IsBusy)
					{
						_statusService.Set(true, 0, "Paused");

						if (_activeLeases.Count > 0)
						{
							_logger.LogInformation("Agent marked itself as busy. Draining any active leases to prevent them from using up local resources...");
							await DrainLeasesAsync(LeaseOutcomeReason.AgentBusy, true);
						}
					}
					else if (_activeLeases.Count == 0)
					{
						_statusService.Set(true, 0, "Waiting for work");
					}
					else
					{
						_statusService.Set(true, _activeLeases.Count, $"Executing {_activeLeases.Count} lease(s)");
					}
				}

				// Update the historical update times
				TimeSpan updateTime = updateTimer.Elapsed;
				while (updateTimes.TryPeek(out TimeSpan firstTime) && firstTime < updateTime - TimeSpan.FromMinutes(1.0))
				{
					updateTimes.Dequeue();
				}
				updateTimes.Enqueue(updateTime);

				// If we're updating too much, introduce an artificial delay
				if (updateTimes.Count > 60)
				{
					_logger.LogWarning("Agent is issuing large number of UpdateSession() calls. Delaying for 10 seconds.");
					await Task.Delay(TimeSpan.FromSeconds(10.0), stoppingToken);
				}
			}
		}

		LeaseHandler CreateLeaseHandler(RpcLease lease)
		{
			Any payload = lease.Payload;
			if (!_typeUrlToLeaseHandler.TryGetValue(payload.TypeUrl, out LeaseHandlerFactory? leaseHandlerFactory))
			{
				_logger.LogError("Invalid lease payload type ({PayloadType})", payload.TypeUrl);
				return new DefaultLeaseHandler(lease, new LeaseResult(LeaseOutcome.Failed, LeaseOutcomeReason.UnexpectedError));
			}
			return leaseHandlerFactory.CreateHandler(lease);
		}

		/// <summary>
		/// Wrapper for <see cref="UpdateSessionInternalAsync"/> which filters/logs exceptions
		/// </summary>
		/// <param name="hordeRpc">The RPC client connection</param>
		/// <param name="updateSessionRequest">The session update request</param>
		/// <param name="waitTask">Task which can be used to jump out of the update early</param>
		/// <returns>Response from the call</returns>
		async Task<RpcUpdateSessionResponse?> UpdateSessionAsync(HordeRpc.HordeRpcClient hordeRpc, RpcUpdateSessionRequest updateSessionRequest, Task waitTask)
		{
			const int MaxRetries = 8;
			const int BaseRetryDelayMs = 1000;
			const int MaxDelayMs = 20000;
			
			RpcUpdateSessionResponse? updateSessionResponse = null;
			try
			{
				updateSessionResponse = await UpdateSessionInternalAsync(hordeRpc, updateSessionRequest, waitTask);
				_updateSessionFailures = 0;
			}
			catch (Exception ex) when (ex is RpcException or HttpRequestException)
			{
				_updateSessionFailures++;
				if (_updateSessionFailures >= MaxRetries)
				{
					throw;
				}
				
				int retryDelayMs = Math.Min((int)(BaseRetryDelayMs * Math.Pow(2, _updateSessionFailures - 1)), MaxDelayMs);
				_logger.LogWarning(ex, "Error while updating session. Backing off {DelayMs} ms before next attempt... ({Attempt} of {MaxAttempts})", retryDelayMs, _updateSessionFailures, MaxRetries);
				await Task.Delay(retryDelayMs);
			}
			return updateSessionResponse;
		}

		/// <summary>
		/// Tries to update the session state on the server.
		/// 
		/// This operation is a little gnarly due to the fact that we want to long-poll for the result.
		/// Since we're doing the update via a gRPC call, the way to do that without using cancellation tokens is to keep the request stream open
		/// until we want to terminate the call (see https://github.com/grpc/grpc/issues/8277). In order to do that, we need to make a 
		/// bidirectional streaming call, even though we only expect one response/response.
		/// </summary>
		/// <param name="rpcClient">The RPC client</param>
		/// <param name="request">The session update request</param>
		/// <param name="waitTask">Task to use to terminate the wait</param>
		/// <returns>The response object</returns>
		async Task<RpcUpdateSessionResponse?> UpdateSessionInternalAsync(HordeRpc.HordeRpcClient rpcClient, RpcUpdateSessionRequest request, Task waitTask)
		{
			DateTime deadline = DateTime.UtcNow + TimeSpan.FromMinutes(2.0);
			using AsyncDuplexStreamingCall<RpcUpdateSessionRequest, RpcUpdateSessionResponse> call = rpcClient.UpdateSession(deadline: deadline);
			_logger.LogDebug("Updating session {SessionId} (Status={Status})", request.SessionId, request.Status);

			// Write the request to the server
			await call.RequestStream.WriteAsync(request);

			// Wait until the server responds or we need to trigger a new update
			Task<bool> moveNextAsync = call.ResponseStream.MoveNext();

			Task task = await Task.WhenAny(moveNextAsync, waitTask);
			if (task == waitTask)
			{
				_logger.LogDebug("Cancelling long poll from client side (new update)");
			}

			// Close the request stream to indicate that we're finished
			await call.RequestStream.CompleteAsync();

			// Wait for a response or a new update to come in, then close the request stream
			RpcUpdateSessionResponse? response = null;
			while (await moveNextAsync)
			{
				response = call.ResponseStream.Current;
				moveNextAsync = call.ResponseStream.MoveNext();
			}
			return response;
		}

		/// <summary>
		/// Background task that updates the capabilities of this agent
		/// </summary>
		async Task UpdateCapabilitiesBackgroundAsync(DirectoryReference workingDir, CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				await Task.Delay(TimeSpan.FromMinutes(5.0), cancellationToken);
				try
				{
					Interlocked.Exchange(ref _capabilities, await _capabilitiesService.GetCapabilitiesAsync(workingDir));
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Unable to query agent capabilities. Ignoring.");
				}
			}
		}

		/// <summary>
		/// Periodically send agent telemetry to the server
		/// </summary>
		async Task SendTelemetryAsync(HordeRpc.HordeRpcClient hordeRpc, CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				try
				{
					CpuMetrics? cpuMetrics = _systemMetrics.GetCpu();
					MemoryMetrics? memMetrics = _systemMetrics.GetMemory();

					DiskMetrics? workingDiskMetrics = _systemMetrics.GetWorkingDisk();
					DiskMetrics? systemDiskMetrics = _systemMetrics.GetSystemDisk();

					if (cpuMetrics != null || memMetrics != null || workingDiskMetrics != null || systemDiskMetrics != null)
					{
						RpcUploadTelemetryRequest request = new RpcUploadTelemetryRequest();
						request.AgentId = _session.AgentId.ToString();
						if (cpuMetrics != null)
						{
							request.UserCpu = cpuMetrics.User;
							request.SystemCpu = cpuMetrics.System;
							request.IdleCpu = cpuMetrics.Idle;
						}
						if (memMetrics != null)
						{
							request.TotalRam = memMetrics.Total / 1024;
							request.FreeRam = memMetrics.Available / 1024;
							request.UsedRam = memMetrics.Used / 1024;
						}
						if (workingDiskMetrics != null)
						{
							request.WorkingFreeDisk = (ulong)(workingDiskMetrics.FreeSpace / (1024 * 1024));
							request.WorkingTotalDisk = (ulong)(workingDiskMetrics.TotalSize / (1024 * 1024));
						}
						if (systemDiskMetrics != null)
						{
							request.SystemFreeDisk = (ulong)(systemDiskMetrics.FreeSpace / (1024 * 1024));
							request.SystemTotalDisk = (ulong)(systemDiskMetrics.TotalSize / (1024 * 1024));
						}

						await hordeRpc.UploadTelemetryAsync(request, cancellationToken: cancellationToken);
					}
				}
				catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Exception sending telemetry data: {Message}", ex.Message);
				}

				await Task.Delay(TimeSpan.FromSeconds(30.0), cancellationToken);
			}
		}
	}
}
