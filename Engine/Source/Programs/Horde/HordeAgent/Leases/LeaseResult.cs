// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using HordeAgent.Services;

namespace HordeAgent.Leases
{
	/// <summary>
	/// Result from executing a lease
	/// </summary>
	internal class LeaseResult
	{
		/// <summary>
		/// Outcome of the lease (whether it completed/failed due to an internal error, etc...)
		/// </summary>
		public LeaseOutcome Outcome { get; }

		/// <summary>
		/// Reason for the outcome of the lease
		/// </summary>
		public LeaseOutcomeReason OutcomeReason { get; }

		/// <summary>
		/// Output from executing the task
		/// </summary>
		public byte[]? Output { get; }

		/// <summary>
		/// If a lease wants to terminate the current session, this specifies the subsequent action to take.
		/// </summary>
		public SessionResult? SessionResult { get; }

		/// <summary>
		/// Static instance of a succesful result without a payload
		/// </summary>
		public static LeaseResult Success { get; } = new (LeaseOutcome.Success, LeaseOutcomeReason.None);

		/// <summary>
		/// Constructor
		/// </summary>
		public LeaseResult(LeaseOutcome outcome, LeaseOutcomeReason outcomeReason)
		{
			Outcome = outcome;
			OutcomeReason = outcomeReason;
		}

		/// <summary>
		/// Constructor for successful results
		/// </summary>
		/// <param name="output"></param>
		public LeaseResult(byte[]? output)
		{
			Outcome = LeaseOutcome.Success;
			Output = output;
		}

		/// <summary>
		/// Constructor for successful results that cause a session state change
		/// </summary>
		/// <param name="sessionResult">Session result</param>
		public LeaseResult(SessionResult sessionResult)
		{
			Outcome = LeaseOutcome.Success;
			SessionResult = sessionResult;
		}
	}
}
