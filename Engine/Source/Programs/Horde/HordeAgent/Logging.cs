// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using EpicGames.Core;
using HordeAgent.Utility;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Logging;
using Serilog;
using Serilog.Core;
using Serilog.Extensions.Logging;
using Serilog.Formatting.Json;
using Serilog.Sinks.SystemConsole.Themes;

namespace HordeAgent
{
	static class Logging
	{
		public static LoggingLevelSwitch LogLevelSwitch = new ();

		private class DatadogVersionLogEnricher : ILogEventEnricher
		{
			public void Enrich(Serilog.Events.LogEvent logEvent, ILogEventPropertyFactory propertyFactory)
			{
				logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.version", AgentApp.Version));
			}
		}

		public static ILoggerFactory CreateLoggerFactory(IConfiguration configuration)
		{
			Serilog.ILogger logger = CreateSerilogLogger(configuration);
			return new SerilogLoggerFactory(logger, true);
		}

		static Serilog.ILogger CreateSerilogLogger(IConfiguration configuration)
		{
			AgentSettings settings = AgentApp.BindSettings(configuration);
			DirectoryReference.CreateDirectory(settings.LogsDir);

			ConsoleTheme theme;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && Environment.OSVersion.Version < new Version(10, 0))
			{
				theme = SystemConsoleTheme.Literate;
			}
			else
			{
				theme = AnsiConsoleTheme.Code;
			}
			
			LoggerConfiguration loggerConfig = new LoggerConfiguration()
				.WriteTo.Console(outputTemplate: "[{Timestamp:HH:mm:ss} {Level:w3}] {Indent}{Message:l}{NewLine}{Exception}", theme: theme)
				.WriteTo.File(FileReference.Combine(settings.LogsDir, "Log-.txt").FullName, fileSizeLimitBytes: 50 * 1024 * 1024, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, retainedFileCountLimit: 10)
				.WriteTo.File(new JsonFormatter(renderMessage: true), FileReference.Combine(settings.LogsDir, "Log-.json").FullName, fileSizeLimitBytes: 50 * 1024 * 1024, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, retainedFileCountLimit: 10)
				.ReadFrom.Configuration(configuration)
				.MinimumLevel.ControlledBy(LogLevelSwitch)
				.Enrich.FromLogContext();
			
			if (settings.OpenTelemetry.EnableDatadogCompatibility)
			{
				loggerConfig = loggerConfig.Enrich.With<DatadogVersionLogEnricher>();
				loggerConfig = loggerConfig.Enrich.With<OpenTelemetryDatadogLogEnricher>();
			}
			
			return loggerConfig.CreateLogger();
		}

		public static ILoggerProvider CreateFileLoggerProvider(DirectoryReference baseDir, string name)
		{
			DirectoryReference.CreateDirectory(baseDir);

			Logger logger = new LoggerConfiguration()
				.WriteTo.File(FileReference.Combine(baseDir, $"{name}.txt").FullName)
				.WriteTo.File(new JsonFormatter(renderMessage: true), FileReference.Combine(baseDir, $"{name}.json").FullName)
				.Enrich.FromLogContext()
				.CreateLogger();

			return new SerilogLoggerProvider(logger, true);
		}
	}
}
