{
  "profiles": {
    "HordeAgent": {
      "commandName": "Project",
      "commandLineArgs": "service run",
      "environmentVariables": {
        "DOTNET_ENVIRONMENT": "Local"
      }
    }
  }
}