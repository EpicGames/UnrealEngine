// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Core;
using HordeAgent.Leases;
using HordeAgent.Utility;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Services;

/// <summary>
/// Describes state of local EC2 instance the agent is running on
/// </summary>
public enum Ec2InstanceState
{
	/// <summary>
	/// Unknown state
	/// Likely because of a failed HTTP request to local IMDS server
	/// </summary>
	Unknown,
	
	/// <summary>
	/// Normal state
	/// </summary>
	InService,

	/// <summary>
	/// Termination is caused by a spot interruption.
	/// </summary>
	TerminatingSpot,

	/// <summary>
	/// Termination is caused by the auto-scaling group
	/// Can be caused by capacity re-balancing, scale-ins etc.
	/// </summary>
	TerminatingAsg
}

internal class Ec2TerminationInfo(Ec2InstanceState state, bool isSpot, TimeSpan timeToLive, DateTime terminateAt, string reason)
{
	public Ec2InstanceState State { get; } = state;
	public bool IsSpot { get; } = isSpot;
	public TimeSpan TimeToLive { get; } = timeToLive;
	public DateTime TerminateAt { get; } = terminateAt;
	public string Reason { get; } = reason;
}

/// <summary>
/// Monitors the local EC2 instance lifecycle state. In particular, auto-scaling group and spot instance events.
/// </summary>
class AwsInstanceLifecycleService : BackgroundService
{
	/// <summary>
	/// Name of the HTTP client used for requests to IMDS
	/// </summary>
	public const string HttpClientName = "Horde.HttpAwsInstanceClient";
	
	public const string ImdsBaseUri = "http://169.254.169.254";
	public const string ImdsTokenPath = "/latest/api/token";
	public const string ImdsInstanceLifeCyclePath = "/latest/meta-data/instance-life-cycle";
	public const string ImdsSpotActionPath = "/latest/meta-data/spot/instance-action";
	public const string ImdsAsgTargetStatePath = "/latest/meta-data/autoscaling/target-lifecycle-state";
	public const string ImdsTokenHeader = "X-aws-ec2-metadata-token";
	public const string ImdsTokenTtlHeader = "X-aws-ec2-metadata-token-ttl-seconds";
	public const string ImdsTokenTtl = "21600"; // 6 hours
	
	private readonly IWorkerService _workerService;
	private readonly HttpClient _httpClient;
	private readonly IClock _clock;
	private readonly FileReference _terminationSignalFile;
	private readonly ILogger<AwsInstanceLifecycleService> _logger;
	private readonly TimeSpan _pollInterval = TimeSpan.FromSeconds(5);
	private string? _imdsToken;
	private DateTime? _imdsTokenExpireTime = DateTime.UnixEpoch;

	internal delegate Task TerminationWarningDelegate(Ec2TerminationInfo info, CancellationToken cancellationToken);
	internal delegate Task TerminationDelegate(Ec2TerminationInfo info, CancellationToken cancellationToken);
	internal TerminationWarningDelegate _terminationWarningCallback;
	internal TerminationDelegate _terminationCallback;

	/// <summary>
	/// Time to live for EC2 instance once a termination is detected coming from the auto-scaling group (ASG)
	/// In practice, this is dictated by the lifecycle hook set for the ASG.
	/// Set to 120 sec to mimic the TTL for spot interruption, leading to similar handling of both for now.
	/// </summary>
	internal TimeSpan _timeToLiveAsg = TimeSpan.FromSeconds(120);

	/// <summary>
	/// Time to live for EC2 instance once a spot interruption is detected. Strictly defined by AWS EC2.
	/// </summary>
	internal TimeSpan _timeToLiveSpot = TimeSpan.FromSeconds(120); // Strictly defined by AWS EC2

	/// <summary>
	/// Duration of the time-to-live to allocate towards shutting down the Horde agent and the machine itself.
	/// Example: if TTL is 120 seconds, 90 seconds will be reported in the termination warning.
	/// </summary>
	internal TimeSpan _terminationBufferTime = TimeSpan.FromSeconds(30);

	/// <summary>
	/// Constructor
	/// </summary>
	public AwsInstanceLifecycleService(IWorkerService workerService, HttpClient httpClient, IClock clock, IOptionsMonitor<AgentSettings> settings, ILogger<AwsInstanceLifecycleService> logger)
	{
		_workerService = workerService;
		_httpClient = httpClient;
		_httpClient.Timeout = TimeSpan.FromSeconds(10);
		_clock = clock;
		_terminationWarningCallback = OnTerminationWarningAsync;
		_terminationCallback = OnTerminationAsync;
		_logger = logger;
		_terminationSignalFile = settings.CurrentValue.GetTerminationSignalFile();
	}
	
	private async Task<HttpResponseMessage> GetMetadataAsync(string path, CancellationToken cancellationToken)
	{
		if (_imdsToken == null || _clock.UtcNow >= _imdsTokenExpireTime)
		{
			// Get a token
			using HttpRequestMessage tokenReq = new (HttpMethod.Put, new Uri($"{ImdsBaseUri}{ImdsTokenPath}"));
			tokenReq.Headers.Add(ImdsTokenTtlHeader, ImdsTokenTtl);
			using HttpResponseMessage tokenRes = await _httpClient.SendAsync(tokenReq, cancellationToken);
			tokenRes.EnsureSuccessStatusCode();
			_imdsToken = await tokenRes.Content.ReadAsStringAsync(cancellationToken);
			_imdsTokenExpireTime = _clock.UtcNow.AddSeconds(Convert.ToInt32(ImdsTokenTtl));
		}
		
		// Perform actual metadata request
		using HttpRequestMessage request = new (HttpMethod.Get, $"{ImdsBaseUri}{path}");
		request.Headers.Add(ImdsTokenHeader, _imdsToken);
		return await _httpClient.SendAsync(request, cancellationToken);
	}
	
	internal async Task<bool> IsSpotInstanceAsync(CancellationToken cancellationToken)
	{
		HttpResponseMessage res = await GetMetadataAsync(ImdsInstanceLifeCyclePath, cancellationToken);
		if (res.StatusCode == HttpStatusCode.OK)
		{
			return await res.Content.ReadAsStringAsync(cancellationToken) == "spot";
		}
		
		throw new Exception($"Request to IMDS server failed. Status={res.StatusCode}");
	}
	
	private async Task<Ec2InstanceState> GetStateAsync(CancellationToken cancellationToken)
	{
		try
		{
			HttpResponseMessage spotRes = await GetMetadataAsync(ImdsSpotActionPath, cancellationToken);
			if (spotRes.StatusCode == HttpStatusCode.OK)
			{
				return Ec2InstanceState.TerminatingSpot;
			}

			HttpResponseMessage asgRes = await GetMetadataAsync(ImdsAsgTargetStatePath, cancellationToken);
			string state = await asgRes.Content.ReadAsStringAsync(cancellationToken);
			return state.Equals("Terminated", StringComparison.OrdinalIgnoreCase)
				? Ec2InstanceState.TerminatingAsg
				: Ec2InstanceState.InService;
		}
		catch (Exception e)
		{
			// HTTP timeouts from the IMDS server can occur, in addition to unknown exceptions.
			// Unclear why timeouts are seen in the first place from the metadata server.
			// It should not be under load at all but it's possible an agent performing heavy work can affect this.
			_logger.LogWarning(e, "Request to IMDS server failed. Reason={Reason}", e.Message);
			return Ec2InstanceState.Unknown;
		}
	}
	
	private async Task<bool> IsImdsAvailableAsync(CancellationToken cancellationToken)
	{
		try
		{
			HttpResponseMessage res = await _httpClient.GetAsync(new Uri(ImdsBaseUri + "/"), cancellationToken);
			return res.StatusCode is HttpStatusCode.OK or HttpStatusCode.Unauthorized;
		}
		catch (Exception)
		{
			// Timed out or other error. Can safely assume the metadata server is not available.
			return false;
		}
	}

	internal async Task MonitorInstanceLifecycleAsync(CancellationToken cancellationToken)
	{
		if (!await IsImdsAvailableAsync(cancellationToken))
		{
			_logger.LogInformation("AWS EC2 metadata server (IMDS) not available. Will not monitor EC2 lifecycle state");
			return;
		}
		
		_logger.LogInformation("Monitoring EC2 instance lifecycle state...");
		while (!cancellationToken.IsCancellationRequested)
		{
			try
			{
				Ec2InstanceState state = await GetStateAsync(cancellationToken);
				if (state is Ec2InstanceState.TerminatingAsg or Ec2InstanceState.TerminatingSpot)
				{
					bool isSpot = await IsSpotInstanceAsync(cancellationToken);
					TimeSpan ttl = GetTimeToLive(state);
					_logger.LogInformation("EC2 instance is terminating. IsSpot={IsSpot} Reason={InstanceState} TimeToLive={Ttk} ms", isSpot, state, ttl.TotalMilliseconds);

					ttl -= _terminationBufferTime;
					ttl = ttl.Ticks >= 0 ? ttl : TimeSpan.Zero;
					DateTime terminateAt = DateTime.UtcNow + ttl;
					Ec2TerminationInfo info = new(state, isSpot, ttl, terminateAt, GetReason(state));

					await _terminationWarningCallback(info, cancellationToken);
					await Task.Delay(ttl, cancellationToken);
					await _terminationCallback(info, cancellationToken);
					return;
				}
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Unhandled exception during EC2 instance monitoring");
				await Task.Delay(TimeSpan.FromMinutes(1), cancellationToken);
			}

			await Task.Delay(_pollInterval, cancellationToken);
		}
	}

	/// <summary>
	/// Determine time to live for the current EC2 instance once a terminating state has been detected
	/// </summary>
	/// <param name="state">Current state</param>
	/// <returns>Time to live</returns>
	/// <exception cref="ArgumentException"></exception>
	private TimeSpan GetTimeToLive(Ec2InstanceState state)
	{
		return state switch
		{
			Ec2InstanceState.TerminatingAsg => _timeToLiveAsg,
			Ec2InstanceState.TerminatingSpot => _timeToLiveSpot,
			_ => throw new ArgumentException($"Invalid state {state}")
		};
	}

	private static string GetReason(Ec2InstanceState state)
	{
		return state switch
		{
			Ec2InstanceState.TerminatingAsg => "AWS EC2 ASG termination",
			Ec2InstanceState.TerminatingSpot => "AWS EC2 Spot interruption",
			_ => throw new ArgumentException($"Invalid state {state}")
		};
	}

	private async Task OnTerminationWarningAsync(Ec2TerminationInfo info, CancellationToken cancellationToken)
	{
		// Request shutdown of entire machine while waiting for the current lease executing to finish.
		// Agent's session loop will pick this up and notify the server, preventing new leases getting scheduled.
		_workerService.TerminateSessionAfterLease();

		// Create and write the termination signal file, containing the time-to-live for the EC2 instance.
		// Workloads executed by the agent that support this protocol can pick this up and prepare/clean up prior to termination
		await LeaseManager.WriteTerminationSignalFileAsync(_terminationSignalFile.FullName, info.Reason, info.TerminateAt, info.TimeToLive, cancellationToken);
	}

	private Task OnTerminationAsync(Ec2TerminationInfo info, CancellationToken cancellationToken)
	{
		if (info.IsSpot)
		{
			_logger.LogInformation("Shutting down");
			return Shutdown.ExecuteAsync(false, _logger, cancellationToken);
		}

		return Task.CompletedTask;
	}

	/// <inheritdoc/>
	protected override async Task ExecuteAsync(CancellationToken stoppingToken)
	{
		try
		{
			await MonitorInstanceLifecycleAsync(stoppingToken);
		}
		catch (OperationCanceledException) when (stoppingToken.IsCancellationRequested)
		{
			// Ignore any exceptions if cancellation has been requested
		}
	}
}