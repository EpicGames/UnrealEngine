// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Net;
using System.Net.Sockets;
using System.Runtime.InteropServices;
using System.Security.Principal;
using System.Text.RegularExpressions;
using System.Xml;
using Amazon.EC2;
using Amazon.EC2.Model;
using Amazon.Util;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Compute;
using HordeAgent.Utility;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using AddressFamily = System.Net.Sockets.AddressFamily;

namespace HordeAgent.Services
{
	internal interface ICapabilitiesService
	{
		/// <summary>
		/// Gets the hardware capabilities of this worker
		/// </summary>
		/// <param name="workingDir">Working directory for the agent</param>
		/// <returns>Worker object for advertising to the server</returns>
		Task<RpcAgentCapabilities> GetCapabilitiesAsync(DirectoryReference? workingDir);
	}
	
	/// <summary>
	/// Marker interface to separate system-specific capability services during dependency-injection
	/// </summary>
	internal interface ISystemCapabilities : ICapabilitiesService;
	
	/// <summary>
	/// Implements the message handling loop for an agent. Runs asynchronously until disposed.
	/// </summary>
	class CapabilitiesService : ICapabilitiesService
	{
		static readonly DateTimeOffset s_startTime = DateTimeOffset.Now;
		static readonly DateTimeOffset s_bootTime = DateTimeOffset.Now - TimeSpan.FromTicks(Environment.TickCount64 * TimeSpan.TicksPerMillisecond);

		readonly IOptionsMonitor<AgentSettings> _settings;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public CapabilitiesService(IOptionsMonitor<AgentSettings> settings, Tracer tracer, ILogger<CapabilitiesService> logger)
		{
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
		}
		
		/// <inheritdoc/>
		public async Task<RpcAgentCapabilities> GetCapabilitiesAsync(DirectoryReference? workingDir)
		{
			const int MaxAttempts = 5;
			const int RetryDelayMs = 5000;
			
			for (int attempt = 0; attempt <= MaxAttempts; attempt++)
			{
				try
				{
					return await GetCapabilitiesWithTimeoutMonitoringAsync(workingDir);
				}
				catch (Exception ex)
				{
					if (attempt >= MaxAttempts)
					{
						_logger.LogError("GetCapabilities failed {Attempts} of {MaxAttempts} attempts. Giving up.", attempt, MaxAttempts);
						throw;
					}
					
					_logger.LogWarning("GetCapabilities failed: {Message}. Waiting {DelayMs} ms before trying again...", ex.Message, RetryDelayMs);
					await Task.Delay(RetryDelayMs);
				}
			}
			
			throw new InvalidOperationException("Unexpected code path");
		}
		
		/// <summary>
		/// Gets the hardware capabilities of this worker
		/// </summary>
		/// <param name="workingDir">Working directory for the agent</param>
		/// <returns>Worker object for advertising to the server</returns>
		public async Task<RpcAgentCapabilities> GetCapabilitiesWithTimeoutMonitoringAsync(DirectoryReference? workingDir)
		{
			try
			{
				_logger.LogInformation("Querying agent capabilities... (may take up to 30 seconds)");
				Stopwatch timer = Stopwatch.StartNew();

				Task<RpcAgentCapabilities> task = GetCapabilitiesInternalAsync(workingDir);
				while (!task.IsCompleted)
				{
					Task delayTask = Task.Delay(TimeSpan.FromSeconds(30.0));
					if (await Task.WhenAny(task, delayTask) == delayTask)
					{
						_logger.LogWarning("GetCapabilitiesInternalAsync() has been running for {Time}", timer.Elapsed);
					}
				}
				
				LogLevel logLevel = timer.ElapsedMilliseconds >= 90000 ? LogLevel.Error : LogLevel.Information;
				_logger.Log(logLevel, "Agent capabilities queried in {Time} ms", timer.ElapsedMilliseconds);

				return await task;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to query agent properties: {Message}", ex.Message);
				throw;
			}
		}

		async Task<RpcAgentCapabilities> GetCapabilitiesInternalAsync(DirectoryReference? workingDir)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(CapabilitiesService)}.{nameof(GetCapabilitiesInternalAsync)}");
			ILogger logger = _logger;

			// Create the primary device
			RpcAgentCapabilities capabilities = new RpcAgentCapabilities();

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				// This should be injected as ISystemCapabilities but as failing WMI/CIM lookups must be addressed,
				// WindowsCapabilities is manually initialized for now.
				WindowsCapabilities winSystemCaps = new (_settings, _logger, _tracer);
				RpcAgentCapabilities winCaps = await winSystemCaps.GetCapabilitiesAsync(workingDir);
				capabilities.Properties.AddRange(winCaps.Properties);
				foreach (KeyValuePair<string, int> entry in winCaps.Resources)
				{
					capabilities.Resources[entry.Key] = entry.Value;
				}
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				using TelemetrySpan _ = _tracer.StartActiveSpan("GetLinuxCapabilities");
				capabilities.Properties.Add($"{KnownPropertyNames.Platform}=Linux");
				capabilities.Properties.Add($"{KnownPropertyNames.PlatformGroup}=Linux");
				capabilities.Properties.Add($"{KnownPropertyNames.PlatformGroup}=Unix");
				capabilities.Properties.Add($"{KnownPropertyNames.PlatformGroup}=Desktop");

				capabilities.Properties.Add($"{KnownPropertyNames.OsFamily}=Linux");
				capabilities.Properties.Add($"{KnownPropertyNames.OsFamilyCompatibility}=Linux");
				
				if (_settings.CurrentValue.WineExecutablePath != null)
				{
					capabilities.Properties.Add($"{KnownPropertyNames.OsFamilyCompatibility}=Windows");
					capabilities.Properties.Add($"{KnownPropertyNames.WineEnabled}=true");
				}
				
				// Parse the CPU info
				List<Dictionary<string, string>>? cpuRecords = await ReadLinuxHwPropsAsync("/proc/cpuinfo", logger);
				if (cpuRecords != null)
				{
					Dictionary<string, string> cpuNames = new Dictionary<string, string>(StringComparer.Ordinal);
					foreach (Dictionary<string, string> cpuRecord in cpuRecords)
					{
						if (cpuRecord.TryGetValue("physical id", out string? physicalId) && cpuRecord.TryGetValue("model name", out string? modelName))
						{
							cpuNames[physicalId] = modelName;
						}
					}

					Dictionary<string, int> nameToCount = new Dictionary<string, int>(StringComparer.Ordinal);
					foreach (string cpuNameTemp in cpuNames.Values)
					{
						nameToCount.TryGetValue(cpuNameTemp, out int count);
						nameToCount[cpuNameTemp] = count + 1;
					}

					HashSet<string> logicalCores = new HashSet<string>();
					HashSet<string> physicalCores = new HashSet<string>();
					foreach (Dictionary<string, string> cpuRecord in cpuRecords)
					{
						if (cpuRecord.TryGetValue("processor", out string? logicalCoreId))
						{
							logicalCores.Add(logicalCoreId);
						}
						if (cpuRecord.TryGetValue("core id", out string? physicalCoreId))
						{
							physicalCores.Add(physicalCoreId);
						}
					}
					
					string cpuName = cpuNames.Values.FirstOrDefault() ?? "Unknown CPU";
					AddCpuInfo(_settings.CurrentValue, capabilities, cpuName, logicalCores.Count, physicalCores.Count);
				}

				// Parse the RAM info
				List<Dictionary<string, string>>? memRecords = await ReadLinuxHwPropsAsync("/proc/meminfo", logger);
				if (memRecords != null && memRecords.Count > 0 && memRecords[0].TryGetValue("MemTotal", out string? memTotal))
				{
					Match match = Regex.Match(memTotal, @"(\d+)\s+kB");
					if (match.Success)
					{
						long totalCapacity = Int64.Parse(match.Groups[1].Value) * 1024;
						int ramGb = (int)(totalCapacity / (1024 * 1024 * 1024));
						AddMemInfo(_settings.CurrentValue, capabilities, ramGb);
					}
				}

				// Add session information
				capabilities.Properties.Add($"User={Environment.UserName}");
				capabilities.Properties.Add($"Domain={Environment.UserDomainName}");
				capabilities.Properties.Add($"Interactive={Environment.UserInteractive}");
				capabilities.Properties.Add($"Elevated={IsUserAdministrator()}");
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				using TelemetrySpan _ = _tracer.StartActiveSpan("GetMacCapabilities");
				capabilities.Properties.Add($"{KnownPropertyNames.Platform}=Mac");
				capabilities.Properties.Add($"{KnownPropertyNames.PlatformGroup}=Apple");
				capabilities.Properties.Add($"{KnownPropertyNames.PlatformGroup}=Desktop");

				capabilities.Properties.Add($"{KnownPropertyNames.OsFamily}=MacOS");
				capabilities.Properties.Add($"{KnownPropertyNames.OsFamilyCompatibility}=MacOS");

				string output;
				using (Process process = new Process())
				{
					process.StartInfo.FileName = "system_profiler";
					process.StartInfo.Arguments = "SPHardwareDataType SPSoftwareDataType SPDisplaysDataType -xml -timeout 15";
					process.StartInfo.CreateNoWindow = true;
					process.StartInfo.RedirectStandardOutput = true;
					process.StartInfo.RedirectStandardInput = false;
					process.StartInfo.UseShellExecute = false;
					process.Start();

					output = await process.StandardOutput.ReadToEndAsync();
				}

				XmlDocument xml = new XmlDocument();
				xml.LoadXml(output);

				XmlNode? hardwareNode = xml.SelectSingleNode("/plist/array/dict/key[. = '_dataType']/following-sibling::string[. = 'SPHardwareDataType']/../key[. = '_items']/following-sibling::array/dict");
				if (hardwareNode != null)
				{
					XmlNode? model = hardwareNode.SelectSingleNode("key[. = 'machine_model']/following-sibling::string");
					if (model != null)
					{
						capabilities.Properties.Add($"Model={model.InnerText}");
					}

					if (RuntimeInformation.OSArchitecture == Architecture.Arm64)
					{
						XmlNode? chipTypeNode = hardwareNode.SelectSingleNode("key[. = 'chip_type']/following-sibling::string");
						if (chipTypeNode != null)
						{
							capabilities.Properties.Add($"CPU={chipTypeNode.InnerText}");
						}
					}
					else
					{
						XmlNode? cpuTypeNode = hardwareNode.SelectSingleNode("key[. = 'cpu_type']/following-sibling::string");
						XmlNode? cpuSpeedNode = hardwareNode.SelectSingleNode("key[. = 'current_processor_speed']/following-sibling::string");
						XmlNode? cpuPackagesNode = hardwareNode.SelectSingleNode("key[. = 'packages']/following-sibling::integer");
						if (cpuTypeNode != null && cpuSpeedNode != null && cpuPackagesNode != null)
						{
							capabilities.Properties.Add((cpuPackagesNode.InnerText != "1") ? $"CPU={cpuPackagesNode.InnerText} x {cpuTypeNode.InnerText} @ {cpuSpeedNode.InnerText}" : $"CPU={cpuTypeNode.InnerText} @ {cpuSpeedNode.InnerText}");
						}
					}

					int numLogicalCores = Environment.ProcessorCount;
					if (_settings.CurrentValue.CpuCount is > 0)
					{
						numLogicalCores = Math.Min(numLogicalCores, _settings.CurrentValue.CpuCount.Value);
					}
					numLogicalCores = (int)(numLogicalCores * _settings.CurrentValue.CpuMultiplier);
					capabilities.Resources.Add(KnownResourceNames.LogicalCores, numLogicalCores);
					capabilities.Properties.Add($"{KnownResourceNames.LogicalCores}={numLogicalCores}");

					if (RuntimeInformation.OSArchitecture == Architecture.Arm64)
					{
						XmlNode? cpuCountNode = hardwareNode.SelectSingleNode("key[. = 'number_processors']/following-sibling::string");
						if (cpuCountNode != null)
						{
							string[] parts = cpuCountNode.InnerText.Split([':'], StringSplitOptions.RemoveEmptyEntries);
							if (parts.Length == 3)
							{
								if (Int32.TryParse(parts[1], out int performanceCores) && Int32.TryParse(parts[2], out int efficiencyCores))
								{
									int physicalCores = performanceCores + efficiencyCores;
									capabilities.Properties.Add($"PhysicalCores={physicalCores}");
									capabilities.Properties.Add($"PerformanceCores={performanceCores}");
									capabilities.Properties.Add($"EfficiencyCores={efficiencyCores}");
								}
							}
						}
					}
					else
					{
						XmlNode? cpuCountNode = hardwareNode.SelectSingleNode("key[. = 'number_processors']/following-sibling::integer");
						if (cpuCountNode != null)
						{
							capabilities.Properties.Add($"PhysicalCores={cpuCountNode.InnerText}");
						}
					}

					XmlNode? memoryNode = hardwareNode.SelectSingleNode("key[. = 'physical_memory']/following-sibling::string");
					if (memoryNode != null)
					{
						string[] parts = memoryNode.InnerText.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);
						if (parts.Length == 2 && parts[1] == "GB" && Int32.TryParse(parts[0], out int ramGb))
						{
							AddMemInfo(_settings.CurrentValue, capabilities, ramGb);
						}
					}
				}
				XmlNode? displayNode = xml.SelectSingleNode("/plist/array/dict/key[. = '_dataType']/following-sibling::string[. = 'SPDisplaysDataType']/../key[. = '_items']/following-sibling::array/dict");
				if (displayNode != null)
				{
					XmlNode? gpuTypeNode = displayNode.SelectSingleNode("key[. = 'sppci_model']/following-sibling::string");
					XmlNode? gpuCountNode = displayNode.SelectSingleNode("key[. = 'sppci_cores']/following-sibling::string");
					if (gpuTypeNode != null)
					{
						capabilities.Properties.Add(gpuCountNode != null
							? $"GPU={gpuTypeNode.InnerText} {gpuCountNode.InnerText}-cores"
							: $"GPU={gpuTypeNode.InnerText}");
					}
				}

				XmlNode? softwareNode = xml.SelectSingleNode("/plist/array/dict/key[. = '_dataType']/following-sibling::string[. = 'SPSoftwareDataType']/../key[. = '_items']/following-sibling::array/dict");
				if (softwareNode != null)
				{
					XmlNode? osVersionNode = softwareNode.SelectSingleNode("key[. = 'os_version']/following-sibling::string");
					if (osVersionNode != null)
					{
						capabilities.Properties.Add($"OSDistribution={osVersionNode.InnerText}");
					}

					XmlNode? kernelVersionNode = softwareNode.SelectSingleNode("key[. = 'kernel_version']/following-sibling::string");
					if (kernelVersionNode != null)
					{
						capabilities.Properties.Add($"OSKernelVersion={kernelVersionNode.InnerText}");
					}
				}

				// Add session information
				capabilities.Properties.Add($"User={Environment.UserName}");
				capabilities.Properties.Add($"Domain={Environment.UserDomainName}");
				capabilities.Properties.Add($"Interactive={Environment.UserInteractive}");
				capabilities.Properties.Add($"Elevated={IsUserAdministrator()}");
			}

			// Add EC2 properties if needed
			if (_settings.CurrentValue.EnableAwsEc2Support)
			{
				await AddAwsPropertiesAsync(capabilities.Properties, logger);
			}

			// Get the IP addresses
			try
			{
				using TelemetrySpan _ = _tracer.StartActiveSpan("ResolveIp");
				using CancellationTokenSource dnsCts = new(3000);
				IPHostEntry entry = await Dns.GetHostEntryAsync(Dns.GetHostName(), dnsCts.Token);
				foreach (IPAddress address in entry.AddressList)
				{
					if (address.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork)
					{
						capabilities.Properties.Add($"Ipv4={address}");
					}
					else if (address.AddressFamily == System.Net.Sockets.AddressFamily.InterNetworkV6)
					{
						capabilities.Properties.Add($"Ipv6={address}");
					}
				}
			}
			catch (Exception ex)
			{
				logger.LogWarning(ex, "Unable to get local IP addresses");
			}
			
			IPAddress? ip;
			if (_settings.CurrentValue.ComputeIp != null)
			{
				ip = IPAddress.Parse(_settings.CurrentValue.ComputeIp);
			}
			else
			{
				ip = await GetLocalIpAddressAsync(_settings.CurrentValue.GetCurrentServerProfile().Url.Host);
			}
			 
			if (ip == null)
			{
				logger.LogWarning("Unable to get IP address for incoming compute requests");
			}

			// Add the compute configuration
			if (ip != null && _settings.CurrentValue.ComputePort != 0)
			{
				capabilities.Properties.Add($"ComputeIp={ip}");
				capabilities.Properties.Add($"ComputePort={_settings.CurrentValue.ComputePort}");
			}

			// Get the time that the machine booted
			capabilities.Properties.Add($"BootTime={s_bootTime}");
			capabilities.Properties.Add($"StartTime={s_startTime}");

			// Add system disk info
			try
			{
				string? systemDriveName;
				systemDriveName = RuntimeInformation.IsOSPlatform(OSPlatform.Windows)
					? Path.GetPathRoot(Environment.SystemDirectory) // Windows (C:\)
					: "/"; // *nix

				if (systemDriveName != null)
				{
					DriveInfo systemInfo = new DriveInfo(systemDriveName);
					capabilities.Properties.Add($"{KnownPropertyNames.SystemDiskFreeSpace}={systemInfo.AvailableFreeSpace}");
					capabilities.Properties.Add($"{KnownPropertyNames.SystemDiskTotalSize}={systemInfo.TotalSize}");
				}
			}
			catch (Exception ex)
			{
				logger.LogWarning(ex, "Unable to query system disk info");
			}

			// Add information about the current session
			if (workingDir != null)
			{
				// Add disk info based on platform
				string? driveName;
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
				{
					driveName = Path.GetPathRoot(workingDir.FullName);
				}
				else
				{
					driveName = workingDir.FullName;
				}

				if (driveName != null)
				{
					try
					{
						DriveInfo info = new DriveInfo(driveName);
						capabilities.Properties.Add($"{KnownPropertyNames.WorkingDiskFreeSpace}={info.AvailableFreeSpace}");
						capabilities.Properties.Add($"{KnownPropertyNames.WorkingDiskTotalSize}={info.TotalSize}");
					}
					catch (Exception ex)
					{
						logger.LogWarning(ex, "Unable to query disk info for path '{DriveName}'", driveName);
					}
				}
				capabilities.Properties.Add($"WorkingDir={workingDir}");
			}

			// Add any horde. env vars for custom properties.
			IEnumerable<string> envVars = Environment.GetEnvironmentVariables().Keys.Cast<string>();
			foreach (string envVar in envVars.Where(x => x.StartsWith("horde.", StringComparison.InvariantCultureIgnoreCase)))
			{
				capabilities.Properties.Add($"{envVar}={Environment.GetEnvironmentVariable(envVar)}");
			}

			// Add the max supported compute protocol version
			capabilities.Properties.Add($"{KnownPropertyNames.ComputeProtocol}={(int)ComputeProtocol.Latest}");

			// Whether the agent is packaged as a self-contained .NET app
			// Used during the transition period over from multi-platform, non-self-contained agent packages.
			capabilities.Properties.Add($"SelfContained={AgentApp.IsSelfContained}");
			capabilities.Properties.Add($"DotNetFramework={RuntimeInformation.FrameworkDescription}");
#pragma warning disable CA1308 // Normalize strings to uppercase
			capabilities.Properties.Add($"ProcessArch={RuntimeInformation.ProcessArchitecture.ToString().ToLowerInvariant()}");
			capabilities.Properties.Add($"OsArch={RuntimeInformation.OSArchitecture.ToString().ToLowerInvariant()}");
#pragma warning restore CA1308
			
			if (VersionInfo.ToolId != null)
			{
				capabilities.Properties.Add($"{KnownPropertyNames.ToolId}={VersionInfo.ToolId}");
			}

			// Add any additional properties from the config file
			capabilities.Properties.AddRange(_settings.CurrentValue.Properties.Select(kvp => $"{kvp.Key}={kvp.Value}"));
			return capabilities;
		}
		
		/// <summary>
		/// Determine if the current user is a running with elevated permissions on Windows (UAC)
		/// </summary>
		public static bool IsUserAdministrator()
		{
			if (!OperatingSystem.IsWindows())
			{
				return false;
			}

			try
			{
				using WindowsIdentity identity = WindowsIdentity.GetCurrent();
				WindowsPrincipal principal = new (identity);
				return principal.IsInRole(WindowsBuiltInRole.Administrator);
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Resolve local IP address of agent
		///
		/// A machine can have multiple valid IP addresses, but not all suitable for accepting incoming traffic.
		/// By establishing a socket to a well-known host on a relevant network, a better guess can be made.  
		/// </summary>
		/// <param name="hostname">A hostname to test against</param>
		/// <param name="timeoutMs">Max time to wait for a connect, in milliseconds</param>
		/// <returns>Local IP address of this machine</returns>
		private async Task<IPAddress?> GetLocalIpAddressAsync(string hostname, int timeoutMs = 2000)
		{
			using TelemetrySpan _ = _tracer.StartActiveSpan(nameof(GetLocalIpAddressAsync));
			try
			{
				using Socket socket = new(AddressFamily.InterNetwork, SocketType.Dgram, ProtocolType.IP);
				using CancellationTokenSource cts = new(timeoutMs);
				// Port here is irrelevant as merely trying to connect is enough to get the local endpoint IP
				await socket.ConnectAsync(hostname, 65530, cts.Token);
				return (socket.LocalEndPoint as IPEndPoint)?.Address;
			}
			catch (SocketException)
			{
				return null;
			}
			catch (TaskCanceledException)
			{
				return null;
			}
		}

		public static void AddCpuInfo(AgentSettings settings, RpcAgentCapabilities capabilities, string cpuName, int numLogicalCores, int numPhysicalCores)
		{
			capabilities.Properties.Add($"CPU={cpuName}");

			// Agent settings can artificially limit number of available cores
			if (settings.CpuCount is > 0)
			{
				numLogicalCores = Math.Min(numLogicalCores, settings.CpuCount.Value);
			}
			numLogicalCores = (int)(numLogicalCores * settings.CpuMultiplier);

			if (numLogicalCores > 0)
			{
				capabilities.Resources.Add(KnownResourceNames.LogicalCores, numLogicalCores);
				capabilities.Properties.Add($"{KnownResourceNames.LogicalCores}={numLogicalCores}");
			}

			if (numPhysicalCores > 0)
			{
				capabilities.Properties.Add($"{KnownResourceNames.PhysicalCores}={numPhysicalCores}");
			}
		}
 
		public static void AddMemInfo(AgentSettings settings, RpcAgentCapabilities capabilities, int ramGb)
		{
			// Agent settings can artificially limit available RAM
			int adjustedRamGb = settings.RamGb is > 0 ? settings.RamGb.Value : ramGb;
			capabilities.Resources.Add(KnownResourceNames.Ram, adjustedRamGb);
			capabilities.Properties.Add($"{KnownResourceNames.Ram}={adjustedRamGb}");
		}

		static async Task<List<Dictionary<string, string>>?> ReadLinuxHwPropsAsync(string fileName, ILogger logger)
		{
			List<Dictionary<string, string>>? records = null;
			if (File.Exists(fileName))
			{
				records = new List<Dictionary<string, string>>();
				using (StreamReader reader = new StreamReader(fileName))
				{
					Dictionary<string, string> record = new Dictionary<string, string>(StringComparer.Ordinal);

					string? line;
					while ((line = await reader.ReadLineAsync()) != null)
					{
						int idx = line.IndexOf(':', StringComparison.Ordinal);
						if (idx == -1)
						{
							if (record.Count > 0)
							{
								records.Add(record);
								record = new Dictionary<string, string>(StringComparer.Ordinal);
							}
						}
						else
						{
							string key = line.Substring(0, idx).Trim();
							string value = line.Substring(idx + 1).Trim();

							if (record.TryGetValue(key, out string? prevValue))
							{
								logger.LogWarning("Multiple entries for {Key} in {File} (was '{Prev}', now '{Next}')", key, fileName, prevValue, value);
							}
							else
							{
								record.Add(key, value);
							}
						}
					}

					if (record.Count > 0)
					{
						records.Add(record);
					}
				}
			}
			return records;
		}

		async Task AddAwsPropertiesAsync(IList<string> properties, ILogger logger)
		{
			try
			{
				await AddAwsPropertiesInternalAsync(properties, logger);
			}
			catch (Exception ex)
			{
				_logger.LogDebug(ex, "Exception while querying EC2 metadata: {Message}", ex.Message);
			}
		}

		async Task AddAwsPropertiesInternalAsync(IList<string> properties, ILogger logger)
		{
			using TelemetrySpan _ = _tracer.StartActiveSpan(nameof(AddAwsPropertiesInternalAsync));
			if (EC2InstanceMetadata.IdentityDocument != null)
			{
				properties.Add("EC2=1");
				AddAwsProperty("aws-instance-id", "/instance-id", properties);
				AddAwsProperty("aws-instance-type", "/instance-type", properties);
				AddAwsProperty("aws-region", "/region", properties);

				try
				{
					using (AmazonEC2Client client = new AmazonEC2Client())
					{
						DescribeTagsRequest request = new DescribeTagsRequest();
						request.Filters = new List<Filter>();
						request.Filters.Add(new Filter("resource-id", new List<string> { EC2InstanceMetadata.InstanceId }));

						DescribeTagsResponse response = await client.DescribeTagsAsync(request);
						foreach (TagDescription tag in response.Tags)
						{
							properties.Add($"aws-tag={tag.Key}:{tag.Value}");
						}
					}
				}
				catch (Exception ex)
				{
					logger.LogDebug(ex, "Unable to query EC2 tags.");
				}
			}
		}

		static void AddAwsProperty(string name, string awsKey, IList<string> properties)
		{
			string? value = EC2InstanceMetadata.GetData(awsKey);
			if (value != null)
			{
				properties.Add($"{name}={value}");
			}
		}
	}
}
