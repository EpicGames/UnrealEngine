// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Sockets;
using EpicGames.Core;
using EpicGames.Horde.Compute.Clients;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Services
{
	/// <summary>
	/// Service which listens for incoming connections that can be assigned for remote execution leases
	/// </summary>
	class ComputeListenerService : IHostedService, IDisposable
	{
		class WaitingClient : IDisposable
		{
			TcpClient? _tcpClient;
			readonly TaskCompletionSource _takenTaskSource = new TaskCompletionSource(TaskCreationOptions.RunContinuationsAsynchronously);

			public WaitingClient(TcpClient tcpClient)
			{
				_tcpClient = tcpClient;
			}

			public TcpClient? TakeClient()
			{
				TcpClient? tcpClient = _tcpClient;
				if (Interlocked.CompareExchange(ref _tcpClient, null, tcpClient) == tcpClient)
				{
					_takenTaskSource.SetResult();
					return tcpClient;
				}
				return null;
			}

			public Task WaitForTransferAsync() => _takenTaskSource.Task;

			public void Dispose()
			{
				_tcpClient?.Dispose();
			}
		}

		readonly IOptionsMonitor<AgentSettings> _settings;
		readonly ILogger _logger;

		Task _serverTask = Task.CompletedTask;
		readonly CancellationTokenSource _cancellationSource = new CancellationTokenSource();

		readonly Lock _lockObject = new Lock();
		readonly Dictionary<ByteString, WaitingClient> _waitingClients = new Dictionary<ByteString, WaitingClient>();
		readonly Dictionary<ByteString, TaskCompletionSource<TcpClient?>> _waitingLeases = new Dictionary<ByteString, TaskCompletionSource<TcpClient?>>();

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeListenerService(IOptionsMonitor<AgentSettings> settings, ILogger<ComputeListenerService> logger)
		{
			_settings = settings;
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_cancellationSource.Dispose();
		}

		/// <summary>
		/// Waits for a client to connect with the given connection id
		/// </summary>
		/// <param name="nonce">Unique id for the connection</param>
		/// <param name="timeout">Timeout for the wait</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>The connected client</returns>
		public async Task<TcpClient?> WaitForClientAsync(ByteString nonce, TimeSpan timeout, CancellationToken cancellationToken)
		{
			// Check if there's already a client ready, and register that we're waiting if not
			TaskCompletionSource<TcpClient?> waitingLease;
			lock (_lockObject)
			{
				if (_waitingClients.TryGetValue(nonce, out WaitingClient? waitingClientInfo))
				{
					return waitingClientInfo.TakeClient();
				}

				waitingLease = new TaskCompletionSource<TcpClient?>(TaskCreationOptions.RunContinuationsAsynchronously);
				_waitingLeases.Add(nonce, waitingLease);
			}

			// Wait until we get a connection or we reach the timeout
			try
			{
				Task waitTask = Task.Delay(timeout, cancellationToken);
				await Task.WhenAny(waitTask, waitingLease!.Task);
			}
			finally
			{
				lock (_lockObject)
				{
					_waitingLeases.Remove(nonce);
				}
			}

			// Try to return the client
			waitingLease.TrySetResult(null);
			return await waitingLease.Task;
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			if (_settings.CurrentValue.ComputePort != 0)
			{
				Start(IPAddress.Any, _settings.CurrentValue.ComputePort);
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Starts listening for connections on the given address and port. Primarily for testing; the hosted service will use settings for this from the configuration file.
		/// </summary>
		/// <param name="address">Address to listen on</param>
		/// <param name="port">Port to listen on</param>
		public void Start(IPAddress address, int port)
		{
			_serverTask = RunTcpListenerAsync(address, port, _cancellationSource.Token);
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cancellationSource.CancelAsync();
			await _serverTask.IgnoreCanceledExceptionsAsync();
		}

		async Task RunTcpListenerAsync(IPAddress address, int port, CancellationToken cancellationToken)
		{
			TcpListener listener = new TcpListener(address, port);
			listener.Start();

			List<Task> tasks = new List<Task>();
			try
			{
				for (; ; )
				{
					TcpClient tcpClient = await listener.AcceptTcpClientAsync(cancellationToken);
					_logger.LogInformation("Received connection from {Remote}", tcpClient.Client.RemoteEndPoint);

					Task task = WaitForLeaseAsync(tcpClient, cancellationToken);
					tasks.Add(task);
				}
			}
			finally
			{
				await Task.WhenAll(tasks);
				listener.Stop();
			}
		}

		async Task WaitForLeaseAsync(TcpClient tcpClient, CancellationToken cancellationToken)
		{
			using WaitingClient waitingClient = new WaitingClient(tcpClient);
			try
			{
				Socket socket = tcpClient.Client;

				// Read the nonce for the connection
				byte[] nonceBuffer = new byte[ServerComputeClient.NonceLength];
				if (!await socket.TryReceiveMessageAsync(nonceBuffer, SocketFlags.None, cancellationToken))
				{
					_logger.LogInformation("No nonce received from {Remote}; closing connection.", tcpClient.Client.RemoteEndPoint);
					return;
				}

				ByteString nonce = new ByteString(nonceBuffer);
				_logger.LogInformation("Incoming connection with nonce {Nonce}", nonce);

				// Register the socket
				try
				{
					// Check if there's already a waiting connection, and add one if not
					TaskCompletionSource<TcpClient?>? waitingLease = null;
					lock (_lockObject)
					{
						if (!_waitingLeases.TryGetValue(nonce, out waitingLease))
						{
							_waitingClients.Add(nonce, waitingClient);
						}
					}

					// Offer ownership of the socket to any interested compute lease, or dispose of the connection if it's already closed.
					if (waitingLease != null)
					{
						if (waitingLease.TrySetResult(tcpClient))
						{
							_ = waitingClient.TakeClient();
						}
						return;
					}

					// Wait 2m for it to be claimed
					Task delayTask = Task.Delay(TimeSpan.FromMinutes(2.0), cancellationToken);
					await Task.WhenAny(waitingClient!.WaitForTransferAsync(), delayTask);
				}
				finally
				{
					lock (_lockObject)
					{
						_waitingClients.Remove(nonce);
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while processing compute task: {Message}", ex.Message);
			}
		}
	}
}
