// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http.Json;
using System.Net.Security;
using EpicGames.Horde.Server;
using Grpc.Core;
using Grpc.Core.Interceptors;
using Grpc.Net.Client;
using Grpc.Net.Client.Configuration;
using HordeAgent.Utility;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Services
{
	/// <summary>
	/// Service which creates a configured Grpc channel
	/// </summary>
	class GrpcService
	{
		private readonly IOptionsMonitor<AgentSettings> _settings;
		private readonly ServerProfile _serverProfile;
		private readonly ILogger _logger;
		private readonly ILoggerFactory _loggerFactory;

		/// <summary>
		/// The current server profile
		/// </summary>
		public ServerProfile ServerProfile => _serverProfile;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="settings"></param>
		/// <param name="logger"></param>
		/// <param name="loggerFactory"></param>
		public GrpcService(IOptionsMonitor<AgentSettings> settings, ILogger<GrpcService> logger, ILoggerFactory loggerFactory)
		{
			_settings = settings;
			_serverProfile = settings.CurrentValue.GetCurrentServerProfile();
			_logger = logger;
			_loggerFactory = loggerFactory;
		}

		/// <summary>
		/// Create a GRPC channel with the given bearer token
		/// </summary>
		/// <returns>New grpc channel</returns>
		public async Task<GrpcChannel> CreateGrpcChannelAsync(string? bearerToken, CancellationToken cancellationToken)
		{
			Uri serverUri = _serverProfile.Url;
			bool useInsecureConnection = serverUri.Scheme.Equals("http", StringComparison.Ordinal);

			// Get the server URL for gRPC traffic. If we're using an unencrpyted connection we need to use a different port for http/2, so 
			// send a http1 request to the server to query it.
			if (useInsecureConnection)
			{
				_logger.LogInformation("Querying server {BaseUrl} for rpc port", serverUri);
				using (HttpClient httpClient = new HttpClient())
				{
					httpClient.DefaultRequestHeaders.Add("Accept", "application/json");
					httpClient.Timeout = TimeSpan.FromSeconds(210); // Need to make sure this doesn't cancel any long running gRPC streaming calls (eg. session update)
					using (HttpResponseMessage response = await httpClient.GetAsync(new Uri(serverUri, "api/v1/server/ports"), cancellationToken))
					{
						GetPortsResponse? ports = await response.Content.ReadFromJsonAsync<GetPortsResponse>(AgentApp.DefaultJsonSerializerOptions, cancellationToken);
						if (ports != null && ports.UnencryptedHttp2.HasValue && ports.UnencryptedHttp2.Value != 0)
						{
							UriBuilder builder = new UriBuilder(serverUri);
							builder.Port = ports.UnencryptedHttp2.Value;
							serverUri = builder.Uri;
						}
					}
				}
			}

#pragma warning disable CA2000 // Dispose objects before losing scope
			// HTTP handler is disposed by GrpcChannel below
			SocketsHttpHandler httpHandler = new SocketsHttpHandler()
			{
				SslOptions = new SslClientAuthenticationOptions
				{
					RemoteCertificateValidationCallback = (sender, cert, chain, errors) => CertificateHelper.CertificateValidationCallBack(_logger, sender, cert, chain, errors, _serverProfile)
				}
			};
#pragma warning restore CA2000 // Dispose objects before losing scope

			// Create options for the new channel
			GrpcChannelOptions channelOptions = new GrpcChannelOptions();
			channelOptions.MaxReceiveMessageSize = 1024 * 1024 * 1024; // 1 GB 		// Required payloads coming from CAS service can be large
			channelOptions.MaxSendMessageSize = 1024 * 1024 * 1024; // 1 GB
			channelOptions.LoggerFactory = _loggerFactory;
			channelOptions.HttpHandler = httpHandler;
			channelOptions.DisposeHttpClient = true;
			channelOptions.ServiceConfig = new ServiceConfig();
			channelOptions.ServiceConfig.MethodConfigs.Add(new MethodConfig
			{
				Names = { MethodName.Default },
				RetryPolicy = new RetryPolicy
				{
					MaxAttempts = 3,
					InitialBackoff = TimeSpan.FromSeconds(1),
					MaxBackoff = TimeSpan.FromSeconds(10),
					BackoffMultiplier = 2.0,
					RetryableStatusCodes = { StatusCode.Unavailable },
				}
			});

			// Configure requests to send the bearer token
			if (!String.IsNullOrEmpty(bearerToken))
			{
				CallCredentials callCredentials = CallCredentials.FromInterceptor((context, metadata) =>
				{
					metadata.Add("Authorization", $"Bearer {bearerToken}");
					return Task.CompletedTask;
				});

				if (useInsecureConnection)
				{
					channelOptions.Credentials = ChannelCredentials.Create(ChannelCredentials.Insecure, callCredentials);
				}
				else
				{
					channelOptions.Credentials = ChannelCredentials.Create(ChannelCredentials.SecureSsl, callCredentials);
				}

				channelOptions.UnsafeUseInsecureChannelCallCredentials = useInsecureConnection;
			}

			// Create the channel
			_logger.LogInformation("gRPC channel connecting to {BaseUrl} ...", serverUri);
			return GrpcChannel.ForAddress(serverUri, channelOptions);
		}

		/// <summary>
		/// Get a gRPC call invoker for the given channel with extra metadata attached,
		/// such as current version and name
		/// </summary>
		/// <param name="channel">gRPC channel to use</param>
		/// <returns>A call invoker</returns>
		public CallInvoker GetInvoker(GrpcChannel channel)
		{
			CallInvoker invoker = channel.Intercept(headers =>
			{
				headers.Add("Horde-Agent-Version", AgentApp.Version);
				headers.Add("Horde-Agent-Name", _settings.CurrentValue.GetAgentName());
				return headers;
			});

			return invoker;
		}
	}
}
