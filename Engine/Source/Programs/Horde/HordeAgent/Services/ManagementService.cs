// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using RoutesDict = System.Collections.Generic.Dictionary<string, System.Action<HordeAgent.Services.IHttpListenerRequest, HordeAgent.Services.IHttpListenerResponse>>;

namespace HordeAgent.Services;

#region Wrapper classes for HttpListener to make it testable

internal interface IHttpListenerRequest
{
	Uri? Url { get; }
}

internal interface IHttpListenerResponse
{
	string? ContentType { get; set; }
	int StatusCode { get; set; }
	long ContentLength64 { get; set; }
	WebHeaderCollection Headers { get; set; }
	CookieCollection Cookies { get; set; }
	Stream OutputStream { get; }
	void Close();
}

internal class WrappedHttpListenerRequest(HttpListenerRequest impl) : IHttpListenerRequest
{
	public Uri? Url => impl.Url;
}

internal class WrappedHttpListenerResponse(HttpListenerResponse impl) : IHttpListenerResponse
{
	public string? ContentType { get => impl.ContentType; set => impl.ContentType = value; }
	public int StatusCode { get => impl.StatusCode; set => impl.StatusCode = value; }
	public long ContentLength64 { get => impl.ContentLength64; set => impl.ContentLength64 = value; }
	public WebHeaderCollection Headers { get => impl.Headers; set => impl.Headers = value; }
	public CookieCollection Cookies { get => impl.Cookies; set => impl.Cookies = value; }
	public Stream OutputStream => impl.OutputStream;
	public void Close() => impl.Close();
}

#endregion

internal class ErrorResponse(bool success, string message)
{
	public bool Success { get; set; } = success;
	public string Message { get; set; } = message;
}

internal class HealthCheckResponse(bool isHealthy, bool isConnected)
{
	public bool IsHealthy { get; set; } = isHealthy;
	public bool IsConnected { get; set; } = isConnected;
}

internal class PoolsResponse(List<string> poolIds)
{
	public List<string> PoolIds { get; set; } = poolIds;
}

/// <summary>
/// Handles agent's built-in HTTP server used for admin tasks and health check
/// </summary>
internal class ManagementService : IHostedService, IDisposable
{
	private readonly HttpListener? _adminListener;
	private readonly HttpListener? _healthListener;
	private readonly Thread? _adminThread;
	private readonly Thread? _healthThread;
	private readonly ILogger<ManagementService> _logger;
	private bool _isStopping = false;
	private bool _isDisposed = false;
	
	public static readonly JsonSerializerOptions JsonSerializerOptions = new()
	{
		WriteIndented = true, PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase
	};
	
	private readonly RoutesDict _adminRoutes = new() { ["/"] = HandleJson((_, _) => "Admin server"), };
	private readonly RoutesDict _healthRoutes;
	
	private static Action<IHttpListenerRequest, IHttpListenerResponse> HandleJson<T>(Func<IHttpListenerRequest, IHttpListenerResponse, T> callback)
	{
		return (req, res) =>
		{
			res.Headers.Set("Server", ""); // Remove the default "Microsoft-HTTPAPI/2.0" getting sent
			res.ContentType = "application/json";
			res.StatusCode = 200;
			T result = callback(req, res);
			JsonSerializer.Serialize(res.OutputStream, result, JsonSerializerOptions);
		};
	}
	
	/// <summary>
	/// Constructor
	/// </summary>
	public ManagementService(IOptionsMonitor<AgentSettings> settings, IWorkerService workerService, ILogger<ManagementService> logger)
	{
		_logger = logger;
		_healthRoutes = new RoutesDict
		{
			["/"] = HandleJson<HealthCheckResponse>((_, res) =>
			{
				bool isHealthy = workerService.IsConnected;
				res.StatusCode = isHealthy ? 200 : 503;
				return new HealthCheckResponse(isHealthy, workerService.IsConnected);
			}),
			["/pools"] = HandleJson<PoolsResponse>((_, res) => new PoolsResponse(workerService.PoolIds.ToList())),
		};
		
		string SettingsEndpointToPrefix(string endpoint) => $"http://{endpoint}/";
		
		if (settings.CurrentValue.AdminEndpoints.Length > 0)
		{
			_adminListener = new HttpListener();
			foreach (string endpoint in settings.CurrentValue.AdminEndpoints)
			{
				_adminListener.Prefixes.Add(SettingsEndpointToPrefix(endpoint));
			}
			
			_logger.LogInformation("Admin HTTP server listening on {Endpoints}", String.Join(",", settings.CurrentValue.AdminEndpoints));
			_adminThread = new Thread(() => RunServerSync(_adminListener, _adminRoutes, "Admin"))
			{
				Name = "AdminHttpServer", IsBackground = true, Priority = ThreadPriority.AboveNormal,
			};
		}
		
		if (settings.CurrentValue.HealthCheckEndpoints.Length > 0)
		{
			_healthListener = new HttpListener();
			foreach (string endpoint in settings.CurrentValue.HealthCheckEndpoints)
			{
				_healthListener.Prefixes.Add(SettingsEndpointToPrefix(endpoint));
			}
			
			_logger.LogInformation("Health check HTTP server listening on {Endpoints}", String.Join(",", settings.CurrentValue.HealthCheckEndpoints));
			_healthThread = new Thread(() => RunServerSync(_healthListener, _healthRoutes, "Health"))
			{
				Name = "HealthHttpServer", IsBackground = true, Priority = ThreadPriority.AboveNormal,
			};
		}
	}
	
	public void Dispose()
	{
		if (!_isDisposed)
		{
			_isStopping = true;
			
			if (_adminListener?.IsListening == true)
			{
				_adminListener.Stop();
			}
			
			if (_healthListener?.IsListening == true)
			{
				_healthListener.Stop();
			}
			
			_adminThread?.Join(TimeSpan.FromSeconds(3));
			_healthThread?.Join(TimeSpan.FromSeconds(3));
			_adminListener?.Close();
			_healthListener?.Close();
			_isDisposed = true;
		}
	}
	
	/// <inheritdoc/>
	public Task StartAsync(CancellationToken cancellationToken)
	{
		try
		{
			if (_adminListener != null)
			{
				_adminListener.Start();
				_adminThread?.Start();
			}
			
			if (_healthListener != null)
			{
				_healthListener.Start();
				_healthThread?.Start();
			}
		}
		catch (Exception e)
		{
			// Access denied exception is raised on Windows trying to bind on all interfaces as Administrator privileges are needed
			// Skip logging the stacktrace to reduce noise in log
			bool isAccessDenied = e.Message.Contains("Access is denied", StringComparison.OrdinalIgnoreCase);
			_logger.LogError(isAccessDenied ? null : e, "Failed to start built-in HTTP server(s) for administration and health checks");
			if (isAccessDenied)
			{
				_logger.LogError("Binding on all interfaces requires elevated process permissions (administrator/root)");
			}
		}
		
		return Task.CompletedTask;
	}
	
	/// <inheritdoc/>
	public Task StopAsync(CancellationToken cancellationToken)
	{
		Dispose();
		return Task.CompletedTask;
	}
	
	internal void TestAdminRoute(IHttpListenerRequest req, IHttpListenerResponse res)
	{
		HandleRequest(req, res, _adminRoutes);
	}
	
	internal void TestHealthRoute(IHttpListenerRequest req, IHttpListenerResponse res)
	{
		HandleRequest(req, res, _healthRoutes);
	}
	
	private void HandleRequest(IHttpListenerRequest req, IHttpListenerResponse res, RoutesDict routes)
	{
		try
		{
			string path = req.Url?.AbsolutePath ?? "/";
			if (routes.TryGetValue(path, out Action<IHttpListenerRequest, IHttpListenerResponse>? handler))
			{
				handler(req, res);
			}
			else
			{
				HandleJson((_, res) =>
				{
					res.StatusCode = 404;
					return new ErrorResponse(false, "Not Found");
				})(req, res);
			}
		}
		catch (Exception e)
		{
			HandleJson((_, _) => new ErrorResponse(false, "Internal Server Error"))(req, res);
			_logger.LogError(e, "Error while serving HTTP request for management service: {Exception}", e);
		}
		finally
		{
			res.Close();
		}
	}
	
	private void RunServerSync(HttpListener listener, RoutesDict routes, string serverType)
	{
		_logger.LogDebug("{ServerType} HTTP server thread started", serverType);
		
		try
		{
			while (!_isStopping)
			{
				try
				{
					HttpListenerContext context = listener.GetContext();
					WrappedHttpListenerRequest wrappedReq = new(context.Request);
					WrappedHttpListenerResponse wrappedRes = new(context.Response);
					HandleRequest(wrappedReq, wrappedRes, routes);
				}
				catch (Exception ex)
				{
					if (!_isStopping)
					{
						_logger.LogError(ex, "Unexpected error in {ServerType} HTTP server", serverType);
						Thread.Sleep(100); // Small delay to prevent tight error loops
					}
				}
			}
		}
		catch (Exception ex)
		{
			_logger.LogError(ex, "{ServerType} HTTP server thread failed unexpectedly", serverType);
		}
		finally
		{
			_logger.LogDebug("{ServerType} HTTP server thread exiting", serverType);
		}
	}
}