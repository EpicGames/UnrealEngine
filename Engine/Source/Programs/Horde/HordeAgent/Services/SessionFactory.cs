// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using System.Security.Cryptography;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Sessions;
using Grpc.Core;
using Grpc.Net.Client;
using HordeAgent.Utility;
using Horde.Common.Rpc;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Services
{
	/// <summary>
	/// Interface for about the current session. 
	/// </summary>
	interface ISession : IAsyncDisposable
	{
		/// <summary>
		/// The agent identifier
		/// </summary>
		AgentId AgentId { get; }

		/// <summary>
		/// Identifier for the current session
		/// </summary>
		SessionId SessionId { get; }

		/// <summary>
		/// Working directory for sandboxes etc..
		/// </summary>
		DirectoryReference WorkingDir { get; }

		/// <summary>
		/// Horde client instance
		/// </summary>
		IHordeClient HordeClient { get; }
	}

	/// <summary>
	/// Interface for a factory to create sessions
	/// </summary>
	interface ISessionFactory
	{
		/// <summary>
		/// Creates a new session
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New session information</returns>
		public Task<ISession> CreateAsync(CancellationToken cancellationToken);
	}

	/// <summary>
	/// Information about the current session. 
	/// </summary>
	sealed class Session : ISession
	{
		/// <inheritdoc/>
		public AgentId AgentId { get; }

		/// <inheritdoc/>
		public SessionId SessionId { get; }

		/// <inheritdoc/>
		public DirectoryReference WorkingDir { get; }

		/// <inheritdoc/>
		public IHordeClient HordeClient { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public Session(AgentId agentId, SessionId sessionId, DirectoryReference workingDir, IHordeClient hordeClient)
		{
			AgentId = agentId;
			SessionId = sessionId;
			WorkingDir = workingDir;
			HordeClient = hordeClient;
		}

		public class AgentRegistrationList
		{
			public List<AgentRegistration> Entries { get; set; } = new List<AgentRegistration>();
		}

		public record class AgentRegistration(Uri Server, string Id, string Token);

		/// <summary>
		/// Creates a new agent session
		/// </summary>
		public static async Task<Session> CreateAsync(CapabilitiesService capabilitiesService, GrpcService grpcService, StatusService statusService, IOptionsMonitor<AgentSettings> settings, HordeClientFactory hordeClientFactory, ILogger logger, CancellationToken cancellationToken)
		{
			AgentSettings currentSettings = settings.CurrentValue;

			// Get the working directory
			if (currentSettings.WorkingDir == null)
			{
				throw new Exception("WorkingDir is not set. Unable to run service.");
			}

			DirectoryReference workingDir = currentSettings.WorkingDir;
			logger.LogInformation("WorkingDir: {WorkingDir}", workingDir);
			DirectoryReference.CreateDirectory(workingDir);

			// Print the server info
			ServerProfile serverProfile = currentSettings.GetCurrentServerProfile();
			logger.LogInformation("Server URL: {Server} Profile: {ProfileName}", serverProfile.Url, serverProfile.Name);

			// Show the worker capabilities
			RpcAgentCapabilities capabilities = await capabilitiesService.GetCapabilitiesAsync(workingDir);
			if (capabilities.Properties.Count > 0)
			{
				logger.LogInformation("Properties:");
				foreach (string property in capabilities.Properties)
				{
					logger.LogInformation("  {AgentProperty}", property);
				}
			}
			if (capabilities.Resources.Count > 0)
			{
				logger.LogInformation("Resources:");
				foreach ((string name, int value) in capabilities.Resources)
				{
					logger.LogInformation("  {Name}={Value}", name, value);
				}
			}

			// Mount all the necessary network shares. Currently only supported on Windows.
			if (currentSettings.ShareMountingEnabled && RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				foreach (MountNetworkShare share in currentSettings.Shares)
				{
					if (share.MountPoint != null && share.RemotePath != null)
					{
						logger.LogInformation("Mounting {RemotePath} as {MountPoint}", share.RemotePath, share.MountPoint);
						NetworkShare.Mount(share.MountPoint, share.RemotePath);
					}
				}
			}

			// Get the location of the registration file
			FileReference registrationFile = GetRegistrationFile();

			// Read existing settings if possible
			AgentRegistrationList registrationList = await ReadRegistrationListAsync(registrationFile, cancellationToken);

			// If they aren't valid, create a new agent registration
			AgentRegistration? registrationInfo = registrationList.Entries.FirstOrDefault(x => x.Server == serverProfile.Url);
			if (registrationInfo == null)
			{
				statusService.SetDescription(AgentStatusMessage.WaitingForEnrollment);

				registrationInfo = await RegisterAgentAsync(hordeClientFactory, currentSettings, capabilities, logger, cancellationToken);
				registrationList.Entries.Add(registrationInfo);

				await WriteRegistrationListAsync(registrationFile, registrationList, cancellationToken);
				logger.LogInformation("Created agent (Id={AgentId}). Server <-> agent registration info saved to {File}", registrationInfo.Id, registrationFile);
			}
			else
			{
				logger.LogInformation("Cached server <-> agent registration info loaded from {File}. AgentId={AgentId}", registrationFile, registrationInfo.Id);
			}
			
			// Try using the token returned from agent registration.
			// For a pre-shared token, this new registration token is to be used (dedicated agent)
			// A workstation agent never receives a registration as it will always authenticate as the current user
			string? accessToken = !String.IsNullOrEmpty(registrationInfo.Token) ? registrationInfo.Token : serverProfile.GetAuthToken();

			// Create the session
			statusService.SetDescription(AgentStatusMessage.ConnectingToServer);

			RpcCreateSessionResponse createSessionResponse;
			await using (HordeClient sessionClient = hordeClientFactory.Create(accessToken: accessToken))
			{
				HordeRpc.HordeRpcClient rpcClient = await sessionClient.CreateGrpcClientAsync<HordeRpc.HordeRpcClient>(cancellationToken);

				// Create the session information
				RpcCreateSessionRequest sessionRequest = new RpcCreateSessionRequest();
				sessionRequest.Id = registrationInfo.Id;
				sessionRequest.Capabilities = capabilities;
				sessionRequest.Version = AgentApp.Version;

				// Create a session
				try
				{
					createSessionResponse = await rpcClient.CreateSessionAsync(sessionRequest, null, null, cancellationToken);
					logger.LogInformation("Created session. AgentName={AgentName} SessionId={SessionId}", currentSettings.GetAgentName(), createSessionResponse.SessionId);
				}
				catch (RpcException ex) when (ex.StatusCode == StatusCode.PermissionDenied || ex.StatusCode == StatusCode.Unauthenticated)
				{
					Uri serverUrl = grpcService.ServerProfile.Url;
					if (registrationList.Entries.RemoveAll(x => x.Server == serverUrl) > 0)
					{
						logger.LogError(ex, "Unable to create session. Invalidating agent registration for server {ServerUrl}.", serverUrl);
						await WriteRegistrationListAsync(registrationFile, registrationList, cancellationToken);
					}
					throw;
				}
			}

			// Open a connection to the server
#pragma warning disable CA2000 // False positive; ownership is transferred to new Session object.
			HordeClient client = hordeClientFactory.Create(accessToken: createSessionResponse.Token);
			return new Session(new AgentId(createSessionResponse.AgentId), SessionId.Parse(createSessionResponse.SessionId), workingDir, client);
#pragma warning restore CA2000
		}

		/// <summary>
		/// Dispose of the current session
		/// </summary>
		/// <returns></returns>
		public async ValueTask DisposeAsync()
		{
			if (HordeClient is IAsyncDisposable disposableClient)
			{
				await disposableClient.DisposeAsync();
			}
		}

		static FileReference GetRegistrationFile()
		{
			// Get the location of the registration file
			DirectoryReference? settingsDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
			if (settingsDir == null)
			{
				settingsDir = DirectoryReference.GetCurrentDirectory();
			}
			else
			{
				settingsDir = DirectoryReference.Combine(settingsDir, "Epic Games", "Horde", "Agent");
			}

			FileReference settingsFile = FileReference.Combine(settingsDir, "servers.json");
			return settingsFile;
		}

		static async Task<AgentRegistrationList> ReadRegistrationListAsync(FileReference settingsFile, CancellationToken cancellationToken)
		{
			AgentRegistrationList? registrationList = null;
			if (FileReference.Exists(settingsFile))
			{
				byte[] settingsData = await FileReference.ReadAllBytesAsync(settingsFile, cancellationToken);
				registrationList = JsonSerializer.Deserialize<AgentRegistrationList>(settingsData, AgentApp.DefaultJsonSerializerOptions);
				registrationList?.Entries.RemoveAll(x => x.Server == null || x.Id == null || x.Token == null);
			}
			return registrationList ??= new AgentRegistrationList();
		}

		static async Task WriteRegistrationListAsync(FileReference settingsFile, AgentRegistrationList registrationList, CancellationToken cancellationToken)
		{
			byte[] data = JsonSerializer.SerializeToUtf8Bytes(registrationList, new JsonSerializerOptions(AgentApp.DefaultJsonSerializerOptions) { WriteIndented = true });
			DirectoryReference.CreateDirectory(settingsFile.Directory);
			await FileReference.WriteAllBytesAsync(settingsFile, data, cancellationToken);
		}

		static async Task<AgentRegistration> RegisterAgentAsync(HordeClientFactory hordeClientFactory, AgentSettings agentSettings, RpcAgentCapabilities capabilities, ILogger logger, CancellationToken cancellationToken)
		{
			ServerProfile serverProfile = agentSettings.GetCurrentServerProfile();
			await using HordeClient sessionClient = hordeClientFactory.Create(accessToken: serverProfile.GetAuthToken());
			using GrpcChannel grpcChannel = await sessionClient.CreateGrpcChannelAsync(cancellationToken);

			if (!String.IsNullOrEmpty(serverProfile.Token) || serverProfile.UseInteractiveAuth)
			{
				logger.LogInformation("Registering agent...");
				HordeRpc.HordeRpcClient rpcClient = new (grpcChannel);

				RpcCreateAgentRequest createAgentRequest = new() { Name = agentSettings.GetAgentName(), Ephemeral = agentSettings.Ephemeral, Mode = agentSettings.GetMode() };
				RpcCreateAgentResponse createAgentResponse = await rpcClient.CreateAgentAsync(createAgentRequest, null, null, cancellationToken);
				return new AgentRegistration(serverProfile.Url, createAgentResponse.Id, createAgentResponse.Token);
			}

			const string FormatString = "$(CPU) ($(LogicalCores) cores, $(RAM)gb RAM, $(OSDistribution))";
			string description = StringUtils.ExpandProperties(FormatString, name => GetProperty(capabilities, name));

			string registrationKey = StringUtils.FormatHexString(RandomNumberGenerator.GetBytes(64));
			for (; ; )
			{
				logger.LogInformation("Waiting for agent to be approved...");
				EnrollmentRpc.EnrollmentRpcClient rpcClient = new EnrollmentRpc.EnrollmentRpcClient(grpcChannel);

				EnrollAgentRequest enrollAgentRequest = new EnrollAgentRequest();
				enrollAgentRequest.Key = registrationKey;
				enrollAgentRequest.HostName = Environment.MachineName;
				enrollAgentRequest.Description = description;

				try
				{
					using AsyncDuplexStreamingCall<EnrollAgentRequest, EnrollAgentResponse> call = rpcClient.EnrollAgent(cancellationToken: cancellationToken);
					await call.RequestStream.WriteAsync(enrollAgentRequest, cancellationToken);

					Task delayTask = Task.Delay(TimeSpan.FromSeconds(10), cancellationToken);
					Task<bool> responseTask = call.ResponseStream.MoveNext(cancellationToken);
					Task completeTask = await Task.WhenAny(delayTask, responseTask);

					if (completeTask == delayTask)
					{
						await call.RequestStream.WriteAsync(enrollAgentRequest, cancellationToken);
					}

					if (await responseTask)
					{
						EnrollAgentResponse enrollAgentResponse = call.ResponseStream.Current;
						return new AgentRegistration(serverProfile.Url, enrollAgentResponse.Id, enrollAgentResponse.Token);
					}
				}
				catch (RpcException ex)
				{
					if (cancellationToken.IsCancellationRequested)
					{
						break;
					}
					
					TimeSpan backoffDelay = TimeSpan.FromSeconds(15);
					logger.LogWarning(ex, "Exception in RPC: {Message}", ex.Message);
					logger.LogInformation("Waiting {BackoffDelay} seconds before retrying...", backoffDelay.TotalSeconds);
					await Task.Delay(backoffDelay, CancellationToken.None);
				}
			}
			
			throw new Exception("Unable to register agent");
		}

		static string? GetProperty(RpcAgentCapabilities capabilities, string name)
		{
			foreach (string property in capabilities.Properties)
			{
				if (property.Length > name.Length && property[name.Length] == '=' && property.StartsWith(name, StringComparison.OrdinalIgnoreCase))
				{
					return property.Substring(name.Length + 1);
				}
			}
			return null;
		}
	}

	/// <summary>
	/// Creates session objects
	/// </summary>
	class SessionFactory : ISessionFactory
	{
		readonly CapabilitiesService _capabilitiesService;
		readonly GrpcService _grpcService;
		readonly StatusService _statusService;
		readonly HordeClientFactory _hordeClientFactory;
		readonly IOptionsMonitor<AgentSettings> _agentSettings;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public SessionFactory(CapabilitiesService capabilitiesService, GrpcService grpcService, StatusService statusService, HordeClientFactory hordeClientFactory, IOptionsMonitor<AgentSettings> agentSettings, ILogger<SessionFactory> logger)
		{
			_capabilitiesService = capabilitiesService;
			_grpcService = grpcService;
			_statusService = statusService;
			_hordeClientFactory = hordeClientFactory;
			_agentSettings = agentSettings;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<ISession> CreateAsync(CancellationToken cancellationToken) => await Session.CreateAsync(_capabilitiesService, _grpcService, _statusService, _agentSettings, _hordeClientFactory, _logger, cancellationToken);
	}
}
