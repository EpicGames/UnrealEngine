// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace HordeAgent.Services
{
	/// <summary>
	/// Outcome from a session
	/// </summary>
	enum SessionOutcome
	{
		/// <summary>
		/// Continue the session with a backoff
		/// </summary>
		BackOff,

		/// <summary>
		/// Session completed normally, terminate the application.
		/// </summary>
		Terminate,

		/// <summary>
		/// Runs the given callback then attempts to reconnect
		/// </summary>
		RunCallback,
	}
	
	/// <summary>
	/// Reason for outcome of a session
	/// </summary>
	enum SessionReason
	{
		/// <summary>
		/// Session completed its operations normally
		/// </summary>
		Completed,
		
		/// <summary>
		/// Session encountered an unrecoverable error
		/// </summary>
		Failed,
		
		/// <summary>
		/// Session was explicitly cancelled by user or system
		/// </summary>
		Cancelled,
	}

	/// <summary>
	/// Result from executing a session
	/// </summary>
	class SessionResult : IEquatable<SessionResult>
	{
		/// <summary>
		/// The outcome code
		/// </summary>
		public SessionOutcome Outcome { get; }
		
		/// <summary>
		/// Reason for the outcome
		/// </summary>
		public SessionReason Reason { get; }

		/// <summary>
		/// Callback for running upgrades
		/// </summary>
		public Func<ILogger, CancellationToken, Task>? CallbackAsync { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public SessionResult(SessionOutcome outcome, SessionReason reason)
		{
			Outcome = outcome;
			Reason = reason;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public SessionResult(Func<ILogger, CancellationToken, Task> callbackAsync)
		{
			Outcome = SessionOutcome.RunCallback;
			CallbackAsync = callbackAsync;
		}
		
		public bool Equals(SessionResult? other)
		{
			return Outcome == other?.Outcome && Reason == other.Reason;
		}
		
		public override bool Equals(object? obj)
		{
			return obj is SessionResult other && Equals(other);
		}
		
		public override int GetHashCode()
		{
			return HashCode.Combine((int)Outcome, (int)Reason);
		}
		
		public override string ToString()
		{
			return $"Outcome={Outcome} Reason={Reason}";
		}
	}
}
