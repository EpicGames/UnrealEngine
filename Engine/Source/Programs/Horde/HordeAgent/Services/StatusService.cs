// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO.Pipes;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Security.AccessControl;
using System.Security.Principal;
using EpicGames.Core;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeAgent.Services
{
	/// <summary>
	/// Tracks the current status of the agent
	/// </summary>
	class StatusService : IHostedService, IAsyncDisposable
	{
		const int NumPipes = 10;

		private AgentStatusMessage _current;
		private bool _isBusy;
		private bool _isStopRequested;

		readonly IOptionsMonitor<AgentSettings> _settings;
		readonly BackgroundTask _task;
		readonly ILogger _logger;

		/// <summary>
		/// The current agent status
		/// </summary>
		public AgentStatusMessage Current => _current;

		/// <summary>
		/// Whether the agent is busy performing other work.
		/// </summary>
		public bool IsBusy
		{
			get => _isBusy;
			set
			{
				if (_isBusy != value)
				{
					_isBusy = value;
					StatusChangedEvent.Set();
				}
			}
		}
		
		/// <summary>
		/// Whether a stop of the agent is requested
		/// </summary>
		public bool IsStopRequested
		{
			get => _isStopRequested;
			set
			{
				if (_isStopRequested != value)
				{
					_isStopRequested = value;
					StatusChangedEvent.Set();
				}
			}
		}

		/// <summary>
		/// Status was updated
		/// </summary>
		public readonly AsyncEvent StatusChangedEvent = new();

		/// <summary>
		/// Constructor
		/// </summary>
		public StatusService(IOptionsMonitor<AgentSettings> settings, ILogger<StatusService> logger)
		{
			_current = new AgentStatusMessage(true, 0, AgentStatusMessage.Starting);
			_settings = settings;
			_task = new BackgroundTask(RunPipeServerAsync);
			_logger = logger;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _task.DisposeAsync();
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			_task.Start();
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _task.StopAsync(cancellationToken);
		}

		/// <summary>
		/// Sets the current status
		/// </summary>
		public void Set(AgentStatusMessage status)
		{
			Interlocked.Exchange(ref _current, status);
			_logger.LogDebug("Updating status: {@Status}", status);
		}

		/// <summary>
		/// Sets the current status
		/// </summary>
		public void Set(bool healthy, int numLeases, string message) => Set(new AgentStatusMessage(healthy, numLeases, message));

		/// <summary>
		/// Sets a status description
		/// </summary>
		public void SetDescription(string description) => Update(status => new AgentStatusMessage(status.Healthy, status.NumLeases, description));

		/// <summary>
		/// Updates the status using a custom function
		/// </summary>
		/// <param name="updateFunc">Function to take the existing status and create an updated version</param>
		public void Update(Func<AgentStatusMessage, AgentStatusMessage> updateFunc)
		{
			for (; ; )
			{
				AgentStatusMessage info = _current;
				if (Interlocked.CompareExchange(ref _current, updateFunc(info), info) == info)
				{
					break;
				}
			}
		}

		private async Task RunPipeServerAsync(CancellationToken cancellationToken)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				List<Task> tasks = new List<Task>();
				for (int idx = 0; idx < NumPipes; idx++)
				{
					tasks.Add(RunSinglePipeServerAsync(cancellationToken));
				}
				await Task.WhenAll(tasks);
			}
		}

		[SupportedOSPlatform("windows")]
		private async Task RunSinglePipeServerAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				try
				{
					await RunPipeServerInternalAsync(cancellationToken);
				}
				catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				catch (UnauthorizedAccessException e)
				{
					_logger.LogError("Unable to start IPC server. Ensure no other Horde agent processes are running! Reason: {Reason}", e.Message);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while running pipe server: {Message}", ex.Message);
				}

				await Task.Delay(TimeSpan.FromSeconds(30), cancellationToken);
			}
		}

		AgentSettingsMessage GetSettingsMessage()
		{
			AgentSettings settings = _settings.CurrentValue;
			ServerProfile profile = settings.GetCurrentServerProfile();
			return new AgentSettingsMessage(profile.Url);
		}

		[SupportedOSPlatform("windows")]
		private async Task RunPipeServerInternalAsync(CancellationToken cancellationToken)
		{
			AgentMessageBuffer request = new AgentMessageBuffer();
			AgentMessageBuffer response = new AgentMessageBuffer();

			_logger.LogDebug("Creating pipe for status updates");

			PipeSecurity pipeSecurity = new PipeSecurity();
			pipeSecurity.AddAccessRule(new PipeAccessRule(WindowsIdentity.GetCurrent().Name, PipeAccessRights.FullControl, AccessControlType.Allow));

			IdentityReference usersReference = new SecurityIdentifier(WellKnownSidType.BuiltinUsersSid, null).Translate(typeof(NTAccount));
			string users = usersReference.ToString().Replace(@"builtin\", "", StringComparison.InvariantCultureIgnoreCase);

			pipeSecurity.AddAccessRule(new PipeAccessRule(users, PipeAccessRights.ReadWrite, AccessControlType.Allow));

			using (NamedPipeServerStream pipeServer = NamedPipeServerStreamAcl.Create(AgentMessagePipe.PipeName, PipeDirection.InOut, NumPipes, PipeTransmissionMode.Byte, PipeOptions.Asynchronous, 0, 0, pipeSecurity))
			{
				await pipeServer.WaitForConnectionAsync(cancellationToken);
				_logger.LogDebug("Received pipe connection");

				while (await request.TryReadAsync(pipeServer, cancellationToken))
				{
					switch (request.Type)
					{
						case AgentMessageType.SetEnabledRequest:
							IsBusy = !request.Parse<AgentEnabledMessage>().IsEnabled;
							break;
						case AgentMessageType.GetStatusRequest:
							response.Set(AgentMessageType.GetStatusResponse, Current);
							await response.SendAsync(pipeServer, cancellationToken);
							break;
						case AgentMessageType.SetSettingsRequest:
							AgentSetSettingsRequest req = request.Parse<AgentSetSettingsRequest>();
							_settings.CurrentValue.CpuCount = req.CpuCount;
							_settings.CurrentValue.CpuMultiplier = req.CpuMultiplier;
							break;
						case AgentMessageType.GetSettingsRequest:
							response.Set(AgentMessageType.GetSettingsResponse, GetSettingsMessage());
							await response.SendAsync(pipeServer, cancellationToken);
							break;
						default:
							response.Set(AgentMessageType.InvalidResponse);
							await response.SendAsync(pipeServer, cancellationToken);
							break;
					}
				}
			}
		}
	}
}
