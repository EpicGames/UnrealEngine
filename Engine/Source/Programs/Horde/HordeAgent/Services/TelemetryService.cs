// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents.Metrics;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using static HordeAgent.Services.WindowsCapabilities;

namespace HordeAgent.Services;

/// <summary>
/// Default implementation of <see cref="ISystemMetrics"/>
/// </summary>
public sealed class DefaultSystemMetrics : ISystemMetrics
{
	/// <inheritdoc/>
	public CpuMetrics? GetCpu() => null;
	
	/// <inheritdoc/>
	public CpuTimeMetrics? GetCpuTime() => null;

	/// <inheritdoc/>
	public MemoryMetrics? GetMemory() => null;

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk() => null;

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk() => null;
}

/// <summary>
/// Windows specific implementation for gathering system metrics
/// </summary>
[SupportedOSPlatform("windows")]
public sealed class WindowsSystemMetrics : ISystemMetrics
{
	private readonly DirectoryReference _workingDir;
	private readonly Stopwatch _driveInfoUpdateTimer = Stopwatch.StartNew();
	private DriveInfo? _workingDriveInfo;
	private DriveInfo? _systemDriveInfo;

	/// <summary>
	/// Constructor
	/// </summary>
	public WindowsSystemMetrics(DirectoryReference workingDir)
	{
		_workingDir = workingDir;
		_workingDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.Working);
		_systemDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.System);

		GetCpu(); // Trigger this to ensure performance counter has a fetched value. Avoids an initial zero result when called later.
	}

	/// <inheritdoc />
	public CpuMetrics GetCpu()
	{
		(float idle, float system, float user) = GetCpuMetrics();
		return new CpuMetrics { User = user, System = system, Idle = idle };
	}

	/// <inheritdoc />
	public CpuTimeMetrics? GetCpuTime()
	{
		try
		{
			if (GetSystemTimes(out FILETIME idle, out FILETIME kernel, out FILETIME user))
			{
				return new CpuTimeMetrics
				{
					Idle = TimeSpan.FromMilliseconds(idle.ToUInt64() / 10000.0),
					User = TimeSpan.FromMilliseconds(user.ToUInt64() / 10000.0),
					// Kernel includes idle time, so must be subtracted
					System = TimeSpan.FromMilliseconds((kernel.ToUInt64() - idle.ToUInt64()) / 10000.0),
				};
			}
		}
		catch
		{
			// Ignore any error.
		}
		return null;
	}

	/// <inheritdoc />
	public MemoryMetrics GetMemory()
	{
		(ulong totalBytes, ulong availableBytes) = GetPhysicalMemory();
		ulong usedBytes = totalBytes - availableBytes;
		return new MemoryMetrics
		{
			Total = (uint)(totalBytes / 1024),
			Available = (uint)(availableBytes / 1024),
			Used = (uint)(usedBytes / 1024),
			UsedPercentage = usedBytes / (float)(totalBytes > 0 ? totalBytes : 1),
		};
	}

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk()
	{
		RefreshDriveInfo();

		if (_workingDriveInfo == null)
		{
			return null;
		}

		return new DiskMetrics
		{
			FreeSpace = _workingDriveInfo.AvailableFreeSpace,
			TotalSize = _workingDriveInfo.TotalSize
		};
	}

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk()
	{
		RefreshDriveInfo();

		if (_systemDriveInfo == null)
		{
			return null;
		}

		return new DiskMetrics
		{
			FreeSpace = _systemDriveInfo.AvailableFreeSpace,
			TotalSize = _systemDriveInfo.TotalSize
		};
	}

	/// <summary>
	/// Restart the drive info poll timer if the time period has elapsed
	/// </summary>
	void RefreshDriveInfo()
	{
		if (_driveInfoUpdateTimer.Elapsed > TimeSpan.FromSeconds(20))
		{
			_workingDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.Working);
			_systemDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.System);
			_driveInfoUpdateTimer.Restart();
		}
	}

	/// <summary>
	/// Fetches root directory based off drive classification
	/// </summary>
	/// <param name="type">Classification of the drive we're fetching telemetry for</param>
	/// <returns>DiskMetrics object representing total size and free space</returns>
	private DriveInfo? GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType type)
	{
		string? rootDir = type == EpicGames.Horde.Agents.Metrics.DriveType.Working
			? Path.GetPathRoot(_workingDir.FullName)
			: Path.GetPathRoot(Environment.SystemDirectory);

		return rootDir == null ? null : new DriveInfo(rootDir);
	}
}

#pragma warning restore CA1416

/// <summary>
/// Linux specific implementation for gathering system metrics
/// </summary>
[SupportedOSPlatform("linux")]
public sealed class LinuxSystemMetrics : ISystemMetrics, IDisposable
{
	private const string ProcMeminfoPath = "/proc/meminfo";
	private const string ProcStatPath = "/proc/stat";
	private const int SC_CLK_TCK = 2;
	private const int UserIndex = 1;
	private const int NiceIndex = 2;
	private const int SystemIndex = 3;
	private const int IdleIndex = 4;
	
	[DllImport("libc", SetLastError = true)]
	private static extern long sysconf(int name);

	private static readonly Lazy<long> s_clockTicksPerSecond = new (() =>
	{
		try
		{
			long result = sysconf(SC_CLK_TCK);
			if (result > 0)
			{
				return result;
			}
		}
		catch
		{
			// Fall through to default
		}
		return 100; // Default fallback
	});

	
	
	private readonly DirectoryReference _workingDir;
	
	/// <summary>
	/// Constructor
	/// </summary>
	public LinuxSystemMetrics(DirectoryReference workingDir)
	{
		_workingDir = workingDir;
	}
	
	/// <inheritdoc/>
	public CpuMetrics? GetCpu()
	{
		const int SampleDurationMs = 100;
		(long user, long system, long idle) firstSample = ReadCpuTimes();
		Thread.Sleep(SampleDurationMs);
		(long user, long system, long idle) secondSample = ReadCpuTimes();
		
		long userDelta = secondSample.user - firstSample.user;
		long systemDelta = secondSample.system - firstSample.system;
		long idleDelta = secondSample.idle - firstSample.idle;
		long totalDelta = userDelta + systemDelta + idleDelta;
		
		if (totalDelta == 0)
		{
			return new CpuMetrics { User = 0, System = 0, Idle = 100 };
		}
		
		return new CpuMetrics
		{
			User = (userDelta * 100.0f) / totalDelta,
			System = (systemDelta * 100.0f) / totalDelta,
			Idle = (idleDelta * 100.0f) / totalDelta,
		};
	}

	/// <inheritdoc/>
	public CpuTimeMetrics GetCpuTime()
	{
		(long user, long system, long idle) cpuTimes = ReadCpuTimes();
		long clockTicksPerSecond = s_clockTicksPerSecond.Value;
		return new CpuTimeMetrics
		{
			User = TimeSpan.FromMilliseconds(cpuTimes.user * 1000.0 / clockTicksPerSecond),
			System = TimeSpan.FromMilliseconds(cpuTimes.system * 1000.0 / clockTicksPerSecond),
			Idle = TimeSpan.FromMilliseconds(cpuTimes.idle * 1000.0 / clockTicksPerSecond),
		};
	}
	
	/// <inheritdoc/>
	public MemoryMetrics? GetMemory()
	{
		long ParseMemInfoValue(string[] parts)
		{
			return Convert.ToInt64(parts[1].Replace(" kB", "", StringComparison.Ordinal));
		}
		
		try
		{
			long memTotal = 0, memAvailable = 0, memFree = 0, buffers = 0, cached = 0, sReclaimAble = 0;
			foreach (string line in File.ReadAllLines(ProcMeminfoPath))
			{
				string[] parts = line.Split(':', StringSplitOptions.TrimEntries);
				string key = parts[0];
				switch (key)
				{
					case "MemTotal": memTotal = ParseMemInfoValue(parts); break;
					case "MemAvailable": memAvailable = ParseMemInfoValue(parts); break;
					case "MemFree": memFree = ParseMemInfoValue(parts); break;
					case "Buffers": buffers = ParseMemInfoValue(parts); break;
					case "Cached": cached = ParseMemInfoValue(parts); break;
					case "SReclaimable": sReclaimAble = ParseMemInfoValue(parts); break;
				}
			}

			long used = memTotal - memFree - buffers - cached - sReclaimAble;
			return new MemoryMetrics
			{
				Total = (uint)memTotal,
				Available = (uint)memAvailable,
				Used = (uint)used,
				UsedPercentage = (float)used / memTotal,
			};
		}
		catch (Exception)
		{
			return null;
		}
	}

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk() => GetDiskMetrics(EpicGames.Horde.Agents.Metrics.DriveType.Working);

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk() => GetDiskMetrics(EpicGames.Horde.Agents.Metrics.DriveType.System);

	/// <summary>
	/// Fetches root directory based off drive classification
	/// </summary>
	/// <param name="drive">Classification of the drive we're fetching telemetry for</param>
	/// <returns>DiskMetrics object representing total size and free space</returns>
	private DiskMetrics? GetDiskMetrics(EpicGames.Horde.Agents.Metrics.DriveType drive)
	{
		string path = drive == EpicGames.Horde.Agents.Metrics.DriveType.Working ? _workingDir.FullName : "/";
		if (Statvfs(path, out StatvfsStruct stats) != 0)
		{
			int errno = Marshal.GetLastPInvokeError();
			throw new IOException($"Failed to get disk metrics for path '{path}'. Error: {Marshal.GetPInvokeErrorMessage(errno)}");
		}
		
		// Calculate total size and free space
		// We use f_bavail (free blocks available to non-privileged users) instead of f_bfree as this is more representative
		long blockSize = (long)stats.f_frsize;
		long totalBlocks = (long)stats.f_blocks;
		long availableBlocks = (long)stats.f_bavail;
		
		return new DiskMetrics
		{
			TotalSize = totalBlocks * blockSize,
			FreeSpace = availableBlocks * blockSize,
		};
	}
	
	private static (long user, long system, long idle) ReadCpuTimes()
	{
		string statLine = File.ReadAllLines(ProcStatPath).First(line => line.StartsWith("cpu ", StringComparison.Ordinal));
		long[] values = statLine.Split(' ', StringSplitOptions.RemoveEmptyEntries)
			.Skip(1) // Skip "cpu" label
			.Select(Int64.Parse)
			.ToArray();
		long user = values[UserIndex - 1];
		long nice = values[NiceIndex - 1];
		long system = values[SystemIndex - 1];
		long idle = values[IdleIndex - 1];
		return (user + nice, system, idle);
	}
	
	[StructLayout(LayoutKind.Sequential)]
	[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "POSIX ABI")]
	[SuppressMessage("Style", "IDE0044:Add readonly modifier", Justification = "POSIX ABI")]
	private struct StatvfsStruct
	{
		public ulong f_bsize;    // File system block size
		public ulong f_frsize;   // Fundamental file system block size
		public ulong f_blocks;   // Total number of blocks on file system in units of f_frsize
		public ulong f_bfree;    // Total number of free blocks
		public ulong f_bavail;   // Number of free blocks available to non-privileged process
		public ulong f_files;    // Total number of file serial numbers
		public ulong f_ffree;    // Total number of free file serial numbers
		public ulong f_favail;   // Number of file serial numbers available to non-privileged process
		public ulong f_fsid;     // File system ID
		public ulong f_flag;     // Bit mask of f_flag values
		public ulong f_namemax;  // Maximum filename length
		private ulong __f_spare0;
		private ulong __f_spare1;
		private ulong __f_spare2;
		private ulong __f_spare3;
		private ulong __f_spare4;
	}
	
	[DllImport("libc", SetLastError = true, EntryPoint = "statvfs")]
	private static extern int Statvfs(string path, out StatvfsStruct buf);
	
	/// <inheritdoc/>
	public void Dispose()
	{
	}
}

/// <summary>
/// macOS specific implementation for gathering system metrics
/// </summary>
[SupportedOSPlatform("macos")]
public sealed class MacSystemMetrics : ISystemMetrics
{
	private readonly DirectoryReference _workingDir;
	private readonly Stopwatch _driveInfoUpdateTimer = Stopwatch.StartNew();
	private DriveInfo? _workingDriveInfo;
	private DriveInfo? _systemDriveInfo;

	/// <summary>
	/// Constructor
	/// </summary>
	public MacSystemMetrics(DirectoryReference workingDir)
	{
		_workingDir = workingDir;
		_workingDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.Working);
		_systemDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.System);
	}

	/// <inheritdoc/>
	public CpuMetrics? GetCpu() => null;

	/// <inheritdoc/>
	public CpuTimeMetrics? GetCpuTime() => null;

	/// <inheritdoc/>
	public MemoryMetrics? GetMemory() => null;

	/// <inheritdoc/>
	public DiskMetrics? GetWorkingDisk()
	{
		RefreshDriveInfo();

		if (_workingDriveInfo == null)
		{
			return null;
		}

		return new DiskMetrics
		{
			FreeSpace = _workingDriveInfo.AvailableFreeSpace,
			TotalSize = _workingDriveInfo.TotalSize
		};
	}

	/// <inheritdoc/>
	public DiskMetrics? GetSystemDisk()
	{
		RefreshDriveInfo();

		if (_systemDriveInfo == null)
		{
			return null;
		}

		return new DiskMetrics
		{
			FreeSpace = _systemDriveInfo.AvailableFreeSpace,
			TotalSize = _systemDriveInfo.TotalSize
		};
	}

	/// <summary>
	/// Restart the drive info poll timer if the time period has elapsed
	/// </summary>
	void RefreshDriveInfo()
	{
		if (_driveInfoUpdateTimer.Elapsed > TimeSpan.FromSeconds(20))
		{
			_workingDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.Working);
			_systemDriveInfo = GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType.System);
			_driveInfoUpdateTimer.Restart();
		}
	}

	/// <summary>
	/// Fetches the drive info for the given drive type
	/// </summary>
	/// <param name="type">Classification of the drive we're fetching telemetry for</param>
	/// <returns>DriveInfo for the matching drive</returns>
	private DriveInfo GetDriveInfo(EpicGames.Horde.Agents.Metrics.DriveType type)
	{
		string driveName = type == EpicGames.Horde.Agents.Metrics.DriveType.Working ? _workingDir.FullName : "/";
		return new DriveInfo(driveName);
	}
}

/// <summary>
/// Send telemetry events back to server at regular intervals
/// </summary>
class TelemetryService : IHostedService, IDisposable
{
	private readonly TimeSpan _heartbeatInterval = TimeSpan.FromSeconds(60);
	private readonly TimeSpan _heartbeatMaxAllowedDiff = TimeSpan.FromSeconds(5);

	private readonly ILogger<TelemetryService> _logger;

	private CancellationTokenSource? _eventLoopHeartbeatCts;
	private Task? _eventLoopTask;
	internal Func<DateTime> GetUtcNow { get; set; } = () => DateTime.UtcNow;

	/// <summary>
	/// Constructor
	/// </summary>
	public TelemetryService(ILogger<TelemetryService> logger)
	{
		_logger = logger;
	}

	/// <inheritdoc/>
	public void Dispose()
	{
		_eventLoopHeartbeatCts?.Dispose();
	}

	/// <inheritdoc />
	public Task StartAsync(CancellationToken cancellationToken)
	{
		_eventLoopHeartbeatCts = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
		_eventLoopTask = EventLoopHeartbeatAsync(_eventLoopHeartbeatCts.Token);
		return Task.CompletedTask;
	}

	public async Task StopAsync(CancellationToken cancellationToken)
	{
		if (_eventLoopHeartbeatCts != null)
		{
			await _eventLoopHeartbeatCts.CancelAsync();
		}

		if (_eventLoopTask != null)
		{
			try
			{
				await _eventLoopTask;
			}
			catch (OperationCanceledException)
			{
				// Ignore cancellation exceptions
			}
		}
	}

	/// <summary>
	/// Continuously run the heartbeat for the event loop
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	private async Task EventLoopHeartbeatAsync(CancellationToken cancellationToken)
	{
		while (!cancellationToken.IsCancellationRequested)
		{
			(bool onTime, TimeSpan diff) = await IsEventLoopOnTimeAsync(_heartbeatInterval, _heartbeatMaxAllowedDiff, cancellationToken);
			if (!onTime)
			{
				_logger.LogWarning("Event loop heartbeat was not on time. Diff {Diff} ms", diff.TotalMilliseconds);
			}
		}
	}

	/// <summary>
	/// Checks if the async event loop is on time
	/// </summary>
	/// <param name="wait">Time to wait</param>
	/// <param name="maxDiff">Max allowed diff</param>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>A tuple with the result</returns>
	public async Task<(bool onTime, TimeSpan diff)> IsEventLoopOnTimeAsync(TimeSpan wait, TimeSpan maxDiff, CancellationToken cancellationToken)
	{
		try
		{
			DateTime before = GetUtcNow();
			await Task.Delay(wait, cancellationToken);
			DateTime after = GetUtcNow();
			TimeSpan diff = after - before - wait;
			return (diff.Duration() < maxDiff, diff);
		}
		catch (TaskCanceledException)
		{
			// Ignore
			return (true, TimeSpan.Zero);
		}
	}

	/// <summary>
	/// Get list of any filter drivers known to be problematic for builds
	/// </summary>
	/// <param name="logger">Logger to use</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	public static async Task<List<string>> GetProblematicFilterDriversAsync(ILogger logger, CancellationToken cancellationToken)
	{
		try
		{
			List<string> problematicDrivers = new()
			{
				// PlayStation SDK related filter drivers known to have negative effects on file system performance
				"cbfilter",
				"cbfsfilter",
				"cbfsconnect",
				"sie-filemon",

				"csagent", // CrowdStrike
			};

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				string output = await ReadFltMcOutputAsync(cancellationToken);
				List<string>? drivers = ParseFltMcOutput(output);
				if (drivers == null)
				{
					logger.LogWarning("Unable to get loaded filter drivers");
					return new List<string>();
				}

				List<string> loadedDrivers = drivers
					.Where(x =>
					{
						foreach (string probDriverName in problematicDrivers)
						{
							if (x.Contains(probDriverName, StringComparison.OrdinalIgnoreCase))
							{
								return true;
							}
						}
						return false;
					})
					.ToList();

				return loadedDrivers;
			}
		}
		catch (Exception e)
		{
			logger.LogError(e, "Error logging filter drivers");
		}

		return new List<string>();
	}

	/// <summary>
	/// Log any filter drivers known to be problematic for builds
	/// </summary>
	/// <param name="logger">Logger to use</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	public static async Task LogProblematicFilterDriversAsync(ILogger logger, CancellationToken cancellationToken)
	{
		List<string> loadedDrivers = await GetProblematicFilterDriversAsync(logger, cancellationToken);
		if (loadedDrivers.Count > 0)
		{
			logger.LogWarning("Agent has problematic filter drivers loaded: {FilterDrivers}", String.Join(',', loadedDrivers));
		}
	}

	internal static async Task<string> ReadFltMcOutputAsync(CancellationToken cancellationToken)
	{
		string fltmcExePath = Path.Combine(Environment.SystemDirectory, "fltmc.exe");
		using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
		cancellationSource.CancelAfter(10000);
		using ManagedProcess process = new(null, fltmcExePath, "filters", null, null, null, ProcessPriorityClass.Normal);
		StringBuilder sb = new(1000);

		while (!cancellationSource.IsCancellationRequested)
		{
			string? line = await process.ReadLineAsync(cancellationSource.Token);
			if (line == null)
			{
				break;
			}

			sb.AppendLine(line);
		}

		await process.WaitForExitAsync(cancellationSource.Token);
		return sb.ToString();
	}

	internal static List<string>? ParseFltMcOutput(string output)
	{
		if (output.Contains("access is denied", StringComparison.OrdinalIgnoreCase))
		{
			return null;
		}

		List<string> filters = new();
		string[] lines = output.Split(new[] { "\r\n", "\r", "\n" }, StringSplitOptions.None);

		foreach (string line in lines)
		{
			if (line.Length < 5)
			{
				continue;
			}
			if (line.StartsWith("---", StringComparison.Ordinal))
			{
				continue;
			}
			if (line.StartsWith("Filter", StringComparison.Ordinal))
			{
				continue;
			}

			string[] parts = line.Split("   ", StringSplitOptions.RemoveEmptyEntries);
			string filterName = parts[0];
			filters.Add(filterName);
		}

		return filters;
	}
}