// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Win32;
using OpenTelemetry.Trace;

namespace HordeAgent.Services;

/// <summary>
/// Get Windows-specific capabilities 
/// </summary>
[SupportedOSPlatform("windows")]
public class WindowsCapabilities(IOptionsMonitor<AgentSettings> settings, ILogger logger, Tracer tracer) : ISystemCapabilities
{
	/// <inheritdoc/>
	public Task<RpcAgentCapabilities> GetCapabilitiesAsync(DirectoryReference? workingDir)
	{
		using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(WindowsCapabilities)}.{nameof(GetCapabilitiesAsync)}");
		RpcAgentCapabilities caps = new();
		
		// OS and platform
		SetProp(caps, KnownPropertyNames.Platform, "Win64");
		SetProp(caps, KnownPropertyNames.PlatformGroup, "Windows");
		SetProp(caps, KnownPropertyNames.PlatformGroup, "Microsoft");
		SetProp(caps, KnownPropertyNames.PlatformGroup, "Desktop");
		SetProp(caps, KnownPropertyNames.OsFamily, "Windows");
		SetProp(caps, KnownPropertyNames.OsFamilyCompatibility, "Windows");
		SetProp(caps, "OSDistribution", RuntimeInformation.OSDescription);
		SetProp(caps, "OSKernelVersion", Environment.OSVersion.Version.ToString());
		
		// User
		SetProp(caps, "User", Environment.UserName);
		SetProp(caps, "Domain", Environment.UserDomainName);
		SetProp(caps, "Interactive", Environment.UserInteractive);
		SetProp(caps, "Elevated", CapabilitiesService.IsUserAdministrator());
		
		// CPU
		int totalPhysicalCores = Environment.ProcessorCount;
		int totalLogicalCores = GetNumLogicalCores();
		CapabilitiesService.AddCpuInfo(settings.CurrentValue, caps, GetCpuName(), totalLogicalCores, totalPhysicalCores);
		
		// Memory
		MEMORYSTATUSEX memStatus = new();
		if (GlobalMemoryStatusEx(memStatus))
		{
			int ramGb = (int)(memStatus.ullTotalPhys / (1024 * 1024 * 1024));
			CapabilitiesService.AddMemInfo(settings.CurrentValue, caps, ramGb);
		}
		else
		{
			int error = Marshal.GetLastWin32Error();
			logger.LogWarning("Unable to get size of physical memory via GlobalMemoryStatusEx. Win32 error: {Error}", error);
		}
		
		// GPU
		SetGpuInfo(caps);
		
		return Task.FromResult(caps);
	}
	
	private static void SetProp(RpcAgentCapabilities caps, string key, string value) => caps.Properties.Add($"{key}={value}");
	private static void SetProp(RpcAgentCapabilities caps, string key, bool value) => caps.Properties.Add($"{key}={value}");
	
	/// <summary>
	/// Gets details of current system's physical memory
	/// </summary>
	/// <returns>The total and available physical memory in bytes. Returns 0 if memory status could not be retrieved</returns>
	public static (ulong total, ulong available) GetPhysicalMemory()
	{
		try
		{
			MEMORYSTATUSEX memStatus = new();
			if (GlobalMemoryStatusEx(memStatus))
			{
				return (memStatus.ullTotalPhys, memStatus.ullAvailPhys);
			}
		}
		catch
		{
			// Ignore any error
		}
		return (0, 0);
	}

    private sealed class CpuMetrics
    {
        internal static FILETIME Idle, System, User;

        internal static bool IsFirstRun()
        {
            return Idle.IsZero && System.IsZero && User.IsZero;
        }
    }

    /// <summary>
    /// Gets details of the current system's CPU
    /// </summary>
    /// <returns>Non-zero if the function succeeds, zero if it fails. On success, the output parameters contain cumulative idle, system, and user times.</returns>
    public static (float idle, float system, float user) GetCpuMetrics()
    {
        try
        {
            if (GetSystemTimes(out FILETIME idle, out FILETIME system, out FILETIME user))
            {
                // On first run, assign CPU metrics and return (mimics PerformanceCounter logic).
                if (CpuMetrics.IsFirstRun())
                {
                    // Snapshot CPU metrics and return.
                    CpuMetrics.Idle = idle;
                    CpuMetrics.System = system;
                    CpuMetrics.User = user;
                    return (0.0f, 0.0f, 0.0f);
                }

                // Compute deltas.
                ulong idleDelta = idle.ToUInt64() - CpuMetrics.Idle.ToUInt64();
                ulong systemDelta = system.ToUInt64() - CpuMetrics.System.ToUInt64();
                ulong userDelta = user.ToUInt64() - CpuMetrics.User.ToUInt64();
                ulong total = systemDelta + userDelta;

                // Compute percentages.
                float idlePercentage = total > 0 ? (idleDelta * 100.0f) / total : 0.0f;
                float systemPercentage = total > 0 ? ((systemDelta - idleDelta) * 100.0f) / total : 0.0f;
                float userPercentage = total > 0 ? (userDelta * 100.0f) / total : 0.0f;

                // Update previous snapshots.
                CpuMetrics.Idle = idle;
                CpuMetrics.System = system;
                CpuMetrics.User = user;

                return (idlePercentage, systemPercentage, userPercentage);
            }
        }
        catch
        {
            // Ignore any error.
        }
        return (0.0f, 0.0f, 0.0f);
    }

    private static int GetNumLogicalCores()
	{
		int total = 0;
		ushort groupCount = GetActiveProcessorGroupCount();
		for (ushort i = 0; i < groupCount; i++)
		{
			total += (int)GetActiveProcessorCount(i);
		}
		
		return total;
	}
	
	private static string GetCpuName()
	{
		try
		{
			using RegistryKey? key = Registry.LocalMachine.OpenSubKey(@"HARDWARE\DESCRIPTION\System\CentralProcessor\0");
			if (key != null)
			{
				return key.GetValue("ProcessorNameString")?.ToString() ?? "Unknown CPU";
			}
		}
		catch (Exception)
		{
			// Ignore exception
		}
		
		return "Unknown CPU";
	}
	
	private static void SetGpuInfo(RpcAgentCapabilities capabilities)
	{
		// GUID for display adapters in Windows
		const string GpuRegistryKey = @"SYSTEM\CurrentControlSet\Control\Class\{4d36e968-e325-11ce-bfc1-08002be10318}";
		string[] skipDevices = ["Remote Display", "Basic Display", "Standard VGA", "RDP", "Generic"];
		int index = 0;
		
		using RegistryKey? baseKey = Registry.LocalMachine.OpenSubKey(GpuRegistryKey);
		if (baseKey == null)
		{
			return;
		}
		
		foreach (string subKeyName in baseKey.GetSubKeyNames())
		{
			if (subKeyName.Equals("Properties", StringComparison.OrdinalIgnoreCase))
			{
				continue;
			}
			
			using RegistryKey? deviceKey = baseKey.OpenSubKey(subKeyName);
			if (deviceKey == null)
			{
				continue;
			}
			
			string? deviceDesc = deviceKey.GetValue("DriverDesc") as string;
			if (String.IsNullOrEmpty(deviceDesc) || skipDevices.Any(term => deviceDesc.Contains(term, StringComparison.Ordinal)))
			{
				continue;
			}
			
			string prefix = $"GPU-{++index}";
			SetProp(capabilities, $"{prefix}-Name", deviceDesc);
			
			string? driverVersion = deviceKey.GetValue("DriverVersion") as string;
			if (!String.IsNullOrEmpty(driverVersion))
			{
				SetProp(capabilities, $"{prefix}-DriverVersion", driverVersion);
			}
		}
	}
	
	#region P/Invoke
	#pragma warning disable
	[DllImport("kernel32.dll")]
	static extern ushort GetActiveProcessorGroupCount();
	
	[DllImport("kernel32.dll")]
	static extern uint GetActiveProcessorCount(ushort GroupNumber);
	
	[DllImport("kernel32.dll", SetLastError = true)]
	[return: MarshalAs(UnmanagedType.Bool)]
	static extern bool GlobalMemoryStatusEx([In, Out] MEMORYSTATUSEX lpBuffer);
	
	[StructLayout(LayoutKind.Sequential)]
	private class MEMORYSTATUSEX
	{
		public uint dwLength = (uint)Marshal.SizeOf(typeof(MEMORYSTATUSEX));
		public uint dwMemoryLoad;
		public ulong ullTotalPhys;
		public ulong ullAvailPhys;
		public ulong ullTotalPageFile;
		public ulong ullAvailPageFile;
		public ulong ullTotalVirtual;
		public ulong ullAvailVirtual;
		public ulong ullAvailExtendedVirtual;
	}

	[DllImport("kernel32.dll", SetLastError = true)]
	[return: MarshalAs(UnmanagedType.Bool)]
	internal static extern bool GetSystemTimes(out FILETIME idle, out FILETIME system, out FILETIME user);

	[StructLayout(LayoutKind.Sequential)]
	internal struct FILETIME
	{
		public uint dwLowDateTime;
		public uint dwHighDateTime;
		public ulong ToUInt64() => ((ulong)dwHighDateTime << 32) | dwLowDateTime;
		public bool IsZero => dwLowDateTime == 0 && dwHighDateTime == 0;
	}
	#pragma warning enable
	
	#endregion P/Invoke
}