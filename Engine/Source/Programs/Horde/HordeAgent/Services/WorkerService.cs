// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using HordeAgent.Leases;
using HordeCommon.Rpc.Messages;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Services
{
	/// <summary>
	/// Interface for WorkerService. Mainly for extracting methods which simplifies testing and dependencies
	/// </summary>
	internal interface IWorkerService
	{
		/// <summary>
		/// Gets all the current leases held by the agent
		/// </summary>
		public List<RpcLease> GetActiveLeases();
		
		/// <summary>
		/// Stop the session loop after at least one lease has finished executing
		/// </summary>
		public void TerminateSessionAfterLease();
		
		/// <summary>
		/// Whether agent is connected to Horde server
		/// </summary>
		public bool IsConnected { get; }
		
		/// <summary>
		/// List of pool IDs this agent (session) currently is a member of
		/// </summary>
		public IReadOnlyList<string> PoolIds { get; }
	}
	
	/// <summary>
	/// Implements the message handling loop for an agent. Runs asynchronously until disposed.
	/// </summary>
	class WorkerService : BackgroundService, IWorkerService
	{
		readonly ILogger _logger;
		readonly ISessionFactory _sessionFactory;
		readonly StatusService _statusService;
		readonly IServiceProvider _serviceProvider;
		LeaseManager? _currentLeaseManager;
		
		/// <inheritdoc/>
		public IReadOnlyList<string> PoolIds => _currentLeaseManager?.PoolIds ?? Array.Empty<string>();

		static readonly TimeSpan[] s_sessionBackOffTime =
		{
			TimeSpan.FromSeconds(5),
			TimeSpan.FromSeconds(10),
			TimeSpan.FromSeconds(30),
			TimeSpan.FromMinutes(1)
		};

		/// <summary>
		/// Constructor. Registers with the server and starts accepting connections.
		/// </summary>
		public WorkerService(ISessionFactory sessionFactory, StatusService statusService, IServiceProvider serviceProvider, ILogger<WorkerService> logger)
		{
			_sessionFactory = sessionFactory;
			_statusService = statusService;
			_logger = logger;
			_serviceProvider = serviceProvider;
		}
		
		/// <inheritdoc/>
		public List<RpcLease> GetActiveLeases()
		{
			return _currentLeaseManager?.GetActiveLeases() ?? new List<RpcLease>();
		}
		
		/// <inheritdoc/>
		public void TerminateSessionAfterLease()
		{
			if (_currentLeaseManager != null)
			{
				_currentLeaseManager.TerminateSessionAfterLease = true;
			}
		}
		
		/// <inheritdoc/>
		public bool IsConnected { get; private set; } = false;
		
		/// <summary>
		/// Executes the ServerTaskAsync method and swallows the exception for the task being cancelled. This allows waiting for it to terminate.
		/// </summary>
		/// <param name="stoppingToken">Indicates that the service is trying to stop</param>
		protected override async Task ExecuteAsync(CancellationToken stoppingToken)
		{
			try
			{
				await ExecuteInnerAsync(stoppingToken);
			}
			catch (Exception ex)
			{
				if (!stoppingToken.IsCancellationRequested)
				{
					_logger.LogCritical(ex, $"Unhandled exception in {nameof(WorkerService)}");	
				}
			}
		}

		/// <summary>
		/// Background task to cycle access tokens and update the state of the agent with the server.
		/// </summary>
		/// <param name="stoppingToken">Indicates that the service is trying to stop</param>
		internal async Task ExecuteInnerAsync(CancellationToken stoppingToken)
		{
			// Show the current client id
			string version = AgentApp.Version;
			_logger.LogInformation("Version: {Version}", version);

			// Print the server info
			_logger.LogInformation("Arguments: {Arguments}", Environment.CommandLine);

			// await TelemetryService.LogProblematicFilterDriversAsync(_logger, stoppingToken);

			// Keep trying to start an agent session with the server
			int failureCount = 0;
			while (!stoppingToken.IsCancellationRequested)
			{
				SessionResult? result = null;
				_statusService.SetDescription(AgentStatusMessage.Starting);

				Stopwatch sessionTime = Stopwatch.StartNew();

				await using (AsyncServiceScope scope = _serviceProvider.CreateAsyncScope())
				{
					try
					{
						Task<IDisposable> mutexTask = SingleInstanceMutex.AcquireAsync("Global\\HordeAgent-DB828ACB-0AA5-4D32-A62A-21D4429B1014", stoppingToken);

						Task delayTask = Task.Delay(TimeSpan.FromSeconds(1.0), stoppingToken);
						if (Task.WhenAny(mutexTask, delayTask) == delayTask)
						{
							_logger.LogInformation("Another agent instance is already running. Waiting for it to terminate.");
						}

						using IDisposable mutex = await mutexTask;

						await using (ISession session = await _sessionFactory.CreateAsync(stoppingToken))
						{
							IsConnected = true;
							_currentLeaseManager = new LeaseManager(session, _serviceProvider);
							result = await _currentLeaseManager.RunAsync(stoppingToken);
							IsConnected = false;
						}

						failureCount = 0;
					}
					catch (OperationCanceledException) when (stoppingToken.IsCancellationRequested)
					{
						throw;
					}
					catch (Exception ex)
					{
						if (sessionTime.Elapsed < TimeSpan.FromMinutes(5.0))
						{
							failureCount++;
						}
						else
						{
							failureCount = 1;
						}

						TimeSpan backOffTime = s_sessionBackOffTime[Math.Min(failureCount - 1, s_sessionBackOffTime.Length - 1)];
						_logger.LogWarning(ex, "Session failure #{FailureNum}. Waiting {Time} and restarting. ({Message})", failureCount, backOffTime, ex.Message);
						_statusService.Set(false, 0, $"Unable to start session: {ex.Message}");
						await Task.Delay(backOffTime, stoppingToken);
					}
				}

				if (result != null)
				{
					if (result.Outcome == SessionOutcome.BackOff)
					{
						await Task.Delay(TimeSpan.FromSeconds(30.0), stoppingToken);
					}
					else if (result.Outcome == SessionOutcome.Terminate)
					{
						break;
					}
					else if (result.Outcome == SessionOutcome.RunCallback)
					{
						await result.CallbackAsync!(_logger, stoppingToken);
					}
				}

				if (sessionTime.Elapsed < TimeSpan.FromSeconds(2.0))
				{
					_logger.LogInformation("Waiting 5 seconds before restarting session...");
					await Task.Delay(TimeSpan.FromSeconds(5.0), stoppingToken);
				}
			}
		}
	}
}
