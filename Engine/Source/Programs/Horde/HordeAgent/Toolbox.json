{
	"manualInstall": true,
	"requiresElevation": true,
	"hidden": true,
	"installCommand": {
		"fileName": "HordeAgent.exe",
		"arguments": ["setup", "install", "-RegisterForUninstall=false"]
	},
	"uninstallCommand": {
		"fileName": "HordeAgent.exe",
		"arguments": ["setup", "uninstall", "-UseRegistryPath=false"]
	}
}