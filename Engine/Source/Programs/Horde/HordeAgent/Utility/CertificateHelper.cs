// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Security;
using System.Security.Cryptography.X509Certificates;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Utility
{
	/// <summary>
	/// Helper functions for dealing with certificates
	/// </summary>
	static class CertificateHelper
	{
		/// <summary>
		/// Provides additional diagnostic information for SSL certificate validation
		/// </summary>
		/// <param name="logger">The logger instance</param>
		/// <param name="sender"></param>
		/// <param name="certificate"></param>
		/// <param name="chain"></param>
		/// <param name="sslPolicyErrors"></param>
		/// <param name="serverProfile">The server profile</param>
		/// <returns>True if the certificate is allowed, false otherwise</returns>
		public static bool CertificateValidationCallBack(ILogger logger, object sender, X509Certificate? certificate, X509Chain? chain, SslPolicyErrors sslPolicyErrors, ServerProfile serverProfile)
		{
			// If the certificate is a valid, signed certificate, return true.
			if (sslPolicyErrors == SslPolicyErrors.None)
			{
				return true;
			}

			// Trust the remote certificate if it has the right thumbprint
			if (sslPolicyErrors == SslPolicyErrors.RemoteCertificateChainErrors)
			{
				if (chain != null && chain.ChainElements.Count == 1)
				{
					X509ChainElement element = chain.ChainElements[0];
					if (element.ChainElementStatus.Length == 1 && (element.ChainElementStatus[0].Status == X509ChainStatusFlags.UntrustedRoot || element.ChainElementStatus[0].Status == X509ChainStatusFlags.PartialChain))
					{
						if (serverProfile.IsTrustedCertificate(element.Certificate.Thumbprint))
						{
							logger.LogDebug("Trusting server certificate {Thumbprint}", element.Certificate.Thumbprint);
							return true;
						}
					}
				}
			}

			// Generate diagnostic information
			StringBuilder builder = new StringBuilder();
			if (sender != null)
			{
				HttpRequestMessage? message = sender as HttpRequestMessage;
				if (message != null)
				{
					builder.Append($"\nSender: {message.Method} {message.RequestUri}");
				}
				else
				{
					string senderInfo = StringUtils.Indent(sender.ToString() ?? String.Empty, "    ");
					builder.Append($"\nSender:\n{senderInfo}");
				}
			}
			if (certificate != null)
			{
				builder.Append($"\nCertificate: {certificate.Subject}");
			}
			if (chain != null)
			{
				if (chain.ChainStatus != null && chain.ChainStatus.Length > 0)
				{
					builder.Append("\nChain status:");
					foreach (X509ChainStatus status in chain.ChainStatus)
					{
						builder.Append($"\n  {status.StatusInformation}");
					}
				}
				if (chain.ChainElements != null)
				{
					builder.Append("\nChain elements:");
					for (int idx = 0; idx < chain.ChainElements.Count; idx++)
					{
						X509ChainElement element = chain.ChainElements[idx];
						builder.Append($"\n  {idx,4} - Certificate: {element.Certificate.Subject}");
						builder.Append($"\n         Thumbprint: {element.Certificate.Thumbprint}");
						if (element.ChainElementStatus != null && element.ChainElementStatus.Length > 0)
						{
							foreach (X509ChainStatus status in element.ChainElementStatus)
							{
								builder.Append($"\n         Status: {status.StatusInformation} ({status.Status})");
							}
						}
						if (!String.IsNullOrEmpty(element.Information))
						{
							builder.Append($"\n         Info: {element.Information}");
						}
					}
				}
			}

			// Print out additional diagnostic information
			logger.LogError("TLS certificate validation failed ({Errors}).{AdditionalInfo}", sslPolicyErrors, StringUtils.Indent(builder.ToString(), "    "));
			return false;
		}
	}
}
