// Copyright Epic Games, Inc. All Rights Reserved.

using Grpc.Core;

namespace HordeAgent.Utility
{
	/// <summary>
	/// Utility methods for exceptions
	/// </summary>
	static class ExceptionUtils
	{
		/// <summary>
		/// Determine if the given exception was triggered due to a cancellation event
		/// </summary>
		/// <param name="ex">The exception to check</param>
		/// <returns>True if the exception is a cancellation exception</returns>
		public static bool IsCancellationException(this Exception ex)
		{
			if (ex is OperationCanceledException)
			{
				return true;
			}

			RpcException? rpcException = ex as RpcException;
			if (rpcException != null && rpcException.StatusCode == StatusCode.Cancelled)
			{
				return true;
			}

			return false;
		}
	}
}
