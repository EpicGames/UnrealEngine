// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;

namespace HordeAgent.Utility
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Utility class for making localized changes to JSON config files
	/// </summary>
	static class JsonConfig
	{
		/// <summary>
		/// Read a JSON config file from disk
		/// </summary>
		public static async Task<JsonObject> ReadAsync(FileReference file)
		{
			byte[] data = await FileReference.ReadAllBytesAsync(file);
			JsonObject? obj = JsonNode.Parse(data, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip }) as JsonObject;
			return obj ?? new JsonObject();
		}

		/// <summary>
		/// Writes the config file to disk
		/// </summary>
		public static async Task WriteAsync(FileReference file, JsonObject root)
		{
			ArrayBufferWriter<byte> buffer = new ArrayBufferWriter<byte>();
			using (Utf8JsonWriter writer = new Utf8JsonWriter(buffer, new JsonWriterOptions { Indented = true }))
			{
				root.WriteTo(writer);
			}
			await FileReference.WriteAllBytesAsync(file, buffer.WrittenMemory.ToArray());
		}

		/// <summary>
		/// Read a node by name
		/// </summary>
		public static T FindOrAddNode<T>(JsonObject obj, string name, Func<T> factory) where T : JsonNode
		{
			JsonNode? node = obj[name];
			if (node != null)
			{
				if (node is T existingTypedNode)
				{
					return existingTypedNode;
				}
				else
				{
					obj.Remove(name);
				}
			}

			T newTypedNode = factory();
			obj.Add(name, newTypedNode);
			return newTypedNode;
		}

		/// <summary>
		/// Find or adds an array element with a particular key
		/// </summary>
		public static JsonObject FindOrAddElementByKey(JsonArray array, string key, string name)
		{
			foreach (JsonNode? element in array)
			{
				if (element is JsonObject obj)
				{
					JsonNode? node = obj[key];
					if (node != null && (string?)node.AsValue() == name)
					{
						return obj;
					}
				}
			}

			JsonObject newObj = new JsonObject();
			newObj[key] = name;
			array.Add(newObj);
			return newObj;
		}
	}
}
