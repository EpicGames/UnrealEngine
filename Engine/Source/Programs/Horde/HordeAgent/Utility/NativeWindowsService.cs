// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.ServiceProcess;

namespace HordeAgent.Utility;

/// <summary>
/// Utility class for Windows services using Win32 API via P/Invoke
/// </summary>
public static class NativeWindowsServiceUtils 
{
	#region P/Invoke
	#pragma warning disable IDE1006 // Naming rule violation
	
	private const uint SC_MANAGER_CONNECT = 0x0001;
	private const uint SC_MANAGER_ENUMERATE_SERVICE = 0x0004;
	private const uint SERVICE_WIN32 = 0x00000030; // SERVICE_WIN32_OWN_PROCESS and SERVICE_WIN32_SHARE_PROCESS combined
	private const uint SERVICE_STATE_ALL = 0x00000003; // SERVICE_ACTIVE and SERVICE_INACTIVE states combined
	private const int ERROR_MORE_DATA = 234;
	private const int SC_ENUM_PROCESS_INFO = 0;
	
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
	private struct SERVICE_STATUS_PROCESS
	{
		public uint dwServiceType;
		public uint dwCurrentState;
		public uint dwControlsAccepted;
		public uint dwWin32ExitCode;
		public uint dwServiceSpecificExitCode;
		public uint dwCheckPoint;
		public uint dwWaitHint;
		public uint dwProcessId;
		public uint dwServiceFlags;
	}
	
	[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
	private struct ENUM_SERVICE_STATUS_PROCESS
	{
		public string lpServiceName;
		public string lpDisplayName;
		public SERVICE_STATUS_PROCESS ServiceStatusProcess;
	}
	
	[DllImport("advapi32.dll", SetLastError = true, CharSet = CharSet.Unicode)]
	private static extern IntPtr OpenSCManager(
		string? machineName,
		string? databaseName,
		uint dwDesiredAccess);
	
	[DllImport("advapi32.dll", SetLastError = true, CharSet = CharSet.Auto)]
	[return: MarshalAs(UnmanagedType.Bool)]
	private static extern bool EnumServicesStatusEx(
		IntPtr hSCManager,
		int InfoLevel, // SC_ENUM_PROCESS_INFO
		uint dwServiceType,
		uint dwServiceState,
		IntPtr lpServices,
		uint cbBufSize,
		out uint pcbBytesNeeded,
		out uint lpServicesReturned,
		IntPtr lpResumeHandle, // Should be 0
		string? pszGroupName); // Should be null
	
	
	[DllImport("advapi32.dll", SetLastError = true)]
	[return: MarshalAs(UnmanagedType.Bool)]
	private static extern bool CloseServiceHandle(IntPtr hSCObject);

	#pragma warning restore IDE1006 // Naming Styles
	#endregion
	
	/// <summary>
	/// Try to find the service controller for the given process ID using Win32 API.
	/// </summary>
	/// <param name="processId">The process ID to search for</param>
	/// <returns>The service controller corresponding to this process, or null if not found</returns>
	[SupportedOSPlatform("windows")]
	public static ServiceController? GetServiceForProcess(int processId)
	{
		if (processId <= 0)
		{
			return null;
		}
		
		IntPtr scmHandle = IntPtr.Zero;
		IntPtr buffer = IntPtr.Zero;
		uint bytesNeeded;
		uint servicesReturned;
		
		try
		{
			scmHandle = OpenSCManager(null, null, SC_MANAGER_CONNECT | SC_MANAGER_ENUMERATE_SERVICE);
			if (scmHandle == IntPtr.Zero)
			{
				throw new InvalidOperationException($"Failed to open service control manager. Win32 Error: {Marshal.GetLastWin32Error()}");
			}
			
			// First call to get the required buffer size
			EnumServicesStatusEx(
				scmHandle,
				SC_ENUM_PROCESS_INFO,
				SERVICE_WIN32,
				SERVICE_STATE_ALL,
				IntPtr.Zero,
				0,
				out bytesNeeded,
				out servicesReturned, // Unused in this call
				IntPtr.Zero,
				null);
			
			int lastError = Marshal.GetLastWin32Error();
			if (lastError != ERROR_MORE_DATA && !(lastError == 0 && bytesNeeded == 0))
			{
				throw new InvalidOperationException($"Failed to query buffer size for services. Win32 Error: {lastError}");
			}
			
			if (bytesNeeded == 0)
			{
				// Zero buffer size needed means nothing to process, no services to be found
				return null;
			}
			
			// Allocate memory for the services information
			buffer = Marshal.AllocHGlobal((int)bytesNeeded);
			if (buffer == IntPtr.Zero)
			{
				throw new InvalidOperationException("Failed to allocate memory for service list.");
			}
			
			// Get the actual data
			if (!EnumServicesStatusEx(scmHandle,
				    SC_ENUM_PROCESS_INFO,
				    SERVICE_WIN32,
				    SERVICE_STATE_ALL,
				    buffer,
				    bytesNeeded,
				    out _,
				    out servicesReturned,
				    IntPtr.Zero,
				    null))
			{
				throw new InvalidOperationException($"Failed to enumerate services. Win32 Error: {Marshal.GetLastWin32Error()}");
			}
			
			IntPtr currentPtr = buffer;
			int structSize = Marshal.SizeOf(typeof(ENUM_SERVICE_STATUS_PROCESS));
			
			for (int i = 0; i < servicesReturned; i++)
			{
				ENUM_SERVICE_STATUS_PROCESS serviceInfo = (ENUM_SERVICE_STATUS_PROCESS)Marshal.PtrToStructure(currentPtr, typeof(ENUM_SERVICE_STATUS_PROCESS))!;
				uint servicePid = serviceInfo.ServiceStatusProcess.dwProcessId;
				if (servicePid != 0 && servicePid == processId)
				{
					return new ServiceController(serviceInfo.lpServiceName);
				}
				
				currentPtr = IntPtr.Add(currentPtr, structSize);
			}
		}
		finally
		{
			if (buffer != IntPtr.Zero)
			{
				Marshal.FreeHGlobal(buffer);
			}
			
			if (scmHandle != IntPtr.Zero)
			{
				CloseServiceHandle(scmHandle);
			}
		}
		
		return null;
	}
}