// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;

namespace HordeAgent.Utility
{
	/// <summary>
	/// Utility functions for mapping network shares
	/// </summary>
	static class NetworkShare
	{
		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		private struct NETRESOURCE
		{
			public uint dwScope;
			public uint dwType;
			public uint dwDisplayType;
			public uint dwUsage;
			public string? lpLocalName;
			public string? lpRemoteName;
			public string? lpComment;
			public string? lpProvider;
		};

		const uint RESOURCETYPE_DISK = 1;
		const uint CONNECT_TEMPORARY = 4;

		const int ERROR_ALREADY_ASSIGNED = 85;
		const int ERROR_MORE_DATA = 234;

		[DllImport("Mpr.dll", CharSet = CharSet.Unicode)]
		private static extern int WNetAddConnection2W(ref NETRESOURCE lpNetResource, string? lpPassword, string? lpUsername, uint dwFlags);

		[DllImport("Mpr.dll", CharSet = CharSet.Unicode)]
		private static extern int WNetCancelConnectionW(string lpName, [MarshalAs(UnmanagedType.Bool)] bool fForce);

		[DllImport("Mpr.dll", CharSet = CharSet.Unicode)]
		private static extern int WNetGetConnectionW(string lpLocalName, IntPtr lpRemoteName, ref int lpnLength);

		/// <summary>
		/// Mounts a network share at the given location
		/// </summary>
		/// <param name="mountPoint">Mount point for the share</param>
		/// <param name="remotePath">Path to the remote resource</param>
		public static void Mount(string mountPoint, string remotePath)
		{
			NETRESOURCE netResource = new NETRESOURCE();
			netResource.dwType = RESOURCETYPE_DISK;
			netResource.lpLocalName = mountPoint;
			netResource.lpRemoteName = remotePath;

			int result = WNetAddConnection2W(ref netResource, null, null, CONNECT_TEMPORARY);
			if (result != 0)
			{
				if (result == ERROR_ALREADY_ASSIGNED)
				{
					string? curRemotePath;
					if (TryGetRemotePath(mountPoint, out curRemotePath))
					{
						if (curRemotePath.Equals(remotePath, StringComparison.OrdinalIgnoreCase))
						{
							return;
						}
						else
						{
							throw new Win32Exception(result, $"Unable to mount network share {remotePath} as {mountPoint} ({result}). Currently connected to {curRemotePath}.");
						}
					}
				}
				throw new Win32Exception(result, $"Unable to mount network share {remotePath} as {mountPoint} ({result}: {new Win32Exception(result).Message})");
			}
		}

		/// <summary>
		/// Unmounts a network share
		/// </summary>
		/// <param name="mountPoint">The mount point to remove</param>
		public static void Unmount(string mountPoint)
		{
			int result = WNetCancelConnectionW(mountPoint, true);
			if (result != 0)
			{
				throw new Win32Exception(result, $"Unable to unmount {mountPoint} ({result})");
			}
		}

		/// <summary>
		/// Gets the currently mounted path for a network share
		/// </summary>
		/// <param name="mountPoint">The mount point</param>
		/// <param name="outRemotePath">Receives the remote path</param>
		/// <returns>True if the remote path is returned</returns>
		public static bool TryGetRemotePath(string mountPoint, [NotNullWhen(true)] out string? outRemotePath)
		{
			int length = 260 * sizeof(char);
			for (; ; )
			{
				IntPtr remotePath = Marshal.AllocHGlobal(length);
				try
				{
					int prevLength = length;
					int result = WNetGetConnectionW(mountPoint, remotePath, ref length);
					if (result == 0)
					{
						outRemotePath = Marshal.PtrToStringUni(remotePath)!;
						return outRemotePath != null;
					}
					else if (result == ERROR_MORE_DATA && length > prevLength)
					{
						continue;
					}
					else
					{
						outRemotePath = null!;
						return false;
					}
				}
				finally
				{
					Marshal.FreeHGlobal(remotePath);
				}
			}
		}
	}
}
