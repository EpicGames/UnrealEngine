// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.Metrics;
using EpicGames.Horde.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Exporter;
using OpenTelemetry.Logs;
using OpenTelemetry.Metrics;
using OpenTelemetry.Resources;
using OpenTelemetry.Trace;
using Serilog.Core;

namespace HordeAgent.Utility;

/// <summary>
/// Serilog event enricher attaching trace and span ID for Datadog using current System.Diagnostics.Activity
/// </summary>
public class OpenTelemetryDatadogLogEnricher : ILogEventEnricher
{
	/// <inheritdoc />
	public void Enrich(Serilog.Events.LogEvent logEvent, ILogEventPropertyFactory propertyFactory)
	{
		if (Activity.Current != null)
		{
			string stringTraceId = Activity.Current.TraceId.ToString();
			string stringSpanId = Activity.Current.SpanId.ToString();
			string ddTraceId = Convert.ToUInt64(stringTraceId.Substring(16), 16).ToString();
			string ddSpanId = Convert.ToUInt64(stringSpanId, 16).ToString();

			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.trace_id", ddTraceId));
			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.span_id", ddSpanId));
		}
	}
}

/// <summary>
/// Helper for configuring OpenTelemetry
/// </summary>
public static class OpenTelemetryHelper
{
	private static ResourceBuilder? s_resourceBuilder;
	private static readonly HttpRequestOptionsKey<string> s_urlTemplateKey = new("url.template");
	
	/// <summary>
	/// Name of resource name attribute in Datadog
	/// Some traces use this for prettier display inside their UI
	/// </summary>
	public const string DatadogResourceAttribute = "resource.name";
	
	/// <summary>
	/// Name of default Horde tracer (aka activity source)
	/// </summary>
	public const string HordeName = "Horde";
	
	/// <summary>
	/// List of all source names configured in this class.
	/// They are needed at startup when initializing OpenTelemetry
	/// </summary>
	public static string[] SourceNames => new[] { HordeName };
	
	/// <summary>
	/// Default tracer used in Horde
	/// Prefer dependency-injected tracer over this static member.
	/// </summary>
	public static readonly Tracer Horde = TracerProvider.Default.GetTracer(HordeName);

	/// <summary>
	/// Configure OpenTelemetry in Horde and ASP.NET
	/// </summary>
	/// <param name="services">Service collection for DI</param>
	/// <param name="settings">Current server settings</param>
	public static void Configure(IServiceCollection services, OpenTelemetrySettings settings)
	{
		// Always configure tracers/meters as they are used in the codebase even when OpenTelemetry is not configured
		services.AddSingleton(Horde);
		services.AddSingleton(OpenTelemetryMeters.Horde);

		if (!settings.Enabled)
		{
			return;
		}

		services.AddOpenTelemetry()
			.WithTracing(builder => ConfigureTracing(builder, settings))
			.WithMetrics(builder => ConfigureMetrics(builder, settings));
	}

	private static void ConfigureTracing(TracerProviderBuilder builder, OpenTelemetrySettings settings)
	{
		void DatadogHttpRequestEnricher(Activity activity, HttpRequestMessage request)
		{
			activity.SetTag("service.name", settings.ServiceName + "-http-client");
			activity.SetTag("operation.name", "http.request");
			string url = $"{request.Method} {request.Headers.Host}{request.RequestUri?.LocalPath}";
			if (request.Options.TryGetValue(s_urlTemplateKey, out string? template))
			{
				activity.DisplayName = $"{request.Method} {template}";
			}
			else
			{
				activity.DisplayName = $"{request.Method} {request.Headers.Host}";
			}
			activity.SetTag("resource.name", url);
		}

		builder
			.AddSource(SourceNames)
			.AddHttpClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			.AddGrpcClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			.SetResourceBuilder(GetResourceBuilder(settings));

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	private static void ConfigureMetrics(MeterProviderBuilder builder, OpenTelemetrySettings settings)
	{
		builder.AddMeter(OpenTelemetryMeters.MeterNames);
		builder.AddHttpClientInstrumentation();

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	/// <summary>
	/// Configure .NET logging with OpenTelemetry
	/// </summary>
	/// <param name="builder">Logging builder to modify</param>
	/// <param name="settings">Current server settings</param>
	public static void ConfigureLogging(ILoggingBuilder builder, OpenTelemetrySettings settings)
	{
		if (!settings.Enabled)
		{
			return;
		}

		builder.AddOpenTelemetry(options =>
		{
			options.IncludeScopes = true;
			options.IncludeFormattedMessage = true;
			options.ParseStateValues = true;
			options.SetResourceBuilder(GetResourceBuilder(settings));

			if (settings.EnableConsoleExporter)
			{
				options.AddConsoleExporter();
			}
		});
	}

	private static ResourceBuilder GetResourceBuilder(OpenTelemetrySettings settings)
	{
		if (s_resourceBuilder != null)
		{
			return s_resourceBuilder;
		}

		List<KeyValuePair<string, object>> attributes = settings.Attributes.Select(x => new KeyValuePair<string, object>(x.Key, x.Value)).ToList();
		s_resourceBuilder = ResourceBuilder.CreateDefault()
			.AddService(settings.ServiceName, serviceNamespace: settings.ServiceNamespace, serviceVersion: settings.ServiceVersion)
			.AddAttributes(attributes)
			.AddTelemetrySdk()
			.AddEnvironmentVariableDetector();

		return s_resourceBuilder;
	}
}

/// <summary>
/// Static initialization of all available OpenTelemetry meters
/// </summary>
public static class OpenTelemetryMeters
{
	/// <summary>
	/// Name of default Horde meter
	/// </summary>
	public const string HordeName = "Horde";

	/// <summary>
	/// List of all source names configured in this class.
	/// They are needed at startup when initializing OpenTelemetry
	/// </summary>
	public static string[] MeterNames => new[] { HordeName };

	/// <summary>
	/// Default meter used in Horde
	/// </summary>
	public static readonly Meter Horde = new(HordeName);
}