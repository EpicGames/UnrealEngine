// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeAgent.Utility;

/// <summary>
/// Monitor execution of the process and .NET runtime
/// Horde agent runs in an environment where resources are often maximally utilized (CPU, memory, disk, network etc)
/// This can lead to unexpected behaviors with delays or timeouts.
/// This class self-monitors and ensures a heartbeat thread and task is "on time".
/// </summary>
public sealed class RuntimeWatchdog : IHostedService, IDisposable
{
	internal TimeSpan? ArtificialThreadDelay { get; set; } = null;
	internal TimeSpan? ArtificialTaskDelay { get; set; } = null;
	
	private long _lastTaskHeartbeatTicks;
	private readonly Thread _heartbeatThread;
	private Task? _heartbeatTask;
	private volatile bool _isRunning;
	private readonly TimeSpan _heartbeatThreadInterval = TimeSpan.FromSeconds(10);
	private readonly TimeSpan _heartbeatTaskInterval = TimeSpan.FromSeconds(5);
	private readonly TimeSpan _maxTaskStallDuration;
	private readonly TimeSpan _maxThreadStallDuration;
	private readonly TimeSpan _maxGcPauseDelta = TimeSpan.FromSeconds(3);
	private readonly ILogger<RuntimeWatchdog> _logger;
	private readonly CancellationTokenSource _cts = new();

	/// <summary>
	/// Constructor
	/// </summary>
	public RuntimeWatchdog(ILogger<RuntimeWatchdog> logger)
	{
		_maxThreadStallDuration = _heartbeatThreadInterval + TimeSpan.FromSeconds(5);
		_maxTaskStallDuration = _heartbeatTaskInterval + TimeSpan.FromSeconds(5);
		_lastTaskHeartbeatTicks = DateTime.UtcNow.Ticks;
		_heartbeatThread = new Thread(HeartbeatThreadLoop) { Name = "RuntimeWatchdog", IsBackground = true, Priority =  ThreadPriority.AboveNormal };
		_logger = logger;
	}
	
	/// <inheritdoc/>
	public void Dispose()
	{
		_cts.Dispose();
	}
	
	/// <inheritdoc/>
	public Task StartAsync(CancellationToken cancellationToken)
	{
		if (!_isRunning)
		{
			_isRunning = true;
			_lastTaskHeartbeatTicks = DateTime.UtcNow.Ticks;
			_heartbeatThread.Start();
			_heartbeatTask = Task.Run(() => HeartbeatTaskLoopAsync(_cts.Token), cancellationToken);
		}
		return Task.CompletedTask;
	}

	/// <inheritdoc/>
	public async Task StopAsync(CancellationToken cancellationToken)
	{
		if (_isRunning)
		{
			_isRunning = false;
			await _cts.CancelAsync();
			if (_heartbeatTask != null)
			{
				await _heartbeatTask;
			}
		}
	}

	private void HeartbeatThreadLoop()
	{
		try
		{
			while (_isRunning)
			{
				TimeSpan lastGcPauseDuration = GC.GetTotalPauseDuration();
				DateTime lastThreadHeartbeat = DateTime.UtcNow;
			
				Thread.Sleep(_heartbeatThreadInterval);
				if (ArtificialThreadDelay != null)
				{
					Thread.Sleep(ArtificialThreadDelay.Value);
				}
			
				TimeSpan heartbeatThreadDuration = DateTime.UtcNow - lastThreadHeartbeat;
				if (heartbeatThreadDuration > _maxThreadStallDuration)
				{
					_logger.LogWarning("Thread loop stalled for {StallTime} ms", (int)heartbeatThreadDuration.TotalMilliseconds);
				}
			
				long ticks = Interlocked.Read(ref _lastTaskHeartbeatTicks);
				TimeSpan heartbeatTaskDuration = DateTime.UtcNow - new DateTime(ticks);
				if (heartbeatTaskDuration > _maxTaskStallDuration)
				{
					_logger.LogWarning("Task loop stalled for {StallTime} ms", (int)heartbeatTaskDuration.TotalMilliseconds);
				}
			
				TimeSpan gcPauseDelta = GC.GetTotalPauseDuration() - lastGcPauseDuration;
				if (gcPauseDelta > _maxGcPauseDelta)
				{
					_logger.LogWarning("GC paused process for {PauseDuration} ms over time window {TimeWindow}", (int)gcPauseDelta.TotalMilliseconds, (int)heartbeatThreadDuration.TotalMilliseconds);
				}
			}
		}
		catch (Exception e)
		{
			_logger.LogError(e, "Error in watchdog thread loop");
		}
		
		_logger.LogInformation("Thread heartbeat loop ended");
	}

	private async Task HeartbeatTaskLoopAsync(CancellationToken cancellationToken)
	{
		try
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Interlocked.Exchange(ref _lastTaskHeartbeatTicks, DateTime.UtcNow.Ticks);
				await Task.Delay(_heartbeatTaskInterval, cancellationToken);
				if (ArtificialTaskDelay != null)
				{
					Thread.Sleep(ArtificialTaskDelay.Value);
				}
			}
		}
		catch (OperationCanceledException)
		{
			// Ignore
		}
		
		_logger.LogInformation("Task heartbeat loop ended");
	}
}