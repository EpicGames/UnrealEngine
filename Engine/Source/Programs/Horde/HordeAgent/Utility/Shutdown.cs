// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using Microsoft.Extensions.Logging;
using EpicGames.Core;

namespace HordeAgent.Utility
{
	static class Shutdown
	{
		public const uint TOKEN_QUERY = 0x0008;
		public const uint TOKEN_ADJUST_PRIVILEGES = 0x0020;

		[DllImport("advapi32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool OpenProcessToken(IntPtr processHandle, uint desiredAccess, out IntPtr tokenHandle);

		[StructLayout(LayoutKind.Sequential)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		public struct LUID
		{
			public uint LowPart;
			public int HighPart;
		}

		[DllImport("advapi32.dll")]
		static extern bool LookupPrivilegeValue(string? lpSystemName, string lpName, ref LUID lpLuid);

		[StructLayout(LayoutKind.Sequential, Pack = 4)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		public struct LUID_AND_ATTRIBUTES
		{
			public LUID Luid;
			public uint Attributes;
		}

		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		struct TOKEN_PRIVILEGES
		{
			public int PrivilegeCount;
			[MarshalAs(UnmanagedType.ByValArray, SizeConst = 1)]
			public LUID_AND_ATTRIBUTES[] Privileges;
		}

		// Use this signature if you do not want the previous state
		[DllImport("advapi32.dll", SetLastError = true)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		[return: MarshalAs(UnmanagedType.Bool)]
		static extern bool AdjustTokenPrivileges(IntPtr TokenHandle,
		   [MarshalAs(UnmanagedType.Bool)] bool DisableAllPrivileges,
		   ref TOKEN_PRIVILEGES NewState,
		   uint Zero,
		   IntPtr Null1,
		   IntPtr Null2);

		const uint SHTDN_REASON_MAJOR_APPLICATION = 0x00040000;
		const uint SHTDN_REASON_MINOR_MAINTENANCE = 0x00000001;

		[DllImport("advapi32.dll", CharSet = CharSet.Auto, SetLastError = true)]
		static extern bool InitiateSystemShutdownEx(string? lpMachineName, string? lpMessage, uint dwTimeout, bool bForceAppsClosed, bool bRebootAfterShutdown, uint dwReason);

		const string SE_SHUTDOWN_NAME = "SeShutdownPrivilege";

		const int SE_PRIVILEGE_ENABLED = 0x00000002;

		/// <summary>
		/// Initiate a shutdown operation
		/// </summary>
		/// <param name="restartAfterShutdown">Whether to restart after the shutdown</param>
		/// <param name="logger">Logger for the operation</param>
		/// <returns></returns>
		public static bool InitiateShutdown(bool restartAfterShutdown, ILogger logger)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && !RuntimePlatform.IsRunningOnWine())
			{
				if (restartAfterShutdown)
				{
					logger.LogInformation("Triggering restart");
				}
				else
				{
					logger.LogInformation("Triggering shutdown");
				}

				IntPtr tokenHandle;
				if (!OpenProcessToken(Process.GetCurrentProcess().Handle, TOKEN_ADJUST_PRIVILEGES | TOKEN_QUERY, out tokenHandle))
				{
					logger.LogError("OpenProcessToken() failed (code 0x{Code:x8})", Marshal.GetLastWin32Error());
					return false;
				}

				// Get the LUID for the shutdown privilege. 
				LUID luid = new LUID();
				if (!LookupPrivilegeValue(null, SE_SHUTDOWN_NAME, ref luid))
				{
					logger.LogError("LookupPrivilegeValue() failed (code 0x{Code:x8})", Marshal.GetLastWin32Error());
					return false;
				}

				TOKEN_PRIVILEGES privileges = new TOKEN_PRIVILEGES();
				privileges.PrivilegeCount = 1;
				privileges.Privileges = new LUID_AND_ATTRIBUTES[1];
				privileges.Privileges[0].Attributes = SE_PRIVILEGE_ENABLED;
				privileges.Privileges[0].Luid = luid;

				if (!AdjustTokenPrivileges(tokenHandle, false, ref privileges, 0, IntPtr.Zero, IntPtr.Zero))
				{
					logger.LogError("AdjustTokenPrivileges() failed (code 0x{Code:x8})", Marshal.GetLastWin32Error());
					return false;
				}

				uint dialogTimeout = 0; // The length of time that the shutdown dialog box should be displayed, in seconds
				if (!InitiateSystemShutdownEx(null, "HordeAgent has initiated shutdown", dialogTimeout, true, restartAfterShutdown, SHTDN_REASON_MAJOR_APPLICATION | SHTDN_REASON_MINOR_MAINTENANCE))
				{
					logger.LogError("Shutdown failed (0x{Code:x8})", Marshal.GetLastWin32Error());
					return false;
				}
				return true;
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX) || RuntimeInformation.IsOSPlatform(OSPlatform.Linux) || RuntimePlatform.IsRunningOnWine())
			{
				string shutdownArgs;
				if (restartAfterShutdown)
				{
					shutdownArgs = "sudo shutdown -r +0 \"Horde Agent is restarting\"";
				}
				else
				{
					shutdownArgs = "sudo shutdown +0 \"Horde Agent is shutting down\"";
				}

				using (Process shutdownProcess = new Process())
				{
					DataReceivedEventHandler handler = new DataReceivedEventHandler((_, args) =>
					{
						if (!String.IsNullOrEmpty(args.Data))
						{
							logger.LogInformation("{Output}", args.Data);
						}
					});

					shutdownProcess.StartInfo.FileName = "/bin/sh";
					shutdownProcess.StartInfo.ArgumentList.Add("-c");
					shutdownProcess.StartInfo.ArgumentList.Add(shutdownArgs);
					shutdownProcess.StartInfo.UseShellExecute = false;
					shutdownProcess.StartInfo.CreateNoWindow = true;
					shutdownProcess.StartInfo.RedirectStandardOutput = true;
					shutdownProcess.StartInfo.RedirectStandardError = true;
					shutdownProcess.ErrorDataReceived += handler;
					shutdownProcess.OutputDataReceived += handler;

					logger.LogInformation("Running {Command} {Arguments}", shutdownProcess.StartInfo.FileName, shutdownProcess.StartInfo.Arguments);

					shutdownProcess.Start();
					shutdownProcess.BeginOutputReadLine();
					shutdownProcess.BeginErrorReadLine();
					shutdownProcess.WaitForExit();

					int exitCode = shutdownProcess.ExitCode;
					if (exitCode != 0)
					{
						logger.LogError("Shutdown failed ({ExitCode})", exitCode);
						return false;
					}

					logger.LogInformation("Exit code {ExitCode}", exitCode);
				}

				return true;
			}
			else
			{
				logger.LogError("Shutdown is not implemented on this platform");
				return false;
			}
		}

		/// <summary>
		/// Attempt to initiate a shutdown and return if it failed
		/// </summary>
		/// <param name="restart">Whether to restart</param>
		/// <param name="logger">Logger for output</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task ExecuteAsync(bool restart, ILogger logger, CancellationToken cancellationToken)
		{
			logger.LogInformation("Initiating shutdown (restart={Restart})", restart);

			if (Shutdown.InitiateShutdown(restart, logger))
			{
				for (int idx = 10; idx > 0; idx--)
				{
					logger.LogInformation("Waiting for shutdown ({Count})", idx);
					try
					{
						await Task.Delay(TimeSpan.FromMinutes(2.0), cancellationToken);
						logger.LogInformation("Shutdown aborted.");
					}
					catch (OperationCanceledException)
					{
						logger.LogInformation("Agent is shutting down.");
						return;
					}
				}
			}
		}
	}
}
