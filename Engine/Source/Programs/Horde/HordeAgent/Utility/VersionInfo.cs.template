// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;

namespace HordeAgent.Utility;

/// <summary>
/// Encodes version by replacing the version pattern below
/// Platform-specific binaries built for Linux and macOS do not encode this information like Windows binaries do
/// </summary>

public static class VersionInfo
{
	/// <summary>
	/// Version (the pattern is replaced prior to compilation)
	/// </summary>
	private const string VersionConstant = "$VERSION$";
	
	/// <summary>
	/// Tool ID (the pattern is replaced prior to compilation)
	/// </summary>
	private const string ToolIdConstant = "$TOOL_ID$";

	/// <summary>
	/// Version of this Horde Agent
	/// </summary>
	public static string Version => String.IsNullOrEmpty(VersionConstant) ? "unknown" : VersionConstant;
	
	/// <summary>
	/// Self-reported tool ID of this agent. This ID will override and define exact variant of agent software any server-initiated upgrades will use.
	/// </summary>
	[SuppressMessage("Maintainability", "CA1508:Avoid dead conditional code", Justification = "Generated code")]
	public static string? ToolId => ToolIdConstant == "unknown" || String.IsNullOrEmpty(ToolIdConstant) ? null : ToolIdConstant;
}