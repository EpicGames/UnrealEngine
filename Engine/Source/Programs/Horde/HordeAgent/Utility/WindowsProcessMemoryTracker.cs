// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.ComponentModel;
using System.Diagnostics;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Win32.SafeHandles;

namespace HordeAgent.Utility
{
	/// <summary>
	/// Memory information for a process, tracking peak values observed across all samples
	/// </summary>
	/// <param name="ProcessId">The process ID</param>
	/// <param name="ProcessName">Name of the process executable (without extension)</param>
	/// <param name="CommandLine">Command line arguments (truncated to MaxCommandLineLength)</param>
	/// <param name="PeakPrivateBytes">Maximum PrivateMemorySize64 observed - best for leak detection</param>
	/// <param name="PeakWorkingSetBytes">Maximum WorkingSet64 observed - RAM impact</param>
	[SupportedOSPlatform("windows")]
	public record ProcessMemoryInfo(
		int ProcessId,
		string ProcessName,
		string CommandLine,
		long PeakPrivateBytes,
		long PeakWorkingSetBytes);

	/// <summary>
	/// Tracks memory usage for all processes in a Windows Job Object.
	/// Samples at configurable intervals and maintains peak memory values per process.
	/// </summary>
	[SupportedOSPlatform("windows")]
	public sealed class WindowsProcessMemoryTracker : IDisposable
	{
		/// <summary>
		/// Maximum length for stored command line strings
		/// </summary>
		public const int MaxCommandLineLength = 500;

		#region Native Interop

#pragma warning disable IDE1006 // Naming Styles
#pragma warning disable CS0649 // Field never assigned

		const int JobObjectBasicProcessIdList = 3;

		[StructLayout(LayoutKind.Sequential)]
		struct JOBOBJECT_BASIC_PROCESS_ID_LIST
		{
			public uint NumberOfAssignedProcesses;
			public uint NumberOfProcessIdsInList;
			// Followed by array of UIntPtr ProcessIdList[1]
		}

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool QueryInformationJobObject(
			SafeFileHandle hJob,
			int JobObjectInformationClass,
			IntPtr lpJobObjectInfo,
			int cbJobObjectInfoLength,
			out int lpReturnLength);

#pragma warning restore CS0649
#pragma warning restore IDE1006

		#endregion

		readonly ManagedProcessGroup _processGroup;
		readonly TimeSpan _sampleInterval;
		readonly ILogger _logger;

		readonly ConcurrentDictionary<int, ProcessMemoryInfo> _peakByPid = new();
		Task? _samplingTask;
		CancellationTokenSource? _cts;

		/// <summary>
		/// Creates a new memory tracker for the specified process group
		/// </summary>
		/// <param name="processGroup">The managed process group containing the job object to monitor</param>
		/// <param name="sampleInterval">How often to sample process memory (e.g., 30 seconds)</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public WindowsProcessMemoryTracker(
			ManagedProcessGroup processGroup,
			TimeSpan sampleInterval,
			ILogger logger)
		{
			_processGroup = processGroup ?? throw new ArgumentNullException(nameof(processGroup));
			_sampleInterval = sampleInterval;
			_logger = logger ?? throw new ArgumentNullException(nameof(logger));
		}

		/// <summary>
		/// Starts the background memory sampling task
		/// </summary>
		public void Start()
		{
			if (_samplingTask != null)
			{
				throw new InvalidOperationException("Memory tracker is already running");
			}

			_cts = new CancellationTokenSource();
			_samplingTask = SamplingLoopAsync(_cts.Token);
		}

		/// <summary>
		/// Returns the top N processes by peak private memory
		/// </summary>
		/// <param name="count">Number of processes to return (default 5)</param>
		/// <returns>List of process memory info sorted by peak private bytes descending</returns>
		public IReadOnlyList<ProcessMemoryInfo> GetTopMemoryProcesses(int count = 5)
		{
			return _peakByPid.Values
				.OrderByDescending(p => p.PeakPrivateBytes)
				.Take(count)
				.ToList();
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_cts != null)
			{
				_cts.Cancel();

				if (_samplingTask != null)
				{
					try
					{
#pragma warning disable VSTHRD002 // Synchronous wait is unavoidable in Dispose
						_samplingTask.GetAwaiter().GetResult();
#pragma warning restore VSTHRD002
					}
					catch (OperationCanceledException)
					{
						// Expected
					}
				}

				_cts.Dispose();
				_cts = null;
			}
		}

		async Task SamplingLoopAsync(CancellationToken cancellationToken)
		{
			// Do an immediate sample at start
			SampleProcessMemory();

			using PeriodicTimer timer = new(_sampleInterval);
			while (await timer.WaitForNextTickAsync(cancellationToken))
			{
				SampleProcessMemory();
			}
		}

		void SampleProcessMemory()
		{
			try
			{
				int[] pids = GetJobProcessIds();
				foreach (int pid in pids)
				{
					SampleProcess(pid);
				}
			}
			catch (Exception ex)
			{
				_logger.LogDebug(ex, "Error during memory sampling");
			}
		}

		int[] GetJobProcessIds()
		{
			SafeFileHandle? jobHandle = _processGroup.JobHandle;
			if (jobHandle == null || jobHandle.IsInvalid || jobHandle.IsClosed)
			{
				return [];
			}

			// Start with space for 64 processes, grow if needed
			int bufferSize = Marshal.SizeOf<JOBOBJECT_BASIC_PROCESS_ID_LIST>() + (64 * IntPtr.Size);
			IntPtr buffer = Marshal.AllocHGlobal(bufferSize);

			try
			{
				while (true)
				{
					if (QueryInformationJobObject(
						jobHandle,
						JobObjectBasicProcessIdList,
						buffer,
						bufferSize,
						out int returnLength))
					{
						JOBOBJECT_BASIC_PROCESS_ID_LIST info = Marshal.PtrToStructure<JOBOBJECT_BASIC_PROCESS_ID_LIST>(buffer);

						// Check if we need more space
						if (info.NumberOfAssignedProcesses > info.NumberOfProcessIdsInList)
						{
							bufferSize = Marshal.SizeOf<JOBOBJECT_BASIC_PROCESS_ID_LIST>() + ((int)info.NumberOfAssignedProcesses * IntPtr.Size);
							IntPtr newBuffer = Marshal.AllocHGlobal(bufferSize);
							Marshal.FreeHGlobal(buffer);
							buffer = newBuffer;
							continue;
						}

						// Read the PIDs
						int[] pids = new int[info.NumberOfProcessIdsInList];
						IntPtr pidArrayStart = buffer + Marshal.SizeOf<JOBOBJECT_BASIC_PROCESS_ID_LIST>();

						for (int i = 0; i < info.NumberOfProcessIdsInList; i++)
						{
							IntPtr pidPtr = Marshal.ReadIntPtr(pidArrayStart, i * IntPtr.Size);
							pids[i] = (int)pidPtr.ToInt64();
						}

						return pids;
					}
					else
					{
						int error = Marshal.GetLastWin32Error();
						// ERROR_MORE_DATA = 234
						if (error == 234)
						{
							bufferSize *= 2;
							IntPtr newBuffer = Marshal.AllocHGlobal(bufferSize);
							Marshal.FreeHGlobal(buffer);
							buffer = newBuffer;
							continue;
						}

						_logger.LogDebug("QueryInformationJobObject failed with error {Error}", error);
						return [];
					}
				}
			}
			finally
			{
				Marshal.FreeHGlobal(buffer);
			}
		}

		void SampleProcess(int pid)
		{
			try
			{
				using Process process = Process.GetProcessById(pid);

				long privateBytes = process.PrivateMemorySize64;
				long workingSet = process.WorkingSet64;
				string processName = process.ProcessName;
				string commandLine = TruncateCommandLine(ProcessUtils.GetProcessCommandLine(pid) ?? string.Empty);

				_peakByPid.AddOrUpdate(pid,
					// Add new entry
					_ => new ProcessMemoryInfo(
						pid,
						processName,
						commandLine,
						privateBytes,
						workingSet),
					// Update existing - keep max values
					(_, existing) => existing with
					{
						PeakPrivateBytes = Math.Max(existing.PeakPrivateBytes, privateBytes),
						PeakWorkingSetBytes = Math.Max(existing.PeakWorkingSetBytes, workingSet)
					});
			}
			catch (ArgumentException)
			{
				// Process exited
			}
			catch (InvalidOperationException)
			{
				// Process exited during query
			}
			catch (Win32Exception)
			{
				// Access denied
			}
		}

		static string TruncateCommandLine(string commandLine)
		{
			if (string.IsNullOrEmpty(commandLine))
			{
				return string.Empty;
			}

			if (commandLine.Length <= MaxCommandLineLength)
			{
				return commandLine;
			}

			return commandLine[..(MaxCommandLineLength - 3)] + "...";
		}
	}
}
