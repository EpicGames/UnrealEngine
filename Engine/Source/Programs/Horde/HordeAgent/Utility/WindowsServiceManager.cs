// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.InteropServices;
using Microsoft.Win32.SafeHandles;

namespace HordeAgent.Utility
{
	/// <summary>
	/// Native methods for manipulating services on Windows
	/// </summary>
	static partial class Native
	{
#pragma warning disable CS0649
		public class ServiceHandle : SafeHandleZeroOrMinusOneIsInvalid
		{
			public ServiceHandle()
				: base(true)
			{
			}

			public ServiceHandle(IntPtr handle)
				: base(true)
			{
				SetHandle(handle);
			}

			protected override bool ReleaseHandle()
			{
				return Native.CloseServiceHandle(handle);
			}
		}

		public const uint SC_MANAGER_ALL_ACCESS = 0xF003F;

		[DllImport("advapi32.dll", EntryPoint = "OpenSCManagerW", ExactSpelling = true, CharSet = CharSet.Unicode, SetLastError = true)]
		public static extern ServiceHandle OpenSCManager(string? machineName, string? databaseName, uint dwAccess);

		public const uint SERVICE_ALL_ACCESS = 0xf01ff;

		public const uint SERVICE_WIN32_OWN_PROCESS = 0x00000010;

		public const uint SERVICE_AUTO_START = 0x00000002;
		public const uint SERVICE_DEMAND_START = 0x00000003;

		public const uint SERVICE_ERROR_NORMAL = 0x00000001;

		[DllImport("advapi32.dll", SetLastError = true, CharSet = CharSet.Auto)]
		public static extern ServiceHandle CreateService(ServiceHandle hSCManager, string lpServiceName, string lpDisplayName, uint dwDesiredAccess, uint dwServiceType, uint dwStartType, uint dwErrorControl, string lpBinaryPathName, string? lpLoadOrderGroup, string? lpdwTagId, string? lpDependencies, string? lpServiceStartName, string? lpPassword);

		[DllImport("advapi32.dll", SetLastError = true, CharSet = CharSet.Auto)]
		public static extern ServiceHandle OpenService(ServiceHandle hSCManager, string lpServiceName, uint dwDesiredAccess);

		[DllImport("advapi32", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool StartService(ServiceHandle hService, int dwNumServiceArgs, string[]? lpServiceArgVectors);

		public const int ERROR_ACCESS_DENIED = 5;
		public const int ERROR_SERVICE_DOES_NOT_EXIST = 1060;
		public const int ERROR_SERVICE_NOT_ACTIVE = 1062;

		[DllImport("advapi32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool CloseServiceHandle(IntPtr hSCObject);

		[DllImport("advapi32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool DeleteService(ServiceHandle hService);

		public const int SC_STATUS_PROCESS_INFO = 0;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public struct SERVICE_STATUS_PROCESS
		{
			public uint dwServiceType;
			public uint dwCurrentState;
			public uint dwControlsAccepted;
			public uint dwWin32ExitCode;
			public uint dwServiceSpecificExitCode;
			public uint dwCheckPoint;
			public uint dwWaitHint;
			public uint dwProcessId;
			public uint dwServiceFlags;
		}

		[DllImport("advapi32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool QueryServiceStatusEx(ServiceHandle hService, int infoLevel, ref SERVICE_STATUS_PROCESS pBuffer, int cbBufSize, out int pcbBytesNeeded);

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "<Pending>")]
		public struct SERVICE_STATUS
		{
			public uint dwServiceType;
			public uint dwCurrentState;
			public uint dwControlsAccepted;
			public uint dwWin32ExitCode;
			public uint dwServiceSpecificExitCode;
			public uint dwCheckPoint;
			public uint dwWaitHint;
		}

		public const int SERVICE_CONTROL_STOP = 1;

		[DllImport("advapi32.dll", SetLastError = true)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool ControlService(ServiceHandle hService, int dwControl, ref SERVICE_STATUS lpServiceStatus);

		public const int SERVICE_CONFIG_DESCRIPTION = 1;

		[StructLayout(LayoutKind.Sequential)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		public struct SERVICE_DESCRIPTION
		{
			[MarshalAs(UnmanagedType.LPWStr)]
			public string lpDescription;
		}

		public const int SERVICE_CONFIG_FAILURE_ACTIONS = 0x2;

		[StructLayout(LayoutKind.Sequential)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		public struct SERVICE_FAILURE_ACTIONS
		{
			public int dwResetPeriod;

			[MarshalAs(UnmanagedType.LPWStr)]
			public string? lpRebootMsg;

			[MarshalAs(UnmanagedType.LPWStr)]
			public string? lpCommand;

			public int cActions;
			public IntPtr lpsaActions;
		}

		[StructLayout(LayoutKind.Sequential)]
		[SuppressMessage("Style", "IDE1006:Naming Styles", Justification = "Native struct")]
		public struct SERVICE_ACTION
		{
			public WindowsServiceRecoverAction type;
			public uint dwDelay;
		}

		[DllImport("advapi32.dll", SetLastError = true, CharSet = CharSet.Auto)]
		[return: MarshalAs(UnmanagedType.Bool)]
		public static extern bool ChangeServiceConfig2(ServiceHandle hService, int dwInfoLevel, IntPtr lpInfo);
#pragma warning restore CS0649

	}

	enum WindowsServiceRecoverAction : int
	{
		/// <summary>
		/// No Recover Action
		/// </summary>
		None = 0,

		/// <summary>
		/// Restart the service.
		/// </summary>
		Restart = 1,

		/// <summary>
		/// Reboot the host.
		/// </summary>
		Reboot = 2,

		/// <summary>
		/// Run a command
		/// </summary>
		RunCommand = 3
	}

	enum WindowsServiceStatus : uint
	{
		/// <summary>
		/// The service has stopped.
		/// </summary>
		Stopped = 1,

		/// <summary>
		/// The service is starting.
		/// </summary>
		Starting = 2,

		/// <summary>
		/// The service is stopping.
		/// </summary>
		Stopping = 3,

		/// <summary>
		/// The service is running
		/// </summary>
		Running = 4,

		/// <summary>
		/// The service is about to continue.
		/// </summary>
		Continuing = 5,

		/// <summary>
		/// The service is pausing
		/// </summary>
		Pausing = 6,

		/// <summary>
		/// The service is paused.
		/// </summary>
		Paused = 7,
	}

	/// <summary>
	/// Wrapper around 
	/// </summary>
	class WindowsService : IDisposable
	{
		/// <summary>
		/// Handle to the service
		/// </summary>
		private readonly Native.ServiceHandle _handle;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="handle">Handle to the service</param>
		public WindowsService(Native.ServiceHandle handle)
		{
			_handle = handle;
		}

		/// <summary>
		/// Determines if the service is valid
		/// </summary>
		public bool IsValid => !_handle.IsInvalid;

		/// <summary>
		/// Gets the current service status
		/// </summary>
		/// <returns>Status code</returns>
		public WindowsServiceStatus GetStatus()
		{
			Native.SERVICE_STATUS_PROCESS status = new Native.SERVICE_STATUS_PROCESS();
			if (!Native.QueryServiceStatusEx(_handle, Native.SC_STATUS_PROCESS_INFO, ref status, Marshal.SizeOf(status), out int _))
			{
				throw new Win32Exception(String.Format("Unable to query process status (0x{0:X8)}", Marshal.GetLastWin32Error()));
			}
			return (WindowsServiceStatus)status.dwCurrentState;
		}

		/// <summary>
		/// Waits for the status to change
		/// </summary>
		/// <param name="transitionStatus">Expected status while transitioning</param>
		/// <param name="maxWaitTime">Maximum time to wait</param>
		public WindowsServiceStatus WaitForStatusChange(WindowsServiceStatus transitionStatus, TimeSpan maxWaitTime)
		{
			Stopwatch timer = Stopwatch.StartNew();
			for (; ; )
			{
				WindowsServiceStatus status = GetStatus();
				if (status != transitionStatus || timer.Elapsed > maxWaitTime)
				{
					return status;
				}
				else
				{
					Thread.Sleep(1000);
				}
			}
		}

		/// <summary>
		/// Sets the service description
		/// </summary>
		/// <param name="description">Description for the service</param>
		public void SetDescription(string description)
		{
			Native.SERVICE_DESCRIPTION descriptionData = new Native.SERVICE_DESCRIPTION();
			descriptionData.lpDescription = description;

			nint descriptionDataPointer = Marshal.AllocHGlobal(Marshal.SizeOf(descriptionData));
			Marshal.StructureToPtr(descriptionData, descriptionDataPointer, false);

			try
			{
				Native.ChangeServiceConfig2(_handle, Native.SERVICE_CONFIG_DESCRIPTION, descriptionDataPointer);
			}
			finally
			{
				Marshal.FreeHGlobal(descriptionDataPointer);
			}
		}

		/// <summary>
		/// Set the recovery actions for the windows service
		/// </summary>
		/// <param name="recoveryActions"></param>
		/// <returns></returns>
		public void ChangeServiceFailureActions(params WindowsServiceRecoverAction[] recoveryActions)
		{
			const int ActionsCount = 3;
			const int ActionsSize = ActionsCount * sizeof(int) * 2;

			if (recoveryActions == null)
			{
				throw new InvalidDataException("No recoveryActions specified, they must be specified.");
			}

			if (recoveryActions.Length != ActionsCount)
			{
				throw new InvalidDataException($"recoveryActions should contain {ActionsCount} and only contains {recoveryActions.Length} actions.");
			}

			// Represents SERVICE_ACTION a collection of ints
			// Back of restart time for each retry attempt to accommodate for server/network issues
			int[] serviceActions =
			{
				(int)recoveryActions[0], 10,
				(int)recoveryActions[1], 30,
				(int)recoveryActions[2], 60,
			};

			nint actionsBuffer = Marshal.AllocHGlobal(ActionsSize);
			Marshal.Copy(serviceActions, 0, actionsBuffer, serviceActions.Length);

			Native.SERVICE_FAILURE_ACTIONS serviceFailureActions = new()
			{
				cActions = ActionsCount,
				dwResetPeriod = (int)TimeSpan.FromDays(30).TotalSeconds,
				lpCommand = null,
				lpRebootMsg = null,
				lpsaActions = new IntPtr(actionsBuffer)
			};

			nint serviceActionsPointer = Marshal.AllocHGlobal(Marshal.SizeOf(serviceFailureActions));
			Marshal.StructureToPtr(serviceFailureActions, serviceActionsPointer, false);

			try
			{
				bool result = Native.ChangeServiceConfig2(_handle, Native.SERVICE_CONFIG_FAILURE_ACTIONS, serviceActionsPointer);
				if (!result)
				{
					throw new Win32Exception($"Unable to set Service Failure Actions (error 0x{Marshal.GetLastWin32Error():X8})");
				}
			}
			finally
			{
				Marshal.FreeHGlobal(actionsBuffer);
				Marshal.FreeHGlobal(serviceActionsPointer);
			}
		}

		/// <summary>
		/// Starts the service
		/// </summary>
		public void Start()
		{
			if (!Native.StartService(_handle, 0, null))
			{
				throw new Win32Exception($"Unable to start service (error 0x{Marshal.GetLastWin32Error():X8})");
			}
		}

		/// <summary>
		/// Stops the service
		/// </summary>
		public void Stop()
		{
			Native.SERVICE_STATUS status = new Native.SERVICE_STATUS();
			if (!Native.ControlService(_handle, Native.SERVICE_CONTROL_STOP, ref status))
			{
				int error = Marshal.GetLastWin32Error();
				if (error != Native.ERROR_SERVICE_NOT_ACTIVE)
				{
					throw new Win32Exception($"Unable to stop service (error 0x{error:X8})");
				}
			}
		}

		/// <summary>
		/// Deletes the service
		/// </summary>
		public void Delete()
		{
			if (!Native.DeleteService(_handle))
			{
				throw new Win32Exception($"Unable to delete service (error 0x{Marshal.GetLastWin32Error():X8})");
			}
		}

		/// <summary>
		/// Dispose of the service handle
		/// </summary>
		public void Dispose()
		{
			_handle.Close();
		}
	}

	/// <summary>
	/// Helper functionality for manipulating Windows services
	/// </summary>
	class WindowsServiceManager : IDisposable
	{
		/// <summary>
		/// Native handle to the service manager
		/// </summary>
		private readonly Native.ServiceHandle _serviceManagerHandle;

		/// <summary>
		/// Constructor. Opens a handle to the service manager.
		/// </summary>
		public WindowsServiceManager()
		{
			_serviceManagerHandle = Native.OpenSCManager(null, null, Native.SC_MANAGER_ALL_ACCESS);
			if (_serviceManagerHandle.IsInvalid)
			{
				int errorCode = Marshal.GetLastWin32Error();
				if (errorCode == Native.ERROR_ACCESS_DENIED)
				{
					throw new Win32Exception("Unable to open service manager (access denied). Check you're running as administrator.");
				}
				else
				{
					throw new Win32Exception($"Unable to open service manager (0x{errorCode:X8}).");
				}
			}
		}

		/// <summary>
		/// Dispose of this object
		/// </summary>
		public void Dispose()
		{
			_serviceManagerHandle.Close();
		}

		/// <summary>
		/// Opens a service with the given name
		/// </summary>
		/// <param name="serviceName">Name of the service</param>
		/// <returns>New service wrapper</returns>
		public WindowsService Open(string serviceName)
		{
			Native.ServiceHandle serviceHandle = Native.OpenService(_serviceManagerHandle, serviceName, Native.SERVICE_ALL_ACCESS);
			if (serviceHandle.IsInvalid)
			{
				int errorCode = Marshal.GetLastWin32Error();
				if (errorCode != Native.ERROR_SERVICE_DOES_NOT_EXIST)
				{
					throw new Win32Exception("Unable to open handle to service");
				}
			}
			return new WindowsService(serviceHandle);
		}

		/// <summary>
		/// Creates a service with the given settings
		/// </summary>
		/// <param name="name">Name of the service</param>
		/// <param name="displayName">Display name</param>
		/// <param name="commandLine">Command line to use when starting the service</param>
		/// <param name="userName">Username to run this service as</param>
		/// <param name="password">Password for the account the service is to run under</param>
		/// <returns>New service instance</returns>
		public WindowsService Create(string name, string displayName, string commandLine, string? userName, string? password)
		{
			Native.ServiceHandle newServiceHandle = Native.CreateService(_serviceManagerHandle, name, displayName, Native.SERVICE_ALL_ACCESS, Native.SERVICE_WIN32_OWN_PROCESS, Native.SERVICE_AUTO_START, Native.SERVICE_ERROR_NORMAL, commandLine, null, null, null, userName, password);
			if (newServiceHandle.IsInvalid)
			{
				throw new Win32Exception(String.Format("Unable to create service (0x{0:X8})", Marshal.GetLastWin32Error()));
			}
			return new WindowsService(newServiceHandle);
		}
	}
}
