{
	"Horde": 
	{
		"Installed": false,
		"EnableAwsEc2Support": false,
		"WriteStepOutputToLogger": true
	}
}
