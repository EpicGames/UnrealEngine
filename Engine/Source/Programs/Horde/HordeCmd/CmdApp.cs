// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.InteropServices;
using EpicGames.Core;
using EpicGames.Horde;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Serilog;
using Serilog.Formatting.Json;
using Serilog.Sinks.SystemConsole.Themes;

namespace Horde
{
	class CmdApp
	{
		const string ToolDescription = "Horde Command-Line Tool";

		static DirectoryReference DataDir { get; } = GetDataDir();

		static async Task<int> Main(string[] args)
		{
			CommandLineArguments arguments = new CommandLineArguments(args);
			IConfiguration configuration = new ConfigurationBuilder()
				.AddJsonFile("appsettings.json", optional: false)
				.AddJsonFile("appsettings.User.json", optional: true)
				.AddEnvironmentVariables()
				.Build();

			using ILoggerFactory loggerFactory = CreateLoggerFactory(configuration, arguments);

			IServiceCollection services = new ServiceCollection();
			services.Configure<HordeOptions>(options => configuration.Bind("Horde", options));
			services.AddCommandsFromAssembly(Assembly.GetExecutingAssembly());
			services.AddSingleton(loggerFactory);
			services.AddLogging();
			services.AddMemoryCache();
			services.AddHorde();

			// Execute all the commands
			await using ServiceProvider serviceProvider = services.BuildServiceProvider();
			return await CommandHost.RunAsync(arguments, serviceProvider, null, ToolDescription);
		}

		static DirectoryReference GetAppDir()
		{
			return new DirectoryReference(Path.GetDirectoryName(Assembly.GetExecutingAssembly().Location)!);
		}

		static DirectoryReference GetDataDir()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				DirectoryReference? programDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData);
				if (programDataDir != null)
				{
					return DirectoryReference.Combine(programDataDir, "Epic", "Horde", "Tool");
				}
			}
			return GetAppDir();
		}

		public static ILoggerFactory CreateLoggerFactory(IConfiguration configuration, CommandLineArguments arguments)
		{
			Serilog.ILogger logger = CreateSerilogLogger(configuration, arguments);
			return new Serilog.Extensions.Logging.SerilogLoggerFactory(logger, true);
		}

		static Serilog.ILogger CreateSerilogLogger(IConfiguration configuration, CommandLineArguments arguments)
		{
			DirectoryReference.CreateDirectory(CmdApp.DataDir);

			ConsoleTheme theme;
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && Environment.OSVersion.Version < new Version(10, 0))
			{
				theme = SystemConsoleTheme.Literate;
			}
			else
			{
				theme = AnsiConsoleTheme.Code;
			}

			Serilog.Events.LogEventLevel consoleLevel = Serilog.Events.LogEventLevel.Information;
			if (arguments.HasOption("-quiet"))
			{
				consoleLevel = Serilog.Events.LogEventLevel.Warning;
			}
			if (arguments.HasOption("-logdebug"))
			{
				consoleLevel = Serilog.Events.LogEventLevel.Debug;
			}
			if (arguments.HasOption("-logtrace"))
			{
				consoleLevel = Serilog.Events.LogEventLevel.Verbose;
			}

			return new LoggerConfiguration()
				.WriteTo.Console(restrictedToMinimumLevel: consoleLevel, outputTemplate: "{Indent}{Message:l}{NewLine}{Exception}", theme: theme)
				.WriteTo.File(FileReference.Combine(CmdApp.DataDir, "Log-.txt").FullName, fileSizeLimitBytes: 50 * 1024 * 1024, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, retainedFileCountLimit: 10)
				.WriteTo.File(new JsonFormatter(renderMessage: true), FileReference.Combine(CmdApp.DataDir, "Log-.json").FullName, fileSizeLimitBytes: 50 * 1024 * 1024, rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, retainedFileCountLimit: 10)
				.ReadFrom.Configuration(configuration)
				.Enrich.FromLogContext()
				.CreateLogger();
		}
	}
}