// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Archives
{
	[Command("archive", "create", "Creates a storage archive from a folder on the local hard drive")]
	class ArchiveCreate : StorageCommandBase
	{
		[CommandLine("-File=")]
		[Description("Output file for the ref. Either -File=.. or -Ref=.. must be set.")]
		public FileReference? File { get; set; }

		[CommandLine("-Ref=")]
		[Description("Output ref for the data. Either -File=.. or -Ref=.. must be set.")]
		public string? Ref { get; set; }

		[CommandLine("-Input=", Required = true)]
		[Description("Input file or directory")]
		public string Input { get; set; } = null!;

		[CommandLine("-Filter=")]
		[Description("Filter for files to include, in P4 syntax (eg. Foo/...).")]
		public string Filter { get; set; } = "...";

		[CommandLine("-CleanOutput")]
		[Description("Clean the output folder before writing any data")]
		public bool CleanOutput { get; set; }

		public ArchiveCreate(HttpStorageClient storageClient, BundleCache bundleCache)
			: base(storageClient, bundleCache)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (CleanOutput && File != null)
			{
				logger.LogInformation("Cleaning {Directory}...", File.Directory);
				FileUtils.ForceDeleteDirectoryContents(File.Directory);
			}

			if (File != null)
			{
				using MemoryMappedFileCache memoryMappedFileCache = new MemoryMappedFileCache();
				IStorageNamespace store = BundleStorageNamespace.CreateFromDirectory(File.Directory, BundleCache, memoryMappedFileCache, logger);
				return await ExecuteInternalAsync(store, logger);
			}
			else if (!String.IsNullOrEmpty(Ref))
			{
				IStorageNamespace store = GetStorageNamespace();
				return await ExecuteInternalAsync(store, logger);
			}
			else
			{
				throw new CommandLineArgumentException("Either -File=... or -Ref=... must be specified.");
			}
		}

		async Task<int> ExecuteInternalAsync(IStorageNamespace store, ILogger logger)
		{
			// Gather the input files
			DirectoryReference baseDir;
			List<FileReference> files = new List<FileReference>();

			if (System.IO.File.Exists(Input))
			{
				FileReference file = new FileReference(Input);
				baseDir = file.Directory;
				files.Add(file);
			}
			else if (Directory.Exists(Input))
			{
				baseDir = new DirectoryReference(Input);
				FileFilter filter = new FileFilter(Filter.Split(';'));
				files.AddRange(filter.ApplyToDirectory(baseDir, true));
			}
			else
			{
				logger.LogError("{Path} does not exist", Input);
				return 1;
			}

			// Create the bundle
			Stopwatch timer = Stopwatch.StartNew();

			await using (IBlobWriter writer = store.CreateBlobWriter())
			{
				ChunkingOptions options = new ChunkingOptions();

				List<FileInfo> fileInfos = files.ConvertAll(x => x.ToFileInfo());
				UpdateStatsLogger updateStatsLogger = new UpdateStatsLogger(files.Count, fileInfos.Sum(x => x.Length), logger);

				IHashedBlobRef<DirectoryNode> nodeRef = await writer.AddFilesAsync(baseDir.ToDirectoryInfo(), fileInfos, options, updateStatsLogger, CancellationToken.None);

				await writer.FlushAsync();

				if (File != null)
				{
					logger.LogInformation("Writing {File}", File);
					await FileReference.WriteAllTextAsync(File, nodeRef.GetLocator().ToString()!);
				}
				else
				{
					logger.LogInformation("Writing ref {Ref}", Ref);
					await store.AddRefAsync(new RefName(Ref!), nodeRef);
				}
			}

			logger.LogInformation("Time: {Time}", timer.Elapsed.TotalSeconds);
			return 0;
		}
	}
}
