// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage.Bundles.V1;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Archives
{
	[Command("archive", "dump", "Dumps the contents of a bundle")]
	internal class ArchiveDump : Command
	{
		[CommandLine("-Input=", Required = true)]
		[Description("Path to the bundle to display information for.")]
		public FileReference Input { get; set; } = null!;

		[CommandLine("-Verbose")]
		[Description("Include more detailed information about the bundle.")]
		public bool Verbose { get; set; }

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			byte[] data = await FileReference.ReadAllBytesAsync(Input);
			logger.LogInformation("Summary for blob {Location}", Input);

			BundleHeader header = BundleHeader.Read(data);

			string[] types = new string[header.Types.Count];
			for (int idx = 0; idx < header.Types.Count; idx++)
			{
				string name = header.Types[idx].Guid.ToString();
				types[idx] = $"{name} v{header.Types[idx].Version}";
			}

			int packetStart = 0;

			logger.LogInformation("");
			logger.LogInformation("Imports: {NumImports}", header.Imports.Count);

			for (int importIdx = 0; importIdx < header.Imports.Count; importIdx++)
			{
				logger.LogInformation("  IMP [{ImportIdx}] = {BlobId}", importIdx, header.Imports[importIdx]);
			}

			logger.LogInformation("");
			logger.LogInformation("Exports: {NumExports}", header.Exports.Count);

			int packetIdx = 0;
			int packetOffset = 0;
			for (int exportIdx = 0; exportIdx < header.Exports.Count; exportIdx++)
			{
				BundleExport export = header.Exports[exportIdx];
				logger.LogInformation("  EXP [{ExportIdx}] EXP = hash: {ExportHash}, type: {Type}, length: {NumBytes:n0}, packet: {PacketIdx}", exportIdx, export.Hash, types[export.TypeIdx], export.Length, packetIdx);

				if (Verbose)
				{
					for (int referenceIdx = 0; referenceIdx < export.References.Count; referenceIdx++)
					{
						BundleExportRef exportRef = export.References[referenceIdx];
						logger.LogInformation("            REF {RefIdx,-3} -> {Node}", referenceIdx, $"{header.Imports[exportRef.ImportIdx]}#{exportRef.NodeIdx}");
					}
				}

				packetOffset += export.Length;
				if (packetOffset >= header.Packets[packetIdx].DecodedLength)
				{
					packetIdx++;
					packetOffset = 0;
				}
			}

			logger.LogInformation("");
			logger.LogInformation("Packets: {NumPackets}", header.Packets.Count);
			for (int idx = 0; idx < header.Packets.Count; idx++)
			{
				BundlePacket packet = header.Packets[idx];
				logger.LogInformation("  PKT {Idx} (file offset: {Offset:n0}, encoded: {EncodedLength:n0}, decoded: {DecodedLength:n0}, ratio: {Ratio}%)", idx, packetStart, packet.EncodedLength, packet.DecodedLength, (int)(packet.EncodedLength * 100) / packet.DecodedLength);
				packetStart += packet.EncodedLength;
			}

			return 0;
		}
	}
}
