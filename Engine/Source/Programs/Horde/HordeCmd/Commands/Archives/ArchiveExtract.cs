// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Backends;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Archives
{
	[Command("archive", "extract", "Extracts data from a bundle to the local hard drive")]
	internal class ArchiveExtract : StorageCommandBase
	{
		[CommandLine("-File=")]
		[Description("Path to a text file containing the root ref to read. -File=..., -Ref=..., or -Node=... must be specified.")]
		public FileReference? File { get; set; }

		[CommandLine("-Ref=")]
		[Description("Name of a ref to read from the default storage namespace. -File=..., -Ref=..., or -Node=... must be specified.")]
		public string? Ref { get; set; }

		[CommandLine("-Node=")]
		[Description("Locator for a node to read as the root. -File=..., -Ref=..., or -Node=... must be specified.")]
		public string? Node { get; set; }

		[CommandLine("-Stats")]
		[Description("Outputs stats about the extraction process.")]
		public bool Stats { get; set; }

		[CommandLine("-OutputDir=", Required = true)]
		[Description("Directory to write extracted files.")]
		public DirectoryReference OutputDir { get; set; } = null!;

		[CommandLine("-CleanOutput")]
		[Description("If set, deletes the contents of the output directory before extraction.")]
		public bool CleanOutput { get; set; }

		[CommandLine("-VerifyOutput")]
		[Description("Hashes output data to ensure it is as expected")]
		public bool VerifyOutput { get; set; }

		public ArchiveExtract(HttpStorageClient storageClient, BundleCache bundleCache)
			: base(storageClient, bundleCache)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (CleanOutput)
			{
				logger.LogInformation("Deleting contents of {OutputDir}...", OutputDir);
				FileUtils.ForceDeleteDirectoryContents(OutputDir);
			}

			ExtractOptions options = new ExtractOptions();
			options.Progress = new ExtractStatsLogger(logger);
			options.VerifyOutput = VerifyOutput;

			if (File != null)
			{
				using MemoryMappedFileCache memoryMappedFileCache = new MemoryMappedFileCache();
				IStorageNamespace store = BundleStorageNamespace.CreateFromDirectory(File.Directory, BundleCache, memoryMappedFileCache, logger);
				IBlobRef<DirectoryNode> handle = store.CreateBlobRef<DirectoryNode>(await FileStorageBackend.ReadRefAsync(File));
				await ExecuteInternalAsync(store, handle, options, logger);
			}
			else if (Ref != null)
			{
				IStorageNamespace store = GetStorageNamespace();
				IBlobRef<DirectoryNode> handle = await store.ReadRefAsync<DirectoryNode>(new RefName(Ref));
				await ExecuteInternalAsync(store, handle, options, logger);
			}
			else if (Node != null)
			{
				IStorageNamespace store = GetStorageNamespace();
				IBlobRef<DirectoryNode> handle = store.CreateBlobRef<DirectoryNode>(new BlobLocator(Node));
				await ExecuteInternalAsync(store, handle, options, logger);
			}
			else
			{
				throw new CommandLineArgumentException("Either -File=... or -Ref=... must be specified");
			}

			return 0;
		}

		protected async Task ExecuteInternalAsync(IStorageNamespace store, IBlobRef<DirectoryNode> handle, ExtractOptions options, ILogger logger)
		{
			Stopwatch timer = Stopwatch.StartNew();

			await handle.ExtractAsync(OutputDir.ToDirectoryInfo(), options, logger, CancellationToken.None);

			logger.LogInformation("Elapsed: {Time}s", timer.Elapsed.TotalSeconds);

			if (Stats)
			{
				StorageStats stats = store.GetStats();
				stats.Print(logger);
			}
		}
	}
}
