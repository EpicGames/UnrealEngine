// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Artifacts
{
	[Command("artifact", "download", "Downloads an artifact by id")]
	class ArtifactDownload : Command
	{
		[CommandLine("-Id=")]
		[Description("Unique identifier for the artifact")]
		public ArtifactId Id { get; set; }

		[CommandLine("-File=")]
		[Description("Path to a .uartifact file to download")]
		public FileReference? File { get; set; }

		[CommandLine("-OutputDir=", Required = true)]
		[Description("Directory to write extracted files.")]
		public DirectoryReference OutputDir { get; set; } = null!;

		[CommandLine("-Stats")]
		[Description("Outputs stats about the extraction process.")]
		public bool Stats { get; set; }

		[CommandLine("-CleanOutput")]
		[Description("If set, deletes the contents of the output directory before extraction.")]
		public bool CleanOutput { get; set; }

		[CommandLine("-VerifyOutput")]
		[Description("Hashes output data to ensure it is as expected")]
		public bool VerifyOutput { get; set; }

		readonly IHordeClient _hordeClient;

		public ArtifactDownload(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			IStorageNamespace store;
			IBlobRef<DirectoryNode> handle;

			if (File != null && Id == default)
			{
				ArtifactDescriptor descriptor = await ArtifactDescriptor.ReadAsync(File, default);

				Uri serverUrl = new Uri(descriptor.BaseUrl.GetLeftPart(UriPartial.Authority));
				if (serverUrl != _hordeClient.ServerUrl)
				{
					logger.LogError("Artifact references server {ServerUrl}, not the active server {ActiveServerUrl}. Please log in to the correct server.", serverUrl, _hordeClient.ServerUrl);
					return 1;
				}

				logger.LogInformation("Downloading artifact {Url}", descriptor.BaseUrl);

				store = _hordeClient.GetStorageNamespace(descriptor.BaseUrl.AbsolutePath);
				handle = await store.ReadRefAsync<DirectoryNode>(descriptor.RefName);
			}
			else if (File == null && Id != default)
			{
				IArtifact? artifact = await _hordeClient.Artifacts.GetAsync(Id);
				if (artifact == null)
				{
					logger.LogError("Artifact {Id} not found", Id);
					return 1;
				}

				logger.LogInformation("Downloading artifact {Id}: {Description}", Id, artifact.Description);

				store = _hordeClient.GetStorageNamespace(artifact.Id);
				handle = await store.ReadRefAsync<DirectoryNode>(new RefName("default"));
			}
			else
			{
				logger.LogError("Either -Id=... or -File=... must be specified.");
				return 1; 
			}

			if (CleanOutput)
			{
				logger.LogInformation("Deleting contents of {OutputDir}...", OutputDir);
				FileUtils.ForceDeleteDirectoryContents(OutputDir);
			}

			ExtractOptions options = new ExtractOptions();
			options.Progress = new ExtractStatsLogger(logger);
			options.VerifyOutput = VerifyOutput;

			Stopwatch timer = Stopwatch.StartNew();
			await handle.ExtractAsync(OutputDir.ToDirectoryInfo(), options, logger, CancellationToken.None);
			logger.LogInformation("Elapsed: {Time}s", timer.Elapsed.TotalSeconds);

			if (Stats)
			{
				StorageStats stats = store.GetStats();
				stats.Print(logger);
			}

			return 0;
		}
	}
}
