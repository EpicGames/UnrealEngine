// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Bundles
{
	[Command("artifact", "find", "Finds artifacts matching the given query parameters")]
	class ArtifactFind : Command
	{
		[CommandLine("-StreamId=")]
		[Description("Stream containing the artifact")]
		public string? StreamId { get; set; }

		[CommandLine("-Name=")]
		[Description("Name of the artifact")]
		public string? Name { get; set; }

		[CommandLine("-Type=")]
		[Description("Type of artifact to find")]
		public string? Type { get; set; }

		[CommandLine("-MinCommit=")]
		[Description("Minimum commit to include in the results")]
		public string? MinCommit { get; set; }

		[CommandLine("-MaxCommit=")]
		[Description("Maximum commit to include in the results")]
		public string? MaxCommit { get; set; }

		[CommandLine("-Key=")]
		[Description("Artifact keys to search for. Multiple keys may be added to artifacts at upload time, eg. 'job:63dd5487c67f8a45453361c5/step:62ce'.")]
		public List<string> Keys { get; } = new List<string>();

		[CommandLine("-JobId=")]
		[Description("Finds artifacts associated with a particular job id")]
		public string? JobId { get; set; }

		[CommandLine("-StepId=")]
		[Description("Finds artifacts produced by a particular job step. Used in conjunction with -Job.")]
		public string? StepId { get; set; }

		readonly IHordeClient _hordeClient;

		public ArtifactFind(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			StreamId? streamId = null;
			if (!String.IsNullOrEmpty(StreamId))
			{
				streamId = new StreamId(StreamId);
			}

			ArtifactName? name = null;
			if (!String.IsNullOrEmpty(Name))
			{
				name = new ArtifactName(Name);
			}

			ArtifactType? type = null;
			if (!String.IsNullOrEmpty(Type))
			{
				type = new ArtifactType(Type);
			}

			CommitId? minCommitId = null;
			if (!String.IsNullOrEmpty(MinCommit))
			{
				minCommitId = new CommitId(MinCommit);
			}

			CommitId? maxCommitId = null;
			if (!String.IsNullOrEmpty(MaxCommit))
			{
				maxCommitId = new CommitId(MaxCommit);
			}

			if (!String.IsNullOrEmpty(JobId))
			{
				if (String.IsNullOrEmpty(StepId))
				{
					Keys.Add($"job:{JobId}");
				}
				else
				{
					Keys.Add($"job:{JobId}/step:{StepId}");
				}
			}

			await foreach(IArtifact artifact in _hordeClient.Artifacts.FindAsync(streamId, minCommitId, maxCommitId, name, type, Keys))
			{
				logger.LogInformation("");
				logger.LogInformation("Artifact {Id} ({Type})", artifact.Id, artifact.Type);
				foreach (string key in artifact.Keys)
				{
					logger.LogInformation("  {Key}", key);
				}
			}

			return 0;
		}
	}
}
