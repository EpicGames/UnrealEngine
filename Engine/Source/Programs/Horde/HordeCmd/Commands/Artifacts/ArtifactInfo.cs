// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Artifacts
{
	[Command("artifact", "info", "Gets information about an artifact")]
	class ArtifactInfo : Command
	{
		[CommandLine("-Id=")]
		[Description("Unique identifier for the artifact")]
		public ArtifactId Id { get; set; }

		readonly IHordeClient _hordeClient;

		public ArtifactInfo(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			HordeHttpClient httpClient = _hordeClient.CreateHttpClient();

			GetArtifactResponse artifact = await httpClient.GetArtifactAsync(Id);
			logger.LogInformation("Id: {Id}", Id);
			logger.LogInformation("Name: {Name}", artifact.Name);
			logger.LogInformation("Type: {Type}", artifact.Type);
			logger.LogInformation("Stream: {Stream}", artifact.StreamId);
			logger.LogInformation("Commit: {Commit}", artifact.CommitId);
			logger.LogInformation("Description: {Desc}", artifact.Description);

			if (artifact.Keys.Count > 0)
			{
				logger.LogInformation("Keys:");
				foreach (string key in artifact.Keys)
				{
					logger.LogInformation("  {Key}", key);
				}
			}
			if (artifact.Metadata.Count > 0)
			{
				logger.LogInformation("Metadata:");
				foreach (string meta in artifact.Metadata)
				{
					logger.LogInformation("  {Meta}", meta);
				}
			}

			return 0;
		}
	}
}