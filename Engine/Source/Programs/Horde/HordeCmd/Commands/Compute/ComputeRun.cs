// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Reflection;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Compute.Clients;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Compute
{
	/// <summary>
	/// Executes a compute command using a task definition and sandbox taken from the local machine
	/// </summary>
	[Command("compute", "run", "Executes a command through the compute API")]
	class ComputeRun : Command
	{
		class JsonComputeTask
		{
			public string Executable { get; set; } = null!;
			public List<string> Arguments { get; set; } = new List<string>();
			public string WorkingDir { get; set; } = String.Empty;
			public Dictionary<string, string?> EnvVars { get; set; } = new Dictionary<string, string?>();
			public List<string> OutputPaths { get; set; } = new List<string>();
			public string? ContainerImageUrl { get; set; }
			public bool ContainerReplaceEntrypoint { get; set; } = false;
			public bool UseWine { get; set; } = false;
		}

		[CommandLine("-Cluster")]
		[Description("Name of the cluster to run the task on.")]
		public string ClusterId { get; set; } = "default";
		
		[CommandLine("-Pool")]
		[Description("ID of pool to run this task in")]
		public string? Pool { get; set; }

		[CommandLine("-Requirements=")]
		[Description("Query string to select the agent to run on")]
		public string? Requirements { get; set; }

		[CommandLine("-Local")]
		[Description("Uses the local compute client, which runs the remote agent in the same process without using a Horde server.")]
		public bool Local { get; set; }

		[CommandLine("-Loopback")]
		[Description("Runs an agent on the local machine in a separate process, and connect to it on the loopback adapter.")]
		public bool Loopback { get; set; }

		[CommandLine("-InProc")]
		[Description("If true, attempts to load and execute the compute process in the host process. The process to remote must be a .NET assembly invoked through the dotnet command.")]
		public bool InProc { get; set; }

		[CommandLine("-Encryption")]
		[Description("Encryption for communicating with compute task (Aes, Ssl, None).")]
		public string? Encryption { get; set; }

		[CommandLine("-Sandbox=")]
		[Description("Specifies the path to use for the remote sandbox.")]
		public DirectoryReference SandboxDir { get; set; } = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData)!, "Horde", "Sandbox");

		[CommandLine("-Task=", Required = true)]
		[Description("Path to a JSON file describing the workload to execute. See ComputeRun.JsonComputeTask for structure of this document.")]
		FileReference TaskFile { get; set; } = null!;

		readonly IHordeClient _hordeClient;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeRun(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (Local)
			{
				await using LocalComputeClient client = new LocalComputeClient(2000, SandboxDir, InProc, logger);
				return await ExecuteAsync(client, logger);
			}
			else if (Loopback)
			{
				AgentComputeClient client = new AgentComputeClient(Assembly.GetExecutingAssembly().Location, 2000, logger);
				return await ExecuteAsync(client, logger);
			}
			else
			{
				IComputeClient client = _hordeClient.Compute;
				return await ExecuteAsync(client, logger);
			}
		}

		async Task<int> ExecuteAsync(IComputeClient client, ILogger logger)
		{
			Requirements? requirements = null;
			if (Requirements != null)
			{
				requirements = new Requirements(Condition.Parse(Requirements));
			}
			
			if (Pool != null)
			{
				requirements ??= new Requirements();
				requirements.Pool = Pool;
			}

			ConnectionMetadataRequest cmr = new();
			switch (Encryption?.ToUpperInvariant())
			{
				case "SSL": cmr.Encryption = EpicGames.Horde.Compute.Encryption.Ssl; break;
				case "AES": cmr.Encryption = EpicGames.Horde.Compute.Encryption.Aes; break;
				case "NONE": cmr.Encryption = EpicGames.Horde.Compute.Encryption.None; break;
			}

			await using IComputeLease? lease = await client.TryAssignWorkerAsync(new ClusterId(ClusterId), requirements, null, cmr, null, logger, CancellationToken.None);
			if (lease == null)
			{
				throw new Exception("Unable to create lease");
			}

			bool result = await HandleRequestAsync(lease, logger, CancellationToken.None);
			return result ? 0 : 1;
		}

		/// <inheritdoc/>
		async Task<bool> HandleRequestAsync(IComputeLease lease, ILogger logger, CancellationToken cancellationToken)
		{
			const int ControlChannelId = 0;

			// Read the task definition
			byte[] data = await FileReference.ReadAllBytesAsync(TaskFile, cancellationToken);
			JsonComputeTask task = JsonSerializer.Deserialize<JsonComputeTask>(data, new JsonSerializerOptions { AllowTrailingCommas = true, PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase })!;

			// Create a sandbox from the data to be uploaded
			BundleStorageNamespace storage = BundleStorageNamespace.CreateInMemory(logger);
			BlobLocator sandbox = await CreateSandboxAsync(TaskFile, storage, cancellationToken);

			// Open a socket and upload the sandbox
			using (AgentMessageChannel channel = lease.Socket.CreateAgentMessageChannel(ControlChannelId, 4 * 1024 * 1024))
			{
				await channel.WaitForAttachAsync(cancellationToken);
				await channel.UploadFilesAsync("", sandbox, storage.Backend, cancellationToken);

				ExecuteProcessFlags execFlags = ExecuteProcessFlags.None;
				execFlags |= task.UseWine ? ExecuteProcessFlags.UseWine : 0;
				execFlags |= task.ContainerReplaceEntrypoint ? ExecuteProcessFlags.ReplaceContainerEntrypoint : 0;

				await using (AgentManagedProcess process = await channel.ExecuteAsync(task.Executable, task.Arguments, task.WorkingDir, task.EnvVars, execFlags, task.ContainerImageUrl, cancellationToken))
				{
					string? line;
					while ((line = await process.ReadLineAsync(cancellationToken)) != null)
					{
						logger.LogInformation("Child Process: {Text}", line);
					}
				}
			}

			return true;
		}

		static async Task<BlobLocator> CreateSandboxAsync(FileReference taskFile, IStorageNamespace storage, CancellationToken cancellationToken)
		{
			await using IBlobWriter writer = storage.CreateBlobWriter(cancellationToken: cancellationToken);

			IHashedBlobRef<DirectoryNode> sandbox = await writer.AddFilesAsync(taskFile.Directory, cancellationToken: cancellationToken);
			await writer.FlushAsync(cancellationToken);

			return sandbox.GetLocator();
		}
	}
}
