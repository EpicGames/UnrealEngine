// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using Microsoft.Extensions.Logging;

namespace Horde.Commands
{
	[Command("config", "Updates the configuration for the Horde tool")]
	class ConfigCommand : Command
	{
		[CommandLine("-Server=")]
		[Description("Updates the server URL")]
		public string? Server { get; set; }

		/// <inheritdoc/>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			if (Server != null)
			{
				HordeOptions.SetDefaultServerUrl(new Uri(Server));
			}

			logger.LogInformation("Server: {Server}", HordeOptions.GetDefaultServerUrl());
			return Task.FromResult<int>(0);
		}
	}
}
