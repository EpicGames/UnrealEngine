// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace Horde.Commands
{
	[Command("install", "Adds the directory containing the tool to the PATH environment variable.")]
	class InstallCommand : Command
	{
		/// <inheritdoc/>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			Execute(true);
			return Task.FromResult(0);
		}

		internal static void Execute(bool install)
		{
			const string EnvVarName = "PATH";

			FileReference assemblyFile = new FileReference(Assembly.GetExecutingAssembly().GetOriginalLocation());
			DirectoryReference assemblyDir = assemblyFile.Directory;

			string? pathVar = Environment.GetEnvironmentVariable(EnvVarName, EnvironmentVariableTarget.User);
			pathVar ??= String.Empty;

			List<string> paths = new List<string>(pathVar.Split(Path.PathSeparator, StringSplitOptions.RemoveEmptyEntries));

			int changes = paths.RemoveAll(x => x.Equals(assemblyDir.FullName, StringComparison.OrdinalIgnoreCase));
			if (install)
			{
				paths.Add(assemblyDir.FullName);
				changes++;
			}
			if (changes > 0)
			{
				pathVar = String.Join(Path.PathSeparator, paths);
				Environment.SetEnvironmentVariable(EnvVarName, pathVar, EnvironmentVariableTarget.User);
			}
		}
	}
}
