// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Server;
using Microsoft.Extensions.Logging;

namespace Horde.Commands
{
	[Command("login", "Logs in to a Horde server")]
	class LoginCommand : Command
	{
		class GetAuthConfigResponse
		{
			public string? Method { get; set; }
			public string? ServerUrl { get; set; }
			public string? ClientId { get; set; }
			public string[]? RedirectUrls { get; set; }
		}

		[CommandLine("-Token")]
		[Description("Echo the bearer token acquired from the server to stdout")]
		public bool Token { get; set; }

		readonly IHordeClient _hordeClient;

		public LoginCommand(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			using HordeHttpClient httpClient = _hordeClient.CreateHttpClient();

			GetServerInfoResponse serverInfo = await httpClient.GetServerInfoAsync();
			logger.LogInformation("Connected to server version: {Version}", serverInfo.ServerVersion);

			if (Token)
			{
				string? accessToken = await _hordeClient.GetAccessTokenAsync(true, CancellationToken.None);
				if (accessToken != null)
				{
					Console.WriteLine($"Bearer {accessToken}");
				}
			}

			return 0;
		}
	}
}
