// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace Horde.Commands.Perforce
{
	abstract class PerforceBase : Command
	{
		[CommandLine("-ServerAndPort")]
		[Description("Specifies the Perforce server and port")]
		protected string ServerAndPort { get; set; } = PerforceSettings.Default.ServerAndPort;

		[CommandLine("-User")]
		[Description("Specifies the Perforce username")]
		protected string UserName { get; set; } = PerforceSettings.Default.UserName;

		[CommandLine("-BaseDir", Required = true)]
		[Description("Base directory to use for syncing workspaces")]
		protected DirectoryReference BaseDir { get; set; } = null!;

		[CommandLine("-Overwrite")]
		[Description("")]
		protected bool Overwrite { get; set; } = false;

		[CommandLine("-PreferNativeClient")]
		[Description("Prefer to use native Perforce client (instead of launching separate p4 process)")]
		protected bool PreferNativeClient { get; set; } = false;

		[CommandLine("-UseHaveTable")]
		[Description("Use have-table for syncing")]
		string UseHaveTable { get; set; } = "true";

		public override void Configure(CommandLineArguments arguments, ILogger logger)
		{
			base.Configure(arguments, logger);

			if (BaseDir == null)
			{
				for (DirectoryReference? parentDir = DirectoryReference.GetCurrentDirectory(); parentDir != null; parentDir = parentDir.ParentDirectory)
				{
					if (ManagedWorkspace.Exists(parentDir))
					{
						BaseDir = parentDir;
						break;
					}
				}

				if (BaseDir == null)
				{
					throw new FatalErrorException("Unable to find existing repository in current working tree. Specify -BaseDir=... explicitly.");
				}
			}
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			PerforceSettings settings = new(ServerAndPort, UserName) { PreferNativeClient = PreferNativeClient };
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(settings, logger);
			InfoRecord info = await perforce.GetInfoAsync(InfoOptions.ShortOutput, CancellationToken.None);
			bool useHaveTable = UseHaveTable.Equals("true", StringComparison.Ordinal);

			ManagedWorkspaceOptions options = new() { UseHaveTable = useHaveTable };
			ManagedWorkspace repo = await ManagedWorkspace.LoadOrCreateAsync(info.ClientHost!, BaseDir, Overwrite, options, TracerProvider.Default.GetTracer("HordeCmd"), logger, CancellationToken.None);
			await ExecuteAsync(perforce, repo, logger);
			return 0;
		}

		protected abstract Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger);

		protected static long ParseSize(string size)
		{
			long value;
			if (size.EndsWith("gb", StringComparison.OrdinalIgnoreCase))
			{
				string sizeValue = size.Substring(0, size.Length - 2).TrimEnd();
				if (Int64.TryParse(sizeValue, out value))
				{
					return value * (1024 * 1024 * 1024);
				}
			}
			else if (size.EndsWith("mb", StringComparison.OrdinalIgnoreCase))
			{
				string sizeValue = size.Substring(0, size.Length - 2).TrimEnd();
				if (Int64.TryParse(sizeValue, out value))
				{
					return value * (1024 * 1024);
				}
			}
			else if (size.EndsWith("kb", StringComparison.OrdinalIgnoreCase))
			{
				string sizeValue = size.Substring(0, size.Length - 2).TrimEnd();
				if (Int64.TryParse(sizeValue, out value))
				{
					return value * 1024;
				}
			}
			else
			{
				if (Int64.TryParse(size, out value))
				{
					return value;
				}
			}
			throw new FatalErrorException("Invalid size '{0}'", size);
		}

		protected static int ParseChangeNumber(string change)
		{
			int changeNumber;
			if (Int32.TryParse(change, out changeNumber) && changeNumber > 0)
			{
				return changeNumber;
			}
			throw new FatalErrorException("Unable to parse change number from '{0}'", change);
		}

		protected static int ParseChangeNumberOrLatest(string change)
		{
			if (change.Equals("Latest", StringComparison.OrdinalIgnoreCase))
			{
				return -1;
			}
			else
			{
				return ParseChangeNumber(change);
			}
		}

		protected static List<string> ExpandFilters(List<string> arguments)
		{
			List<string> filters = new List<string>();
			foreach (string argument in arguments)
			{
				foreach (string singleArgument in argument.Split(';').Select(x => x.Trim()))
				{
					if (singleArgument.Length > 0)
					{
						filters.Add(singleArgument);
					}
				}
			}
			return filters;
		}
	}
}
