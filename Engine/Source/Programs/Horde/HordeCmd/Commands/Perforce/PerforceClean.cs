// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "clean", "Cleans all modified files from the workspace.")]
	class PerforceClean : PerforceBase
	{
		[CommandLine("-Incremental")]
		[Description("Performs an incremental sync, without removing intermediates")]
		public bool Incremental { get; set; } = false;

		protected override Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			return repo.CleanAsync(perforce, !Incremental, CancellationToken.None);
		}
	}
}
