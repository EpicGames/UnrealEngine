// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "clear", "Empties the staging directory of any files, returning them to the cache")]
	class PerforceClear : PerforceBase
	{
		protected override Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			return repo.ClearAsync(CancellationToken.None);
		}
	}
}
