// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "dump", "Dumps the contents of the repository to the log for analysis")]
	class PerforceDump : PerforceBase
	{
		protected override Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			repo.Dump();
			return Task.CompletedTask;
		}
	}
}
