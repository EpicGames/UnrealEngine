// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "populatecache", "Populates the cache with the head revision of the given streams")]
	class PerforcePopulate : PerforceBase
	{
		[CommandLine("-ClientAndStream=")]
		[Description("Specifies client and stream pairs, in the format Client:Stream")]
		List<string> ClientAndStreamParams { get; set; } = new List<string>();

		[CommandLine("-Filter=")]
		[Description("Filters for the files to sync, in P4 syntax (eg. /Engine/...)")]
		List<string> Filters { get; set; } = new List<string>();

		[CommandLine("-FakeSync")]
		[Description("Simulates the sync without actually fetching any files")]
		bool FakeSync { get; set; } = false;

		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			List<string> expandedFilters = ExpandFilters(Filters);

			List<PopulateRequest> populateRequests = new List<PopulateRequest>();
			foreach (string clientAndStreamParam in ClientAndStreamParams)
			{
				int idx = clientAndStreamParam.IndexOf(':', StringComparison.Ordinal);
				if (idx == -1)
				{
					throw new FatalErrorException("Expected -ClientAndStream=<ClientName>:<StreamName>");
				}

				using IPerforceConnection perforceClient = await perforce.WithClientAsync(clientAndStreamParam.Substring(0, idx));
				populateRequests.Add(new PopulateRequest(perforceClient, clientAndStreamParam.Substring(idx + 1), expandedFilters));
			}

			await repo.PopulateAsync(populateRequests, FakeSync, CancellationToken.None);
		}
	}
}
