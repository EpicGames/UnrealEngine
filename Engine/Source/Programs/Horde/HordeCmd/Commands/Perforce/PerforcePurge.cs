// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "purgecache", "Shrink the size of the cache to the given size")]
	class PerforcePurge : PerforceBase
	{
		[CommandLine("-Size=")]
		[Description("Maximum size of the cache")]
		string? SizeParam { get; set; } = null;

		protected override Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			long size = 0;
			if (SizeParam != null)
			{
				size = ParseSize(SizeParam);
			}

			return repo.PurgeAsync(size, CancellationToken.None);
		}
	}
}
