// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "revert", "Revert all files that are open in the current workspace. Does not replace them with valid revisions.")]
	class PerforceRevert : PerforceBase
	{
		[CommandLine("-Client=", Required = true)]
		[Description("Client to revert all files for")]
		string ClientName { get; set; } = null!;

		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			using IPerforceConnection perforceClient = await perforce.WithClientAsync(ClientName);
			await repo.RevertAsync(perforceClient, CancellationToken.None);
		}
	}
}
