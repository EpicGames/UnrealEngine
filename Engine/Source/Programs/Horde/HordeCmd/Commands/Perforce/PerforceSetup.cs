// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "setup", "Creates or updates a client to use a given stream")]
	class PerforceSetup : PerforceBase
	{
		[CommandLine("-Client=", Required = true)]
		[Description("Name of the client to create")]
		string ClientName { get; set; } = null!;

		[CommandLine("-Stream=", Required = true)]
		[Description("Name of the stream to configure")]
		string StreamName { get; set; } = null!;

		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			using IPerforceConnection perforceClient = await perforce.WithClientAsync(ClientName);
			await repo.SetupAsync(perforceClient, StreamName, CancellationToken.None);
		}
	}
}
