// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "sizes", "Gathers stats for which streams take the most amount of space in the cache for the given configuration")]
	class PerforceSizes : PerforceBase
	{
		[CommandLine("-TempClient=", Required = true)]
		[Description("Name of a temporary client to switch between streams gathering metadata. Will be created if it does not exist.")]
		string TempClientName { get; set; } = null!;

		[CommandLine("-Stream=")]
		[Description("Streams that should be included in the output")]
		List<string> StreamNames { get; set; } = new List<string>();

		[CommandLine("-Filter=")]
		[Description("Filters for the files to sync, in P4 syntax (eg. /Engine/...)")]
		List<string> Filters { get; set; } = new List<string>();

		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			using IPerforceConnection perforceClient = await perforce.WithClientAsync(TempClientName);
			await repo.StatsAsync(perforceClient, StreamNames, Filters, CancellationToken.None);
		}
	}
}
