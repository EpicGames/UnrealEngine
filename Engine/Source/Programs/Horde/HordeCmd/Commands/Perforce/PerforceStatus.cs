// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "status", "Prints information about the state of the cache and workspace")]
	class PerforceStatus : PerforceBase
	{
		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			await repo.StatusAsync();
		}
	}
}
