// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Perforce;
using EpicGames.Perforce.Managed;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Perforce
{
	[Command("perforce", "sync", "Syncs the files for a particular stream and changelist")]
	class PerforceSync : PerforceBase
	{
		[CommandLine("-Client=", Required = true)]
		[Description("Name of the client to sync. Will be created if it does not exist.")]
		public string ClientName { get; set; } = null!;

		[CommandLine("-Stream=", Required = true)]
		[Description("The stream to sync")]
		public string StreamName { get; set; } = null!;

		[CommandLine("-Change=")]
		[Description("The change to sync. May be a changelist number, or 'Latest'")]
		public string Change { get; set; } = "Latest";

		[CommandLine("-Preflight=")]
		[Description("The change to unshelve into the workspace")]
		public int PreflightChange { get; set; } = -1;

		[CommandLine]
		[Description("Optional path to a cache file used to store workspace metadata. Using a location on a network share allows multiple machines syncing the same CL to only query Perforce state once.")]
		FileReference? CacheFile { get; set; } = null;

		[CommandLine("-Filter=")]
		[Description("Filters for the files to sync, in P4 syntax (eg. /Engine/...)")]
		public List<string> Filters { get; set; } = new List<string>();

		[CommandLine("-Incremental")]
		[Description("Performs an incremental sync, without removing intermediates")]
		public bool IncrementalSync { get; set; } = false;

		[CommandLine("-FakeSync")]
		[Description("Simulates the sync without actually fetching any files")]
		public bool FakeSync { get; set; } = false;

		protected override async Task ExecuteAsync(IPerforceConnection perforce, ManagedWorkspace repo, ILogger logger)
		{
			int changeNumber = ParseChangeNumberOrLatest(Change);
			List<string> expandedFilters = ExpandFilters(Filters);

			using IPerforceConnection perforceClient = await perforce.WithClientAsync(ClientName);
			await repo.SyncAsync(perforceClient, StreamName, changeNumber, expandedFilters, !IncrementalSync, FakeSync, CacheFile, CancellationToken.None);

			if (PreflightChange != -1)
			{
				await repo.UnshelveAsync(perforceClient, PreflightChange, CancellationToken.None);
			}
		}
	}
}
