// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using System.Threading.Channels;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Profile
{
	[Command("profile", "chunking", "Profiles chunking over a source directory")]
	class ProfileChunking : StorageCommandBase
	{
		record class ChunkInfo(long Offset, int Length, IoHash Hash);

		[CommandLine("-Input=", Required = true)]
		[Description("Input file or directory")]
		public string Input { get; set; } = null!;

		[CommandLine("-TraceDir=", Required = true)]
		[Description("Directory to write traces to")]
		public DirectoryReference TraceDir { get; set; } = null!;

		[CommandLine("-Verify")]
		public bool Verify { get; set; }

		public ProfileChunking(HttpStorageClient storageClient, BundleCache bundleCache)
			: base(storageClient, bundleCache)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference baseDir = new DirectoryReference(Input);
			List<FileReference> files = DirectoryReference.EnumerateFiles(baseDir, "*", SearchOption.AllDirectories).ToList();

			// Create the bundle
			logger.LogInformation("Creating archive from files in {BaseDir}...", baseDir);

			List<ChunkInfo>? referenceChunks = null;
			for (int idx = 0; ; idx++)
			{
				logger.LogInformation("PARALLEL ATTEMPT {Idx}", idx);
				logger.LogInformation("");

				int suffix = Math.Min(idx, 2);
				List<ChunkInfo> chunks = await ChunkDataAsync(FileReference.Combine(TraceDir, $"output-{suffix}.txt"), files, false, logger);

				logger.LogInformation("-----");

				if (Verify)
				{
					if (referenceChunks == null)
					{
						logger.LogInformation("CREATING REFERENCE DATA");
						logger.LogInformation("");
						referenceChunks = await ChunkDataAsync(FileReference.Combine(TraceDir, "reference.txt"), files, true, logger);
					}

					if (!Enumerable.SequenceEqual(chunks, referenceChunks))
					{
						logger.LogError("MISMATCH");
						break;
					}
				}
			}

			return 0;
		}

		static async Task ReadFilesAsync(List<FileReference> files, ChannelWriter<ChunkerSource> writer, CancellationToken cancellationToken)
		{
			foreach (FileReference file in files)
			{
				FileInfo fileInfo = file.ToFileInfo();
				await writer.WriteAsync(new FileChunkerSource(fileInfo, null), cancellationToken);
			}
			writer.Complete();
		}

		class ChunkReader
		{
			public int _filesRead;
			public int _chunksRead;
			public long _bytesRead;

			readonly ILogger _logger;

			public List<ChunkInfo> _chunks = new List<ChunkInfo>();

			public ChunkReader(ILogger logger)
				=> _logger = logger;

			public async Task ReadChunksAsync(ChannelReader<ChunkerOutput> reader, CancellationToken cancellationToken)
			{
				long lastBytesRead = 0;
				while (await reader.WaitToReadAsync(cancellationToken))
				{
					if (reader.TryRead(out ChunkerOutput? output))
					{
						Interlocked.Increment(ref _filesRead);
						while (await output.MoveNextAsync(cancellationToken))
						{
							_chunks.Add(new ChunkInfo(_bytesRead, (int)output.Data.Length, IoHash.Compute(output.Data.Span)));

							Interlocked.Increment(ref _chunksRead);
							long bytesRead = Interlocked.Add(ref _bytesRead, output.Data.Length);

							if (bytesRead > lastBytesRead + (1024 * 1024 * 256))
							{
								_logger.LogInformation("Read {Size:n0}", bytesRead);
								lastBytesRead = bytesRead;
							}
						}
					}
				}
			}
		}

		static async Task<List<ChunkInfo>> ChunkDataAsync(FileReference file, List<FileReference> files, bool useSerialPipeline, ILogger logger)
		{
			byte[] buffer = new byte[2 * 1024 * 1024];
			Stopwatch timer = Stopwatch.StartNew();

			await using (AsyncPipeline pipeline = new AsyncPipeline(CancellationToken.None))
			{
				LeafChunkedDataNodeOptions options = LeafChunkedDataNodeOptions.Default;// new LeafChunkedDataNodeOptions(4 * 1024, 8 * 1024, 16 * 1024);

				ContentChunker cdc;
				if (useSerialPipeline)
				{
					cdc = new SerialBuzHashChunker(pipeline, buffer, options);
				}
				else
				{
					cdc = new ParallelBuzHashChunker(pipeline, buffer, options);
				}

				_ = pipeline.AddTask(ctx => ReadFilesAsync(files, cdc.SourceWriter, ctx));

				ChunkReader reader = new ChunkReader(logger);
				_ = pipeline.AddTask(ctx => reader.ReadChunksAsync(cdc.OutputReader, ctx));

				await pipeline.WaitForCompletionAsync();

				logger.LogInformation("Files: {Files:n0}", reader._filesRead);
				logger.LogInformation("Chunks: {Chunks:n0}", reader._chunksRead);
				logger.LogInformation("Bytes: {Bytes:n0}", reader._bytesRead);
				logger.LogInformation("Speed: {Rate:n0} bytes/sec", reader._bytesRead / timer.Elapsed.TotalSeconds);
				logger.LogInformation("Time: {Time:n0} sec", timer.Elapsed.TotalSeconds);

				using (StreamWriter writer = new StreamWriter(file.FullName))
				{
					foreach (ChunkInfo chunkInfo in reader._chunks)
					{
						await writer.WriteLineAsync($"{chunkInfo.Offset},{chunkInfo.Length},{chunkInfo.Hash}");
					}
				}

				return reader._chunks;
			}
		}
	}
}

