// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;

namespace Horde.Commands
{
	/// <summary>
	/// Base class for commands that require a configured storage namespace
	/// </summary>
	abstract class StorageCommandBase : Command
	{
		/// <summary>
		/// Namespace to use
		/// </summary>
		[CommandLine("-Namespace=")]
		[Description("Namespace for data to manipulate")]
		public NamespaceId Namespace { get; set; } = new NamespaceId("default");

		/// <summary>
		/// Base URI to upload to
		/// </summary>
		[CommandLine("-Path=")]
		[Description("Relative path on the server for the store to write to/from (eg. api/v1/storage/default)")]
		public string? Path { get; set; }

		/// <summary>
		/// Cache for storage
		/// </summary>
		public BundleCache BundleCache { get; }

		readonly HttpStorageClient _storageClient;

		/// <summary>
		/// Constructor
		/// </summary>
		public StorageCommandBase(HttpStorageClient storageClient, BundleCache bundleCache)
		{
			_storageClient = storageClient;

			BundleCache = bundleCache;
		}

		/// <summary>
		/// Creates a new client instance
		/// </summary>
		public IStorageNamespace GetStorageNamespace()
		{
			if (String.IsNullOrEmpty(Path))
			{
				return _storageClient.GetNamespace(Namespace);
			}
			else
			{
				return _storageClient.GetNamespaceWithPath(Path);
			}
		}
	}
}
