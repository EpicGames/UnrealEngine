// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.IO.Compression;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.Logging;

namespace HordeCmd.Commands.Tools;

[Command("tool", "download", "Downloads a tool from the server to a local directory")]
class ToolDownload(IHordeClient hordeClient) : Command
{
	[CommandLine("-Id=", Required = true)]
	[Description("The tool to download")]
	public ToolId Id { get; set; }

	[CommandLine("-Deployment=")]
	[Description("Deployment to fetch")]
	public ToolDeploymentId? DeploymentId { get; set; }

	[CommandLine("-OutputDir=", Required = true)]
	[Description("Output directory for the downloaded tool data")]
	public DirectoryReference OutputDir { get; set; } = null!;
	
	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		DirectoryReference.CreateDirectory(OutputDir);

		FileReference zipFile = FileReference.Combine(OutputDir, "temp.zip");
		await using (FileStream zipOutputStream = FileReference.Open(zipFile, FileMode.Create, FileAccess.Write))
		{
			using HordeHttpClient httpClient = hordeClient.CreateHttpClient();
			await using Stream stream = await httpClient.GetToolDeploymentZipAsync(Id, DeploymentId, CancellationToken.None);
			await stream.CopyToAsync(zipOutputStream);
		}

		await ZipFile.ExtractToDirectoryAsync(zipFile.FullName, OutputDir.FullName, true, CancellationToken.None);
		return 0;
	}
}
