// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.Logging;

namespace HordeCmd.Commands.Tools;

[Command("tool", "list", "List all tools available for download")]
class ToolList(IHordeClient hordeClient) : Command
{
	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		using HordeHttpClient httpClient = hordeClient.CreateHttpClient();
		logger.LogInformation("Horde server URL {ServerUrl}", httpClient.BaseUrl);
		GetToolsSummaryResponse response = await httpClient.GetToolsAsync();
		
		logger.LogInformation("  ---------------------------------------------------------------------------------------------");
		logger.LogInformation("  {ToolId,-30} {Deployment,-30} {Version,-15}", "ID", "DeploymentId", "Version");
		logger.LogInformation("  ---------------------------------------------------------------------------------------------");
		foreach (GetToolSummaryResponse tool in response.Tools)
		{
			logger.LogInformation("  {ToolId,-30} {Deployment,-30} {Version,-15}", tool.Id, tool.DeploymentId, tool.Version);
		}
		return 0;
	}
}
