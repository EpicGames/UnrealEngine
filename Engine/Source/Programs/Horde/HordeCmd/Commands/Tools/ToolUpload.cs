// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.Logging;

namespace HordeCmd.Commands.Tools;

[Command("tool", "upload", "Uploads a tool from a local directory")]
class ToolUpload(IHordeClient hordeClient) : Command
{
	[CommandLine("-Id=", Required = true)]
	[Description("Identifier for the tool to upload")]
	public ToolId ToolId { get; set; }
	
	[CommandLine("-Version=")]
	[Description("Optional version number for the new upload")]
	public string? Version { get; set; }
	
	[CommandLine("-InputDir=", Required = true)]
	[Description("Directory containing files to upload for the tool")]
	public DirectoryReference InputDir { get; set; } = null!;
	
	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(ToolId);
		
		IHashedBlobRef<DirectoryNode> target;
		await using (IBlobWriter writer = storageNamespace.CreateBlobWriter(ToolId.ToString(), null))
		{
			target = await writer.AddFilesAsync(InputDir);
		}
		
		using HordeHttpClient httpClient = hordeClient.CreateHttpClient();
		logger.LogInformation("Horde server URL {ServerUrl}", httpClient.BaseUrl);
		ToolDeploymentId deploymentId = await httpClient.CreateToolDeploymentAsync(ToolId, Version, null, null, target.GetRefValue());
		logger.LogInformation("Created deployment {DeploymentId}", deploymentId);
		
		return 0;
	}
}