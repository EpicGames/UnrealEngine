// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace Horde.Commands
{
	[Command("uninstall", "Removes the directory containing the tool from the PATH environment variable.")]
	class UinstallCommand : Command
	{
		/// <inheritdoc/>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			InstallCommand.Execute(false);
			return Task.FromResult(0);
		}
	}
}
