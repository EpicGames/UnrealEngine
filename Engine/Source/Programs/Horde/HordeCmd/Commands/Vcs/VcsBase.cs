// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	abstract class VcsBase : Command
	{
		[CommandLine("-BaseDir=")]
		public DirectoryReference? BaseDir { get; set; }

		[CommandLine("-Namespace=")]
		public NamespaceId NamespaceId { get; set; } = new NamespaceId("horde-perforce");

		protected class FileState
		{
			public IoHash Hash { get; set; }
			public long Length { get; private set; }
			public long LastModifiedTimeUtc { get; private set; }

			public FileState(FileInfo fileInfo, IoHash hash = default)
			{
				Update(fileInfo);
				Hash = hash;
			}

			public FileState(FileState other)
			{
				Length = other.Length;
				LastModifiedTimeUtc = other.LastModifiedTimeUtc;
			}

			public FileState(IMemoryReader reader)
			{
				Hash = reader.ReadIoHash();
				Length = reader.ReadInt64();
				LastModifiedTimeUtc = reader.ReadInt64();
			}

			public bool Modified(FileInfo fileInfo) => Length != fileInfo.Length || LastModifiedTimeUtc != fileInfo.LastWriteTimeUtc.Ticks;

			public void Update(FileInfo fileInfo)
			{
				Length = fileInfo.Length;
				LastModifiedTimeUtc = fileInfo.LastWriteTimeUtc.Ticks;
			}

			public void Write(IMemoryWriter writer)
			{
				Debug.Assert(Hash != IoHash.Zero);
				writer.WriteIoHash(Hash);
				writer.WriteInt64(Length);
				writer.WriteInt64(LastModifiedTimeUtc);
			}
		}

		protected class DirectoryState
		{
			public IoHash Hash { get; set; }
			public Dictionary<string, DirectoryState> Directories { get; }
			public Dictionary<string, FileState> Files { get; }

			public DirectoryState()
			{
				Directories = new Dictionary<string, DirectoryState>(StringComparer.OrdinalIgnoreCase);
				Files = new Dictionary<string, FileState>(StringComparer.OrdinalIgnoreCase);
			}

			public DirectoryState(IMemoryReader reader)
			{
				Hash = reader.ReadIoHash();

				int numDirectories = reader.ReadInt32();
				Directories = new Dictionary<string, DirectoryState>(numDirectories, StringComparer.OrdinalIgnoreCase);

				for (int idx = 0; idx < numDirectories; idx++)
				{
					string name = reader.ReadString();
					Directories[name] = new DirectoryState(reader);
				}

				int numFiles = reader.ReadInt32();
				Files = new Dictionary<string, FileState>(numFiles, StringComparer.OrdinalIgnoreCase);

				for (int idx = 0; idx < numFiles; idx++)
				{
					string name = reader.ReadString();
					Files[name] = new FileState(reader);
				}
			}

			public void Write(IMemoryWriter writer)
			{
				writer.WriteIoHash(Hash);

				writer.WriteInt32(Directories.Count);
				foreach ((string name, DirectoryState state) in Directories)
				{
					writer.WriteString(name);
					state.Write(writer);
				}

				writer.WriteInt32(Files.Count);
				foreach ((string name, FileState state) in Files)
				{
					writer.WriteString(name);
					state.Write(writer);
				}
			}
		}

		protected class WorkspaceState
		{
			public RefName Branch { get; set; }
			public int Change { get; set; }
			public DirectoryState Tree { get; set; }

			public WorkspaceState()
				: this(new RefName("main"), 0, new DirectoryState())
			{
			}

			public WorkspaceState(RefName branch, int change, DirectoryState tree)
			{
				Branch = branch;
				Change = change;
				Tree = tree;
			}

			public WorkspaceState(IMemoryReader reader)
			{
				Branch = new RefName(reader.ReadUtf8String());
				Change = reader.ReadInt32();
				Tree = new DirectoryState(reader);
			}

			public void Write(IMemoryWriter writer)
			{
				writer.WriteUtf8String(Branch.Text);
				writer.WriteInt32(Change);
				Tree.Write(writer);
			}
		}

		const string DataDir = ".horde";
		const string StateFileName = "state.dat";

		readonly IStorageClient _storageClient;

		protected VcsBase(IStorageClient storageClient)
		{
			_storageClient = storageClient;
		}

		protected IStorageNamespace GetStorageNamespace() => _storageClient.GetNamespace(NamespaceId);

		protected static async Task<WorkspaceState> ReadStateAsync(DirectoryReference rootDir)
		{
			FileReference stateFile = FileReference.Combine(rootDir, DataDir, StateFileName);
			byte[] data = await FileReference.ReadAllBytesAsync(stateFile);
			return new WorkspaceState(new MemoryReader(data));
		}

		protected static async Task WriteStateAsync(DirectoryReference rootDir, WorkspaceState state)
		{
			ByteArrayBuilder builder = new ByteArrayBuilder();
			state.Write(builder);

			FileReference stateFile = FileReference.Combine(rootDir, DataDir, StateFileName);
			DirectoryReference.CreateDirectory(stateFile.Directory);

			using (FileStream stream = FileReference.Open(stateFile, FileMode.Create, FileAccess.Write))
			{
				foreach (ReadOnlyMemory<byte> chunk in builder.AsSequence())
				{
					await stream.WriteAsync(chunk);
				}
			}
		}

		protected static async Task InitAsync(DirectoryReference rootDir)
		{
			await WriteStateAsync(rootDir, new WorkspaceState());
		}

		protected DirectoryReference FindRootDir()
		{
			if (BaseDir != null)
			{
				return BaseDir;
			}

			DirectoryReference startDir = DirectoryReference.GetCurrentDirectory();
			for (DirectoryReference? currentDir = startDir; currentDir != null; currentDir = currentDir.ParentDirectory)
			{
				FileReference markerFile = FileReference.Combine(currentDir, DataDir, StateFileName);
				if (FileReference.Exists(markerFile))
				{
					return currentDir;
				}
			}
			throw new InvalidDataException($"No root directory found under {startDir}");
		}

		protected static Task<DirectoryState> GetCurrentDirectoryState(DirectoryReference rootDir, DirectoryState? oldState) => GetCurrentDirectoryStateAsync(rootDir.ToDirectoryInfo(), oldState);

		protected static async Task<DirectoryState> GetCurrentDirectoryStateAsync(DirectoryInfo directoryInfo, DirectoryState? oldState)
		{
			List<DirectoryInfo> subDirectoryInfos = new List<DirectoryInfo>();
			foreach (DirectoryInfo subDirectoryInfo in directoryInfo.EnumerateDirectories().Where(x => !x.Name.StartsWith(".", StringComparison.Ordinal)))
			{
				subDirectoryInfos.Add(subDirectoryInfo);
			}

			Task<DirectoryState>[] tasks = new Task<DirectoryState>[subDirectoryInfos.Count];
			for (int idx = 0; idx < subDirectoryInfos.Count; idx++)
			{
				DirectoryInfo subDirectoryInfo = subDirectoryInfos[idx];

				DirectoryState? prevSubDirectoryState = null;
				if (oldState != null)
				{
					oldState.Directories.TryGetValue(subDirectoryInfo.Name, out prevSubDirectoryState);
				}

				tasks[idx] = Task.Run(() => GetCurrentDirectoryStateAsync(subDirectoryInfo, prevSubDirectoryState));
			}

			DirectoryState newState = new DirectoryState();
			for (int idx = 0; idx < subDirectoryInfos.Count; idx++)
			{
				newState.Directories[subDirectoryInfos[idx].Name] = await tasks[idx];
			}
			foreach (FileInfo fileInfo in directoryInfo.EnumerateFiles())
			{
				FileState? fileState;
				if (oldState == null || !oldState.Files.TryGetValue(fileInfo.Name, out fileState) || fileState.Modified(fileInfo))
				{
					fileState = new FileState(fileInfo);
				}
				newState.Files[fileInfo.Name] = fileState;
			}

			if (oldState != null && oldState.Directories.Count == newState.Directories.Count && oldState.Files.Count == newState.Files.Count)
			{
				if (newState.Directories.Values.All(x => x.Hash != IoHash.Zero) && newState.Files.Values.All(x => x.Hash != IoHash.Zero))
				{
					return oldState;
				}
			}

			return newState;
		}

		protected static void RemoveAddedFiles(DirectoryState oldState, DirectoryState newState)
		{
			List<(string, DirectoryState?, DirectoryState?)> directoryDeltas = EnumerableExtensions.Zip(oldState.Directories, newState.Directories).ToList();
			foreach ((string name, DirectoryState? oldSubDirState, _) in directoryDeltas)
			{
				if (oldSubDirState == null)
				{
					newState.Directories.Remove(name);
				}
			}

			List<(string, FileState?, FileState?)> fileDeltas = EnumerableExtensions.Zip(oldState.Files, newState.Files).ToList();
			foreach ((string name, FileState? oldFileState, _) in fileDeltas)
			{
				if (oldFileState == null)
				{
					newState.Files.Remove(name);
				}
			}
		}

		protected static DirectoryState DedupTrees(DirectoryState oldState, DirectoryState newState)
		{
			if (oldState == newState)
			{
				return newState;
			}

			foreach ((_, DirectoryState? oldSubDirState, DirectoryState? newSubDirState) in EnumerableExtensions.Zip(oldState.Directories, newState.Directories))
			{
				if (oldSubDirState == null || newSubDirState == null || oldSubDirState != DedupTrees(oldSubDirState, newSubDirState))
				{
					return newState;
				}
			}

			foreach ((_, FileState? oldFileState, FileState? newFileState) in EnumerableExtensions.Zip(oldState.Files, newState.Files))
			{
				if (oldFileState != newFileState)
				{
					return newState;
				}
			}

			return oldState;
		}

		protected static void PrintDelta(DirectoryState oldState, DirectoryState newState, ILogger logger) => PrintDelta("", oldState, newState, logger);

		static void PrintDelta(string prefix, DirectoryState oldState, DirectoryState newState, ILogger logger)
		{
			foreach ((string name, DirectoryState? oldSubDirState, DirectoryState? newSubDirState) in EnumerableExtensions.Zip(oldState.Directories, newState.Directories))
			{
				if (oldSubDirState == null)
				{
					logger.LogInformation("{Path} (added)", $"{prefix}{name}/");
				}
				else if (newSubDirState == null)
				{
					logger.LogInformation("{Path} (removed)", $"{prefix}{name}/");
				}
				else if (oldSubDirState != newSubDirState)
				{
					PrintDelta($"{prefix}{name}/", oldSubDirState, newSubDirState, logger);
				}
			}

			foreach ((string name, FileState? oldFileState, FileState? newFileState) in EnumerableExtensions.Zip(oldState.Files, newState.Files))
			{
				if (oldFileState == null)
				{
					logger.LogInformation("{Path} (added)", $"{prefix}{name}");
				}
				else if (newFileState == null)
				{
					logger.LogInformation("{Path} (deleted)", $"{prefix}{name}");
				}
				else if (oldFileState != newFileState)
				{
					logger.LogInformation("{Path} (modified)", $"{prefix}{name}");
				}
			}
		}

		protected static async Task<CommitNode?> GetCommitAsync(IStorageNamespace storageNamespace, RefName branchName, int change = 0)
		{
			CommitNode tip = await storageNamespace.ReadRefTargetAsync<CommitNode>(branchName);
			if (change != 0)
			{
				while (tip.Number != change)
				{
					if (tip.Parent == null || tip.Number < change)
					{
						return null;
					}
					tip = await tip.Parent.ReadBlobAsync();
				}
			}
			return tip;
		}
	}
}
