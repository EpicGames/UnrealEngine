// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "branch", "Switch to a new branch", Advertise = false)]
	class VcsBranch : VcsBase
	{
		[CommandLine(Prefix = "-Name=", Required = true)]
		public string Name { get; set; } = "";

		public VcsBranch(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference rootDir = FindRootDir();

			WorkspaceState workspaceState = await ReadStateAsync(rootDir);

			IStorageNamespace store = GetStorageNamespace();

			RefName branchName = new RefName(Name);
			if (await store.RefExistsAsync(branchName))
			{
				logger.LogError("Branch {BranchName} already exists - use checkout instead.", branchName);
				return 1;
			}

			logger.LogInformation("Starting work in new branch {BranchName}", branchName);

			workspaceState.Branch = new RefName(Name);
			workspaceState.Tree = new DirectoryState();
			await WriteStateAsync(rootDir, workspaceState);

			return 0;
		}
	}
}
