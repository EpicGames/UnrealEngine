// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "checkout", "Checkout a particular branch/change", Advertise = false)]
	class VcsCheckout : VcsBase
	{
		[CommandLine("-Branch")]
		public string? Branch { get; set; }

		[CommandLine("-Change=")]
		public int Change { get; set; }

		[CommandLine("-Clean")]
		public bool Clean { get; set; }

		[CommandLine("-Force")]
		public bool Force { get; set; }

		public VcsCheckout(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference rootDir = FindRootDir();

			WorkspaceState workspaceState = await ReadStateAsync(rootDir);

			DirectoryState oldState = workspaceState.Tree;
			DirectoryState newState = await GetCurrentDirectoryState(rootDir, oldState);

			if (!Clean)
			{
				RemoveAddedFiles(oldState, newState);
				newState = DedupTrees(oldState, newState);
			}

			if (oldState != newState && !Force)
			{
				logger.LogInformation("Current workspace has modified files. Resolve before checking out a different change, or run with -Force.");
				PrintDelta(oldState, newState, logger);
				return 0;
			}

			RefName branchName = (Branch != null) ? new RefName(Branch) : workspaceState.Branch;

			IStorageNamespace store = GetStorageNamespace();

			CommitNode? tip = await GetCommitAsync(store, branchName, Change);
			if (tip == null)
			{
				logger.LogError("Unable to find change {Change}", Change);
				return 1;
			}

			workspaceState.Tree = await RealizeAsync(tip.Contents, rootDir, newState, Clean, logger);
			await WriteStateAsync(rootDir, workspaceState);

			logger.LogInformation("Updated workspace to change {Number}", tip.Number);
			return 0;
		}

		protected static async Task<DirectoryState> RealizeAsync(DirectoryNodeRef directoryRef, DirectoryReference dirPath, DirectoryState? directoryState, bool clean, ILogger logger)
		{
			DirectoryReference.CreateDirectory(dirPath);

			DirectoryState newState = new DirectoryState();

			DirectoryNode directoryNode = await directoryRef.Handle.ReadBlobAsync();
			foreach ((string name, DirectoryEntry? subDirEntry, DirectoryState? subDirState) in EnumerableExtensions.Zip(directoryNode.NameToDirectory, directoryState?.Directories))
			{
				DirectoryReference subDirPath = DirectoryReference.Combine(dirPath, name.ToString());
				if (subDirEntry == null)
				{
					if (clean)
					{
						DirectoryReference.Delete(subDirPath);
					}
				}
				else
				{
					newState.Directories[name] = await RealizeAsync(subDirEntry, subDirPath, subDirState, clean, logger);
				}
			}

			foreach ((string name, FileEntry? fileEntry, FileState? fileState) in EnumerableExtensions.Zip(directoryNode.NameToFile, directoryState?.Files))
			{
				FileReference filePath = FileReference.Combine(dirPath, name.ToString());
				if (fileEntry == null)
				{
					if (clean)
					{
						FileReference.Delete(filePath);
					}
				}
				else if (fileState == null || fileState.Hash != fileEntry.Hash)
				{
					newState.Files[name] = await CheckoutFileAsync(fileEntry, filePath.ToFileInfo(), logger);
				}
				else
				{
					newState.Files[name] = fileState;
				}
			}

			newState.Hash = directoryRef.Handle.Hash;
			return newState;
		}

		static async Task<FileState> CheckoutFileAsync(FileEntry fileRef, FileInfo fileInfo, ILogger logger)
		{
			logger.LogInformation("Updating {File} to {Hash}", fileInfo, fileRef.Hash);
			await fileRef.Target.CopyToFileAsync(fileInfo, CancellationToken.None);
			fileInfo.Refresh();
			return new FileState(fileInfo, fileRef.Hash);
		}
	}
}
