// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "clone", "Initialize a directory for VCS-like operations", Advertise = false)]
	class VcsClone : VcsCheckout
	{
		public VcsClone(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			BaseDir ??= DirectoryReference.GetCurrentDirectory();
			await InitAsync(BaseDir);
			logger.LogInformation("Initialized in {RootDir}", BaseDir);

			WorkspaceState workspaceState = new WorkspaceState();

			RefName branchName = new RefName(Branch ?? "ue5-main");

			IStorageNamespace storageNamespace = GetStorageNamespace();

			CommitNode? tip = await GetCommitAsync(storageNamespace, branchName, Change);
			if (tip == null)
			{
				logger.LogError("Unable to find change {Change}", Change);
				return 1;
			}

			workspaceState.Tree = await RealizeAsync(tip.Contents, BaseDir, null, true, logger);
			await WriteStateAsync(BaseDir, workspaceState);

			return 0;
		}
	}
}
