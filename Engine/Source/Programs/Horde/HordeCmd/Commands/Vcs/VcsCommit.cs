// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "commit", "Commits data to the VCS store", Advertise = false)]
	class VcsCommit : VcsBase
	{
		[CommandLine("-User=")]
		public string UserName { get; set; } = Environment.UserName;

		[CommandLine("-Message=")]
		public string? Message { get; set; }

		public VcsCommit(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override Task<int> ExecuteAsync(ILogger logger)
		{
			throw new NotImplementedException();
			/*
			DirectoryReference rootDir = FindRootDir();

			WorkspaceState workspaceState = await ReadStateAsync(rootDir);

			DirectoryState oldState = workspaceState.Tree;
			DirectoryState newState = await GetCurrentDirectoryState(rootDir, oldState);

			if (oldState == newState)
			{
				logger.LogInformation("No files modified");
				return 0;
			}

			if (Message == null)
			{
				logger.LogError("Missing -Message=... argument");
				return 1;
			}

			PrintDelta(oldState, newState, logger);

			IStorageClient store = await GetStorageClientAsync();

			NodeRef<CommitNode>? tipRef = await store.TryReadRefTargetAsync<CommitNode>(workspaceState.Branch);
			CommitNode? tip = (tipRef == null)? null : await tipRef.ExpandAsync();

			DirectoryNode rootNode;
			if (tip == null)
			{
				rootNode = new DirectoryNode();
			}
			else
			{
				rootNode = await tip.Contents.ExpandAsync();
			}

//			DirectoryNodeRef rootRef = new DirectoryNodeRef(rootNode);

			List<(DirectoryNode, FileInfo, FileState)> files = new List<(DirectoryNode, FileInfo, FileState)>();
			List<(DirectoryNodeRef, DirectoryState)> directories = new List<(DirectoryNodeRef, DirectoryState)>();
			await UpdateTreeAsync(rootRef, rootDir, oldState, newState, files, directories);

			await using IStorageWriter writer = store.CreateWriter();
			await DirectoryNode.CopyFromDirectoryAsync(files.ConvertAll(x => (x.Item1, x.Item2)), new ChunkingOptions(), writer, null, CancellationToken.None);

			DirectoryNodeRef rootRef = new DirectoryNodeRef(rootNode.Length, await writer.WriteNodeAsync(rootNode));

			CommitNode newTip;
			if (tip == null)
			{
				newTip = new CommitNode(1, null, UserName, Message, DateTime.UtcNow, rootRef);
			}
			else
			{
				newTip = new CommitNode(tip.Number + 1, tipRef, UserName, Message, DateTime.UtcNow, rootRef);
			}
			await store.WriteNodeAsync(workspaceState.Branch, newTip);

			foreach ((DirectoryNodeRef directoryRef, DirectoryState directoryState) in directories)
			{
				directoryState.Hash = directoryRef.Handle.Hash;
			}

			foreach ((DirectoryNode directoryNode, FileInfo fileInfo, FileState fileState) in files)
			{
				FileEntry fileEntry = directoryNode.GetFileEntry(fileInfo.Name);
				fileState.Hash = fileEntry.Hash;
			}

			workspaceState.Change = newTip.Number;
			workspaceState.Tree = newState;
			await WriteStateAsync(rootDir, workspaceState);

			logger.LogInformation("Commited in change {Number}", newTip.Number);
			return 0;*/
		}
		/*
		private async Task UpdateTreeAsync(DirectoryNodeRef rootRef, DirectoryReference rootDir, DirectoryState? oldState, DirectoryState newState, List<(DirectoryNode, FileInfo, FileState)> files, List<(DirectoryNodeRef, DirectoryState)> directories)
		{
			directories.Add((rootRef, newState));

			DirectoryNode root = await rootRef.ExpandAsync();
			foreach ((Utf8String name, DirectoryState? oldSubDirState, DirectoryState? newSubDirState) in EnumerableExtensions.Zip(oldState?.Directories, newState.Directories))
			{
				if (newSubDirState == null)
				{
					root.DeleteDirectory(name);
				}
				else if (oldSubDirState != newSubDirState)
				{
					await UpdateTreeAsync(root.FindOrAddDirectoryEntry(name), DirectoryReference.Combine(rootDir, name.ToString()), oldSubDirState, newSubDirState, files, directories);
				}
			}

			foreach ((Utf8String name, FileState? oldFileState, FileState? newFileState) in EnumerableExtensions.Zip(oldState?.Files, newState.Files))
			{
				if (newFileState == null)
				{
					root.DeleteFile(name);
				}
				else if (oldFileState != newFileState)
				{
					files.Add((root, FileReference.Combine(rootDir, name.ToString()).ToFileInfo(), newFileState));
				}
			}
		}*/
	}
}
