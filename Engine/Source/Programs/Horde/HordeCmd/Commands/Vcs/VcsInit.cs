// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "init", "Initialize a directory for VCS-like operations", Advertise = false)]
	class VcsInit : VcsBase
	{
		public VcsInit(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			BaseDir ??= DirectoryReference.GetCurrentDirectory();
			await InitAsync(BaseDir);
			logger.LogInformation("Initialized in {RootDir}", BaseDir);
			return 0;
		}
	}
}
