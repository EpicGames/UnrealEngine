// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "log", "Print a history of commits", Advertise = false)]
	class VcsLog : VcsBase
	{
		[CommandLine("-Count")]
		public int Count { get; set; } = 20;

		public VcsLog(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference rootDir = FindRootDir();

			WorkspaceState workspaceState = await ReadStateAsync(rootDir);

			IStorageNamespace store = GetStorageNamespace();

			using MemoryCache cache = new MemoryCache(new MemoryCacheOptions());

			List<CommitNode> commits = new List<CommitNode>();

			CommitNode? tip = await store.TryReadRefTargetAsync<CommitNode>(workspaceState.Branch);
			if (tip != null)
			{
				for (int idx = 0; idx < Count; idx++)
				{
					commits.Add(tip);

					if (tip.Parent == null)
					{
						break;
					}

					tip = await tip.Parent.ReadBlobAsync();
				}
			}

			foreach (CommitNode commit in commits)
			{
				logger.LogInformation("Commit {Number}: {Message}", commit.Number, commit.Message);
			}

			return 0;
		}
	}
}
