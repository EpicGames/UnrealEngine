// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Vcs
{
	[Command("vcs", "status", "Find status of local files", Advertise = false)]
	class VcsStatus : VcsBase
	{
		public VcsStatus(IStorageClient storageClient)
			: base(storageClient)
		{
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference rootDir = FindRootDir();

			WorkspaceState workspaceState = await ReadStateAsync(rootDir);

			DirectoryState oldState = workspaceState.Tree;
			DirectoryState newState = await GetCurrentDirectoryState(rootDir, oldState);

			logger.LogInformation("On branch {BranchName}", workspaceState.Branch);

			if (oldState == newState)
			{
				logger.LogInformation("No files modified");
			}
			else
			{
				PrintDelta(oldState, newState, logger);
			}

			return 0;
		}
	}
}
