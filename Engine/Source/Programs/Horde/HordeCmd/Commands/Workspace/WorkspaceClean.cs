// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "clean", "Removes all untracked files from the workspace")]
	class WorkspaceClean : WorkspaceStatusBase
	{
		protected override Task ExecuteInternalAsync(Workspace workspace, FileFilter? filter, WorkspaceStatusFlags flags, CancellationToken cancellationToken)
			=> workspace.CleanAsync(filter, flags, cancellationToken: cancellationToken);
	}
}
