// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "drop", "Drops a layer from a workspace")]
	class WorkspaceDrop : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the workspace.")]
		public DirectoryReference? RootDir { get; set; }

		[CommandLine("-Layer=")]
		[Description("Name of the layer to remove.")]
		public WorkspaceLayerId LayerId { get; set; } = WorkspaceLayerId.Default;

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace? workspace = await Workspace.TryFindAndOpenAsync(RootDir, logger, cancellationToken);
			if (workspace == null)
			{
				logger.LogError("No workspace has been initialized in {RootDir}. Use 'workspace init' to create a new workspace.", RootDir);
				return 1;
			}

			await workspace.SyncAsync(LayerId, null, cancellationToken: cancellationToken);
			workspace.RemoveLayer(LayerId);
			await workspace.SaveAsync(cancellationToken);

			logger.LogInformation("Removed layer {LayerId}", LayerId);

			return 0;
		}
	}
}
