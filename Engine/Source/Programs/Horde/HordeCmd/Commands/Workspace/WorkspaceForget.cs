// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "forget", "Removes files in the workspace from tracking", AcceptsPositionalArguments = true)]
	class WorkspaceForget : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the workspace.")]
		public DirectoryReference? RootDir { get; set; }

		[CommandLine("-Layer=")]
		[Description("Layer to remove files from")]
		public WorkspaceLayerId LayerId { get; set; } = WorkspaceLayerId.Default;

		[CommandLine("-Path=", Positional = true, Required = true)]
		[Description("Filter for the files to remove")]
		List<string> Paths { get; set; } = new List<string>();

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace? workspace = await Workspace.TryFindAndOpenAsync(RootDir, logger, cancellationToken);
			if (workspace == null)
			{
				logger.LogError("No workspace has been initialized in {RootDir}. Use 'workspace init' to create a new workspace.", RootDir);
				return 1;
			}

			FileFilter filter = WorkspaceTrack.CreateFilter(RootDir, Paths);
			await workspace.ForgetAsync(LayerId, filter, cancellationToken);

			return 0;
		}
	}
}
