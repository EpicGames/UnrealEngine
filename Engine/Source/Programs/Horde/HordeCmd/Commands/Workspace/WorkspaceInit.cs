// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "init", "Initialize a managed workspace in the current folder")]
	class WorkspaceInit : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the managed workspace.")]
		public DirectoryReference? RootDir { get; set; }

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace workspace = await Workspace.CreateAsync(RootDir, logger);
			logger.LogInformation("Created workspace in {RootDir}", workspace.RootDir);

			return 0;
		}
	}
}
