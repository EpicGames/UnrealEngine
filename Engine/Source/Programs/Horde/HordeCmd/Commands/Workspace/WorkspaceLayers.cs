// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "layers", "Lists all layers in the workspace")]
	class WorkspaceLayers : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the workspace.")]
		public DirectoryReference? RootDir { get; set; }

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace? workspace = await Workspace.TryFindAndOpenAsync(RootDir, logger, cancellationToken);
			if (workspace == null)
			{
				logger.LogError("No workspace has been initialized in {RootDir}. Use 'workspace init' to create a new workspace.", RootDir);
				return 1;
			}

			logger.LogInformation("Layers:");
			logger.LogInformation("  {LayerId}", WorkspaceLayerId.Default);
			foreach (WorkspaceLayerId layerId in workspace.CustomLayers)
			{
				logger.LogInformation("  {LayerId}", layerId);
			}

			return 0;
		}
	}
}
