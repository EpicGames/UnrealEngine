// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "status", "Prints the current status of the workspace", AcceptsPositionalArguments = true)]
	class WorkspaceStatus : WorkspaceStatusBase
	{
		protected override Task ExecuteInternalAsync(Workspace workspace, FileFilter? filter, WorkspaceStatusFlags flags, CancellationToken cancellationToken)
			=> workspace.StatusAsync(filter, flags, cancellationToken);
	}

	abstract class WorkspaceStatusBase : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the workspace.")]
		public DirectoryReference? RootDir { get; set; }

		[CommandLine("-Path=", Positional = true)]
		[Description("Path of the files to query")]
		public List<string> Paths { get; set; } = new List<string>();

		[CommandLine("-All")]
		[Description("Show all files")]
		public bool All { get; set; }

		[CommandLine("-Added")]
		[Description("Show added files")]
		public bool Added { get; set; }

		[CommandLine("-Modified")]
		[Description("Show modified files")]
		public bool Modified { get; set; }

		[CommandLine("-Unmodified")]
		[Description("Show all unmodified files")]
		public bool Unmodified { get; set; }

		[CommandLine("-Removed")]
		[Description("Show removed files")]
		public bool Removed { get; set; }

		[CommandLine("-IgnoreAdded")]
		[Description("Ignore added files")]
		public bool IgnoreAdded { get; set; }

		[CommandLine("-IgnoreModified")]
		[Description("Ignore modified files")]
		public bool IgnoreModified { get; set; }

		[CommandLine("-IgnoreUnmodified")]
		[Description("Ignore unmodified files")]
		public bool IgnoreUnmodified { get; set; }

		[CommandLine("-IgnoreRemoved")]
		[Description("Ignore removed files")]
		public bool IgnoreRemoved { get; set; }

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace? workspace = await Workspace.TryFindAndOpenAsync(RootDir, logger, cancellationToken);
			if (workspace == null)
			{
				logger.LogError("No workspace has been initialized in {RootDir}. Use 'workspace init' to create a new workspace.", RootDir);
				return 1;
			}

			WorkspaceStatusFlags flags = WorkspaceStatusFlags.None;
			if (All)
			{
				flags |= WorkspaceStatusFlags.FileAdded | WorkspaceStatusFlags.FileModified | WorkspaceStatusFlags.FileRemoved | WorkspaceStatusFlags.FileUnmodified | WorkspaceStatusFlags.DirectoryAdded | WorkspaceStatusFlags.DirectoryRemoved;
			}

			if (Added)
			{
				flags |= WorkspaceStatusFlags.FileAdded | WorkspaceStatusFlags.DirectoryAdded;
			}
			if (Modified)
			{
				flags |= WorkspaceStatusFlags.FileModified;
			}
			if (Removed)
			{
				flags |= WorkspaceStatusFlags.FileRemoved | WorkspaceStatusFlags.DirectoryRemoved;
			}
			if (Unmodified)
			{
				flags |= WorkspaceStatusFlags.FileUnmodified;
			}

			if (flags == WorkspaceStatusFlags.None)
			{
				flags = WorkspaceStatusFlags.Default;
			}

			if (IgnoreAdded)
			{
				flags &= ~WorkspaceStatusFlags.FileAdded | WorkspaceStatusFlags.DirectoryAdded;
			}
			if (IgnoreModified)
			{
				flags &= ~WorkspaceStatusFlags.FileModified;
			}
			if (IgnoreRemoved)
			{
				flags &= ~WorkspaceStatusFlags.FileRemoved | WorkspaceStatusFlags.DirectoryRemoved;
			}

			FileFilter filter = WorkspaceTrack.CreateFilter(RootDir, Paths);
			await ExecuteInternalAsync(workspace, filter, flags, cancellationToken);

			return 0;
		}

		protected abstract Task ExecuteInternalAsync(Workspace workspace, FileFilter? filter, WorkspaceStatusFlags flags, CancellationToken cancellationToken);
	}
}
