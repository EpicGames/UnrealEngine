// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "sync", "Extracts an archive into the workspace")]
	class WorkspaceSync : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the managed workspace.")]
		public DirectoryReference? RootDir { get; set; }

		[CommandLine("-Target=")]
		[Description("Directory to extract files to. Must be under the workspace root.")]
		public DirectoryReference? TargetDir { get; set; }

		[CommandLine("-Artifact=")]
		[Description("Artifact to extract into the workspace")]
		public string? Artifact { get; set; }

		[CommandLine("-Namespace=")]
		[Description("Namespace containing the node or ref to extract")]
		public string Namespace { get; set; } = "default";

		[CommandLine("-Ref=")]
		[Description("Name of a ref to extract to this workspace.")]
		public string? Ref { get; set; }

		[CommandLine("-Node=")]
		[Description("Locator for a node to extract to this workspace.")]
		public string? Node { get; set; }

		[CommandLine("-Layer=")]
		[Description("Name of the layer to extract to.")]
		public WorkspaceLayerId LayerId { get; set; } = WorkspaceLayerId.Default;

		[CommandLine("-Stats")]
		[Description("Outputs stats for the extraction operation.")]
		public bool Stats { get; set; }

		readonly IHordeClient _hordeClient;

		public WorkspaceSync(IHordeClient hordeClient)
		{
			_hordeClient = hordeClient;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			if (Artifact != null)
			{
				ArtifactId artifactId = ArtifactId.Parse(Artifact);

				IArtifact? artifact = await _hordeClient.Artifacts.GetAsync(artifactId);
				if (artifact == null)
				{
					logger.LogError("Artifact {Id} not found", artifactId);
					return 1;
				}

				logger.LogInformation("Downloading artifact {Id}: {Description}", artifactId, artifact.Description);

				IStorageNamespace store = _hordeClient.GetStorageNamespace(artifact.Id);
				IBlobRef<DirectoryNode> handle = await store.ReadRefAsync<DirectoryNode>(new RefName("default"));
				return await ExecuteInternalAsync(store, handle, logger);
			}
			else
			{
				IStorageNamespace store = _hordeClient.GetStorageNamespace(new NamespaceId(Namespace));

				IBlobRef<DirectoryNode> handle;
				if (Ref != null)
				{
					handle = await store.ReadRefAsync<DirectoryNode>(new RefName(Ref));
				}
				else if (Node != null)
				{
					handle = store.CreateBlobRef<DirectoryNode>(new BlobLocator(Node));
				}
				else
				{
					throw new CommandLineArgumentException("Missing -Artifact=..., -File=... or -Ref=... option");
				}

				return await ExecuteInternalAsync(store, handle, logger);
			}
		}

		async Task<int> ExecuteInternalAsync(IStorageNamespace store, IBlobRef<DirectoryNode> handle, ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;

			Workspace? workspace;
			if (RootDir != null)
			{
				workspace = await Workspace.CreateOrOpenAsync(RootDir, logger, cancellationToken);
			}
			else
			{
				DirectoryReference workspaceDir = TargetDir ?? DirectoryReference.GetCurrentDirectory();

				workspace = await Workspace.TryFindAndOpenAsync(workspaceDir, logger, cancellationToken);
				if (workspace == null)
				{
					logger.LogInformation("Creating new workspace in {Path}", workspaceDir);
					workspace = await Workspace.CreateAsync(workspaceDir, logger, cancellationToken);
				}
			}

			string mountPath = "";
			if (TargetDir != null)
			{
				if (!TargetDir.IsUnderDirectory(workspace.RootDir))
				{
					logger.LogError("Output directory ({OutputDir}) is not under workspace root ({RootDir})", TargetDir, workspace.RootDir);
					return 1;
				}
				mountPath = TargetDir.MakeRelativeTo(workspace.RootDir);
			}

			Stopwatch timer = Stopwatch.StartNew();

			logger.LogInformation("Refreshing metadata...");
			await workspace.ReconcileAsync(cancellationToken);

			logger.LogInformation("Syncing into layer '{LayerId}'...", LayerId);

			await workspace.SyncAsync(LayerId, mountPath, handle, cancellationToken: cancellationToken);
			await workspace.SaveAsync(cancellationToken);

			logger.LogInformation("Elapsed: {Time}s", timer.Elapsed.TotalSeconds);

			if (Stats)
			{
				StorageStats stats = store.GetStats();
				stats.Print(logger);
			}

			return 0;
		}
	}
}
