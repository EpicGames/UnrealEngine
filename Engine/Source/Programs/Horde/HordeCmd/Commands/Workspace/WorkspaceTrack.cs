// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.Logging;

namespace Horde.Commands.Workspace
{
	using Workspace = EpicGames.Horde.Storage.Workspace;

	[Command("workspace", "track", "Tracks files in their current state with the workspace", AcceptsPositionalArguments = true)]
	class WorkspaceTrack : Command
	{
		[CommandLine("-Root=")]
		[Description("Root directory for the workspace.")]
		public DirectoryReference? RootDir { get; set; }

		[CommandLine("-Layer=")]
		[Description("Layer to add new files to")]
		public WorkspaceLayerId LayerId { get; set; } = WorkspaceLayerId.Default;

		[CommandLine("-Path=", Positional = true)]
		[Description("List of files to reconcile")]
		public List<string> Paths { get; set; } = new List<string>();

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			CancellationToken cancellationToken = CancellationToken.None;
			RootDir ??= DirectoryReference.GetCurrentDirectory();

			Workspace? workspace = await Workspace.TryFindAndOpenAsync(RootDir, logger, cancellationToken);
			if (workspace == null)
			{
				logger.LogError("No workspace has been initialized in {RootDir}. Use 'workspace init' to create a new workspace.", RootDir);
				return 1;
			}

			workspace.AddLayer(LayerId);

			FileFilter filter = CreateFilter(RootDir, Paths);
			await workspace.TrackAsync(LayerId, filter, cancellationToken);

			return 0;
		}

		public static FileFilter CreateFilter(DirectoryReference rootDir, List<string> paths)
		{
			if (paths.Count == 0)
			{
				return new FileFilter(FileFilterType.Include);
			}

			FileFilter filter = new FileFilter();
			foreach (string path in paths)
			{
				string trimPath = path.TrimStart('/');
				try
				{
					DirectoryReference subDir = DirectoryReference.Combine(rootDir, path);
					if (DirectoryReference.Exists(subDir))
					{
						trimPath = $"{trimPath}/...";
					}
				}
				catch { }

				filter.AddRule($"/{trimPath}", FileFilterType.Include);
			}
			return filter;
		}
	}
}
