<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <OutputType>Exe</OutputType>
    <AssemblyName>Horde</AssemblyName>
    <ImplicitUsings>enable</ImplicitUsings>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.OIDC\EpicGames.OIDC.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.Extensions.Configuration.EnvironmentVariables" Version="10.0.6" />
    <PackageReference Include="Serilog" Version="3.1.1" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="Serilog.Extensions.Logging" Version="8.0.0" />
    <PackageReference Include="Serilog.Settings.Configuration" Version="8.0.4" />
    <PackageReference Include="Serilog.Sinks.Console" Version="5.0.1" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="appsettings.json" Condition="Exists('appsettings.json')">
      <Generator></Generator>
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <CopyToPublishDirectory>Always</CopyToPublishDirectory>
    </Content>
    <Content Include="Toolbox.json">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
    <Content Include="UgsTool.json">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

</Project>
