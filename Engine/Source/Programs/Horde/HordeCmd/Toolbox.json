{
    "installCommand": {
        "fileName": "horde.exe",
        "arguments": [
            "install"
        ]
    },
    "uninstallCommand": {
        "fileName": "horde.exe",
        "arguments": [
            "uninstall"
        ]
    }
}