{
	"installCommand": "horde.exe install",
	"uninstallCommand": "horde.exe uninstall"
}
