{
	"Serilog": {
		"MinimumLevel": {
			"Default": "Verbose",
			"Override": {
				"Microsoft": "Information",
				"System.Net.Http.HttpClient": "Warning",
				"Microsoft.AspNetCore.Routing.EndpointMiddleware": "Warning",
				"Microsoft.AspNetCore.Authorization.DefaultAuthorizationService": "Warning",
				"HordeServer.Authentication.HordeJwtBearerHandler": "Warning",
				"HordeServer.Authentication.OktaHandler": "Warning",
				"Microsoft.AspNetCore.Hosting.Diagnostics": "Warning",
				"Microsoft.AspNetCore.Mvc.Infrastructure.ControllerActionInvoker": "Warning",
				"Serilog.AspNetCore.RequestLoggingMiddleware": "Warning",
				"Grpc.Net.Client.Internal.GrpcCall": "Warning",
				"EpicGames.Horde.Compute.ComputeSocket": "Verbose",
				"EpicGames.Horde.Compute.ComputeChannel": "Verbose"
			}
		}
	}
}
