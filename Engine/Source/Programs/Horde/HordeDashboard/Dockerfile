FROM node:20.11.0 as build-env
ARG VersionInfo
ARG DashboardConfig
ENV VITE_VERSION_INFO=${VersionInfo}
ENV VITE_DASHBOARD_CONFIG=${DashboardConfig}
WORKDIR /app
RUN wget https://s3.amazonaws.com/rds-downloads/rds-combined-ca-bundle.pem
COPY Source/Programs/Horde/HordeDashboard ./Dashboard
RUN cd ./Dashboard && npm install --legacy-peer-deps
RUN cd ./Dashboard && npm run build
COPY Source/Programs/Horde/HordeDashboard/documentation ./Dashboard/dist/documentation

FROM nginx:1.17.10 as runtime-env
## Remove default nginx index page and conf
RUN rm -rf /usr/share/nginx/html/*
RUN rm /etc/nginx/conf.d/default.conf 
WORKDIR /app
COPY Source/Programs/Horde/HordeDashboard/nginx/nginx.conf /etc/nginx/conf.d
COPY --from=build-env /app/Dashboard/dist /usr/share/nginx/html
COPY --from=build-env /app/rds-combined-ca-bundle.pem .
EXPOSE 80
CMD ["nginx", "-g", "daemon off;"]

