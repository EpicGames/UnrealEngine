FROM node:20.11.0 as build-env
ARG VersionInfo
ENV VITE_VERSION_INFO=${VersionInfo}
WORKDIR /app

COPY Programs/Horde/HordeDashboard ./Dashboard
RUN cd ./Dashboard && npm install --legacy-peer-deps
RUN cd ./Dashboard && npm run build

