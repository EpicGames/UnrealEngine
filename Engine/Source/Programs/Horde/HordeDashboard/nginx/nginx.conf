server {
listen 80;
access_log off;
error_log off;
large_client_header_buffers 8 64k;
client_header_buffer_size 64k;

location ~ (^\/fonts\/|^\/images\/|^\/static\/|^\/modules\/|^\/src\/|^index.js|^manifest.json) {
    root   /usr/share/nginx/html;
    try_files $uri $uri/ =404;
  }

location / {
    root   /usr/share/nginx/html;
    index  index.html;
    if ( $uri = '' ) {
        add_header 'Cache-Control' 'no-store, no-cache, must-revalidate, proxy-revalidate, max-age=0';
    }
    if ( $uri = '/' ) {
        add_header 'Cache-Control' 'no-store, no-cache, must-revalidate, proxy-revalidate, max-age=0';
    }
    if ( $uri = '/index.html' ) {
        add_header 'Cache-Control' 'no-store, no-cache, must-revalidate, proxy-revalidate, max-age=0';
    }	
    try_files $uri $uri/ /index.html;
  }
}
