{
  "name": "horde-dashboard",
  "private": true,
  "version": "0.1.3",
  "type": "module",
  "scripts": {
    "dev": "vite",
    "build": "tsc && vite build",
    "lint": "eslint . --ext ts,tsx --report-unused-disable-directives --max-warnings 0",
    "preview": "vite preview",
    "license-check": "license-checker-rseidelsohn --production --excludePackages horde-dashboard --depth 0 --limitAttributes licenses --csv --customPath ./license-format.json --onlyAllow \"Apache-2.0;MIT;ISC\"",
    "test": "vitest run"
  },
  "dependencies": {
    "@blueprintjs/icons": "^3.13.0",
    "@datadog/browser-logs": "^4.30.0",
    "@fluentui/react": "8.106.2",
    "@fluentui/react-charting": "5.2.0",
    "d3": "^7.6.1",
    "lz-string": "^1.5.0",
    "markdown-to-jsx": "7.4.7",
    "mobx": "^6.12.2",
    "mobx-react-lite": "^4.0.7",
    "moment-timezone": "^0.5.40",
    "react": "^18.2.0",
    "react-custom-scrollbars": "^4.2.1",
    "react-dom": "^18.2.0",
    "react-flame-graph": "^1.4.0",
    "react-highlighter": "^0.4.3",
    "react-router-dom": "6.6.2",
    "react-router-hash-link": "^2.4.3",
    "react-sparklines": "^1.7.0",
    "react-text-marquee": "^1.0.1"
  },
  "devDependencies": {
    "@types/d3": "^7.4.0",
    "@types/react": "^18.2.66",
    "@types/react-custom-scrollbars": "^4.0.10",
    "@types/react-dom": "^18.2.22",
    "@types/react-highlighter": "^0.3.5",
    "@types/react-router-dom": "^5.3.3",
    "@types/react-sparklines": "^1.7.5",
    "@typescript-eslint/eslint-plugin": "^7.2.0",
    "@typescript-eslint/parser": "^7.2.0",
    "@vitejs/plugin-react": "^4.2.1",
    "eslint": "^8.57.0",
    "eslint-config-react-app": "^7.0.1",
    "eslint-plugin-react-hooks": "^4.6.0",
    "eslint-plugin-react-refresh": "^0.4.6",
    "license-checker-rseidelsohn": "4.3.0",
    "patch-package": "^8.0.1",
    "rimraf": "^3.0.0",
    "typescript": "^5.4.5",
    "vite": "^5.2.0",
    "vite-tsconfig-paths": "^4.3.2",
    "vitest": "^1.0.0"
  }
}