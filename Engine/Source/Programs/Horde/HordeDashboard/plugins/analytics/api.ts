// Copyright Epic Games, Inc. All Rights Reserved.

import backend from "horde/backend";

export type MetricsQuery = {
	id: string[];
	minTime?: string;
	maxTime?: string;
	group?: string;
	results?: number;
}

/// Metrics matching a particular query
export type GetTelemetryMetricsResponse = {

	metricId: string;

	groupBy: string;

	/// Metrics matching the search terms	
	metrics: GetTelemetryMetricResponse[];
}

/// Information about a particular metric
export type GetTelemetryMetricResponse = {

	/// Start time for the sample	
	time: Date;

	/// Name of the group	
	group?: string;

	/// Value for the metric	
	value: number;

	// Added locally in the dashboard
	// GetTelemetryMetricsResponse id
	id: string;

	// added locally by dashboard
	key: string;

	// added locally by dashboard
	keyElements: string[];

	threshold?: number;

	// calculated on dashboard, group name => value
	groupValues?: Record<string, string>;
}

export type TelemetryDisplayType = "Time" | "Ratio" | "Value";
export type TelemetryGraphType = "Line" | "Indicator";

/// Metric attached to a telemetry chart	
export type GetTelemetryChartMetricResponse = {

	/// Associated metric id	
	metricId: string;

	/// The threshold for KPI values	
	threshold?: number;

	/// The metric alias for display purposes	
	alias?: string;

}

/// Telemetry chart configuraton
export type GetTelemetryChartResponse = {

	/// The name of the chart, will be displayed on the dashboard	
	name: string;

	/// The unit to display	
	display: TelemetryDisplayType;

	/// The graph type 	
	graph: TelemetryGraphType;

	/// List of configured metrics	
	metrics: GetTelemetryChartMetricResponse[];

	/// The min unit value for clamping chart	
	min?: number;

	/// The max unit value for clamping chart	
	max?: number;
}

/// A chart categody, will be displayed on the dashbord under an associated pivot
export type GetTelemetryCategoryResponse = {

	/// The name of the category
	name: string;

	/// The charts contained within the category
	charts: GetTelemetryChartResponse[];
}

/// A telemetry view variable used for filtering the charting data
export type GetTelemetryVariableResponse = {
	/// The name of the variable for display purposes
	name: string;

	/// The associated data group attached to the variable 
	group: string;

	/// default values to select
	defaults: string[];

	/// Populated on dashboard
	values: string[];
}

/// A telemetry view of related metrics, divided into categofies
export type GetTelemetryViewResponse = {

	/// Identifier for the view
	id: string;

	/// The name of the view
	name: string;

	/// The telemetry store id the view uses
	telemetryStoreId: string;

	///  The variables used to filter the view data
	variables: GetTelemetryVariableResponse[];

	/// The categories contained within the view
	categories: GetTelemetryCategoryResponse[];
}

export async function getViews(): Promise<GetTelemetryViewResponse[]> {
	return new Promise<GetTelemetryViewResponse[]>((resolve, reject) => {
		backend.fetch.get(`api/v1/telemetry/views`).then((response) => {
			const result = response.data as GetTelemetryViewResponse[];
			resolve(result);
		}).catch(reason => { reject(reason); });
	});
}

export async function getMetrics(telemetryStoreId: string, query: MetricsQuery): Promise<GetTelemetryMetricsResponse[]> {
	query.id = query.id.map(id => encodeURIComponent(id));
	return new Promise<GetTelemetryMetricsResponse[]>((resolve, reject) => {		
		backend.fetch.get(`api/v1/telemetry/${telemetryStoreId}/metrics`, { params: query }).then((response) => {
			const result = response.data as GetTelemetryMetricsResponse[];
			result?.forEach(r => {
				r?.metrics.forEach(m => {
					if (m.time) {
						m.time = new Date(m.time);
					} else {
						console.warn("Metrics missing time property");
						m.time = new Date();
					}
				})
			})
			resolve(result);
		}).catch(reason => { reject(reason); });
	});
}



