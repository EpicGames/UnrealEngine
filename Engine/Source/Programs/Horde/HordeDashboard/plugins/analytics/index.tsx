// Copyright Epic Games, Inc. All Rights Reserved.

import { registerHordePlugin, MountType } from "..";
import { TelemetryView } from "./telemetryView";

registerHordePlugin({
   id: "analytics",
   routes: [{ path: "analytics", element: <TelemetryView /> }],
   mount: {
      type: MountType.TopNav,
      context: "Tools",
      text: "Analytics",
      route: `/analytics`
   }
})