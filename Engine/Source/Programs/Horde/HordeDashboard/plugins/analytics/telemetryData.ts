// Copyright Epic Games, Inc. All Rights Reserved.

import { getMetrics, GetTelemetryMetricsResponse, GetTelemetryViewResponse } from "./api";

export type MetricId = string;
export type TelemetryViewId = string;

export const graphColors = [
    "#ff780a", // o2
    "#3274d9", // b2
    "#56a64b", // g2
    "#e02f44", // r2
    "#f2cc0c", // y2
    "#a352cc", // p2
    "#ff9830", // o3
    "#5794f2", // b3
    "#73bf69", // g3
    "#f2495c", // r3
    "#fade2a", // y3
    "#b877d9", // p3
    "#fa6400", // o1
    "#1f60c4", // b1
    "#37872d", // g1
    "#c4162a", // r1
    "#e0b400", // y1
    "#e0b400", // p1
    "#ffb357", // o4
    "#8ab8ff", // b4
    "#96d98d", // g4
    "#733d47", // r4
    "#ffee52", // y4
    "#ca95e5", // p4
    "#ffcb7d", // o5
    "#c0d8ff", // b5
    "#c8f2c2", // g5
    "#ffa6b0", // r5
    "#fff899", // y5
    "#deb6f2" // p5
]

const groupRegex = /,(?=(?:[^"]*"[^"]*")*[^"]*$)/

const loadedMetrics = new Map<string, GetTelemetryMetricsResponse>();

export const clearTelemetryViewMetrics = () => {
    loadedMetrics.clear();
}

const getTelemetryViewMetrics = async (view: GetTelemetryViewResponse, categoryName: string, minTime: string, maxTime: string): Promise<GetTelemetryMetricsResponse[] | undefined> => {

    return new Promise<GetTelemetryMetricsResponse[] | undefined>(async (resolve, reject) => {

        const category = view.categories.find(c => c.name === categoryName);
        if (!category) {
            reject("Invalid category");
            return;
        }

        // get set of metrics
        const viewMetrics = new Set<string>(category.charts.map(c => c.metrics.map(m => m.metricId).flat()).flat());
        const needMetrics = new Set(viewMetrics);

        loadedMetrics.forEach((_, id) => needMetrics.delete(id));

        if (needMetrics.size) {

            const need: string[] = Array.from(needMetrics);
            const allMetrics = await getMetrics(view.telemetryStoreId, { id: need, minTime: minTime, maxTime: maxTime, results: 4096 * 32 });

            for (let i = 0; i < need.length; i++) {
                const metricId = need[i];
                const metrics = allMetrics.find(m => m.metricId === metricId);
                if (!metrics) {
                    loadedMetrics.set(metricId, { metricId: metricId, groupBy: "", metrics: [] });
                    continue;
                }

                const chartMetric = category.charts.map(c => c.metrics).flat().find(m => m.metricId === metricId);

                let groupBy = metrics.groupBy;
                if (!groupBy) {
                    reject(`Unable to get groupby for ${metricId}`);
                    return;
                }
                metrics.groupBy = groupBy.replaceAll("$.Payload.", "");

                const groups = metrics.groupBy.split(",");

                metrics.metrics = metrics.metrics.filter(m => {
                    m.id = metricId;
                    if (m.group) {

                        // todo: can remove this, filtering out some bad escaping from initial implementation
                        if (m.group.indexOf("\"\"") !== -1) {
                            return false;
                        }

                        const groupValues = m.group.split(groupRegex).map((v => v.replaceAll("\"", "")));

                        m.groupValues = {};
                        let key: string[] = [];
                        let skip = false;
                        groups.forEach((g, idx) => {
                            const value = groupValues[idx];
                            if (!value) {
                                skip = true;
                                return;
                            }
                            m.groupValues![g] = value;
                            key.push(value);
                        })

                        if (skip) {
                            return false;
                        }

                        if (chartMetric?.alias) {
                            key.push(chartMetric?.alias)
                        }
                        m.key = key.join(":");
                        m.keyElements = key;

                        if (chartMetric?.threshold) {
                            m.threshold = chartMetric.threshold;
                        }
                    }
                    return true;
                })

                loadedMetrics.set(metricId, metrics);
            }
        }

        const result: GetTelemetryMetricsResponse[] = [];

        viewMetrics.forEach(id => {

            result.push(loadedMetrics.get(id)!);
        })

        resolve(result);
    });
}

export type TelemetryViewData = {
    minTime: Date;
    maxTime: Date;
    metrics: GetTelemetryMetricsResponse[];
}

export const getTelemetryViewData = async (view: GetTelemetryViewResponse, categoryName: string, minDate: Date, maxDate: Date): Promise<TelemetryViewData | undefined> => {

    return new Promise<TelemetryViewData | undefined>(async (resolve, reject) => {

        const cat = view.categories.find(c => c.name === categoryName);
        if (!cat) {
            reject("Unable to get view category");
            return;
        }

        const viewMetrics: GetTelemetryMetricsResponse[] = await getTelemetryViewMetrics(view, categoryName, minDate.toISOString(), maxDate.toISOString()) as GetTelemetryMetricsResponse[];

        if (!viewMetrics?.length) {
            reject("Unable to get view metrics");
            return;
        }

        // get variables
        view.variables?.forEach(v => {
            const values = new Set<string>();
            const varGroup = v.group;
            viewMetrics.forEach(metric => {

                const groups = metric.groupBy.split(",");
                for (let i = 0; i < groups.length; i++) {
                    const group = groups[i];
                    if (varGroup.toLowerCase() === group.toLowerCase()) {
                        metric.metrics.forEach(m => {
                            if (m.groupValues) {
                                values.add(m.groupValues[group]);
                            }
                        })
                    }
                }
            })

            v.values = Array.from(values).sort((a, b) => a.localeCompare(b));

        })

        const result: TelemetryViewData = {
            minTime: minDate,
            maxTime: maxDate,
            metrics: viewMetrics
        }

        resolve(result);
    });

}