// Copyright Epic Games, Inc. All Rights Reserved.

import * as d3 from "d3";
import moment from "moment";
import { GetTelemetryMetricResponse, GetTelemetryMetricsResponse, GetTelemetryChartResponse } from "./api";
import dashboard from "horde/backend/Dashboard";
import { displayTimeZone, msecToElapsed } from "horde/base/utilities/timeUtils";
import { graphColors } from "./telemetryData";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type Zoom = d3.ZoomBehavior<Element, unknown>;
type Scalar = d3.ScaleLinear<number, number>;

type DataPoint = [number, number, string, number, number];

export class TelemetryLineRenderer {

   render(chart: GetTelemetryChartResponse, metrics: GetTelemetryMetricsResponse[], legend: string[], minTime: Date, maxTime: Date, container: HTMLDivElement, onZoom: (name: string, event: any) => void, onTimeSelect: (name: string, minTime: Date, maxTime: Date) => void, onDataHover: (key: string, x: number, y: number, time: Date, value: number, color: string) => void, scale = 1.0): any {

      let minValue = Number.MAX_SAFE_INTEGER
      let maxValue = Number.MIN_SAFE_INTEGER

      const allMetrics: GetTelemetryMetricResponse[] = [];

      metrics.forEach(metric => {
         metric.metrics.forEach(m => {
            allMetrics.push(m);
            minValue = Math.min(m.value, minValue);
            maxValue = Math.max(m.value, maxValue);
         })
      })

      const ratio = chart.display === "Ratio"

      minValue = chart.min ?? 0;

      if (ratio) {
         maxValue = 1.0
      }

      let svg = this.svg;
      const width = 1000;
      const height = 300;
      const margin = { top: 16, right: 32, bottom: 0, left: 64 };

      const x = this.scaleX = d3.scaleLinear()
         .domain([minTime, maxTime].map(d => d.getTime() / 1000))
         .range([margin.left, width - margin.right])

      const y = this.scaleY = d3.scaleLinear()
         .domain([minValue, maxValue / scale])
         .range([height - margin.bottom, margin.top])

      if (!svg) {
         svg = d3.select(container)
            .append("svg") as any as SelectionType

         this.svg = svg;
      } else {
         // remove tooltip
         d3.select(container).selectAll('div').remove();
         svg.selectAll("*").remove();
      }

      svg.attr("viewBox", [0, 0, width, height] as any)
         .attr("width", width)
         .attr("height", height)

      const clipId = `metrics_${chart.name}_clip}`;

      svg.append("clipPath")
         .attr("id", clipId)
         .append("rect")
         .attr("x", margin.left)
         .attr("y", 0)
         .attr("width", width - margin.left - margin.right + 2)
         .attr("height", height);

      const points = allMetrics.map((m, index) => [x(m.time.getTime() / 1000), y(m.value), m.key, legend.indexOf(m.key) % graphColors.length, index]) as DataPoint[];
      const groups = _d3.rollup(points, v => Object.assign(v, { z: v[0][2] }), d => d[2]);
      const gvalues = Array.from(groups.values());

      const line = d3.line().curve(d3.curveMonotoneX);
      svg.append("g")
         .attr("clip-path", `url(#${clipId})`)
         .attr("fill", "none")
         .attr("stroke-width", 1.5)
         .attr("stroke-linejoin", "round")
         .attr("stroke-linecap", "round")
         .selectAll("path")
         .data(gvalues)
         .join("path")
         .attr("stroke", (d:any) => { return d[0][3] !== undefined ? graphColors[d[0][3] as number] : "#8ab8ff" })
         .attr("d", line as any);

      svg.append("g")
         .selectAll("circle")
         .data(points)
         .join("circle")
         .attr("id", i => {
            return `circle_id_${i[4]}`
         })
         .attr("cx", (i) => {
            return i[0];
         })
         .attr("cy", i => i[1])
         .attr("fill", i => { return i[3] !== undefined ? graphColors[i[3] as number] : "#8ab8ff" })
         .attr("r", 2);

      let ticks: number[] = [];
      const inc = (maxTime.getTime() - minTime.getTime()) / 10;
      for (let i = 0; i < 10; i++) {
         ticks.push((minTime.getTime() + inc * i) / 1000);
      }

      const xAxis = (g: SelectionType) => {

         g.attr("transform", `translate(0,18)`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "11px")
            .call(d3.axisTop(x)
               .tickValues(ticks)
               .tickFormat(d => {
                  const time = moment(new Date((d as number) * 1000)).tz(displayTimeZone());
                  return time.format("MM/DD HH:mm")
               })
               .tickSizeOuter(0))
            .call(g => g.select(".domain").remove())
            .call(g => g.selectAll(".tick line").attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
               .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
               .attr("y2", height - margin.bottom))
      }

      // left axis
      const yAxis = (g: SelectionType) => {

         g.attr("transform", `translate(${margin.left},0)`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "9px")
            .call(d3.axisLeft(this.scaleY!)
               .ticks(10)
               .tickFormat((d) => {


                  if (ratio) {
                     return (d as number * 100).toString() + "%"
                  }

                  if (!d) {
                     return "";
                  }

                  if (chart.display === "Value") {
                     return d.toString();
                  }

                  return msecToElapsed((d as number) * 1000, true, true);

               }))
      }

      svg.append("g").attr("class", "x-axis").call(xAxis)
      svg.append("g").attr("class", "y-axis").call(yAxis)

      /*
      // zoom
      const zoom = this.zoom = d3.zoom()
         .scaleExtent([1, 12])
         .extent([[margin.left, 0], [width - margin.right, height]])
         .translateExtent([[margin.left, -Infinity], [width - margin.right, Infinity]])
         .on("zoom", zoomed as any);

      function zoomed(event: any, propogate = true) {

         if (propogate) {
            onZoom(chart.name, event);
         }

         x.range([margin.left, width - margin.right].map(d => event.transform.applyX(d)));

         const npoints = allMetrics.map((d) => [x(d.time.getTime() / 1000), y(d.value), d.key]);
         const ngroups = d3.rollup(npoints, v => Object.assign(v, { z: v[0][2] }), d => d[2]);

         svg!.selectAll("path")
            .data(ngroups.values())
            .join("path")
            .attr("d", line as any);


         svg!.selectAll(".x-axis").call(xAxis as any);
      }

      svg.call(zoom as any);
      */

      const brush = d3.brushX().extent([[margin.left, margin.top], [width - margin.right, height]]).on('end', (event: any) => {
         const extent = event?.selection
         if (!extent) {
            return;
         }

         const min = extent[0];// - margin.left;
         const max = extent[1];// - margin.left;
         const startDate = x.invert(min);
         const endDate = x.invert(max);

         onTimeSelect(chart.name, new Date(startDate * 1000), new Date(endDate * 1000))

      });

      svg.append("g")
         .attr("class", "brush")
         .call(brush);

      //svg.on("wheel", (event) => { event.preventDefault(); })

      const keyPoints = new Map<string, DataPoint[]>();
      points.forEach(p => {

         if (!keyPoints.has(p[2])) {
            keyPoints.set(p[2], [])
         }
         keyPoints.get(p[2])!.push(p);
      });

      const handleMouseMove = (event: any) => {

         const mouseX = _d3.pointer(event)[0];
         const mouseY = _d3.pointer(event)[1];

         // find closest point on x axis for each data key
         const closestX = new Map<string, DataPoint>();

         keyPoints.forEach((values, key) => {

            let closest = values.reduce((best, data, i) => {

               let absx = Math.abs(data[0] - mouseX);

               if (absx < best.value) {
                  return { index: i, value: absx };
               }
               else {
                  return best;
               }

            }, { index: 0, value: Number.MAX_SAFE_INTEGER });

            closestX.set(key, values[closest.index]);
         })

         svg!.selectAll(`circle`).attr("r", 2);

         let closestY: DataPoint | undefined;

         closestX.forEach((point) => {

            if (!closestY) {
               closestY = point;
            } else if (Math.abs(point[1] - mouseY) < Math.abs(closestY[1] - mouseY)) {
               closestY = point;
            }
            svg!.select(`#circle_id_${point[4]}`).attr("r", 4);
         })

         if (closestY && Math.abs(closestY[1] - mouseY) < 16 && Math.abs(closestY[0] - mouseX) < 16) {
            onDataHover(closestY[2], mouseX, mouseY, new Date(x.invert(closestY[0]) * 1000), y.invert(closestY[1]), closestY[3] !== undefined ? graphColors[closestY[3] as number] : "#8ab8ff")
         } else {
            onDataHover("__clear__", 0, 0, new Date(), 0, "");
         }
      }

      const handleMouseLeave = (event: any) => {
         svg!.selectAll(`circle`).attr("r", 2);
         onDataHover("__clear__", 0, 0, new Date(), 0, "");
      }

      svg.on("mousemove", (event) => handleMouseMove(event))
      svg.on("mouseleave", (event) => handleMouseLeave(event))

      return undefined;  //zoomed;
   }

   svg?: SelectionType;
   zoom?: Zoom;
   scaleX?: Scalar;;
   scaleY?: Scalar;;
   metrics?: GetTelemetryMetricsResponse;
}

