// Copyright Epic Games, Inc. All Rights Reserved.

import { ComboBox, DefaultButton, DirectionalHint, FontIcon, IComboBox, IComboBoxOption, IComboBoxStyles, ITooltipHostStyles, ITooltipProps, Icon, Pivot, PivotItem, SelectableOptionMenuItemType, Spinner, SpinnerSize, Stack, Text, TooltipHost, mergeStyleSets, mergeStyles } from "@fluentui/react";
import { useConst } from '@fluentui/react-hooks';
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { useWindowSize } from "horde/base/utilities/hooks";
import { displayTimeZone, msecToElapsed } from "horde/base/utilities/timeUtils";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import React, { useEffect, useId, useState } from "react";
import { useNavigate, useSearchParams } from "react-router-dom";
import { GetTelemetryChartResponse, GetTelemetryMetricResponse, GetTelemetryMetricsResponse, GetTelemetryVariableResponse, GetTelemetryViewResponse, getViews } from "./api";
import { TelemetryViewData, clearTelemetryViewMetrics, getTelemetryViewData, graphColors } from "./telemetryData";
import { TelemetryLineRenderer } from "./telemetryLineGraph";

const timeSelections: TimeSelection[] = [
   {
      text: "Past 1 Hour", key: "time_1_hour", minutes: 60
   },
   {
      text: "Past 2 Hours", key: "time_2_hours", minutes: 60 * 2
   },
   {
      text: "Past 4 Hours", key: "time_4_hours", minutes: 60 * 4
   },
   {
      text: "Past 1 Day", key: "time_1_day", minutes: 60 * 24
   },
   {
      text: "Past 2 Days", key: "time_2_days", minutes: 60 * 24 * 2
   },
   {
      text: "Past 1 Week", key: "time_1_week", minutes: 60 * 24 * 7
   },
   {
      text: "Past 2 Weeks", key: "time_2_weeks", minutes: 60 * 24 * 7 * 2
   },
   {
      text: "Past Month", key: "time_4_weeks", minutes: 60 * 24 * 7 * 4
   },
   {
      text: "Custom Range", key: "time_custom", minutes: 0, hidden: true
   }
]

type LegendEntry = {
   display: string;
   key: string;
   min: number;
   max: number;
   change: number;
}


type SearchState = {
   viewId?: string;
   category?: string;
   variables?: string[];
   minutes?: number;
}

class MetricsHandler {
   constructor() {
      makeObservable(this);
   }

   subscribe() {
      if (this.updated) { }
   }

   subscribeToSearch() {
      if (this.searchUpdated) { }
   }

   async set(category: string) {

      if (!this.view || category === this._category) {
         return;
      }

      this._category = category;
      this.querying = true;
      this.setUpdated();

      this.metrics = await getTelemetryViewData(this.view, category, this.minDate, this.maxDate);

      this.querying = false;
      this.setUpdated();
   }

   getCharts() {
      if (!this.searchState.category) {
         return [];
      }

      const category = this.view?.categories.find(c => c.name === this.searchState.category);
      if (!category) {
         return [];
      }

      return category.charts;
   }

   getChartVariables(): GetTelemetryVariableResponse[] {

      const variables = this.view?.variables;

      if (!variables?.length || !this.metrics) {
         return [];
      }

      const charts = this.getCharts();
      const allMetrics = charts.map(c => this.getAllChartMetrics(c.name).map(m => m.metrics).flat()).flat();

      const vars = variables.filter(v => {

         if (!v.values.length) {
            return false;
         }

         return !!allMetrics.find(m => m.groupValues && m.groupValues[v.group] !== undefined);
      });

      return vars;
   }

   getChart(chartName: string): GetTelemetryChartResponse | undefined {
      const category = this.view?.categories.find(c => !!c.charts.find(ch => ch.name === chartName));
      return category?.charts.find(ch => ch.name === chartName);
   }

   getChartLegend(chartName: string): LegendEntry[] {

      const mins = new Map<string, number>();
      const maxs = new Map<string, number>();
      const keyMetrics = new Map<string, GetTelemetryMetricResponse[]>();

      const chart = this.getChart(chartName);
      if (!chart) {
         return [];
      }

      const metrics = this.getFilteredChartMetrics(chartName);

      const replace: string[] = [];
      this.searchState.variables?.forEach(v => {
         const elements = v.split(";");
         if (elements.length === 2) {
            replace.push(elements[1] + ":")
         }
      });

      const legendSet = new Set<string>();

      metrics.forEach(metric => {

         metric.metrics.forEach(m => {

            mins.set(m.key, Math.min(mins.get(m.key) ?? Number.MAX_SAFE_INTEGER, m.value));
            maxs.set(m.key, Math.max(maxs.get(m.key) ?? Number.MIN_SAFE_INTEGER, m.value));

            if (!keyMetrics.has(m.key)) {
               keyMetrics.set(m.key, []);
            }

            keyMetrics.get(m.key)!.push(m);

            legendSet.add(m.key)
         })
      })

      const legend = Array.from(legendSet).sort((a, b) => a.localeCompare(b)).map((key) => {

         let display = key;
         for (let i = 0; i < replace.length; i++) {
            display = display.replace(replace[i], "")
         }

         let change = 0;

         const metrics = keyMetrics.get(key)!.sort((a, b) => {
            return a.time.getTime() - b.time.getTime();
         })

         if (metrics.length > 1) {
            const m1 = metrics[0].value;
            const m2 = metrics[metrics.length - 1].value;
            if (m1) {
               change = (m2 - m1) / m1;
            }

         }

         return {
            display: display,
            key: key,
            min: mins.get(key)!,
            max: maxs.get(key)!,
            change: change
         }
      });

      function getPrefix(words: string[]) {

         if (!words[0] || words.length === 1) return words[0] || "";

         const elementMap = new Map<number, Set<string>>();

         const length = words[0].split(":").length;

         if (length <= 1)
         {
            return "";
         }

         if (words.find(w => w.split(":").length !== length))
         {
            return "";
         }

         for (let i = 0; i < length; i++)
         {
            elementMap.set(i, new Set);
         }

         words.forEach(w => {

            const elements = w.split(":");

            elements.forEach((element, index) => {
               elementMap.get(index)!.add(element);
            })
         })

         let replace:string[] = [];         
         for (let count = 0; count < length - 1; count++)
         {
            if (elementMap.get(count)!.size > 1)
            {
               break;
            }

            replace.push(Array.from(elementMap.get(count)!)[0])            
         }         

         let replaceValue = replace.join(":");
         if (replaceValue) {
            replaceValue += ":";
         }

         return replaceValue;

      }

      // collapse to common value is left most variable only has one value
      const vars = handler.getChartVariables();
      if (vars.length > 0 && this.searchState.variables) {

         const v = this.searchState.variables.find(v => {
            const group = v.split(";")[0];
            return group === vars[0].group;
         })

         if (v && legend.length > 0 && v.split(";").length === 2) {
            const display = legend.map(v => v.display);            
            const prefix = getPrefix(display);
            if (prefix) {
               legend.forEach(v => {
                  const ndisplay = v.display.replace(prefix, "");
                  if (ndisplay) {
                     v.display = ndisplay;
                  }
               });
            }
         }
      }

      return legend;

   }

   private getAllChartMetrics(chartName: string): GetTelemetryMetricsResponse[] {

      if (!this.metrics) {
         return [];
      }

      const chart = this.getChart(chartName);
      if (!chart) {
         return [];
      }

      const cmetrics = new Set<string>(chart.metrics.map(cm => cm.metricId));

      let metrics = this.metrics.metrics.filter(m => {
         return cmetrics.has(m.metricId);
      })

      return metrics;

   }

   getFilteredChartMetrics(chartName: string, latest = false): GetTelemetryMetricsResponse[] {

      if (!this.metrics) {
         return [];
      }

      const chart = this.getChart(chartName);
      if (!chart) {
         return [];
      }

      const cmetrics = new Set<string>(chart.metrics.map(cm => cm.metricId));

      let metrics = this.metrics.metrics.filter(m => {

         return cmetrics.has(m.metricId);
      }).map(m => { return { ...m } as GetTelemetryMetricsResponse });

      if (chart.min !== undefined || chart.max !== undefined) {
         metrics.forEach(metric => {
            metric.metrics = metric.metrics.filter(m => {
               if (chart.min !== undefined && m.value < chart.min) {
                  return false;
               }

               if (chart.max !== undefined && m.value > chart.max) {
                  return false;
               }

               return true;
            })
         })
      }


      if (latest) {
         const found = new Set<string>();

         metrics.forEach(metric => {
            metric.metrics = metric.metrics.sort((a, b) => a.time.getTime() - b.time.getTime()).filter(m => {

               if (found.has(m.key)) {
                  return false;
               }
               found.add(m.key);
               return true;
            })
         })

      }

      this.searchState.variables?.forEach(v => {

         const values = v.split(";");
         const group = values.shift()!;

         metrics.forEach(metric => {
            metric.metrics = metric.metrics.filter(m => {

               if (m.groupValues && m.groupValues[group]) {
                  if (values.indexOf(m.groupValues[group]) === -1) {
                     return false;
                  }
               }

               return true;
            })
         })

      })

      return metrics;
   }

   clear() {
      this.initialized = false;
      this._category = undefined;
      this.anchorMinDate = undefined;
      this.anchorMaxDate = undefined;
      this.search = new URLSearchParams();
      this.searchState = {};
      this.view = undefined;
      this.telemetryViews = undefined;
      this.metrics = undefined;
      this.filteredKeys.clear();
      this.zoomHandler.clear();
      clearTelemetryViewMetrics();
   }

   valueToString(chart: GetTelemetryChartResponse, valueIn: number): string {

      let value = "";

      if (chart.display === "Ratio") {
         value = Math.round((valueIn * 100)).toString() + "%"
      }
      else if (chart.display === "Value") {
         value = Math.round(valueIn).toString();
      }
      else {
         value = msecToElapsed((valueIn) * 1000, true, true);
      }

      return value;
   }

   async initialize(viewId?: string, categoryIn?: string, callback?: () => void) {

      if (this.initialized || (viewId && this.view?.id === viewId)) {
         if (callback) {
            callback();
         }
         return;
      }

      this.initialized = true;

      this.telemetryViews = await getViews();

      this.view = undefined;

      if (!this.allViews.length) {
         console.log("No telemetry views configured");
         if (callback) {
            callback();
         }
         return;
      }

      this.search = new URLSearchParams();

      if (!viewId) {
         const query = new URLSearchParams(window.location.search).get("query");
         if (query?.length) {
            try {
               this.search = new URLSearchParams(atob(query));
            } catch (reason) {
               console.error(reason);
            }
         }

         this.searchState = this.stateFromSearch();
         this.view = this.allViews.find(v => v.id === this.searchState.viewId) ?? this.allViews[0];

      } else {
         this.searchState = {};
         this.searchState.minutes = 60 * 24;
         this.searchState.category = categoryIn;
         this.view = this.allViews.find(v => !viewId || v.id === viewId) ?? this.allViews[0];
         this.updateTime(this.searchState.minutes);
      }

      if (!this.view || !this.view.categories.length) {
         console.error("No view or no categories");
         return;
      }

      this.searchState.viewId = this.view.id;

      const category = this.searchState.category ?? this.view.categories[0].name;

      await this.setCategory(category)

      if (!this.searchState?.variables?.length) {

         const vars = this.getChartVariables();
         vars.forEach(v => {
            if (v.defaults?.length) {
               this.setVariables(v.group, v.defaults);
            }
         })
      }

      this.setUpdated();

      if (callback) {
         callback();
      }

   }

   async setCategory(category?: string, fromUser?: boolean) {

      if (fromUser) {
         this.anchorMinDate = this.anchorMaxDate = undefined;
      }

      if (category?.length) {
         if (this.searchState.category !== category) {
            this.searchState.category = category;
            this.updateSearch();
         }

         await this.set(category)
      }
   }

   setVariables(group: string, values: string[]) {
      let newVars = this.searchState.variables?.filter(v => {
         const [vgroup,] = v.split(";");
         if (group === vgroup) {
            return false;
         }
         return true;
      })

      if (!newVars) {
         newVars = [];
      }

      if (values.length) {
         newVars.push(`${group};` + values.join(";"));
      }

      newVars = newVars.sort((a, b) => a.localeCompare(b));
      this.searchState.variables = newVars.length ? newVars : undefined;

      this.updateSearch();
      this.setUpdated();

   }

   setFilterKeys(keys: Set<string>) {

      this.filteredKeys = keys;

      this.setUpdated();
   }

   @action
   setUpdated() {
      this.updated++;
   }

   @action
   setSearchUpdated() {
      this.searchUpdated++;
   }

   updateSearch(): boolean {

      const state = { ...this.searchState } as SearchState;

      const search = new URLSearchParams();
      const csearch = this.search.toString();

      if (state.viewId?.length) {
         search.append("view", state.viewId);
      }

      if (state.category?.length) {
         search.append("category", state.category);
      }

      if (state.minutes) {
         search.append("minutes", state.minutes.toString());
      }

      state.variables?.forEach(v => {
         search.append("v", v);
      });


      if (search.toString() !== csearch) {
         this.search = search;
         this.setSearchUpdated();
         return true;
      }

      return false;
   }

   stateFromSearch() {

      const state: SearchState = {};

      state.viewId = this.search.get("view") ?? undefined;
      state.category = this.search.get("category") ?? undefined;
      state.variables = this.search.getAll("v") ?? undefined;
      let minutes = Number.parseInt(this.search.get("minutes") ?? "0")
      if (!minutes) {
         minutes = 60 * 24;
      }
      state.minutes = minutes;

      this.updateTime(minutes);

      return state;
   }

   updateTime(minutes: number) {

      this.anchorMinDate = undefined;
      this.anchorMaxDate = undefined;

      this.selectMinDate = new Date(new Date().getTime() - (minutes * 60000));
      this.selectMaxDate = new Date();
   }

   reload() {

      // force an update
      const category = this._category;
      this._category = undefined;
      this.searchState.category = undefined;
      clearTelemetryViewMetrics();
      this.setCategory(category);

   }

   setTimeSelection(time: TimeSelection) {

      if (this.searchState.minutes === time.minutes && !this.anchorMinDate) {
         return;
      }

      this.searchState.minutes = time.minutes;
      this.updateTime(time.minutes);

      this.reload();
   }

   setView(viewId: string | undefined) {

      if (!viewId || this.view?.id === viewId) {
         return;
      }

      let category: string | undefined;

      if (this.view) {

         const newView = this.allViews.find(v => v.id === viewId);
         const ccat = this.view.categories.find(c => c.name === this._category);

         if (ccat && newView) {
            category = newView.categories.find(c => ccat.name === c.name)?.name;
         }
      }

      this._category = undefined;
      this.anchorMinDate = undefined;
      this.anchorMaxDate = undefined;
      this.search = new URLSearchParams();
      this.searchState = {};
      this.filteredKeys.clear();
      this.metrics = undefined;

      clearTelemetryViewMetrics();

      this.initialized = false;
      this.initialize(viewId, category);
   }


   setZoomHandler(chartName: string, zoomed: any) {
      this.zoomHandler.set(chartName, zoomed);
   }

   onZoom(originatingChartName: string, event: any) {
      this.zoomHandler.forEach((zoomed, chartName) => {
         if (originatingChartName !== chartName) {
            zoomed(event, false);
         }
      })
   }

   onTimeSelect(chartName: string, minTime: Date, maxTime: Date) {

      this.anchorMinDate = minTime;
      this.anchorMaxDate = maxTime;

      this.reload();
   }

   @observable
   private updated = 0;

   @observable
   private searchUpdated: number = 0;

   view?: GetTelemetryViewResponse;

   anchorMinDate?: Date;
   anchorMaxDate?: Date;
   restoreMinutes?: number;

   selectMinDate: Date = new Date();
   selectMaxDate: Date = new Date();

   get minDate(): Date {
      return this.anchorMinDate ?? this.selectMinDate;
   }

   get maxDate(): Date {
      return this.anchorMaxDate ?? this.selectMaxDate;
   }

   querying = false;

   initialized = false;

   filteredKeys = new Set<string>();
   zoomHandler = new Map<string, any>();

   searchState: SearchState = {}

   search: URLSearchParams = new URLSearchParams();

   get allViews() { return this.telemetryViews ?? [] }

   private telemetryViews?: GetTelemetryViewResponse[];

   private metrics?: TelemetryViewData;

   private _category?: string;
}

const handler = new MetricsHandler();

const pivotClasses = mergeStyleSets({
   pivot: {
      selectors: {
         ".ms-Pivot-link": {
            lineHeight: "36px",
            height: "36px",
            paddingTop: 0,
            paddingBottom: 0
         }
      }
   }
});

const ViewChooser: React.FC = observer(() => {

   handler.subscribe();

   const options: IComboBoxOption[] = handler.allViews.map(v => {
      return {
         key: `view_option_${v.id}`,
         text: v.name,
         data: v
      }
   });

   let key: string | undefined;

   if (handler.view) {
      key = `view_option_${handler.view.id}`
   }

   return <Stack>
      <ComboBox
         styles={{ root: { width: 120 } }}
         options={options}
         selectedKey={key}
         onChange={(ev, option, index, value) => {
            handler.setView(option?.data?.id);
         }}
      />
   </Stack>
})

const TelemetryPivot: React.FC = observer(() => {

   handler.subscribe();

   const view = handler.view;

   if (!view) {
      return null;
   }

   const links = view.categories.map(cat => {
      const key = `item_key_${cat.name}`;
      return <PivotItem headerText={cat.name} key={key} itemKey={key} />
   });

   return <Stack horizontal tokens={{ childrenGap: 18 }} verticalAlign="center" verticalFill={false}>
      <Pivot className={pivotClasses.pivot} defaultSelectedKey={`item_key_${handler.searchState.category}`} linkSize="normal" linkFormat="links" onLinkClick={(item, ev) => {
         if (item) {

            const catname = item.props.itemKey!.replace("item_key_", "");
            handler.setCategory(catname, true);
         }
      }}>
         {links}
      </Pivot>
   </Stack>;
})

let multiComboBoxId = 0;

const VariableChooser: React.FC<{ group: string, label?: string, multiSelect: boolean, optionsIn: IComboBoxOption[], initialKeysIn: string[], updateKeys: (group: string, selectedKeys: string[]) => void }> = ({ group, label, multiSelect, optionsIn, initialKeysIn, updateKeys }) => {

   const comboBoxRef = React.useRef<IComboBox>(null);

   let initialKeys = [...initialKeysIn];
   let options = [...optionsIn];

   if (options.length === initialKeys.length) {
      initialKeys.push('selectAll');
   }

   if (multiSelect && options.length > 3) {
      options.unshift({ key: 'selectAll', text: 'Select All', itemType: SelectableOptionMenuItemType.SelectAll });
   }

   const comboBoxStyles: Partial<IComboBoxStyles> = { root: { width: 270 } };

   return <ComboBox componentRef={comboBoxRef} key={`multi_option_${group}_${multiComboBoxId++}`} label={label} placeholder="None" defaultSelectedKey={initialKeys} multiSelect={multiSelect} options={options} onResolveOptions={() => options}
      onChange={multiSelect ? undefined : (event, option, index, value) => {
         setTimeout(() => { multiComboBoxId++; updateKeys(group, [value ?? ""]) }, 250);
      }}
      onMenuDismiss={!multiSelect ? undefined : () => {
         if (comboBoxRef?.current?.selectedOptions) {
            const selectedKeys = comboBoxRef.current.selectedOptions.map(o => o.key as string).filter(k => k !== 'selectAll');
            setTimeout(() => { multiComboBoxId++; updateKeys(group, selectedKeys) }, 250);
         }
      }} styles={comboBoxStyles} />
};


const TelemetryChooser: React.FC = observer(() => {

   handler.subscribe();

   const view = handler.view;

   if (!view) {
      return null;
   }

   const varStacks: JSX.Element[] = [];

   const vars = handler.getChartVariables();

   vars.forEach(v => {

      const valueSet = new Set<string>();
      v.defaults.forEach(v => valueSet.add(v));
      v.values.forEach(v => valueSet.add(v));

      const options: IComboBoxOption[] = Array.from(valueSet).sort((a, b) => a.localeCompare(b)).map(name => {
         return {
            key: name,
            text: name
         }
      })

      if (!options.length) {
         return;
      }

      let defaultKeys: string[] = options.map(o => o.key as string);

      const state = handler.searchState;
      if (state.variables?.length) {
         const sv = state.variables?.find(sv => sv.startsWith(`${v.group};`))
         if (sv) {
            defaultKeys = sv.split(";");
            defaultKeys.shift();
         }
      }

      const stack = <Stack key={`key_chooser_${handler.view?.id}_${v.group}`}>
         <VariableChooser
            group={v.group}
            label={v.name}
            multiSelect={true}
            optionsIn={options}
            initialKeysIn={defaultKeys}
            updateKeys={(group, keys) => {
               handler.setVariables(group, keys);
            }}
         />
      </Stack>

      varStacks.push(stack);

   })

   if (!varStacks.length) {
      return null;
   }

   return <Stack horizontal tokens={{ childrenGap: 24 }} verticalAlign="center" verticalFill={false}>
      {varStacks}
   </Stack>;
})

type TimeSelection = {
   text: string;
   key: string;
   minutes: number;
   hidden?: boolean;
}

const TimeChooser: React.FC = observer(() => {

   handler.subscribe();

   let timeComboWidth = 180;

   let key: string | undefined;

   if (handler.anchorMinDate) {
      key = "time_custom";
   } else {
      key = timeSelections.find(t => t.minutes === handler.searchState.minutes)?.key;
      if (!key) {
         return null;
      }
   }

   return <Stack>
      <ComboBox
         styles={{ root: { width: timeComboWidth } }}
         options={timeSelections}
         selectedKey={key}
         onChange={(ev, option, index, value) => {
            const select = option as TimeSelection;
            handler.setTimeSelection(select);
         }}
      />
   </Stack>

})

const Legend: React.FC<{ chart: GetTelemetryChartResponse }> = observer(({ chart }) => {

   const { modeColors } = getHordeStyling();

   const tooltipId = useId();

   handler.subscribe();

   const legend = handler.getChartLegend(chart.name);

   const legendStacks: JSX.Element[] = [];

   const calloutProps = { gapSpace: 0 };
   const hostStyles: Partial<ITooltipHostStyles> = { root: { display: 'inline-block' } };

   legend.forEach((v, index) => {

      const filtered = handler.filteredKeys.has(v.key);

      const tooltipProps: ITooltipProps = {

         onRenderContent: () => {

            const entry = legend.find(n => n.key === v.key);
            if (!entry) {
               return null;
            }

            return <Stack style={{ backgroundColor: modeColors.background, border: "solid", borderWidth: "1px", borderRadius: "3px", borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F" }}>
               <Stack style={{ padding: "16px 16px" }} tokens={{ childrenGap: 8 }}>
                  <Stack>
                     <Text variant="small">Min: {handler.valueToString(chart, entry.min)}</Text>
                  </Stack>
                  <Stack>
                     <Text variant="small">Max: {handler.valueToString(chart, entry.max)}</Text>
                  </Stack>
                  <Stack>
                     <Text variant="small">Change: {Math.round((entry.change * 100)).toString() + "%"}</Text>
                  </Stack>
               </Stack>
            </Stack>
         }
      };

      const stack = <TooltipHost
         tooltipProps={tooltipProps}
         id={tooltipId}
         delay={0}
         directionalHint={DirectionalHint.leftCenter}
         calloutProps={calloutProps}
         styles={hostStyles}>
         <Stack key={`key_legend_${v.key}`} style={{ userSelect: "none" }} horizontal verticalAlign="center" tokens={{ childrenGap: 8 }} onClick={(ev) => {

            let keys = legend.map(l => l.key);

            const mod = ev.shiftKey || ev.ctrlKey;
            if (!mod) {

               if (handler.filteredKeys.size && !handler.filteredKeys.has(v.key)) {
                  handler.setFilterKeys(new Set());
               } else {
                  const nset = new Set<string>();
                  keys.forEach(k => {
                     if (k !== v.key) {
                        nset.add(k);
                     }
                  })
                  handler.setFilterKeys(nset);
               }
            } else {

               let nset = new Set(handler.filteredKeys);
               if (nset.has(v.key)) {
                  nset.delete(v.key);
               } else {
                  if (!nset.size) {
                     nset = new Set<string>();

                     keys.forEach(k => {
                        if (k !== v.key) {
                           nset.add(k);
                        }
                     })
                  } else {
                     nset.add(v.key);
                  }
               }

               if (nset.size === legend.length) {
                  nset = new Set();
               }

               handler.setFilterKeys(nset);

            }

         }}>
            <Stack>
               <FontIcon style={{ color: filtered ? "#999999" : graphColors[index % graphColors.length], paddingTop: 2 }} iconName="Square" />
            </Stack>
            <Stack>
               <Text style={{ fontSize: "11px", color: filtered ? "#999999" : undefined }}>{v.display}</Text>
            </Stack>
         </Stack>
      </TooltipHost>

      legendStacks.push(stack)

   })

   return <Stack style={{ cursor: "pointer", width: 340, height: 300, overflowY: "auto" }} tokens={{ childrenGap: 4 }}>{legendStacks}</Stack>
})


const indicatorStyles = mergeStyleSets({

   stripes: {
      backgroundImage: 'repeating-linear-gradient(-45deg, rgba(255, 255, 255, .2) 25%, transparent 25%, transparent 50%, rgba(255, 255, 255, .2) 50%, rgba(255, 255, 255, .2) 75%, transparent 75%, transparent)',
   }

});

export type IndicatorBarStack = {
   value: number,
   title?: string,
   titleValue?: number,
   color?: string,
   onClick?: () => void,
   stripes?: boolean,
   brightness?: number
}

export const IndicatorBar: React.FC<{ stack: IndicatorBarStack[], width: number, height: number, basecolor?: string, style?: any }> = ({ stack, width, height, basecolor, style }) => {
   stack = stack.filter(s => s.value > 0);

   return (
      <div className={mergeStyles({ backgroundColor: basecolor, width: width, height: height, verticalAlign: 'middle', display: "flex" }, style)}>
         {stack.map((item) => {

            let boxShadow = !item.brightness ? `0 0 3px ${item.color}` : undefined;
            let filter = item.brightness ? `brightness(${item.brightness})` : undefined;
            if (!dashboard.darktheme) {
               boxShadow = undefined;
            }

            const iwidth = width * (item.value / 100);
            return <span key={`${item.title!}_${metricIdCounter++}`}
               onClick={item.onClick}
               className={item.stripes ? indicatorStyles.stripes : undefined}
               style={{
                  width: `${iwidth}px`, height: '100%',
                  backgroundColor: item.color,
                  margin: "1px",
                  display: 'block',
                  borderRadius: "2px",
                  cursor: item.onClick ? 'pointer' : 'inherit',
                  backgroundSize: `${height * 2}px ${height * 2}px`,
                  boxShadow: boxShadow,
                  filter: filter
               }} />
         })}
      </div>
   );
}

let metricIdCounter = 0;

const IndicatorTile: React.FC<{ chart: GetTelemetryChartResponse }> = observer(({ chart }) => {

   const { modeColors } = getHordeStyling();

   handler.subscribe();

   const metrics = handler.getFilteredChartMetrics(chart.name);

   const legend = handler.getChartLegend(chart.name);

   if (!metrics?.length) {
      return <Text>No Data</Text>;
   }

   const colors = dashboard.getStatusColors();

   let allMetrics = metrics.map(m => m.metrics).flat().sort((a, b) => a.key.localeCompare(b.key));

   const keyValues = new Map<string, number[]>();
   allMetrics.forEach(m => {

      if (!keyValues.has(m.key)) {
         keyValues.set(m.key, []);
      }
      keyValues.get(m.key)!.push(m.value);
   })

   const keyAverages = new Map<string, number>();
   keyValues.forEach((values, key) => {
      let avg = 0;
      values.forEach(v => avg += v);
      avg /= values.length;
      keyAverages.set(key, Math.round(avg));
   })

   const found = new Set<string>();

   allMetrics = allMetrics.sort((a, b) => a.time.getTime() - b.time.getTime());
   allMetrics = allMetrics.filter(m => {

      if (found.has(m.key)) {
         return false;
      }

      found.add(m.key);
      return true;
   })

   allMetrics.forEach(m => {
      m.value = keyAverages.get(m.key) ?? 0;
   })

   const elements: JSX.Element[] = [];

   allMetrics.forEach(m => {

      const barStack: IndicatorBarStack[] = [];

      const max = chart.max ?? 100;
      const threshold = m.threshold ?? max;

      const v = m.value / max;
      const t = threshold / max;
      for (let i = 0.0; i < 1; i += .1) {

         let color = i <= t ? colors.get(StatusColor.Success)! : colors.get(StatusColor.Failure)!;

         let brightness: number | undefined;
         if (i > v) {
            brightness = dashboard.darktheme ? 0.4 : 1;
            if (!dashboard.darktheme) {
               color += "4B"
            }
         }
         barStack.push({ value: 10, color: color, brightness: brightness });
      }

      const name = legend.find(v => v.key === m.key)?.display ?? m.key;

      const element = <Stack horizontal verticalAlign="center" key={`indicator_bar_${metricIdCounter++}`}>
         <Stack style={{ width: 340 }}>
            <Text variant="small">{name}</Text>
         </Stack>
         <Stack>
            <IndicatorBar stack={barStack} width={160} height={14} />
         </Stack>
         <Stack style={{ width: 90 }} horizontalAlign="end">
            <Text variant="small">{chart.display === "Value" ? m.value : msecToElapsed(m.value * 1000)}</Text>
         </Stack>
      </Stack>

      elements.push(element)
   })

   return <Stack style={{ paddingTop: 12 }}>
      <Stack tokens={{ childrenGap: 6 }} style={{ backgroundColor: modeColors.background, padding: 18, height: 300, overflowY: "auto" }}>
         {elements}
      </Stack>
   </Stack>
})


class Tooltip {
   constructor() {
      makeObservable(this);
   }

   subscribe() {
      if (this.updated) { }
   }

   @action
   set(key: string, x: number, y: number, time: Date, value: number, color: string) {

      if (key === "__clear__") {
         this.show = false;
         this.updated++;
         return;
      }
      this.show = true;
      this.color = color;
      this.key = key;
      this.x = x;
      this.y = y;
      this.time = time;
      this.value = value;
      this.updated++;
   }

   show = false;

   key: string = "";
   x: number = 0;
   y: number = 0;
   time: Date = new Date();
   value: number = 0;
   color: string = "";

   @observable
   private updated = 0;
}

const GraphTooltip: React.FC<{ chart: GetTelemetryChartResponse, tooltip: Tooltip, legend: LegendEntry[] }> = observer(({ chart, tooltip, legend }) => {

   const { modeColors } = getHordeStyling();

   tooltip.subscribe();

   if (!tooltip.show) {
      return null;
   }

   let tipX = tooltip.x;
   let offsetX = 32;
   let translateX = "0%";

   if (tipX > 800) {
      offsetX = -32;
      translateX = "-100%";
   }

   const translateY = "-50%";

   const time = moment(tooltip.time).tz(displayTimeZone());

   let value = handler.valueToString(chart, tooltip.value);

   let name = legend.find(k => k.key === tooltip.key)?.display ?? tooltip.key;

   return <div style={{
      position: "absolute",
      display: "block",
      top: `${tooltip.y}px`,
      left: `${tooltip.x + offsetX}px`,
      backgroundColor: modeColors.background,
      zIndex: 1,
      border: "solid",
      borderWidth: "1px",
      borderRadius: "3px",
      width: "max-content",
      borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F",
      pointerEvents: "none",
      transform: `translate(${translateX}, ${translateY})`
   }}>
      <Stack style={{ padding: "16px 16px" }} tokens={{ childrenGap: 8 }}>
         <Stack horizontal tokens={{ childrenGap: 4 }}>
            <FontIcon style={{ color: tooltip.color, paddingTop: 3 }} iconName="Square" />
            <Text>{name}</Text>
         </Stack>
         <Stack>
            <Text>{time.format("MM/DD HH:mm")}</Text>
         </Stack>
         <Stack>
            <Text>{value}</Text>
         </Stack>
      </Stack>
   </div>

})

const LineGraphTile: React.FC<{ chart: GetTelemetryChartResponse }> = observer(({ chart }) => {

   const [scale] = useState(1);
   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const renderer = useConst(chart.graph === "Line" ? new TelemetryLineRenderer() : new TelemetryLineRenderer());
   const tooltip = useConst(new Tooltip());

   const { hordeClasses, modeColors } = getHordeStyling();

   handler.subscribe();

   const graph_container_id = `metric_graph_container_${chart.name}}`;

   const metrics = handler.getFilteredChartMetrics(chart.name);

   let hasMetrics = false;
   metrics.forEach(metric => {
      metric.metrics = metric.metrics.filter(m => !handler.filteredKeys.has(m.key));
      if (metric.metrics.length) {
         hasMetrics = true;
      }
   })

   if (!hasMetrics) {
      return <Stack style={{ paddingTop: 12 }}><Text>No Matching Data</Text></Stack>;
   }

   const legend = handler.getChartLegend(chart.name);

   if (container) {
      try {

         const onZoom = (chartName: string, event: any) => {
            handler.onZoom(chartName, event);
         };

         const onTimeSelect = (chartName: string, minTime: Date, maxTime: Date) => {
            handler.onTimeSelect(chartName, minTime, maxTime);
         }

         const onDataHover = (key: string, x: number, y: number, time: Date, value: number, color: string) => {

            tooltip.set(key, x, y, time, value, color);
         }

         const zoomed = renderer.render(chart, metrics, legend.map(v => v.key), handler.minDate!, handler.maxDate!, container, onZoom, onTimeSelect, onDataHover, scale);
         handler.setZoomHandler(chart.name, zoomed);

      } catch (err) {
         console.error(err);
      }
   }
   const width = 1024;

   return <Stack className={hordeClasses.horde} key={`metric_graph_stack_${chart.name}`}>
      <Stack style={{ width: "100%", paddingTop: 16, paddingBottom: 16, paddingLeft: 16, backgroundColor: modeColors.background }} horizontal tokens={{ childrenGap: 12 }}>
         <Stack style={{ width: width, position: "relative" }}>
            <GraphTooltip chart={chart} tooltip={tooltip} legend={legend} />
            <div id={graph_container_id} style={{ shapeRendering: "geometricPrecision", userSelect: "none" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
         </Stack>
         <Stack>
            <Legend chart={chart} />
         </Stack>
      </Stack>

   </Stack>;
})

const TelemetryViewInternal: React.FC = observer(() => {

   const { hordeClasses } = getHordeStyling();

   handler.subscribe();

   if (handler.querying) {
      return <Stack>
         <Text>Loading Data</Text>
         <Spinner size={SpinnerSize.large} />
      </Stack>
   }

   const charts = handler.getCharts();

   const indicators = charts.filter(c => c.graph === "Indicator");
   const lines = charts.filter(c => c.graph === "Line");

   const ipanels = indicators.map(chart => {
      return <Stack key={`telemetry_view_panel_${chart.name}`} styles={{ root: { paddingTop: 0, paddingRight: 12, width: 720 } }}>
         <Stack className={hordeClasses.raised} >
            <Stack tokens={{ childrenGap: 12 }} grow>
               <Stack >
                  <Stack >
                     <Text>{chart.name}</Text>
                  </Stack>
                  <Stack>
                     <IndicatorTile chart={chart} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   })

   const linepanels = lines.map(chart => {
      return <Stack key={`telemetry_view_panel_${chart.name}`} styles={{ root: { paddingTop: 0, paddingRight: 12 } }}>
         <Stack className={hordeClasses.raised} >
            <Stack tokens={{ childrenGap: 12 }} grow>
               <Stack >
                  <Stack >
                     <Text>{chart.name}</Text>
                  </Stack>
                  <Stack>
                     <LineGraphTile chart={chart} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   })

   return <Stack tokens={{ childrenGap: 12 }}>
      <Stack horizontal>
         {ipanels}
      </Stack>
      {linepanels}
   </Stack>
})

export const SearchUpdate: React.FC = observer(() => {

   const [, setSearchParams] = useSearchParams();
   const csearch = "query=" + btoa(handler.search.toString());

   useEffect(() => {
      if (handler.search.toString().length) {
         setSearchParams(csearch, { replace: true });
      }

   }, [csearch, setSearchParams])

   // subscribe
   handler.subscribeToSearch();

   return null;
});

export const TelemetryView: React.FC = () => {

   const [initialized, setInitialized] = useState(false);

   useEffect(() => {
      handler.initialize(undefined, undefined, () => setInitialized(true));
      return () => {
         handler.clear();
      };
   }, []);

   const windowSize = useWindowSize();
   const navigate = useNavigate();

   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   const rootWidth = 1440;
   const centerAlign = vw / 2 - 720 /*890*/;
   const key = `windowsize_metrics_view_${windowSize.width}_${windowSize.height}`;

   const telemetrybDocs = "/docs/Config/Analytics.md";

   return <Stack className={hordeClasses.horde} key="key_metrics_graph_test">
      <SearchUpdate />
      <TopNav />
      <Breadcrumbs items={[{ text: 'Analytics' }]} />
      <Stack horizontal styles={{ root: { backgroundColor: modeColors.background } }}>
         <Stack styles={{ root: { width: "100%" } }}>
            {!initialized && <Stack style={{ paddingTop: 24 }}><Spinner size={SpinnerSize.large} /></Stack>}
            {initialized && !handler.allViews.length && <Stack horizontal tokens={{ childrenGap: 6 }} horizontalAlign="center" style={{ paddingTop: 30 }}>
               <Text variant="mediumPlus">No analytic views found, for more information please see</Text>
               <a href={telemetrybDocs} style={{ fontSize: "18px", "cursor": "pointer" }} onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(telemetrybDocs) }}> Horde analytics documentation.</a>
            </Stack>}
            {initialized && !!handler.allViews.length && <Stack>
               <Stack horizontal>
                  <Stack key={`${key}_1`} style={{ paddingLeft: centerAlign }} />
                  <Stack style={{ width: rootWidth - 8, maxWidth: windowSize.width - 12, paddingLeft: 0, paddingTop: 24, paddingBottom: 24, paddingRight: 0 }} >
                     <Stack>
                        <Stack horizontal verticalAlign="center">
                           <TelemetryPivot />
                           <Stack grow />
                           <Stack horizontal tokens={{ childrenGap: 14 }} verticalAlign="center" verticalFill>
                              <Stack>
                                 <ViewChooser />
                              </Stack>
                              <Stack >
                                 <TimeChooser />
                              </Stack>
                              <Stack>
                                 <DefaultButton style={{ minWidth: 52, height: 34 }} onClick={() => handler.reload()}>
                                    <Icon iconName='Refresh' />
                                 </DefaultButton>
                              </Stack>
                           </Stack>
                        </Stack>
                     </Stack>
                     <Stack horizontal>
                        <Stack grow />
                        <Stack style={{ paddingRight: 2, paddingTop: 12, paddingBottom: 12 }}>
                           <TelemetryChooser />
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
               <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
                  <Stack horizontal>
                     <Stack /*className={classNames.pointerSuppress} */ style={{ position: "relative", width: "100%", height: 'calc(100vh - 228px)' }}>
                        <div id="hordeContentArea" style={{ overflowX: "auto", overflowY: "visible" }}>
                           <Stack horizontal style={{ paddingBottom: 48 }}>
                              <Stack key={`${key}_2`} style={{ paddingLeft: centerAlign }} />
                              <Stack style={{ width: rootWidth }} tokens={{ childrenGap: 12 }}>
                                 <TelemetryViewInternal />
                              </Stack>
                           </Stack>
                        </div>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>}
         </Stack>
      </Stack>
   </Stack>
}