// Copyright Epic Games, Inc. All Rights Reserved.

import backend from "horde/backend";

export async function getGreeting(): Promise<string> {
    return new Promise<string>((resolve, reject) => {
        backend.fetch.get(`/api/v1/exampleplugin/greeting`).then((response) => {
            resolve(response.data as string);
        }).catch(reason => { reject(reason); });
    });
}