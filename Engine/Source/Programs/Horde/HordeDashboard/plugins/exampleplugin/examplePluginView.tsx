// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from "@fluentui/react"
import { getGreeting } from "./api"
import { useState } from "react";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";


export const ExamplePluginView: React.FC = () => {

    const [greeting, setGreeting] = useState("Getting Greeting");

    getGreeting().then(greeting => {
        setGreeting(greeting);
    })

    const hordeClasses = getHordeStyling();

    return <Stack className={hordeClasses.horde} key="key_metrics_graph_test">
            <TopNav />
            <Breadcrumbs items={[{ text: 'Example Plugin' }]} />
            <Stack horizontalAlign="center" style={{ paddingTop: 24 }}>
                <Text variant="mediumPlus">{greeting}</Text>
            </Stack>
        </Stack>


}