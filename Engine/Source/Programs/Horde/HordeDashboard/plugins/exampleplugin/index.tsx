// Copyright Epic Games, Inc. All Rights Reserved.

import { registerHordePlugin, MountType } from "..";
import { ExamplePluginView } from "./examplePluginView";

registerHordePlugin({
   id: "exampleplugin",
   routes: [{ path: "exampleplugin", element: <ExamplePluginView /> }],
   mount: {
      type: MountType.TopNav,
      context: "Tools",
      text: "ExamplePlugin",
      route: `/exampleplugin`
   }
})