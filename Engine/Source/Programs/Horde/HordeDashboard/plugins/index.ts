// Copyright Epic Games, Inc. All Rights Reserved.

import { RouteObject } from "react-router-dom";
import { ServerPluginInfoResponse } from "horde/backend/Api";

// Routes => components
// backend queries
// mounting points???

export enum MountType {
   TopNav
}

export type Mount = {
   type: MountType,
   context?: string;
   text: string;
   route: string;
   // ACLs required to show this mount point (all must be present)
   requiredAcls?: string[];
}

export type HordePlugin = {
   id: string;
   routes: RouteObject[];
   // single plugin mount point
   mount?: Mount;
   // multiple plugin mount points
   mounts?: Mount[];
   enabled?: boolean;
   // ACLs required to enable the entire plugin (all must be present)
   requiredAcls?: string[];
   // User's ACLs for this plugin (populated from server response)
   userAcls?: string[];
}

const plugins: HordePlugin[] = [];

export function registerHordePlugin(plugin: HordePlugin) {
   const existing = plugins.find(p => p.id === plugin.id);
   if (existing) {
      throw `Duplicate plugin registration for id: ${plugin.id}`;
   }

   plugins.push(plugin);
}

/**
 * Enables plugins based on server plugin info, checking both loaded status and ACL requirements.
 *
 * ACL semantics from server:
 * - undefined/null: Plugin doesn't implement dashboard ACLs (no restrictions)
 * - empty array []: Plugin uses ACLs but user has no permissions
 * - non-empty array: User's granted ACL actions for this plugin
 *
 * @param serverPlugins Plugin information from the server, including user's ACLs
 */
export function enableHordePlugins(serverPlugins: ServerPluginInfoResponse[]) {
   // Use lowercase keys for case-insensitive matching between frontend plugin IDs and server plugin names
   const pluginMap = new Map(serverPlugins.map(p => [p.name.toLowerCase(), p]));

   plugins.forEach(p => {
      const serverPlugin = pluginMap.get(p.id.toLowerCase());

      // Plugin must be loaded on the server
      if (!serverPlugin?.loaded) {
         p.enabled = false;
         p.userAcls = undefined;
         return;
      }

      // Store the user's ACLs for this plugin (undefined means no ACL restrictions)
      p.userAcls = serverPlugin.userAcls;

      // Check plugin-level ACL requirements
      if (p.requiredAcls?.length) {
         const userAclSet = new Set(serverPlugin.userAcls ?? []);
         const hasAllRequired = p.requiredAcls.every(acl => userAclSet.has(acl));
         p.enabled = hasAllRequired;
      } else {
         p.enabled = true;
      }
   });
}

export function getHordePlugins(): HordePlugin[] {
   return plugins.filter(p => !!p.enabled);
}

/**
 * Gets the filtered mounts for a plugin based on the user's ACLs
 * @param plugin The plugin to get mounts for
 * @returns Array of mounts the user has access to
 */
export function getFilteredMounts(plugin: HordePlugin): Mount[] {
   const allMounts = plugin.mounts ?? (plugin.mount ? [plugin.mount] : []);
   const userAclSet = new Set(plugin.userAcls ?? []);

   return allMounts.filter(mount => {
      // If no required ACLs, mount is visible to all
      if (!mount.requiredAcls?.length) {
         return true;
      }
      // All required ACLs must be present
      return mount.requiredAcls.every(acl => userAclSet.has(acl));
   });
}

/**
 * Check if the current user has a specific permission for a plugin.
 *
 * @param pluginId - The plugin identifier (e.g., 'buildhealth')
 * @param aclAction - The ACL action name (e.g., 'BuildHealthViewAI')
 * @returns true if user has permission, false otherwise
 *
 * Semantics:
 * - Plugin not found/not enabled → false
 * - Plugin doesn't implement dashboard ACLs (userAcls undefined) → false
 * - User has the specific ACL → true
 * - Otherwise → false
 */
export function hasPluginPermission(pluginId: string, aclAction: string): boolean {
   const plugin = plugins.find(p => p.id === pluginId);
   if (!plugin?.enabled) {
      return false;
   }
   // undefined userAcls means plugin doesn't implement dashboard ACLs - fail closed
   if (plugin.userAcls === undefined) {
      return false;
   }
   return plugin.userAcls.includes(aclAction);
}
