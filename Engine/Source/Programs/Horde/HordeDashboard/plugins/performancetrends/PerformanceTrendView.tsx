// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, Dialog, DialogType, DirectionalHint, Dropdown, IDropdownOption, Label, PrimaryButton, Spinner, SpinnerSize, Stack, TooltipHost } from "@fluentui/react"
import { observer } from "mobx-react-lite";
import React, { useCallback, useEffect, useLayoutEffect, useMemo, useRef, useState } from "react";
import { getHordeStyling } from "horde/styles/Styles";
import { TopNav } from "horde/components/TopNav";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { useWindowSize } from "horde/base/utilities/hooks";
import { Text } from '@fluentui/react';
import { TestProjectDropdownSingle } from "./components/TestProjectDropdownComponent";
import { runInAction } from "mobx";
import { CommitDropdownSingle } from "./components/CommitDropdownComponent";
import { MetricSelectionHandler } from "./components/LineGraphComponent";
import { MetricSummaryTypeDropdownSingle } from "./components/MetricSummaryTypeDropdownSingle";
import { PerformanceTrendUrlSync } from "./components/PerformanceTrendUrl";
import { PlatformDropdownMulti } from "./components/PlatformDropdownComponent";
import { StreamDropdownMulti } from "./components/StreamDropdownComponent";
import { TestIdentitiesTypeDropdownSingle } from "./components/TestIdentitiesTypeDropdownSingle";
import { TestTypeDropdownMulti } from "./components/TestTypeDropdownComponent";
import { ViewablePropertiesDropdown } from "./components/ViewablePropertiesDropdownComponent";
import { PerformanceTrendOptionsController } from "./filters/PerformanceTrendOptionsController";
import { PerformanceTrendOptionsDataHandler } from "./filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionsState } from "./filters/PerformanceTrendOptionsState";
import { GroupByOption, PerformanceTrendUIOptionsState } from "./filters/PerformanceTrendUIOptionsState";
import { KeyStatsData } from "./metrictypes/KeyStatsData";
import { MetricConstraint, NO_RESOLVE_TYPE_FOUND, MetricTypeRegistry, PerformanceTrendContext } from "./metrictypes/PerformanceTrendsTypes";
import { PerformanceTrendDataHandler, PerformanceTrendFilter } from "./PerformanceTrendsDataHandler";
import { decodeMetricTypeKey, decodeStreamKey, decodePlatformKey } from "./responses/FilterKeys";
import { IMetricViewGenerator, IMetricDetailedView, IMetricView } from "./viewgenerators/PerformanceTrendRenderTypes";
import { BudgetModal } from "./components/BudgetModal";
import { getBudgets, PerformanceBudgetResponse } from "./api";
import { keyStatsReferenceAdapter } from "./adapters/KeyStatsReferenceAdapter";
import { MetricReferencePoint } from "./viewgenerators/PerformanceTrendRenderTypes";

// #region -- Visual Components --

const DEFAULT_RENDER_COMPONENT_WIDTH = 1200;
const DEFAULT_RENDER_COMPONENT_HEIGHT = 500;

/**
 * Shared TooltipHost configuration for sidebar controls. Every sidebar tooltip anchors the
 * callout to the right of its target so the hint never covers other controls in the column.
 * Spread into a TooltipHost via `{...sidebarTooltipProps}`.
 */
const sidebarTooltipProps = {
    calloutProps: { directionalHint: DirectionalHint.rightCenter }
} as const;

/**
 * Shared inline style for the inner `<div>` used as TooltipHost content. Clamps the tooltip's
 * rendered width so longer descriptions wrap to a readable column rather than stretching to
 * fit the longest line.
 */
const sidebarTooltipContentStyle: React.CSSProperties = { maxWidth: 300 };

function selectDataProviderAndViewGenerator(performanceTrendOptions: PerformanceTrendOptionsController): { dataProvider: PerformanceTrendDataHandler<MetricConstraint> | undefined, viewGenerator: IMetricViewGenerator<MetricConstraint> | undefined } {
    const metricType = performanceTrendOptions.state.publishedEnabledMetricSummaryTypes.length === 0 ? NO_RESOLVE_TYPE_FOUND : decodeMetricTypeKey(performanceTrendOptions.state.publishedEnabledMetricSummaryTypes[0])?.summaryType ?? NO_RESOLVE_TYPE_FOUND;

    return MetricTypeRegistry.get(metricType);
}

interface PerformanceTrendSidebarProps {
    performanceTrendOptions: PerformanceTrendOptionsController;
    onBudgetChange?: (budget: PerformanceBudgetResponse | undefined) => void;
}

const PerformanceTrendSidebar: React.FC<PerformanceTrendSidebarProps> = observer(function ConstructPerformanceTrendsSidebar({ performanceTrendOptions, onBudgetChange }) {
    const { hordeClasses, modeColors } = getHordeStyling();

    // #region -- UI Controls --

    const [curveGraphLine, setCurveGraphLine] = useState(() => performanceTrendOptions.uiState.curveGraphLine)
    const [groupByOption, setGroupByOption] = useState<GroupByOption>(() => performanceTrendOptions.uiState.groupByOption);
    const [showBudgetModal, setShowBudgetModal] = useState(false);
    const [budgetToEdit, setBudgetToEdit] = useState<PerformanceBudgetResponse | undefined>(undefined);
    const [availableBudgets, setAvailableBudgets] = useState<PerformanceBudgetResponse[]>([]);
    const [selectedBudgetId, setSelectedBudgetId] = useState<string | undefined>(undefined);
    const [loadingBudgets, setLoadingBudgets] = useState(false);
    const { dataProvider, viewGenerator } = selectDataProviderAndViewGenerator(performanceTrendOptions);

    // Get the selected stream, test project, and platforms for fetching budgets
    const selectedStreamKeys = performanceTrendOptions.state.publishedEnabledStreamKeys;
    const selectedTestProjects = performanceTrendOptions.state.publishedEnabledProjectKeys;
    const selectedPlatformKeys = performanceTrendOptions.state.publishedEnabledPlatforms;

    const firstComputedStream = useMemo(() => {
        if (selectedStreamKeys.length === 0) return undefined;
        const decoded = decodeStreamKey(selectedStreamKeys[0]);
        return decoded?.stream;
    }, [selectedStreamKeys]);
    const firstTestProject = selectedTestProjects[0];

    // Decode selected platforms for client-side filtering
    const selectedPlatforms = useMemo(() => {
        const platforms: string[] = [];
        for (const key of selectedPlatformKeys) {
            const decoded = decodePlatformKey(key);
            if (decoded) {
                platforms.push(decoded.platform);
            } else {
                platforms.push(key);
            }
        }
        return platforms;
    }, [selectedPlatformKeys]);

    // Fetch budgets when stream or test project changes
    const fetchBudgets = useCallback(async () => {
        if (!firstComputedStream) {
            setAvailableBudgets([]);
            setSelectedBudgetId(undefined);
            return;
        }

        setLoadingBudgets(true);
        try {
            // Fetch all budgets for stream/testProject (no platform filter)
            // We'll filter client-side based on selected platforms
            const budgets = await getBudgets(firstComputedStream, firstTestProject);
            setAvailableBudgets(budgets);
            // Clear selection if current budget is no longer available
            if (selectedBudgetId && !budgets.some(b => b.id === selectedBudgetId)) {
                setSelectedBudgetId(undefined);
            }
        } catch (error) {
            console.error('Failed to fetch budgets:', error);
            setAvailableBudgets([]);
        } finally {
            setLoadingBudgets(false);
        }
    }, [firstComputedStream, firstTestProject, selectedBudgetId]);

    useEffect(() => {
        fetchBudgets();
    }, [firstComputedStream, firstTestProject]);

    // Filter budgets based on selected platforms and build dropdown options
    const budgetOptions: IDropdownOption[] = useMemo(() => {
        const options: IDropdownOption[] = [{ key: '', text: '(No Budget Selected)' }];

        for (const budget of availableBudgets) {
            // Budget applies if:
            // 1. No platforms specified (applies to all), OR
            // 2. At least one selected platform matches the budget's platforms
            const appliesToAllPlatforms = !budget.platforms || budget.platforms.length === 0;
            const matchesSelectedPlatforms = selectedPlatforms.length === 0 ||
                appliesToAllPlatforms ||
                selectedPlatforms.some(p => budget.platforms?.includes(p));

            if (matchesSelectedPlatforms) {
                options.push({
                    key: budget.id,
                    text: budget.name,
                    title: budget.description || undefined
                });
            }
        }
        return options;
    }, [availableBudgets, selectedPlatforms]);

    // Clear selected budget if it no longer matches the current platform selection
    useEffect(() => {
        if (selectedBudgetId && !budgetOptions.some(opt => opt.key === selectedBudgetId)) {
            setSelectedBudgetId(undefined);
        }
    }, [budgetOptions, selectedBudgetId]);

    // Notify parent when selected budget changes
    useEffect(() => {
        const selectedBudget = selectedBudgetId
            ? availableBudgets.find(b => b.id === selectedBudgetId)
            : undefined;
        onBudgetChange?.(selectedBudget);
    }, [selectedBudgetId, availableBudgets, onBudgetChange]);

    const groupByOptions: IDropdownOption<GroupByOption>[] = [
        { key: 'platform', text: 'Platform', data: 'platform' },
        { key: 'stream', text: 'Stream', data: 'stream' },
        { key: 'testType', text: 'Test Type', data: 'testType' },
        { key: 'platform+stream', text: 'Platform + Stream', data: 'platform+stream' },
        { key: 'platform+testType', text: 'Platform + Test Type', data: 'platform+testType' },
        { key: 'stream+testType', text: 'Stream + Test Type', data: 'stream+testType' },
        { key: 'platform+stream+testType', text: 'Platform + Stream + Test Type', data: 'platform+stream+testType' },
    ];

    const updateCurveGraphLine = (isChecked) => {
        setCurveGraphLine(isChecked);
        performanceTrendOptions.setCurveGraphLine(isChecked);
    }

    const updateGroupByOption = (option: GroupByOption) => {
        setGroupByOption(option);
        performanceTrendOptions.setGroupByOption(option);
    }

    useEffect(() => {
        setCurveGraphLine(performanceTrendOptions.uiState.curveGraphLine);
    }, [performanceTrendOptions.uiState.curveGraphLine]);

    useEffect(() => {
        setGroupByOption(performanceTrendOptions.uiState.groupByOption);
    }, [performanceTrendOptions.uiState.groupByOption]);

    // #endregion -- UI Controls --

    return (
        <Stack style={{ minWidth: 375, maxWidth: 500, paddingRight: 18 }}>
            <Stack className={hordeClasses.modal} tokens={{ childrenGap: 12 }}>
                <Stack
                    styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                        },
                    }}
                    tokens={{ childrenGap: 12 }}
                >
                    <Stack
                        tokens={{ childrenGap: 12 }}
                        styles={{
                            root: {
                                border: `1px solid ${modeColors.content}`,
                                borderRadius: 4,
                                padding: 8,
                            },
                        }}>
                        <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                            <Label>Performance Trend Filters</Label>
                        </Stack>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The project to view data on.
                            </div>
                        } {...sidebarTooltipProps}>
                            <TestProjectDropdownSingle handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The test identity to filter for. This is the test methodology, and is most appropriately thought of as a test method (e.g. fly through; multi-camera; sequence) & level combination. E.g. 'Manhattan_CentralPark-Fly-through'.
                            </div>
                        } {...sidebarTooltipProps}>
                            <TestIdentitiesTypeDropdownSingle handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The performance summary metric to filter on. This is the fundamental performance data used throughout, and is often a collection of common metrics & measures.
                            </div>
                        } {...sidebarTooltipProps}>
                            <MetricSummaryTypeDropdownSingle handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The type of test, which fundamentally sets the context. E.g. Perf, GPU Perf. This often has specific settings enabled, and as a result is best considered individually.
                            </div>
                        } {...sidebarTooltipProps}>
                            <TestTypeDropdownMulti handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The streams to include results from.
                            </div>
                        } {...sidebarTooltipProps}>
                            <StreamDropdownMulti handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The platforms to include results from.
                            </div>
                        } {...sidebarTooltipProps}>
                            <PlatformDropdownMulti handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                The changelist range to include results from.
                            </div>
                        } {...sidebarTooltipProps}>
                            <CommitDropdownSingle handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                Clears all filter selections (Project, Test Identity, Summary Type, Test Type, Streams, Platforms, Commit Range).
                            </div>
                        } {...sidebarTooltipProps}>
                            <PrimaryButton text="Clear" onClick={() => {
                                runInAction(() => {
                                    performanceTrendOptions.getTransactionSession().clearAll();
                                    optionsHandler.reset();
                                    performanceTrendOptions.commitTransactionSession();
                                });
                            }} />
                        </TooltipHost>
                    </Stack>
                    <Stack
                        styles={{
                            root: {
                                border: `1px solid ${modeColors.content}`,
                                borderRadius: 4,
                                padding: 8,
                            },
                        }}>
                        <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                            <Label>Performance Budgets</Label>
                        </Stack>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                <div>
                                    Select a saved performance budget to compare against. Budgets define threshold values for metrics.
                                </div>
                            </div>
                        } {...sidebarTooltipProps}>
                            <Dropdown
                                label="Active Budget"
                                selectedKey={selectedBudgetId ?? ''}
                                style={{ paddingBottom: "10px" }}
                                options={budgetOptions}
                                onChange={(_, option) => setSelectedBudgetId(option?.key as string || undefined)}
                                disabled={loadingBudgets || !firstComputedStream}
                                placeholder={loadingBudgets ? "Loading budgets..." : "(No Budget Selected)"}
                            />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                <div>
                                    {selectedBudgetId
                                        ? "Edit the currently selected performance budget."
                                        : "Create a new performance budget for the currently selected test types and metrics."}
                                </div>
                            </div>
                        } {...sidebarTooltipProps}>
                            <PrimaryButton
                                text={selectedBudgetId ? "Edit Budget" : "Create Budget"}
                                iconProps={{ iconName: selectedBudgetId ? 'Edit' : 'Add' }}
                                onClick={() => {
                                    if (selectedBudgetId) {
                                        const budget = availableBudgets.find(b => b.id === selectedBudgetId);
                                        if (budget) {
                                            setBudgetToEdit(budget);
                                            setShowBudgetModal(true);
                                        }
                                    } else {
                                        setBudgetToEdit(undefined);
                                        setShowBudgetModal(true);
                                    }
                                }}
                                disabled={!selectedBudgetId && (performanceTrendOptions.state.publishedEnabledTestTypeKeys.length === 0 || performanceTrendOptions.state.publishedEnabledStreamKeys.length === 0)}
                            />
                        </TooltipHost>
                    </Stack>
                    <Stack
                        styles={{
                            root: {
                                border: `1px solid ${modeColors.content}`,
                                borderRadius: 4,
                                padding: 8,
                            },
                        }}>
                        <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                            <Label>Options</Label>
                        </Stack>
                        <ViewablePropertiesDropdown viewGenerator={viewGenerator} handler={optionsHandler} performanceTrendOptions={performanceTrendOptions} />
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                <div>
                                    Draw line graphs with curve-interpolated lines.
                                </div>
                            </div>
                        } {...sidebarTooltipProps}>
                            <Checkbox
                                label="Curved Lines"
                                checked={curveGraphLine}
                                onChange={(_, isChecked) => updateCurveGraphLine(!!isChecked)}
                            />
                        </TooltipHost>
                        <TooltipHost content={
                            <div style={sidebarTooltipContentStyle}>
                                <div>
                                    Group graph series by the selected dimensions.
                                </div>
                            </div>
                        } {...sidebarTooltipProps}>
                            <Dropdown
                                label="Group By"
                                selectedKey={groupByOption}
                                options={groupByOptions}
                                onChange={(_, option) => {
                                    if (option?.data) {
                                        updateGroupByOption(option.data);
                                    }
                                }}
                            />
                        </TooltipHost>
                    </Stack>
                </Stack>
            </Stack>
            {showBudgetModal && (
                <BudgetModal
                    performanceTrendOptions={performanceTrendOptions}
                    viewGenerator={viewGenerator}
                    budgetToEdit={budgetToEdit}
                    onDismiss={() => {
                        setShowBudgetModal(false);
                        setBudgetToEdit(undefined);
                        // Refresh budgets list after modal closes
                        fetchBudgets();
                    }}
                />
            )}
        </Stack>
    )
});

interface AnalyticsGraphClusterProps {
    dataProvider: PerformanceTrendDataHandler<MetricConstraint>;
    viewGenerator: IMetricViewGenerator<MetricConstraint>;
    performanceTrendOptions: PerformanceTrendOptionsController;
    selectionHandler?: MetricSelectionHandler<PerformanceTrendContext>;
    referencePoints?: MetricReferencePoint<MetricConstraint>[];
}

type MetricDetailModalProps = {
    context: PerformanceTrendContext[];
    performanceTrendOptions: PerformanceTrendOptionsController;
    viewGenerator: IMetricViewGenerator<MetricConstraint>;
    onDismiss: () => void;
    isFilteredContext: boolean;
    referencePoints?: MetricReferencePoint<MetricConstraint>[];
};

const MetricDetailModal: React.FC<MetricDetailModalProps> = ({ context, performanceTrendOptions, viewGenerator, onDismiss, isFilteredContext, referencePoints }) => {
    const view: IMetricDetailedView = useMemo(() => viewGenerator.getSelectedDetailView(context, isFilteredContext, performanceTrendOptions.uiState, referencePoints), [context, viewGenerator, performanceTrendOptions.uiOptionsChangeVersion, isFilteredContext, referencePoints]);
    const Render = view.Render;

    return (
        <Dialog
            modalProps={{
                isBlocking: false,
                topOffsetFixed: true,
                styles: {
                    root: {
                        selectors: {
                            ".ms-Dialog-title": {
                                paddingTop: '24px',
                                paddingLeft: '32px'
                            }
                        }
                    }
                }
            }}
            onDismiss={onDismiss}
            hidden={false}
            minWidth={1600}
            dialogContentProps={{
                type: DialogType.close,
                onDismiss: onDismiss,
                title: "Metric Detail",
            }}>
            <Stack tokens={{ childrenGap: 8 }}>
                <Render
                    width={DEFAULT_RENDER_COMPONENT_WIDTH}
                    height={DEFAULT_RENDER_COMPONENT_HEIGHT}
                />
            </Stack>
        </Dialog >
    );
};

const AnalyticsGraphCluster: React.FC<AnalyticsGraphClusterProps> = observer(({ dataProvider, viewGenerator, performanceTrendOptions, selectionHandler, referencePoints }) => {
    const containerRef = useRef<HTMLDivElement>(null);
    const [containerWidth, setContainerWidth] = useState(0);

    useLayoutEffect(() => {
        if (!containerRef.current) return;

        const resizeObserver = new ResizeObserver((entries) => {
            for (const entry of entries) {
                const newWidth = entry.contentRect.width;
                // Only update if width changed by more than 1px to prevent flickering
                // from sub-pixel layout shifts during tooltip interactions
                setContainerWidth(prev => {
                    return Math.abs(prev - newWidth) > 1 ? newWidth : prev
                }
                );
            }
        });

        resizeObserver.observe(containerRef.current);
        return () => resizeObserver.disconnect();
    }, []);

    // Note: this is by design where we are monitoring data, the ui options change version.
    // The rationale is as follows:
    //  - This is a primary extension point for licensees to add new view generators
    //  - We never know what downstream view generators will have as a UI requirement (this struct will evolve, and implement new options)
    //  - We do not know if it will be observed properly (derived components in getViews() impl may not be observed, as an example)
    //  - Any closures within the getViews delegate (where observed views are modulated) outside of the observable section will un-observe parameters
    //  - This is the safest way to guarantee that we perform a redraw
    const views: IMetricView[] = useMemo(() => viewGenerator.getViews(dataProvider.data, performanceTrendOptions.uiState, selectionHandler, referencePoints), [viewGenerator, dataProvider.data, performanceTrendOptions.uiOptionsChangeVersion, referencePoints]);

    const columnWidth = containerWidth ? Math.floor((containerWidth - 12) / 2) : 300;

    return (
        <div ref={containerRef} style={{ width: "100%" }}>
            <Stack horizontal wrap tokens={{ childrenGap: 12 }}>
                {views.map((view, index) => (
                    <Stack key={`${KeyStatsData.name}${index}`} styles={{ root: { width: columnWidth, minWidth: 600 } }}>
                        <view.Render width={columnWidth} height={DEFAULT_RENDER_COMPONENT_HEIGHT} />
                    </Stack>
                ))}
            </Stack>
        </div>
    );
});

interface PerformanceTrendComponentProps {
    performanceTrendOptions: PerformanceTrendOptionsController;
    filter: PerformanceTrendFilter;
    selectedBudget?: PerformanceBudgetResponse;
}

const PerformanceTrendComponent: React.FC<PerformanceTrendComponentProps> = observer(({ performanceTrendOptions, filter, selectedBudget }) => {
    const { hordeClasses, modeColors } = getHordeStyling();
    const [selectedMetric, setSelectedMetric] = useState<PerformanceTrendContext[] | null>(null);
    const [filteredContext, setFilteredContext] = useState<boolean>(false);
    const { dataProvider, viewGenerator } = selectDataProviderAndViewGenerator(performanceTrendOptions);

    const handleMetricSelection: MetricSelectionHandler<PerformanceTrendContext> = (context, isFiltered) => {
        setSelectedMetric(context);
        setFilteredContext(isFiltered);
    };

    // Convert selected budget to one-or-more reference points. The adapter splits a budget
    // carrying thresholds for multiple testTypes into multiple reference points — each scoped
    // to its testType via its own `appliesTo` predicate — so downstream cell coloring and
    // graph reference lines render correctly per row.
    const referencePoints = useMemo(() => {
        if (!selectedBudget) return [];
        return keyStatsReferenceAdapter.fromBudget(selectedBudget);
    }, [selectedBudget]);

    useEffect(() => {
        if (dataProvider) {
            dataProvider.setFilter(filter);
        }
    }, [filter, dataProvider]);

    if (!viewGenerator || !dataProvider) {
        return null;
    }

    if (dataProvider.querying) {
        return <Spinner style={{ paddingTop: 16 }} size={SpinnerSize.large} />;
    }

    return dataProvider.data.length === 0 ? (
        <Text
            variant="medium"
            styles={{ root: { paddingTop: 16, color: modeColors.textSecondary } }}
        >
            No data available for the selected filters.
        </Text>) : (
        <Stack styles={{ root: { width: "100%" } }} tokens={{ childrenGap: 24 }}>
            {selectedMetric && (<MetricDetailModal context={selectedMetric} viewGenerator={viewGenerator} performanceTrendOptions={performanceTrendOptions} onDismiss={() => setSelectedMetric(null)} isFilteredContext={filteredContext} referencePoints={referencePoints} />)}
            <AnalyticsGraphCluster dataProvider={dataProvider} viewGenerator={viewGenerator} performanceTrendOptions={performanceTrendOptionsController} selectionHandler={handleMetricSelection} referencePoints={referencePoints} />
            <Stack></Stack>
            {/*<CSVTable dataProvider={dataProvider} columnProvider={dataProvider} />*/}
        </Stack>
    );
});

/**
 * Horde Performance Trends view.
 * @returns React component.
 */
export const PerformanceTrendsView: React.FC = observer(() => {
    const { hordeClasses, modeColors } = getHordeStyling();
    const windowSize = useWindowSize();
    const [selectedBudget, setSelectedBudget] = useState<PerformanceBudgetResponse | undefined>(undefined);

    const filter: PerformanceTrendFilter = useMemo(() => {
        return {
            testProjects: performanceTrendOptionsState.publishedEnabledProjectKeys,
            testIdentities: performanceTrendOptionsState.publishedEnabledTestIdentityKeys,
            testTypes: performanceTrendOptionsState.publishedEnabledTestTypeKeys,
            metricTypes: performanceTrendOptionsState.publishedEnabledMetricSummaryTypes,
            streams: performanceTrendOptionsState.publishedEnabledStreamKeys,
            platforms: performanceTrendOptionsState.publishedEnabledPlatforms,
            startCommit: performanceTrendOptionsState.publishedEnabledStartCommit,
            endCommit: performanceTrendOptionsState.publishedEnabledEndCommit
        };
    }, [
        performanceTrendOptionsState.publishedEnabledProjectKeys,
        performanceTrendOptionsState.publishedEnabledTestIdentityKeys,
        performanceTrendOptionsState.publishedEnabledTestTypeKeys,
        performanceTrendOptionsState.publishedEnabledMetricSummaryTypes,
        performanceTrendOptionsState.publishedEnabledStreamKeys,
        performanceTrendOptionsState.publishedEnabledPlatforms,
        performanceTrendOptionsState.publishedEnabledStartCommit,
        performanceTrendOptionsState.publishedEnabledEndCommit
    ]);

    const handleBudgetChange = useCallback((budget: PerformanceBudgetResponse | undefined) => {
        setSelectedBudget(budget);
    }, []);

    return (
        <Stack className={hordeClasses.horde}>
            <PerformanceTrendUrlSync buildHealthController={performanceTrendOptionsController} handler={optionsHandler} />
            <TopNav />
            <Breadcrumbs items={[{ text: 'Performance Trends' }]} />
            <Stack horizontal>
                <div key={`windowsize_performance_trends_view_${windowSize.width}_${windowSize.height}`} style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack horizontalAlign="center" grow styles={{ root: { width: "100%", padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack styles={{ root: { width: "100%" } }}>
                        <Stack horizontal styles={{ root: { minHeight: '85vh', maxHeight: '85vh' } }} >
                            <PerformanceTrendSidebar performanceTrendOptions={performanceTrendOptionsController} onBudgetChange={handleBudgetChange} />
                            <Stack id="parent-performance-trends" styles={{
                                root: {
                                    overflowX: "hidden",
                                    overflowY: "auto",
                                    minWidth: 0,
                                    position: "relative",
                                    border: `${modeColors.content} solid 2px`,
                                    display: "flex",
                                    flexDirection: "column",
                                    flex: 1,
                                }
                            }}>
                                <PerformanceTrendComponent performanceTrendOptions={performanceTrendOptionsController} filter={filter} selectedBudget={selectedBudget} />
                            </Stack>
                        </Stack>
                    </Stack>
                </Stack>
            </Stack>
        </Stack>
    )
});

// #endregion -- Visual Components --

// #region -- Module Scoped Variables --

const performanceTrendOptionsState = new PerformanceTrendOptionsState();
const performanceTrendUIOptionsState = new PerformanceTrendUIOptionsState();
const performanceTrendOptionsController = new PerformanceTrendOptionsController(performanceTrendOptionsState, performanceTrendUIOptionsState);
const optionsHandler = new PerformanceTrendOptionsDataHandler(performanceTrendOptionsState, performanceTrendOptionsController);

// #endregion -- Module Scoped Variables --