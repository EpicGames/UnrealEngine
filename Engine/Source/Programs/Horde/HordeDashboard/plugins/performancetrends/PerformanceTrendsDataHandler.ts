// Copyright Epic Games, Inc. All Rights Reserved.

import { action, computed, makeObservable, observable, runInAction } from "mobx";
import { ISummaryMetricProvider, MetricConstraint } from "./metrictypes/PerformanceTrendsTypes";
import { DecodedTestIdentityKey, decodeTestIdentity, DecodedTestTypeKey, decodeTestTypeKey, decodeStreamKey, DecodedStreamKey, decodePlatformKey, DecodedPlatformKey } from "./responses/FilterKeys";
import { IDataProvider, IColumnProvider } from "hordePlugins/structuredanalytics/CSVComponent";
import { getMetrics } from "./api";

/**
 * The filter interface supported by the @see PerformanceTrendsDataHandler .
 */
export interface PerformanceTrendFilter {
    testProjects: string[];
    testIdentities: string[];
    testTypes: string[];
    metricTypes: string[];
    streams: string[];
    platforms: string[];
    startCommit: number | null;
    endCommit: number | null;
}

// #region -- Data Handler --

/**
 * General purpose data handler for metric retrieval, based on types.
 */
export class PerformanceTrendDataHandler<TMetric extends MetricConstraint> implements IDataProvider, IColumnProvider {

    // #region -- Private Members --

    private ctor: ISummaryMetricProvider<TMetric>
    private activeFilter: PerformanceTrendFilter | null = null;
    private performanceReportTelemetryData: TMetric[] = [];

    @observable
    private activeRequests = 0;

    // #endregion -- Private Members --

    // #region -- Public Members --

    // IColumnProvider
    @observable
    columns: string[] = [];

    // IDataProvider
    @observable
    data: TMetric[] = [];

    // Query state
    @computed
    get querying(): boolean {
        return this.activeRequests > 0;
    }
    // #endregion -- Public Members --

    // #region -- Constructor --

    constructor(ctor: ISummaryMetricProvider<TMetric>) {
        makeObservable(this);
        this.ctor = ctor;
        this.columns = [];
        this.data = [];
    }

    // #endregion -- Constructor --

    // #region -- Public Api --

    /**
     * Queries the underlying database to populate the handler with performance trend data.
     */
    async query() {
        if (!this.activeFilter || !this.activeFilter.testIdentities || !this.activeFilter.testProjects || this.activeFilter.testIdentities.length === 0 || this.activeFilter.testTypes.length === 0 || this.activeFilter.testProjects.length === 0 || this.activeFilter.streams.length === 0 || this.activeFilter.platforms.length === 0) {
            this.data = [];

            return;
        }

        let targetTestProject: string;

        // We will just support one test project for now.
        if (this.activeFilter.testProjects.length > 0) {
            targetTestProject = this.activeFilter.testProjects[0];
        } else {
            console.warn("[PerformanceTrendsDataHandler] Received no test project.");
            return;
        }

        let targetTestIdentity: string;
        let targetTestTypes: string[] = [];
        let targetStreams: string[] = [];
        let targetPlatforms: string[] = [];

        // We will just support one test identity for now.
        if (this.activeFilter.testIdentities.length > 0) {
            let decodedTestIdentity: DecodedTestIdentityKey | null = decodeTestIdentity(this.activeFilter.testIdentities[0]);
            if (decodedTestIdentity === null) {
                console.warn(`[PerformanceTrendDataHandler] Handler received an invalid test identity option: ${this.activeFilter.testIdentities[0]}`);
                return;
            }

            targetTestIdentity = decodedTestIdentity.testIdentity;
        } else {
            console.warn("[PerformanceTrendsDataHandler] Received no test identity.");
            return;
        }

        // We will just support one test type for now.

        if (this.activeFilter.testTypes.length > 0) {
            targetTestTypes = this.activeFilter.testTypes
                .map(x => decodeTestTypeKey(x))
                .filter((x: DecodedTestTypeKey | null): x is DecodedTestTypeKey => x !== null)
                .map(x => x.testType);
        }

        if (this.activeFilter.streams.length > 0) {
            targetStreams = this.activeFilter.streams
                .map(x => decodeStreamKey(x))
                .filter((x: DecodedStreamKey | null): x is DecodedStreamKey => x !== null)
                .map(x => x.stream);
        }

        if (this.activeFilter.platforms.length > 0) {
            targetPlatforms = this.activeFilter.platforms
                .map(x => decodePlatformKey(x))
                .filter((x: DecodedPlatformKey | null): x is DecodedPlatformKey => x !== null)
                .map(x => x.platform);
        }

        runInAction(() => {
            this.activeRequests++;
        });

        try {
            this.performanceReportTelemetryData = await getMetrics<TMetric>(this.ctor, targetTestProject, targetTestIdentity, targetTestTypes, targetStreams, targetPlatforms, this.activeFilter.startCommit, this.activeFilter.endCommit);
        } catch (errorReason) {
            console.log(errorReason);

            runInAction(() => {
                this.activeRequests--;
            });

            return;
        }

        runInAction(() => {
            this.activeRequests--;
        });

        this.data = this.performanceReportTelemetryData;
        this.columns =
            this.performanceReportTelemetryData.length >= 1
                ? Object.keys(this.performanceReportTelemetryData[0])
                : [];
    }

    /**
     * Sets a new filter for the handler, which will control the underlying dataset.
     * @param filter The new filter to use.
     */
    @action
    setFilter(filter: PerformanceTrendFilter | null) {
        this.activeFilter = filter;
        this.query();
    }

    // #endregion -- Public Api --
}

// #endregion