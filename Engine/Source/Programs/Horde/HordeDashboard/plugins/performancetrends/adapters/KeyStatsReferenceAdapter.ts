// Copyright Epic Games, Inc. All Rights Reserved.

import { MetricThresholdResponse, PerformanceBudgetResponse } from "../api";
import { KeyStatsData } from "../metrictypes/KeyStatsData";
import { PerformanceTrendContext } from "../metrictypes/PerformanceTrendsTypes";
import { MetricReferencePoint, ReferenceThresholds } from "../viewgenerators/PerformanceTrendRenderTypes";

/**
 * Adapter interface for converting external data sources to strongly-typed reference points.
 *
 * Each adapter call returns an array: a single budget that carries thresholds for multiple
 * testTypes is split into multiple reference points — one per testType — so each can be
 * scoped via its own @see MetricReferencePoint.appliesTo" predicate and so the
 * one-threshold-per-property invariant on@see ReferenceThresholds" is preserved.
 */
export interface IReferencePointAdapter<T extends PerformanceTrendContext> {
    /**
     * Converts a budget response into one or more strongly-typed reference points.
     * @param budget The budget response from the API.
     * @returns An array of reference points, each scoped to a single testType.
     */
    fromBudget(budget: PerformanceBudgetResponse): MetricReferencePoint<T>[];
}

/**
 * Adapter for converting PerformanceBudgetResponse to MetricReferencePoint<KeyStatsData>.
 *
 * Each PerformanceBudgetResponse may declare thresholds across multiple testTypes
 * (e.g. frametime=16ms for FlyThrough, frametime=33ms for Sequence). Because
 * @see ReferenceThresholds" allows only one threshold per property,
 * we emit one MetricReferencePoint per distinct testType, with an
 * @see MetricReferencePoint.appliesTo" predicate that scopes the reference
 * to rows matching that testType (and the budget's platform allow-list, if any).
 */
export class KeyStatsReferenceAdapter implements IReferencePointAdapter<KeyStatsData> {
    /** @inheritdoc */
    fromBudget(budget: PerformanceBudgetResponse): MetricReferencePoint<KeyStatsData>[] {
        // A null/empty platform list means the budget applies to every platform — the predicate
        // treats both cases as "no platform filter" so the predicate stays uniform.
        const platformAllowList = budget.platforms && budget.platforms.length > 0 ? budget.platforms : null;

        // Group thresholds by testType so each reference point lands at one-threshold-per-property.
        const byTestType = new Map<string, MetricThresholdResponse[]>();
        for (const threshold of budget.thresholds) {
            const bucket = byTestType.get(threshold.testType);
            if (bucket) {
                bucket.push(threshold);
            } else {
                byTestType.set(threshold.testType, [threshold]);
            }
        }

        const results: MetricReferencePoint<KeyStatsData>[] = [];
        for (const [testType, thresholds] of byTestType) {
            const thresholdMap: ReferenceThresholds<KeyStatsData> = {};
            for (const threshold of thresholds) {
                const propKey = threshold.metricName as keyof ReferenceThresholds<KeyStatsData>;
                thresholdMap[propKey] = {
                    thresholdValue: threshold.thresholdValue,
                    largerIsWorse: threshold.largerIsWorse,
                    name: budget.name,
                };
            }

            results.push({
                id: `budget:${budget.id}:${testType}`,
                label: `Budget: ${budget.name} (${testType})`,
                thresholds: thresholdMap,
                // The predicate captures testType and platformAllowList by closure. Rows whose
                // gauntletSubTest doesn't match this slice are skipped entirely; platform filter
                // is applied only when the budget declared one.
                appliesTo: (row: KeyStatsData) => {
                    if (row.gauntletSubTest !== testType) return false;
                    if (platformAllowList !== null && !platformAllowList.includes(row.platform)) return false;
                    return true;
                },
                // Mirror appliesTo for the header renderer — undefined platforms means "all".
                scope: {
                    platforms: platformAllowList ?? undefined,
                    testType,
                },
            });
        }

        return results;
    }
}

/**
 * Singleton instance of the KeyStatsReferenceAdapter.
 */
export const keyStatsReferenceAdapter = new KeyStatsReferenceAdapter();
