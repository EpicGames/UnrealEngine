// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, DetailsList, DetailsListLayoutMode, Dialog, DialogFooter, DialogType, Dropdown, IColumn, IDropdownOption, Label, MessageBar, MessageBarType, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useCallback, useMemo, useState } from "react";
import { getHordeStyling } from "horde/styles/Styles";
import { createBudget, updateBudget, deleteBudget, PerformanceBudgetAddRequest, PerformanceBudgetUpdateRequest, MetricThresholdRequest, PerformanceBudgetResponse } from "../api";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { IMetricViewGenerator } from "../viewgenerators/PerformanceTrendRenderTypes";
import { MetricConstraint } from "../metrictypes/PerformanceTrendsTypes";
import { decodeTestTypeKey, decodeStreamKey, decodePlatformKey, encodeViewablePropertyKey } from "../responses/FilterKeys";

const ALL_PLATFORMS_KEY = '__all_platforms__';

/**
 * Defines a budget.
 */
interface BudgetEntry {
    testType: string;
    metricName: string;
    metricLabel: string;
    thresholdValue: string;
    largerIsWorse: boolean;
    enabled: boolean;
}

/**
 * Defines the properties to interact with the budget modal.
 */
interface BudgetModalProps {
    performanceTrendOptions: PerformanceTrendOptionsController;
    viewGenerator: IMetricViewGenerator<MetricConstraint> | undefined;
    budgetToEdit?: PerformanceBudgetResponse;
    onDismiss: () => void;
}

/**
 * Produces a budget modal.
 * @returns React Component.
 */
export const BudgetModal: React.FC<BudgetModalProps> = observer(({ performanceTrendOptions, viewGenerator, budgetToEdit, onDismiss }) => {
    const { modeColors } = getHordeStyling();
    const isEditMode = !!budgetToEdit;

    // Get current filter selections and decode them
    const selectedStreamKeys = performanceTrendOptions.state.publishedEnabledStreamKeys;
    const selectedTestProjects = performanceTrendOptions.state.publishedEnabledProjectKeys;
    const selectedTestTypeKeys = performanceTrendOptions.state.publishedEnabledTestTypeKeys;
    const selectedPlatformKeys = performanceTrendOptions.state.publishedEnabledPlatforms;

    // Decode streams to get actual stream values
    const decodedStreams = useMemo(() => {
        const streams: { key: string, stream: string, testProject: string }[] = [];
        for (const key of selectedStreamKeys) {
            const decoded = decodeStreamKey(key);
            if (decoded) {
                streams.push({ key, stream: decoded.stream, testProject: decoded.testProject });
            }
        }
        return streams;
    }, [selectedStreamKeys]);

    // Decode test types to get actual test type values
    const decodedTestTypes = useMemo(() => {
        const testTypes: { key: string, testType: string, testProject: string }[] = [];
        for (const key of selectedTestTypeKeys) {
            const decoded = decodeTestTypeKey(key);
            if (decoded) {
                testTypes.push({ key, testType: decoded.testType, testProject: decoded.testProject });
            }
        }
        return testTypes;
    }, [selectedTestTypeKeys]);

    // Decode platforms to get actual platform values
    const decodedPlatforms = useMemo(() => {
        const platforms: { key: string, platform: string }[] = [];
        for (const key of selectedPlatformKeys) {
            const decoded = decodePlatformKey(key);
            if (decoded) {
                platforms.push({ key, platform: decoded.platform });
            }
        }
        return platforms;
    }, [selectedPlatformKeys]);

    // State - initialize from budgetToEdit if in edit mode
    const [budgetName, setBudgetName] = useState<string>(budgetToEdit?.name ?? '');
    const [budgetDescription, setBudgetDescription] = useState<string>(budgetToEdit?.description ?? '');
    const [selectedStreamKey, setSelectedStreamKey] = useState<string>(() => {
        if (budgetToEdit) {
            // Find the stream key that matches the budget's computedStream
            const match = decodedStreams.find(s => s.stream === budgetToEdit.computedStream);
            return match?.key ?? '';
        }
        return selectedStreamKeys[0] ?? '';
    });

    // State - test project & platform selection 
    const [selectedTestProject, setSelectedTestProject] = useState<string>(budgetToEdit?.testProject ?? selectedTestProjects[0] ?? '');
    const [selectedPlatformKeys2, setSelectedPlatformKeys] = useState<string[]>(() => {
        if (budgetToEdit) {
            // If budget has no platforms or empty platforms, it applies to all
            if (!budgetToEdit.platforms || budgetToEdit.platforms.length === 0) {
                return [ALL_PLATFORMS_KEY];
            }
            // Find platform keys that match the budget's platforms
            return budgetToEdit.platforms.map(p => {
                const match = decodedPlatforms.find(dp => dp.platform === p);
                return match?.key ?? p;
            });
        }
        return selectedPlatformKeys;
    });

    // Get the currently enabled viewable properties to pre-select metrics
    const enabledViewableProperties = performanceTrendOptions.uiState.enabledViewableProperties;

    const [budgetEntries, setBudgetEntries] = useState<BudgetEntry[]>(() => {
        const entries: BudgetEntry[] = [];
        if (viewGenerator) {
            const properties = viewGenerator.getViewableProperties();

            const testTypesToUse = isEditMode
                ? [...new Set([
                    ...budgetToEdit.thresholds.map(t => t.testType),
                    ...decodedTestTypes.map(t => t.testType),
                ])]
                : decodedTestTypes.map(t => t.testType);

            for (const testType of testTypesToUse) {
                for (const [fieldName, meta] of properties) {
                    // Check if this metric has a threshold in the budget being edited
                    const existingThreshold = budgetToEdit?.thresholds.find(
                        t => t.testType === testType && t.metricName === fieldName
                    );

                    // Check if this metric is currently visible in the view (for new budgets)
                    const viewableKey = encodeViewablePropertyKey(testType, fieldName);
                    const isCurrentlyVisible = viewableKey in enabledViewableProperties;

                    entries.push({
                        testType: testType,
                        metricName: fieldName,
                        metricLabel: meta.label + (meta.unit ? ` (${meta.unit})` : ''),
                        thresholdValue: existingThreshold ? String(existingThreshold.thresholdValue) : '',
                        largerIsWorse: existingThreshold?.largerIsWorse ?? true,
                        enabled: isEditMode ? !!existingThreshold : isCurrentlyVisible
                    });
                }
            }
        }
        return entries;
    });
    const [isSubmitting, setIsSubmitting] = useState(false);
    const [isDeleting, setIsDeleting] = useState(false);
    const [showDeleteConfirm, setShowDeleteConfirm] = useState(false);
    const [error, setError] = useState<string | null>(null);
    const [successMessage, setSuccessMessage] = useState<string | null>(null);

    // Get the actual stream value from the selected key
    const selectedStream = useMemo(() => {
        if (budgetToEdit) return budgetToEdit.computedStream;
        const found = decodedStreams.find(s => s.key === selectedStreamKey);
        return found?.stream ?? '';
    }, [decodedStreams, selectedStreamKey, budgetToEdit]);

    // Get the actual platform values from the selected keys
    // Returns empty array if "All Platforms" is selected (which means applies to all)
    const selectedPlatforms = useMemo(() => {
        if (selectedPlatformKeys2.includes(ALL_PLATFORMS_KEY)) {
            return [];
        }
        return selectedPlatformKeys2.map(key => {
            const found = decodedPlatforms.find(p => p.key === key);
            return found?.platform ?? key;
        }).filter(p => p !== '');
    }, [decodedPlatforms, selectedPlatformKeys2]);

    // Build dropdown options with decoded display values
    const streamOptions: IDropdownOption[] = useMemo(() =>
        decodedStreams.map(s => ({ key: s.key, text: s.stream })),
        [decodedStreams]
    );

    const testProjectOptions: IDropdownOption[] = useMemo(() =>
        selectedTestProjects.map(p => ({ key: p, text: p })),
        [selectedTestProjects]
    );

    const platformOptions: IDropdownOption[] = useMemo(() => {
        return [
            { key: ALL_PLATFORMS_KEY, text: '(All Platforms)' },
            ...decodedPlatforms.map(p => ({ key: p.key, text: p.platform }))
        ];
    }, [decodedPlatforms]);

    // Check if "All Platforms" is selected
    const isAllPlatformsSelected = selectedPlatformKeys2.includes(ALL_PLATFORMS_KEY);

    // Group entries by test type for display
    const entriesByTestType = useMemo(() => {
        const grouped = new Map<string, BudgetEntry[]>();
        for (const entry of budgetEntries) {
            if (!grouped.has(entry.testType)) {
                grouped.set(entry.testType, []);
            }
            grouped.get(entry.testType)!.push(entry);
        }
        return grouped;
    }, [budgetEntries]);

    const updateEntry = useCallback((testType: string, metricName: string, updates: Partial<BudgetEntry>) => {
        setBudgetEntries(prev => prev.map(entry =>
            entry.testType === testType && entry.metricName === metricName
                ? { ...entry, ...updates }
                : entry
        ));
    }, []);

    const enabledEntries = useMemo(() => budgetEntries.filter(e => e.enabled && e.thresholdValue !== ''), [budgetEntries]);

    // #region -- Callbacks --

    const handleSubmit = useCallback(async () => {
        if (!budgetName.trim()) {
            setError('Please enter a budget name.');
            return;
        }

        if (!isEditMode && (!selectedStream || !selectedTestProject)) {
            setError('Please select a stream and test project.');
            return;
        }

        const toSave = enabledEntries;
        if (toSave.length === 0) {
            setError('Please enable at least one metric and set a threshold value.');
            return;
        }

        // Validate all threshold values are numbers
        for (const entry of toSave) {
            const value = parseFloat(entry.thresholdValue);
            if (isNaN(value)) {
                setError(`Invalid threshold value for ${entry.metricLabel}: "${entry.thresholdValue}"`);
                return;
            }
        }

        setIsSubmitting(true);
        setError(null);

        try {
            // Build the thresholds array
            const thresholds: MetricThresholdRequest[] = toSave.map(entry => ({
                testType: entry.testType,
                metricName: entry.metricName,
                thresholdValue: parseFloat(entry.thresholdValue),
                largerIsWorse: entry.largerIsWorse
            }));

            if (isEditMode) {
                // Update existing budget
                const request: PerformanceBudgetUpdateRequest = {
                    name: budgetName.trim(),
                    description: budgetDescription.trim() || undefined,
                    platforms: selectedPlatforms.length > 0 ? selectedPlatforms : [],
                    thresholds: thresholds
                };

                await updateBudget(budgetToEdit.id, request);
                setSuccessMessage(`Successfully updated budget "${budgetName}"!`);
            } else {
                // Create new budget
                const request: PerformanceBudgetAddRequest = {
                    name: budgetName.trim(),
                    description: budgetDescription.trim() || undefined,
                    computedStream: selectedStream,
                    testProject: selectedTestProject,
                    platforms: selectedPlatforms.length > 0 ? selectedPlatforms : undefined,
                    thresholds: thresholds
                };

                await createBudget(request);
                setSuccessMessage(`Successfully created budget "${budgetName}" with ${thresholds.length} threshold${thresholds.length > 1 ? 's' : ''}!`);
            }

            // Close modal after short delay to show success
            setTimeout(() => onDismiss(), 1500);
        } catch (err: any) {
            setError(`Failed to ${isEditMode ? 'update' : 'create'} budget: ${err.message || err}`);
        } finally {
            setIsSubmitting(false);
        }
    }, [budgetName, budgetDescription, selectedStream, selectedTestProject, selectedPlatforms, enabledEntries, onDismiss, isEditMode, budgetToEdit]);

    const handleSaveAs = useCallback(async () => {
        if (!budgetName.trim()) {
            setError('Please enter a budget name.');
            return;
        }

        if (!selectedStream || !selectedTestProject) {
            setError('Please select a stream and test project.');
            return;
        }

        const toSave = enabledEntries;
        if (toSave.length === 0) {
            setError('Please enable at least one metric and set a threshold value.');
            return;
        }

        // Validate all threshold values are numbers
        for (const entry of toSave) {
            const value = parseFloat(entry.thresholdValue);
            if (isNaN(value)) {
                setError(`Invalid threshold value for ${entry.metricLabel}: "${entry.thresholdValue}"`);
                return;
            }
        }

        setIsSubmitting(true);
        setError(null);

        try {
            // Build the thresholds array
            const thresholds: MetricThresholdRequest[] = toSave.map(entry => ({
                testType: entry.testType,
                metricName: entry.metricName,
                thresholdValue: parseFloat(entry.thresholdValue),
                largerIsWorse: entry.largerIsWorse
            }));

            // Create new budget as a copy
            const request: PerformanceBudgetAddRequest = {
                name: budgetName.trim(),
                description: budgetDescription.trim() || undefined,
                computedStream: selectedStream,
                testProject: selectedTestProject,
                platforms: selectedPlatforms.length > 0 ? selectedPlatforms : undefined,
                thresholds: thresholds
            };

            await createBudget(request);
            setSuccessMessage(`Successfully created budget copy "${budgetName}"!`);

            // Close modal after short delay to show success
            setTimeout(() => onDismiss(), 1500);
        } catch (err: any) {
            setError(`Failed to create budget copy: ${err.message || err}`);
        } finally {
            setIsSubmitting(false);
        }
    }, [budgetName, budgetDescription, selectedStream, selectedTestProject, selectedPlatforms, enabledEntries, onDismiss]);

    const handleDelete = useCallback(async () => {
        if (!budgetToEdit) return;

        setIsDeleting(true);
        setError(null);

        try {
            await deleteBudget(budgetToEdit.id);
            setSuccessMessage(`Successfully deleted budget "${budgetToEdit.name}"!`);

            // Close modal after short delay to show success
            setTimeout(() => onDismiss(), 1500);
        } catch (err: any) {
            setError(`Failed to delete budget: ${err.message || err}`);
        } finally {
            setIsDeleting(false);
            setShowDeleteConfirm(false);
        }
    }, [budgetToEdit, onDismiss]);

    // #endregion -- Callbacks --

    // Validation checks
    const canSubmit = budgetName.trim() && (isEditMode || (selectedStream && selectedTestProject)) && enabledEntries.length > 0 && !isSubmitting && !isDeleting;
    const canSaveAs = budgetName.trim() && selectedStream && selectedTestProject && enabledEntries.length > 0 && !isSubmitting && !isDeleting;

    return (
        <Dialog
            hidden={false}
            onDismiss={onDismiss}
            minWidth={1000}
            maxWidth={1200}
            dialogContentProps={{
                type: DialogType.close,
                title: isEditMode ? "Edit Performance Budget" : "Create Performance Budget",
            }}
            modalProps={{
                isBlocking: isSubmitting || isDeleting
            }}
        >
            <Stack tokens={{ childrenGap: 16 }}>
                {/* Budget Name and Description */}
                <Stack tokens={{ childrenGap: 8 }}>
                    <Text variant="mediumPlus" styles={{ root: { fontWeight: 600 } }}>Budget Details</Text>
                    <TextField
                        label="Budget Name"
                        value={budgetName}
                        onChange={(_, val) => setBudgetName(val ?? '')}
                        placeholder="e.g., Beta Console Targets"
                        required
                        maxLength={200}
                    />
                    <TextField
                        label="Description"
                        value={budgetDescription}
                        onChange={(_, val) => setBudgetDescription(val ?? '')}
                        placeholder="Optional description of this budget group"
                        multiline
                        rows={2}
                        maxLength={1000}
                    />
                </Stack>

                {/* Context Selection */}
                <Stack tokens={{ childrenGap: 8 }}>
                    <Text variant="mediumPlus" styles={{ root: { fontWeight: 600 } }}>Budget Scope</Text>
                    <Stack horizontal tokens={{ childrenGap: 12 }} wrap>
                        <Dropdown
                            label="Stream"
                            selectedKey={selectedStreamKey}
                            options={isEditMode ? [{ key: selectedStreamKey, text: budgetToEdit?.computedStream ?? '' }] : streamOptions}
                            onChange={(_, opt) => setSelectedStreamKey(opt?.key as string ?? '')}
                            disabled={isEditMode || streamOptions.length <= 1}
                            styles={{ root: { minWidth: 200 } }}
                            required
                        />
                        <Dropdown
                            label="Test Project"
                            selectedKey={selectedTestProject}
                            options={isEditMode ? [{ key: selectedTestProject, text: selectedTestProject }] : testProjectOptions}
                            onChange={(_, opt) => setSelectedTestProject(opt?.key as string ?? '')}
                            disabled={isEditMode || testProjectOptions.length <= 1}
                            styles={{ root: { minWidth: 200 } }}
                            required
                        />
                        <Dropdown
                            label="Platforms"
                            selectedKeys={selectedPlatformKeys2}
                            options={platformOptions}
                            onChange={(_, opt) => {
                                if (opt) {
                                    const key = opt.key as string;
                                    if (opt.selected) {
                                        if (key === ALL_PLATFORMS_KEY) {
                                            // Selecting "All Platforms" clears specific platforms
                                            setSelectedPlatformKeys([ALL_PLATFORMS_KEY]);
                                        } else {
                                            // Selecting a specific platform removes "All Platforms"
                                            setSelectedPlatformKeys(prev => [
                                                ...prev.filter(k => k !== ALL_PLATFORMS_KEY),
                                                key
                                            ]);
                                        }
                                    } else {
                                        // Deselecting
                                        setSelectedPlatformKeys(prev => prev.filter(k => k !== key));
                                    }
                                }
                            }}
                            multiSelect
                            styles={{ root: { minWidth: 200 } }}
                            placeholder="Select platforms..."
                        />
                    </Stack>
                </Stack>

                {/* Metric Budgets */}
                <Stack tokens={{ childrenGap: 12 }}>
                    <Text variant="mediumPlus" styles={{ root: { fontWeight: 600 } }}>Metric Thresholds</Text>
                    <Text variant="small" styles={{ root: { color: modeColors.textSecondary } }}>
                        Enable metrics and set threshold values. Check "Larger is Worse" for metrics like frame time where exceeding the threshold is bad.
                    </Text>

                    <div style={{ maxHeight: 300, overflowY: 'auto' }}>
                        {Array.from(entriesByTestType.entries()).map(([testType, entries]) => {
                            const columns: IColumn[] = [
                                {
                                    key: 'enabled',
                                    name: 'Enable',
                                    minWidth: 50,
                                    maxWidth: 60,
                                    onRender: (item: BudgetEntry) => (
                                        <Checkbox
                                            checked={item.enabled}
                                            onChange={(_, checked) => updateEntry(item.testType, item.metricName, { enabled: !!checked })}
                                        />
                                    )
                                },
                                {
                                    key: 'metric',
                                    name: 'Metric',
                                    minWidth: 200,
                                    isRowHeader: true,
                                    onRender: (item: BudgetEntry) => <Text>{item.metricLabel}</Text>
                                },
                                {
                                    key: 'threshold',
                                    name: 'Threshold',
                                    minWidth: 100,
                                    maxWidth: 120,
                                    onRender: (item: BudgetEntry) => (
                                        <TextField
                                            placeholder="Value"
                                            value={item.thresholdValue}
                                            onChange={(_, val) => updateEntry(item.testType, item.metricName, { thresholdValue: val ?? '' })}
                                            disabled={!item.enabled}
                                            styles={{ root: { width: 100 } }}
                                            type="number"
                                        />
                                    )
                                },
                                {
                                    key: 'largerIsWorse',
                                    name: 'Larger is Worse',
                                    minWidth: 100,
                                    maxWidth: 120,
                                    onRender: (item: BudgetEntry) => (
                                        <Checkbox
                                            checked={item.largerIsWorse}
                                            onChange={(_, checked) => updateEntry(item.testType, item.metricName, { largerIsWorse: !!checked })}
                                            disabled={!item.enabled}
                                        />
                                    )
                                }
                            ];

                            return (
                                <Stack key={testType} tokens={{ childrenGap: 4 }} styles={{ root: { marginBottom: 16 } }}>
                                    <Label styles={{ root: { fontWeight: 600, borderBottom: `1px solid ${modeColors.content}`, paddingBottom: 4 } }}>
                                        {testType}
                                    </Label>
                                    <DetailsList
                                        items={entries}
                                        columns={columns}
                                        selectionMode={SelectionMode.none}
                                        layoutMode={DetailsListLayoutMode.justified}
                                        isHeaderVisible={true}
                                        compact={true}
                                    />
                                </Stack>
                            );
                        })}
                    </div>
                </Stack>

                {/* Messages */}
                {error && (
                    <MessageBar messageBarType={MessageBarType.error} onDismiss={() => setError(null)}>
                        {error}
                    </MessageBar>
                )}
                {successMessage && (
                    <MessageBar messageBarType={MessageBarType.success}>
                        {successMessage}
                    </MessageBar>
                )}
            </Stack>

            <DialogFooter>
                {showDeleteConfirm ? (
                    <Stack horizontal tokens={{ childrenGap: 8 }} horizontalAlign="end" verticalAlign="center">
                        <Text styles={{ root: { color: modeColors.textSecondary } }}>Are you sure you want to delete this budget?</Text>
                        {isDeleting && <Spinner size={SpinnerSize.small} />}
                        <DefaultButton onClick={() => setShowDeleteConfirm(false)} text="Cancel" disabled={isDeleting} />
                        <PrimaryButton
                            onClick={handleDelete}
                            text="Confirm Delete"
                            disabled={isDeleting}
                            styles={{ root: { backgroundColor: '#a80000' }, rootHovered: { backgroundColor: '#c20000' } }}
                        />
                    </Stack>
                ) : (
                    <Stack horizontal tokens={{ childrenGap: 8 }} horizontalAlign="space-between" styles={{ root: { width: '100%' } }}>
                        <Stack horizontal tokens={{ childrenGap: 8 }}>
                            {isEditMode && (
                                <DefaultButton
                                    onClick={() => setShowDeleteConfirm(true)}
                                    text="Delete"
                                    iconProps={{ iconName: 'Delete' }}
                                    disabled={isSubmitting || isDeleting}
                                />
                            )}
                        </Stack>
                        <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="center">
                            {(isSubmitting || isDeleting) && <Spinner size={SpinnerSize.small} />}
                            <DefaultButton onClick={onDismiss} text="Cancel" disabled={isSubmitting || isDeleting} />
                            {isEditMode && (
                                <DefaultButton
                                    onClick={handleSaveAs}
                                    text="Save As Copy"
                                    iconProps={{ iconName: 'Copy' }}
                                    disabled={!canSaveAs}
                                />
                            )}
                            <PrimaryButton
                                onClick={handleSubmit}
                                text={isEditMode ? "Save" : "Create"}
                                iconProps={{ iconName: isEditMode ? 'Save' : 'Add' }}
                                disabled={!canSubmit}
                            />
                        </Stack>
                    </Stack>
                )}
            </DialogFooter>
        </Dialog>
    );
});
