// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption, SpinnerSize, Spinner } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { EMPTY_REFRESH_REQUEST, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionList, PerformanceTrendOptionData, BasicListParameter } from "./DropdownComponent";

/**
 * Component used to model all known commits (for the currently selected test project, test identity, summary table, and platform), as a single select dropdown.
 * @return The react component.
 */
export const CommitDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    if (handler.isLoadingCommits) {
        return (
            <Stack>
                <Label>Commit Range</Label>
                <Spinner size={SpinnerSize.small} label="Loading commits for project..." />
            </Stack>
        );
    }

    const selectedStartCommit = performanceTrendOptions.state.enabledStartCommit;
    const selectedEndCommit = performanceTrendOptions.state.enabledEndCommit;

    // Build all commit items sorted numerically (flatten arrays from each stream response)
    const seenCommits = new Set<number>();
    const allCommitItems: { id: string; text: string; value: number }[] = [];

    for (const commitResponse of handler.commits) {
        for (const commitId of commitResponse.commitIds) {
            if (!seenCommits.has(commitId)) {
                seenCommits.add(commitId);
                allCommitItems.push({
                    id: commitId.toFixed(),
                    text: commitId.toFixed(),
                    value: commitId
                });
            }
        }
    }

    allCommitItems.sort((a, b) => a.value - b.value);

    // Filter start commit options: if end commit is selected, only show commits <= end commit
    const startCommitItems = selectedEndCommit
        ? allCommitItems.filter(item => item.value <= selectedEndCommit)
        : allCommitItems;

    // Filter end commit options: if start commit is selected, only show commits >= start commit
    const endCommitItems = selectedStartCommit
        ? allCommitItems.filter(item => item.value >= selectedStartCommit)
        : allCommitItems;

    const startDropDownParams: PerformanceTrendOptionList = {
        label: "",
        items: startCommitItems,
        tooltip: "Select the start commit for the range."
    };

    const endDropDownParams: PerformanceTrendOptionList = {
        label: "",
        items: endCommitItems,
        tooltip: "Select the end commit for the range."
    };

    const selectedStartKey = selectedStartCommit?.toFixed();
    const selectedEndKey = selectedEndCommit?.toFixed();

    return allCommitItems.length > 0 && (
        <Stack tokens={{ childrenGap: 8 }}>
            <Label>Commit Range</Label>
            <Stack horizontal tokens={{ childrenGap: 16 }}>
                <Stack >
                    <BasicListParameter
                        placeholder="Earliest"
                        handler={handler}
                        performanceTrendOptions={performanceTrendOptions}
                        params={startDropDownParams}
                        selectedKey={selectedStartKey}
                        request={EMPTY_REFRESH_REQUEST}
                        onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                            const key = option.key as string;
                            performanceTrendOptions.getTransactionSession().toggleStartCommitRange(Number.parseInt(key));
                        }}
                        selectorWidth={150}
                        calloutWidth={150}
                    />
                </Stack>

                <Stack >
                    <BasicListParameter
                        placeholder="Latest"
                        handler={handler}
                        performanceTrendOptions={performanceTrendOptions}
                        params={endDropDownParams}
                        selectedKey={selectedEndKey}
                        request={EMPTY_REFRESH_REQUEST}
                        onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                            const key = option.key as string;
                            performanceTrendOptions.getTransactionSession().toggleEndCommitRange(Number.parseInt(key));
                        }}
                        selectorWidth={150}
                        calloutWidth={150}
                    />
                </Stack>
            </Stack>
        </Stack>
    );
});