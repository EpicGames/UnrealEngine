// Copyright Epic Games, Inc. All Rights Reserved.

import { DirectionalHint, Dropdown, IDropdownOption } from "@fluentui/react";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { DataHandlerRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { KEY_SEPARATOR } from "../responses/FilterKeys";

// #region -- Performance Trend Dropdown Params --

export const PARAMETER_KEY_PREFIX: string = "parameter_key";

export type PerformanceTrendOptionData = {
    id: string,
    text: string,
    group?: string,
    tooltip?: string
}

export type PerformanceTrendOptionList = {
    items: PerformanceTrendOptionData[]
    label: string,
    tooltip: string
}

// #endregion -- Performance Trend Dropdown Params --

/**
 * General purpose basic list dropdown.
 * @param placeholder The placeholder string for the dropdown.
 * @param handler The options data handler.
 * @param performanceTrendOptions The options controller.
 * @param params The list of performance trend options.
 * @param disabled Whether the dropdown is disabled or not.
 * @param selectedKey The currently selected key.
 * @param request The data refresh request to use upon selection.
 * @param onChange The on change delegate to invoke upon dropdown selection change.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export const BasicListParameter: React.FC<{ placeholder: string, handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController, params: PerformanceTrendOptionList, disabled?: boolean, selectedKey?: string, request: DataHandlerRefreshRequest, onChange: (option: IDropdownOption<PerformanceTrendOptionData>) => void, selectorWidth?: number, calloutWidth?: number }> = function ConstructBasicListParamater({ placeholder, params: param, handler, performanceTrendOptions, selectedKey, request, onChange, selectorWidth = 200, calloutWidth = 300 }) {
    const key = `${PARAMETER_KEY_PREFIX}${KEY_SEPARATOR}${param.label}`;
    const doptions: IDropdownOption<PerformanceTrendOptionData>[] = [];

    param.items.forEach((item) => {
        doptions.push({
            key: item.id,
            text: item.text,
        });
    });

    return (<Dropdown id={key}
        key={key}
        label={param.label}
        placeholder={placeholder}
        options={doptions}
        selectedKey={selectedKey}
        dropdownWidth={selectorWidth}
        styles={{ dropdown: { width: calloutWidth } }}
        calloutProps={{
            directionalHint: DirectionalHint.rightTopEdge,
            alignTargetEdge: true,
        }}
        onChange={(_ev, option) => {
            let castedOption = option as IDropdownOption<PerformanceTrendOptionData>;
            if (!castedOption) {
                return;
            }
            onChange(castedOption as IDropdownOption<PerformanceTrendOptionData>);
        }}
        onDismiss={() => {
            performanceTrendOptions.commitTransactionSession();
            handler.requestHierarchicalRefresh(request, undefined, (_request: DataHandlerRefreshRequest) => { });
        }}
    />);
};