// Copyright Epic Games, Inc. All Rights Reserved.

import React, { useEffect, useRef, useState } from "react";
import * as d3 from "d3";
import { getHordeStyling, preloadFonts } from "horde/styles/Styles";
import { graphColors } from "hordePlugins/analytics/telemetryData";
import { PerformanceTrendContext } from "../metrictypes/PerformanceTrendsTypes";

/**
 * Muted color palette for reference lines (budgets, thresholds, etc.).
 * Designed to be visually distinct from data series colors while remaining subtle.
 * Uses desaturated hues for better differentiation.
 */
const referenceLineColors = [
    "#7c3aed", // violet-600
    "#0891b2", // cyan-600
    "#c026d3", // fuchsia-600
    "#ea580c", // orange-600
    "#4f46e5", // indigo-600
    "#0d9488", // teal-600
    "#db2777", // pink-600
    "#65a30d", // lime-600
];

export type MetricSelectionHandler<T extends PerformanceTrendContext> = (selectedSeriesElements: T[], isFiltered: boolean) => void

/**
 * Represents a horizontal reference line to be drawn on the chart.
 */
export interface ReferenceLine {
    /** The y-axis value at which to draw the line. */
    value: number;
    /** Display label for the reference line (shown in legend). */
    label: string;
    /** Color of the reference line. Defaults to a red tone. */
    color?: string;
    /** Whether to render as a dashed line. Defaults to true. */
    dashed?: boolean;
}

const LEGEND_SIZE_DEFAULT: number = 225;
const DEFAULT_TOOLTIP_WIDTH_ESTIMATE = 200;

// Module-scoped cache that persists the legend's selected-series set across LineGraph
// unmount/remount cycles. The parent tree can re-evaluate `view.Render` references on
// chart-click (modal opening, ResizeObserver-triggered width changes, etc.), causing React
// to treat LineGraph as a new component type and discard its local state. Keyed by a
// chart-identifying string derived from the props so each distinct chart maintains its own
// selection independently.
const persistentSelectionCache = new Map<string, Set<number>>();

function buildSelectionCacheKey(yAxisLabel: string, xAxisLabel: string, seriesLabels: string[]): string {
    return `${yAxisLabel}␟${xAxisLabel}␟${seriesLabels.join("␟")}`;
}

/**
 * Properties for the line graph component.
 */
export interface LineGraphProps<T extends PerformanceTrendContext> {
    /**
     * 2d array representing a list of series, each containing it's own numerical data.
     */
    series: SeriesConfig<T>[];

    /**
     * Defines an x accessor for the graph.
     * @param item The item.
     * @param index The index of the item.
     * @returns The numerical representation along the x axis.
     */
    xAccessor?: (item: T, index: number) => number;

    /**
     * The y axis label.
     */
    yAxisLabel: string,

    /**
     * The x axis label.
     */
    xAxisLabel: string,

    /**
     * The maximum width of the graph.
     */
    width?: number;

    /**
     * The prescribed maximum height of the graph.
     */
    height?: number;

    /**
     * Whether to have the y axis start at zero, or whether to perform min-max clamping for the limits.
     */
    yAxisZeroScale?: boolean;

    /**
     * Whether to apply curve smoothing to the line graph.
     */
    applyCurveSmoothing?: boolean;

    /**
     * Onclick delegate
     * @returns
     */
    onClick?: MetricSelectionHandler<T>;

    /**
     * Optional reference lines to draw as horizontal lines on the chart.
     * Typically used to show budget thresholds or target values.
     */
    referenceLines?: ReferenceLine[];
}

export interface SeriesConfig<T extends PerformanceTrendContext> {
    data: T[];
    yAccessor: (item: T) => number;
    label: string;
    color?: string;
}

/**
 * LineGraph component used to draw a line graph using input data, labels, etc.
 * @param props The properties to use in constructing the Line Graph.
 * @returns The line graph component.
 */
export function LineGraph<T extends PerformanceTrendContext>({ series, xAccessor, yAxisLabel, xAxisLabel, width = 300, height = 150, yAxisZeroScale = false, applyCurveSmoothing = false, onClick = undefined, referenceLines = [] }: LineGraphProps<T>): React.ReactElement {
    const { modeColors, _hordeClasses, _detailClasses } = getHordeStyling();

    // #region -- Graph Component Refs & State --

    const svgRef = useRef<SVGSVGElement | null>(null);
    const graphRef = useRef<d3.Selection<SVGGElement, unknown, null, undefined> | null>(null);
    const tooltipRef = useRef<HTMLDivElement | null>(null);

    // Stable identifier for this chart's selection state. Used as the cache key so the
    // selected-series set survives remounts that we cannot otherwise prevent from this file.
    const selectionCacheKey = buildSelectionCacheKey(yAxisLabel, xAxisLabel, series.map(s => s.label));

    const [selectedIndices, setSelectedIndices] = useState<Set<number>>(
        () => persistentSelectionCache.get(selectionCacheKey) ?? new Set()
    );

    // Mirror every selectedIndices change back into the module-level cache so the next
    // mount of an equivalent chart rehydrates the same selection.
    useEffect(() => {
        if (selectedIndices.size === 0) {
            persistentSelectionCache.delete(selectionCacheKey);
        } else {
            persistentSelectionCache.set(selectionCacheKey, selectedIndices);
        }
    }, [selectedIndices, selectionCacheKey]);

    // Ref mirror of selectedIndices so the d3 mousemove handler (installed inside the chart-building
    // useEffect, which intentionally does NOT re-run on selection changes) can read the live selection
    // without stale closure bugs.
    const selectedIndicesRef = useRef<Set<number>>(selectedIndices);
    useEffect(() => {
        selectedIndicesRef.current = selectedIndices;
    }, [selectedIndices]);

    // #endregion -- Graph Component Refs --

    type LineGraphData = {
        min: number;
        max: number;
        avg: number;
        data: T[];
    }

    // #region -- Tool Tip Helpers --

    const clearTableTooltipRows = (tooltipContent: HTMLDivElement) => {
        const container = tooltipContent.querySelector(
            ".tooltipContentContainer"
        ) as HTMLDivElement | null;

        if (!container) {
            return;
        }

        // Fast + safe
        container.replaceChildren();
    };

    let generateTableTooltipRow = (tooltipContent: HTMLDivElement, label: string, value: string, color: string) => {
        // Find or create the parent container
        let container = tooltipContent.querySelector(
            ".tooltipContentContainer"
        ) as HTMLDivElement | null;

        if (!container) {
            container = document.createElement("div");
            container.className = "tooltipContentContainer";
            tooltipContent.appendChild(container);
        }

        // Create the row
        const row = document.createElement("div");
        row.className = "tooltipRow";

        const labelEl = document.createElement("strong");
        labelEl.textContent = `${label}: `;

        const valueEl = document.createElement("span");
        valueEl.textContent = value;

        if (color) {
            row.style.color = color
        }

        row.appendChild(labelEl);
        row.appendChild(valueEl);

        container.appendChild(row);
    }

    let generateTooltipSubheader = (tooltipContent: HTMLDivElement, text: string) => {
        let container = tooltipContent.querySelector(
            ".tooltipContentContainer"
        ) as HTMLDivElement | null;

        if (!container) {
            container = document.createElement("div");
            container.className = "tooltipContentContainer";
            tooltipContent.appendChild(container);
        }

        const subheader = document.createElement("div");
        subheader.className = "tooltipSubheader";
        subheader.style.marginTop = "8px";
        subheader.style.marginBottom = "4px";
        subheader.style.fontWeight = "bold";
        subheader.style.borderBottom = "1px solid #666";
        subheader.style.paddingBottom = "2px";
        subheader.textContent = text;

        container.appendChild(subheader);
    }

    // #endregion -- Tool Tip Helpers --

    // #region -- Data Processing Helpers -- 

    function summarizeSeriesData(summarizedSeries: Map<number, { min: number; max: number; avg: number; data: T[]; }>[], minYAxisValue: number, maxYAxisValue: number, maxXAxisValue: number, minXAxisValue: number) {
        let count: number = 0;

        for (let idx: number = 0; idx < series.length; ++idx) {
            let seriesSet: Map<number, LineGraphData> = new Map<number, LineGraphData>;
            summarizedSeries.push(seriesSet);

            for (let idxY: number = 0; idxY < series[idx].data.length; ++idxY) {
                let element: T = series[idx].data[idxY];
                let resolvedXBucket = xAccessor ? xAccessor(series[idx].data[idxY], count) : count;

                let value: number = series[idx].yAccessor(element);

                minYAxisValue = minYAxisValue < value ? minYAxisValue : value;
                maxYAxisValue = maxYAxisValue > value ? maxYAxisValue : value;

                if (!seriesSet.has(resolvedXBucket)) {
                    let datum: LineGraphData = { min: value, max: value, data: [element], avg: value };
                    seriesSet.set(resolvedXBucket, datum);
                    maxXAxisValue = maxXAxisValue > resolvedXBucket ? maxXAxisValue : resolvedXBucket;
                    minXAxisValue = minXAxisValue < resolvedXBucket ? minXAxisValue : resolvedXBucket;
                } else {
                    let datum: LineGraphData = seriesSet.get(resolvedXBucket)!;

                    datum.max = datum.max > value ? datum.max : value;
                    datum.min = datum.min < value ? datum.min : value;

                    datum.data.push(element);
                    datum.avg = datum.avg + (value - datum.avg) / datum.data.length;
                }

                count += 1;
            }
        }
        return { minYAxisValue, maxYAxisValue, maxXAxisValue, minXAxisValue };
    }

    // Find closest bucket to hovered X
    const findClosestBucket = (seriesMap: Map<number, LineGraphData>, xLocation: number): [number, LineGraphData] | null => {
        const buckets = Array.from(seriesMap.keys()).sort((a, b) => a - b);

        if (buckets.length === 0) {
            return null;
        }

        let closest = buckets[0];
        let closestDist = Math.abs(xLocation - closest);

        for (const bucket of buckets) {
            const dist = Math.abs(xLocation - bucket);
            if (dist < closestDist) {
                closest = bucket;
                closestDist = dist;
            }
        }

        return seriesMap.has(closest) ? [closest, seriesMap.get(closest)!] : null;
    };

    // #endregion -- Data Processing Helpers --

    // #region -- Use Effects --

    useEffect(() => {
        if (!svgRef.current) return;

        let summarizedSeries: Map<number, LineGraphData>[] = [];
        let maxYAxisValue: number = Number.NEGATIVE_INFINITY;
        let minYAxisValue: number = Number.MAX_VALUE;
        let maxXAxisValue: number = Number.NEGATIVE_INFINITY;
        let minXAxisValue: number = Number.MAX_VALUE;

        ({ minYAxisValue, maxYAxisValue, maxXAxisValue, minXAxisValue } = summarizeSeriesData(summarizedSeries, minYAxisValue, maxYAxisValue, maxXAxisValue, minXAxisValue));

        // Include reference line values in Y-axis extent calculations
        for (const refLine of referenceLines) {
            if (refLine.value > maxYAxisValue) {
                maxYAxisValue = refLine.value;
            }
            if (refLine.value < minYAxisValue) {
                minYAxisValue = refLine.value;
            }
        }

        let colorLabels: string[] = series.map((x) => x.color ?? null).filter((value, idx) => value !== null);

        // #region -- Graph Dimensions & Measures --
        // SVG width is already reduced by legend width in the render
        const svgWidth = width - LEGEND_SIZE_DEFAULT;

        // Dimensions
        const margin = { top: 10, right: 10, bottom: 45, left: 55 };
        const innerWidth = svgWidth - margin.left - margin.right;
        const innerHeight = height - margin.top - margin.bottom;

        // Compute min, max, midpoint for centered line
        const mid = (maxYAxisValue + minYAxisValue) / 2;
        const halfRange = (maxYAxisValue - minYAxisValue) / 2;
        const lowerBoundYAxis = yAxisZeroScale ? 0 : mid - halfRange;
        const upperBoundYAxis = yAxisZeroScale ? (maxYAxisValue) * 1.25 : mid + halfRange;

        // Scales
        // Collect all unique x buckets across all series
        const allBucketsSet = new Set<number>();
        for (const seriesMap of summarizedSeries) {
            for (const bucket of seriesMap.keys()) {
                allBucketsSet.add(bucket);
            }
        }
        const sortedBuckets = Array.from(allBucketsSet).sort((a, b) => a - b);

        // When xAccessor is provided, use scalePoint for evenly-spaced discrete values (changelists)
        // Otherwise, use scaleLinear for proportional numeric spacing
        let xScale: (value: number) => number;
        let xInvert: (pixelX: number) => number;

        if (xAccessor) {
            const xPoint = d3.scalePoint<number>()
                .domain(sortedBuckets)
                .range([0, innerWidth]);

            xScale = (value: number) => xPoint(value) ?? 0;

            xInvert = (pixelX: number): number => {
                if (sortedBuckets.length === 0) return 0;
                if (sortedBuckets.length === 1) return sortedBuckets[0];

                let closestBucket = sortedBuckets[0];
                let closestDist = Math.abs(pixelX - (xPoint(closestBucket) ?? 0));

                for (const bucket of sortedBuckets) {
                    const bucketPixel = xPoint(bucket) ?? 0;
                    const dist = Math.abs(pixelX - bucketPixel);
                    if (dist < closestDist) {
                        closestBucket = bucket;
                        closestDist = dist;
                    }
                }
                return closestBucket;
            };
        } else {
            const xLinear = d3.scaleLinear()
                .domain([minXAxisValue, maxXAxisValue])
                .range([0, innerWidth]);

            xScale = (value: number) => xLinear(value);
            xInvert = (pixelX: number) => Math.round(xLinear.invert(pixelX));
        }

        const y = d3.scaleLinear()
            .domain([upperBoundYAxis, lowerBoundYAxis])
            .range([0, innerHeight]);

        // #endregion -- Graph Dimensions & Measures --

        // #region -- Core Graph Setup --

        // Clear previous contents
        const svg = d3.select(svgRef.current);
        svg.selectAll("*").remove();

        // Main graph group
        const g = svg.append("g")
            .attr("transform", `translate(${margin.left},${margin.top})`);

        graphRef.current = g;

        // #endregion -- Core Graph Setup --

        // #region -- Axis --

        // Axes
        if (xAccessor) {
            // For scalePoint: use sparse ticks if there are many buckets; we cap at 10 for now
            const xPoint = d3.scalePoint<number>()
                .domain(sortedBuckets)
                .range([0, innerWidth]);

            const maxTicks = 10;
            const step = Math.ceil(sortedBuckets.length / maxTicks);
            const sparseBuckets = sortedBuckets.filter((_, i) => i % step === 0);

            g.append("g")
                .attr("transform", `translate(0,${innerHeight})`)
                .call(d3.axisBottom(xPoint).tickValues(sparseBuckets).tickFormat(d => d.toString()))
                .style("font-family", preloadFonts[0]);
        } else {
            // For scaleLinear: use default ticks
            const xLinear = d3.scaleLinear()
                .domain([minXAxisValue, maxXAxisValue])
                .range([0, innerWidth]);

            g.append("g")
                .attr("transform", `translate(0,${innerHeight})`)
                .call(d3.axisBottom(xLinear).ticks(5))
                .style("font-family", preloadFonts[0]);
        }

        g.append("g")
            .call(d3.axisLeft(y).ticks(5))
            .style("font-family", preloadFonts[0]);

        // Add Axis Labels
        // X axis label
        g.append("text")
            .attr("x", innerWidth / 2)
            .attr("y", innerHeight + margin.bottom - 4)
            .attr("text-anchor", "middle")
            .attr("fill", modeColors.text)
            .style("font-family", preloadFonts[0])
            .style("font-size", "12px")
            .text(xAxisLabel);

        // Y axis label
        g.append("text")
            .attr("transform", `rotate(-90)`)
            .attr("x", -innerHeight / 2)
            .attr("y", -margin.left + 12)
            .attr("text-anchor", "middle")
            .attr("fill", modeColors.text)
            .style("font-family", preloadFonts[0])
            .style("font-size", "12px")
            .text(yAxisLabel);

        // #endregion -- Axis --

        // #region -- Line Drawing --

        // Line generator
        // Line generator for bucketed data
        const lineGenerator = d3.line<[number, LineGraphData]>()
            .x(([xBucket, _]) => xScale(xBucket))
            .y(([_, data]) => y(data.avg));  // or data.min, data.max

        if (applyCurveSmoothing) {
            lineGenerator.curve(d3.curveMonotoneX);
        }

        // if anyone didn't specify colour, just default to graph colours for now.
        const colors = colorLabels.length === series.length ? colorLabels : graphColors;

        // Area generator for min/max range
        const areaGenerator = d3.area<[number, LineGraphData]>()
            .x(([xBucket, _]) => xScale(xBucket))
            .y0(([_, data]) => y(data.min))
            .y1(([_, data]) => y(data.max));

        if (applyCurveSmoothing) {
            areaGenerator.curve(d3.curveMonotoneX);
        }

        // Draw range bands (before lines so lines are on top)
        for (let idx: number = 0; idx < summarizedSeries.length; ++idx) {
            const sortedEntries = Array.from(summarizedSeries[idx].entries())
                .sort(([a], [b]) => a - b);

            g.append("path")
                .datum(sortedEntries)
                .attr("fill", colors[idx])
                .attr("fill-opacity", 0.15)
                .attr("stroke", "none")
                .attr("d", areaGenerator)
                .attr("class", `series-range series-${idx}`);
        }

        // Draw lines from bucketed data
        for (let idx: number = 0; idx < summarizedSeries.length; ++idx) {
            const seriesMap = summarizedSeries[idx];
            const sortedEntries = Array.from(seriesMap.entries())
                .sort(([a], [b]) => a - b);

            g.append("path")
                .datum(sortedEntries)
                .attr("fill", "none")
                .attr("stroke", colors[idx])
                .attr("stroke-width", 2)
                .attr("d", lineGenerator)
                .attr("class", `series-line series-${idx}`);
        }

        // #endregion -- Line Drawing --

        // #region -- Reference Lines --

        referenceLines.forEach((refLine, refIdx) => {
            const yPos = y(refLine.value);
            const lineColor = refLine.color ?? referenceLineColors[refIdx % referenceLineColors.length];
            const isDashed = refLine.dashed !== false; // Default to dashed

            // Only draw if the line is within the visible y-axis range
            if (yPos >= 0 && yPos <= innerHeight) {
                g.append("line")
                    .attr("x1", 0)
                    .attr("x2", innerWidth)
                    .attr("y1", yPos)
                    .attr("y2", yPos)
                    .attr("stroke", lineColor)
                    .attr("stroke-width", 1.5)
                    .attr("stroke-dasharray", isDashed ? "6,4" : "none")
                    .attr("class", "reference-line");
            }
        });

        // #endregion -- Reference Lines --

        // #region -- Tooltip & Accessibility Controls --

        // Vertical hover line
        const hoverLine = g.append("line")
            .attr("stroke", "gray")
            .attr("stroke-width", 1)
            .attr("y1", 0)
            .attr("y2", innerHeight)
            .style("opacity", 0);

        // Tooltip styling
        const tooltipElement = tooltipRef.current;

        if (tooltipElement) {
            tooltipElement.style.position = "fixed";
            tooltipElement.style.pointerEvents = "none";
            tooltipElement.style.background = modeColors.background;
            tooltipElement.style.border = "1px solid #aaa";
            tooltipElement.style.padding = "4px 6px";
            tooltipElement.style.borderRadius = "4px";
            tooltipElement.style.fontFamily = preloadFonts[0];
            tooltipElement.style.fontSize = "12px";
            tooltipElement.style.color = modeColors.text;
            tooltipElement.style.opacity = "0";
            tooltipElement.style.zIndex = "9999";
        }

        // #endregion -- Tooltip & Accessibility Controls --

        const highlightLayer = g.append("g")
            .attr("class", "tooltip-highlights");

        // Overlay rectangle to capture mouse
        g.append("rect")
            .attr("width", innerWidth)
            .attr("height", innerHeight)
            .style("fill", "none")
            .style("pointer-events", "all")
            .on("mousemove", (event: MouseEvent) => {
                if (!tooltipElement) {
                    return;
                }

                const parentRect = tooltipElement.parentElement!.getBoundingClientRect();
                const mouseX = event.clientX - parentRect.left - margin.left;
                const mouseY = event.clientY - parentRect.top - margin.top;
                const hoveredX = xInvert(mouseX);

                // Check bounds
                if (hoveredX < minXAxisValue || hoveredX > maxXAxisValue) {
                    return;
                }

                tooltipElement.style.opacity = "1";

                // Flip tooltip to left side if it would overflow the viewport
                const tooltipWidth = tooltipElement.offsetWidth || DEFAULT_TOOLTIP_WIDTH_ESTIMATE; // estimate if not yet rendered
                const wouldOverflow = event.clientX + 15 + tooltipWidth > window.innerWidth;

                if (wouldOverflow) {
                    tooltipElement.style.left = `${event.clientX - tooltipWidth - 15}px`;
                } else {
                    tooltipElement.style.left = `${event.clientX + 15}px`;
                }
                tooltipElement.style.top = `${event.clientY + 10}px`;

                clearTableTooltipRows(tooltipElement);
                highlightLayer.selectAll("*").remove();

                let snappedX: number | null = null;

                for (let idx: number = 0; idx < summarizedSeries.length; ++idx) {
                    const result = findClosestBucket(summarizedSeries[idx], hoveredX);
                    if (!result) continue;

                    const [bucket, data] = result;

                    const label = legendLabels[idx];

                    // Only show this series if it matches the snapped bucket
                    if (hoveredX !== bucket) {
                        continue;
                    }

                    // Use first series' bucket for vertical line
                    if (snappedX === null) {
                        snappedX = bucket;
                        generateTableTooltipRow(tooltipElement, `(${bucket})`, "", modeColors.messageText);
                    }

                    // Tooltip: show avg, min, max, count
                    const tooltipText = `avg: ${data.avg.toFixed(2)} | min: ${data.min.toFixed(2)} | max: ${data.max.toFixed(2)} | n: ${data.data.length}`;
                    const selection = selectedIndicesRef.current;
                    if (selection.size === 0 || selection.has(idx)) {
                        generateTableTooltipRow(tooltipElement, label, tooltipText, colors[idx]);
                    }

                    // Highlight circle at avg
                    const xpos = xScale(bucket);
                    const ypos = y(data.avg);

                    highlightLayer.append("circle")
                        .attr("cx", xpos)
                        .attr("cy", ypos)
                        .attr("r", 5)
                        .attr("fill", colors[idx]);

                    // Draw a line across the range
                    highlightLayer.append("line")
                        .attr("x1", xpos)
                        .attr("x2", xpos)
                        .attr("y1", y(data.min))
                        .attr("y2", y(data.max))
                        .attr("stroke", colors[idx])
                        .attr("stroke-width", 2)
                        .attr("opacity", 0.5);

                    // Annotate the range with smaller dots
                    highlightLayer.append("circle")
                        .attr("cx", xpos)
                        .attr("cy", y(data.min))
                        .attr("r", 2)
                        .attr("fill", colors[idx]);

                    highlightLayer.append("circle")
                        .attr("cx", xpos)
                        .attr("cy", y(data.max))
                        .attr("r", 2)
                        .attr("fill", colors[idx]);
                }

                // Add reference lines to tooltip (always shown at hover position, regardless of data)
                if (referenceLines.length > 0) {
                    // Use snapped position if available, otherwise use direct hover position
                    const xPosForRef = snappedX !== null ? xScale(snappedX) : xScale(hoveredX);

                    // If no data was found at hover position, still show the CL header
                    if (snappedX === null) {
                        generateTableTooltipRow(tooltipElement, `(${hoveredX})`, "", modeColors.messageText);
                    }

                    generateTooltipSubheader(tooltipElement, "Reference");

                    referenceLines.forEach((refLine, refIdx) => {
                        const refColor = refLine.color ?? referenceLineColors[refIdx % referenceLineColors.length];
                        const yPos = y(refLine.value);

                        // Add tooltip row for reference line
                        const refLabel = refLine.label.split('\n')[0]; // Use first line of label
                        generateTableTooltipRow(tooltipElement, refLabel, refLine.value.toFixed(2), refColor);

                        // Draw highlight dot on reference line at current x position
                        if (yPos >= 0 && yPos <= innerHeight) {
                            highlightLayer.append("circle")
                                .attr("cx", xPosForRef)
                                .attr("cy", yPos)
                                .attr("r", 5)
                                .attr("fill", refColor)
                                .attr("stroke", "#fff")
                                .attr("stroke-width", 1);
                        }
                    });

                    // Show vertical hover line even if no data at this position
                    if (snappedX === null) {
                        hoverLine
                            .attr("x1", xPosForRef)
                            .attr("x2", xPosForRef)
                            .style("opacity", 1);
                    }
                }

                // Show vertical line at snapped bucket
                if (snappedX !== null) {
                    const snappedPixel = xScale(snappedX);
                    hoverLine
                        .attr("x1", snappedPixel)
                        .attr("x2", snappedPixel)
                        .style("opacity", 1);
                }
            })
            .on("mouseleave", () => {
                if (!tooltipElement) {
                    return;
                }
                highlightLayer.selectAll("*").remove();

                tooltipElement.style.opacity = "0";
                hoverLine.style("opacity", 0);
            })
            .on("click", (event: MouseEvent) => {
                if (!tooltipElement || !onClick) {
                    return;
                }

                const parentRect = tooltipElement.parentElement!.getBoundingClientRect();
                const mouseX = event.clientX - parentRect.left - margin.left;
                const index = xInvert(mouseX);

                const selection = selectedIndicesRef.current;
                const filterToSelected = selection.size > 0;

                let selectedItems: T[] = [];
                for (let idx: number = 0; idx < summarizedSeries.length; ++idx) {
                    if (filterToSelected && !selection.has(idx)) {
                        continue;
                    }

                    const result = findClosestBucket(summarizedSeries[idx], index);

                    if (!result) {
                        continue;
                    }

                    const [bucket, data] = result;

                    if (bucket === index) {
                        selectedItems.push(...data.data);
                    }
                }

                onClick(selectedItems, filterToSelected);
            });

    }, [series, width, height, selectedIndices, yAxisZeroScale, applyCurveSmoothing, referenceLines]);

    // Deps mirror the chart-building useEffect above so dimming is reapplied after every
    // chart rebuild (width changes from ResizeObserver, series/referenceLines updates, etc.).
    // Without these extra deps, a rebuild would replace the .series-line/.series-range paths
    // with fresh full-opacity ones and the dimming would not run again until selectedIndices
    // itself changed — making the legend look filtered while the chart shows everything.
    useEffect(() => {
        if (!graphRef.current) {
            return;
        }

        const lines = graphRef.current.selectAll<SVGPathElement, unknown>(".series-line");
        const ranges = graphRef.current.selectAll<SVGPathElement, unknown>(".series-range");

        if (selectedIndices.size === 0) {
            // Nothing selected — everything visible at full opacity.
            lines.style("opacity", 1);
            ranges.style("opacity", 0.65);
        } else {
            // Dim everything first, then re-brighten the selected indices.
            lines.style("opacity", 0.2);
            ranges.style("opacity", 0.05);

            for (const idx of selectedIndices) {
                graphRef.current.select<SVGPathElement>(`.series-line.series-${idx}`).style("opacity", 1);
                graphRef.current.select<SVGPathElement>(`.series-range.series-${idx}`).style("opacity", 0.65);
            }
        }
    }, [series, width, height, selectedIndices, yAxisZeroScale, applyCurveSmoothing, referenceLines]);

    // #endregion -- Use Effects --

    // Compute colors for legend (same logic as in useEffect)
    const legendColors = series.map((s, i) => s.color ?? graphColors[i % graphColors.length]);
    const legendLabels = series.map((s) => s.label);
    const maxLegendHeight = Math.max(height - 40, 100); // Leave some margin, min 100px

    return (
        <div style={{ position: "relative", display: "flex" }}>
            <svg ref={svgRef} width={width - LEGEND_SIZE_DEFAULT} height={height}></svg>
            <div
                style={{
                    width: LEGEND_SIZE_DEFAULT,
                    maxHeight: maxLegendHeight,
                    overflowY: "auto",
                    overflowX: "hidden",
                    background: modeColors.content,
                    borderRadius: 4,
                    padding: 6,
                    boxSizing: "border-box",
                    fontFamily: preloadFonts[0],
                    fontSize: 10,
                    color: modeColors.text,
                    marginTop: 20,
                }}
            >
                {series.map((s, i) => (
                    <div
                        key={i}
                        title="Click to isolate. Ctrl/Cmd-click to add or remove from the active selection."
                        style={{
                            display: "flex",
                            alignItems: "flex-start",
                            marginBottom: 6,
                            cursor: "pointer",
                            opacity: selectedIndices.size === 0 || selectedIndices.has(i) ? 1 : 0.2,
                        }}
                        onClick={(e) => {
                            const isModifier = e.ctrlKey || e.metaKey;
                            setSelectedIndices(prev => {
                                const next = new Set(prev);
                                if (isModifier) {
                                    if (next.has(i)) {
                                        next.delete(i);
                                    } else {
                                        next.add(i);
                                    }
                                } else {
                                    if (next.size === 1 && next.has(i)) {
                                        next.clear();
                                    } else {
                                        next.clear();
                                        next.add(i);
                                    }
                                }
                                return next;
                            });
                        }}
                    >
                        <div
                            style={{
                                width: 20,
                                height: 2,
                                backgroundColor: legendColors[i],
                                marginTop: 6,
                                marginRight: 6,
                                flexShrink: 0,
                            }}
                        />
                        <div style={{ flex: 1, minWidth: 0 }}>
                            {legendLabels[i].split('\n').map((line, lineIdx) => (
                                <div
                                    key={lineIdx}
                                    style={{
                                        whiteSpace: "nowrap",
                                        overflow: "hidden",
                                        textOverflow: "ellipsis",
                                    }}
                                    title={line}
                                >
                                    {line}
                                </div>
                            ))}
                        </div>
                    </div>
                ))}
                {/* Reference Lines Legend */}
                {referenceLines.map((refLine, i) => (
                    <div
                        key={`ref-${i}`}
                        style={{
                            display: "flex",
                            alignItems: "flex-start",
                            marginBottom: 6,
                        }}
                    >
                        <div
                            style={{
                                width: 20,
                                height: 0,
                                borderTop: `2px ${refLine.dashed !== false ? "dashed" : "solid"} ${refLine.color ?? referenceLineColors[i % referenceLineColors.length]}`,
                                marginTop: 6,
                                marginRight: 6,
                                flexShrink: 0,
                            }}
                        />
                        <div style={{ flex: 1, minWidth: 0 }}>
                            {refLine.label.split('\n').map((line, lineIdx) => (
                                <div
                                    key={lineIdx}
                                    style={{
                                        whiteSpace: "nowrap",
                                        overflow: "hidden",
                                        textOverflow: "ellipsis",
                                    }}
                                    title={lineIdx === 0 ? `${refLine.label}: ${refLine.value}` : line}
                                >
                                    {line}
                                </div>
                            ))}
                        </div>
                    </div>
                ))}
            </div>
            <div ref={tooltipRef}></div>
        </div>
    );
};