// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { DataHandlerRefreshRequest, expandRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionList, PerformanceTrendOptionData, BasicListParameter } from "./DropdownComponent";
import { GetTestProjectResponse } from "../responses/GetTestProjectResponse";
import { encodeMetricTypeKey } from "../responses/FilterKeys";

/**
 * Component used to model all known metric summary types (for the currently selected test project, and test identity), as a single select dropdown.
 * @return The react component.
 */
export const MetricSummaryTypeDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    const metricSummaryDropdownParams: PerformanceTrendOptionList = {
        label: "",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    metricSummaryDropdownParams.items = handler.metricSummaryTypes.map((metricTypeParentProject: GetTestProjectResponse) => {
        let listParam: PerformanceTrendOptionData = {
            id: encodeMetricTypeKey(metricTypeParentProject),
            text: metricTypeParentProject.summaryType,
        };

        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    let selectedSummaryMetricTypeKey = Object.keys(performanceTrendOptions.state.enabledMetricSummaryTypes)[0] ?? null;
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ testTypes: true });
    
    return metricSummaryDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Summary Type</Label>
            <BasicListParameter placeholder={"Select a metric summary type"} handler={handler} performanceTrendOptions={performanceTrendOptions} params={metricSummaryDropdownParams} selectedKey={selectedSummaryMetricTypeKey} request={request} onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                const key = option.key as string;
                performanceTrendOptions.getTransactionSession().toggleSingleMetricSummaryType(key, option.text);
            }} />
        </Stack>
    );
});