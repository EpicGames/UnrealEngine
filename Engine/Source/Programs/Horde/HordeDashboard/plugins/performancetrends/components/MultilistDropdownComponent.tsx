import { DefaultButton, DropdownMenuItemType, IDropdownOption, ISelectableOption, mergeStyleSets, SelectableOptionMenuItemType, Stack } from "@fluentui/react";
import { DataHandlerRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { PARAMETER_KEY_PREFIX, PerformanceTrendOptionData, PerformanceTrendOptionList } from "./DropdownComponent";
import { useCallback, useEffect, useLayoutEffect, useMemo, useRef, useState } from "react";

// We should generalize this
import { SearchableDropdown } from "horde/components/buildhealth/SearchableDropdownComponent";
import { PerformanceTrendOptionsStateDiff } from "../filters/PerformanceTrendOptionsWriter";
import { KEY_SEPARATOR } from "../responses/FilterKeys";

const SELECT_CLEAR_ALL_SPECIAL_GROUP: string = "select_clear_all";

const dropdownOptions = mergeStyleSets({
    base: {
        padding: "4px 8px",
    }
});

/**
 * React Component that is a multi select drop down.
 * @returns React Component.
 */
export const MultiListParameter: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController, params: PerformanceTrendOptionList, disabled?: boolean, selectedKeys: string[], request: DataHandlerRefreshRequest, onChange: (option: IDropdownOption<PerformanceTrendOptionData>) => void, enabledSelectAll?: boolean, onSelectAll?: (groupOption: IDropdownOption<PerformanceTrendOptionData>, optionSelectAllContext?: IDropdownOption<PerformanceTrendOptionData>[]) => void, onHierarchicalRefreshCompleted?: (receipts?: PerformanceTrendOptionsStateDiff) => void }> = function ConstructMultilistParamter({ params, handler, performanceTrendOptions, disabled, selectedKeys, request, onChange, enabledSelectAll = false, onSelectAll, onHierarchicalRefreshCompleted }) {
    const [filterText, setFilterText] = useState("");
    const changeVersion = performanceTrendOptions.optionsChangeVersion;
    const key = `${PARAMETER_KEY_PREFIX}${KEY_SEPARATOR}${params.label}`;
    const selectedSet = useMemo(() => new Set(selectedKeys), [selectedKeys]);
    const groupRefs = useRef<(HTMLDivElement | null)[]>([]);

    useEffect(() => { groupRefs.current = []; }, [params.items]);

    // #region -- Data Memos --

    const doptions = useMemo(() => {
        const groupMap = new Map<string, PerformanceTrendOptionData[]>();

        for (const item of params.items) {
            const group = item.group ?? "__nogroup";
            if (!groupMap.has(group)) groupMap.set(group, []);
            groupMap.get(group)!.push(item);
        }

        const groups = Array.from(groupMap.keys()).sort();
        const options: IDropdownOption<PerformanceTrendOptionData>[] = [];

        for (const group of groups) {
            if (group !== "__nogroup") {
                options.push({ key: `group_${group}`, text: group, itemType: DropdownMenuItemType.Header });
                if (enabledSelectAll) {
                    options.push({
                        key: `group_select_all${KEY_SEPARATOR}${group}`,
                        itemType: DropdownMenuItemType.Header,
                        text: SELECT_CLEAR_ALL_SPECIAL_GROUP,
                    });
                }
            }

            for (const item of groupMap.get(group)!) {
                options.push({
                    key: item.id,
                    text: item.text,
                    data: item,
                    selected: selectedSet.has(item.id),
                });
            }
        }

        return options;
    }, [params.items, selectedSet, enabledSelectAll]);

    // #endregion -- Data Memos --
    const filteredOptions = useMemo(() => {
        const lowerFilter = filterText.toLowerCase();
        return doptions.filter(o =>
            o.itemType === DropdownMenuItemType.Header || !filterText
                ? true
                : o.text.toLowerCase().includes(lowerFilter)
        );
    }, [doptions, filterText]);

    const filteredOptionsRef = useRef<IDropdownOption<PerformanceTrendOptionData>[]>([]);

    useLayoutEffect(() => {
        filteredOptionsRef.current = filteredOptions;
    }, [filteredOptions]);

    // #region -- Dropdown Callbacks --

    const onRenderOption = useCallback(
        (option?: ISelectableOption) => {
            if (!option) {
                return null;
            }

            // Select & Clear all for the given filtered list.
            if (option.itemType === DropdownMenuItemType.Header && option.text === SELECT_CLEAR_ALL_SPECIAL_GROUP) {
                let filteredOptionList = filteredOptionsRef.current.length === doptions.length ? undefined : filteredOptionsRef.current.filter(x => x.itemType === undefined || x.itemType !== SelectableOptionMenuItemType.Header);
                let selectAllLabel = (filteredOptionList !== undefined && filteredOptionList.length > 0) ? "Select Filtered" : "Select All";
                let showButton = filteredOptionList === undefined || filteredOptionList.length > 0;
                return (
                    <Stack horizontal tokens={{ childrenGap: 8 }} styles={{
                        root:
                        {
                            width: "100%",
                            height: "100%",
                            alignItems: "center",
                            justifyContent: "center",
                        }
                    }}>
                        {/* We have opted to use buttons here as we: 1. don't want to persist the state of a select all option; 2. the semantics of storing a "Select/Clear All" value gets complicated when the action is applied to a filtered set; */}
                        {showButton &&
                            <DefaultButton title="Selects all options (or currently filtered) within the group." styles={{ root: { flex: 1, height: 20 } }} text={selectAllLabel}
                                onClick={() => {
                                    if (!option) return;

                                    const synthesizedOption: IDropdownOption<PerformanceTrendOptionData> = {
                                        ...option,
                                        selected: true,
                                    };

                                    // Only send a suggested list if we actually have filtered something.
                                    let filteredOptionList = filteredOptionsRef.current.length === doptions.length ? undefined : filteredOptionsRef.current;
                                    option && onSelectAll?.(synthesizedOption, filteredOptionList)
                                }}
                            />
                        }
                        <DefaultButton title="Clears all options within the group." styles={{ root: { flex: 1, height: 20 } }} text="Clear All"
                            onClick={() => {
                                if (!option) return;

                                const synthesizedOption: IDropdownOption<PerformanceTrendOptionData> = {
                                    ...option,
                                    selected: false,
                                };

                                // We do not send a suggested list for clear all; we just clear all.
                                option && onSelectAll?.(synthesizedOption, undefined)
                            }} />
                    </Stack>
                );
            }
            if (option.itemType === DropdownMenuItemType.Header) {
                return <div ref={el => groupRefs.current[option.key as string] = el} style={{ fontWeight: "bold", padding: "4px 8px" }}>{option.text}</div>;
            }

            return (
                // Due to some load testing, when we get 1000s of entires we don't want to bloat the load time with ToolTipHost construction.
                <div title={`Group: ${option.data?.group || ""}\n${option.text}\n${option.data?.tooltip}`} className={dropdownOptions.base}>{option.text}</div>
            );
        }, []);

    const handleDropdownChange = useCallback((_ev, option) => {
        if (option && option.itemType !== DropdownMenuItemType.Header) {
            onChange(option);
        }

    }, [onChange]);

    const handleMenuDismiss = useCallback(() => {
        setFilterText("");
        let buildHealthOptionStateReceipts = performanceTrendOptions.commitTransactionSession();
        handler.requestHierarchicalRefresh(request, undefined,
            // Upon completion of the hierarchical refresh, we want to commit the transaction session for the options. 
            // We do this because the data handler can populate the template & step cache which may be useful for any downstream receipt processing, and actions that depend on the side-effects of updated hierarchy data.
            (_request: DataHandlerRefreshRequest) => {
                if (onHierarchicalRefreshCompleted) {
                    onHierarchicalRefreshCompleted(buildHealthOptionStateReceipts);
                }
            });

    }, [request, performanceTrendOptions]);

    // #endregion -- Dropdown Callbacks --

    return (
        <SearchableDropdown
            id={changeVersion.toString() + key}
            key={key}
            placeholder={params.label}
            options={filteredOptions}
            multiSelect
            groupRefs={groupRefs}
            selectedKeys={selectedKeys}
            onChange={handleDropdownChange}
            onDismiss={handleMenuDismiss}
            onSearchValueChanged={(newValue) => {
                // This stems from a very awkward bug where we need to force the underlying dropdown to resynchronize 
                // on deletions by sending a empty filter - forcing the whole collection to be re-populated.
                // We also need to force a complete animation frame in order to propagate this through, resulting in the
                // underlying Dropdown component internal index synchronization.
                // Without doing so, upon "deletion" changes the onChange will receive the wrong DropdownOption.
                if (newValue.length < filterText.length) {
                    setFilterText("");
                    requestAnimationFrame(() => {
                        setFilterText(newValue);
                    });
                } else {
                    setFilterText(newValue);
                }
            }}
            onRenderOption={onRenderOption}
            styles={{
                dropdown: {
                    width: 300, maxWidth: 900,
                },
            }}
            disabled={disabled}
        />
    );
};