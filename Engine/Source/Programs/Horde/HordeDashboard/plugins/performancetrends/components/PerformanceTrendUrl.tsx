// Copyright Epic Games, Inc. All Rights Reserved.

import { useEffect } from "react";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { useLocation, useNavigate } from "react-router";
import { PerformanceTrendOptionsState } from "../filters/PerformanceTrendOptionsState";
import { GroupByOption, PerformanceTrendUIOptionsState } from "../filters/PerformanceTrendUIOptionsState";
import { IPerformanceTrendOptionWriter } from "../filters/PerformanceTrendOptionsWriter";
import { PerformanceTrendQueryParams, PerformanceTrendUIParams } from "../filters/PerformanceTrendUrlParams";

import LZString from "lz-string"
import { runInAction } from "mobx";
import { observer } from "mobx-react-lite";
import { DecodedPlatformKey, DecodedStreamKey, DecodedTestIdentityKey, DecodedTestTypeKey, decodeMetricTypeKey, decodePlatformKey, decodeStreamKey, decodeTestIdentity, decodeTestTypeKey, decodeViewablePropertyKey } from "../responses/FilterKeys";

// #region -- Consts --

// #region -- Options --

export const QUERY_SCHEMA_PARAM_NAME: string = "query_schema";
export const PROJECTS_URL_PARAM_NAME: string = "projects";
export const TEST_IDENTITIES_URL_PARAM_NAME: string = "testIdentities";
export const TEST_TYPES_URL_PARAM_NAME: string = "testTypes";
export const METRIC_SUMMARIES_URL_PARAM_NAME: string = "metricSummaries";
export const STREAMS_URL_PARAM_NAME: string = "streams";
export const PLATFORM_URL_PARAM_NAME: string = "platforms";
export const START_COMMIT_PARAM_NAME: string = "start_commit";
export const END_COMMIT_PARAM_NAME: string = "end_commit";

// #endregion -- Options --

// #region -- UI Options --

export const UI_CURVE_SMOOTHING_PARAM_NAME: string = "lineSmoothing";
export const UI_GROUP_BY_PARAM_NAME: string = "groupBy";
export const UI_VIEWABLE_PROPERTIES_PARAM_NAME: string = "viewProps";

// #endregion -- UI Options --

// #endregion -- Consts --

/**
 * React Component used to manage the URL query params, and refresh it based on build health options. This is separate from the main component to scope rerenders to solely this component.
 * @returns React Component.
 */
export const PerformanceTrendUrlSync: React.FC<{ buildHealthController: PerformanceTrendOptionsController; handler: PerformanceTrendOptionsDataHandler; }> = observer(function PerformanceTrendUrlSync({ buildHealthController, handler }) {
    const location = useLocation();
    const navigate = useNavigate();

    // We only ever receive from location search once.
    useEffect(() => {
        const initializeFromUrl = async () => {
            let searchParams: URLSearchParams = new URLSearchParams(location.search)
            const params = parseQueryParams(searchParams);
            const upgradedParams = upgradeQueryParams(params.performanceTrendQueryParams, buildHealthController.querySchemaVersion);

            await loadOptionsFromParams(upgradedParams, params.performanceTrendUIParams, buildHealthController, handler);
        };

        initializeFromUrl();
    }, []);

    useEffect(() => {
        const desiredSearch = toNavigationQuery(buildHealthController.state, buildHealthController.uiState, buildHealthController.querySchemaVersion, handler).toString();
        if (desiredSearch && location.search !== `?${desiredSearch}`) {
            navigate(`${location.pathname}?${desiredSearch}`, { replace: true });
        }
    }, [navigate, buildHealthController.optionsChangeVersion, buildHealthController.uiOptionsChangeVersion, location.pathname, location.search]);

    return null;
});

// #region -- Query Serialization & Deserialization --

/**
 * Parses the query params from the search url.
 * @param searchParams The search params to parse.
 * @returns The resulting query params.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export function parseQueryParams(searchParams: URLSearchParams): { performanceTrendQueryParams: PerformanceTrendQueryParams, performanceTrendUIParams: PerformanceTrendUIParams } {
    let testProjects: string[] = [];
    let testIdentities: string[] = [];
    let testTypes: string[] = [];
    let metricSummaries: string[] = [];
    let streams: string[] = [];
    let platforms: string[] = [];
    let startCommit: number | null = null;
    let endCommit: number | null = null;

    const getList = (key: string) => {
        const raw = searchParams.get(key);
        if (!raw) return [];

        const decompressed = LZString.decompressFromEncodedURIComponent(raw);
        if (!decompressed) return [];

        return decompressed.split(",").filter(Boolean);
    };

    const getBoolParam = (key: string): boolean => {
        return searchParams.get(key)?.toLowerCase() === "true";
    }

    const querySchemaVersion = Number(searchParams.get(QUERY_SCHEMA_PARAM_NAME)) || 1;

    testProjects = getList(PROJECTS_URL_PARAM_NAME);
    testIdentities = getList(TEST_IDENTITIES_URL_PARAM_NAME);
    testTypes = getList(TEST_TYPES_URL_PARAM_NAME);
    metricSummaries = getList(METRIC_SUMMARIES_URL_PARAM_NAME);
    streams = getList(STREAMS_URL_PARAM_NAME);
    platforms = getList(PLATFORM_URL_PARAM_NAME);

    let rawStartCommit = searchParams.get(START_COMMIT_PARAM_NAME);
    startCommit = rawStartCommit === null || rawStartCommit === "-1" ? null : Number(rawStartCommit);

    let rawEndCommit = searchParams.get(END_COMMIT_PARAM_NAME);
    endCommit = rawEndCommit === null || rawEndCommit === "-1" ? null : Number(rawEndCommit);

    const curveSmoothing = getBoolParam(UI_CURVE_SMOOTHING_PARAM_NAME);
    const groupByRaw = searchParams.get(UI_GROUP_BY_PARAM_NAME);
    const groupBy: GroupByOption = isValidGroupByOption(groupByRaw) ? groupByRaw : 'platform+stream';
    const viewableProperties = getList(UI_VIEWABLE_PROPERTIES_PARAM_NAME);

    return {
        performanceTrendQueryParams: { testProjects: testProjects, testIdentities: testIdentities, testTypes: testTypes, metricSummaries: metricSummaries, streams: streams, platforms: platforms, startCommit: startCommit, endCommit: endCommit, querySchemaVersion: querySchemaVersion },
        performanceTrendUIParams: { applyCurveSmoothing: curveSmoothing, groupBy: groupBy, viewableProperties: viewableProperties }
    };
}

const validGroupByOptions = ['platform', 'stream', 'testType', 'platform+stream', 'platform+testType', 'stream+testType', 'platform+stream+testType'] as const;

function isValidGroupByOption(value: string | null): value is GroupByOption {
    return value !== null && (validGroupByOptions as readonly string[]).includes(value);
}

/**
 * Upgrades the query params given the current schema versions.
 * @param params The params to consider upgrading.
 * @param currentSchemaVersion The current schema.
 * @returns The upgraded params.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export function upgradeQueryParams(params: PerformanceTrendQueryParams, currentSchemaVersion: number): PerformanceTrendQueryParams {
    return params;
}

/**
 * Loads options from query params into the options writer.
 * @param params The params to use to initialize the options system with.
 * @param uiParams The UI params to use to initialize the options system with.
 * @param options The options controller to perform the operations.
 * @param handler The options data handler to use in acquiring subsequent data.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health. 
 */
export async function loadOptionsFromParams(
    params: PerformanceTrendQueryParams,
    uiParams: PerformanceTrendUIParams,
    options: PerformanceTrendOptionsController,
    handler: PerformanceTrendOptionsDataHandler) {

    let optionsWriter: IPerformanceTrendOptionWriter = options.getTransactionSession();

    // Handle UI first
    options.setCurveGraphLine(uiParams.applyCurveSmoothing);
    options.setGroupByOption(uiParams.groupBy);

    // Load viewable properties from URL
    if (uiParams.viewableProperties.length > 0) {
        for (const compositeKey of uiParams.viewableProperties) {
            const decoded = decodeViewablePropertyKey(compositeKey);
            if (decoded) {
                options.toggleViewableProperty(compositeKey, decoded.propertyKey, true);
            }
        }
    }

    // We currently support single test project selection
    if (params.testProjects.length === 1) {
        runInAction(() => {
            optionsWriter.toggleSingleProject(params.testProjects[0], params.testProjects[0]);
        });
    }

    // We currently support single test identity selection
    if (params.testIdentities.length === 1) {
        runInAction(() => {
            let decodedTestIdentity: DecodedTestIdentityKey | null = decodeTestIdentity(params.testIdentities[0]);

            if (decodedTestIdentity !== null) {
                optionsWriter.toggleSingleTestIdentity(params.testIdentities[0], decodedTestIdentity.testIdentity);
            } else {
                console.warn(`[loadOptionsFromParams] Discarding option param for test identities: ${params.testIdentities[0]}`);
            }
        });
    }

    // We currently support single metric type selection
    if (params.metricSummaries.length === 1) {
        runInAction(() => {
            let decodedMetricType: { testProject: string, testIdentity: string, summaryType: string } | null = decodeMetricTypeKey(params.metricSummaries[0]);

            if (decodedMetricType !== null) {
                optionsWriter.toggleSingleMetricSummaryType(params.metricSummaries[0], decodedMetricType.summaryType);
            } else {
                console.warn(`[loadOptionsFromParams] Discarding option param for metric types: ${params.metricSummaries[0]}`);
            }
        });
    }

    // Support multiple test type selections
    if (params.testTypes.length > 0) {
        runInAction(() => {
            for (let idx: number = 0; idx < params.testTypes.length; ++idx) {
                let testType: string = params.testTypes[idx];
                const decodedTestType: DecodedTestTypeKey | null = decodeTestTypeKey(testType);

                if (decodedTestType !== null) {
                    optionsWriter.toggleTestType(testType, decodedTestType.testType, true);
                } else {
                    console.warn(`[loadOptionsFromParams] Discarding option param for test type: ${testType}`);
                }
            }
        });
    }

    if (params.streams.length >= 1) {
        runInAction(() => {
            for (let idx: number = 0; idx < params.streams.length; ++idx) {
                const decodedStream: DecodedStreamKey | null = decodeStreamKey(params.streams[idx]);

                if (decodedStream !== null) {
                    optionsWriter.toggleStream(params.streams[idx], decodedStream.stream, true);
                } else {
                    console.warn(`[loadOptionsFromParams] Discarding option param for stream: ${params.streams[idx]}`);
                }
            }
        });
    }

    if (params.platforms.length >= 1) {
        runInAction(() => {
            for (let idx: number = 0; idx < params.platforms.length; ++idx) {
                const decodedPlatform: DecodedPlatformKey | null = decodePlatformKey(params.platforms[idx]);

                if (decodedPlatform !== null) {
                    optionsWriter.togglePlatform(params.platforms[idx], decodedPlatform.platform, true);
                } else {
                    console.warn(`[loadOptionsFromParams] Discarding option param for platform: ${params.platforms[idx]}`);
                }
            }
        });
    }

    if (params.startCommit !== null) {
        runInAction(() => {
            optionsWriter.toggleStartCommitRange(params.startCommit!);
        });
    }

    if (params.endCommit !== null) {
        runInAction(() => {
            optionsWriter.toggleEndCommitRange(params.endCommit!);
        });
    }

    await handler.requestHierarchicalRefresh();

    options.commitTransactionSession();
}

/**
 * Serializes the options state into a navigation query.
 * @param optionsState The options to use for serialization.
 * @param uiOptionsState The ui options to use for serialization.
 * @param querySchemaVersion The current schema version.
 * @param optionsDataHandler The data handler to use in performing serialization.
 * @returns The serialized URLSearchParams that describes the option set.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export function toNavigationQuery(optionsState: PerformanceTrendOptionsState, uiOptionsState: PerformanceTrendUIOptionsState, querySchemaVersion: number, optionsDataHandler: PerformanceTrendOptionsDataHandler): URLSearchParams {
    const params = new URLSearchParams(location.search);
    let projects: string = "";
    let testIdentities: string = "";
    let testTypes: string = "";
    let metricSummaries: string = "";
    let streams: string = "";
    let platforms: string = "";

    projects = Object.keys(optionsState.enabledProjects).join(",");
    testIdentities = Object.keys(optionsState.enabledTestIdentities).join(",");
    testTypes = Object.keys(optionsState.enabledTestTypes).join(",");
    metricSummaries = Object.keys(optionsState.enabledMetricSummaryTypes).join(",");
    streams = Object.keys(optionsState.enabledStreams).join(",");
    platforms = Object.keys(optionsState.enabledPlatforms).join(",");

    params.set(PROJECTS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(projects));
    params.set(TEST_IDENTITIES_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(testIdentities));
    params.set(TEST_TYPES_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(testTypes));
    params.set(METRIC_SUMMARIES_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(metricSummaries));
    params.set(STREAMS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(streams));
    params.set(PLATFORM_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(platforms));
    params.set(QUERY_SCHEMA_PARAM_NAME, querySchemaVersion.toString());

    params.set(START_COMMIT_PARAM_NAME, optionsState.enabledStartCommit !== null ? optionsState.enabledStartCommit.toString() : "-1");
    params.set(END_COMMIT_PARAM_NAME, optionsState.enabledEndCommit !== null ? optionsState.enabledEndCommit.toString() : "-1");

    // Handle UI Option State
    params.set(UI_CURVE_SMOOTHING_PARAM_NAME, String(uiOptionsState.curveGraphLine));
    params.set(UI_GROUP_BY_PARAM_NAME, uiOptionsState.groupByOption);

    // Serialize viewable properties
    const viewableProperties = Object.keys(uiOptionsState.enabledViewableProperties).join(",");
    if (viewableProperties) {
        params.set(UI_VIEWABLE_PROPERTIES_PARAM_NAME, LZString.compressToEncodedURIComponent(viewableProperties));
    } else {
        params.delete(UI_VIEWABLE_PROPERTIES_PARAM_NAME);
    }

    return params;
}

// #endregion -- Query Serialization & Deserialization --