// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption, SpinnerSize, Spinner } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { DataHandlerRefreshRequest, expandRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionList, PerformanceTrendOptionData, BasicListParameter } from "./DropdownComponent";
import { GetTestProjectPlatformResponse } from "../responses/GetTestProjectResponse";
import { useCallback } from "react";
import { MultiListParameter } from "./MultilistDropdownComponent";
import { encodePlatformKey, KEY_SEPARATOR } from "../responses/FilterKeys";

/**
 * Component used to model all known platform types (for the currently selected test project, test identity, summary table), as a single select dropdown.
 * @return The react component.
 */
export const PlatformDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    if (handler.isLoadingPlatforms) {
        return (
            <Stack>
                <Label>Platform</Label>
                <Spinner size={SpinnerSize.small} label="Loading platforms for project..." />
            </Stack>
        );
    }

    const platformDropdownParams: PerformanceTrendOptionList = {
        label: "",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    handler.platforms.forEach((platform: GetTestProjectPlatformResponse) => {
        platform.platforms.forEach((platformStr: string) => {
            let listParam: PerformanceTrendOptionData = {
                id: encodePlatformKey(platform.owningTestProject, platformStr),
                text: platformStr,
            };
            platformDropdownParams.items.push(listParam);
        })
    });

    platformDropdownParams.items.sort((a, b) => a.text.localeCompare(b.text));

    let selectedPlatformKey = Object.keys(performanceTrendOptions.state.enabledPlatforms)[0] ?? null;
    let request = expandRefreshRequest({ commits: true });

    return platformDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Platform</Label>
            <BasicListParameter placeholder={"Select a platform"} handler={handler} performanceTrendOptions={performanceTrendOptions} params={platformDropdownParams} selectedKey={selectedPlatformKey} request={request} onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                const key = option.key as string;
                performanceTrendOptions.getTransactionSession().toggleSinglePlatform(key, option.text);
            }} />
        </Stack>
    );
});

/**
 * Component used to model all known platform types (for the currently selected test project, test identity, summary table), as a multi select dropdown.
 * @return The react component.
 */
export const PlatformDropdownMulti: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdown({ handler, performanceTrendOptions }) {
    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<PerformanceTrendOptionData>,
            optionSelectAllContext?: IDropdownOption<PerformanceTrendOptionData>[]
        ) => {
            const key = option.key as string;
            const [, projectId] = key.split(KEY_SEPARATOR);
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known platforms of the (project).
            const platformIds: { key: string, platform: string }[] =
                optionSelectAllContext === undefined
                    ? handler.platforms.flatMap(x => x.platforms.map(platformStr => ({ key: encodePlatformKey(x.owningTestProject, platformStr), platform: platformStr })))
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;

                            return (
                                opt.data.group === projectId
                            );
                        })
                        .map((x) => ({ key: x.data!.id, platform: x.data!.text! }));

           // Do not close the transaction session; the user still has the modal open
            performanceTrendOptions.getTransactionSession().toggleAllPlatforms(platformIds, isSelected);
        },
        [performanceTrendOptions, handler]
    );

    if (handler.isLoadingPlatforms) {
        return (
            <Stack>
                <Label>Platform</Label>
                <Spinner size={SpinnerSize.small} label="Loading platforms for project..." />
            </Stack>
        );
    }

    const platformDropdownParams: PerformanceTrendOptionList = {
        label: "Select Platform(s)",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    // Build a map of platform -> all streams for that platform
    const platformStreamMap = new Map<string, Set<string>>();
    for (const p of handler.platforms) {
        for (const platformStr of p.platforms) {
            if (!platformStreamMap.has(platformStr)) {
                platformStreamMap.set(platformStr, new Set<string>());
            }
            platformStreamMap.get(platformStr)!.add(p.owningTestProject.computedStream);
        }
    }

    // Build unique platform items by flattening all platform arrays
    const seenPlatforms = new Set<string>();
    const platformItems: PerformanceTrendOptionData[] = [];

    for (const p of handler.platforms) {
        for (const platformStr of p.platforms) {
            if (!seenPlatforms.has(platformStr)) {
                seenPlatforms.add(platformStr);
                const streams = platformStreamMap.get(platformStr);
                const streamList = streams ? Array.from(streams).sort().join('\n  - ') : '';

                platformItems.push({
                    id: encodePlatformKey(p.owningTestProject, platformStr),
                    text: platformStr,
                    group: p.owningTestProject.testName,
                    tooltip: streamList ? `Streams:\n  - ${streamList}` : undefined,
                });
            }
        }
    }

    platformDropdownParams.items = platformItems.sort((a, b) => a.text.localeCompare(b.text));

    let selectedPlatformKeys: string[] = Object.keys(performanceTrendOptions.state.enabledPlatforms);
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ commits: true });

    return platformDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Platform(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} performanceTrendOptions={performanceTrendOptions} params={platformDropdownParams} selectedKeys={selectedPlatformKeys} request={request}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                    onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                        const key = option.key as string;
                        const isSelected = !!option.selected;

                        performanceTrendOptions.getTransactionSession().togglePlatform(key, option.text, isSelected);
                    }} />
            </Stack>
        </Stack>
    );
});