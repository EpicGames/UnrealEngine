import { IDropdownOption, Stack, Label } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { useCallback } from "react";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { PerformanceTrendOptionsDataHandler, DataHandlerRefreshRequest, expandRefreshRequest } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionData, PerformanceTrendOptionList } from "./DropdownComponent";
import { MultiListParameter } from "./MultilistDropdownComponent";
import { KEY_SEPARATOR, encodeStreamKey } from "../responses/FilterKeys";

/**
 * Component used to model all known streams(for the currently selected test project, test identity, summary table), as a single select dropdown.
 * @return The react component.
 */
export const StreamDropdownMulti: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdown({ handler, performanceTrendOptions }) {
    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<PerformanceTrendOptionData>,
            optionSelectAllContext?: IDropdownOption<PerformanceTrendOptionData>[]
        ) => {
            const key = option.key as string;
            const [, projectId] = key.split(KEY_SEPARATOR);
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known unique streams of the (project).
            const seenStreamsInSelectAll = new Set<string>();
            const uniqueStreams = handler.streams.filter(x => {
                if (seenStreamsInSelectAll.has(x.computedStream)) {
                    return false;
                }
                seenStreamsInSelectAll.add(x.computedStream);
                return true;
            });
            const streamIds: { key: string, stream: string }[] =
                optionSelectAllContext === undefined
                    ? uniqueStreams.map(x => ({ key: encodeStreamKey(x), stream: x.computedStream }))
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;

                            return (
                                opt.data.group === projectId
                            );
                        })
                        .map((x) => ({ key: x.data!.id, stream: x.data!.text! }));

            // Do not close the transaction session; the user still has the modal open
            performanceTrendOptions.getTransactionSession().toggleAllStreams(streamIds, isSelected);
        },
        [performanceTrendOptions, handler]
    );

    const streamDropdownParams: PerformanceTrendOptionList = {
        label: "Select Streams(s)",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    // Build a map of stream -> all test types for that stream
    const streamTestTypeMap = new Map<string, Set<string>>();
    for (const stream of handler.streams) {
        if (!streamTestTypeMap.has(stream.computedStream)) {
            streamTestTypeMap.set(stream.computedStream, new Set<string>());
        }
        streamTestTypeMap.get(stream.computedStream)!.add(stream.testType);
    }

    // Build unique stream items
    const seenStreams = new Set<string>();
    const streamItems: PerformanceTrendOptionData[] = [];

    for (const stream of handler.streams) {
        if (!seenStreams.has(stream.computedStream)) {
            seenStreams.add(stream.computedStream);
            const testTypes = streamTestTypeMap.get(stream.computedStream);
            const testTypeList = testTypes ? Array.from(testTypes).sort().join('\n  - ') : '';

            streamItems.push({
                id: encodeStreamKey(stream),
                text: stream.computedStream,
                group: stream.testName,
                tooltip: testTypeList ? `Test Types:\n  - ${testTypeList}` : undefined,
            });
        }
    }

    streamDropdownParams.items = streamItems.sort((a, b) => a.text.localeCompare(b.text));

    let selectedStreamKeys: string[] = Object.keys(performanceTrendOptions.state.enabledStreams);
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ platforms: true });

    return streamDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Stream(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} performanceTrendOptions={performanceTrendOptions} params={streamDropdownParams} selectedKeys={selectedStreamKeys} request={request}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                    onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                        const key = option.key as string;
                        const isSelected = !!option.selected;

                        performanceTrendOptions.getTransactionSession().toggleStream(key, option.text, isSelected);
                    }} />
            </Stack>
        </Stack>
    );
});