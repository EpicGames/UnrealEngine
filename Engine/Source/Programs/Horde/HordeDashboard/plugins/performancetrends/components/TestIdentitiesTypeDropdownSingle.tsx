// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { DataHandlerRefreshRequest, expandRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionList, PerformanceTrendOptionData, BasicListParameter } from "./DropdownComponent";
import { GetTestProjectResponse } from "../responses/GetTestProjectResponse";
import { encodeTestIdentity } from "../responses/FilterKeys";

/**
 * Component used to model all known test identities (for the currently selected test project), as a single select dropdown.
 * @return The react component.
 */
export const TestIdentitiesTypeDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    const testIdentityDropdownParams: PerformanceTrendOptionList = {
        label: "",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    testIdentityDropdownParams.items = handler.testIdentities.map((testIdentityParentProject: GetTestProjectResponse) => {
        let listParam: PerformanceTrendOptionData = {
            id: encodeTestIdentity(testIdentityParentProject),
            text: testIdentityParentProject.testIdentity,
        };

        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    let selectedTestIdentityKey = Object.keys(performanceTrendOptions.state.enabledTestIdentities)[0] ?? null;
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ metricTypes: true });

    return testIdentityDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Test Identity</Label>
            <BasicListParameter placeholder={"Select a test identity"} handler={handler} performanceTrendOptions={performanceTrendOptions} params={testIdentityDropdownParams} selectedKey={selectedTestIdentityKey} request={request} onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                const key = option.key as string;
                performanceTrendOptions.getTransactionSession().toggleSingleTestIdentity(key, option.text);
            }}
                selectorWidth={400} />
        </Stack>
    );
});