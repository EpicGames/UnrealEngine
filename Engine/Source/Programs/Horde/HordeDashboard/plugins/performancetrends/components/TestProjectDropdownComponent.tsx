// Copyright Epic Games, Inc. All Rights Reserved.

import { IDropdownOption, Label, Spinner, SpinnerSize, Stack } from "@fluentui/react";
import { BasicListParameter, PerformanceTrendOptionData, PerformanceTrendOptionList } from "./DropdownComponent";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { observer } from "mobx-react-lite";
import { DataHandlerRefreshRequest, expandRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";

/**
 * Component used to model all known test projects, as a single select dropdown.
 * @return The react component.
 */
export const TestProjectDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    if (handler.isLoadingProjects) {
        return (
            <Stack>
                <Label>Test Project</Label>
                <Spinner size={SpinnerSize.small} label="Loading projects..." />
            </Stack>
        );
    }

    const testProjectDropdownParams: PerformanceTrendOptionList = {
        label: "",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    const uniqueTestNames = Array.from(
        new Set(handler.projectsData.map(p => p.testName))
    );

    testProjectDropdownParams.items = uniqueTestNames
        .map(testName => ({
            id: testName,
            text: testName,
        }))
        .sort((a, b) => a.text.localeCompare(b.text));

    let selectedTestProject = Object.keys(performanceTrendOptions.state.enabledProjects)[0] ?? null;

    let request: DataHandlerRefreshRequest = expandRefreshRequest({ testIdentities: true });

    return (
        <Stack>
            <Label>Test Project</Label>
            <BasicListParameter placeholder={"Select a test project"} handler={handler} performanceTrendOptions={performanceTrendOptions} params={testProjectDropdownParams} selectedKey={selectedTestProject} request={request} onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                const key = option.key as string;
                performanceTrendOptions.getTransactionSession().toggleSingleProject(key, option.text);
            }} />
        </Stack>
    );
});