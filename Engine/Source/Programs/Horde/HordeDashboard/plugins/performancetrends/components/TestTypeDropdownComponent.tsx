// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption, Spinner, SpinnerSize } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { DataHandlerRefreshRequest, expandRefreshRequest, PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { PerformanceTrendOptionList, PerformanceTrendOptionData, BasicListParameter } from "./DropdownComponent";
import { GetTestProjectResponse } from "../responses/GetTestProjectResponse";
import { encodeTestTypeKey, KEY_SEPARATOR } from "../responses/FilterKeys";
import { useCallback } from "react";
import { MultiListParameter } from "./MultilistDropdownComponent";

/**
 * Component used to model all known test types (for the currently selected test project, and test identity), as a single select dropdown.
 * @return The react component.
 */
export const TestTypeDropdownSingle: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, performanceTrendOptions }) {
    const testTypeDropdownParams: PerformanceTrendOptionList = {
        label: "",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    testTypeDropdownParams.items = handler.testTypes.map((testType: GetTestProjectResponse) => {
        let listParam: PerformanceTrendOptionData = {
            id: encodeTestTypeKey(testType),
            text: testType.testType,
        };

        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    let selectedTestTypeKey = Object.keys(performanceTrendOptions.state.enabledTestTypes)[0] ?? null;
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ streams: true });

    return testTypeDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Test Type</Label>
            <BasicListParameter placeholder={"Select a test type"} handler={handler} performanceTrendOptions={performanceTrendOptions} params={testTypeDropdownParams} selectedKey={selectedTestTypeKey} request={request} onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                const key = option.key as string;
                performanceTrendOptions.getTransactionSession().toggleSingleTestType(key, option.text);
            }} />
        </Stack>
    );
});

/**
 * Component used to model all known test types (for the currently selected test project, test identity, summary table), as a multi select dropdown.
 * @return The react component.
 */
export const TestTypeDropdownMulti: React.FC<{ handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ConstructProjectDropdown({ handler, performanceTrendOptions }) {
    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<PerformanceTrendOptionData>,
            optionSelectAllContext?: IDropdownOption<PerformanceTrendOptionData>[]
        ) => {
            const key = option.key as string;
            const [, projectId] = key.split(KEY_SEPARATOR);
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known test types of the (project).
            const testTypeIds: { key: string, testType: string }[] =
                optionSelectAllContext === undefined
                    ? handler.testTypes.map(x => ({ key: encodeTestTypeKey(x), testType: x.testType }))
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;

                            return (
                                opt.data.group === projectId
                            );
                        })
                        .map((x) => ({ key: x.data!.id, testType: x.data!.text! }));

            // Do not close the transaction session; the user still has the modal open
            performanceTrendOptions.getTransactionSession().toggleAllTestTypes(testTypeIds, isSelected);
        },
        [performanceTrendOptions, handler]
    );

    const testTypeDropdownParams: PerformanceTrendOptionList = {
        label: "Select Test Type(s)",
        items: [
        ],
        tooltip: "Select an option from the dropdown."
    };

    testTypeDropdownParams.items = handler.testTypes.map((testType: GetTestProjectResponse) => {
        let listParam: PerformanceTrendOptionData = {
            id: encodeTestTypeKey(testType),
            group: testType.testName,
            text: testType.testType,
        };

        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    let selectedTestTypeKeys: string[] = Object.keys(performanceTrendOptions.state.enabledTestTypes);
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ streams: true });

    return testTypeDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Test Type(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} performanceTrendOptions={performanceTrendOptions} params={testTypeDropdownParams} selectedKeys={selectedTestTypeKeys} request={request}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                    onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                        const key = option.key as string;
                        const isSelected = !!option.selected;

                        performanceTrendOptions.getTransactionSession().toggleTestType(key, option.text, isSelected);
                    }} />
            </Stack>
        </Stack>
    );
});