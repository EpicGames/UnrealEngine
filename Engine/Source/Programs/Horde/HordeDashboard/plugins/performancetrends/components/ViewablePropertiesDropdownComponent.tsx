// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, IDropdownOption } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { PerformanceTrendOptionsController } from "../filters/PerformanceTrendOptionsController";
import { IMetricViewGenerator } from "../viewgenerators/PerformanceTrendRenderTypes";
import { MetricConstraint } from "../metrictypes/PerformanceTrendsTypes";
import { MultiListParameter } from "./MultilistDropdownComponent";
import { PerformanceTrendOptionList, PerformanceTrendOptionData } from "./DropdownComponent";
import { PerformanceTrendOptionsDataHandler } from "../filters/PerformanceTrendOptionsDataHandler";
import { useCallback, useEffect, useMemo } from "react";
import { decodeTestTypeKey, decodeViewablePropertyKey, encodeViewablePropertyKey, KEY_SEPARATOR } from "../responses/FilterKeys";

/**
 * Dropdown for selecting which metric properties to display in the view.
 * Groups properties by test type when multiple test types are selected.
 * Only visible when at least one test type is selected.
 */
export const ViewablePropertiesDropdown: React.FC<{ viewGenerator: IMetricViewGenerator<MetricConstraint> | undefined, handler: PerformanceTrendOptionsDataHandler, performanceTrendOptions: PerformanceTrendOptionsController }> = observer(function ViewablePropertiesDropdown({ viewGenerator, handler, performanceTrendOptions }) {

    // Get selected test types from published state (synchronized after commit)
    // Use optionsChangeVersion as dependency to properly react to MobX state changes
    const selectedTestTypes = useMemo(() => {
        const testTypeKeys = performanceTrendOptions.state.publishedEnabledTestTypeKeys;
        return testTypeKeys.map(key => {
            const decoded = decodeTestTypeKey(key);
            return decoded ? { key, testType: decoded.testType } : null;
        }).filter((x): x is { key: string, testType: string } => x !== null);
    }, [performanceTrendOptions.optionsChangeVersion]);

    // Build dropdown items from viewGenerator's viewable properties, grouped by test type
    const dropdownParams = useMemo((): PerformanceTrendOptionList => {
        const items: PerformanceTrendOptionData[] = [];

        if (viewGenerator && selectedTestTypes.length > 0) {
            const properties = viewGenerator.getViewableProperties();

            for (const { testType } of selectedTestTypes) {
                for (const [fieldName, meta] of properties) {
                    const compositeKey = encodeViewablePropertyKey(testType, fieldName);
                    items.push({
                        id: compositeKey,
                        text: meta.unit ? `${meta.label} (${meta.unit})` : meta.label,
                        group: testType,
                        tooltip: meta.category,
                    });
                }
            }
        }

        // Sort by group (test type) then by label
        items.sort((a, b) => {
            const groupCompare = (a.group ?? '').localeCompare(b.group ?? '');
            if (groupCompare !== 0) return groupCompare;
            return a.text.localeCompare(b.text);
        });

        return {
            label: "Select Properties",
            items,
            tooltip: "Select which properties to display in the graphs for each test type."
        };
    }, [viewGenerator, selectedTestTypes]);

    // Initialize default properties when test types change
    useEffect(() => {
        if (viewGenerator && selectedTestTypes.length > 0) {
            const defaultProps = viewGenerator.getDefaultVisibleProperties();
            for (const { testType } of selectedTestTypes) {
                performanceTrendOptions.initializeDefaultViewableProperties(testType, defaultProps);
            }
        }
    }, [viewGenerator, selectedTestTypes, performanceTrendOptions]);

    const handleSelectAll = useCallback(
        (option: IDropdownOption<PerformanceTrendOptionData>, optionSelectAllContext?: IDropdownOption<PerformanceTrendOptionData>[]) => {
            const isSelected = !!option.selected;
            const key = option.key as string;
            const group = key.split(KEY_SEPARATOR)[1];

            const properties: { key: string, propertyKey: string }[] = optionSelectAllContext === undefined
                ? dropdownParams.items.map(x => {
                    const decoded = decodeViewablePropertyKey(x.id);
                    return { key: x.id, testType: decoded?.testType, propertyKey: decoded?.propertyKey ?? '' };
                }).filter(x => x.testType === group)
                : optionSelectAllContext
                    .filter(opt => opt.data?.group === group)
                    .map(x => {
                        const decoded = decodeViewablePropertyKey(x.data!.id);
                        return { key: x.data!.id, propertyKey: decoded?.propertyKey ?? '' };
                    });

            performanceTrendOptions.toggleAllViewableProperties(properties, isSelected);
        },
        [performanceTrendOptions, dropdownParams]
    );

    // Don't show if no test types selected or no view generator
    if (!viewGenerator || selectedTestTypes.length === 0 || dropdownParams.items.length === 0) {
        return null;
    }

    const selectedKeys: string[] = Object.keys(performanceTrendOptions.uiState.enabledViewableProperties);

    return (
        <Stack>
            <Label>Visible Properties</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter
                    handler={handler}
                    performanceTrendOptions={performanceTrendOptions}
                    params={dropdownParams}
                    selectedKeys={selectedKeys}
                    request={{}} // No data refresh needed - view-only filter
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                    onChange={(option: IDropdownOption<PerformanceTrendOptionData>) => {
                        const key = option.key as string;
                        const decoded = decodeViewablePropertyKey(key);
                        if (decoded) {
                            const isSelected = !!option.selected;
                            performanceTrendOptions.toggleViewableProperty(key, decoded.propertyKey, isSelected);
                        }
                    }}
                />
            </Stack>
        </Stack>
    );
});
