// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { PerformanceTrendOptionsState } from "./PerformanceTrendOptionsState";
import { GroupByOption, PerformanceTrendUIOptionsState } from "./PerformanceTrendUIOptionsState";
import { PerformanceTrendOptionsDataHandler } from "./PerformanceTrendOptionsDataHandler";
import { IPerformanceTrendOptionWriter as IPerformanceTrendOptionWriter, PerformanceTrendOptionsStateDiff, PerformanceTrendsOptionsWriter as PerformanceTrendOptionsWriter } from "./PerformanceTrendOptionsWriter";

/**
 * Primary class used to interact with the performance trend option state.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export class PerformanceTrendOptionsController {
    // #region -- Private Members --

    readonly state: PerformanceTrendOptionsState;
    readonly uiState: PerformanceTrendUIOptionsState;
    private writer: IPerformanceTrendOptionWriter | null;
    private lastSynchronize = -1;

    // #endregion -- Private Members --

    // #region -- Public Members --

    @observable optionsChangeVersion = 0;
    @observable uiOptionsChangeVersion = 0;
    querySchemaVersion: number = 1;

    // #endregion -- Public Members --

    // #region -- Constructor --

    constructor(state: PerformanceTrendOptionsState, uiState: PerformanceTrendUIOptionsState) {
        this.state = state;
        this.uiState = uiState;

        makeObservable(this);
    }

    // #endregion -- Constructor --

    // #region -- Public Api --

    /**
     * Sets whether to use curved graph lines, or not.
     * @param isActive True to use curved graph lines, false otherwise.
     */
    @action
    setCurveGraphLine(isActive: boolean) {
        this.uiState.curveGraphLine = isActive;
        this.setUIOptionsChanged();
    }

    /**
     * Sets the group by option for graph series.
     * @param option The group by option to use.
     */
    @action
    setGroupByOption(option: GroupByOption) {
        this.uiState.groupByOption = option;
        this.setUIOptionsChanged();
    }

    /**
     * Toggles a viewable property on or off.
     * @param key The composite key "{testType}::{propertyKey}".
     * @param propertyKey The property key for display.
     * @param isSelected Whether to enable or disable.
     */
    @action
    toggleViewableProperty(key: string, propertyKey: string, isSelected: boolean) {
        if (isSelected) {
            this.uiState.enabledViewableProperties[key] = propertyKey;
        } else {
            delete this.uiState.enabledViewableProperties[key];
        }
        this.setUIOptionsChanged();
    }

    /**
     * Toggles all provided viewable properties.
     * @param properties Array of { key, propertyKey } to toggle.
     * @param isSelected Whether to enable or disable.
     */
    @action
    toggleAllViewableProperties(properties: { key: string, propertyKey: string }[], isSelected: boolean) {
        for (const prop of properties) {
            if (isSelected) {
                this.uiState.enabledViewableProperties[prop.key] = prop.propertyKey;
            } else {
                delete this.uiState.enabledViewableProperties[prop.key];
            }
        }
        this.setUIOptionsChanged();
    }

    /**
     * Initializes default viewable properties for a test type.
     * @param testType The test type.
     * @param defaultProperties Array of property keys that are visible by default.
     */
    @action
    initializeDefaultViewableProperties(testType: string, defaultProperties: string[]) {
        const prefix = `${testType}::`;
        // Only initialize if no properties are set for this test type
        const hasExisting = Object.keys(this.uiState.enabledViewableProperties).some(k => k.startsWith(prefix));
        if (!hasExisting) {
            for (const prop of defaultProperties) {
                this.uiState.enabledViewableProperties[`${prefix}${prop}`] = prop;
            }
            this.setUIOptionsChanged();
        }
    }

    /**
     * Starts a transaction session for updating the performance trend options.
     * @returns 
     */
    getTransactionSession(): IPerformanceTrendOptionWriter {
        if (!this.writer) {
            this.writer = new PerformanceTrendOptionsWriter(this.state, this.uiState);
        }

        return this.writer;
    }

    /**
     * Completes a transaction session for updating the performance trend options.
     * @returns 
     */
    commitTransactionSession(): PerformanceTrendOptionsStateDiff {
        if (this.writer === null || this.writer === undefined) {
            return {
                added: undefined,
                removed: undefined,
            };
        }

        let finalReceipts = this.writer.produceReceiptDiff();
        this.writer = null;

        // Only request a synchronization if we have produced diffs in our receipt.
        if (finalReceipts.hasDiff) {
            this.setOptionsChanged();
            this.synchronizeDerivedKeys();
        }

        return finalReceipts;
    }

    // #endregion -- Public Api --

    // #region -- Private Api --

    /**
     * Synchronize the options that are bound to UI elements, with the data observed by consumers of the option set.
     * This is important to keep separate as it allows us to control when we signal/flush a finalized set of options to the consumers.
     */
    @action
    private synchronizeDerivedKeys() {
        if (this.lastSynchronize < this.optionsChangeVersion) {
            this.state.publishedEnabledTemplateKeys = Object.keys(this.state.enabledTemplates);
            this.state.publishedEnabledProjectKeys = Object.keys(this.state.enabledProjects);
            this.state.publishedEnabledTestIdentityKeys = Object.keys(this.state.enabledTestIdentities);
            this.state.publishedEnabledTestTypeKeys = Object.keys(this.state.enabledTestTypes);
            this.state.publishedEnabledMetricSummaryTypes = Object.keys(this.state.enabledMetricSummaryTypes);
            this.state.publishedEnabledStreamKeys = Object.keys(this.state.enabledStreams);
            this.state.publishedEnabledPlatforms = Object.keys(this.state.enabledPlatforms);
            this.state.publishedEnabledStartCommit = this.state.enabledStartCommit;
            this.state.publishedEnabledEndCommit = this.state.enabledEndCommit;
            this.lastSynchronize = this.optionsChangeVersion;
        }
    }

    @action
    private setOptionsChanged() {
        this.optionsChangeVersion++;
    }

    @action
    private setUIOptionsChanged() {
        this.uiOptionsChangeVersion++;
    }

    // #endregion -- Private Api --
}