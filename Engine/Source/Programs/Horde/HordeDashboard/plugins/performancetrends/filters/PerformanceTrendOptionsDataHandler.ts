// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable, runInAction } from "mobx";
import { PerformanceTrendOptionsState } from "./PerformanceTrendOptionsState";
import { PerformanceTrendOptionsController } from "./PerformanceTrendOptionsController";
import { GetTestProjectCommitResponse, GetTestProjectPlatformResponse, GetTestProjectResponse } from "../responses/GetTestProjectResponse";
import { decodeTestTypeKey, decodeMetricTypeKey, DecodedTestIdentityKey, decodeTestIdentity, decodeStreamKey, DecodedStreamKey, decodePlatformKey, DecodedPlatformKey } from "../responses/FilterKeys";
import { getCommits, getPlatforms, getTestProjects } from "../api";

// #region -- Data Refresh Request --

/**
 * Models a request to the data handler.
 */
export type DataHandlerRefreshRequest = {
    projects?: boolean;
    testIdentities?: boolean;
    testTypes?: boolean;
    metricTypes?: boolean;
    streams?: boolean;
    platforms?: boolean;
    commits?: boolean;
}

export const EMPTY_REFRESH_REQUEST: DataHandlerRefreshRequest = { projects: false, testIdentities: false, testTypes: false, metricTypes: false, streams: false, platforms: false, commits: false };
export const DEFAULT_REFRESH_REQUEST: DataHandlerRefreshRequest = { projects: true, testIdentities: true, testTypes: true, metricTypes: true, streams: true, platforms: true, commits: true };

/**
 * Utility function that will expand a Data Handler Refresh Request to include all the dependents in it's refresh.
 * @param request The request to expand.
 * @returns The expanded request.
 */
export function expandRefreshRequest(request: DataHandlerRefreshRequest): DataHandlerRefreshRequest {
    const expanded: DataHandlerRefreshRequest = { ...request };

    if (request.projects) {
        expanded.testIdentities = true;
    }

    if (expanded.testIdentities) {
        expanded.testTypes = true;
    }

    if (expanded.testTypes) {
        expanded.metricTypes = true;
    }

    if (expanded.streams) {
        expanded.metricTypes = true;
    }

    if (expanded.metricTypes) {
        expanded.streams = true;
    }

    if (expanded.streams) {
        expanded.platforms = true;
    }

    if (expanded.platforms) {
        expanded.commits = true;
    }

    return expanded;
}

// #endregion -- Data Refresh Request --

/**
 * Data handler responsible for obtaining data used in options controls.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export class PerformanceTrendOptionsDataHandler {
    // #region -- Private Members --

    private lastRefresh = -1;
    private performanceTrendOptionsState: PerformanceTrendOptionsState;
    private performanceTrendOptionsController: PerformanceTrendOptionsController;

    // #endregion -- Private Members --

    // #region -- Public Members --

    @observable
    isLoadingProjects: boolean = false;

    @observable
    isLoadingPlatforms: boolean = false;

    @observable
    isLoadingCommits: boolean = false;

    @observable.shallow
    projectsData: GetTestProjectResponse[] = [];

    @observable.shallow
    testIdentities: GetTestProjectResponse[] = [];

    @observable.shallow
    testTypes: GetTestProjectResponse[] = [];

    @observable.shallow
    metricSummaryTypes: GetTestProjectResponse[] = [];

    @observable.shallow
    streams: GetTestProjectResponse[] = [];

    @observable.shallow
    platforms: GetTestProjectPlatformResponse[] = [];

    @observable.shallow
    commits: GetTestProjectCommitResponse[] = [];

    // #endregion -- Public Members --

    // #region -- Constructor --

    constructor(performanceTrendOptionsState: PerformanceTrendOptionsState, performanceTrendOptionsController: PerformanceTrendOptionsController) {
        makeObservable(this);

        this.performanceTrendOptionsState = performanceTrendOptionsState;
        this.performanceTrendOptionsController = performanceTrendOptionsController;
    }

    // #endregion -- Constructor --

    // #region -- Private API --

    private async refreshData(request: DataHandlerRefreshRequest) {
        return this.getProjectsData(request);
    }

    // #endregion -- Private API --

    // #region -- Public API --

    /**
     * Requests a full option hierarchy refresh.
     * @param request The request which dictates the degree of refresh.
     * @param force Whether to force the operation or not.
     * @param onComplete The delegate to invoke upon completion of data refresh.
     */
    async requestHierarchicalRefresh(request: DataHandlerRefreshRequest = DEFAULT_REFRESH_REQUEST, force?: boolean, onComplete?: (request: DataHandlerRefreshRequest) => void) {
        if (this.lastRefresh < this.performanceTrendOptionsController.optionsChangeVersion || force) {
            this.lastRefresh = this.performanceTrendOptionsController.optionsChangeVersion;
            await this.refreshData(request);
            onComplete?.(request);
        }
    }

    /**
     * Resets the handler, returning it to a project-only data state.
     */
    @action
    reset() {
        this.projectsData = [];
        this.testIdentities = [];
        this.metricSummaryTypes = [];
        this.testTypes = [];
        this.streams = [];
        this.platforms = [];
        this.commits = [];
    }

    /**
     * Hierarchical reset of the options.
     * @param request The request to use in processing the reset.
     */
    @action
    hierarchyReset(request: DataHandlerRefreshRequest) {
        let cascade = false;

        if (request.projects || cascade) {
            this.projectsData = [];
            cascade = true;
        }
        if (request.testIdentities || cascade) {
            this.testIdentities = [];
            cascade = true;
        }
        if (request.metricTypes || cascade) {
            this.metricSummaryTypes = [];
            cascade = true;
        }
        if (request.testTypes || cascade) {
            this.testTypes = [];
            cascade = true;
        }
        if (request.streams || cascade) {
            this.streams = [];
            cascade = true;
        }
        if (request.platforms || cascade) {
            this.platforms = [];
            cascade = true;
        }
        if (request.commits || cascade) {
            this.commits = [];
        }
    }

    /**
     * Obtains the test project data given the current performance trend options.
     * @param request The data refresh request message to use in order to control refresh depth.
     */
    async getProjectsData(request: DataHandlerRefreshRequest) {
        if (this.projectsData.length === 0 || request.projects) {
            runInAction(() => {
                this.isLoadingProjects = true;
            });

            const projectDataResult = (await getTestProjects());

            runInAction(() => {
                this.projectsData = projectDataResult;
                this.isLoadingProjects = false;
            });
        }

        return this.getTestIdentityData(request);
    }

    /**
     * Obtains the test identity data given the currently enabled test projects.
     * @param request The refresh request to cascade to downstream metric types.
     */
    async getTestIdentityData(request: DataHandlerRefreshRequest) {
        if (request.testIdentities && Object.keys(this.performanceTrendOptionsState.enabledProjects).length > 0) {
            runInAction(() => {
                this.testIdentities = [];
            });

            // For all of our enabled projects, add test identities that belong to them to the selection set.
            for (const key in this.performanceTrendOptionsState.enabledProjects) {
                let testIdentityData: GetTestProjectResponse[] = [];
                let testIdentityTargetSet: Set<string> = new Set<string>();

                // Make sure we are inserting test identities that falls under the current enabled project selection
                this.projectsData.forEach((value: GetTestProjectResponse) => {
                    if (key === value.testName && !testIdentityTargetSet.has(value.testIdentity)) {
                        testIdentityData.push(value);
                        testIdentityTargetSet.add(value.testIdentity);
                    }
                });
                runInAction(() => {
                    this.testIdentities.push(...testIdentityData);
                });
            }
        }

        return this.getMetricTypeData(request);
    }

    /**
    * Obtains the metric types given the currently enabled test projects, and test identities.
    * @param request The refresh request to cascade to downstream streams.
    */
    async getMetricTypeData(request: DataHandlerRefreshRequest) {
        if (request.metricTypes && Object.keys(this.performanceTrendOptionsState.enabledTestIdentities).length > 0) {
            runInAction(() => {
                this.metricSummaryTypes = [];
            });

            // For all of our enabled test types, add metric summary types that belong to them.
            for (const key in this.performanceTrendOptionsState.enabledTestIdentities) {
                let metricTypeData: GetTestProjectResponse[] = [];
                let metricTypeTargetSet: Set<string> = new Set<string>();
                let decodedKey = decodeTestIdentity(key);

                // Make sure we are inserting metric data that falls under the current enabled project selection
                this.projectsData.forEach((value: GetTestProjectResponse) => {
                    if (decodedKey?.testProject === value.testName && decodedKey?.testIdentity === value.testIdentity && !metricTypeTargetSet.has(value.summaryType)) {
                        metricTypeData.push(value);
                        metricTypeTargetSet.add(value.summaryType);
                    }
                });

                runInAction(() => {
                    this.metricSummaryTypes = metricTypeData;
                });
            }
        }

        return this.getTestTypeData(request);
    }

    /**
    * Obtains the test identity data given the currently enabled test projects.
    * @param request The refresh request to cascade to downstream metric types.
    */
    async getTestTypeData(request: DataHandlerRefreshRequest) {
        if (request.testTypes && Object.keys(this.performanceTrendOptionsState.enabledMetricSummaryTypes).length > 0) {
            runInAction(() => {
                this.testTypes = [];
            });

            // For all of our enabled projects, add test types that belong to them to the selection set.
            for (const key in this.performanceTrendOptionsState.enabledMetricSummaryTypes) {
                let testTypeData: GetTestProjectResponse[] = [];
                let testTypeTargetSet: Set<string> = new Set<string>();
                let decodedMetricSummaryType = decodeMetricTypeKey(key);

                // Make sure we are inserting test types that falls under the current enabled project selection
                this.projectsData.forEach((value: GetTestProjectResponse) => {
                    if (decodedMetricSummaryType?.testProject === value.testName && decodedMetricSummaryType?.testIdentity === value.testIdentity && decodedMetricSummaryType?.summaryType === value.summaryType && !testTypeTargetSet.has(value.testType)) {
                        testTypeData.push(value);
                        testTypeTargetSet.add(value.testType);
                    }
                });
                runInAction(() => {
                    this.testTypes.push(...testTypeData);
                });
            }
        }

        return this.getStreams(request);
    }

    /**
    * Obtains the streams given the currently enabled test projects, test identities, and metric types.
    * @param request The refresh request to cascade to downstream platforms.
    */
    async getStreams(request: DataHandlerRefreshRequest) {
        if (request.streams && Object.keys(this.performanceTrendOptionsState.enabledTestTypes).length > 0) {
            runInAction(() => {
                this.streams = [];
            });

            let streamData: GetTestProjectResponse[] = [];

            // For all of our enabled test types, add streams that belong to them (and the other related ancestors).
            for (const key in this.performanceTrendOptionsState.enabledTestTypes) {
                let singleStreamData: GetTestProjectResponse[] = [];
                const streamTargetSet: Set<string> = new Set<string>();
                const decodedKey = decodeTestTypeKey(key);

                // Make sure we are inserting metric data that falls under the current enabled project selection
                this.projectsData.forEach((value: GetTestProjectResponse) => {
                    if (decodedKey?.metricSummaryType === value.summaryType && decodedKey?.testProject === value.testName && decodedKey?.testIdentity === value.testIdentity && decodedKey?.testType === value.testType && !streamTargetSet.has(value.computedStream)) {
                        singleStreamData.push(value);
                        streamTargetSet.add(value.computedStream);
                    }
                });

                streamData.push(...singleStreamData);
            }

            runInAction(() => {
                this.streams = streamData;
            });
        }

        return this.getPlatforms(request);
    }

    /**
    * Obtains the platforms with valid results, given the currently enabled test projects, test identities, and summary type.
    * @param request The refresh request to cascade to downstream metric types.
    */
    async getPlatforms(request: DataHandlerRefreshRequest) {
        if (request.platforms && Object.keys(this.performanceTrendOptionsState.enabledStreams).length > 0) {

            runInAction(() => {
                this.platforms = [];
            });

            const publishedEnabledProjectKeys = Object.keys(this.performanceTrendOptionsState.enabledProjects);
            const publishedEnabledTestIdentityKeys = Object.keys(this.performanceTrendOptionsState.enabledTestIdentities);
            const publishedEnabledMetricSummaryTypes = Object.keys(this.performanceTrendOptionsState.enabledMetricSummaryTypes);
            const publishedEnabledStreamKeys = Object.keys(this.performanceTrendOptionsState.enabledStreams);

            const testProject = publishedEnabledProjectKeys[0];

            const decodedTestIdentity: DecodedTestIdentityKey | null = decodeTestIdentity(publishedEnabledTestIdentityKeys[0]);

            if (!decodedTestIdentity) {
                console.warn("[GetPlatforms] Was unable to obtain a test identity from the enabled test identity key.");

                this.hierarchyReset(expandRefreshRequest({ commits: true }));

                return;
            }

            let testIdentity = decodedTestIdentity.testIdentity;

            const decodedMetricSummaryType = decodeMetricTypeKey(publishedEnabledMetricSummaryTypes[0]);

            if (!decodedMetricSummaryType) {
                console.warn("[GetPlatforms] Was unable to obtain a metric summary type from the enabled metric summary type.");

                this.hierarchyReset(expandRefreshRequest({ commits: true }));

                return;
            }

            let metricSummaryType = decodedMetricSummaryType.summaryType;

            const decodedStreamKeys = publishedEnabledStreamKeys
                .map(x => decodeStreamKey(x))
                .filter((x: DecodedStreamKey | null): x is DecodedStreamKey => x !== null);

            if (decodedStreamKeys.length === 0) {
                console.warn("[GetPlatforms] Was unable to obtain a stream key from the enabled stream.");

                this.hierarchyReset(expandRefreshRequest({ commits: true }));

                return;
            }

            let streams = decodedStreamKeys.map(x => x.stream);

            runInAction(() => {
                this.isLoadingPlatforms = true;
            });

            const platformResult = (await getPlatforms(testProject, testIdentity, metricSummaryType, streams));

            runInAction(() => {
                this.isLoadingPlatforms = false;
                this.platforms = platformResult;
            });
        } else if (request.platforms) {
            runInAction(() => {
                this.platforms = [];
            });
        }

        return this.getCommits(request);
    }

    /**
    * Obtains the commits with valid results, given the currently enabled test projects, test identities, summary type, and platform.
    * @param request The refresh request to cascade to downstream metric types.
    */
    async getCommits(request: DataHandlerRefreshRequest) {
        if (request.commits && Object.keys(this.performanceTrendOptionsState.enabledPlatforms).length > 0) {
            runInAction(() => {
                this.commits = [];
            });

            const publishedEnabledProjectKeys = Object.keys(this.performanceTrendOptionsState.enabledProjects);
            const publishedEnabledTestIdentityKeys = Object.keys(this.performanceTrendOptionsState.enabledTestIdentities);
            const publishedEnabledMetricSummaryTypes = Object.keys(this.performanceTrendOptionsState.enabledMetricSummaryTypes);
            const publishedEnabledStreamKeys = Object.keys(this.performanceTrendOptionsState.enabledStreams);
            const publishedPlatformKeys = Object.keys(this.performanceTrendOptionsState.enabledPlatforms);

            const testProject = publishedEnabledProjectKeys[0];

            const decodedTestIdentity: DecodedTestIdentityKey | null = decodeTestIdentity(publishedEnabledTestIdentityKeys[0]);

            if (!decodedTestIdentity) {
                console.warn("[GetCommits] Was unable to obtain a test identity from the enabled test identity key.");

                return;
            }

            const testIdentity = decodedTestIdentity.testIdentity;

            const decodedMetricSummaryType = decodeMetricTypeKey(publishedEnabledMetricSummaryTypes[0]);

            if (!decodedMetricSummaryType) {
                console.warn("[GetCommits] Was unable to obtain a metric summary type from the enabled metric summary type.");

                return;
            }

            const metricSummaryType = decodedMetricSummaryType.summaryType;

            const decodedStreamKeys = publishedEnabledStreamKeys
                .map(x => decodeStreamKey(x))
                .filter((x: DecodedStreamKey | null): x is DecodedStreamKey => x !== null);

            if (decodedStreamKeys.length === 0) {
                console.warn("[GetCommits] Was unable to obtain a stream key from the enabled stream.");

                return;
            }

            const streams = decodedStreamKeys.map(x => x.stream);

            const decodedPlatformKeys = publishedPlatformKeys
                .map(x => decodePlatformKey(x))
                .filter((x: DecodedPlatformKey | null): x is DecodedPlatformKey => x !== null);

            if (decodedPlatformKeys.length === 0) {
                console.warn("[GetCommits] Was unable to obtain a platform key from the enabled platform.");

                return;
            }

            const platforms = decodedPlatformKeys.map(x => x.platform);

            runInAction(() => {
                this.isLoadingCommits = true;
            });

            const commitsResult = (await getCommits(testProject, testIdentity, metricSummaryType, streams, platforms));

            runInAction(() => {
                this.isLoadingCommits = false;
                this.commits = commitsResult;
            });
        } else if (request.commits) {
            runInAction(() => {
                this.commits = [];
            });
        }
    }

    // #endregion -- Public Api --
}