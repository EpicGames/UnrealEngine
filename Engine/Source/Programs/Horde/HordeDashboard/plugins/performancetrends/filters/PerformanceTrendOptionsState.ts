// Copyright Epic Games, Inc. All Rights Reserved.

import { makeObservable, observable } from "mobx";
import { PerformanceTrendOptionStateReceipt } from "./PerformanceTrendOptionsWriter";

/**
 * * The state object for query options on the Performance Trend dashboard.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export class PerformanceTrendOptionsState {

    // #region -- Published Options --

    @observable publishedEnabledProjectKeys: string[] = [];
    @observable publishedEnabledTestIdentityKeys: string[] = [];
    @observable publishedEnabledTestTypeKeys: string[] = [];
    @observable publishedEnabledMetricSummaryTypes: string[] = [];
    @observable publishedEnabledStartCommit: number | null = null;
    @observable publishedEnabledEndCommit: number | null = null;
    @observable publishedEnabledPlatforms: string[] = [];
    @observable publishedEnabledStreamKeys: string[] = [];
    @observable publishedEnabledTemplateKeys: string[] = [];

    // #endregion -- Published Options --

    // #region -- User Selected Options --

    @observable enabledProjects: Record<string, string> = {};
    @observable enabledTestIdentities: Record<string, string> = {};
    @observable enabledTestTypes: Record<string, string> = {};
    @observable enabledMetricSummaryTypes: Record<string, string> = {};
    @observable enabledPlatforms: Record<string, string> = {};
    @observable enabledStartCommit: number | null = null;
    @observable enabledEndCommit: number| null = null;
    @observable enabledStreams: Record<string, string> = {};
    @observable enabledTemplates: Record<string, string> = {};

    // #endregion -- User Selected Options --

    // #region -- Constructor --

    constructor() {
        makeObservable(this);
    }

    // #endregion -- Constructor --

    // #region -- Public Api --

    /**
     * Generates a state receipt for the options in it's current state.
     * @returns A receipt representation of the current performance trend options.
     */
    generateStateReceipt(): PerformanceTrendOptionStateReceipt {
        let receipt: PerformanceTrendOptionStateReceipt = new PerformanceTrendOptionStateReceipt(
            Object.keys(this.enabledProjects),
            Object.keys(this.enabledTestIdentities),
            Object.keys(this.enabledTestTypes),
            Object.keys(this.enabledMetricSummaryTypes),
            Object.keys(this.enabledPlatforms),
            this.enabledStartCommit,
            this.enabledEndCommit,
            Object.keys(this.enabledStreams),
            Object.keys(this.enabledTemplates)
        );

        return receipt;
    }

    // #endregion -- Public Api --
}