// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable } from "mobx";
import { PerformanceTrendOptionsState } from "./PerformanceTrendOptionsState";
import { PerformanceTrendUIOptionsState } from "./PerformanceTrendUIOptionsState";

/**
 * Interface that describes the difference between two option states.
 */
export interface PerformanceTrendOptionsStateDiff {
    added?: PerformanceTrendOptionStateReceipt;
    removed?: PerformanceTrendOptionStateReceipt;
    hasDiff?: boolean;
}

/**
 * Performance Trend Option state receipt.
 */
export class PerformanceTrendOptionStateReceipt {
    enabledProjects: ReadonlySet<string>;
    enabledTestIdentities: ReadonlySet<string>;
    enabledTestTypes: ReadonlySet<string>;
    enabledMetricSummaries: ReadonlySet<string>;
    enabledPlatforms: ReadonlySet<string>;
    enabledStartCommit: number | null;
    enabledEndCommit: number | null;
    enabledStreams: ReadonlySet<string>;
    enabledTemplates: ReadonlySet<string>;

    constructor(
        projects: Iterable<string> = [],
        testIdentities: Iterable<string> = [],
        testTypes: Iterable<string> = [],
        metricSummaries: Iterable<string> = [],
        enabledPlatforms: Iterable<string> = [],
        enabledStartCommit: number | null,
        enabledEndCommit: number | null,
        streams: Iterable<string> = [],
        templates: Iterable<string> = [],

    ) {
        this.enabledProjects = new Set(projects);
        this.enabledTestIdentities = new Set(testIdentities);
        this.enabledTestTypes = new Set(testTypes);
        this.enabledMetricSummaries = new Set(metricSummaries);
        this.enabledPlatforms = new Set(enabledPlatforms);
        this.enabledStartCommit = enabledStartCommit;
        this.enabledEndCommit = enabledEndCommit;
        this.enabledStreams = new Set(streams);
        this.enabledTemplates = new Set(templates);
    }

    /**
        * Produces a build health option state diff between two provided receipts.
        * @param prev The previous option state.
        * @param next The next option state.
        * @returns The diff between the two receipts.
        */
    static diff(
        prev: PerformanceTrendOptionStateReceipt,
        next: PerformanceTrendOptionStateReceipt
    ): PerformanceTrendOptionsStateDiff {

        const added = new PerformanceTrendOptionStateReceipt(
            PerformanceTrendOptionStateReceipt.difference(next.enabledProjects, prev.enabledProjects),
            PerformanceTrendOptionStateReceipt.difference(next.enabledTestIdentities, prev.enabledTestIdentities),
            PerformanceTrendOptionStateReceipt.difference(next.enabledTestTypes, prev.enabledTestTypes),
            PerformanceTrendOptionStateReceipt.difference(next.enabledMetricSummaries, prev.enabledMetricSummaries),
            PerformanceTrendOptionStateReceipt.difference(next.enabledPlatforms, prev.enabledPlatforms),
            next.enabledStartCommit,
            next.enabledEndCommit,
            PerformanceTrendOptionStateReceipt.difference(next.enabledStreams, prev.enabledStreams),
            PerformanceTrendOptionStateReceipt.difference(next.enabledTemplates, prev.enabledTemplates),
        );

        const removed = new PerformanceTrendOptionStateReceipt(
            PerformanceTrendOptionStateReceipt.difference(prev.enabledProjects, next.enabledProjects),
            PerformanceTrendOptionStateReceipt.difference(prev.enabledTestIdentities, next.enabledTestIdentities),
            PerformanceTrendOptionStateReceipt.difference(prev.enabledTestTypes, next.enabledTestTypes),
            PerformanceTrendOptionStateReceipt.difference(prev.enabledMetricSummaries, next.enabledMetricSummaries),
            PerformanceTrendOptionStateReceipt.difference(prev.enabledPlatforms, next.enabledPlatforms),
            prev.enabledStartCommit,
            prev.enabledEndCommit,
            PerformanceTrendOptionStateReceipt.difference(prev.enabledStreams, next.enabledStreams),
            PerformanceTrendOptionStateReceipt.difference(prev.enabledTemplates, next.enabledTemplates),
        );

        let startCommitChanged: boolean = next.enabledStartCommit !== prev.enabledStartCommit;
        let endCommitChanged: boolean = next.enabledEndCommit !== prev.enabledEndCommit;

        const hasDiff = this.receiptHasChanges(added) || this.receiptHasChanges(removed) || startCommitChanged || endCommitChanged;

        return { added, removed, hasDiff };
    }

    private static receiptHasChanges(r: PerformanceTrendOptionStateReceipt): boolean {
        return (
            r.enabledProjects.size > 0 ||
            r.enabledTestIdentities.size > 0 ||
            r.enabledTestTypes.size > 0 ||
            r.enabledMetricSummaries.size > 0 ||
            r.enabledPlatforms.size > 0 ||
            r.enabledStreams.size > 0 ||
            r.enabledTemplates.size > 0
        );
    }

    private static difference<T>(a: ReadonlySet<T>, b: ReadonlySet<T>): Set<T> {
        const out = new Set<T>();
        for (const item of a) {
            if (!b.has(item)) out.add(item);
        }
        return out;
    }
}

/**
 * Defines an interface to modify the performance trend options.
 */
export interface IPerformanceTrendOptionWriter {

    /**
     * Toggles a single test project to be used in the resulting view.
     * @param key The key.
     * @param testProjectName The name of the project.
     */
    toggleSingleProject(key: string, testProjectName: string): void;

    /**
     * Toggles a single test identity to be used in the resulting view. test identity belongs to a test project.
     * @param key The key.
     * @param testIdentity The name of the test identity.
     */
    toggleSingleTestIdentity(key: string, testIdentity: string): void;

    /**
     * Toggles a single metric summary type to be used in the resulting view. Metric summary type belongs to a test project & test identity.
     * @param key The key.
     * @param metricSummaryType The name of the metric summary type.
     */
    toggleSingleMetricSummaryType(key: string, metricSummaryType: string): void;

    /**
     * Toggles a single test type to be used in the resulting view. Test type belongs to a test project & test identity.
     * @param key The key.
     * @param testType The name of the test type.
     */
    toggleSingleTestType(key: string, testType: string): void;

    /**
     * Toggles a test type to be used in the resulting view. Test type belongs to a test project & test identity.
     * @param key The key.
     * @param testType The name of the test type.
     */
    toggleTestType(key: string, testType: string, isSelected: boolean): void;

    /**
     * Toggles all provided test types.
     * @param testTypes The list of test types to toggle.
     * @param isSelected Whether to enable the test type, or not.
     */
    toggleAllTestTypes(testTypes: { key: string, testType: string }[], isSelected: boolean): void;

    /**
     * Toggles a single stream to be used in the resulting view. Stream belongs to a test project, test identity, and metric summary type.
     * @param key The key.
     * @param stream The name of the stream.
     */
    toggleStream(key: string, stream: string, isSelected: boolean): void;

    /**
     * Toggles all provided streams.
     * @param streams The list of streams to toggle.
     * @param isSelected Whether to enable the stream, or not.
     */
    toggleAllStreams(streams: { key: string, stream: string }[], isSelected: boolean): void;

    /**
     * Toggles a single platform to be used in the resulting view. Platform belongs to a test project, test identity, and metric summary type.
     * @param key The key.
     * @param platform The name of the platform.
     */
    toggleSinglePlatform(key: string, platform: string): void;

    /**
     * Toggles a single platform to be used in the resulting view. Platform belongs to a test project, test identity, and metric summary type.
     * @param key The key.
     * @param platform The name of the platform.
     */
    togglePlatform(key: string, platform: string, isSelected: boolean): void;

    /**
     * Toggles all provided platforms.
     * @param platforms The list of platforms to toggle.
     * @param isSelected Whether to enable the platform, or not.
     */
    toggleAllPlatforms(platforms: { key: string, platform: string }[], isSelected: boolean): void;

    /**
     * Sets the start commit range to filter from.
     * @param commitIdOrdered The ordered commit id to start filtering from.
     */
    toggleStartCommitRange(commitIdOrdered: number): void;

    /**
     * Sets the end commit range to filter to.
     * @param commitIdOrdered The ordered commit id to end filtering on.
     */
    toggleEndCommitRange(commitIdOrdered: number): void;

    /**
     * Clears all the currently selected options.
     */
    clearAll(): void;

    /**
     * Clears all the currently selected test projects.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearTestProjects(cascade?: boolean): void;

    /**
     * Clears all the currently selected test identities.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearTestIdentities(cascade?: boolean): void;

    /**
     * Clears all the currently selected test types.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearTestTypes(cascade?: boolean): void;

    /**
     * Clears all the currently selected metric summary types.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearMetricSummaries(cascade?: boolean): void;

    /**
     * Clears all the currently selected streams.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearStreams(cascade?: boolean): void;

    /**
     * Clears all the currently selected platforms.
     * @param cascade Whether to cascade the clear operation down to the dependency options.
     */
    clearPlatforms(cascade?: boolean): void;

    /**
     * Produces the set of additions and removals for all options.
     */
    produceReceiptDiff(): PerformanceTrendOptionsStateDiff;
}

/**
 * Class that implements the performance trend option writer interface.
 */
export class PerformanceTrendsOptionsWriter implements IPerformanceTrendOptionWriter {

    constructor(state: PerformanceTrendOptionsState, uiState: PerformanceTrendUIOptionsState) {
        makeObservable(this);
        this.state = state;
        this.uiState = uiState;
        this.activeTransactionReceipt = this.state.generateStateReceipt();
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleSingleProject(key: string, testProjectName: string) {
        this.state.enabledProjects = { [key]: testProjectName };
        this.clearTestIdentities(true);
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleSingleTestIdentity(key: string, testIdentity: string) {
        this.state.enabledTestIdentities = { [key]: testIdentity };
        this.clearMetricSummaries(true);
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleSingleMetricSummaryType(key: string, metricSummaryType: string) {
        this.state.enabledMetricSummaryTypes = { [key]: metricSummaryType };
        this.clearTestTypes(true);
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleSingleTestType(key: string, testType: string) {
        this.state.enabledTestTypes = { [key]: testType };
        this.clearStreams(true);
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleTestType(key: string, testType: string, isSelected: boolean) {
        if (isSelected) {
            this.state.enabledTestTypes[key] = testType;
        } else {
            delete this.state.enabledTestTypes[key];
            this.clearStreams(true);
        }
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleAllTestTypes(testTypes: { key: string; testType: string; }[], isSelected: boolean): void {
        for (let idx: number = 0; idx < testTypes.length; ++idx) {
            this.toggleTestType(testTypes[idx].key, testTypes[idx].testType, isSelected);
        }
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleStream(key: string, stream: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledStreams[key] = stream;
        } else {
            delete this.state.enabledStreams[key];
            this.clearPlatforms(true);
        }
    }

    @action
    toggleAllStreams(streams: { key: string, stream: string }[], isSelected: boolean) {
        for (let idx: number = 0; idx < streams.length; ++idx) {
            this.toggleStream(streams[idx].key, streams[idx].stream, isSelected);
        }
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleSinglePlatform(key: string, platform: string) {
        this.state.enabledPlatforms = { [key]: platform };
    }

    /**
     * @inheritdoc 
     */
    @action
    togglePlatform(key: string, platform: string, isSelected: boolean) {
        if (isSelected) {
            this.state.enabledPlatforms[key] = platform;
        } else {
            delete this.state.enabledPlatforms[key];
            this.clearCommits(true);
        }
    }

    @action
    toggleAllPlatforms(platforms: { key: string, platform: string }[], isSelected: boolean) {
        for (let idx: number = 0; idx < platforms.length; ++idx) {
            this.togglePlatform(platforms[idx].key, platforms[idx].platform, isSelected);
        }
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleStartCommitRange(commitIdOrdered: number) {
        this.state.enabledStartCommit = commitIdOrdered;

        // clamp higher if smaller
        if (this.state.enabledEndCommit !== null && this.state.enabledStartCommit > this.state.enabledEndCommit) {
            this.state.enabledEndCommit = commitIdOrdered;
        }
    }

    /**
     * @inheritdoc 
     */
    @action
    toggleEndCommitRange(commitIdOrdered: number) {
        this.state.enabledEndCommit = commitIdOrdered;

        // clamp higher if smaller
        if (this.state.enabledStartCommit !== null && this.state.enabledEndCommit < this.state.enabledStartCommit) {
            this.state.enabledStartCommit = commitIdOrdered;
        }
    }

    /**
    * @inheritdoc
    */
    clearAll() {
        this.clearTestProjects(true);
    }

    /**
     * @inheritdoc
     */
    clearTestProjects(cascade?: boolean) {
        Object.keys(this.state.enabledProjects).forEach(key => {
            delete this.state.enabledProjects[key];
        });

        if (cascade) {
            this.clearTestIdentities(cascade);
        }
    }

    /**
     * @inheritdoc
     */
    clearTestIdentities(cascade?: boolean) {
        Object.keys(this.state.enabledTestIdentities).forEach(key => {
            delete this.state.enabledTestIdentities[key];
        });

        if (cascade) {
            this.clearMetricSummaries(cascade);
        }
    }

    /**
     * @inheritdoc
     */
    clearMetricSummaries(cascade?: boolean) {
        Object.keys(this.state.enabledMetricSummaryTypes).forEach(key => {
            delete this.state.enabledMetricSummaryTypes[key];
        });

        if (cascade) {
            this.clearTestTypes(cascade);
        }
    }

    /**
     * @inheritdoc
     */
    clearTestTypes(cascade?: boolean) {
        Object.keys(this.state.enabledTestTypes).forEach(key => {
            delete this.state.enabledTestTypes[key];
        });

        if (cascade) {
            this.clearStreams(cascade);
        }
    }

    /**
     * @inheritdoc
     */
    clearStreams(cascade?: boolean) {
        Object.keys(this.state.enabledStreams).forEach(key => {
            delete this.state.enabledStreams[key];
        });

        if (cascade) {
            this.clearPlatforms(cascade);
        }
    }

    /**
    * @inheritdoc
    */
    clearPlatforms(cascade?: boolean) {
        Object.keys(this.state.enabledPlatforms).forEach(key => {
            delete this.state.enabledPlatforms[key];
        });

        this.clearCommits(cascade);
    }

    /**
    * @inheritdoc
    */
    clearCommits(cascade?: boolean) {
        this.state.enabledStartCommit = null;
        this.state.enabledEndCommit = null;
    }

    produceReceiptDiff(): PerformanceTrendOptionsStateDiff {
        let transactionStartReceipt = this.state.generateStateReceipt();
        let returnSet = PerformanceTrendOptionStateReceipt.diff(this.activeTransactionReceipt, transactionStartReceipt);

        return returnSet;
    }

    readonly state: PerformanceTrendOptionsState;
    readonly uiState: PerformanceTrendUIOptionsState;
    private activeTransactionReceipt: PerformanceTrendOptionStateReceipt;
}