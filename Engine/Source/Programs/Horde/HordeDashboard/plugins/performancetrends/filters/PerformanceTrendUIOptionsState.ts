// Copyright Epic Games, Inc. All Rights Reserved.

import { makeObservable, observable } from "mobx";

export type GroupByOption =
    | 'platform'
    | 'stream'
    | 'testType'
    | 'platform+stream'
    | 'platform+testType'
    | 'stream+testType'
    | 'platform+stream+testType';

/**
 * The state object for UI options on the Performance Trend dashboard.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export class PerformanceTrendUIOptionsState {
    @observable curveGraphLine: boolean = false;
    @observable groupByOption: GroupByOption = 'platform+stream';

    /**
     * Enabled viewable properties keyed by "{testType}::{propertyKey}".
     * Value is the property key for display purposes.
     */
    @observable enabledViewableProperties: Record<string, string> = {};

    constructor() {
        makeObservable(this);
    }
}