// Copyright Epic Games, Inc. All Rights Reserved.

import { GroupByOption } from "./PerformanceTrendUIOptionsState";

/**
 * Type that describes the url query params supported by the Performance Trends system that effect querying.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export type PerformanceTrendQueryParams = {
    testProjects: string[];
    testIdentities: string[];
    testTypes: string[];
    metricSummaries: string[];
    streams: string[];
    platforms: string[];
    startCommit: number | null;
    endCommit: number | null;
    querySchemaVersion: number;
};

/**
 * * Type that describes the url query params supported by the Performance Trends system that effect UI preferences.
 * @todo This is a code-sharing candidate between Performance Trends and Build Health.
 */
export type PerformanceTrendUIParams = {
    applyCurveSmoothing: boolean;
    groupBy: GroupByOption;
    viewableProperties: string[];
}
