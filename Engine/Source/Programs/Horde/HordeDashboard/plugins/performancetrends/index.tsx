// Copyright Epic Games, Inc. All Rights Reserved.

import { MountType, registerHordePlugin } from "hordePlugins";
import { PerformanceTrendsView } from "./PerformanceTrendView";

registerHordePlugin({
    id: "performancetrends",
    routes: [
        { path: "performancetrends", element: <PerformanceTrendsView/> },
    ],
    mounts: [
        {
            type: MountType.TopNav,
            context: "Tools",
            text: "Analytics - Performance Trends",
            route: "/performancetrends"
        }
    ]
});