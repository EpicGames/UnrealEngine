// Copyright Epic Games, Inc. All Rights Reserved.

import { KeyStatsViewGenerator } from "../viewgenerators/KeyStatsViewGenerator";
import { ISummaryMetric, MetricTypeRegistry, PerformanceTrendContext } from "./PerformanceTrendsTypes";
import { viewable } from "./StatMetadata";

/**
 * A key stats metric data structure.
 */
export class KeyStatsData extends PerformanceTrendContext implements ISummaryMetric {

    // #region -- Public Properties --

    // Identification
    csvId: string;

    // Performance metrics
    @viewable({ label: 'GPU Time', category: 'Performance', unit: 'ms', defaultVisible: true, precision: 2 })
    gpuTimeAvg: number;

    @viewable({ label: 'Game Thread', category: 'Timing', unit: 'ms', defaultVisible: true, precision: 2 })
    gameThreadtimeAvg: number;

    @viewable({ label: 'Render Thread', category: 'Timing', unit: 'ms', defaultVisible: true, precision: 2 })
    renderThreadtimeAvg: number;

    @viewable({ label: 'Frame Time', category: 'Timing', unit: 'ms', defaultVisible: true, precision: 2 })
    frametimeAvg: number;

    @viewable({ label: 'Peak Physical Memory', category: 'Memory', unit: 'MB', defaultVisible: true, precision: 1 })
    physicalUsedMbMax: number;

    @viewable({ label: 'Hitches/Min', category: 'Performance', defaultVisible: true, precision: 1 })
    hitchesMin: number;

    @viewable({ label: 'Hitch Time', category: 'Performance', unit: '%', precision: 2 })
    hitchTimePercent: number;

    @viewable({ label: 'MVP', category: 'Performance', precision: 2 })
    mvp: number;

    @viewable({ label: 'Dynamic Resolution Avg', category: 'Resolution', unit: '%', precision: 1 })
    dynamicResolutionPercentageAvg: number;

    @viewable({ label: 'Dynamic Resolution Max', category: 'Resolution', unit: '%', precision: 1 })
    dynamicResolutionPercentageMax: number;

    @viewable({ label: 'VSM Nanite Resolution', category: 'Resolution', precision: 2 })
    dynamicResolutionVsmNanite: number;

    // #endregion -- Public Properties --

    // #region -- Implicit ISummaryMetricProvider interface --

    /**
     * @inheritdoc 
     */
    static readonly metricType = "KeyStats";

    /**
     * @inheritdoc 
     */
    constructor() {
        super();
    }

    // #endregion -- Implicit ISummaryMetricProvider interface --
};

MetricTypeRegistry.register(KeyStatsData, new KeyStatsViewGenerator());