// Copyright Epic Games, Inc. All Rights Reserved.

import { PerformanceTrendDataHandler } from "../PerformanceTrendsDataHandler";
import { IMetricViewGenerator } from "../viewgenerators/PerformanceTrendRenderTypes";

// #region -- Response Factory Types --

/**
 * Constant used to describe a placeholder non-resolution type.
 */
export const NO_RESOLVE_TYPE_FOUND: string = "no-resolved-type";

/**
 * Unified metric constraint which defines a performance trend context that is validateable, and initializable.
 * All metric types must extend PerformanceTrendContext to participate in the visualization system.
 */
export type MetricConstraint = PerformanceTrendContext & IValidatable & IInitializable;

// #region -- Metric Registry, Providers, and Factories --

/**
 * Facade pattern to simplify interaction. Use this to register a metrics type to participate in the Performance Trend Dashboard.
 */
export class MetricTypeRegistry {
    /**
     * Registers a metric type to participate in the metric retrieval, constrction, and visualization system of Performance Trends.
     * @param metricType The metric type to register.
     * @param viewGenerator The view generator instance to use in rendering metrics of @see metricType .
     */
    static register<T extends MetricConstraint>(metricType: ISummaryMetricProvider<T>, viewGenerator: IMetricViewGenerator<T>): void {
        MetricFactory.register(metricType);
        ViewGeneratorProvider.register(metricType, viewGenerator);
        PerformanceTrendHandlerProvider.register(metricType);
    }

    /**
     * Gets the data provider and view generator associated with the provided type, if it exists.
     * @param metricType The metric type to request a data provider and / or view generator for.
     * @returns The data provider and view generator for the requested type.
     */
    static get(metricType: string): { dataProvider: PerformanceTrendDataHandler<MetricConstraint> | undefined, viewGenerator: IMetricViewGenerator<MetricConstraint> | undefined } {
        return {
            dataProvider: PerformanceTrendHandlerProvider.get(metricType),
            viewGenerator: ViewGeneratorProvider.get(metricType)
        };
    }

    /**
     * Creates an instance of the requested type, given a raw payload.
     * @param metricType The name of the type to create.
     * @param raw The payload to use to initialize the object.
     * @returns A initialized instance of the metric, using the payload as the initialization context.
     */
    static create<T extends MetricConstraint>(metricType: string, raw: any): T | null {
        return MetricFactory.create<T>(metricType, raw);
    }
}

/**
 * Perofrmance Trend Handler Provider which is responsible for registering and obtaining data handlers based on metric type.
 */
class PerformanceTrendHandlerProvider {
    private static registry = new Map<string, PerformanceTrendDataHandler<MetricConstraint>>();

    /**
     * Registers a metric of a provided name, with a corresponding provider interface.
     * @param metric The metric type to associated with a provider.
     */
    static register<T extends MetricConstraint>(metric: ISummaryMetricProvider<T>): void {
        if (!this.registry.has(metric.metricType)) {
            this.registry.set(metric.metricType, new PerformanceTrendDataHandler<T>(metric));
        }
    }

    /**
     * Gets the data handler for the given metric type.
     * @param metricType The metric type key.
     * @returns The handler, or undefined if not registered.
     */
    static get(metricType: string): PerformanceTrendDataHandler<MetricConstraint> | undefined {
        return this.registry.get(metricType);
    }
}

/**
 * View Generator Provider which is responsible for registering and obtaining view generators based on metric type.
 */
class ViewGeneratorProvider {
    private static registry = new Map<string, IMetricViewGenerator<MetricConstraint>>();

    /**
     * Registers a metric of a provided name, with a corresponding generator interface.
     * @param metric The metric type to associate with a the generator.
     * @param generator The generator to associate with the metric type.
     */
    static register<T extends MetricConstraint>(metric: ISummaryMetricProvider<T>, generator: IMetricViewGenerator<T>
    ) {
        this.registry.set(metric.metricType, generator);
    }

    /**
     * Gets the view generator associated with a metric type.
     * @param metricType The name of the metric.
     * @returns An instance of the view generator if it exists, otherwise undefined.
     */
    static get(metricType: string): IMetricViewGenerator<MetricConstraint> | undefined {
        return this.registry.get(metricType);
    }
}

/**
 * Metric factory used for creating instances of requested types.
 */
class MetricFactory {
    private static registry = new Map<string, ISummaryMetricProvider<any>>();

    /**
     * Registers a metric of a provided name, with a corresponding construction interface.
     * @param ctor The ctor to use when requested for a type.
     */
    static register<T extends MetricConstraint>(ctor: ISummaryMetricProvider<T>) {
        this.registry.set(ctor.metricType, ctor);
    }

    /**
     * Creates an object of a specific type based off of the name.
     * @param typeName the type to create.
     * @param raw The raw representation of the data.
     * @returns The constructed object of the specific type @see T.
     */
    static create<T extends MetricConstraint>(typeName: string, raw: any): T | null {
        const ctor = this.registry.get(typeName);

        if (!ctor) {
            throw new Error(`No metric registered for type '${typeName}'`);
        }

        const instance: T = new ctor();

        Object.assign(instance, raw);
        instance.initialize(raw);

        if (!instance.validate()) {
            console.warn("Instance failed validation - discarding.");

            return null;
        }

        return instance;
    }
}

// #endregion -- Metric Registry, Providers, and Factories --

// #endregion -- Response Factory Types --

// #region -- Summary Metric Interfaces --

/**
 * An interface used to represent a summary metric.
 */
export interface ISummaryMetric {
    summaryName: string;
}

/**
 * An interface used to represent the validity of a type.
 */
export interface IValidatable {
    /**
     * Validates the object.
     * @returns True if this is a valid object, false otherwise.
     */
    validate(): boolean;
}

/**
 * An interface used to represent an initializable type.
 * This is necessary due to Object.assign as the base init for some types, and thus acts as a late init for non-property mapped fields.
 */
export interface IInitializable {
    /**
     * Late initialization function to handle more nuanced initialization.
     * @param raw Raw input used for specialized non-property-mapped initialization.
     */
    initialize(raw: any): void;
}

/**
 * An interface used to request and retrieve metrics.
 */
export interface ISummaryMetricProvider<T extends PerformanceTrendContext> {

    /**
     * Constructs a summary metric from a raw payload.
     */
    new(): T;

    /**
     * The metric type to request, and reconstruct.
     */
    readonly metricType: string;
}

// #endregion -- Summary Metric Interfaces --

// #region -- Shared Metric Abstract Classes  --

/**
 * Base context for all telemetry.
 */
export abstract class BaseContext implements IInitializable, IValidatable {
    eventName: string;
    schemaVersion: number;

    constructor() {
    }

    /**
     * @inheritdoc 
     */
    validate(): boolean {
        return true;
    }

    /**
     * @inheritdoc 
     */
    initialize(raw: any) { }
}

/**
 * Common context for correlatable metrics & telemetry.
 */
export abstract class CorrelatableContext extends BaseContext {
    sessionId?: string;
    sessionLabel?: string;

    constructor() {
        super();
    }
}

/**
 * Common context for Horde telemetry.
 */
export abstract class HordeContext extends CorrelatableContext {
    hordeUrlStr: string;
    streamId: string;
    jobId?: string;
    stepId?: string;
    commitId: string;
    commitIdOrdered: number;
    isBuildMachine: boolean;

    constructor() {
        super();
    }
}

/**
 * Common context for performance trend metrics & telemetry.
 */
export abstract class PerformanceTrendContext extends HordeContext implements ISummaryMetric {
    testId: string;
    testName: string;
    testConfigName: string;
    gauntletTestType: string;
    gauntletSubTest: string;
    summaryName: string;
    platform: string;
    collated: boolean;
    engineVersion: string;
    engineReleaseVersion: string;
    buildVersion: string;
    computedStream: string;
    startTimestamp: number;
    endTimestamp: number;

    constructor() {
        super()
    }

    /**
     * @inheritdoc 
     */
    override validate(): boolean {
        return this.summaryName !== undefined;
    }
}

// #endregion -- Shared Metric Abstract Classes  --