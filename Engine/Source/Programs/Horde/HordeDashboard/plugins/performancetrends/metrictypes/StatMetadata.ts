// Copyright Epic Games, Inc. All Rights Reserved.

export interface PropertyMeta {
    label: string;
    category?: string;
    defaultVisible?: boolean;
    unit?: string;
    precision?: number;
}

// Store metadata on the class constructor
const viewableRegistry = new WeakMap<Function, Map<string, PropertyMeta>>();

/**
 * Decorator to mark a property as viewable in the UI.
 * Works with legacy experimentalDecorators.
 */
export function viewable(meta: PropertyMeta): PropertyDecorator {
    return function (target: Object, propertyKey: string | symbol): void {
        const ctor = target.constructor;

        if (!viewableRegistry.has(ctor)) {
            viewableRegistry.set(ctor, new Map());
        }

        viewableRegistry.get(ctor)!.set(String(propertyKey), meta);
    };
}

/**
 * Get all viewable properties from a class.
 */
export function getViewableProperties(target: Function | object): Map<string, PropertyMeta> {
    const ctor = typeof target === 'function' ? target : target.constructor;
    return viewableRegistry.get(ctor) ?? new Map();
}

/**
 * Get viewable properties grouped by category.
 */
export function getViewablePropertiesByCategory(target: Function | object): Map<string, [string, PropertyMeta][]> {
    const properties = getViewableProperties(target);
    const grouped = new Map<string, [string, PropertyMeta][]>();

    for (const [fieldName, meta] of properties) {
        const category = meta.category ?? 'Other';
        if (!grouped.has(category)) {
            grouped.set(category, []);
        }
        grouped.get(category)!.push([fieldName, meta]);
    }

    return grouped;
}

/**
 * Get default visible property names.
 */
export function getDefaultVisibleProperties(target: Function | object): string[] {
    const properties = getViewableProperties(target);
    const result: string[] = [];

    for (const [fieldName, meta] of properties) {
        if (meta.defaultVisible === true) {
            result.push(fieldName);
        }
    }

    return result;
}