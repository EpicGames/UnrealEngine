// Copyright Epic Games, Inc. All Rights Reserved.

import { GetTestProjectResponse } from "./GetTestProjectResponse";

/**
 * Separates aggregate parts of key.
 */
export const KEY_SEPARATOR: string = "::";

/**
 * Decoded test identity key.
 */
export type DecodedTestIdentityKey =
    {
        testProject: string,
        testIdentity: string
    }

/**
 * Decoded stream key.
 */
export type DecodedStreamKey = {
    testProject: string,
    testIdentity: string,
    summaryType: string,
    stream: string
}

/**
 * Decoded platform key.
 */
export type DecodedPlatformKey = {
    testProject: string,
    testIdentity: string,
    summaryType: string,
    platform: string
}

/**
 * Decoded metric summary key.
 */
export type DecodedMetricSummaryKey = {
    testProject: string,
    testIdentity: string,
    summaryType: string
}

/**
 * Decoded test type key.
 */
export type DecodedTestTypeKey = {
    testProject: string,
    testIdentity: string,
    metricSummaryType: string,
    testType: string
}

/**
 * Decoded viewable property key.
 */
export type DecodedViewablePropertyKey = {
    testType: string,
    propertyKey: string
}

/**
 * Encodes a stream unique key based on the input @see GetTestProjectResponse .
 * @param testProject The input test project to base the stream key off of.
 * @returns The encoded stream key.
 */
export function encodeStreamKey(testProject: GetTestProjectResponse): string {
    return `${testProject.testName}${KEY_SEPARATOR}${testProject.testIdentity}${KEY_SEPARATOR}${testProject.summaryType}${KEY_SEPARATOR}${testProject.computedStream}`;
}

/**
 * Obtains the stream, metric summary type, test identity, and owning test project from an encoded input stream type key. 
 * @param encodedStreamKey The encoded platform key.
 * @returns The resulting test project, test identity, metric summary type, and platform of the encoded key.
 */
export function decodeStreamKey(encodedStreamKey: string): DecodedStreamKey | null {
    if (!encodedStreamKey || !encodedStreamKey.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid stream key: ${encodedStreamKey}`);

        return null;
    }

    const [testProject, testIdentity, summaryType, stream] = encodedStreamKey.split(KEY_SEPARATOR);
    return { testProject, testIdentity, summaryType, stream };
}

/**
 * Encodes a platform unique key based on the input @see GetTestProjectPlatformResponse .
 * @param testProject The input test project to base the platform key off of.
 * @returns The encoded platform key.
 */
export function encodePlatformKey(testProject: GetTestProjectResponse, platform: string): string {
    return `${testProject.testName}${KEY_SEPARATOR}${testProject.testIdentity}${KEY_SEPARATOR}${testProject.summaryType}${KEY_SEPARATOR}${platform}`;
}

/**
 * Obtains the platform, metric summary type, test identity, and owning test project from an encoded input platform type key.
 * @param encodedPlatformKey The encoded platform key.
 * @returns The resulting test project, test identity, metric summary type, and platform of the encoded key.
 */
export function decodePlatformKey(encodedPlatformKey: string): DecodedPlatformKey | null {
    if (!encodedPlatformKey || !encodedPlatformKey.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid platform key: ${encodedPlatformKey}`);

        return null;
    }

    const [testProject, testIdentity, summaryType, platform] = encodedPlatformKey.split(KEY_SEPARATOR);
    return { testProject, testIdentity, summaryType, platform };
}

/**
 * Encodes a test type unique key based on the input @see GetTestProjectResponse .
 * @param testProject The input test project to base the test type key off of.
 * @returns The encoded test type key.
 */
export function encodeTestTypeKey(testProject: GetTestProjectResponse): string {
    return `${testProject.testName}${KEY_SEPARATOR}${testProject.testIdentity}${KEY_SEPARATOR}${testProject.summaryType}${KEY_SEPARATOR}${testProject.testType}`;
}

/**
 * Obtains the test type, test identity, and owning test project from an encoded input test type key.
 * @param encodedTestType The encoded test type key.
 * @returns The resulting test project, test identity, and test type of the encoded key.
 */
export function decodeTestTypeKey(encodedTestType: string): DecodedTestTypeKey | null {
    if (!encodedTestType || !encodedTestType.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid test type key: ${encodedTestType}`);

        return null;
    }

    const [testProject, testIdentity, metricSummaryType, testType] = encodedTestType.split(KEY_SEPARATOR);
    return { testProject, testIdentity, metricSummaryType, testType };
}

/**
 * Encodes a metric summary type unique key based on the input @see GetTestProjectResponse .
 * @param testProject The input test project to base the metric summary type key off of.
 * @returns The encoded metric summary key.
 */
export function encodeMetricTypeKey(testProject: GetTestProjectResponse): string {
    return `${testProject.testName}${KEY_SEPARATOR}${testProject.testIdentity}${KEY_SEPARATOR}${testProject.summaryType}`;
}

/**
 * Obtains the metric summary type, test identity, and owning test project from an encoded input metric summary type key.
 * @param encodedMetricTypeKey The encoded metric summary type key.
 * @returns The resulting test project, test identity, and metric summary type of the encoded key.
 */
export function decodeMetricTypeKey(encodedMetricTypeKey: string): DecodedMetricSummaryKey | null {
    if (!encodedMetricTypeKey || !encodedMetricTypeKey.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid metric type key: ${encodedMetricTypeKey}`);

        return null;
    }

    const [testProject, testIdentity, summaryType] = encodedMetricTypeKey.split(KEY_SEPARATOR);
    return { testProject, testIdentity, summaryType };
}

/**
 * Encodes a test identity unique key based on the input @see GetTestProjectResponse .
 * @param testProject The input test project to base the test identity key off of.
 * @returns The encoded test identity key.
 */
export function encodeTestIdentity(testProject: GetTestProjectResponse): string {
    return `${testProject.testName}${KEY_SEPARATOR}${testProject.testIdentity}`;
}

/**
 * Obtains the test identity, and owning test project from an encoded input test identity key.
 * @param encodedTestIdentity The encoded test identity key.
 * @returns The resulting test project and test identity of the encoded key.
 */
export function decodeTestIdentity(encodedTestIdentity: string): DecodedTestIdentityKey | null {
    if (!encodedTestIdentity.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid test identity key: ${encodedTestIdentity}`);

        return null;
    }

    const [testProject, testIdentity] = encodedTestIdentity.split(KEY_SEPARATOR);
    return { testProject, testIdentity };
}

/**
 * Encodes a viewable property unique key based on test type and property key.
 * @param testType The test type (e.g., "PerfTest").
 * @param propertyKey The property key (e.g., "gpuTimeAvg").
 * @returns The encoded viewable property key.
 */
export function encodeViewablePropertyKey(testType: string, propertyKey: string): string {
    return `${testType}${KEY_SEPARATOR}${propertyKey}`;
}

/**
 * Obtains the test type and property key from an encoded viewable property key.
 * @param encodedViewablePropertyKey The encoded viewable property key.
 * @returns The resulting test type and property key, or null if invalid.
 */
export function decodeViewablePropertyKey(encodedViewablePropertyKey: string): DecodedViewablePropertyKey | null {
    if (!encodedViewablePropertyKey || !encodedViewablePropertyKey.includes(KEY_SEPARATOR)) {
        console.warn(`Invalid viewable property key: ${encodedViewablePropertyKey}`);

        return null;
    }

    const [testType, propertyKey] = encodedViewablePropertyKey.split(KEY_SEPARATOR);
    return { testType, propertyKey };
}