// Copyright Epic Games, Inc. All Rights Reserved.

/** 
 * Response describing a Performance Report Tool Test Project.
 * */
export type GetTestProjectResponse = {
    summaryType: string;
    testName: string;
    testIdentity: string;
    testType: string;
    computedStream: string;
}

/** 
 * Response describing a distinct platform entry for a Performance Report Tool Test Project entry.
 * */
export type GetTestProjectPlatformResponse = {
    owningTestProject: GetTestProjectResponse;
    platforms: string[];
}

/** 
 * Response describing a distinct commit entry for a Performance Report Tool Test Project's platform entry.
 * */
export type GetTestProjectCommitResponse = {
    owningTestProject: GetTestProjectResponse;
    commitIds: number[];
}