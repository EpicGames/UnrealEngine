// Copyright Epic Games, Inc. All Rights Reserved.

import { graphColors } from "hordePlugins/analytics/telemetryData";
import { LineGraph, MetricSelectionHandler, ReferenceLine, SeriesConfig } from "../components/LineGraphComponent";
import { KeyStatsData } from "../metrictypes/KeyStatsData";
import { IMetricDetailedView, IMetricView, IMetricViewGenerator, MetricReferencePoint, ReferenceThresholds, ThresholdMeta } from "./PerformanceTrendRenderTypes";
import { GroupByOption, PerformanceTrendUIOptionsState } from "../filters/PerformanceTrendUIOptionsState";
import { Label } from "@fluentui/react";
import { KEY_SEPARATOR } from "../responses/FilterKeys";
import { getDefaultVisibleProperties, getViewableProperties, PropertyMeta } from "../metrictypes/StatMetadata";
import { observable } from "mobx";
import { ColumnDef, CSVTable } from "hordePlugins/structuredanalytics/CSVComponent";
import { decorateWithReference } from "./ReferenceCellRenderer";
import { getHordeStyling } from "horde/styles/Styles";

// #region -- Helpers --

/**
 * Type-safe property name accessor - ensures property exists on type at compile time.
 */
function keyOf<T>(key: keyof T): keyof T {
    return key;
}

/**
 * Helper to build a map of propertyKey -> Set of test types where it's enabled.
 */
function buildPropertyToTestTypesMap(enabledProps: Record<string, string>): Map<string, Set<string>> {
    const propertyToTestTypes = new Map<string, Set<string>>();
    for (const compositeKey of Object.keys(enabledProps)) {
        const [testType, propertyKey] = compositeKey.split('::');
        if (!propertyToTestTypes.has(propertyKey)) {
            propertyToTestTypes.set(propertyKey, new Set());
        }
        propertyToTestTypes.get(propertyKey)!.add(testType);
    }
    return propertyToTestTypes;
}

/**
 * Build y-axis label from PropertyMeta.
 */
function buildYAxisLabel(meta: PropertyMeta): string {
    if (meta.unit) {
        return `${meta.label} (${meta.unit})`;
    }
    return meta.label;
}

/**
 * Dynamic property accessor - gets value from object using property key.
 */
function getPropertyValue(obj: KeyStatsData, propertyKey: string): number {
    return (obj as unknown as Record<string, number>)[propertyKey];
}

/**
 * Build reference lines for a specific property across every reference point that carries a
 * threshold for it. Each emitted reference point becomes its own line so multi-budget /
 * multi-testType graphs visualise every applicable threshold distinctly.
 */
function buildReferenceLines(
    referencePoints: MetricReferencePoint<KeyStatsData>[] | undefined,
    propertyKey: keyof ReferenceThresholds<KeyStatsData>
): ReferenceLine[] {
    if (!referencePoints || referencePoints.length === 0) return [];

    const lines: ReferenceLine[] = [];
    for (const ref of referencePoints) {
        const threshold = ref.thresholds[propertyKey];
        if (!threshold) continue;
        lines.push({
            value: threshold.thresholdValue,
            label: `${threshold.name}\n(${ref.label})`,
            dashed: true
        });
    }
    return lines;
}

// #endregion -- Helpers --

/**
 * KeyStats view generator.
 * Uses @viewable decorator metadata to dynamically generate views for each enabled property.
 */
export class KeyStatsViewGenerator implements IMetricViewGenerator<KeyStatsData> {

    // #region -- Private Helpers --

    private getGroupKey(metric: KeyStatsData, groupBy: GroupByOption): string {
        const platform = metric.platform ?? 'Unknown';
        const stream = metric.computedStream ?? 'Unknown';
        const testType = metric.gauntletSubTest ?? 'Unknown';

        switch (groupBy) {
            case 'platform':
                return platform;
            case 'stream':
                return stream;
            case 'testType':
                return testType;
            case 'platform+stream':
                return `${platform}${KEY_SEPARATOR}${stream}`;
            case 'platform+testType':
                return `${platform}${KEY_SEPARATOR}${testType}`;
            case 'stream+testType':
                return `${stream}${KEY_SEPARATOR}${testType}`;
            case 'platform+stream+testType':
                return `${platform}${KEY_SEPARATOR}${stream}${KEY_SEPARATOR}${testType}`;
        }
    }

    private getGroupLabel(key: string, groupBy: GroupByOption): string {
        const parts = key.split(KEY_SEPARATOR);
        switch (groupBy) {
            case 'platform':
            case 'stream':
            case 'testType':
                return key;
            case 'platform+stream':
            case 'platform+testType':
            case 'stream+testType':
                return `${parts[0]}, ${parts[1]}`;
            case 'platform+stream+testType':
                return `${parts[0]}, ${parts[1]}, ${parts[2]}`;
        }
    }

    private groupMetrics(metricsToGroup: KeyStatsData[], groupBy: GroupByOption): Map<string, KeyStatsData[]> {
        const grouped = new Map<string, KeyStatsData[]>();

        for (const metric of metricsToGroup) {
            const key = this.getGroupKey(metric, groupBy);

            if (!grouped.has(key)) {
                grouped.set(key, []);
            }
            grouped.get(key)!.push(metric);
        }

        return grouped;
    }

    // #endregion -- Private Helpers --

    /**
     * @inheritdoc
     */
    getViews(
        metrics: KeyStatsData[],
        uiState: PerformanceTrendUIOptionsState,
        selectionHandler?: MetricSelectionHandler<KeyStatsData>,
        referencePoints?: MetricReferencePoint<KeyStatsData>[]
    ): IMetricView[] {

        const groupBy: GroupByOption = uiState.groupByOption;
        const metricsByGroup = this.groupMetrics(metrics, groupBy);
        const sortedGroupKeys = Array.from(metricsByGroup.keys()).sort();

        // Get viewable properties metadata from @viewable decorators
        const viewableProps = getViewableProperties(KeyStatsData);

        // Check which properties are enabled for which test types
        const propertyToTestTypes = buildPropertyToTestTypesMap(uiState.enabledViewableProperties);

        const createGroupedSeries = <T extends KeyStatsData>(propertyKey: string, yAccessor: (m: T) => number, labelPrefix: string, baseColorIndex: number): SeriesConfig<T>[] => {
            const enabledTestTypes = propertyToTestTypes.get(propertyKey);
            if (!enabledTestTypes) return [];

            return sortedGroupKeys.map((groupKey, idx) => {
                const groupData = metricsByGroup.get(groupKey)! as T[];
                const filteredData = groupData.filter(m => enabledTestTypes.has(m.gauntletSubTest));
                return {
                    data: filteredData,
                    yAccessor,
                    label: `${labelPrefix}\n(${this.getGroupLabel(groupKey, groupBy)})`,
                    color: graphColors[(baseColorIndex + idx) % graphColors.length]
                };
            }).filter(s => s.data.length > 0);
        };

        const changelistAccessor = (value: KeyStatsData, _index: number): number => value.commitIdOrdered;

        // Get metadata for each property (type-safe property names)
        const gpuMeta = viewableProps.get(keyOf<KeyStatsData>('gpuTimeAvg'))!;
        const hitchesMeta = viewableProps.get(keyOf<KeyStatsData>('hitchesMin'))!;
        const memoryMeta = viewableProps.get(keyOf<KeyStatsData>('physicalUsedMbMax'))!;
        const gameThreadMeta = viewableProps.get(keyOf<KeyStatsData>('gameThreadtimeAvg'))!;
        const renderThreadMeta = viewableProps.get(keyOf<KeyStatsData>('renderThreadtimeAvg'))!;
        const frametimeMeta = viewableProps.get(keyOf<KeyStatsData>('frametimeAvg'))!;

        // Create series arrays for each metric type (filtered by enabled properties)
        const gpuSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('gpuTimeAvg'),
            (m) => m.gpuTimeAvg,
            gpuMeta.label,
            10
        );

        const hitchesPerMinSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('hitchesMin'),
            (m) => m.hitchesMin,
            hitchesMeta.label,
            11
        );

        const memoryUsedMaxSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('physicalUsedMbMax'),
            (m) => m.physicalUsedMbMax,
            memoryMeta.label,
            13
        );

        const gameThreadSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('gameThreadtimeAvg'),
            (m) => m.gameThreadtimeAvg,
            gameThreadMeta.label,
            0
        );

        const renderThreadSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('renderThreadtimeAvg'),
            (m) => m.renderThreadtimeAvg,
            renderThreadMeta.label,
            5
        );

        const frametimeSeries = createGroupedSeries<KeyStatsData>(
            keyOf<KeyStatsData>('frametimeAvg'),
            (m) => m.frametimeAvg,
            frametimeMeta.label,
            8
        );

        // Calculate averages across all platforms
        const gpuTimes: number[] = metrics.map(m => m.gpuTimeAvg);
        const memoryUsedMax: number[] = metrics.map(m => m.physicalUsedMbMax);
        const hitchesPerMin: number[] = metrics.map(m => m.hitchesMin);
        const avgGpuTimes = gpuTimes.length === 0 ? undefined : gpuTimes.reduce((s, v) => s + v, 0) / gpuTimes.length;
        const avgHitchesPerMin = hitchesPerMin.length === 0 ? undefined : hitchesPerMin.reduce((s, v) => s + v, 0) / hitchesPerMin.length;
        const avgMemoryUsed = memoryUsedMax.length === 0 ? undefined : memoryUsedMax.reduce((s, v) => s + v, 0) / memoryUsedMax.length;

        // Build reference lines for each property — one line per reference point that carries a
        // threshold for the property. Multi-budget / multi-testType setups produce multiple lines.
        const gpuRefLines = buildReferenceLines(referencePoints, 'gpuTimeAvg');
        const hitchesRefLines = buildReferenceLines(referencePoints, 'hitchesMin');
        const memoryRefLines = buildReferenceLines(referencePoints, 'physicalUsedMbMax');
        const gameThreadRefLines = buildReferenceLines(referencePoints, 'gameThreadtimeAvg');
        const renderThreadRefLines = buildReferenceLines(referencePoints, 'renderThreadtimeAvg');
        const frametimeRefLines = buildReferenceLines(referencePoints, 'frametimeAvg');

        const GpuTimeView = ({ width, height }: { width: number; height: number }) => {
            return (
                <div>
                    <div>
                        <span>{gpuMeta.label} (all metrics)</span><br />
                        <span>(Avg) {avgGpuTimes?.toFixed(gpuMeta.precision ?? 1)} {gpuMeta.unit ? `(${gpuMeta.unit})` : ''}</span>
                    </div>
                    <LineGraph
                        series={gpuSeries}
                        xAxisLabel="Changelist"
                        yAxisLabel={buildYAxisLabel(gpuMeta)}
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={uiState.curveGraphLine}
                        onClick={selectionHandler}
                        xAccessor={changelistAccessor}
                        referenceLines={gpuRefLines.length > 0 ? gpuRefLines : undefined}
                    />
                </div>
            );
        };

        const HitchesPerMinView = ({ width, height }: { width: number; height: number }) => {
            return (
                <div>
                    <div>
                        <span>{hitchesMeta.label}</span> <br />
                        <span>(Avg) {avgHitchesPerMin?.toFixed(hitchesMeta.precision ?? 1)}</span>
                    </div>
                    <LineGraph
                        series={hitchesPerMinSeries}
                        xAxisLabel="Changelist"
                        yAxisLabel={hitchesMeta.label}
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={uiState.curveGraphLine}
                        onClick={selectionHandler}
                        xAccessor={changelistAccessor}
                        referenceLines={hitchesRefLines.length > 0 ? hitchesRefLines : undefined}
                    />
                </div>
            );
        };

        const ThreadTimesView = ({ width, height }: { width: number; height: number }) => {
            // Combine reference lines for all thread timing metrics across every reference point.
            const threadRefLines: ReferenceLine[] = [...gameThreadRefLines, ...renderThreadRefLines, ...frametimeRefLines];
            return (
                <div>
                    <div>
                        <span>Thread Times</span> <br />
                    </div>
                    <LineGraph
                        series={[...gameThreadSeries, ...renderThreadSeries, ...frametimeSeries]}
                        xAxisLabel="Changelist"
                        yAxisLabel={buildYAxisLabel(gameThreadMeta)}
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={uiState.curveGraphLine}
                        onClick={selectionHandler}
                        xAccessor={changelistAccessor}
                        referenceLines={threadRefLines.length > 0 ? threadRefLines : undefined}
                    />
                </div>
            );
        };

        const PeakPhysicalMemoryView = ({ width, height }: { width: number; height: number }) => {
            return (
                <div>
                    <div>
                        <span>{memoryMeta.label}</span> <br />
                        <span>(Avg) {avgMemoryUsed?.toFixed(memoryMeta.precision ?? 1)} {memoryMeta.unit ? `(${memoryMeta.unit})` : ''}</span>
                    </div>
                    <LineGraph
                        series={memoryUsedMaxSeries}
                        xAxisLabel="Changelist"
                        yAxisLabel={buildYAxisLabel(memoryMeta)}
                        width={width}
                        height={height}
                        yAxisZeroScale={true}
                        applyCurveSmoothing={uiState.curveGraphLine}
                        onClick={selectionHandler}
                        xAccessor={changelistAccessor}
                        referenceLines={memoryRefLines.length > 0 ? memoryRefLines : undefined}
                    />
                </div>
            );
        };

        const views: IMetricView[] = [];

        if (gpuSeries.length > 0) {
            views.push({ label: gpuMeta.label, Render: GpuTimeView });
        }
        if (hitchesPerMinSeries.length > 0) {
            views.push({ label: hitchesMeta.label, Render: HitchesPerMinView });
        }
        if (memoryUsedMaxSeries.length > 0) {
            views.push({ label: memoryMeta.label, Render: PeakPhysicalMemoryView });
        }
        if (gameThreadSeries.length > 0 || renderThreadSeries.length > 0) {
            views.push({ label: "Thread Timings", Render: ThreadTimesView });
        }

        return views;
    }

    /**
     * Dynamically generates views based on enabled properties using @viewable decorator metadata.
     */
    getViewsDynamic(metrics: KeyStatsData[], uiState: PerformanceTrendUIOptionsState, selectionHandler?: MetricSelectionHandler<KeyStatsData>): IMetricView[] {

        const enabledProps = uiState.enabledViewableProperties;

        // If no properties are enabled, show nothing
        if (Object.keys(enabledProps).length === 0) {
            return [];
        }

        const propertyToTestTypes = buildPropertyToTestTypesMap(enabledProps);
        const groupBy: GroupByOption = uiState.groupByOption;
        const changelistAccessor = (value: KeyStatsData, _index: number): number => value.commitIdOrdered;

        // Get viewable properties metadata from @viewable decorators
        const viewableProps = getViewableProperties(KeyStatsData);

        // Create a view for each enabled property
        const views: IMetricView[] = [];
        let colorOffset = 0;

        for (const [propertyKey, enabledTestTypes] of propertyToTestTypes) {
            const meta = viewableProps.get(propertyKey);

            if (!meta) {
                continue;
            }

            // Filter metrics to only include those from test types where THIS property is enabled
            const propertyMetrics = metrics.filter(m => enabledTestTypes.has(m.gauntletSubTest));

            if (propertyMetrics.length === 0) {
                continue;
            }

            // Group the filtered metrics for this property
            const metricsByGroup = this.groupMetrics(propertyMetrics, groupBy);
            const sortedGroupKeys = Array.from(metricsByGroup.keys()).sort();

            // Use incrementing color offset for each property
            const baseColorIndex = colorOffset;
            colorOffset += sortedGroupKeys.length;

            // Create series for this property using dynamic accessor
            const series: SeriesConfig<KeyStatsData>[] = sortedGroupKeys.map((groupKey, idx) => ({
                data: metricsByGroup.get(groupKey)!,
                yAccessor: (m: KeyStatsData) => getPropertyValue(m, propertyKey),
                label: `${meta.label}\n(${this.getGroupLabel(groupKey, groupBy)})`,
                color: graphColors[(baseColorIndex + idx) % graphColors.length]
            }));

            // Calculate average for display
            const values = propertyMetrics.map(m => getPropertyValue(m, propertyKey)).filter(v => v !== undefined && !isNaN(v));
            const avg = values.length > 0 ? values.reduce((s, v) => s + v, 0) / values.length : undefined;

            // Build y-axis label from metadata
            const yAxisLabel = buildYAxisLabel(meta);

            const PropertyView = ({ width, height }: { width: number; height: number }) => {
                return (
                    <div>
                        <div>
                            <span>{meta.label}</span><br />
                            <span>(Avg) {avg?.toFixed(meta.precision ?? 1)} {meta.unit ? `(${meta.unit})` : ''}</span>
                        </div>
                        <LineGraph
                            series={series}
                            xAxisLabel="Changelist"
                            yAxisLabel={yAxisLabel}
                            width={width}
                            height={height}
                            yAxisZeroScale={true}
                            applyCurveSmoothing={uiState.curveGraphLine}
                            onClick={selectionHandler}
                            xAccessor={changelistAccessor}
                        />
                    </div>
                );
            };

            views.push({
                label: meta.label,
                Render: PropertyView,
            });
        }

        return views;
    }

    /**
     * @inheritdoc
     */
    getSelectedDetailView(
        metrics: KeyStatsData[],
        isFilteredContext: boolean,
        _uiState: PerformanceTrendUIOptionsState,
        referencePoints?: MetricReferencePoint<KeyStatsData>[]
    ): IMetricDetailedView {
        const refs = referencePoints ?? [];
        return {
            Render: (props) => <KeyStatsDetailView metrics={metrics} isFilteredContext={isFilteredContext} referencePoints={refs} {...props} />
        };
    }

    /**
     * @inheritdoc 
     */
    getViewableProperties(): Map<string, PropertyMeta> {
        return getViewableProperties(KeyStatsData);
    }

    /**
     * @inheritdoc 
     */
    getDefaultVisibleProperties(): string[] {
        return getDefaultVisibleProperties(KeyStatsData);
    }
};

/**
 * Detail view component for KeyStats data.
 * Extracted as standalone component to avoid MobX observer warnings.
 */
const KeyStatsDetailView: React.FC<{ metrics: KeyStatsData[], isFilteredContext: boolean, referencePoints: MetricReferencePoint<KeyStatsData>[], width: number, height: number }> = ({ metrics, isFilteredContext, referencePoints, width, height }) => {
    const { modeColors } = getHordeStyling();

    if (!metrics || metrics.length === 0) {
        return <span>No data selected</span>;
    }

    const ensureTrailingSlash = (url: string): string => url.endsWith('/') ? url : `${url}/`;

    // Each metric column gets wrapped so its cells colour against the aggregate verdict across
    // applicable reference points. decorateWithReference is a no-op for columns no reference
    // touches, so non-metric columns ("platform", "csvId", etc.) pass through unchanged.
    // Uses the default Fluent-aligned palette baked into decorateWithReference.
    const decorate = (key: keyof KeyStatsData, base?: ColumnDef<KeyStatsData>) =>
        decorateWithReference<KeyStatsData>(base ?? { key: key as string }, key, referencePoints);

    const columns: (string | ColumnDef<KeyStatsData>)[] = [
        {
            key: "hordeLink",
            label: "Horde Step",
            render: (_, row, cell) => {
                const url = `${ensureTrailingSlash(row.hordeUrlStr)}job/${row.jobId}${row.stepId ? `?step=${row.stepId}` : ""}`;
                cell.append("a")
                    .attr("href", url)
                    .attr("target", "_blank")
                    .attr("rel", "noopener noreferrer")
                    .text("View Step")
                    .style("width", "140px")
            }
        },
        {
            key: "csvId",
            label: "Csv Id",
            render: (value, row, cell) => {
                cell.style("font-weight", "bold")
                    .text(value);
            }
        },
        {
            key: "gauntletSubTest",
            label: "Test Type"
        },
        "platform",
        decorate("gpuTimeAvg"),
        decorate("gameThreadtimeAvg"),
        decorate("renderThreadtimeAvg"),
        decorate("physicalUsedMbMax"),
        decorate("hitchesMin"),
        decorate("hitchTimePercent"),
        decorate("mvp"),
        decorate("dynamicResolutionPercentageAvg"),
    ];

    // Wrap data in observable to satisfy CSVTable's observer wrapper
    const dataProvider = observable({ data: metrics });

    // Header annotation: one line per reference point describing the budget, the platform/testType
    // scope it covers, and every threshold value it carries. Iterating directly (rather than
    // deduping per-budget) keeps multi-testType splits visible — they often differ by threshold.
    // Format: "Budget: <name> [<platforms>] (<testType>): <metric> <≤|≥> <value>, …"
    const formatReferenceLine = (ref: MetricReferencePoint<KeyStatsData>): string => {
        // Pull the budget portion off the auto-generated label; the scope text replaces what the
        // label tacked on in parentheses so we don't end up with "(FlyThrough) (FlyThrough)".
        const budgetLabel = ref.label.replace(/\s*\([^)]*\)\s*$/, "");

        const platforms = ref.scope?.platforms;
        const platformText = platforms && platforms.length > 0 ? `[${platforms.join(", ")}]` : "[all platforms]";
        const scopeText = ref.scope?.testType ? `${platformText} (${ref.scope.testType})` : platformText;

        const thresholds = ref.thresholds as Record<string, ThresholdMeta | undefined>;
        const thresholdParts: string[] = [];
        for (const [key, meta] of Object.entries(thresholds)) {
            if (!meta) continue;
            const comparator = meta.largerIsWorse ? "≤" : "≥";
            thresholdParts.push(`${key} ${comparator} ${meta.thresholdValue}`);
        }
        const thresholdText = thresholdParts.length > 0 ? `: ${thresholdParts.join(", ")}` : "";

        return `${budgetLabel} ${scopeText}${thresholdText}`;
    };

    const referenceLines = referencePoints.map(formatReferenceLine);

    return (
        <>
            <Label>Key Stats</Label>
            {isFilteredContext && (
                <div style={{ fontStyle: "italic", fontSize: 12, opacity: 0.75, marginBottom: 4, border: `1px solid ${modeColors.background}` }}>
                    Filtered result set.
                </div>
            )}
            {referenceLines.length > 0 && (
                <div style={{ fontStyle: "italic", fontSize: 12, opacity: 0.75, marginBottom: 4, border: `1px solid ${modeColors.background}` }}>
                    <div>Coloring Criteria:</div>
                    {referenceLines.map((line, idx) => (
                        <div key={idx} style={{ marginLeft: 12 }}>{line}</div>
                    ))}
                </div>
            )}
            <CSVTable
                dataProvider={dataProvider}
                columnProvider={{ columns }}
            />
        </>
    );
};