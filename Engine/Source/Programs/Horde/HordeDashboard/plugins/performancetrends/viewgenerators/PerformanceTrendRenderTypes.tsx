// Copyright Epic Games, Inc. All Rights Reserved.

import { MetricSelectionHandler } from "../components/LineGraphComponent";
import { PerformanceTrendUIOptionsState } from "../filters/PerformanceTrendUIOptionsState";
import { PerformanceTrendContext } from "../metrictypes/PerformanceTrendsTypes";
import { PropertyMeta } from "../metrictypes/StatMetadata";

// #region -- Reference Point Types --

/**
 * Threshold metadata for a single metric property.
 */
export interface ThresholdMeta {
    /** The threshold value for this metric. */
    thresholdValue: number;
    /** Indicates whether a larger value is worse for this metric. */
    largerIsWorse: boolean;
    /** The name of the threshold. */
    name: string;
}

/**
 * Transforms all numeric properties of T into optional ThresholdMeta properties.
 * This provides strongly-typed direct property access (e.g., ref.thresholds.frametimeAvg)
 * with autocomplete support.
 */
export type ReferenceThresholds<T> = {
    [K in keyof T as T[K] extends number ? K : never]?: ThresholdMeta;
};

/**
 * Reference point structure passed to ViewGenerator.
 * Strongly typed to the metric type. The ViewGenerator is responsible for
 * matching thresholds to the appropriate metrics based on testType/platform.
 *
 * Each reference point describes thresholds for a single applicability scope
 * (e.g. a budget restricted to one testType + platform set). When a single budget
 * carries thresholds for multiple testTypes, the adapter splits it into multiple
 * reference points — one per testType — each with its own @see appliesTo"
 * predicate. This keeps `thresholds` a clean one-per-property map.
 */
export interface MetricReferencePoint<T extends PerformanceTrendContext> {
    /** Identifier (e.g., "budget:abc123:FlyThrough" or "reference:streamId"). */
    id: string;

    /** Display label for legend (e.g., "Budget: My Budget (FlyThrough)"). */
    label: string;

    /**
     * Strongly-typed threshold values - direct property access with autocomplete.
     * Example: referencePoint.thresholds.frametimeAvg?.thresholdValue
     * One threshold per property per reference point. To represent multiple
     * thresholds for the same metric (e.g. different limits per testType),
     * emit multiple reference points each scoped via @see appliesTo.
     */
    thresholds: ReferenceThresholds<T>;

    /**
     * Optional predicate identifying which rows this reference applies to.
     * When omitted, the reference is treated as applying to every row.
     * Used by the detail-view cell coloring to filter references per-row before
     * verdict aggregation; lets a single graph carry budgets scoped to
     * different testType / platform slices without cross-contamination.
     */
    appliesTo?: (row: T) => boolean;

    /**
     * Optional descriptive scope information for human-facing rendering (e.g. the detail-view
     * header that lists which budgets are colouring the table). Mirrors the data already baked
     * into @see appliesTo" but in introspectable form, so the UI can show "Budget X
     * [PS5, XSX] (FlyThrough)" without inverting the predicate. Optional — leave undefined for
     * references whose scope is "applies to everything" or whose scope isn't usefully describable.
     */
    scope?: {
        /** Platforms this reference applies to. Undefined / empty = applies to every platform. */
        platforms?: string[];
        /** Test type this reference applies to. Undefined = applies to every testType. */
        testType?: string;
    };
}

// #endregion -- Reference Point Types --

/**
 * Interface that defines a render action.
 */
export interface IMetricView {
    /** Optional label for the metric */
    label?: string;

    /** 
     * React component representing the visualization of this metric.
     * @returns React component to render element.
    */
    Render: React.FC<{ width: number; height: number }>;
}

/**
 * Interface that defines a detailed render action.
 */
export interface IMetricDetailedView {
    Render: React.FC<{ width: number; height: number }>;
}

/**
 * Interface that describes a class capable of generating a view representing @see TMetric.
 */
export interface IMetricViewGenerator<TMetric extends PerformanceTrendContext> {
    /**
     * Gets a list of views for a specific metric.
     * @param metrics The metrics to visualize.
     * @param uiState The UI options.
     * @param selectionHandler Optional handler for metric selection events.
     * @param referencePoint Optional reference point containing threshold data for rendering reference lines.
     * @returns A list of @see IMetricView objects that can be rendered.
     */
    getViews(
        metrics: TMetric[],
        uiState: PerformanceTrendUIOptionsState,
        selectionHandler?: MetricSelectionHandler<TMetric>,
        referencePoints?: MetricReferencePoint<TMetric>[]
    ): IMetricView[];

    /**
     * Gets a view that represents a detailed perspective when a set of metrics is selected for analysis.
     * @param metrics The metrics to provide a detailed view for.
     * @param isFilteredContext Whether the metric set was filtered down to a single series.
     * @param uiState The UI options.
     * @param referencePoints Optional reference points whose thresholds drive cell coloring in the detail view.
     * @returns A single @see IMetricDetailedView representing a detailed analysis.
     */
    getSelectedDetailView(
        metrics: TMetric[],
        isFilteredContext: boolean,
        uiState: PerformanceTrendUIOptionsState,
        referencePoints?: MetricReferencePoint<TMetric>[]
    ): IMetricDetailedView;

    /**
     * Gets the viewable properties for this metric type.
     * @returns A map of property key to metadata.
     */
    getViewableProperties(): Map<string, PropertyMeta>;

    /**
     * Gets the default visible property keys.
     * @returns Array of property keys visible by default.
     */
    getDefaultVisibleProperties(): string[];
}