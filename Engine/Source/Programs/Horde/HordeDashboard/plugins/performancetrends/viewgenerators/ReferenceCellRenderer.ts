// Copyright Epic Games, Inc. All Rights Reserved.

import { getColorRecords } from "horde/components/buildhealth/stepoutcome/StepOutcomeSharedUIComponents";
import { ColumnDef } from "hordePlugins/structuredanalytics/CSVComponent";
import { MetricReferencePoint, ThresholdMeta } from "./PerformanceTrendRenderTypes";
import { PerformanceTrendContext } from "../metrictypes/PerformanceTrendsTypes";

/**
 * Verdict for a single value against (potentially many) thresholds. Ordering for aggregation
 * is bad > warn > good > neutral — the worst applicable verdict wins.
 *  - "good"    : value is on the better side of the threshold, outside the warning band.
 *  - "warn"    : value is within +/- warningBandRatio of the threshold (on either side).
 *  - "bad"     : value is on the worse side of the threshold, outside the warning band.
 *  - "neutral" : no threshold applied (no reference point covered this column for this row,
 *                or the value itself was missing). Renders with default cell styling.
 */
export type ReferenceVerdict = "good" | "warn" | "bad" | "neutral";

/**
 * Theme-relevant colors needed by @see colorFor. Each verdict maps to a CSS color
 * string. When a field is omitted, @see colorFor falls back to the user's configured
 * Horde dashboard status colors (Success / Warnings / Failure) — the same source that drives
 * every other build-health colour in Horde, including users' colorblind-friendly overrides.
 * Callers pass a partial palette to override piecewise without losing the user-theme fallback.
 */
export interface ReferenceColorPalette {
    /** Color for "good" verdicts (value is comfortably better than the threshold). */
    successText?: string;
    /** Color for "warn" verdicts (value sits within the warning band around the threshold). */
    warningText?: string;
    /** Color for "bad" verdicts (value is on the worse side past the warning band). */
    errorText?: string;
}

const VERDICT_SEVERITY: Record<ReferenceVerdict, number> = {
    neutral: 0,
    good: 1,
    warn: 2,
    bad: 3,
};

/**
 * Returns the most-severe verdict in the list (bad > warn > good > neutral). Empty list = neutral.
 */
export function worstVerdict(verdicts: ReferenceVerdict[]): ReferenceVerdict {
    let worst: ReferenceVerdict = "neutral";
    let worstSeverity = VERDICT_SEVERITY.neutral;
    for (const v of verdicts) {
        if (VERDICT_SEVERITY[v] > worstSeverity) {
            worst = v;
            worstSeverity = VERDICT_SEVERITY[v];
        }
    }
    return worst;
}

/**
 * Classifies one value against one threshold.
 *  - Missing value or missing threshold returns "neutral".
 *  - Within +/- warningBandRatio of the threshold (regardless of side) returns "warn".
 *  - Otherwise "good" or "bad" depending on which side of the threshold the value lands,
 *    using threshold.largerIsWorse to determine which side is "bad".
 */
export function classifyAgainstThreshold(
    value: number | undefined,
    threshold: ThresholdMeta | undefined,
    warningBandRatio: number = 0.1,
): ReferenceVerdict {
    if (threshold === undefined || value === undefined || Number.isNaN(value)) {
        return "neutral";
    }

    const limit = threshold.thresholdValue;
    const margin = Math.abs(limit) * warningBandRatio;

    // Warning band straddles the threshold by +/- margin. Sits on top of good/bad so a value
    // exactly at the limit reads as a warning rather than a pass — useful for nudging operators
    // before they fail outright.
    if (Math.abs(value - limit) <= margin) {
        return "warn";
    }

    const valueIsAbove = value > limit;
    const aboveIsBad = threshold.largerIsWorse;
    const isBad = valueIsAbove === aboveIsBad;
    return isBad ? "bad" : "good";
}

/**
 * Aggregates verdicts for a single row's value across every reference point whose
 *   1. @see MetricReferencePoint.appliesTo predicate accepts the row (or is undefined), AND
 *   2. thresholds map carries an entry for the named column key.
 * Returns the worst-verdict aggregate alongside the per-reference breakdown for any caller
 * that wants to show "this row failed budget A, passed budget B" UI.
 */
export function classifyAcrossReferences<T extends PerformanceTrendContext>(
    value: number | undefined,
    columnKey: keyof T,
    row: T,
    referencePoints: MetricReferencePoint<T>[],
    warningBandRatio?: number,
): {
    aggregate: ReferenceVerdict;
    perReference: Array<{ ref: MetricReferencePoint<T>; verdict: ReferenceVerdict }>;
} {
    const perReference: Array<{ ref: MetricReferencePoint<T>; verdict: ReferenceVerdict }> = [];
    for (const ref of referencePoints) {
        // Skip references that don't apply to this row.
        if (ref.appliesTo && !ref.appliesTo(row)) {
            continue;
        }
        const threshold = (ref.thresholds as Record<string, ThresholdMeta | undefined>)[columnKey as string];
        if (!threshold) {
            continue;
        }
        perReference.push({ ref, verdict: classifyAgainstThreshold(value, threshold, warningBandRatio) });
    }

    const aggregate = worstVerdict(perReference.map(p => p.verdict));
    return { aggregate, perReference };
}

/**
 * Theme-aware color lookup. Returns undefined for "neutral" so the caller leaves the default
 * cell styling in place rather than overriding it. Resolves verdict colors from the user's
 * Horde dashboard status colors (Success / Warnings / Failure) so users' colorblind-friendly
 * status-color overrides apply uniformly across the dashboard.
 */
export function colorFor(verdict: ReferenceVerdict): string | undefined {
    if (verdict === "neutral") return undefined;
    // getColorRecords reads dashboard.getStatusColors() — picks up the user's current overrides.
    const themed = getColorRecords();
    switch (verdict) {
        case "good": return themed["Success"];
        case "warn": return themed["Warnings"];
        case "bad": return themed["Failure"];
        default: return undefined;
    }
}

/**
 * Wraps an existing ColumnDef so its rendered cell is colored against the aggregate verdict
 * across applicable reference points. Chains over any pre-existing render callback so existing
 * styling (bold, links, etc.) is preserved. Returns the column unchanged when no reference point
 * carries a threshold for this column — avoids adding a no-op render wrapper to columns that
 * will never be coloured.
 *
 * @param base Either a string column key (treated as the property name) or a full ColumnDef.
 * @param key The property key on T to use both for the column data and for threshold lookup.
 *            Must match the same string as thresholds key on the reference points.
 * @param referencePoints The reference points to evaluate against.
 * @param warningBandRatio Optional override for the +/- band that produces a "warn" verdict.
 */
export function decorateWithReference<T extends PerformanceTrendContext>(
    base: string | ColumnDef<T>,
    key: keyof T,
    referencePoints: MetricReferencePoint<T>[],
    warningBandRatio?: number,
): string | ColumnDef<T> {
    const someApplies = referencePoints.some(
        ref => (ref.thresholds as Record<string, ThresholdMeta | undefined>)[key as string] !== undefined,
    );
    if (!someApplies) {
        return base;
    }

    const baseDef: ColumnDef<T> = typeof base === "string"
        ? { key: base }
        : base;

    const previousRender = baseDef.render;

    return {
        ...baseDef,
        render: (value, row, cell) => {
            // Run the prior render (if any) so existing styling sticks; otherwise emit the value.
            if (previousRender) {
                previousRender(value, row, cell);
            } else {
                cell.text(value === undefined || value === null ? "" : String(value));
            }
            const { aggregate } = classifyAcrossReferences(value, key, row, referencePoints, warningBandRatio);
            // d3's style() overloads don't unify across `string | null`, so branch the call:
            // pass `null` to remove the inline style, or the color string to set it.
            const color = colorFor(aggregate);
            if (color) {
                cell.style("color", color);
            } else {
                cell.style("color", null);
            }
        },
    };
}

