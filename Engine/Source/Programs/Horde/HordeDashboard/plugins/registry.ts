// Copyright Epic Games, Inc. All Rights Reserved.

import "./exampleplugin"
import "./analytics"
import "./test-automation"
import "./performancetrends"
import "./structuredanalytics"
