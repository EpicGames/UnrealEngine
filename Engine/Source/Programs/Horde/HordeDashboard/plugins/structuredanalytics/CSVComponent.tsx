// Copyright Epic Games, Inc. All Rights Reserved.

import { getColorRecords } from "horde/components/buildhealth/stepoutcome/StepOutcomeSharedUIComponents";
import { getHordeStyling, preloadFonts } from "horde/styles/Styles";
import { observer } from "mobx-react-lite";
import { useCallback, useEffect, useMemo, useRef, useState } from "react";
import * as d3 from "d3";

type SortDirection = "asc" | "desc" | null;

/**
 * Interface that describes a linear data provider.
 */
export interface IDataProvider {
    data: any[];
}

/**
 * Interface that defines a more flexible variant for how to handle a column data.
 */
export interface ColumnDef<T = any> {
    key: string;
    label?: string;

    /**
     * Delegate to produce custom rendering & style of a specific value in a row.
     * @param value The value to render.
     * @param row The strongly typed row data.
     * @param cell The D3 cell to update with custom rendering & styling.
     */
    render?: (value: any, row: T, cell: d3.Selection<HTMLTableCellElement, unknown, null, undefined>) => void;
}

/**
 * Interface that describes a column provider.
 */
export interface IColumnProvider<T = any> {
    columns: (string | ColumnDef<T>)[];
}

/**
 * Generic CSV Table component.
 * @returns React Component that represents a CSV table.
 */
export const CSVTable: React.FC<{ dataProvider: IDataProvider, columnProvider: IColumnProvider }> = observer(({ dataProvider, columnProvider }) => {

    // #region -- Refs & State --

    const containerRef = useRef<HTMLDivElement>(null);
    const [scrollWidth, setScrollWidth] = useState<number>(0);
    const [sortColumn, setSortColumn] = useState<string | null>(null);
    const [sortDirection, setSortDirection] = useState<SortDirection>(null);
    const { modeColors, _hordeClasses, _detailClasses } = getHordeStyling();

    const colorRecords: Record<string, string> = getColorRecords();
    const hasConstructed = useRef<boolean>(false);

    const scaffold = useRef<{
        wrapper: d3.Selection<HTMLDivElement, unknown, null, undefined>;
        table: d3.Selection<HTMLTableElement, unknown, null, undefined>;
        thead: d3.Selection<HTMLTableSectionElement, unknown, null, undefined>;
        theaderrow: d3.Selection<HTMLTableRowElement, unknown, null, undefined>;
        tbody: d3.Selection<HTMLTableSectionElement, unknown, null, undefined>;
        tooltip: d3.Selection<HTMLDivElement, unknown, HTMLElement, any>;
    } | null>(null);

    //#endregion -- Refs & State --

    // #region -- Common Colors --

    const summaryColorScaleBase = d3.scaleLinear<string>()
        .domain([0, 0.5, 1])
        .range([colorRecords["Failure"], colorRecords["Warnings"], colorRecords["Success"]]);

    const summaryColorScale = (value: number): string => {
        if (value === -1) {
            return colorRecords["Waiting"];
        }
        return summaryColorScaleBase(value);
    };

    // #endregion -- Common Colors --

    // #region -- Callback Construction -- 

    const handleHeaderClick = useCallback((columnKey: string) => {
        // If we have clicked on existing table, cycle the options.
        // asc -> desc -> no sort -> asc -> ...
        if (sortColumn === columnKey) {
            if (sortDirection === "asc") {
                setSortDirection("desc");
            } else if (sortDirection === "desc") {
                setSortColumn(null);
                setSortDirection(null);
            } else {
                setSortDirection("asc");
            }
        } else {
            setSortColumn(columnKey);
            setSortDirection("asc");
        }
    }, [sortColumn, sortDirection]);

    // We must stash the refs here in order to properly enclose the sortColumn useState
    const handleHeaderClickRef = useRef(handleHeaderClick);
    handleHeaderClickRef.current = handleHeaderClick;

    const getSortIndicator = useCallback((columnKey: string): string => {
        if (sortColumn !== columnKey) {
            return "";
        }
        return sortDirection === "asc" ? " ↑" : " ↓";
    }, [sortColumn, sortDirection]);

    const sortedData = useMemo(() => {
        if (!sortColumn || !sortDirection) {
            return dataProvider.data;
        }

        return [...dataProvider.data].sort((a, b) => {
            const aVal = a[sortColumn];
            const bVal = b[sortColumn];

            if (aVal === bVal) {
                return 0;
            }
            if (aVal === null || aVal === undefined) {
                return 1;
            }
            if (bVal === null || bVal === undefined) {
                return -1;
            }

            let comparison: number;
            if (typeof aVal === "number" && typeof bVal === "number") {
                comparison = aVal - bVal;
            } else if (typeof aVal === "string" && typeof bVal === "string") {
                comparison = aVal.localeCompare(bVal);
            } else {
                comparison = String(aVal).localeCompare(String(bVal));
            }

            return sortDirection === "asc" ? comparison : -comparison;
        });
    }, [dataProvider.data, sortColumn, sortDirection]);

    // #endregion -- Callback Construction --

    // #region -- Use Effects --

    useEffect(() => {
        if (!containerRef.current || scaffold.current) return;

        // Create our base container.
        const wrapper = d3.select(containerRef.current)
            .append("div")
            .attr("id", "csv-container");

        // Create the table element.
        const table = wrapper.append("table")
            .style("border-collapse", "separate")
            .style("border-spacing", "2px")
            .style("width", "fit-content")
            .style("font-family", preloadFonts[0])
            .style("font-size", "12px")
            .style("color", modeColors.text);

        // Add header && tbody.
        const thead = table.append("thead");
        const tbody = table.append("tbody");

        // Add tool tip div; will be used throughout.
        d3.selectAll(".d3-tooltip").remove();

        const tooltip = d3.select("body")
            .append("div")
            .attr("class", "d3-tooltip")
            .style("position", "absolute")
            .style("pointer-events", "none")
            .style("padding", "6px 12px")
            .style("background", "rgba(0,0,0,0.95)")
            .style("color", "#fff")
            .style("z-index", "5")
            .style("min-width", "250px")
            .style("font-family", preloadFonts[0])
            .style("font-size", "12px")
            .style("border-radius", "2px")
            .style("line-height", "1.4")
            .style("opacity", 0);

        const theaderrow = thead.append("tr")
            .style("height", "20px");

        scaffold.current = { wrapper, table, thead, theaderrow, tbody, tooltip };

        return () => {
            d3.selectAll(".d3-tooltip").remove();
        };
    }, []);

    type ColumnType = string | ColumnDef<any>;

    useEffect(() => {
        if (hasConstructed.current === false) {
            if (!containerRef.current || !scaffold.current) return;

            scaffold.current.theaderrow.selectAll<HTMLTableCellElement, ColumnType>("th.col")
                .data(columnProvider.columns)
                .join("th")
                .attr("class", "col")
                .style("position", "sticky")
                .style("top", "0")
                .style("z-index", "3")
                .style("background-color", modeColors.content)
                .style("border", `1px solid ${modeColors.background}`)
                .style("padding", "8px")
                .style("box-sizing", "border-box")
                .style("fontWeight", "bold")
                .style("min-width", "150px")
                .style("cursor", "pointer")
                .style("user-select", "none")
                .on("click", function (_event, d) {
                    const col = d as unknown as ColumnType;

                    if (!col) {
                        return;
                    }

                    const key = typeof col === "string" ? col : col.key;
                    handleHeaderClickRef.current(key);
                })
                .text(d => {
                    const label = typeof d === "string" ? d : (d.label ?? d.key);
                    const key = typeof d === "string" ? d : d.key;
                    return label + getSortIndicator(key);
                });

            hasConstructed.current = true;
        }
    }, [columnProvider.columns, getSortIndicator]);

    const renderCell = (cell: d3.Selection<HTMLTableCellElement, unknown, null, undefined>, col: string |
        ColumnDef<any>, rowData: any) => {
        const key = typeof col === "string" ? col : col.key;

        // Note: we are explicitly declaring any | undefined because an undefined value likely represents a computed column based on the row datum.
        const value: any | undefined = rowData[key];

        if (typeof col !== "string" && col.render) {
            col.render(value, rowData, cell);
        } else {
            cell.text(value);
        }
    };

    // Update header sort indicators when sort state changes
    useEffect(() => {
        if (!scaffold.current) {
            return;
        }

        scaffold.current.theaderrow.selectAll("th.col")
            .data(columnProvider.columns)
            .text(d => {
                const label = typeof d === "string" ? d : (d.label ?? d.key);
                const key = typeof d === "string" ? d : d.key;
                return label + getSortIndicator(key);
            });
    }, [sortColumn, sortDirection, columnProvider.columns, getSortIndicator]);

    useEffect(() => {
        if (!containerRef.current || !scaffold.current) return;
        const rows = scaffold.current.tbody.selectAll("tr")
            .data(sortedData)
            .join("tr");

        rows.each(function (rowData: any) {
            const tr = d3.select(this);

            tr.selectAll("td")
                .data(columnProvider.columns)
                .join(
                    enter => enter.append("td")
                        .style("border-right", `1px solid ${modeColors.content}`)
                        .style("padding", "8px")
                        .style("box-sizing", "border-box")
                        .each(function (col) {
                            renderCell(d3.select(this as HTMLTableCellElement), col, rowData);
                        }),
                    update => update.each(function (col) {
                        const cell = d3.select(this as HTMLTableCellElement);
                        cell.selectAll("*").remove();
                        cell.text("");
                        renderCell(cell, col, rowData);
                    }),
                    exit => exit.remove()
                );
        });

    }, [sortedData])

    // Update the top scrollbar width to match the table's scroll width
    useEffect(() => {
        if (!containerRef.current) return;

        const updateScrollWidth = () => {
            if (containerRef.current) {
                setScrollWidth(containerRef.current.scrollWidth);
            }
        };

        // Initial update
        updateScrollWidth();

        // Use ResizeObserver to detect when the table content changes size
        const resizeObserver = new ResizeObserver(updateScrollWidth);
        resizeObserver.observe(containerRef.current);

        return () => {
            resizeObserver.disconnect();
        };
    }, [dataProvider.data, columnProvider.columns]);

    // #endregion -- Use Effects --

    return (
        <>
            {/* Table container */}
            <div
                id="d3-container"
                ref={containerRef}
                style={{
                    height: '80%',
                    width: 'calc(100% - 4px)',
                    maxHeight: '80vh',
                    overflowX: 'auto',
                    overflowY: 'auto',
                    display: 'block',
                    marginRight: '4px',
                }}
            />
        </>
    )
})