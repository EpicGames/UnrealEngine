import backend from "horde/backend";

// #region -- Telemetry Schema Types --

/**
 * Fundamental schema data types. Mirrors the server's `SchemaDataType` enum so the
 * dashboard can reason about column shapes without a separate type translation layer.
 */
export type SchemaDataType =
    | 'String'
    | 'Int64'
    | 'Double'
    | 'Bool'
    | 'DateTime'
    | 'Object'
    | 'Array';

/**
 * One column in a telemetry schema as returned by the server.
 */
export type SchemaColumnResponse = {
    propertyName: string;
    columnName: string;
    clrTypeName: string;
    dataType: SchemaDataType;
    arrayElementType?: SchemaDataType;
    nestedSchemaEventName?: string;
    isNullable: boolean;
    order?: number;
};

/**
 * Response shape for an approved telemetry schema version.
 */
export type GetTelemetrySchemaResponse = {
    id: string;
    eventName: string;
    tableName: string;
    schemaName?: string;
    version: number;
    columns: SchemaColumnResponse[];
    createdAtUtc: string;
    approvedByUserId?: string;
    clrTypeName?: string;
    isNestedSchema: boolean;
};

/**
 * Per-column diff between two schema versions.
 */
export type ColumnChangeResponse = {
    columnName: string;
    oldClrTypeName?: string;
    newClrTypeName?: string;
    oldDataType?: SchemaDataType;
    newDataType?: SchemaDataType;
    oldIsNullable?: boolean;
    newIsNullable?: boolean;
};

/**
 * Comparison between two schema versions, used to render approval diffs.
 */
export type SchemaComparisonResponse = {
    addedColumns: string[];
    removedColumns: string[];
    modifiedColumns: ColumnChangeResponse[];
    tableNameChanged: boolean;
    oldTableName?: string;
};

/**
 * A pending schema update awaiting admin approval.
 */
export type GetPendingSchemaUpdateResponse = {
    id: string;
    eventName: string;
    proposedVersion: number;
    proposedTableName: string;
    proposedSchemaName?: string;
    proposedColumns: SchemaColumnResponse[];
    detectedAtUtc: string;
    changeType: string;
    changeDescription: string;
    comparison?: SchemaComparisonResponse;
    clrTypeName?: string;
    isNestedSchema: boolean;
};

/**
 * Request body for creating a schema column via POST /api/v1/telemetry-schemas.
 */
export type CreateSchemaColumnRequest = {
    propertyName: string;
    columnName: string;
    clrTypeName: string;
    dataType: SchemaDataType;
    arrayElementType?: SchemaDataType;
    nestedSchemaEventName?: string;
    isNullable: boolean;
};

/**
 * Request body for creating a new schema version via POST /api/v1/telemetry-schemas.
 */
export type CreateTelemetrySchemaRequest = {
    eventName: string;
    tableName: string;
    schemaName?: string;
    columns: CreateSchemaColumnRequest[];
    clrTypeName?: string;
    isNestedSchema: boolean;
};

/**
 * Request body for POST /api/v1/telemetry-schemas/infer.
 */
export type InferSchemaRequest = {
    eventName: string;
    tableName?: string;
    schemaName?: string;
    /** A single example payload as a raw JSON object. */
    samplePayload: unknown;
};

/**
 * One warning emitted during schema inference.
 */
export type InferenceWarningResponse = {
    path: string;
    message: string;
};

/**
 * Response from POST /api/v1/telemetry-schemas/infer. The proposal is intended to
 * pre-fill the dashboard authoring form; the admin then submits via createSchema.
 */
export type InferSchemaResponse = {
    proposal: CreateTelemetrySchemaRequest;
    nestedProposals: CreateTelemetrySchemaRequest[];
    warnings: InferenceWarningResponse[];
};

/**
 * Mirrors the server's MigrationStepKind enum (serialized as the enum name).
 *  - AddColumn: ALTER TABLE ADD COLUMN. Non-destructive.
 *  - DropColumn: ALTER TABLE DROP COLUMN. Destroys all stored history for the column.
 *  - DropAndAddColumn: drop + readd (used for both renames and type changes).
 *  - NewTableLocation: table or database name changed; old data left in place,
 *    new table created lazily on first event.
 */
export type MigrationStepKind = 'AddColumn' | 'DropColumn' | 'DropAndAddColumn' | 'NewTableLocation';

/** One DDL step that would run if the migration plan is applied. */
export type MigrationStepResponse = {
    kind: MigrationStepKind;
    columnName: string;
    detail?: string;
};

/**
 * Response from POST /api/v1/telemetry-schemas/preview-migration. Empty steps +
 * warnings means the create/approve will run without DDL (first version, or no
 * migrator registered for the active backend).
 */
export type MigrationPlanResponse = {
    eventName: string;
    tableName: string;
    schemaName?: string;
    isDestructive: boolean;
    warnings: string[];
    steps: MigrationStepResponse[];
};

// #endregion -- Telemetry Schema Types --

// #region -- Telemetry Schema Request APIs --

const SCHEMAS_ROOT = `/api/v1/telemetry-schemas`;

/**
 * Lists the latest version of every telemetry schema.
 * @param includeNested When true, includes synthetic nested schemas (those with
 * EventName starting `__nested.`). Defaults to false so the list shows only
 * top-level event types.
 */
export async function getAllSchemas(includeNested: boolean = false): Promise<GetTelemetrySchemaResponse[]> {
    const response = await backend.fetch.get(SCHEMAS_ROOT, { params: { includeNested } });
    return response.data as GetTelemetrySchemaResponse[];
}

/**
 * Fetches the latest schema for a single event.
 */
export async function getSchema(eventName: string): Promise<GetTelemetrySchemaResponse> {
    const response = await backend.fetch.get(`${SCHEMAS_ROOT}/${encodeURIComponent(eventName)}`);
    return response.data as GetTelemetrySchemaResponse;
}

/**
 * Fetches the full version history for an event, latest first.
 */
export async function getSchemaVersions(eventName: string): Promise<GetTelemetrySchemaResponse[]> {
    const response = await backend.fetch.get(`${SCHEMAS_ROOT}/${encodeURIComponent(eventName)}/versions`);
    return response.data as GetTelemetrySchemaResponse[];
}

/**
 * Fetches a specific version of an event's schema.
 */
export async function getSchemaVersion(eventName: string, version: number): Promise<GetTelemetrySchemaResponse> {
    const response = await backend.fetch.get(`${SCHEMAS_ROOT}/${encodeURIComponent(eventName)}/versions/${version}`);
    return response.data as GetTelemetrySchemaResponse;
}

/**
 * Creates a new schema version. Used by both authoring modes — the explicit form
 * submits a manually filled body, and the from-sample mode submits the proposal
 * returned by inferSchema after the admin has reviewed/edited it.
 */
export async function createSchema(request: CreateTelemetrySchemaRequest): Promise<GetTelemetrySchemaResponse> {
    const response = await backend.fetch.post(SCHEMAS_ROOT, request);
    return response.data as GetTelemetrySchemaResponse;
}

/**
 * Deletes a specific schema version. The latest remaining version becomes active.
 */
export async function deleteSchemaVersion(eventName: string, version: number): Promise<void> {
    await backend.fetch.delete(`${SCHEMAS_ROOT}/${encodeURIComponent(eventName)}/versions/${version}`);
}

/**
 * Lists all pending schema updates. These come from auto-detected drift
 * (currently the reflection-based startup scanner) and require admin triage.
 */
export async function getPendingUpdates(includeNested: boolean = false): Promise<GetPendingSchemaUpdateResponse[]> {
    const response = await backend.fetch.get(`${SCHEMAS_ROOT}/pending`, { params: { includeNested } });
    return response.data as GetPendingSchemaUpdateResponse[];
}

/**
 * Fetches a single pending update by event name.
 */
export async function getPendingUpdate(eventName: string): Promise<GetPendingSchemaUpdateResponse> {
    const response = await backend.fetch.get(`${SCHEMAS_ROOT}/pending/${encodeURIComponent(eventName)}`);
    return response.data as GetPendingSchemaUpdateResponse;
}

/**
 * Approves a pending update, creating a new active schema version.
 * Returns the newly created schema.
 */
export async function approvePendingUpdate(eventName: string): Promise<GetTelemetrySchemaResponse> {
    const response = await backend.fetch.post(`${SCHEMAS_ROOT}/pending/${encodeURIComponent(eventName)}/approve`, {});
    return response.data as GetTelemetrySchemaResponse;
}

/**
 * Rejects (deletes) a pending update without creating a schema.
 */
export async function rejectPendingUpdate(eventName: string): Promise<void> {
    await backend.fetch.post(`${SCHEMAS_ROOT}/pending/${encodeURIComponent(eventName)}/reject`, {});
}

/**
 * Walks a sample JSON payload server-side and returns a schema proposal plus any
 * nested-object proposals and inference warnings. Does NOT write — the response
 * pre-fills the authoring form so the admin can review/edit and submit via
 * createSchema. Body limit on the server is 256 KB.
 */
export async function inferSchema(request: InferSchemaRequest): Promise<InferSchemaResponse> {
    const response = await backend.fetch.post(`${SCHEMAS_ROOT}/infer`, request);
    return response.data as InferSchemaResponse;
}

/**
 * Dry-run computation of the migration plan that would run if `request` were
 * approved/created right now. No DDL executes, no metadata is written. The
 * dashboard uses this to surface a confirmation dialog before destructive
 * operations (column drops, type changes, table renames). An empty plan
 * (no steps) means "nothing to confirm — submit safely".
 */
export async function previewMigration(request: CreateTelemetrySchemaRequest): Promise<MigrationPlanResponse> {
    const response = await backend.fetch.post(`${SCHEMAS_ROOT}/preview-migration`, request);
    return response.data as MigrationPlanResponse;
}

// #endregion -- Telemetry Schema Request APIs --
