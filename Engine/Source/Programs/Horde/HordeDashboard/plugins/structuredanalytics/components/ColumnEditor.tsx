// Copyright Epic Games, Inc. All Rights Reserved.

import {
    DefaultButton,
    Dropdown,
    IDropdownOption,
    IconButton,
    Stack,
    TextField,
    Toggle
} from "@fluentui/react";
import React, { useCallback } from "react";
import { CreateSchemaColumnRequest, SchemaDataType } from "../api";

const DATA_TYPE_OPTIONS: IDropdownOption[] = [
    { key: 'String', text: 'String' },
    { key: 'Int64', text: 'Int64' },
    { key: 'Double', text: 'Double' },
    { key: 'Bool', text: 'Bool' },
    { key: 'DateTime', text: 'DateTime' },
    { key: 'Object', text: 'Object' },
    { key: 'Array', text: 'Array' }
];

// Element-type dropdown excludes Array (no nested arrays in the current schema model).
const ELEMENT_TYPE_OPTIONS: IDropdownOption[] = DATA_TYPE_OPTIONS.filter(o => o.key !== 'Array');

/**
 * Editor for a list of schema columns. Used by SchemaAuthorView (and any future
 * authoring surface). Each row exposes the full CreateSchemaColumnRequest shape;
 * conditional fields appear when DataType is Array (ArrayElementType) or Object
 * (NestedSchemaEventName) so the form stays compact for simple columns.
 */
export const ColumnEditor: React.FC<{
    columns: CreateSchemaColumnRequest[];
    onChange: (next: CreateSchemaColumnRequest[]) => void;
    /** Marks rows whose ColumnName is duplicated within the list — used for inline error styling. */
    duplicateColumnNames?: ReadonlySet<string>;
}> = ({ columns, onChange, duplicateColumnNames }) => {

    const update = useCallback((index: number, patch: Partial<CreateSchemaColumnRequest>) => {
        const next = columns.map((col, i) => {
            if (i !== index) {
                return col;
            }
            const merged = { ...col, ...patch };
            // Admin-authored schemas don't have a CLR property -> DB column mapping;
            // the JSON key the producer sends IS the column name. Keep PropertyName
            // and ColumnName in lockstep so the sink (which reads via PropertyName at
            // runtime) finds the right field and producers can rely on column == key.
            if (patch.columnName !== undefined) {
                merged.propertyName = patch.columnName;
            }
            return merged;
        });
        onChange(next);
    }, [columns, onChange]);

    const remove = useCallback((index: number) => {
        onChange(columns.filter((_, i) => i !== index));
    }, [columns, onChange]);

    const addColumn = useCallback(() => {
        onChange([
            ...columns,
            {
                propertyName: '',
                columnName: '',
                clrTypeName: '',
                dataType: 'String',
                isNullable: true
            }
        ]);
    }, [columns, onChange]);

    return (
        <Stack tokens={{ childrenGap: 8 }}>
            {/* Header row labels every always-present field. Conditional fields
                (Element type, Nested schema) carry their own placeholders since
                they appear in different rows depending on the column's DataType. */}
            {columns.length > 0 && <ColumnHeaderRow />}

            {columns.map((col, idx) => {
                const dup = !!col.columnName && duplicateColumnNames?.has(col.columnName);
                return (
                    <Stack
                        key={idx}
                        horizontal
                        tokens={{ childrenGap: 8 }}
                        verticalAlign="center"
                        wrap
                    >
                        <TextField
                            ariaLabel="Column name"
                            placeholder="job_id"
                            value={col.columnName}
                            onChange={(_, v) => update(idx, { columnName: v ?? '' })}
                            errorMessage={dup ? 'Duplicate column name' : undefined}
                            styles={{ root: { width: 220 } }}
                        />
                        <Dropdown
                            ariaLabel="Data type"
                            selectedKey={col.dataType}
                            options={DATA_TYPE_OPTIONS}
                            onChange={(_, opt) => opt && update(idx, {
                                dataType: opt.key as SchemaDataType,
                                // Reset conditional fields when the type changes so stale state
                                // doesn't get submitted (e.g. ArrayElementType lingering after switching to String).
                                arrayElementType: opt.key === 'Array' ? (col.arrayElementType ?? 'String') : undefined,
                                nestedSchemaEventName: opt.key === 'Object' ? col.nestedSchemaEventName : undefined
                            })}
                            styles={{ root: { width: 140 } }}
                        />
                        {col.dataType === 'Array' && (
                            <Dropdown
                                ariaLabel="Element type"
                                placeholder="Element type"
                                selectedKey={col.arrayElementType ?? 'String'}
                                options={ELEMENT_TYPE_OPTIONS}
                                onChange={(_, opt) => opt && update(idx, { arrayElementType: opt.key as SchemaDataType })}
                                styles={{ root: { width: 140 } }}
                            />
                        )}
                        {col.dataType === 'Object' && (
                            <TextField
                                ariaLabel="Nested schema event name"
                                placeholder="__nested.MyEvent.field"
                                value={col.nestedSchemaEventName ?? ''}
                                onChange={(_, v) => update(idx, { nestedSchemaEventName: v ?? '' })}
                                styles={{ root: { width: 260 } }}
                            />
                        )}
                        <Toggle
                            label="Nullable"
                            inlineLabel
                            checked={col.isNullable}
                            onChange={(_, v) => update(idx, { isNullable: !!v })}
                            styles={{ root: { marginBottom: 0 } }}
                        />
                        <IconButton
                            iconProps={{ iconName: 'Delete' }}
                            title="Remove column"
                            ariaLabel="Remove column"
                            onClick={() => remove(idx)}
                        />
                    </Stack>
                );
            })}
            <DefaultButton
                text="Add column"
                iconProps={{ iconName: 'Add' }}
                onClick={addColumn}
                styles={{ root: { alignSelf: 'flex-start', marginTop: 4 } }}
            />
        </Stack>
    );
};

/**
 * Static header row for the column editor. Widths must match the data-row controls
 * exactly so the labels sit above the right inputs. Conditional fields are
 * deliberately not labelled here — they only appear for some rows and get inline
 * placeholders instead.
 */
const ColumnHeaderRow: React.FC = () => {
    const labelStyle: React.CSSProperties = {
        fontSize: 12,
        fontWeight: 600,
        // Match Fluent's TextField label color closely enough that the header
        // reads as a label rather than primary text.
        opacity: 0.85
    };

    return (
        <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="center">
            <div style={{ width: 220, ...labelStyle }}>Column name</div>
            <div style={{ width: 140, ...labelStyle }}>Data type</div>
            {/* Conditional column area + Nullable + delete: no header, since
                these positions are not stable across rows with different types.
                Nullable carries its own inline label inside the toggle. */}
        </Stack>
    );
};

/**
 * Returns the set of column names that appear more than once in the list. Used
 * to drive inline error highlights without requiring the parent form to track
 * validation state imperatively.
 */
export function findDuplicateColumnNames(columns: CreateSchemaColumnRequest[]): Set<string> {
    const counts = new Map<string, number>();
    for (const col of columns) {
        if (!col.columnName) {
            continue;
        }
        counts.set(col.columnName, (counts.get(col.columnName) ?? 0) + 1);
    }

    const dupes = new Set<string>();
    counts.forEach((count, name) => {
        if (count > 1) {
            dupes.add(name);
        }
    });
    return dupes;
}
