// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from "@fluentui/react";
import React from "react";
import { ColumnChangeResponse, SchemaColumnResponse, SchemaComparisonResponse } from "../api";

/**
 * Side-by-side render of a SchemaComparison. Used by both PendingUpdatesView (to
 * show what an admin would be approving) and any future detail-view diff.
 *
 * Lives as a shared component so both call sites stay visually consistent.
 */
export const SchemaDiff: React.FC<{
    comparison: SchemaComparisonResponse;
    secondaryColor: string;
}> = ({ comparison, secondaryColor }) => {
    return (
        <Stack horizontal tokens={{ childrenGap: 24 }} wrap>
            <Stack tokens={{ childrenGap: 4 }} styles={{ root: { minWidth: 200 } }}>
                <Text variant="mediumPlus">Added ({comparison.addedColumns.length})</Text>
                {comparison.addedColumns.length === 0
                    ? <Text styles={{ root: { color: secondaryColor } }}>—</Text>
                    : comparison.addedColumns.map(c => <Text key={c}>+ {c}</Text>)}
            </Stack>
            <Stack tokens={{ childrenGap: 4 }} styles={{ root: { minWidth: 200 } }}>
                <Text variant="mediumPlus">Removed ({comparison.removedColumns.length})</Text>
                {comparison.removedColumns.length === 0
                    ? <Text styles={{ root: { color: secondaryColor } }}>—</Text>
                    : comparison.removedColumns.map(c => <Text key={c}>− {c}</Text>)}
            </Stack>
            <Stack tokens={{ childrenGap: 4 }} styles={{ root: { minWidth: 280 } }}>
                <Text variant="mediumPlus">Modified ({comparison.modifiedColumns.length})</Text>
                {comparison.modifiedColumns.length === 0
                    ? <Text styles={{ root: { color: secondaryColor } }}>—</Text>
                    : comparison.modifiedColumns.map(c => <ModifiedColumnRow key={c.columnName} change={c} secondaryColor={secondaryColor} />)}
            </Stack>
            {comparison.tableNameChanged && (
                <Stack tokens={{ childrenGap: 4 }} styles={{ root: { minWidth: 200 } }}>
                    <Text variant="mediumPlus">Table renamed</Text>
                    <Text styles={{ root: { color: secondaryColor } }}>
                        {comparison.oldTableName ?? '∅'} → (proposed)
                    </Text>
                </Stack>
            )}
        </Stack>
    );
};

/**
 * Compact "old → new" delta row for a single modified column.
 */
const ModifiedColumnRow: React.FC<{ change: ColumnChangeResponse; secondaryColor: string }> = ({ change, secondaryColor }) => {
    const parts: string[] = [];
    if (change.oldDataType && change.newDataType && change.oldDataType !== change.newDataType) {
        parts.push(`type: ${change.oldDataType} → ${change.newDataType}`);
    }
    if (change.oldIsNullable !== change.newIsNullable && change.oldIsNullable !== undefined && change.newIsNullable !== undefined) {
        parts.push(`nullable: ${change.oldIsNullable} → ${change.newIsNullable}`);
    }
    if (change.oldClrTypeName !== change.newClrTypeName) {
        parts.push(`clr: ${change.oldClrTypeName ?? '∅'} → ${change.newClrTypeName ?? '∅'}`);
    }

    return (
        <Stack>
            <Text>~ {change.columnName}</Text>
            {parts.length > 0 && (
                <Text styles={{ root: { color: secondaryColor, paddingLeft: 12 } }}>
                    {parts.join(' · ')}
                </Text>
            )}
        </Stack>
    );
};

/**
 * Read-only listing of proposed/active columns. Used to show "what columns will
 * exist after approval" or "what columns the active schema has".
 */
export const ProposedColumnsBlock: React.FC<{
    columns: SchemaColumnResponse[];
    secondaryColor: string;
    title?: string;
}> = ({ columns, secondaryColor, title }) => {
    if (!columns?.length) {
        return null;
    }
    return (
        <Stack tokens={{ childrenGap: 4 }}>
            <Text variant="mediumPlus">{title ?? `Proposed columns (${columns.length})`}</Text>
            <Stack tokens={{ childrenGap: 2 }} styles={{ root: { paddingLeft: 8 } }}>
                {columns.map(col => (
                    <Text key={col.columnName} styles={{ root: { color: secondaryColor } }}>
                        {col.columnName}: {col.dataType}
                        {col.arrayElementType ? `<${col.arrayElementType}>` : ''}
                        {col.isNullable ? ' (nullable)' : ''}
                        {col.nestedSchemaEventName ? ` → ${col.nestedSchemaEventName}` : ''}
                    </Text>
                ))}
            </Stack>
        </Stack>
    );
};
