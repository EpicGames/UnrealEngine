// Copyright Epic Games, Inc. All Rights Reserved.

import { MountType, registerHordePlugin } from "hordePlugins";
import { PendingUpdatesView } from "./views/PendingUpdatesView";
import { SchemaAuthorView } from "./views/SchemaAuthorView";
import { SchemaDetailView } from "./views/SchemaDetailView";
import { SchemaListView } from "./views/SchemaListView";

// Single TopNav entry. The Schemas list page is the hub: it links to New Schema,
// Pending Updates, and per-event detail views via in-page buttons.
registerHordePlugin({
    id: "structuredanalytics",
    routes: [
        { path: "structuredanalytics/schemas", element: <SchemaListView /> },
        { path: "structuredanalytics/schemas/new", element: <SchemaAuthorView /> },
        { path: "structuredanalytics/schemas/pending", element: <PendingUpdatesView /> },
        { path: "structuredanalytics/schemas/:eventName", element: <SchemaDetailView /> }
    ],
    mount: {
        type: MountType.TopNav,
        context: "Tools",
        text: "Analytics - Schemas",
        route: "/structuredanalytics/schemas"
    }
});
