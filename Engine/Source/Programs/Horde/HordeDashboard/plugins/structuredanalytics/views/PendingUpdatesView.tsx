// Copyright Epic Games, Inc. All Rights Reserved.

import {
    DefaultButton,
    DetailsList,
    DetailsListLayoutMode,
    Dialog,
    DialogFooter,
    DialogType,
    IColumn,
    IconButton,
    MessageBar,
    MessageBarType,
    PrimaryButton,
    SelectionMode,
    Spinner,
    SpinnerSize,
    Stack,
    Text,
    Toggle
} from "@fluentui/react";
import React, { useCallback, useEffect, useMemo, useState } from "react";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import {
    CreateTelemetrySchemaRequest,
    GetPendingSchemaUpdateResponse,
    MigrationPlanResponse,
    approvePendingUpdate,
    getPendingUpdates,
    previewMigration,
    rejectPendingUpdate
} from "../api";
import { ProposedColumnsBlock, SchemaDiff } from "../components/SchemaDiff";

type ToastState = { type: 'success' | 'error'; message: string } | null;

/**
 * Lists pending schema updates and lets admins approve or reject each one.
 * Approval calls the existing POST /pending/{eventName}/approve endpoint, which
 * promotes the proposal to a new active schema version. Rejection deletes the
 * pending record without creating a schema.
 */
export const PendingUpdatesView: React.FC = () => {
    const { hordeClasses, modeColors } = getHordeStyling();

    const [pending, setPending] = useState<GetPendingSchemaUpdateResponse[]>([]);
    const [loading, setLoading] = useState<boolean>(true);
    const [error, setError] = useState<string | null>(null);
    const [toast, setToast] = useState<ToastState>(null);
    const [includeNested, setIncludeNested] = useState<boolean>(false);
    const [acting, setActing] = useState<Set<string>>(new Set());
    const [expanded, setExpanded] = useState<Set<string>>(new Set());
    const [confirmReject, setConfirmReject] = useState<GetPendingSchemaUpdateResponse | null>(null);

    // When set, the user clicked Approve on a pending update and the previewed
    // plan is destructive. The Dialog gates the actual approve call until the
    // user explicitly consents. Non-destructive plans skip this dialog.
    const [destructiveApproval, setDestructiveApproval] = useState<{
        item: GetPendingSchemaUpdateResponse;
        plan: MigrationPlanResponse;
    } | null>(null);

    const refresh = useCallback(async (nested: boolean) => {
        setLoading(true);
        setError(null);
        try {
            const data = await getPendingUpdates(nested);
            data.sort((a, b) => a.eventName.localeCompare(b.eventName));
            setPending(data);
        } catch (e) {
            setError(`Failed to load pending updates: ${(e as Error)?.message ?? String(e)}`);
        } finally {
            setLoading(false);
        }
    }, []);

    useEffect(() => {
        refresh(includeNested);
    }, [refresh, includeNested]);

    const beginAction = (eventName: string) => {
        setActing(prev => {
            const next = new Set(prev);
            next.add(eventName);
            return next;
        });
    };

    const endAction = (eventName: string) => {
        setActing(prev => {
            const next = new Set(prev);
            next.delete(eventName);
            return next;
        });
    };

    /**
     * The actual approval write. Only invoked after either a non-destructive
     * preview or explicit user consent on a destructive one.
     */
    const doApprove = useCallback(async (item: GetPendingSchemaUpdateResponse) => {
        beginAction(item.eventName);
        try {
            const created = await approvePendingUpdate(item.eventName);
            setToast({ type: 'success', message: `Approved ${created.eventName} v${created.version}` });
            await refresh(includeNested);
        } catch (e) {
            setToast({ type: 'error', message: `Approve failed: ${(e as Error)?.message ?? String(e)}` });
        } finally {
            endAction(item.eventName);
        }
    }, [refresh, includeNested]);

    /**
     * Translates a pending update into a CreateTelemetrySchemaRequest shape so
     * the same /preview-migration endpoint handles both authoring and approval
     * flows uniformly.
     */
    const buildRequestFromPending = (item: GetPendingSchemaUpdateResponse): CreateTelemetrySchemaRequest => ({
        eventName: item.eventName,
        tableName: item.proposedTableName,
        schemaName: item.proposedSchemaName,
        clrTypeName: item.clrTypeName,
        isNestedSchema: item.isNestedSchema,
        columns: item.proposedColumns.map(c => ({
            propertyName: c.propertyName,
            columnName: c.columnName,
            clrTypeName: c.clrTypeName,
            dataType: c.dataType,
            arrayElementType: c.arrayElementType,
            nestedSchemaEventName: c.nestedSchemaEventName,
            isNullable: c.isNullable
        }))
    });

    /**
     * Approve click handler. Always previews the migration first; destructive
     * plans surface in the confirmation Dialog. Non-destructive plans go straight
     * through to doApprove.
     */
    const onApprove = useCallback(async (item: GetPendingSchemaUpdateResponse) => {
        beginAction(item.eventName);
        try {
            const plan = await previewMigration(buildRequestFromPending(item));
            if (plan.isDestructive) {
                setDestructiveApproval({ item, plan });
                endAction(item.eventName);
                return;
            }
            // Endaction lives inside doApprove's finally — keep it acting until then.
            await doApprove(item);
        } catch (e) {
            setToast({ type: 'error', message: `Migration preview failed: ${(e as Error)?.message ?? String(e)}` });
            endAction(item.eventName);
        }
    }, [doApprove]);

    const onConfirmDestructiveApproval = useCallback(async () => {
        const ctx = destructiveApproval;
        setDestructiveApproval(null);
        if (ctx) {
            await doApprove(ctx.item);
        }
    }, [destructiveApproval, doApprove]);

    const onReject = useCallback(async (item: GetPendingSchemaUpdateResponse) => {
        setConfirmReject(null);
        beginAction(item.eventName);
        try {
            await rejectPendingUpdate(item.eventName);
            setToast({ type: 'success', message: `Rejected pending update for ${item.eventName}` });
            await refresh(includeNested);
        } catch (e) {
            setToast({ type: 'error', message: `Reject failed: ${(e as Error)?.message ?? String(e)}` });
        } finally {
            endAction(item.eventName);
        }
    }, [refresh, includeNested]);

    const toggleExpanded = useCallback((eventName: string) => {
        setExpanded(prev => {
            const next = new Set(prev);
            if (next.has(eventName)) {
                next.delete(eventName);
            } else {
                next.add(eventName);
            }
            return next;
        });
    }, []);

    const columns: IColumn[] = useMemo(() => [
        {
            key: 'expand',
            name: '',
            minWidth: 28,
            maxWidth: 28,
            onRender: (item: GetPendingSchemaUpdateResponse) => (
                <IconButton
                    iconProps={{ iconName: expanded.has(item.eventName) ? 'ChevronDown' : 'ChevronRight' }}
                    title={expanded.has(item.eventName) ? 'Collapse' : 'Expand'}
                    ariaLabel="Toggle details"
                    onClick={() => toggleExpanded(item.eventName)}
                />
            )
        },
        {
            key: 'eventName',
            name: 'Event Name',
            fieldName: 'eventName',
            minWidth: 220,
            maxWidth: 360,
            isResizable: true
        },
        {
            key: 'changeType',
            name: 'Type',
            fieldName: 'changeType',
            minWidth: 80,
            maxWidth: 100,
            isResizable: true
        },
        {
            key: 'changeDescription',
            name: 'Change',
            fieldName: 'changeDescription',
            minWidth: 280,
            isResizable: true,
            isMultiline: true
        },
        {
            key: 'proposedVersion',
            name: 'Proposed Ver.',
            fieldName: 'proposedVersion',
            minWidth: 90,
            maxWidth: 120,
            isResizable: true
        },
        {
            key: 'detectedAtUtc',
            name: 'Detected (UTC)',
            fieldName: 'detectedAtUtc',
            minWidth: 160,
            maxWidth: 220,
            isResizable: true,
            onRender: (item: GetPendingSchemaUpdateResponse) =>
                new Date(item.detectedAtUtc).toISOString().replace('T', ' ').slice(0, 19)
        },
        {
            key: 'actions',
            name: 'Actions',
            minWidth: 220,
            maxWidth: 240,
            onRender: (item: GetPendingSchemaUpdateResponse) => {
                const inFlight = acting.has(item.eventName);
                return (
                    <Stack horizontal tokens={{ childrenGap: 8 }}>
                        <PrimaryButton
                            text="Approve"
                            disabled={inFlight}
                            onClick={() => onApprove(item)}
                        />
                        <DefaultButton
                            text="Reject"
                            disabled={inFlight}
                            onClick={() => setConfirmReject(item)}
                        />
                    </Stack>
                );
            }
        }
    ], [acting, expanded, onApprove, toggleExpanded]);

    // DetailsList does not natively support per-row expansion in v8; we render an
    // expanded panel beneath the list when the user toggles a chevron. Keeping it
    // simple beats fighting the IDetailsRowProps render override for v0.1.
    const expandedItems = pending.filter(p => expanded.has(p.eventName));

    return (
        <Stack className={hordeClasses.horde}>
            <TopNav />
            <Breadcrumbs items={[{ text: 'Structured Analytics' }, { text: 'Pending Updates' }]} />
            <Stack horizontal>
                <div style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack grow styles={{ root: { width: '100%', padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }} styles={{ root: { paddingBottom: 8 } }}>
                        <Text variant="xLarge">Pending Schema Updates</Text>
                        <Stack.Item grow><span /></Stack.Item>
                        <Toggle
                            label="Include nested"
                            inlineLabel
                            checked={includeNested}
                            onChange={(_, checked) => setIncludeNested(!!checked)}
                        />
                        <DefaultButton
                            text="Refresh"
                            iconProps={{ iconName: 'Refresh' }}
                            onClick={() => refresh(includeNested)}
                        />
                    </Stack>

                    {toast && (
                        <MessageBar
                            messageBarType={toast.type === 'success' ? MessageBarType.success : MessageBarType.error}
                            onDismiss={() => setToast(null)}
                            styles={{ root: { marginBottom: 8 } }}
                        >
                            {toast.message}
                        </MessageBar>
                    )}

                    {error && (
                        <MessageBar messageBarType={MessageBarType.error} styles={{ root: { marginBottom: 8 } }}>
                            {error}
                        </MessageBar>
                    )}

                    {loading ? (
                        <Spinner size={SpinnerSize.large} styles={{ root: { paddingTop: 16 } }} />
                    ) : pending.length === 0 ? (
                        <Text styles={{ root: { paddingTop: 16, color: modeColors.textSecondary } }}>
                            No pending updates. Auto-detected schema changes will appear here for review.
                        </Text>
                    ) : (
                        <>
                            <DetailsList
                                items={pending}
                                columns={columns}
                                selectionMode={SelectionMode.none}
                                layoutMode={DetailsListLayoutMode.justified}
                                getKey={(item: GetPendingSchemaUpdateResponse) => item.eventName}
                            />

                            {expandedItems.map(item => (
                                <PendingDetailPanel
                                    key={item.eventName}
                                    item={item}
                                    background={modeColors.content}
                                    secondary={modeColors.textSecondary}
                                />
                            ))}
                        </>
                    )}
                </Stack>
            </Stack>

            <Dialog
                hidden={!confirmReject}
                onDismiss={() => setConfirmReject(null)}
                dialogContentProps={{
                    type: DialogType.normal,
                    title: 'Reject pending update?',
                    subText: confirmReject
                        ? `This permanently deletes the pending update for "${confirmReject.eventName}". The active schema (if any) is not affected.`
                        : ''
                }}
            >
                <DialogFooter>
                    <PrimaryButton
                        text="Reject"
                        onClick={() => confirmReject && onReject(confirmReject)}
                    />
                    <DefaultButton text="Cancel" onClick={() => setConfirmReject(null)} />
                </DialogFooter>
            </Dialog>

            <Dialog
                hidden={!destructiveApproval}
                onDismiss={() => setDestructiveApproval(null)}
                modalProps={{ isBlocking: true }}
                dialogContentProps={{
                    type: DialogType.normal,
                    title: 'Confirm destructive migration',
                    subText: destructiveApproval
                        ? `Approving this update for "${destructiveApproval.item.eventName}" will run irreversible DDL against the underlying table. Review carefully.`
                        : ''
                }}
                minWidth={560}
            >
                {destructiveApproval && (
                    <Stack tokens={{ childrenGap: 8 }}>
                        {destructiveApproval.plan.warnings.length > 0 && (
                            <Stack tokens={{ childrenGap: 2 }}>
                                <Text variant="mediumPlus">Warnings</Text>
                                {destructiveApproval.plan.warnings.map((w, i) => (
                                    <Text key={i}>· {w}</Text>
                                ))}
                            </Stack>
                        )}
                        {destructiveApproval.plan.steps.length > 0 && (
                            <Stack tokens={{ childrenGap: 2 }}>
                                <Text variant="mediumPlus">DDL steps ({destructiveApproval.plan.steps.length})</Text>
                                {destructiveApproval.plan.steps.map((s, i) => (
                                    <Text key={i}>
                                        {s.kind}
                                        {s.columnName ? ` · ${s.columnName}` : ''}
                                        {s.detail ? ` · ${s.detail}` : ''}
                                    </Text>
                                ))}
                            </Stack>
                        )}
                    </Stack>
                )}
                <DialogFooter>
                    <PrimaryButton text="Confirm and approve" onClick={onConfirmDestructiveApproval} />
                    <DefaultButton text="Cancel" onClick={() => setDestructiveApproval(null)} />
                </DialogFooter>
            </Dialog>
        </Stack>
    );
};

const PendingDetailPanel: React.FC<{
    item: GetPendingSchemaUpdateResponse;
    background: string;
    secondary: string;
}> = ({ item, background, secondary }) => {
    return (
        <Stack
            tokens={{ childrenGap: 8 }}
            styles={{ root: { padding: 12, marginTop: 4, marginBottom: 8, border: `1px solid ${background}` } }}
        >
            <Text variant="large">{item.eventName} — proposed v{item.proposedVersion}</Text>
            <Text styles={{ root: { color: secondary } }}>
                Table: {item.proposedTableName}
                {item.proposedSchemaName ? ` (database: ${item.proposedSchemaName})` : ''}
                {item.clrTypeName ? ` · CLR-backed (${item.clrTypeName})` : ' · admin-authored'}
            </Text>
            {item.comparison ? (
                <SchemaDiff comparison={item.comparison} secondaryColor={secondary} />
            ) : (
                <Text styles={{ root: { color: secondary } }}>
                    No prior version — this is a new schema. All proposed columns are listed below.
                </Text>
            )}
            <ProposedColumnsBlock columns={item.proposedColumns} secondaryColor={secondary} />
        </Stack>
    );
};
