// Copyright Epic Games, Inc. All Rights Reserved.

import {
    DefaultButton,
    Dialog,
    DialogFooter,
    DialogType,
    MessageBar,
    MessageBarType,
    Pivot,
    PivotItem,
    PrimaryButton,
    Spinner,
    SpinnerSize,
    Stack,
    Text,
    TextField,
    Toggle
} from "@fluentui/react";
import React, { useCallback, useMemo, useState } from "react";
import { useNavigate } from "react-router-dom";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import {
    CreateTelemetrySchemaRequest,
    InferenceWarningResponse,
    MigrationPlanResponse,
    createSchema,
    inferSchema,
    previewMigration
} from "../api";
import { ColumnEditor, findDuplicateColumnNames } from "../components/ColumnEditor";

type ToastState = { type: 'success' | 'error'; message: string } | null;

/**
 * Returns an empty-but-valid CreateTelemetrySchemaRequest. Used as the initial
 * Explicit-pivot state and as the reset point after a successful submission.
 */
function emptyRequest(): CreateTelemetrySchemaRequest {
    return {
        eventName: '',
        tableName: '',
        schemaName: undefined,
        columns: [],
        clrTypeName: undefined,
        isNestedSchema: false
    };
}

/**
 * Two-pivot authoring surface:
 *   - Explicit: form-driven creation, submits via createSchema.
 *   - From Sample: paste a JSON example, server infers a proposal, dashboard
 *     pre-fills the Explicit pivot's form with the proposal and surfaces any
 *     warnings. The admin then reviews/edits and submits via the same Explicit
 *     submit button — there is only one place that writes.
 *
 * Nested-schema proposals returned by inference are kept alongside the parent
 * proposal and submitted in dependency order on save (children first).
 */
export const SchemaAuthorView: React.FC = () => {
    const { hordeClasses, modeColors } = getHordeStyling();
    const navigate = useNavigate();

    // Explicit-pivot form state — the canonical data the form binds to. Both
    // pivots converge here: the Explicit pivot edits it directly, and the From-
    // Sample pivot replaces it after a successful inference.
    const [request, setRequest] = useState<CreateTelemetrySchemaRequest>(emptyRequest());
    const [nestedProposals, setNestedProposals] = useState<CreateTelemetrySchemaRequest[]>([]);
    const [warnings, setWarnings] = useState<InferenceWarningResponse[]>([]);

    // From-Sample pivot input state, isolated from the explicit form so a stray
    // edit on one pivot never corrupts the other.
    const [sampleEventName, setSampleEventName] = useState<string>('');
    const [sampleTableName, setSampleTableName] = useState<string>('');
    const [sampleJson, setSampleJson] = useState<string>('');

    const [activePivot, setActivePivot] = useState<string>('explicit');
    const [submitting, setSubmitting] = useState<boolean>(false);
    const [inferring, setInferring] = useState<boolean>(false);
    const [toast, setToast] = useState<ToastState>(null);

    // When set, the user clicked Submit and the previewed plan is destructive.
    // The Dialog shown for `pendingDestructivePlan` is the only path that calls doSubmit
    // for destructive changes — the user must explicitly confirm. Non-destructive
    // plans skip the dialog and submit straight through.
    const [pendingDestructivePlan, setPendingDestructivePlan] = useState<MigrationPlanResponse | null>(null);

    const duplicateColumnNames = useMemo(
        () => findDuplicateColumnNames(request.columns),
        [request.columns]
    );

    // Submit gate: the server validates again on POST, but client-side guards
    // catch obvious problems (no event name, no columns, dup column names)
    // before round-tripping.
    const submitDisabled = submitting
        || !request.eventName.trim()
        || !request.tableName.trim()
        || request.columns.length === 0
        || duplicateColumnNames.size > 0
        || request.columns.some(c => !c.propertyName.trim() || !c.columnName.trim());

    const onInfer = useCallback(async () => {
        if (!sampleEventName.trim()) {
            setToast({ type: 'error', message: 'Event name is required to infer.' });
            return;
        }

        let parsed: unknown;
        try {
            parsed = JSON.parse(sampleJson);
        } catch (e) {
            setToast({ type: 'error', message: `Sample JSON is not valid: ${(e as Error).message}` });
            return;
        }

        if (typeof parsed !== 'object' || parsed === null || Array.isArray(parsed)) {
            setToast({ type: 'error', message: 'Sample payload must be a JSON object at the top level.' });
            return;
        }

        setInferring(true);
        setToast(null);
        try {
            const result = await inferSchema({
                eventName: sampleEventName.trim(),
                tableName: sampleTableName.trim() || undefined,
                samplePayload: parsed
            });
            setRequest(result.proposal);
            setNestedProposals(result.nestedProposals);
            setWarnings(result.warnings);
            setActivePivot('explicit');
            setToast({
                type: 'success',
                message: `Inferred ${result.proposal.columns.length} columns and ${result.nestedProposals.length} nested schemas. Review and submit when ready.`
            });
        } catch (e) {
            setToast({ type: 'error', message: `Inference failed: ${(e as Error)?.message ?? String(e)}` });
        } finally {
            setInferring(false);
        }
    }, [sampleEventName, sampleTableName, sampleJson]);

    /**
     * The actual write path. Only invoked after either (a) the preview returned a
     * non-destructive plan, or (b) the user confirmed a destructive plan via the
     * dialog. Submits nested children first so the parent's NestedSchemaEventName
     * references resolve at lookup time.
     */
    const doSubmit = useCallback(async () => {
        setSubmitting(true);
        setToast(null);
        try {
            for (const nested of nestedProposals) {
                await createSchema(nested);
            }
            const created = await createSchema(request);
            setToast({ type: 'success', message: `Created ${created.eventName} v${created.version}` });
            navigate(`/structuredanalytics/schemas/${encodeURIComponent(created.eventName)}`);
        } catch (e) {
            setToast({ type: 'error', message: `Submit failed: ${(e as Error)?.message ?? String(e)}` });
        } finally {
            setSubmitting(false);
        }
    }, [request, nestedProposals, navigate]);

    /**
     * Submit click handler. Always previews the migration first so destructive
     * operations (drops, type changes, table renames) require explicit confirmation.
     * First-version schemas and additive-only changes get an empty plan and
     * proceed straight to doSubmit.
     */
    const onSubmit = useCallback(async () => {
        setSubmitting(true);
        setToast(null);
        try {
            const plan = await previewMigration(request);
            if (plan.isDestructive) {
                setPendingDestructivePlan(plan);
                setSubmitting(false);
                return;
            }
            // No destructive ops — skip the confirmation dialog.
            await doSubmit();
        } catch (e) {
            setToast({ type: 'error', message: `Migration preview failed: ${(e as Error)?.message ?? String(e)}` });
            setSubmitting(false);
        }
    }, [request, doSubmit]);

    const onConfirmDestructive = useCallback(async () => {
        setPendingDestructivePlan(null);
        await doSubmit();
    }, [doSubmit]);

    const resetForm = useCallback(() => {
        setRequest(emptyRequest());
        setNestedProposals([]);
        setWarnings([]);
        setSampleEventName('');
        setSampleTableName('');
        setSampleJson('');
        setToast(null);
    }, []);

    return (
        <Stack className={hordeClasses.horde}>
            <TopNav />
            <Breadcrumbs items={[
                { text: 'Structured Analytics' },
                { text: 'Schemas', link: '/structuredanalytics/schemas' },
                { text: 'New' }
            ]} />
            <Stack horizontal>
                <div style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack grow styles={{ root: { width: '100%', padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }} styles={{ root: { paddingBottom: 8 } }}>
                        <Text variant="xLarge">New Schema</Text>
                        <Stack.Item grow><span /></Stack.Item>
                        <DefaultButton text="Reset" iconProps={{ iconName: 'Clear' }} onClick={resetForm} disabled={submitting || inferring} />
                    </Stack>

                    {toast && (
                        <MessageBar
                            messageBarType={toast.type === 'success' ? MessageBarType.success : MessageBarType.error}
                            onDismiss={() => setToast(null)}
                            styles={{ root: { marginBottom: 8 } }}
                        >
                            {toast.message}
                        </MessageBar>
                    )}

                    <Pivot
                        selectedKey={activePivot}
                        onLinkClick={item => item && setActivePivot(item.props.itemKey ?? 'explicit')}
                    >
                        <PivotItem itemKey="explicit" headerText="Explicit">
                            <ExplicitForm
                                request={request}
                                setRequest={setRequest}
                                nestedProposals={nestedProposals}
                                setNestedProposals={setNestedProposals}
                                warnings={warnings}
                                duplicateColumnNames={duplicateColumnNames}
                                submitting={submitting}
                                submitDisabled={submitDisabled}
                                onSubmit={onSubmit}
                                secondary={modeColors.textSecondary}
                                contentBackground={modeColors.content}
                            />
                        </PivotItem>
                        <PivotItem itemKey="fromSample" headerText="From Sample">
                            <FromSampleForm
                                eventName={sampleEventName}
                                setEventName={setSampleEventName}
                                tableName={sampleTableName}
                                setTableName={setSampleTableName}
                                json={sampleJson}
                                setJson={setSampleJson}
                                onInfer={onInfer}
                                inferring={inferring}
                                secondary={modeColors.textSecondary}
                            />
                        </PivotItem>
                    </Pivot>
                </Stack>
            </Stack>

            <DestructiveMigrationDialog
                plan={pendingDestructivePlan}
                onConfirm={onConfirmDestructive}
                onCancel={() => setPendingDestructivePlan(null)}
            />
        </Stack>
    );
};

/**
 * Confirmation dialog shown before a destructive submit. Lists every warning
 * and every step so the admin sees exactly what'll happen before consenting.
 * Displayed only when the previewed plan has IsDestructive=true; non-destructive
 * plans bypass this dialog entirely.
 */
const DestructiveMigrationDialog: React.FC<{
    plan: MigrationPlanResponse | null;
    onConfirm: () => void;
    onCancel: () => void;
}> = ({ plan, onConfirm, onCancel }) => {
    return (
        <Dialog
            hidden={!plan}
            onDismiss={onCancel}
            modalProps={{ isBlocking: true }}
            dialogContentProps={{
                type: DialogType.normal,
                title: 'Confirm destructive migration',
                subText: plan
                    ? `Submitting this schema for "${plan.eventName}" will run irreversible DDL against the underlying table. Review carefully.`
                    : ''
            }}
            minWidth={560}
        >
            {plan && (
                <Stack tokens={{ childrenGap: 8 }}>
                    {plan.warnings.length > 0 && (
                        <Stack tokens={{ childrenGap: 2 }}>
                            <Text variant="mediumPlus">Warnings</Text>
                            {plan.warnings.map((w, i) => (
                                <Text key={i}>· {w}</Text>
                            ))}
                        </Stack>
                    )}
                    {plan.steps.length > 0 && (
                        <Stack tokens={{ childrenGap: 2 }}>
                            <Text variant="mediumPlus">DDL steps ({plan.steps.length})</Text>
                            {plan.steps.map((s, i) => (
                                <Text key={i}>
                                    {s.kind}
                                    {s.columnName ? ` · ${s.columnName}` : ''}
                                    {s.detail ? ` · ${s.detail}` : ''}
                                </Text>
                            ))}
                        </Stack>
                    )}
                </Stack>
            )}
            <DialogFooter>
                <PrimaryButton text="Confirm and submit" onClick={onConfirm} />
                <DefaultButton text="Cancel" onClick={onCancel} />
            </DialogFooter>
        </Dialog>
    );
};

/**
 * The Explicit pivot. Owns the parent form fields, the column editor for the
 * parent schema, an editable list of nested-schema proposals, and the submit
 * button. Both pivots ultimately route through this one to write.
 */
const ExplicitForm: React.FC<{
    request: CreateTelemetrySchemaRequest;
    setRequest: React.Dispatch<React.SetStateAction<CreateTelemetrySchemaRequest>>;
    nestedProposals: CreateTelemetrySchemaRequest[];
    setNestedProposals: React.Dispatch<React.SetStateAction<CreateTelemetrySchemaRequest[]>>;
    warnings: InferenceWarningResponse[];
    duplicateColumnNames: Set<string>;
    submitting: boolean;
    submitDisabled: boolean;
    onSubmit: () => void;
    secondary: string;
    contentBackground: string;
}> = ({ request, setRequest, nestedProposals, setNestedProposals, warnings, duplicateColumnNames, submitting, submitDisabled, onSubmit, secondary, contentBackground }) => {

    const updateNested = useCallback((idx: number, patch: Partial<CreateTelemetrySchemaRequest>) => {
        setNestedProposals(prev => prev.map((p, i) => i === idx ? { ...p, ...patch } : p));
    }, [setNestedProposals]);

    return (
        <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12 } }}>
            {warnings.length > 0 && (
                <MessageBar messageBarType={MessageBarType.warning}>
                    <Stack tokens={{ childrenGap: 4 }}>
                        <Text>
                            Inference produced {warnings.length} warning{warnings.length === 1 ? '' : 's'} —
                            review the columns below before submitting.
                        </Text>
                        {warnings.map((w, i) => (
                            <Text key={i}>· <strong>{w.path}</strong>: {w.message}</Text>
                        ))}
                    </Stack>
                </MessageBar>
            )}

            <Stack horizontal tokens={{ childrenGap: 12 }} wrap>
                <TextField
                    label="Event name"
                    required
                    value={request.eventName}
                    onChange={(_, v) => setRequest(r => ({ ...r, eventName: v ?? '' }))}
                    placeholder="State.Agent.Telemetry"
                    styles={{ root: { width: 280 } }}
                />
                <TextField
                    label="Table name"
                    required
                    value={request.tableName}
                    onChange={(_, v) => setRequest(r => ({ ...r, tableName: v ?? '' }))}
                    placeholder="state_agent_telemetry"
                    styles={{ root: { width: 240 } }}
                />
                <TextField
                    label="Database (optional)"
                    value={request.schemaName ?? ''}
                    onChange={(_, v) => setRequest(r => ({ ...r, schemaName: v ? v : undefined }))}
                    placeholder="(default)"
                    styles={{ root: { width: 200 } }}
                />
                <Toggle
                    label="Nested schema"
                    inlineLabel
                    checked={request.isNestedSchema}
                    onChange={(_, v) => setRequest(r => ({ ...r, isNestedSchema: !!v }))}
                />
            </Stack>

            <Stack tokens={{ childrenGap: 8 }}>
                <Text variant="mediumPlus">Columns ({request.columns.length})</Text>
                <ColumnEditor
                    columns={request.columns}
                    onChange={cols => setRequest(r => ({ ...r, columns: cols }))}
                    duplicateColumnNames={duplicateColumnNames}
                />
            </Stack>

            {nestedProposals.length > 0 && (
                <Stack tokens={{ childrenGap: 12 }} styles={{ root: { padding: 12, border: `1px solid ${contentBackground}` } }}>
                    <Text variant="mediumPlus">
                        Nested schemas ({nestedProposals.length})
                    </Text>
                    <Text styles={{ root: { color: secondary } }}>
                        These will be created first so the parent's <code>NestedSchemaEventName</code>
                        references resolve. Each nested schema can be edited independently below.
                    </Text>
                    {nestedProposals.map((nested, idx) => (
                        <Stack key={idx} tokens={{ childrenGap: 8 }} styles={{ root: { padding: 8, border: `1px dashed ${contentBackground}` } }}>
                            <Stack horizontal tokens={{ childrenGap: 12 }} wrap>
                                <TextField
                                    label="Event name"
                                    value={nested.eventName}
                                    onChange={(_, v) => updateNested(idx, { eventName: v ?? '' })}
                                    styles={{ root: { width: 320 } }}
                                />
                                <TextField
                                    label="Table name"
                                    value={nested.tableName}
                                    onChange={(_, v) => updateNested(idx, { tableName: v ?? '' })}
                                    styles={{ root: { width: 240 } }}
                                />
                            </Stack>
                            <ColumnEditor
                                columns={nested.columns}
                                onChange={cols => updateNested(idx, { columns: cols })}
                            />
                        </Stack>
                    ))}
                </Stack>
            )}

            <Stack horizontal tokens={{ childrenGap: 8 }}>
                <PrimaryButton
                    text={submitting ? 'Submitting…' : 'Submit'}
                    onClick={onSubmit}
                    disabled={submitDisabled}
                />
                {submitting && <Spinner size={SpinnerSize.small} />}
            </Stack>
        </Stack>
    );
};

/**
 * The From-Sample pivot. Single text field for the event name, optional table
 * override, and a large JSON editor. On Infer, the parent SchemaAuthorView
 * receives the proposal and switches to the Explicit pivot for review.
 */
const FromSampleForm: React.FC<{
    eventName: string;
    setEventName: (v: string) => void;
    tableName: string;
    setTableName: (v: string) => void;
    json: string;
    setJson: (v: string) => void;
    onInfer: () => void;
    inferring: boolean;
    secondary: string;
}> = ({ eventName, setEventName, tableName, setTableName, json, setJson, onInfer, inferring, secondary }) => {
    return (
        <Stack tokens={{ childrenGap: 12 }} styles={{ root: { paddingTop: 12 } }}>
            <Text styles={{ root: { color: secondary } }}>
                Paste one example payload. The server walks the JSON, returns a draft schema,
                and the dashboard pre-fills the Explicit tab so you can review and submit.
                Inference never writes — only the Submit button on the Explicit tab does.
            </Text>

            <Stack horizontal tokens={{ childrenGap: 12 }} wrap>
                <TextField
                    label="Event name"
                    required
                    value={eventName}
                    onChange={(_, v) => setEventName(v ?? '')}
                    placeholder="State.Agent.Telemetry"
                    styles={{ root: { width: 280 } }}
                />
                <TextField
                    label="Table name (optional)"
                    value={tableName}
                    onChange={(_, v) => setTableName(v ?? '')}
                    placeholder="(derived from event name)"
                    styles={{ root: { width: 280 } }}
                />
            </Stack>

            <TextField
                label="Sample JSON"
                multiline
                rows={18}
                value={json}
                onChange={(_, v) => setJson(v ?? '')}
                placeholder='{ "agentId": "testAgent", "userCpu": 3.05, "schemaVersion": 1 }'
                styles={{
                    root: { width: '100%' },
                    field: { fontFamily: 'monospace', fontSize: 12, minHeight: 280 }
                }}
            />

            <Stack horizontal tokens={{ childrenGap: 8 }}>
                <PrimaryButton
                    text={inferring ? 'Inferring…' : 'Infer'}
                    iconProps={{ iconName: 'BranchSearch' }}
                    onClick={onInfer}
                    disabled={inferring || !eventName.trim() || !json.trim()}
                />
                {inferring && <Spinner size={SpinnerSize.small} />}
            </Stack>
        </Stack>
    );
};
