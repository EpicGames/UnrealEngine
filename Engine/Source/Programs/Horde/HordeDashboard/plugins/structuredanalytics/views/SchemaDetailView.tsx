// Copyright Epic Games, Inc. All Rights Reserved.

import {
    DefaultButton,
    DetailsList,
    DetailsListLayoutMode,
    Dropdown,
    IColumn,
    IDropdownOption,
    MessageBar,
    MessageBarType,
    SelectionMode,
    Spinner,
    SpinnerSize,
    Stack,
    Text
} from "@fluentui/react";
import React, { useCallback, useEffect, useMemo, useState } from "react";
import { Link, useParams } from "react-router-dom";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import { GetTelemetrySchemaResponse, SchemaColumnResponse, getSchemaVersions } from "../api";

/**
 * Detail view for a single telemetry event. Lists all versions in a dropdown
 * (latest selected by default) and renders that version's columns and metadata.
 *
 * Reachable via /structuredanalytics/schemas/:eventName.
 */
export const SchemaDetailView: React.FC = () => {
    const { hordeClasses, modeColors } = getHordeStyling();

    // react-router gives us :eventName from the path. Fall back to empty so the
    // not-found state renders cleanly if the route is hit without a param.
    const { eventName: rawEventName } = useParams<{ eventName: string }>();
    const eventName = rawEventName ?? '';

    const [versions, setVersions] = useState<GetTelemetrySchemaResponse[]>([]);
    const [selectedVersion, setSelectedVersion] = useState<number | null>(null);
    const [loading, setLoading] = useState<boolean>(true);
    const [error, setError] = useState<string | null>(null);

    const refresh = useCallback(async () => {
        if (!eventName) {
            setError('Missing event name in URL');
            setLoading(false);
            return;
        }

        setLoading(true);
        setError(null);
        try {
            const data = await getSchemaVersions(eventName);
            // Server returns highest version first; preserve that order.
            setVersions(data);
            // Default to the latest version on first load (or when refreshing).
            setSelectedVersion(prev => {
                if (data.length === 0) {
                    return null;
                }
                if (prev !== null && data.some(v => v.version === prev)) {
                    return prev;
                }
                return data[0].version;
            });
        } catch (e) {
            setError(`Failed to load versions: ${(e as Error)?.message ?? String(e)}`);
        } finally {
            setLoading(false);
        }
    }, [eventName]);

    useEffect(() => {
        refresh();
    }, [refresh]);

    const selected = useMemo(
        () => versions.find(v => v.version === selectedVersion) ?? null,
        [versions, selectedVersion]
    );

    const versionOptions: IDropdownOption[] = useMemo(
        () => versions.map(v => ({
            key: v.version,
            text: `v${v.version} — ${new Date(v.createdAtUtc).toISOString().slice(0, 10)}${v === versions[0] ? ' (latest)' : ''}`
        })),
        [versions]
    );

    const columnTableColumns: IColumn[] = useMemo(() => [
        { key: 'order', name: '#', minWidth: 30, maxWidth: 40, onRender: (_: SchemaColumnResponse, idx?: number) => (idx ?? 0) + 1 },
        { key: 'columnName', name: 'Column', fieldName: 'columnName', minWidth: 160, maxWidth: 240, isResizable: true },
        { key: 'propertyName', name: 'Property', fieldName: 'propertyName', minWidth: 140, maxWidth: 220, isResizable: true },
        {
            key: 'dataType',
            name: 'Type',
            minWidth: 110,
            maxWidth: 160,
            isResizable: true,
            onRender: (item: SchemaColumnResponse) =>
                `${item.dataType}${item.arrayElementType ? `<${item.arrayElementType}>` : ''}`
        },
        {
            key: 'isNullable',
            name: 'Nullable',
            minWidth: 70,
            maxWidth: 90,
            isResizable: true,
            onRender: (item: SchemaColumnResponse) => item.isNullable ? 'yes' : 'no'
        },
        {
            key: 'nestedSchemaEventName',
            name: 'Nested Schema',
            fieldName: 'nestedSchemaEventName',
            minWidth: 200,
            maxWidth: 320,
            isResizable: true,
            onRender: (item: SchemaColumnResponse) =>
                item.nestedSchemaEventName
                    ? <Text>{item.nestedSchemaEventName}</Text>
                    : <Text styles={{ root: { color: modeColors.textSecondary } }}>—</Text>
        },
        {
            key: 'clrTypeName',
            name: 'CLR Type',
            fieldName: 'clrTypeName',
            minWidth: 200,
            isResizable: true,
            onRender: (item: SchemaColumnResponse) =>
                item.clrTypeName
                    ? <Text styles={{ root: { color: modeColors.textSecondary } }}>{item.clrTypeName}</Text>
                    : <Text styles={{ root: { color: modeColors.textSecondary } }}>—</Text>
        }
    ], [modeColors.textSecondary]);

    return (
        <Stack className={hordeClasses.horde}>
            <TopNav />
            <Breadcrumbs items={[
                { text: 'Structured Analytics' },
                { text: 'Schemas', link: '/structuredanalytics/schemas' },
                { text: eventName || '(unknown)' }
            ]} />
            <Stack horizontal>
                <div style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack grow styles={{ root: { width: '100%', padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }} styles={{ root: { paddingBottom: 8 } }}>
                        <Text variant="xLarge">{eventName}</Text>
                        <Stack.Item grow><span /></Stack.Item>
                        <DefaultButton
                            text="Refresh"
                            iconProps={{ iconName: 'Refresh' }}
                            onClick={refresh}
                        />
                        <Link to="/structuredanalytics/schemas/pending" style={{ textDecoration: 'none' }}>
                            <DefaultButton text="Pending Updates" iconProps={{ iconName: 'PageList' }} />
                        </Link>
                    </Stack>

                    {error && (
                        <MessageBar messageBarType={MessageBarType.error} styles={{ root: { marginBottom: 8 } }}>
                            {error}
                        </MessageBar>
                    )}

                    {loading ? (
                        <Spinner size={SpinnerSize.large} styles={{ root: { paddingTop: 16 } }} />
                    ) : versions.length === 0 ? (
                        <Text styles={{ root: { paddingTop: 16, color: modeColors.textSecondary } }}>
                            No schema versions found for this event.
                        </Text>
                    ) : (
                        <Stack tokens={{ childrenGap: 12 }}>
                            <Stack horizontal tokens={{ childrenGap: 16 }} verticalAlign="end">
                                <Dropdown
                                    label="Version"
                                    selectedKey={selectedVersion ?? undefined}
                                    options={versionOptions}
                                    onChange={(_, opt) => opt && setSelectedVersion(opt.key as number)}
                                    styles={{ root: { minWidth: 260 } }}
                                />
                                {selected && (
                                    <Stack tokens={{ childrenGap: 2 }}>
                                        <Text styles={{ root: { color: modeColors.textSecondary } }}>
                                            Table: <strong>{selected.tableName}</strong>
                                            {selected.schemaName ? ` · Database: ${selected.schemaName}` : ' · Database: (default)'}
                                        </Text>
                                        <Text styles={{ root: { color: modeColors.textSecondary } }}>
                                            Created {new Date(selected.createdAtUtc).toISOString().replace('T', ' ').slice(0, 19)} UTC
                                            {selected.approvedByUserId ? ` · Approved by ${selected.approvedByUserId}` : ''}
                                            {selected.clrTypeName ? ` · CLR-backed (${selected.clrTypeName})` : ' · Admin-authored'}
                                        </Text>
                                    </Stack>
                                )}
                            </Stack>

                            {selected && (
                                // Cap the list height so schemas with many columns scroll inside the
                                // page rather than pushing the rest of the layout off-screen. 70vh leaves
                                // room for the topnav, breadcrumbs, header row, and version dropdown.
                                <div style={{ maxHeight: '70vh', overflowY: 'auto', overflowX: 'hidden' }}>
                                    <DetailsList
                                        items={selected.columns}
                                        columns={columnTableColumns}
                                        selectionMode={SelectionMode.none}
                                        layoutMode={DetailsListLayoutMode.justified}
                                        getKey={(item: SchemaColumnResponse) => item.columnName}
                                    />
                                </div>
                            )}
                        </Stack>
                    )}
                </Stack>
            </Stack>
        </Stack>
    );
};
