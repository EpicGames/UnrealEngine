// Copyright Epic Games, Inc. All Rights Reserved.

import {
    DefaultButton,
    DetailsList,
    DetailsListLayoutMode,
    IColumn,
    MessageBar,
    MessageBarType,
    PrimaryButton,
    SelectionMode,
    Spinner,
    SpinnerSize,
    Stack,
    Text,
    Toggle
} from "@fluentui/react";
import React, { useCallback, useEffect, useMemo, useState } from "react";
import { Link } from "react-router-dom";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import { GetTelemetrySchemaResponse, getAllSchemas } from "../api";

/**
 * Lists every active telemetry schema (latest version per event). Admins use this
 * page to verify what schemas are registered and whether they came from the
 * reflection-based startup scanner (CLR-backed) or admin authoring (no CLR type).
 */
export const SchemaListView: React.FC = () => {
    const { hordeClasses, modeColors } = getHordeStyling();

    const [schemas, setSchemas] = useState<GetTelemetrySchemaResponse[]>([]);
    const [loading, setLoading] = useState<boolean>(true);
    const [error, setError] = useState<string | null>(null);
    const [includeNested, setIncludeNested] = useState<boolean>(false);

    const refresh = useCallback(async (nested: boolean) => {
        setLoading(true);
        setError(null);
        try {
            const data = await getAllSchemas(nested);
            // Sort by event name for stable display.
            data.sort((a, b) => a.eventName.localeCompare(b.eventName));
            setSchemas(data);
        } catch (e) {
            setError(`Failed to load schemas: ${(e as Error)?.message ?? String(e)}`);
        } finally {
            setLoading(false);
        }
    }, []);

    useEffect(() => {
        refresh(includeNested);
    }, [refresh, includeNested]);

    const columns: IColumn[] = useMemo(() => [
        {
            key: 'eventName',
            name: 'Event Name',
            fieldName: 'eventName',
            minWidth: 220,
            maxWidth: 360,
            isResizable: true,
            // Clickable link drills into the per-event detail view.
            onRender: (item: GetTelemetrySchemaResponse) => (
                <Link to={`/structuredanalytics/schemas/${encodeURIComponent(item.eventName)}`}>
                    {item.eventName}
                </Link>
            )
        },
        {
            key: 'tableName',
            name: 'Table',
            fieldName: 'tableName',
            minWidth: 180,
            maxWidth: 280,
            isResizable: true
        },
        {
            key: 'schemaName',
            name: 'Database',
            fieldName: 'schemaName',
            minWidth: 100,
            maxWidth: 160,
            isResizable: true,
            onRender: (item: GetTelemetrySchemaResponse) => item.schemaName ?? <Text styles={{ root: { color: modeColors.textSecondary, fontStyle: 'italic' } }}>(default)</Text>
        },
        {
            key: 'version',
            name: 'Version',
            fieldName: 'version',
            minWidth: 60,
            maxWidth: 80,
            isResizable: true
        },
        {
            key: 'columnCount',
            name: 'Columns',
            minWidth: 70,
            maxWidth: 90,
            isResizable: true,
            onRender: (item: GetTelemetrySchemaResponse) => item.columns.length
        },
        {
            key: 'source',
            name: 'Source',
            minWidth: 120,
            maxWidth: 160,
            isResizable: true,
            // ClrTypeName populated => reflection-detected; null => admin-authored.
            onRender: (item: GetTelemetrySchemaResponse) =>
                item.clrTypeName
                    ? <Text styles={{ root: { color: modeColors.textSecondary } }}>CLR-backed</Text>
                    : <Text>Admin-authored</Text>
        },
        {
            key: 'createdAtUtc',
            name: 'Created (UTC)',
            fieldName: 'createdAtUtc',
            minWidth: 160,
            maxWidth: 220,
            isResizable: true,
            onRender: (item: GetTelemetrySchemaResponse) => new Date(item.createdAtUtc).toISOString().replace('T', ' ').slice(0, 19)
        }
    ], [modeColors.textSecondary]);

    return (
        <Stack className={hordeClasses.horde}>
            <TopNav />
            <Breadcrumbs items={[{ text: 'Structured Analytics' }, { text: 'Schemas' }]} />
            <Stack horizontal>
                <div style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack grow styles={{ root: { width: '100%', padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }} styles={{ root: { paddingBottom: 8 } }}>
                        <Text variant="xLarge">Telemetry Schemas</Text>
                        <Stack.Item grow>
                            <span />
                        </Stack.Item>
                        <Toggle
                            label="Include nested schemas"
                            inlineLabel
                            checked={includeNested}
                            onChange={(_, checked) => setIncludeNested(!!checked)}
                        />
                        <DefaultButton
                            text="Refresh"
                            iconProps={{ iconName: 'Refresh' }}
                            onClick={() => refresh(includeNested)}
                        />
                        <Link to="/structuredanalytics/schemas/new" style={{ textDecoration: 'none' }}>
                            <PrimaryButton text="New Schema" iconProps={{ iconName: 'Add' }} />
                        </Link>
                        <Link to="/structuredanalytics/schemas/pending" style={{ textDecoration: 'none' }}>
                            <DefaultButton text="Pending Updates" iconProps={{ iconName: 'PageList' }} />
                        </Link>
                    </Stack>

                    {error && (
                        <MessageBar messageBarType={MessageBarType.error} styles={{ root: { marginBottom: 8 } }}>
                            {error}
                        </MessageBar>
                    )}

                    {loading ? (
                        <Spinner size={SpinnerSize.large} styles={{ root: { paddingTop: 16 } }} />
                    ) : schemas.length === 0 ? (
                        <Text styles={{ root: { paddingTop: 16, color: modeColors.textSecondary } }}>
                            No schemas registered yet. Auto-detected schemas appear here once the server's
                            reflection scanner has run, or admins can create one explicitly.
                        </Text>
                    ) : (
                        <DetailsList
                            items={schemas}
                            columns={columns}
                            selectionMode={SelectionMode.none}
                            layoutMode={DetailsListLayoutMode.justified}
                            getKey={(item: GetTelemetrySchemaResponse) => `${item.eventName}@${item.version}`}
                        />
                    )}
                </Stack>
            </Stack>
        </Stack>
    );
};
