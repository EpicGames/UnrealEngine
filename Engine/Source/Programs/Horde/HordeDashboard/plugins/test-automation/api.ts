// Copyright Epic Games, Inc. All Rights Reserved.

import backend from "horde/backend";
import { AuditLogEntry, AuditLogQuery, GetUserResponse } from "horde/backend/Api";

/// A test data response
export type GetTestDataResponse = {

	/// The unique id of the test data
	id: string;

	/// The key associated with this test data
	key: string;

	/// The commit id related to the job
	commitId: {name: string, order: number};

    /// The prefligth commit id if any
    preflightCommitId: string;

	/// The job id of this test data
	jobId: string;

	/// The step id of this test data
	stepId: string;

	/// The stream id of this test data
	streamId: string;

	/// The template ref id of the related job
	templateRefId: string;
}

/// A test data
export type TestData = GetTestDataResponse & {
    data: object;
}

/// A test name request
export type GetTestsRequest = {

    /// The id of the tests to request information from
    testIds: string[];
}

/// A test metadata request
export type GetMetadataRequest = {

    /// The ids of the test metadata to request information from
    metadataIds?: string[];

    /// The entries to search for
    entries?: Record<string, string>;
}

/// A test tag request
export type GetTestTagsRequest = {

    /// The ids of the test tags to request information from
    tagIds?: string[];

    /// The tag names to search for
    tagNames?: string[];
}

/// A test phase request
export type GetTestPhasesRequest = {

    /// The id of the tests to request information from
    testIds?: string[];

    /// The key of the tests to request information from
    testKeys?: string[];
}

/// Stream ids to get test and meta collections
export type GetStreamTestsRequest = {

    /// The id of the streams to request information from
    streamIds: string[];
}

/// Test session reference
export type GetTestSessionsRequest = {

    /// The ids of the streams to request information from
    streamIds: string[];

    /// The ids of the tests to request information from
    testIds?: string[];

    /// The ids of the metadata to request information from
    metaIds?: string[];
}

/// Test phase session reference
export type GetPhaseSessionsRequest = {

    /// The id of the streams to request information from
    streamIds: string[];

    /// The id of the test phases to request information from
    phaseIds?: string[];
}

/// Test description
export type TestSessionDescriptionResponse = {

    /// The team associated with this test
    team?: string;

    /// The harness associated with this test
    harness?: string;

    /// The result profile associated with this test
    profile?: string;

    /// The intent associated with this test
    intent?: string;
}

/// Test name reference
export type GetTestNameResponse = TestSessionDescriptionResponse & {

    /// The id of the test
    id: string;

    /// The name of the test
    name: string;

    /// The key associated with this test
    key: string;
}

/// Test metadata reference
export type GetTestMetadataResponse = {

    /// The id of the test metadata
    id: string;

    /// The key/value entries associated with the metadata
    entries: Record<string, string>;
}

/// Test tag reference
export type GetTestTagResponse = {

    /// The id of the test tag
    id: string;

    /// The name of the test tag
    name: string;
}

/// Describe the tests running in a stream
export type GetTestSessionStreamResponse = {

    /// The stream id
    streamId: string;

    /// Individual tests which run in the stream
    tests: GetTestNameResponse[];

    /// Test metadata that run in the stream
    testMetadata: GetTestMetadataResponse[];

    /// Test tags that run in the stream
    testTags: GetTestTagResponse[];
}

/// Test phase reference
export type GetPhaseResponse = {

    /// The id of the test phase
    id: string;

    /// The name of the test phase
    name: string;

    /// The key associated with this test phase
    key: string;
}

/// Descibe the phases running in a test
export type GetTestPhaseResponse = {

    /// The test id
    testId: string;

    /// The test name
    testName: string;

    /// The test key
    testKey: string;

    /// The test phases
    phases: GetPhaseResponse[];
}

/// Test outcome
export enum TestOutcome {

    /// The test was successful
    Success = "Success",

    /// The test failed
    Failure = "Failure",

    /// The test was skipped
    Skipped = "Skipped",

    /// The test had an unspecified result
    Unspecified = "Unspecified"
}

export enum TestPhaseOutcome {

    // The test phase failed
    Failed = "Failed",

    /// The test phase was successful
    Success = "Success",

    /// The test phase was not run
    NotRun = "NotRun",

    /// The test phase was interrupted
    Interrupted = "Interrupted",

    /// The test phase was skipped
    Skipped = "Skipped",

    /// The test phase was successful but with warning
    Warning = "Warning",

    /// The test phase outcome is unknown
    Unknown = "Unknown"
}

/// Error Fingerprint Entry
export type GetErrorFingerprintEntry = {

    /// The error fingerprint key
    key: string;

    /// The list of impacted phases
    phases: string[];
}

/// Describes test session outcome
export type GetTestSessionResponse = {

    /// The test session id
    id: string;

    /// The stream associated with the test session
    streamId: string;

    /// The metadata associated with the test session
    metadataId: string;

    /// The test data id associated with the test session
    testDataId: string;

    /// The commit id associated with the test session
    commitId: string;

    /// The commit order associated with the test session
    commitOrder: number;

    /// The tag ids associated with the test session
    tagIds?: string[]

    /// The duration of the test session in seconds
    duration: number;

    /// The date time at which the test session started
    startDateTime: string;

    /// The test name reference associated with the test session
    nameRef: string;

    /// The recipe associated with the test session
    recipeId?: string;

    /// The outcome of the test session
    outcome: TestOutcome;

    /// The total cound of phases
    phasesTotalCount: number;

    /// The count of succeeded phases
    phasesSucceededCount: number;

    /// The count of undefined phases
    phasesUndefinedCount: number;

    /// The count of failed phases
    phasesFailedCount: number;

    /// The job id associated with the test session
    jobId: string;

    /// The step job id associated with the test session
    stepId: string;

    /// The error finerprints associated with this test session
    errorFingerprints?: GetErrorFingerprintEntry[];
}

/// Describes test phase session outcome
export type GetTestPhaseSessionResponse = {

    /// The phase session id
    id: string;

    /// The stream associated with the phase session
    streamId: string;

    /// The metadata associated with the phase session
    metadataId: string;

    /// The test session id associated with the phase session
    sessionId: string;

    /// The commit id associated with the phase session
    commitId: string;

    /// The commit order associated with the phase session
    commitOrder: number;

    /// The duration of the phase session in seconds
    duration: number;

    /// The date time at which the phase session started
    startDateTime: string;

    /// The test phase reference associated with the phase session
    phaseRef: string;

    /// The outcome of the phase session
    outcome: TestPhaseOutcome;

    /// The job id associated with the phase session
    jobId: string;

    /// The step job id associated with the phase session
    stepId: string;

    /// The path to the event stream
    eventStreamPath?: string;

    /// Whether the test phase encountered at least one warning
    hasWarning?: boolean;

    /// The error finerprint from the event stream
    errorFingerprint?: string;

    /// The tag ids associated with the phase session
    tagIds?: string[]
}

export type TestSessionSummaryResponse = {

    /// The name of the test associated with the session
    testName: string;

    /// The date time at which the session started
    dateTime: string;
    
    /// The time duration of the session
    timeElapseSec: number;
    
    /// The total count of the phases to run during the session
    phasesTotalCount: number;
    
    /// The count of the phases that succeeded
    phasesSucceededCount: number;
    
    /// The count of the phases that which state stayed undefined
    phasesUndefinedCount: number;
    
    /// The count of the phases that failed
    phasesFailedCount: number;

    /// The command line used to run the session
    commandline?: string;
}

export type PhaseSessionDetailsResponse = {
    
    /// The name of the phase
    name: string;
    
    /// The key of the phase
    key: string;
    
    /// The date time at which the phase started
    dateTime: string;
    
    /// The duration of the phase
    timeElapseSec: number;
    
    /// The outcome of the phase run
    outcome: TestPhaseOutcome;

    /// The device keys associated with this phase
    deviceKeys: string[];
    
    /// The error fingerprint associated with the phase failure
    errorFingerprint?: string;
    
    /// Whether or not the phase has warning reported
    hasWarning?: boolean;
    
    /// The event stream path that details the phase events
    eventStreamPath?: string;

    /// The tag associated with the phase session
    tags?: string[]
}

export type DeviceDetailsResponse = {

    /// The name of the device
    name: string;

    /// The path to the log artifact
    appInstanceLogPath: string;

    /// The metadata associated with the device
    metadata: Record<string, string>;
}

export type TestSessionRecipeResponse = {

    /// The test node class
    nodeClass: string,

    /// The version of the test
    version: string,

    /// The timeout in minutes used by this recipe
    timeoutMinutes: number,

    /// The command line parameters associated with this recipe
    params?: Record<string, any>
}

export type TestSessionDetailsResponse = {

    /// The test session summary
    summary: TestSessionSummaryResponse;

    /// The phase sessions run during the test session
    phases: PhaseSessionDetailsResponse[];

    /// The device details utilized during this test session
    devices: Record<string, DeviceDetailsResponse>;

    /// The metadata associated with the test session
    metadata: Record<string, string>;

    /// The tags associated with the test session
    tags?: string[];

    /// The test description associated with the test session
    description?: TestSessionDescriptionResponse;

    /// The test recipe associated with the test session
    recipe?: TestSessionRecipeResponse;
}

export type JobStepTestDataItem = {

    /// The testdata id
    testDataId: string,

    /// The associated test key
    testKey: string,

    /// The associated test name
    testName: string,

    /// The associated test ref id
    testNameRef: string
}

export type TestRecipeRequest = {

    /// The recipe ids requested 
    recipeIds?: string[],

    /// The test ids associated with the recipe requested
    testIds?: string[],

    /// The stream ids associated with the recipe requested
    streamIds?: string[]
}

export type TestRecipeResponse = TestSessionRecipeResponse & {

    /// The test recipe id
    id: string,

    /// The test name reference
    nameRef: string,

    /// The stream id associated with this recipe
    streamId: string,

    /// The job template id associated with this recipe
    jobTemplateId: string,

    /// The job step node name associated with this recipe
    jobStep: string,

    /// The workflow id associated with this recipe
    workflowId?: string,
}

export type TestAuditPost = {

    /// Whether or not the test is scheduled to run
    enableSchedule: boolean,

    /// The date of the last audit (ISO formatted)
    lastAuditDate?: string,

    /// Whether or not the test is under audit
    isUnderAudit?: boolean,

    /// The owner of the test
    owner?: string,

    /// The customers of the test
    customers?: string[],

    /// The notification settings as stream/workflow pair
    notifications?: Record<string, string>,

    // The team associated with this test
    team?: string,

    /// The hardness associated with this test
    harness?: string,

    /// The intent associated with this test
    intent?: string,

    /// The profile associated with this test
    profile?: string,

    /// The notes associated with this audit post
    notes?: string
}

export type TestAuditPostRequest = TestAuditPost & {

    /// The test id associated with the audit post
    testId: string
}

export type TestAuditResponse = TestAuditPost & {

    /// The test id associated with this audit
    testId: string,

    /// The used id associated with this audit
    userId: string,

    /// The date of the audit post
    userInputDate: string,
}

export async function getTestDataV2(id: string): Promise<TestData> {
    return new Promise<TestData>((resolve, reject) => {
        backend.fetch.get(`/api/v3/testdata/${id}`, {
            params: {}
        }).then((value) => {
            resolve(value.data as TestData);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getStreamTestsV2(streamIds: string[]): Promise<GetTestSessionStreamResponse[]> {
    const request: GetStreamTestsRequest = { streamIds: streamIds };
    return new Promise<GetTestSessionStreamResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/streams`, request).then((value) => {
            resolve(value.data as GetTestSessionStreamResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestNamesV2(testIds: string[]): Promise<GetTestNameResponse[]> {
    const request: GetTestsRequest = { testIds: testIds };
    return new Promise<GetTestNameResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/tests`, request).then((value) => {
            resolve(value.data as GetTestNameResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestMetadataV2(metaIds?: string[], entries?: Record<string, string>): Promise<GetTestMetadataResponse[]> {
    const request: GetMetadataRequest = { metadataIds: metaIds, entries: entries };
    return new Promise<GetTestMetadataResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/metadata`, request).then((value) => {
            resolve(value.data as GetTestMetadataResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestTagsV2(tagIds?: string[], tagNames?: string[]): Promise<GetTestTagResponse[]> {
    const request: GetTestTagsRequest = { tagIds: tagIds, tagNames: tagNames };
    return new Promise<GetTestTagResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/tags`, request).then((value) => {
            resolve(value.data as GetTestTagResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestRecipesV2(recipeIds?: string[], testIds?: string[], streamIds?: string[]): Promise<TestRecipeResponse[]> {
    const request: TestRecipeRequest = { recipeIds: recipeIds, testIds: testIds, streamIds: streamIds };
    return new Promise<TestRecipeResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/recipes`, request).then((value) => {
            resolve(value.data as TestRecipeResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestPhasesV2(testIds?: string[], testKeys?: string[]): Promise<GetTestPhaseResponse[]> {
    const request: GetTestPhasesRequest = { testIds: testIds, testKeys: testKeys };
    return new Promise<GetTestPhaseResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/phases`, request).then((value) => {
            resolve(value.data as GetTestPhaseResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestSessionsV2(streamIds: string[], testIds?: string[], metaIds?: string[], minCreateTime?: string, maxCreateTime?: string, minChange?: string, maxChange?: string): Promise<GetTestSessionResponse[]> {
    const query: any = {};
    if (minCreateTime) {
        query.minCreateTime = minCreateTime;
    }
    if (maxCreateTime) {
        query.maxCreateTime = maxCreateTime;
    }
    if (minChange) {
        query.minChange = minChange;
    }
    if (maxChange) {
        query.maxChange = maxChange;
    }
    const request: GetTestSessionsRequest = { streamIds: streamIds };
    if (testIds) {
        request.testIds = testIds;
    }
    if (metaIds) {
        request.metaIds = metaIds;
    }
    return new Promise<GetTestSessionResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/session/tests`, request, { params: query }).then((value) => {
            resolve(value.data as GetTestSessionResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestPhaseSessionsV2(streamIds: string[], phaseIds?: string[], minCreateTime?: string, maxCreateTime?: string, minChange?: string, maxChange?: string): Promise<GetTestPhaseSessionResponse[]> {
    const query: any = {};
    if (minCreateTime) {
        query.minCreateTime = minCreateTime;
    }
    if (maxCreateTime) {
        query.maxCreateTime = maxCreateTime;
    }
    if (minChange) {
        query.minChange = minChange;
    }
    if (maxChange) {
        query.maxChange = maxChange;
    }
    const request: GetPhaseSessionsRequest = { streamIds: streamIds };
    if (phaseIds) {
        request.phaseIds = phaseIds;
    }
    return new Promise<GetTestPhaseSessionResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/session/phases`, request, { params: query }).then((value) => {
            resolve(value.data as GetTestPhaseSessionResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getJobStepTestDataV2(jobId: string, stepId: string): Promise<JobStepTestDataItem[]> {
    return new Promise<JobStepTestDataItem[]>((resolve, reject) => {
        backend.fetch.get(`/api/v3/testdata/`, {
            params: {jobId: jobId, stepId: stepId}
        }).then((value) => {
            resolve(value.data as JobStepTestDataItem[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function postTestAuditV2(testId: string, auditPost: TestAuditPost): Promise<TestAuditResponse> {
    const request: TestAuditPostRequest = { testId: testId, ...auditPost };
    return new Promise<TestAuditResponse>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/audit/post`, request).then((value) => {
            resolve(value.data as TestAuditResponse);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestAuditsV2(testIds: string[], minCreateTime?: string): Promise<TestAuditResponse[]> {
    const query: any = {};
    if (minCreateTime) {
        query.minCreateTime = minCreateTime;
    }
    const request = { testIds: testIds };
    return new Promise<TestAuditResponse[]>((resolve, reject) => {
        backend.fetch.post(`/api/v3/testdata/audit/history`, request, query).then((value) => {
            resolve(value.data as TestAuditResponse[]);
        }).catch((reason) => {
            reject(reason);
        });
    });
}

export async function getTestAuditLogHistoryV2(id: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
    return new Promise<AuditLogEntry[]>((resolve, reject) => {
        backend.fetch.get(`/api/v3/testdata/audit/log/${encodeURIComponent(id)}`, { params: query }).then((response) => {
            const history = (response.data?.entries ?? []) as AuditLogEntry[];
            resolve(history);
        }).catch(reason => { reject(reason); });
    });
}

export async function getUsersInfo(userIds: string[]): Promise<GetUserResponse[]> {
    const results = await backend.getUsers({
        ids: userIds,
        includeAvatar: true
    });

    return results;
}
