// Copyright Epic Games, Inc. All Rights Reserved.

import { registerHordePlugin, MountType } from "..";
import { TestAuditLogView } from "./testAuditLog";
import { TestAutomationView } from "./testAutomationView";

registerHordePlugin({
   id: "build",
   routes: [
      { path: "test-automation", element: <TestAutomationView /> },
      { path: "test-automation/log/:testId", element: <TestAuditLogView /> }
   ],
   mount: {
      type: MountType.TopNav,
      context: "Tools",
      text: "Test Automation Hub",
      route: `/test-automation`
   }
})