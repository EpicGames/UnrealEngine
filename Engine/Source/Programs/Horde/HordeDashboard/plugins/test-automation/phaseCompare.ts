// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { TestDataHandler, TestPhaseStatus, TestSessionDetails } from "./testData";
import { JobParameters } from "horde/components/ChangeButton";

export type JobContext = {job?: JobParameters, rangeCL?: number, step?: {id: string}};

export const getJobParams = (fetchedSessionDetails: TestSessionDetails, handler: TestDataHandler): JobContext => {
    return {
        job: {
            id: fetchedSessionDetails.jobId,
            streamId: fetchedSessionDetails.streamId,
            change: fetchedSessionDetails.commitOrder,
            preflightChange: fetchedSessionDetails.preflightCommitId ? Number.parseInt(fetchedSessionDetails.preflightCommitId) || undefined : undefined
        },
        rangeCL: fetchedSessionDetails.session ?
            handler.getStatusStream(fetchedSessionDetails.streamId)?.getPreviousSession(fetchedSessionDetails.test!, fetchedSessionDetails.meta!, fetchedSessionDetails.session)?.commitOrder
            : undefined,
        step : {id: fetchedSessionDetails.stepId}
    }
}

export class PhaseComparisonContext {
    constructor() {
        makeObservable(this);
        this.reset();
    }

    reset() {
        this.isEnable = false;
        this.commitId = undefined;
        this.metaKey = undefined;
        this.streamId = undefined;
        this.isLoadingPhase = false;
        this.phaseStatus = undefined;
        this.testSessionId = undefined;
        this.jobParams = undefined;
        this.setUpdated();
    }

    setTestSesssionId(id?: string) {
        if (this.testSessionId === id) return;
        this.testSessionId = id;
        this.setUpdated();
    }

    setMetaKey(key?: string) {
        if (this.metaKey === key) return;
        this.metaKey = key;
        this.setUpdated();
    }

    setStreamId(id?: string) {
        if (this.streamId === id) return;
        this.streamId = id;
        this.setUpdated();
    }

    setPhaseStatus(phase?: TestPhaseStatus) {
        if (this.phaseStatus === phase) return;
        this.phaseStatus = phase;
        if (!!phase) this.setLoadingPhase(false);
        this.setUpdated();
    }

    setLoadingPhase(flag: boolean) {
        this.isLoadingPhase = flag;
    }

    setEnable() {
        this.isEnable = true;
    }

    @action
    setUpdated() {
        this.updated++;
    }

    subscribe() {
        if (this.updated) { }
    }

    @observable
    updated: number = 0;

    isEnable: boolean = false;
    isLoadingPhase: boolean = false;
    commitId?: string;
    metaKey?: string;
    streamId?: string;
    phaseStatus?: TestPhaseStatus;
    testSessionId?: string;
    jobParams?: JobContext;
}
