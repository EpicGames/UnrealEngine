// Copyright Epic Games, Inc. All Rights Reserved.

import { ContextualMenuItemType } from "@fluentui/react";
import { TestDataHandler, TestPhaseStatus, MetadataRef, TestPhaseRef, TestMetaStatus, TestNameRef, defaultQueryWeeks, consecutivePassesThreshold } from "./testData";
import { CircularList, DomainNameColors, ICheckListOption, IPhaseFilter } from "./testAutomationCommon";
import { action, makeObservable, observable } from "mobx";
import { TestPhaseOutcome } from "./api";

export type MetadataItem = { meta: MetadataRef, name: string, orderMarker: string, visible: boolean };

export class PhaseStatusGroup implements IPhaseFilter {
    constructor(phase: TestPhaseRef) {
        this.phase = phase;
        this.metadata = new Map();
    }

    addMetadata(metaItem: MetadataItem, status: TestPhaseStatus): void {
        if (!this.hasMetadata(metaItem)) {
            this.metadata.set(metaItem, status);
            if (status.errorFingerprint) {
                this.addErrorFingerprint(status.errorFingerprint, metaItem);
            }
        }
    }

    getMetadataStatus(metaItem: MetadataItem): TestPhaseStatus | undefined {
        return this.metadata.get(metaItem);
    }

    hasMetadata(metaItem: MetadataItem): boolean {
        return this.metadata.has(metaItem);
    }

    hasOutcome(outcome: TestPhaseOutcome): boolean {
        return this.metadata.entries().filter(([metaItem,]) => metaItem.visible).some(([,status]) => status.outcome === outcome);
    }

    hasOutcomes(outcomes: Set<TestPhaseOutcome>): boolean {
        return this.metadata.entries().filter(([metaItem,]) => metaItem.visible).some(([,status]) => outcomes.has(status.outcome));
    }

    hasOnlyOutcomes(outcomes: Set<TestPhaseOutcome>): boolean {
        return this.metadata.entries().filter(([metaItem,]) => metaItem.visible).every(([,status]) => outcomes.has(status.outcome));
    }

    hasWarnings(): boolean {
        return this.metadata.entries().filter(([metaItem,]) => metaItem.visible).some(([,status]) => status.outcome === TestPhaseOutcome.Success && status.hasWarning);
    }

    isMatch(names: string[], outcomes?: Set<TestPhaseOutcome>, tags?: string[]): boolean {
        return this.metadata.entries().filter(([metaItem,]) => metaItem.visible).some(([,status]) => status.isMatch(names, outcomes, tags));
    }

    getMetadataWithOutcome(outcome: TestPhaseOutcome): MetadataRef[] {
        return Array.from(this.metadata.keys()).filter(metaItem => metaItem.visible && this.metadata.get(metaItem)?.outcome === outcome).map(metaItem => metaItem.meta);
    }

    getMetadataWithoutOutcomes(outcomes: Set<TestPhaseOutcome>): MetadataRef[] {
        return Array.from(this.metadata.keys()).filter(metaItem => metaItem.visible && !outcomes.has(this.metadata.get(metaItem)?.outcome ?? TestPhaseOutcome.Unknown)).map(metaItem => metaItem.meta);
    }

    getMetadataWithWarnings(): MetadataRef[] {
        return Array.from(this.metadata.keys()).filter(metaItem => metaItem.visible && this.metadata.get(metaItem)?.outcome === TestPhaseOutcome.Success && this.metadata.get(metaItem)?.hasWarning).map(metaItem => metaItem.meta);
    }

    addErrorFingerprint(errorFingerprint: string, metaItem: MetadataItem): void {
        if (!this.errorFingerprints) {
            this.errorFingerprints = new Map();
        }
        if (!this.errorFingerprints.has(errorFingerprint)) {
            this.errorFingerprints.set(errorFingerprint, []);
        }
        this.errorFingerprints.get(errorFingerprint)!.push(metaItem);
    }

    phase: TestPhaseRef;
    metadata: Map<MetadataItem, TestPhaseStatus>;
    componentRef?: any;
    errorFingerprints?: Map<string, MetadataItem[]>;
}

const notFailureOutcomes = new Set([TestPhaseOutcome.Success, TestPhaseOutcome.NotRun, TestPhaseOutcome.Skipped]);

export class MetaSelectorContext {
    constructor(handler: TestDataHandler, test?: TestNameRef) {
        this._handler = handler;
        this.test = test;
        this.metaValueColors = new DomainNameColors();
        this._metas = [];
        this._commonKeys = [];
        this._phaseGroups = [];
        this._filteredPhases = [];
        this.initiateContext();
        this._phaseCallbacks = new Map();
        this._filterOptions = this.generateFilterOptions();
        makeObservable(this);
    }

    @observable
    metaVisibility: number = 0;
    @action
    updateMetaVisibility() {
        this.metaVisibility++;
    }
    subscribeMetaVisibility() {
        if (this.metaVisibility) { }
    }

    @observable
    phaseGroupsUpdated: number = 0;
    @action
    updatePhaseGroups() {
        this.phaseGroupsUpdated++;
    }
    subscribePhaseGroups() {
        if (this.phaseGroupsUpdated) { }
    }

    @observable
    filteredPhasesUpdated: number = 0;
    @action
    updateFilteredPhases() {
        this.filteredPhasesUpdated++;
    }
    subscribeFilteredPhases() {
        if (this.filteredPhasesUpdated) { }
    }

    @observable
    selectedMetaUpdated: number = 0;
    @action
    updateSelectedMeta() {
        this.selectedMetaUpdated++;
    }
    subscribeSelectedMeta() {
        if (this.selectedMetaUpdated) { }
    }

    @observable
    selectedPhaseUpdated: number = 0;
    @action
    updateSelectedPhase() {
        this.selectedPhaseUpdated++;
    }
    subscribeSelectedPhase() {
        if (this.selectedPhaseUpdated) { }
    }

    @observable
    metaFilterUpdated: number = 0;
    @action
    updateMetaFilter() {
        this.metaFilterUpdated++;
    }
    subscribeMetaFilter() {
        if (this.metaFilterUpdated) { }
    }

    private initiateContext() {

        const filteredMeta = new Set(this._handler.filteredMetadata);
        const testStatus = MetaSelectorContext.getTestStatus(this._handler, this.test);

        const testMetadata = testStatus?.getMetadata().filter(meta => filteredMeta.has(meta)).map(meta => meta) ?? [];
        this._commonKeys = MetadataRef.identifyCommonKeys(testMetadata);
        this.commonMetaString  = testMetadata.at(0)?.getCommonValues(this._commonKeys)?.join(" / ");

        this._metas = testMetadata.map(m => {
            const name = m.getValuesExcept(this._commonKeys).join(" / ");
            return ({meta: m, name, orderMarker: name, visible: true});
        });

        const keys: string[] = [];
        const seen = new Set<string>();
        this._metas.forEach(metaItem => {
            metaItem.meta.orderedKeys.forEach(key => {
                if (!this._commonKeys.includes(key) && !seen.has(key)) {
                    seen.add(key);
                    keys.push(key);
                }
            });
        });
        this.availableMetaKeys = keys;

        const set = new Set(this._metas.flatMap(metaItem => metaItem.meta.getValuesExcept(this._commonKeys)));
        this.metaValueColors.setDomain(Array.from(set).sort((a, b) => a.localeCompare(b)));

        this.findForkingMetaKey();

    }

    findForkingMetaKey(): string | undefined {
        // Select the first meta key from allMetaKeys that gives different value across visible meta items
        const visibleMetas = this._metas.filter(m => m.visible);

        this.forkingMetaKey = this.availableMetaKeys?.find(key => {
            if (!key) return false;
            const firstVal = visibleMetas[0]?.meta.getValue?.(key);
            return visibleMetas.some(m => m.meta.getValue?.(key) !== firstVal);
        }) ?? this.availableMetaKeys?.at(-1); // fallback to first key if none differ

        visibleMetas.forEach(m => {
            m.orderMarker = !!this.forkingMetaKey ? m.meta.getValue(this.forkingMetaKey) ?? "" : m.name;
        });
        this._metas.sort((a, b) => a.orderMarker.localeCompare(b.orderMarker));    

        this.metaKeysBeforeFork = !!this.forkingMetaKey ? this.availableMetaKeys?.slice(0, this.availableMetaKeys.indexOf(this.forkingMetaKey)) : undefined;

        return this.forkingMetaKey;
    }

    getForkingMetaKeyValues(): string[] {
        const forkingKey = this.forkingMetaKey;
        if (!forkingKey) return [];
        return this._metas.filter(m => m.visible).map(item => item.meta.getValue(forkingKey)).filter((m, index, self) => self.indexOf(m) === index) as string[];
    }

    getMetaValuesBeforeForkingKey(): string[] {
        const forkingKey = this.forkingMetaKey;
        if (!forkingKey) return [];
        const visibleMetas = this._metas.filter(m => m.visible);
        return visibleMetas.map(m => this.metaKeysBeforeFork?.map(key => m.meta.getValue(key))).flat().filter((v, index, self) => !!v && self.indexOf(v) === index) as string[];
    }

    getMetasContainingValue(value: string): MetadataItem[] {
        return this._metas.filter(m => m.visible && m.meta.getValuesExcept(this._commonKeys).includes(value));
    }

    getVisibleMetasContainingForkingKeyValue(value: string): MetadataItem[] {
        return this._metas.filter(m => m.visible && m.meta.getValue(this.forkingMetaKey!) === value);
    }

    private generateFilterOptions(): ICheckListOption[] {
        // Group all possible metadata values by metadata key
        const groupedByKey: Map<string, Map<string, MetadataItem[]>> = new Map();
        this._metas.forEach(metaItem => {
            metaItem.meta.getKeys().forEach(key => {
                if (this._commonKeys.includes(key)) return;
                const value = metaItem.meta.getValue?.(key);
                if (value === undefined) return; // Skip keys that do not yield a value
                if (!groupedByKey.has(key)) groupedByKey.set(key, new Map());
                const group = groupedByKey.get(key)!;
                if (!group.has(value)) group.set(value, []);
                const groupItems = group.get(value)!;
                groupItems.push(metaItem);
            });
        });

        const gridFilterItems = this.getSearchGridFilterItems();

        // Section headers per key, and their meta values/items
        const options = this.availableMetaKeys!.map(k => ({key: k, values: groupedByKey.get(k)!})).flatMap((group) => [
            { key: `key-${group.key}`, itemType: ContextualMenuItemType.Header, text: group.key } as ICheckListOption, // Section header
            ...group.values.entries().toArray().sort((a, b) => a[0].localeCompare(b[0])).map(([value, items]) => {
                const option = {
                    key: `value-${value}-${group.key}`,
                    text: value,
                    checked: gridFilterItems?.includes(value) ?? false,
                    color: this.metaValueColors?.get(value),
                    data: {key: group.key, items: items},
                } as ICheckListOption;
                return option;
            })
        ]);

        this.updateMetasVisibility(options);

        return options;
    }

    get filterOptions(): ICheckListOption[] {
        return this._filterOptions;
    }

    updateMetasVisibility(items: ICheckListOption[]) {
        // Create a Map associated by item key
        const toCheckPerKey = new Map<string, ICheckListOption[]>();
        items.filter(i => i.itemType !== ContextualMenuItemType.Header && i.checked).forEach(i => {
            if (!i.data?.key) return;
            if (!toCheckPerKey.has(i.data.key)) {
                toCheckPerKey.set(i.data.key, []);
            }
            toCheckPerKey.get(i.data.key)!.push(i);
        });
        this._metas.forEach(m => {
            m.visible = toCheckPerKey.size === 0 || toCheckPerKey.values().every(group => group.some(i => i.data?.items.includes(m)));
        });
        this.updateMetaVisibility();
    }

    get searchUpdated(): number {
        return this._handler.searchUpdated;
    }

    get daysQueried(): number {
        return (this._handler.filterState.weeks ?? defaultQueryWeeks) * 7;
    }

    getCommonKeysToFork(): string[] {
        return this._commonKeys.concat(this.metaKeysBeforeFork ?? []);
    }

    static getTestStatus(handler: TestDataHandler, test?: TestNameRef): TestMetaStatus | undefined {
        return test && handler.getStatusStream(handler.stream!)?.tests.get(test);
    }

    getSearchGridFilterItems(): string[] | undefined {
        const gridFilter = this._handler.getSearchParam('gridfilter') as string | string[] | undefined;
        return !gridFilter ? undefined : Array.isArray(gridFilter) ? gridFilter : [gridFilter];
    }

    setSearchGridFilterItems(items: string[], replace?: boolean) {
        if (items) {
            this._handler.setSearchParam('gridfilter', items, replace);
        } else {
            this._handler.removeSearchParam('gridfilter', replace);
        }
    }

    get phaseGroups(): PhaseStatusGroup[] {
        return this._phaseGroups;
    }

    setPhaseGroups(groups: PhaseStatusGroup[]) {
        this._phaseGroups = groups;
        this.updatePhaseGroups();
    }

    get filteredPhases(): PhaseStatusGroup[] {
        return this._filteredPhases;
    }

    setFilteredPhases(filtered: PhaseStatusGroup[]) {
        this._filteredPhases = filtered;
        this.updateFilteredPhases();
        this.updateFilter(filtered);
    }

    get selectedMeta(): MetadataItem | undefined {
        return this._selectedMeta;
    }

    setSelectedMeta(meta: MetadataItem | undefined) {
        if (meta === this._selectedMeta) return false;
        this._selectedMeta = meta;
        this.updateSelectedMeta();
        return true;
    }

    setSelectedMetaById(metaId: string | undefined) {
        if (!metaId) {
            return this.setSelectedMeta(undefined);
        } else if (metaId !== this._selectedMeta?.meta.id) {
            return this.setSelectedMeta(this._metas.find(m => m.meta.id === metaId));
        }
        return false;
    }

    get selectedPhase(): string | undefined {
        return this._selectedPhase;
    }

    setSelectedPhase(phase: string | undefined) {
        if (phase === this._selectedPhase) return false;
        this._selectedPhase = phase;
        this.focusPhase = phase;
        this.updateSelectedPhase();
        return true;
    }

    get failureCursor(): CircularList<string> | undefined {
        return this._failureCursor;
    }
    get interruptedCursor(): CircularList<string> | undefined {
        return this._interruptedCursor;
    }
    get warningCursor(): CircularList<string> | undefined {
        return this._warningCursor;
    }

    get metas(): MetadataItem[] {
        return this._metas;
    }

    private updateFilter(groups: PhaseStatusGroup[]) {
        const failures = groups.filter((p) => !p.hasOnlyOutcomes(notFailureOutcomes)).map((p) => p.phase.key) ?? [];
        this._failureCursor = new CircularList(failures);
        this._failureCursor.setCursor(this._selectedPhase);
        const interrupted = groups.filter((p) => p.hasOutcome(TestPhaseOutcome.Interrupted)).map((p) => p.phase.key) ?? [];
        this._interruptedCursor = new CircularList(interrupted);
        this._interruptedCursor.setCursor(this._selectedPhase);
        const warnings = groups.filter((p) => p.hasWarnings()).map((p) => p.phase.key) ?? [];
        this._warningCursor = new CircularList(warnings);
        this._warningCursor.setCursor(this._selectedPhase);
        this._filteredPhases = groups;
    }

    getNextFailure(): [string | undefined, MetadataItem | undefined] {
        const nextPhase = this._failureCursor?.next();
        if (nextPhase) {
            const nextMetadata = this._filteredPhases.find((g) => g.phase.key === nextPhase)?.getMetadataWithoutOutcomes(notFailureOutcomes).at(0);
            this._interruptedCursor?.setCursor(nextPhase);
            return [nextPhase, this._metas.find(m => m.meta === nextMetadata)];
        }
        return [undefined, undefined];
    }

    getPreviousFailure(): [string | undefined, MetadataItem | undefined] {
        const previousPhase = this._failureCursor?.back();
        if (previousPhase) {
            const previousMetadata = this._filteredPhases.find((g) => g.phase.key === previousPhase)?.getMetadataWithoutOutcomes(notFailureOutcomes).at(0);
            this._interruptedCursor?.setCursor(previousPhase);
            return [previousPhase, this._metas.find(m => m.meta === previousMetadata)];
        }
        return [undefined, undefined];
    }

    getNextInterrupted(): [string | undefined, MetadataItem | undefined] {
        const nextPhase = this._interruptedCursor?.next();
        if (nextPhase) {
            const nextMetadata = this._filteredPhases.find((g) => g.phase.key === nextPhase)?.getMetadataWithOutcome(TestPhaseOutcome.Interrupted).at(0);
            this._failureCursor?.setCursor(nextPhase);
            return [nextPhase, this._metas.find(m => m.meta === nextMetadata)];
        }
        return [undefined, undefined];
    }

    getPreviousInterrupted(): [string | undefined, MetadataItem | undefined] {
        const previousPhase = this._interruptedCursor?.back();
        if (previousPhase) {
            const previousMetadata = this._filteredPhases.find((g) => g.phase.key === previousPhase)?.getMetadataWithOutcome(TestPhaseOutcome.Interrupted).at(0);
            this._failureCursor?.setCursor(previousPhase);
            return [previousPhase, this._metas.find(m => m.meta === previousMetadata)];
        }
        return [undefined, undefined];
    }

    getNextWarning(): [string | undefined, MetadataItem | undefined] {
        const nextPhase = this._warningCursor?.next();
        if (nextPhase) {
            const nextMetadata = this._filteredPhases.find((g) => g.phase.key === nextPhase)?.getMetadataWithWarnings().at(0);
            return [nextPhase, this._metas.find(m => m.meta === nextMetadata)];
        }
        return [undefined, undefined];
    }

    getPreviousWarning(): [string | undefined, MetadataItem | undefined] {
        const previousPhase = this._warningCursor?.back();
        if (previousPhase) {
            const previousMetadata = this._filteredPhases.find((g) => g.phase.key === previousPhase)?.getMetadataWithWarnings().at(0);
            return [previousPhase, this._metas.find(m => m.meta === previousMetadata)];
        }
        return [undefined, undefined];
    }

    registerOnMetaHovered(onMetaHovered: (meta?: MetadataItem) => void) {
        this.onMetaHovered = onMetaHovered;

        return () => {
            this.onMetaHovered = undefined;
        }
    }

    registerOnPhaseDataLoaded(phaseKey: string, onDataLoaded: () => void) {
        const callbacks = this._phaseCallbacks.get(phaseKey) ?? {onDataLoaded: undefined};
        callbacks.onDataLoaded = onDataLoaded;
        this._phaseCallbacks.set(phaseKey, callbacks);
        return () => {
            callbacks.onDataLoaded = undefined;
        }
    }

    notifyOnPhaseDataLoaded(phaseKey: string) {
        const callback = this._phaseCallbacks.get(phaseKey);
        if (callback) {
            callback.onDataLoaded?.();
        }
    }

    getFirstTimeErrorsCaught(group: PhaseStatusGroup): Map<string, Date> {
        const errorsCaught = new Map<string, Date>();
        const testStatus = MetaSelectorContext.getTestStatus(this._handler, this.test);
        group.metadata.forEach((status, metaItem) => {
            if (status?.errorFingerprint) {
                let startDate = (!!status.session?.commitId ? this._handler.commitIdDates.get(status.session?.commitId) : undefined)
                    ?? (status.session?.start ?? new Date());
                const metaStatus = testStatus?.getPhaseSessions(group.phase.key, metaItem.meta);
                let consecutivePasses = 0;
                metaStatus?.some(s => {
                    if (!s.errorFingerprint) {
                        consecutivePasses++;
                        return consecutivePasses > consecutivePassesThreshold;
                    }
                    const sStartDate = this._handler.commitIdDates.get(s.commitId) ?? s.start;
                    if (sStartDate && sStartDate < startDate) {
                        startDate = sStartDate;
                    }
                    consecutivePasses = 0;
                    return false;
                });
                errorsCaught.set(metaItem.meta.id, startDate);
            }
        });
        return errorsCaught;
    }

    onMetaHovered?: (meta?: MetadataItem) => void;
    metaValueColors: DomainNameColors;
    availableMetaKeys?: string[];
    forkingMetaKey?: string;
    metaKeysBeforeFork?: string[];
    test?: TestNameRef;
    commonMetaString?: string;
    focusPhase?: string;
    private _metas: MetadataItem[];
    private _commonKeys: string[];
    private _handler: TestDataHandler;
    private _filterOptions: ICheckListOption[];
    private _phaseGroups: PhaseStatusGroup[];
    private _filteredPhases: PhaseStatusGroup[];
    private _selectedMeta?: MetadataItem;
    private _selectedPhase?: string;
    private _failureCursor?: CircularList<string>;
    private _interruptedCursor?: CircularList<string>;
    private _warningCursor?: CircularList<string>;
    private _phaseCallbacks: Map<string, {onDataLoaded?: () => void}>;
}
