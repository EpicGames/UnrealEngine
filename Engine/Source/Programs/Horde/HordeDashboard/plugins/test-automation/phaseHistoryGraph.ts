// Copyright Epic Games, Inc. All Rights Reserved.

import * as d3 from "d3";
import { TestDataHandler, TestNameRef, MetadataRef, PhaseSessionResult, TestMetaStatus, consecutivePassesThreshold } from './testData';
import dashboard from 'horde/backend/Dashboard';
import { getShortNiceTime } from 'horde/base/utilities/timeUtils';
import { getHordeStyling } from 'horde/styles/Styles';
import { getPhaseSessionStatusColor,  SessionListSelector } from "./testAutomationCommon";
import { renderToString } from "react-dom/server";
import { timelineHandler, Margin, constructTimeline, calculateAverageSessionFrequencyHours, getSessionsNearTimeFactory } from "./timelineGraph";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type DivSelectionType = d3.Selection<HTMLDivElement, unknown, HTMLElement, undefined>;
type SelectionBaseType = d3.Selection<d3.BaseType, unknown, null, undefined>;
type FingerprintSeries = { 
    fingerprint: string,
    color: string,
    count: number,
    start: number,
    end: number,
    y: number
};

const circleSize = 5;
const minimumBandLength = circleSize * 2;
const minimumBandCount = 2;
const highFrequencyThreshold = 4; // in hours

const width = 1000;
const bandHeight = 4
const bandOpacities = [ 0.34, 0.64 ];

export class PhaseHistoryGraph {

    constructor(test: TestNameRef, phase: string, stream: string, handler: TestDataHandler, onClickCallback?: (session: PhaseSessionResult) => void) {
        this.test = test;
        this.phase = phase;

        this.handler = handler;
        this.onClickCallback = onClickCallback;

        this.status = handler.getStatusStream(stream)!.tests.get(test)!;

        this.margin = { top: 0, right: 32, bottom: 0, left: 32 };
    }

    initData(meta: MetadataRef) {
        this.meta = meta;
        this.sessions = this.status.sessions.get(meta)?.getPhaseSessions(this.phase) ?? [];
    }

    render(container: HTMLDivElement, meta: MetadataRef, selectedPhaseSessionId?: string, withoutHeader?: boolean, reverse?: boolean) {

        const { modeColors } = getHordeStyling();

        this.clear(container);
        this.cleanupCallback();
        this.initData(meta);

        // initialize timeline handle if not set
        timelineHandler.init(this.test.id, this.handler, width, this.margin, reverse);
        if (timelineHandler.isTimelineAtExtreme && this.averageFrequencyHours < highFrequencyThreshold) {
            timelineHandler.rescaleTimeline(this.averageFrequencyHours / highFrequencyThreshold);
        }

        this.phaseSessionId = selectedPhaseSessionId;

        const handler = this.handler;
        const sessions = this.sessions;

        const X = _d3.map(sessions, (r) => handler.commitIdDates.get(r.commitId)!.getTime() / 1000) as number[];
        const Y = _d3.map(sessions, (r) => 0) as number[];

        const yDomain = new _d3.InternSet(Y as any);

        const I = d3.range(X.length);

        const yPadding = 1;
        const height = withoutHeader? 20 : Math.ceil((yDomain.size + yPadding) * 13) + this.margin.top + this.margin.bottom;

        const yRange = [this.margin.top, height - this.margin.bottom];
        const yScale = _d3.scalePoint(yDomain, yRange).round(true).padding(yPadding);

        const getSessionsNearTime = getSessionsNearTimeFactory(sessions, X, this.handler.filterState!.weeks! * 7);

        const getFingerprintSeries = () => {
            // group sessions into consecutive runs of the same error fingerprint
            const fingerprintSeries: FingerprintSeries[] = [];
            let currentSeries: FingerprintSeries | undefined = undefined;
            let lastFingerprint: string | undefined = undefined;
            let consecutivePasses = 0;

            sessions.forEach((session, idx) => {
                const fingerprint = session.errorFingerprint;
                if (!fingerprint) {
                    consecutivePasses++;
                    if (consecutivePasses > consecutivePassesThreshold) {
                        // reset current series if we had more than consecutivePassesThreshold
                        currentSeries = undefined;
                        lastFingerprint = undefined;
                    }
                    return;
                }

                const xdate = X[idx];
                if (xdate < timelineHandler.startDate! || xdate > timelineHandler.endDate!) return;

                if (fingerprint !== lastFingerprint && (!currentSeries || currentSeries.end !== xdate)) {
                    // New series started
                    currentSeries = { fingerprint, color: getPhaseSessionStatusColor(session), count: 0, start: xdate, end: xdate, y: Y[idx] };
                    fingerprintSeries.push(currentSeries);
                }
                currentSeries!.count++;
                consecutivePasses = 0;
                if (xdate < currentSeries!.start) {
                    currentSeries!.start = xdate;
                }
                lastFingerprint = fingerprint;
            });

            return fingerprintSeries;
        }

        const svg = d3.select(container)
            .append("svg")
            .attr("width", width)
            .attr("height", height + 24)
            .attr("viewBox", [0, 0, width, height + 24] as any);

        this.timelineCleanupCallback = constructTimeline(
            svg,
            () => {
                // update error bands
                errorBands.call(drawErrorBands);
                // update meta groups
                metaGroups.call(drawMetaGroups);
            },
            height - this.margin.bottom,
            reverse,
            withoutHeader
        );

        const errorBands = svg.append("g")
            .attr("id", "error-bands");

        const drawErrorBands = (bands: SelectionType) => {
            // add group for bands below all marks
            const fingerprintSeries = getFingerprintSeries();
            bands.selectAll(".error-band")
                .data(fingerprintSeries.filter(band => band.count >= minimumBandCount && (Math.abs(timelineHandler.timelineXScale!(band.end) - timelineHandler.timelineXScale!(band.start)) >= minimumBandLength)), (band: any) => band.start)
                .join(
                    enter => enter.append("g")
                        .attr("class", "error-band")
                        .each((band, i, nodes) => {
                            const bandNode = d3.select(nodes[i]);
                            const bandOpacity = bandOpacities[i % bandOpacities.length];
                            const xStart = Math.min(timelineHandler.timelineXScale!(band.start), timelineHandler.timelineXScale!(band.end));
                            const xEnd = Math.max(timelineHandler.timelineXScale!(band.start), timelineHandler.timelineXScale!(band.end));
                            const bandWidth = (xEnd - xStart) + 8;
                            bandNode
                                .attr("transform", `translate(${xStart - 4.5},${yScale(band.y)! + 16 - bandHeight/2})`);
                            bandNode.append("rect")
                                .attr("width", bandWidth)
                                .attr("height", bandHeight)
                                .attr("fill", band.color)
                                .attr("opacity", bandOpacity)
                                .attr("stroke", "none");
                            bandNode.append("line")
                                .attr("x1", 1)
                                .attr("x2", 1)
                                .attr("y1", bandHeight)
                                .attr("y2", bandHeight*3)
                                .attr("stroke", band.color)
                                .attr("stroke-width", 2)
                                .attr("stroke-opacity", bandOpacity);
                            bandNode.append("text")
                                .attr("text-anchor", "start")
                                .style("alignment-baseline", "left")
                                .style("font-family", "Horde Open Sans Regular")
                                .style("font-size", 9)
                                .style("font-weight", 800)
                                .attr("y", 18)
                                .attr("x", 3)
                                .attr("fill", band.color)
                                .text(`#${band.fingerprint.substring(0,2)}`);
                        }),
                    update => update.each((band, i, nodes) => {
                        const bandNode = d3.select(nodes[i]);
                        const xStart = Math.min(timelineHandler.timelineXScale!(band.start), timelineHandler.timelineXScale!(band.end));
                        const xEnd = Math.max(timelineHandler.timelineXScale!(band.start), timelineHandler.timelineXScale!(band.end));
                        const bandWidth = (xEnd - xStart) + 8;
                        bandNode.attr("transform", `translate(${xStart - 4.5},${yScale(band.y)! + 16 - bandHeight/2})`);
                        bandNode.select("rect").attr("width", bandWidth);
                    }),
                    exit => exit.remove(),
                );
        }
        errorBands.call(drawErrorBands);

        const metaGroups = svg.append("g")
            .attr("id", "meta-groups");

        const drawMetaGroups = (groups: SelectionType) => {
            const group = groups.selectAll(".meta")
                .data(_d3.group(I, i => Y[i]), ([y]: any) => y)
                .join(
                    enter => enter.append("g")
                        .attr("class", "meta")
                        .attr("id", ([y]: any) => `meta${y as any}`)
                        .attr("transform", ([y]: any) => `translate(0,${(yScale(y) as any) + 16})`)
                        .each(([y]: any, i, nodes: any) => {
                            const node = d3.select(nodes[i]);
                            node.append("line")
                                .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
                                .attr("stroke-width", 1)
                                .attr("stroke-linecap", 4)
                                .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
                                .attr("x1", this.margin.left)
                                .attr("x2", width - this.margin.right);
                        }),
                    update => update.attr("transform", ([y]: any) => `translate(0,${(yScale(y) as any) + 16})`),
                 );

            group.selectAll(".session-circle")
                .data(
                    ([, I]: any) => _d3.groups(I, i => X[i])
                        .filter(([, items]: any) => X[items[0] as any] >= timelineHandler.startDate! && X[items[0] as any] <= timelineHandler.endDate!),
                    ([, items]: any) => sessions[items[0]].id
                )
                .join(
                    enter => enter.append("g")
                        .attr("class", "session-circle")
                        .attr("id", ([, items]: any) => `circle${sessions[items[0]].id}`)
                        .attr("transform", ([, items]: any) => `translate(${timelineHandler.timelineXScale!(X[items[0] as any])},0)`)
                        .append("g")
                        .attr("class", "circleScale")
                        .style("cursor", "pointer")
                        .each(([, items]: any, i, nodes: any) => SessionCircle(nodes[i], items.map(i => sessions[i]), this.phaseSessionId, circleSize, !!this.phaseSessionId && items.some(i => sessions[i].id === this.phaseSessionId))),
                    update => update.attr("transform", ([, items]: any) => `translate(${timelineHandler.timelineXScale!(X[items[0] as any])},0)`),
                 );
        }
        metaGroups.call(drawMetaGroups);

        const setCircleStatus = (selection: SelectionBaseType, items: PhaseSessionResult[], index: number) => {
            const sessionId = items[index].id;
            const session = sessions.length === 1 || !sessionId? sessions[0] : sessions.find(s => s.id === sessionId) ?? sessions[0];
            const color = getPhaseSessionStatusColor(session);
            selection.select(".status")
                .attr("fill", color as any);
        }

        // Attempt to select an existing div with id 'tooltip' from the body, or create one if it doesn't exist
        let tooltip: DivSelectionType = d3.select("body").select("div#tooltip");
        if (tooltip.empty()) {
            tooltip = d3.select("body")
                .append("div")
                .attr("id", "tooltip")
                .style("display", "none")
                .style("background-color", modeColors.background)
                .style("border", "solid")
                .style("border-width", "1px")
                .style("border-radius", "3px")
                .style("border-color", dashboard.darktheme ? "#413F3D" : "#2D3F5F")
                .style("padding", "6px")
                .style("position", "absolute")
                .style("pointer-events", "none");
        }
        this.tooltip = tooltip;

        const closestData = (x: number, y: number): PhaseSessionResult[] | undefined => {

            if (x < this.margin.left - 16 || y < this.margin.top + 16) {
                return undefined;
            }

            y -= 16;

            const closest = getSessionsNearTime(timelineHandler.timelineXScale!.invert(x)).reduce((best, session: PhaseSessionResult) => {
                const absy = Math.abs(yScale(Y[0]) - y);
                const sessionTime = handler.commitIdDates.get(session.commitId)!.getTime() / 1000;
                const absx = Math.abs(timelineHandler.timelineXScale!(sessionTime) - x);

                const lengthSqr = absy * absy + absx * absx;

                if (lengthSqr < best.value) {
                    return { sessions: [session], value: lengthSqr };
                } else {
                     if (lengthSqr === best.value) best.sessions.push(session);
                    return best;
                }
            }, { sessions: [] as PhaseSessionResult[], value: Number.MAX_SAFE_INTEGER });

            if (closest.sessions.length) {
                return closest.sessions;
            }

            return undefined;

        }

        const getInfoSession = (items: PhaseSessionResult[], index: number) => {
            const closest = items[index];
            const date = getShortNiceTime(closest.start, true, true);
            let desc = "";
            if (items.length > 1) {
                desc += renderToString(SessionListSelector(items, index, '(mouse wheel to select)'));
            }
            desc += `${closest.outcome}${items.length > 1? `::${index + 1}` : ''}`
            if (!!closest.errorFingerprint) {
                desc += ` #${closest.errorFingerprint}`;
            }
            desc += `<br/>`;
            desc += `on Commit ${closest.commitId} <br/>`;
            desc += `${date} <br/>`;

            return `<span style="color: ${modeColors.text};">${desc}</span>`;
        }

        const handleMouseMove = (event: any) => {

            const mouseX = _d3.pointer(event)[0];
            const mouseY = _d3.pointer(event)[1];

            const closestItems = closestData(mouseX, mouseY);
            if (!closestItems) {
                handleMouseLeave(undefined);
                return;
            }

            const closest = closestItems[0];
            if (this.tooltipItem?.sessionId !== closest.id) {
                handleMouseLeave(undefined);

                const metaIndex = 0;
                let index = closestItems.length > 1? closestItems.findIndex(item => item.id === this.phaseSessionId) : 0;
                if (index < 0) index = 0;
                const circleSelection = svg.select(`#circle${closest.id}`);
                circleSelection.raise();
                circleSelection.select(".circleScale").attr("transform", `scale(2)`);
                this.tooltipItem = { items: closestItems, index: index, sessionId: closest.id, delta: 0, circleSelection: circleSelection };

                const timeStamp = handler.commitIdDates.get(closest.commitId)!.getTime() / 1000;
                const tx = timelineHandler.timelineXScale!(timeStamp);
                const ty = yScale(metaIndex)!;
                const desc = getInfoSession(closestItems, index);
                // Compute the bounding rectangle of the container to get its absolute position
                const containerRect = container.getBoundingClientRect();
                const offsetX = containerRect.left + window.scrollX;
                const offsetY = containerRect.top + window.scrollY;
                const absTx = tx + offsetX;
                const absTy = ty + offsetY;
                this.updateTooltip(absTx, absTy, desc);
            }

        }

        const handleMouseLeave = (event: any) => {

            if (this.tooltipItem) {
                tooltip.style("display", "none");
                const circleSelection = this.tooltipItem.circleSelection;
                circleSelection.select(".circleScale").attr("transform", `scale(1)`);
                if (this.tooltipItem.items.length > 1) {
                    let index = this.tooltipItem.items.findIndex(item => item.id === this.phaseSessionId);
                    if (index < 0) index = 0;
                    setCircleStatus(circleSelection, this.tooltipItem.items, index);
                }
                this.tooltipItem = undefined;
            }

        }

        const handleMouseClick = (event: any) => {

            const mouseX = _d3.pointer(event)[0];
            const mouseY = _d3.pointer(event)[1];

            if(this.tooltipItem) {
                const closestItems = closestData(mouseX, mouseY);
                if (closestItems && this.onClickCallback) {
                    this.onClickCallback(closestItems[this.tooltipItem.index]);
                    handleMouseLeave(undefined);
                }
            }

        }

        const handleMouseWheel = (event: any) => {

            if (this.tooltipItem && this.tooltipItem.items.length > 1) {
                this.tooltipItem.delta += event.wheelDelta;
                if (Math.abs(this.tooltipItem.delta) >= 240) {
                    this.tooltipItem.index += this.tooltipItem.delta > 0? 1 : -1;
                    this.tooltipItem.delta = 0;

                    if (this.tooltipItem.index < 0) {
                        this.tooltipItem.index += this.tooltipItem.items.length;
                    } else if (this.tooltipItem.index >= this.tooltipItem.items.length) {
                        this.tooltipItem.index -= this.tooltipItem.items.length;
                    }

                    setCircleStatus(this.tooltipItem.circleSelection, this.tooltipItem.items, this.tooltipItem.index);
                    const desc = getInfoSession(this.tooltipItem.items, this.tooltipItem.index);
                    this.updateTooltip(undefined, undefined, desc);
                }
                event.preventDefault();
            }
            
        }

        svg.on("mousemove", (event) => handleMouseMove(event));
        svg.on("mouseleave", (event) => handleMouseLeave(event));
        svg.on("click", (event) => handleMouseClick(event));
        svg.on("wheel", (event) => handleMouseWheel(event));
    }

    updateTooltip(x?: number, y?: number, html?: string) {
        if (!this.tooltip) {
            return;
        }

        if (x !== undefined && y != undefined) {
            this.tooltip
                .style("top", `${y}px`)
                .style("left", `${x}px`);
        }

        this.tooltip
            .style("display", "block")
            .html(html ?? "")
            .style("position", `absolute`)
            .style("width", `max-content`)
            .style("transform", "translate(10%, 52%)")
            .style("font-family", "Horde Open Sans Semibold")
            .style("font-size", "10px")
            .style("line-height", "16px")
            .style("shapeRendering", "crispEdges")
            .style("stroke", "none")
            .style("z-index", 2000000);

    }

    clear(container: HTMLDivElement) {
        d3.select(container).selectAll("*").remove();
    }

    cleanupCallback() {
        this.timelineCleanupCallback?.();
        this.timelineCleanupCallback = undefined;
    }

    get averageFrequencyHours() {
        if (this._averageSessionFrequencyHours !== undefined) {
            return this._averageSessionFrequencyHours;
        }
        this._averageSessionFrequencyHours = calculateAverageSessionFrequencyHours(this.handler, this.sessions);
        return this._averageSessionFrequencyHours;
    }

    // refs
    test: TestNameRef;
    meta: MetadataRef;
    phase: string;
    phaseSessionId?: string;

    onClickCallback?: (session: PhaseSessionResult) => void;
    timelineCleanupCallback?: () => void;

    handler: TestDataHandler;
    margin: Margin;

    hasRendered = false;

    status: TestMetaStatus;
    sessions: PhaseSessionResult[] = [];

    tooltip?: DivSelectionType;
    tooltipItem?: {index: number, items: PhaseSessionResult[], sessionId: string, delta: number, circleSelection: SelectionBaseType};

    private _averageSessionFrequencyHours?: number;
}

const SessionCircle = (container: SVGGElement, sessions: PhaseSessionResult[], sessionId: string | undefined, radius: number, selected?: boolean) => {
    const session = sessions.length === 1 || !sessionId? sessions[0] : sessions.find(s => s.id === sessionId) ?? sessions[0];
    const color = getPhaseSessionStatusColor(session);
    const g = d3.select(container);
    g.append("circle")
        .attr("class", "status")
        .attr("stroke", "none")
        .attr("fill", color as any)
        .attr("r", radius);

    if (selected) {
        g.append("circle")
            .attr("stroke", dashboard.darktheme ? "#c1c4c9af" : "#3d4654af")
            .attr("fill", "transparent")
            .attr("stroke-width", 2)
            .attr("r", radius + 3);
    }

    if (sessions.length > 1) {
        g.append("text")
            .attr("text-anchor", "start")
            .style("alignment-baseline", "left")
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", 10)
            .attr("y", -4)
            .attr("x", 8)
            .attr("fill", dashboard.darktheme ? "#E0E0E0" : "#2D3F5F")
            .text(`${sessions.length}`);
    }

    return g;
}
