// Copyright Epic Games, Inc. All Rights Reserved.
import { useParams } from "react-router-dom";
import { useEffect, useMemo, useState } from "react";
import { Stack } from "@fluentui/react";
import { AuditLine, AuditLogPanel, ItemAuditLogConfig } from "horde/components/AuditLog";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import { getTestAuditLogHistoryV2, getTestNamesV2 } from "./api";

class TestAuditLogLine extends AuditLine {

}

export const TestAuditLogView: React.FC = () => {

   const { testId } = useParams<{ testId: string }>();
   const [testName, setTestName] = useState<string | undefined>(undefined);
   
   useEffect(() => {
      if (testId) {
         getTestNamesV2([testId]).then((response) => 
            {
               if (response.length) {
                  setTestName(response[0].name);
               } else {
                  setTestName(testId);
               }
            });
      }
   }, [testId]);

   const { hordeClasses } = getHordeStyling();

   const itemAudit: ItemAuditLogConfig | undefined = useMemo(() => !testId && !testName? undefined : {
      id: testId!,
      lineFactory: (entry, search) => new TestAuditLogLine(entry, search),
      getHistory: (testId, minTime, maxTime) => getTestAuditLogHistoryV2(testId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 }),
      breadCrumbs: [
         {text: `Test Health`},
         {text: `Test Audit Log - ${testName}`}
      ]
   }, [testId, testName]);

   return <Stack className={hordeClasses.horde}>
            <TopNav />
            <Stack>
               {!!itemAudit && <AuditLogPanel itemAudit={itemAudit} />}
            </Stack>
         </Stack>

}
