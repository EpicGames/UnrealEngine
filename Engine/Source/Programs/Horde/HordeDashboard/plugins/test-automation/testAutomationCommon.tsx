// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyles, mergeStyleSets, Text, ComboBox, IComboBoxOption, SelectableOptionMenuItemType, IContextualMenuProps, IContextualMenuItem, IconButton, IContextualMenuListProps, Stack, Checkbox, FontIcon, Modal, DefaultButton, ScrollablePane, SelectionZone, DetailsList, Selection, DetailsListLayoutMode, SelectionMode, IColumn, PrimaryButton, Label, TextField, MessageBar, MessageBarType, ContextualMenuItemType } from "@fluentui/react";
import { MetadataRef, PhaseSessionResult, TestDataHandler, TestNameRef, TestPhaseStatus, TestSessionDetails, TestSessionResult } from "./testData";
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { memo, useCallback, useEffect, useMemo, useState } from "react";
import { TestOutcome, TestPhaseOutcome } from "./api";
import { PhaseHistoryGraph } from "./phaseHistoryGraph";
import { projectStore } from "horde/backend/ProjectStore";
import { getHordeStyling } from "horde/styles/Styles";

import { GetUserResponse } from "horde/backend/Api";
import { UserSelect } from "horde/components/UserSelect";
import { Markdown } from "horde/base/components/Markdown";

import * as d3 from "d3";

export type PhaseSessionCallback = (session: PhaseSessionResult) => void;

export const styles = {
    labelSmall: {
        fontSize: 12,
        fontWeight: "bold"
    },
    textSmall: {
        fontSize: 12
    },
    textLarge: {
        fontSize: 24
    },
    defaultButton: {
        backgroundColor: "#035ca1"
    },
    defaultFilter: {
        backgroundColor: "#0078d4"
    },
    highlightFilter: {
        backgroundColor: "#2a95e8ff"
    },
    defaultCheckbox: {
        root: {
            height: 15
        },
        text: {
            fontSize: 11,
            marginLeft: 0,
            lineHeight: 15
        },
        checkbox: {
            height: 15,
            width: 15
        }
    },
    icon: {
        cursor: "default",
        height: 15,
        paddingTop: 1
    },
    stripes: {
        backgroundImage: 'repeating-linear-gradient(-45deg, rgba(255, 255, 255, .2) 25%, transparent 25%, transparent 50%, rgba(255, 255, 255, .2) 50%, rgba(255, 255, 255, .2) 75%, transparent 75%, transparent)',
    },
    streamBadge: {
        width: 200,
        height: 24,
        backgroundColor: "#8e8e8e48",
        padding: '3px 16px',
        borderRadius: 2
    }
}

let _styles: any;

const getStyles = () => {

    const border = `1px solid ${dashboard.darktheme ? "#2D2B29" : "#EDEBE9"}`;
    const highlight = dashboard.darktheme ? "brightness(1.2)" : "brightness(0.9)";
    const textEmphasisColor = dashboard.darktheme? "#ffffff57" : "#00000057";

    const styles = _styles ?? mergeStyleSets({
        list: {
            selectors: {
                'a': {
                    height: "unset !important",
                },
                '.ms-List-cell': {
                    borderTop: border,
                    borderRight: border,
                    borderLeft: border
                },
                '.ms-List-cell:nth-last-child(-n + 1)': {
                    borderBottom: border
                },
                ".ms-DetailsRow #artifactview": {
                    opacity: 0
                },
                ".ms-DetailsRow:hover #artifactview": {
                    opacity: 1
                },
            }
        },
        editable: {
            selectors: {
                '> .Edit': {
                    backgroundColor: "#67676748",
                    visibility: 'hidden',
                    cursor: 'pointer'
                },
                '> .Edit:hover': {
                    filter: highlight,
                },
                ':hover > .Edit': {
                    visibility: 'visible'
                }
            }
        },
        placeholder: {
            color: textEmphasisColor
        },
   });

   _styles = styles;

   return styles;
}

export const getStatusColors = (): Map<StatusColor, string> => {

    const dashboardStatusColors = dashboard.getStatusColors();

    const colors = new Map<StatusColor, string>([
        [StatusColor.Success, dashboardStatusColors.get(StatusColor.Success)!],
        [StatusColor.Warnings, dashboardStatusColors.get(StatusColor.Warnings)!],
        [StatusColor.Failure, dashboardStatusColors.get(StatusColor.Failure)!],
        [StatusColor.Skipped,  dashboard.darktheme ? "#c5c4c3ff" : "#a1a1a1ff"],
        [StatusColor.Unspecified, dashboardStatusColors.get(StatusColor.User1)!]
    ]);

    return colors;
}

export const statusTexts = new Map<string, string>([
    [TestOutcome.Success, "Passed"],
    [TestPhaseOutcome.Warning, "With Warning"],
    [TestOutcome.Failure, "Failure"],
    [TestOutcome.Skipped, "Skipped"],
    [TestOutcome.Unspecified, "Catastrophic"]
]);

export enum TestViewType {
    /// The summary view
    Summary = "0",
    /// The health view
    Health = "1"
}

export const testViewIcons = new Map<TestViewType, string>([
    [TestViewType.Summary, "ShowResults"],
    [TestViewType.Health, "Health"]
]);

export const MultiOptionChooser: React.FC<{options: IComboBoxOption[], initialSelection: string[], updateKeys: (selectedKeys: string[]) => void, placeholder?: string, style?: any, disabled?: boolean }> = ({ options, initialSelection, updateKeys, placeholder, style, disabled }) => {
    const [selection, setSelection] = useState<string[]>(initialSelection);
    const optionsWithSelectAll = [...options];

    useEffect(() => {
        // keep input selection in sync
        if (selection.toString() !== initialSelection.toString()) {
            setSelection(initialSelection);
        }
    }, [initialSelection]);

    if (selection.length === options.length) {
        selection.push('selectAll');
    }

    optionsWithSelectAll.unshift({ key: 'selectAll', text: 'Select All', itemType: SelectableOptionMenuItemType.SelectAll });

    return <ComboBox styles={{ root: style }} placeholder={placeholder ?? "None"} selectedKey={selection} multiSelect options={optionsWithSelectAll} disabled={disabled}
        onChange={(_, option) => {
            if (option) {
                if (option.itemType === SelectableOptionMenuItemType.SelectAll) {
                    setSelection(option.selected? optionsWithSelectAll.map((o) => o.key as string) : []);
                    return;
                }
                const key = option.key as string;
                const index = selection.indexOf(key);
                if (index === -1 && option.selected) {
                    selection.push(key);
                } else if (index >= 0 && !option.selected) {
                    selection.splice(index, 1);
                }
                setSelection([...selection]);
            }
        }}
        onMenuDismissed={() => {
            updateKeys(selection.filter((k) => k !== 'selectAll'));
        }} />
};

export type StatusBarStack = {
    value: number,
    title?: string,
    titleValue?: string,
    color?: string,
    onClick?: () => void,
    stripes?: boolean,
}

export const StatusBar = (stack: StatusBarStack[], width: number, height: number, basecolor?: string, style?: any): JSX.Element => {

    stack = stack.filter(s => s.value > 0);

    const mainTitle = stack.map((item) => {
        return !item.titleValue ? `${Math.ceil(item.value)}% ${item.title}` : `${item.titleValue} ${item.title}`
    }).join(' ');

    return (
        <div className={mergeStyles({ backgroundColor: basecolor, width: width, height: height, verticalAlign: 'middle', display: "flex" }, style)} title={mainTitle}>
            {stack.map((item) => <span key={item.title!}
                onClick={item.onClick}
                style={{
                    width: `${item.value}%`, height: '100%',
                    backgroundColor: item.color,
                    display: 'block',
                    cursor: item.onClick ? 'pointer' : 'inherit',
                    backgroundSize: `${height * 2}px ${height * 2}px`,
                    backgroundImage: item.stripes ? styles.stripes.backgroundImage : undefined
                }} />)}
        </div>
    );
}

export const SessionStatusBar = (session: TestSessionResult | TestSessionDetails, width: number, height: number): JSX.Element | undefined => {
    const sessionTotalCount = session.phasesSucceededCount + session.phasesFailedCount + session.phasesUnspecifiedCount;
    const metaFailedFactor = Math.ceil(session.phasesFailedCount / (sessionTotalCount || 1) * 50) / 50;
    const metaUnspecifiedFactor = Math.ceil(session.phasesUnspecifiedCount / (sessionTotalCount || 1) * 50) / 50;

    const bDisplayBar = (metaFailedFactor + metaUnspecifiedFactor) > 0 && metaFailedFactor < 1 && metaUnspecifiedFactor < 1;
    if (!bDisplayBar) return undefined

    const statusColors = getStatusColors();

    const stack: StatusBarStack[] = [
        {
            value: metaUnspecifiedFactor * 100,
            title: statusTexts.get(TestOutcome.Unspecified),
            titleValue: `${ Math.ceil(session.phasesUnspecifiedCount / (sessionTotalCount || 1) * 100)}%`,
            color: statusColors.get(StatusColor.Unspecified)!,
            stripes: true
        },
        {
            value: metaFailedFactor * 100,
            title: statusTexts.get(TestOutcome.Failure),
            titleValue: `${ Math.ceil(session.phasesFailedCount / (sessionTotalCount || 1) * 100)}%`,
            color: statusColors.get(StatusColor.Failure)!,
            stripes: true
        }
    ];

    return StatusBar(stack, width, height, statusColors.get(StatusColor.Success)!, { margin: '3px !important' });
}

export const StatusPie = (stack: StatusBarStack[], radius: number, style?: any): JSX.Element => {

    const width = radius * 2;
    const height = width;

    stack = stack.filter(s => s.value > 0);
    let offset = 0;
    const stackOffsets = stack.map(s => {
        const stepOffset = offset;
        offset += s.value;
        return stepOffset;
    });

    const mainTitle = stack.map((item) => {
        return item.titleValue === undefined ? `${Math.ceil(item.value)}% ${item.title}` : `${item.titleValue} ${item.title}`
    }).join(' ');

    return (
        <div className={mergeStyles({ width: width, height: height, verticalAlign: 'middle', display: "flex" }, style)} title={mainTitle}>
            <svg width={width} height={height} viewBox={`0 0 64 64`}>
                {stack.map((item, index) => <circle key={item.title!} r="25%" cx="50%" cy="50%" strokeWidth="50%" transform={`rotate(-90) translate(-64)`} fill="transparent"
                                        stroke={item.color}
                                        strokeDashoffset={`-${stackOffsets[index]}`}
                                        strokeDasharray={`${item.value} 100`}
                                        onClick={item.onClick} />)
                }
            </svg>
        </div>
    );
}

export const SessionStatusPie = (session: TestSessionResult, radius: number): JSX.Element | undefined => {
    const sessionTotalCount = session.phasesSucceededCount + session.phasesFailedCount + session.phasesUnspecifiedCount;
    const metaFailedFactor = Math.ceil(session.phasesFailedCount / (sessionTotalCount || 1) * 100) / 100;
    const metaUnspecifiedFactor = Math.ceil(session.phasesUnspecifiedCount / (sessionTotalCount || 1) * 100) / 100;

    const bDisplayBar = (metaFailedFactor + metaUnspecifiedFactor) > 0 && (metaFailedFactor + metaUnspecifiedFactor) < 1;
    if (!bDisplayBar) return;

    const statusColors = getStatusColors();

    const stack: StatusBarStack[] = [
        {
            value: metaUnspecifiedFactor * 100,
            title: statusTexts.get(TestOutcome.Unspecified),
            color: statusColors.get(StatusColor.Unspecified)!
        },
        {
            value: metaFailedFactor * 100,
            title: statusTexts.get(TestOutcome.Failure),
            color: statusColors.get(StatusColor.Failure)!
        },
        {
            value: (1 - (metaFailedFactor + metaUnspecifiedFactor)) * 100,
            title: statusTexts.get(TestOutcome.Success),
            color: statusColors.get(StatusColor.Success)!
        }
    ];

    return StatusPie(stack, radius);
}

export const PhasesStatusPie = (phases: TestPhaseStatus[], radius: number): JSX.Element | undefined => {
    const totalCount = phases.length || 1;
    let phasesFailedCount = 0;
    let phasesUnspecifiedCount = 0;
    let phasesSkippedCount = 0
    phases.forEach(p => {
        switch (p.outcome) {
            case TestPhaseOutcome.Skipped:
                ++phasesSkippedCount;
                break;
            case TestPhaseOutcome.Failed:
                ++phasesFailedCount;
                break;
            case TestPhaseOutcome.Interrupted:
            case TestPhaseOutcome.Unknown:
            case TestPhaseOutcome.NotRun:
                ++phasesUnspecifiedCount;
                break;
        }
    });

    const failedFactor = Math.ceil(phasesFailedCount / totalCount * 100) / 100;
    const unspecifiedFactor = Math.ceil(phasesUnspecifiedCount / totalCount * 100) / 100;
    const skippedFactor = Math.ceil(phasesSkippedCount / totalCount * 100) / 100;

    const statusColors = getStatusColors();

    const stack: StatusBarStack[] = [
        {
            value: unspecifiedFactor * 100,
            title: statusTexts.get(TestOutcome.Unspecified),
            color: statusColors.get(StatusColor.Unspecified)!
        },
        {
            value: skippedFactor * 100,
            title: statusTexts.get(TestOutcome.Skipped),
            color: statusColors.get(StatusColor.Skipped)!
        },
        {
            value: failedFactor * 100,
            title: statusTexts.get(TestOutcome.Failure),
            color: statusColors.get(StatusColor.Failure)!
        },
        {
            value: (1 - (failedFactor + unspecifiedFactor + skippedFactor)) * 100,
            title: statusTexts.get(TestOutcome.Success),
            color: statusColors.get(StatusColor.Success)!
        }
    ];

    return StatusPie(stack, radius);
}

export const PhaseValues = (phases: TestPhaseStatus[], style?: any): JSX.Element => {
    let phasesFailedCount = 0;
    let phasesUnspecifiedCount = 0;
    let phasesSkippedCount = 0
    phases.forEach(p => {
        switch (p.outcome) {
            case TestPhaseOutcome.Skipped:
                ++phasesSkippedCount;
                break;
            case TestPhaseOutcome.Failed:
                ++phasesFailedCount;
                break;
            case TestPhaseOutcome.Interrupted:
            case TestPhaseOutcome.Unknown:
            case TestPhaseOutcome.NotRun:
                ++phasesUnspecifiedCount;
                break;
        }
    });

    const statusColors = getStatusColors();
    const border = `1px solid ${dashboard.darktheme ? "#ffffff3f" : "#0000003f"}`;
    const background = dashboard.darktheme ? "#ffffff0f" : "#0000000f";
    return <Stack horizontal style={style} tokens={{childrenGap: 3}}>
                <Text style={{color: statusColors.get(StatusColor.Unspecified), whiteSpace: 'nowrap', border: border, borderRadius: '5px', padding: '0px 4px', background: background}} title={statusTexts.get(TestOutcome.Unspecified)}>{statusTexts.get(TestOutcome.Unspecified)?.at(0)}: {phasesUnspecifiedCount}</Text>
                <Text style={{color: statusColors.get(StatusColor.Failure), whiteSpace: 'nowrap', border: border, borderRadius: '5px', padding: '0px 4px', background: background}} title={statusTexts.get(TestOutcome.Failure)}>{statusTexts.get(TestOutcome.Failure)?.at(0)}: {phasesFailedCount}</Text>
                <Text style={{color: statusColors.get(StatusColor.Skipped), whiteSpace: 'nowrap', border: border, borderRadius: '5px', padding: '0px 4px', background: background}} title={statusTexts.get(TestOutcome.Skipped)}>{statusTexts.get(TestOutcome.Skipped)?.at(0)}: {phasesSkippedCount}</Text>
            </Stack>
}

export const SessionValues = (session: TestSessionResult, style?: any): JSX.Element => {
    const statusColors = getStatusColors();
    return <Text style={style}>
                {!!session.phasesUnspecifiedCount && <span style={{color: statusColors.get(StatusColor.Unspecified)}}> {statusTexts.get(TestOutcome.Unspecified)}: {session.phasesUnspecifiedCount}</span>}
                {!!session.phasesFailedCount && <span style={{color: statusColors.get(StatusColor.Failure)}}> {statusTexts.get(TestOutcome.Failure)}: {session.phasesFailedCount}</span>}
                {!!session.phasesSucceededCount && <span style={{color: statusColors.get(StatusColor.Success)}}> {statusTexts.get(TestOutcome.Success)}: {session.phasesSucceededCount}</span>}
            </Text>
}

export const getPhaseSessionStatusColor = (phase: PhaseSessionResult | TestPhaseStatus) => {
    const statusColors = getStatusColors();
    let color = statusColors.get(StatusColor.Unspecified)!
    switch(phase.outcome) {
        case TestPhaseOutcome.Skipped:
            color = statusColors.get(StatusColor.Skipped)!
            break;
        case TestPhaseOutcome.Failed:
            color = statusColors.get(StatusColor.Failure)!
            break;
        case TestPhaseOutcome.Success:
            color = statusColors.get(phase.hasWarning? StatusColor.Warnings : StatusColor.Success)!
            break;
    }

    return color;
}

export const getTestSessionStatusColor = (test: TestSessionResult | TestSessionDetails) => {
    const statusColors = getStatusColors();
    let color = statusColors.get(StatusColor.Success);
    if (test.outcome === TestOutcome.Failure) {
        color = statusColors.get(StatusColor.Failure);
    } else if (test.outcome === TestOutcome.Unspecified) {
        color = statusColors.get(StatusColor.Unspecified);
    }

    return color;
}

const isTestSessionResult = (item: TestSessionResult | PhaseSessionResult): item is TestSessionResult => {
    return !!(item as TestSessionResult).nameRef;
}

export const getSessionColor = (item: TestSessionResult | PhaseSessionResult) => {
    if(isTestSessionResult(item)) {
        return getTestSessionStatusColor(item);
    } else {
        return getPhaseSessionStatusColor(item);
    }
}

export const PhaseHistoryGraphWidget: React.FC<{test: TestNameRef, phaseKey: string, meta: MetadataRef, streamId: string, sessionId?: string, handler: TestDataHandler, onClick: PhaseSessionCallback, withoutHeader?: boolean, reverse?: boolean}> = memo(({ test, phaseKey, meta, streamId, sessionId, handler, onClick, withoutHeader, reverse }) => {
    const [container, setContainer] = useState<HTMLDivElement | null>(null);
    const [state, setState] = useState<{ graph?: PhaseHistoryGraph }>({});

    useEffect(() => {
        const graph = new PhaseHistoryGraph(test, phaseKey, streamId, handler, onClick);
        setState({ graph: graph });
        return () => graph.cleanupCallback();
    }, [test, phaseKey, streamId, onClick]);

    useEffect(() => {
        // refresh on meta or sessionId update
        try {
            container && state.graph?.render(container, meta, sessionId, withoutHeader, reverse);
        } catch (err) {
            console.error(err);
        }
    }, [container, state, meta, sessionId, withoutHeader, reverse])

    if (!state.graph) {
        return null;
    }

    const graph_container_id = `test_history_graph_container_${test!.id}_${meta.id}_${phaseKey}_${streamId}`;

    return <div id={graph_container_id} className="phase-history" style={{ userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)}/>
});

export const getStreamOptions = (streams: string[], onClick: (stream: string) => void, selected?: string): IContextualMenuProps => {
    const projectItems: Map<string, Map<string, IContextualMenuItem[]>> = new Map();

    projectStore.projects.sort(
        (a, b) => a.order - b.order
    ).forEach(p => {
        p.categories?.forEach(category => {
            category.streams.forEach(a => {
                if (!category.showOnNavMenu || !streams.includes(a)) return;

                const projectName = p.name ?? 'Unknown';
                if (!projectItems.has(projectName)) {
                    projectItems.set(projectName, new Map());
                }

                const categoryItems = projectItems.get(projectName)!;
                if (!categoryItems.has(category.name)) {
                    categoryItems.set(category.name, []);
                }

                categoryItems.get(category.name)?.push({
                    key: `streams_${projectStore.streamById(a)?.fullname}`,
                    text: projectStore.streamById(a)?.name ?? a,
                    onClick: () => onClick(a),
                    disabled: a === selected
                });
            });
        });
    });

    const options: IContextualMenuItem[] = projectItems.entries().map(
        ([name, categories]) => {
            // if more than one catergory and more than 8 streams overall, do a full break down of the catergories
            const hasCategoriesAndManyItems = categories.size > 1 && categories.values().map(c => c.length).reduce((acc, l) => acc + l, 0) > 8;
            return {
                key: `project_${name}`,
                text: name,
                subMenuProps: {
                    shouldFocusOnMount: true,
                    subMenuHoverDelay: 0,
                    items: (
                        hasCategoriesAndManyItems ?
                            categories.entries().map(
                                ([cat, items]) => {
                                    return {
                                        key: `category_${cat}`,
                                        text: cat,
                                        subMenuProps: {
                                            shouldFocusOnMount: true,
                                            subMenuHoverDelay: 0,
                                            items: items
                                        }
                                    }
                            }).toArray()
                            : categories.values().toArray().flat())
                }
            }
    }).toArray();

    return {
            shouldFocusOnMount: true,
            subMenuHoverDelay: 0,
            items: options,
        };
}

export type ICheckListOption = IContextualMenuItem & {
    checked: boolean,
    color?: string,
}

export const getPhaseOutcomeOptions = (): ICheckListOption[] => {
    const statusColors = getStatusColors();
    return [
        {
            key: TestPhaseOutcome.Interrupted,
            text: statusTexts.get(TestOutcome.Unspecified),
            checked: true,
            color: statusColors.get(StatusColor.Unspecified)
        },
        {
            key: TestPhaseOutcome.Failed,
            text: statusTexts.get(TestOutcome.Failure),
            checked: true,
            color: statusColors.get(StatusColor.Failure)
        },
        {
            key: TestPhaseOutcome.Success,
            text: statusTexts.get(TestOutcome.Success),
            checked: true,
            color: statusColors.get(StatusColor.Success)
        },
        {
            key: TestPhaseOutcome.Warning,
            text: statusTexts.get(TestPhaseOutcome.Warning),
            checked: true,
            color: statusColors.get(StatusColor.Warnings)
        },
        {
            key: TestPhaseOutcome.Skipped,
            text: statusTexts.get(TestOutcome.Skipped),
            checked: true,
            color: statusColors.get(StatusColor.Skipped)
        },
    ]
}

export interface IPhaseFilter {
    isMatch(names: string[], outcomes?: Set<TestPhaseOutcome>, tags?: string[]): boolean;
}

class PhaseFiltersContext {
    constructor() {
        this._outcomeOptions = getPhaseOutcomeOptions();
        this._names = new Map();
        this._tags = new Map();
    }

    get outcomeOptions() {
        return this._outcomeOptions
    }

    filterTestPhases(testId: string, phases: IPhaseFilter[]) {
        const outcomes = this.getFilteredOutcomes();
        const names = this.getFilteredNames(testId);
        const tags = this.getFilteredTags(testId);
        return phases.filter(phase => phase.isMatch(names, outcomes, tags));
    }

    getFilteredOutcomes() {
        const outcome: Set<TestPhaseOutcome> = new Set();
        this._outcomeOptions.filter(option => option.checked).forEach(option => {
            switch(option.key) {
                case TestPhaseOutcome.Interrupted:
                    outcome.add(TestPhaseOutcome.NotRun);
                    outcome.add(TestPhaseOutcome.Unknown);
                    outcome.add(TestPhaseOutcome.Interrupted);
                    break;
                case TestPhaseOutcome.Skipped:
                    outcome.add(TestPhaseOutcome.Skipped);
                    break;
                case TestPhaseOutcome.Failed:
                    outcome.add(TestPhaseOutcome.Failed);
                    outcome.add(TestPhaseOutcome.Interrupted);
                    break;
                case TestPhaseOutcome.Warning:
                    outcome.add(TestPhaseOutcome.Warning);
                    break;
                case TestPhaseOutcome.Success:
                    outcome.add(TestPhaseOutcome.Success);
                    break;
            }
        });
        return outcome;
    }

    getFilteredNames(testId: string) {
        return this._names.get(testId) ?? [];
    }

    setFilteredNames(testId: string, names: string[]) {
        const filteredNames = names.filter(i => !i.startsWith("#"));
        this._names.set(testId, filteredNames.map(n => n.toLowerCase()));
        if (filteredNames.length !== names.length) {
            this.setFilteredTags(testId, names.filter(i => i.startsWith("#")).map(i => i.substring(1).toLowerCase()));
        } else {
            this.setFilteredTags(testId, []);
        }
    }

    getFilteredTags(testId: string) {
        return this._tags.get(testId);
    }

    setFilteredTags(testId: string, tags: string[]) {
        if (tags.length === 0) {
            this._tags.delete(testId);
        } else {
            this._tags.set(testId, tags);
        }
    }

    private _outcomeOptions: ICheckListOption[];
    private _names: Map<string, string[]>;
    private _tags: Map<string, string[]>;
}
export const phaseFiltersContext = new PhaseFiltersContext();

class UserNavData {
    constructor() {
        this._userNavData = new Map();
    }

    getData(key: string) {
        return this._userNavData.get(key);
    }

    setData(key: string, data: any) {
        this._userNavData.set(key, data);
    }

    private _userNavData: Map<string, any>;
}
export const userNavData = new UserNavData();

export const CheckListOption: React.FC<{options: ICheckListOption[], onChange: (items: ICheckListOption[]) => void}> = ({options, onChange}) => {
    const [items, setItems] = useState<ICheckListOption[]>([]);
    const [update, setUpdate] = useState<number>(0);

    useEffect(() => {
        const initItems: ICheckListOption[] = [{
            key: 'SelectAll',
            text: 'Select All',
            checked: !options.some(i => !i.checked)
        }];
        
        initItems.push(...options);

        setItems(initItems);
        setUpdate(1);
    }, [options]);

    const isFiltering : boolean = useMemo(() => items.slice(1).some(i => !i.checked), [items, update]);

    if (!items.length && !update) return;

    const fitlerOptions: IContextualMenuProps = {
        items: items,
        onRenderMenuList: (props: IContextualMenuListProps) =>
                <Stack tokens={{childrenGap: 6}} style={{margin: '3px 2px'}}>
                    {props.items.map(item => <Checkbox
                                                key={item.key}
                                                label={item.text}
                                                checked={item.checked}
                                                indeterminate={item.key === 'SelectAll' && !item.checked && props.items.slice(1).some(i => i.checked)}
                                                onChange={
                                                    (ev, checked) => {
                                                        item.checked = checked;
                                                        if (item.key === 'SelectAll') {
                                                            props.items.forEach(i => i.checked = checked);
                                                        } else {
                                                            props.items[0].checked = !props.items.slice(1).some(i => !i.checked);
                                                        }
                                                        setUpdate(update + 1);
                                                        onChange(items.slice(1));
                                                    }
                                                }
                                                onRenderLabel={(props) => {
                                                    return <Stack horizontal verticalAlign="center">
                                                                <Stack style={{ paddingRight: 5 }}>
                                                                    <FontIcon style={{ fontSize: 12, color: item.color }} iconName={!!item.color? "Square" : "MultiSelect"} />
                                                                </Stack>
                                                                <Text>{props?.label?.substring(0, 16)}{props?.label && props.label.length > 16 && '...'}</Text>
                                                            </Stack>
                                                }}
                                            />)}
                </Stack>
    }

    const iconName : string = isFiltering? "ReportWarning" : "PageListFilter";
    const iconColor : string = isFiltering? styles.highlightFilter.backgroundColor : "";

    return  <IconButton
                iconProps={{iconName: iconName}}
                menuProps={fitlerOptions}
                style={{
                    fontSize:16, height: 28,
                    border: '1px solid', borderColor: dashboard.darktheme ? "#4D4C4B" : "#6D6C6B", borderRadius: 4,
                    backgroundColor: dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "",
                    color: dashboard.darktheme ? 'white' : 'grey'
                }}
                styles={{icon: {color: iconColor}}}
            />
}

export const CheckListOptionSmall: React.FC<{options: ICheckListOption[], onChange: (items: ICheckListOption[]) => void, onDismissed?: () => void}> = ({options, onChange, onDismissed}) => {
    const [update, setUpdate] = useState<number>(0);

    const isFiltering : boolean = useMemo(() => options.filter(i => i.itemType !== ContextualMenuItemType.Header).some(i => i.checked), [options, update]);

    const fitlerOptions: IContextualMenuProps = {
        items: options,
        onMenuDismissed: onDismissed,
        onRenderMenuList: (props: IContextualMenuListProps) =>
                <Stack tokens={{childrenGap: 6}} style={{margin: '3px 2px'}}>
                    <DefaultButton
                        iconProps={{ iconName: "MultiSelect" }}
                        text="Clear All"
                        disabled={!isFiltering}
                        onClick={() => {
                            options.forEach(i => i.checked = false);
                            setUpdate(update + 1);
                            onChange(options);
                        }}
                        style={{ fontSize: 12, borderRadius: 6, height: 22, width: 120, paddingLeft: 5 }}
                    />
                    {props.items.map(item => {
                        return (item.itemType === ContextualMenuItemType.Header ?
                                    <Text key={item.key} style={{color: styles.highlightFilter.backgroundColor}}>{item.text}</Text>
                                    : <Checkbox
                                        key={item.key}
                                        label={item.text}
                                        checked={item.checked}
                                        onChange={
                                            (ev, checked) => {
                                                item.checked = checked;
                                                setUpdate(update + 1);
                                                onChange(options);
                                            }
                                        }
                                        onRenderLabel={(props) => {
                                            return <Stack verticalAlign="center" tokens={{childrenGap: 2}} style={{background: item.color ? `${item.color}60` : undefined}}>
                                                        <Text style={{padding: '0 4px'}}>{props?.label?.substring(0, 16)}{props?.label && props.label.length > 16 && '...'}</Text>
                                                        <Stack style={{ height: 3, backgroundColor: item.color }}></Stack>
                                                    </Stack>
                                        }}
                                    />)
                        }
                    )}
                </Stack>
    }

    const iconName : string = isFiltering? "Filter" : "FilterSettings";
    const iconColor : string = isFiltering? styles.highlightFilter.backgroundColor : "";

    return  <IconButton
                iconProps={{iconName: iconName}}
                menuProps={fitlerOptions}
                style={{
                    fontSize:16, height: 28,
                    border: '1px solid', borderColor: dashboard.darktheme ? "#4D4C4B" : "#6D6C6B", borderRadius: 4,
                    backgroundColor: dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "",
                    color: dashboard.darktheme ? 'white' : 'grey'
                }}
                styles={{icon: {color: iconColor}}}
            />
}

type ArtifactHref = {
    name: string;
    reference: string;
    href: string;
}

export const PhaseArtifactsModal: React.FC<{ phase: TestPhaseStatus, artifactPaths: string[], onClose?: () => void }> = ({ phase, artifactPaths, onClose }) => {
    const [selectVer, setSelectVer] = useState(0);
    const [items, setItems] = useState<ArtifactHref[]>([]);

    useEffect(() => {
        if (artifactPaths.length === 0) return;
        const hrefs: (Promise<string> | undefined)[] = [];
        for (const artifact of artifactPaths) {
            hrefs.push(phase.artifacts?.getLink(artifact));
        }
        // get common prefix end position
        let pos = 0;
        const sortedPaths = artifactPaths.concat().sort();
        const shortest = sortedPaths[0];
        const longest = sortedPaths[sortedPaths.length - 1];
        if (shortest === longest) {
            pos = shortest.lastIndexOf('/') + 1;
        } else {
            while (pos < shortest.length && shortest.charAt(pos) === longest.charAt(pos) ) pos++;
            pos = shortest.lastIndexOf('/', pos) + 1;
        }
        
        Promise.all(hrefs).then(results => {
            const items: ArtifactHref[] = [];
            artifactPaths.forEach((artifact, index) => {
                const result = results[index];
                if (!result) return;
                items.push({name: artifact.substring(pos), reference: artifact, href: result});
            });
            setItems(items);
        });
    }, [phase, artifactPaths]);

    const { hordeClasses } = getHordeStyling();
    const styles = getStyles();

    const selector = useMemo(() => new Selection({onSelectionChanged: () => setSelectVer(Math.random)}), [phase, artifactPaths]);

    const downloadText = useMemo(() => {
        let text = "Download";
        const count = selector.getSelectedCount() || artifactPaths.length;
        if (count === 1) {
            text += ` (1 file)`;
        } else if (count > 1) {
            text += ` (${count} files)`;
        }
        return text;
    }, [selectVer]);

    const downloadZip = useCallback(() => {
        if (artifactPaths.length === 0) return;

        let items = selector.getSelection() as ArtifactHref[];
        if (items.length === 0) items = selector.getItems() as ArtifactHref[];

        // download a single file
        if (items.length === 1) {
            phase.artifacts?.download(items[0].reference);
            return;
        }

        phase.artifacts?.downloadZip(items.map(item => item.reference));

    }, [phase, artifactPaths]);

    const columns: IColumn[] = useMemo(() => [
        { key: 'column1', name: 'Name', minWidth: 794 - 32, maxWidth: 794 - 32, isResizable: false, isPadded: false },
        { key: 'column2', name: 'View_Download', minWidth: 64 + 32, maxWidth: 64 + 32, isResizable: false, isPadded: false }
    ], []);

    const renderItem = useCallback((item: ArtifactHref, index?: number, column?: IColumn) => {
        if (!column)  return null;

        if (column.name === "View_Download") {

            const href = item.href;

            return <Stack data-selection-disabled verticalAlign="center" verticalFill horizontal horizontalAlign="end" style={{ paddingTop: 0, paddingBottom: 0 }}>
                        <IconButton id="artifactview" href={`${href}&inline=true`} target="_blank" style={{ paddingTop: 1, color: "#106EBE" }} iconProps={{ iconName: "Eye", styles: { root: { fontSize: "14px" } } }} />
                        <IconButton id="artifactview" href={href} target="_blank" style={{ paddingTop: 1, color: "#106EBE" }} iconProps={{ iconName: "CloudDownload", styles: { root: { fontSize: "14px" } } }} />
                    </Stack>
        }

        if (column.name === "Name") {

            const path = (item.name as string).split("/");
            const start_index = path.length > 5? path.length - 5 : 0;

            const pathElements = path.slice(start_index).map((t, index) => {
                const last = (index + start_index) === (path.length - 1);
                let color = last ? (dashboard.darktheme ? "#FFFFFF" : "#605E5C") : undefined;
                const font = last ? undefined : "Horde Open Sans Light";
                const sep = last ? undefined : "/"
                return <Text key={`p-${index}`} styles={{ root: { fontFamily: font } }} style={{ color: color }}>{t}{sep}</Text>
            });
            if (start_index !== 0) pathElements.splice(0, 0, <Text key={`p-${item.name}`} >... /</Text>);

            return <Stack verticalFill verticalAlign="center" style={{ cursor: "pointer" }}>
                        <Stack horizontal tokens={{ childrenGap: 8 }} title={item.name}>
                            <Stack>
                                <FontIcon style={{ paddingTop: 1, fontSize: 16 }} iconName="Document" />
                            </Stack>
                            <Stack horizontal>
                                {pathElements}
                            </Stack>
                        </Stack>
                    </Stack>
        }

        return null;
    }, []);

    return <Stack>
            <Modal isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1180, height: 820, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={onClose} className={hordeClasses.modal}>
                <Stack className="horde-no-darktheme" styles={{ root: { paddingTop: 10, paddingRight: 12 } }}>
                    <Stack style={{ paddingLeft: 24, paddingRight: 24 }}>
                        <Stack tokens={{ childrenGap: 12 }} style={{ height: 800 }}>
                            <Stack horizontal verticalAlign="start">
                                <Stack style={{ paddingTop: 3 }}>
                                    <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Artifacts</Text>
                                </Stack>
                                <Stack grow />
                                <Stack horizontalAlign="end">
                                    <IconButton
                                        iconProps={{ iconName: 'Cancel' }}
                                        onClick={onClose}
                                    />
                                </Stack>
                            </Stack>
                            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
                                <Stack tokens={{ childrenGap: 12 }}>
                                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 18 }} style={{ paddingBottom: 12 }}>
                                        <Text style={{fontSize: 13, fontWeight: 600, whiteSpace: 'nowrap', overflow: 'hidden', textOverflow: 'ellipsis', cursor: "text"}}>{phase.name}</Text>
                                        <Stack grow />
                                        <PrimaryButton text={downloadText} onClick={downloadZip}/>
                                    </Stack >
                                    <Stack style={{ height: 492 + 160, position: "relative" }}>
                                        <ScrollablePane style={{ height: 492 + 160 }}>
                                            <SelectionZone selection={selector}>
                                                <DetailsList
                                                    styles={{ root: { overflowX: "hidden" } }}
                                                    className={styles.list}
                                                    isHeaderVisible={false}
                                                    compact={true}
                                                    items={items}
                                                    columns={columns}
                                                    layoutMode={DetailsListLayoutMode.fixedColumns}
                                                    selectionMode={SelectionMode.multiple}
                                                    enableUpdateAnimations={false}
                                                    selection={selector}
                                                    selectionPreservedOnEmptyClick={true}
                                                    onShouldVirtualize={() => false}
                                                    onItemInvoked={(item: ArtifactHref) => {
                                                        window.open(`${item.href}&inline=true`, "_blank");
                                                    }}
                                                    onRenderItemColumn={renderItem}
                                                />
                                            </SelectionZone>
                                        </ScrollablePane>
                                    </Stack>
                                </Stack >
                            </Stack>
                        </Stack>
                    </Stack>
                </Stack>
            </Modal>
        </Stack>
}

export class DomainNameColors {    
    setDomain(Names: string[]) {
        // Only reset the colors if the requested name array is different than what we've already got
        if ((Names.length > 0) && (JSON.stringify(this._colorWheel.domain()) !== JSON.stringify(Names))) {
            this._colorWheel.domain(Array.from(Names));
        }
    }
    
    getDomain() {
        return this._colorWheel.domain();
    }

    get(name?: string, opacity?: string) {
        if (!name) return;
        return this._colorWheel(name) as string + (opacity ? opacity : "");
    }

    private _colorRange = dashboard.darktheme ? [...d3.schemeDark2, ...d3.schemeCategory10] : [...d3.schemeSet2,...d3.schemeTableau10];
    private _colorWheel = d3.scaleOrdinal().range(this._colorRange);
}

class TagColors {
    
    setTagColors(tagNames: string[]) {
        // Strip off the leading "#" tag indicator if it's present
        const cleanTagNames = tagNames.map(t => t.startsWith("#") ? t.substring(1) : t);

        // Only reset the colors if the requested tag array is different than what we've already got
        if ((tagNames.length > 0) && (JSON.stringify(this._tagColors.domain()) !== JSON.stringify(cleanTagNames))) {
            this._tagColors.domain(Array.from(cleanTagNames));
        }
    }

    getTagColor(tag: string) {
        // Strip off the leading "#" tag indicator if it's present
        if (tag.startsWith("#")) {
            tag = tag.substring(1);
        }
        return this._tagColors(tag) as string;
    }

    private _tagColorRange = dashboard.darktheme ? [...d3.schemeDark2, ...d3.schemeCategory10] : [...d3.schemeSet2,...d3.schemeTableau10];
    private _tagColors = d3.scaleOrdinal().range(this._tagColorRange);
}
export const tagColors = new TagColors();

export const getTestUniqueMetaIdentifiers = (testMeta: Map<string, [TestNameRef, MetadataRef[]][]>): Map<string, string[]> => {
    // Convoluted way to get unique meta identifier for each test with the same name
    return new Map(testMeta.entries().filter(([, testRefs]) => testRefs.length > 1).map(([testName, testRefs]) => {
        const testCommonKeys: string[][] = testRefs.map(([, metaRefs]) => MetadataRef.identifyCommonKeys(metaRefs));
        const commonKeys: string[] = MetadataRef.identifyCommonKeys(testRefs.map(([, metaRefs]) => metaRefs).flat());
        const uniqueKeys = testCommonKeys.reduce(
            (current, testCommonKey) => current.filter((key) => testCommonKey.filter(k => !commonKeys.includes(k)).includes(key)),
            testCommonKeys.shift() ?? []
        );
        return [testName, uniqueKeys];
    }));
}

export const StreamSelector: React.FC<{ streams: string[], onClick: (stream: string) => void, selected?: string, disabled?: boolean }> = memo(({ streams, onClick, selected, disabled }) => {
    const menuProps: IContextualMenuProps = getStreamOptions(streams, onClick, selected);
    return <DefaultButton
                style={{ height: 24, fontSize: 11, whiteSpace: 'nowrap', borderWidth: 0, backgroundColor: styles.streamBadge.backgroundColor }}
                menuProps={menuProps}
                text={selected? projectStore.streamById(selected)?.fullname ?? selected : "Compare stream"}
                disabled={disabled}
            />
});

export const EditableTextField: React.FC<{ label: string, onChange: (value: string | undefined) => void, text?: string, placeholder?: string, multiline?: boolean, style?: any, loaded?: boolean }> = ({ label, text, onChange, placeholder, multiline, style, loaded }) => {

    const [enableEditing, setEnableEditing] = useState<boolean>(false); 

    const styles = getStyles();

    return <Stack className={styles.editable} horizontal tokens={{childrenGap: 6}} style={style} grow>
                <Stack horizontal tokens={{childrenGap: 6}} grow verticalAlign="start">
                    <Label>{label}: </Label>
                    {!enableEditing &&
                        <Stack style={{cursor: 'pointer', padding: 5}} grow verticalFill
                            onClick={() => {setEnableEditing(true)}}
                        >
                            {!!text && <Markdown styles={{ subComponentStyles: { paragraph: { root: { marginTop: 0 } } } }}>{text}</Markdown>}
                            {!text && <Text className={styles.placeholder}>{placeholder}</Text>}
                        </Stack>}
                    {enableEditing &&
                        <TextField
                            key={`text-${loaded}`}
                            defaultValue={text}
                            onNotifyValidationResult={(_, value) => { if (!value) { value = undefined; } if (value !== text) { onChange(value); } setEnableEditing(false); }}
                            validateOnFocusOut
                            validateOnLoad={false}
                            multiline={multiline}
                            borderless
                            autoAdjustHeight
                            styles={{root: {width: '100%'}}}
                            placeholder={placeholder}
                            autoFocus
                        />
                    }
                </Stack>
                <Stack className="Edit" style={{padding: 2, minHeight: multiline? 50 : undefined}} verticalFill verticalAlign="center" onClick={() => {setEnableEditing(!enableEditing);}}>
                    <FontIcon style={{ fontSize: 12 }} iconName="Edit" />
                </Stack>
            </Stack>    
}

export const UserPickerModal: React.FC<{ defaultUser?: GetUserResponse, onSelect: (userId?: string) => void, onClose: () => void }> = ({ defaultUser, onSelect, onClose }) => {

    const [state, setState] = useState<{ error?: string, userId?: string }>({});

    const { hordeClasses } = getHordeStyling();

    const onValidate = async () => {
        const userId = state.userId;

        if (userId && userId === defaultUser?.id) {
            state.error = `Already selected this user`;
            setState({ ...state });
            return;
        }

        onSelect(userId);
        onClose();
    }

    return <Modal isOpen={true} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 540 } }} onDismiss={() => { onClose() }}>
            <Stack horizontal styles={{ root: { padding: 8 } }}>
                <Stack.Item grow={2}>
                    <Text variant="mediumPlus">Select a user</Text>
                </Stack.Item>
                <Stack.Item grow={0}>
                    <IconButton
                        iconProps={{ iconName: 'Cancel' }}
                        ariaLabel="Close popup modal"
                        onClick={() => { onClose(); }}
                    />
                </Stack.Item>
            </Stack>

            {!!state.error && <MessageBar
                messageBarType={MessageBarType.error}
                isMultiline={false}> {state.error} </MessageBar>}

            <Stack styles={{ root: { padding: 8 } }}>
                <Stack grow>
                    <Stack tokens={{ childrenGap: 4 }}>
                    <UserSelect autoFocus handleSelection={(id => {
                        setState({ ...state, userId: id })
                    })} userIdHints={defaultUser? [defaultUser.id] : undefined} defaultUser={defaultUser} noResultsFoundText="No users found" />
                    <Text variant="small" style={{ paddingTop: 4 }}>Leave blank to unassign</Text>
                    </Stack>
                </Stack>
                
                <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
                    <Stack grow />
                    <PrimaryButton text="Select" onClick={() => { onValidate(); }} />
                    <DefaultButton text="Cancel" onClick={() => { onClose(); }} />
                </Stack>
            </Stack>
        </Modal>;
};

export const SessionListSelector = (items: TestSessionResult[] | PhaseSessionResult[], index: number, text?: string): JSX.Element => {
    const selectedFilter = dashboard.darktheme ? "brightness(1.5)" : "brightness(0.5)";
    const backColor = dashboard.darktheme ? "#2D2B29" : "#EDEBE9";
    const textColor = dashboard.darktheme ? "#979797ff" : "#707070ff";

    return <Stack>
                <Stack style={{height: 3, backgroundColor: backColor}} grow horizontal tokens={{childrenGap: items.length < 20? 1: 0}}>
                    { items.map( (item, i) => {
                            const itemFilter = index === i? selectedFilter : undefined;
                            return <Stack key={`s-${i}`} grow verticalFill style={{backgroundColor: getSessionColor(item), filter: itemFilter}}></Stack>
                        } )
                    }
                </Stack>
                {!!text && <Text style={{color: textColor}} variant="xSmall">&ensp;{text}</Text>}
            </Stack>
}

export class CircularList<T> {
    constructor(items: T[]) {
        this._items = items;
        this._cursor = -1;
    }

    setCursor(item: T | undefined): number {
        if (item === undefined) {
            this._cursor = -1;
        } else if (this.selected !== item) {
            this._cursor = this._items.findIndex(i => i === item);
        }

        return this._cursor;
    }

    get selected(): T | undefined {
        if (this._cursor === -1) return;
        return this._items[this._cursor];
    }

    next(): T {
        if (++this._cursor === this._items.length) this._cursor = 0;
        return this._items[this._cursor];
    }

    back(): T {
        if (--this._cursor < 0) this._cursor = this._items.length - 1;
        return this._items[this._cursor];
    }

    get cursor() {
        return this._cursor;
    }

    get length() {
        return this._items.length;
    }

    private _cursor: number;
    private _items: T[];
}
