// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, DirectionalHint, FontIcon, HoverCard, HoverCardType, Image, IPlainCardProps, Link, mergeStyles, mergeStyleSets, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { Artifacts, EmbeddedHTML, EventEntry, EventLevel, ImageCompare, LogPropertyType, URLLink } from "./testEventsModel";
import { TestPhaseOutcome } from "./api";
import { memo, useCallback, useEffect, useState } from "react";
import { ArtifactFactory, DeviceRef, TestPhaseStatus } from "./testData";
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { getShortNiceTime, msecToElapsed } from "horde/base/utilities/timeUtils";
import { DashboardPreference } from "horde/backend/Api";
import { PhaseArtifactsModal, styles, getStatusColors } from "./testAutomationCommon";

let eventStyles: any = undefined;
export const getEventStyling = () => {
    if (eventStyles) return eventStyles;

    const colors = getStatusColors();
    const errorColorPref = dashboard.getPreference(DashboardPreference.ColorError);
    const warningColorPref = dashboard.getPreference(DashboardPreference.ColorWarning);
    const highlight = dashboard.darktheme ? "brightness(1.2)" : "brightness(0.9)";
    const strongHighlight = dashboard.darktheme ? "brightness(1.8)" : "brightness(0.8)";

    const eventClass = mergeStyles({
        fontSize: "12px",
        fontFamily: "Horde Cousine Regular",
        paddingBottom: 3,
        paddingTop: 3,
        paddingRight: 5,
        paddingLeft: 3
    });

    eventStyles = mergeStyleSets({
        event: eventClass,
        eventWarning: [{
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",
            borderLeftStyle: 'solid',
            borderLeftColor: colors.get(StatusColor.Warnings)!
        }, eventClass],
        eventError: [{
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
            borderLeftStyle: 'solid',
            borderLeftColor: colors.get(StatusColor.Failure)!
        }, eventClass],
        warningButton: {
            minWidth: 16,
            backgroundColor: warningColorPref ? warningColorPref : dashboard.darktheme ? "#9D840E" : "#F7D154",
            borderStyle: "hidden",
            selectors: {
                ':active,:hover': {
                    backgroundColor: warningColorPref ? warningColorPref : "#E7C144",
                    filter: highlight
                }
            }
        },
        interruptedButton: {
            minWidth: 16,
            backgroundColor: colors.get(StatusColor.Unspecified)!,
            borderStyle: "hidden",
            color: "#FFFFFF",
            selectors: {
                ':active,:hover': {
                    color: "#FFFFFF",
                    backgroundColor: colors.get(StatusColor.Unspecified)!,
                    filter: highlight
                }
            }
        },
        failureButton: {
            minWidth: 16,
            backgroundColor: errorColorPref ? errorColorPref : dashboard.darktheme ? "#9D1410" : "#EC4C47",
            borderStyle: "hidden",
            color: "#FFFFFF",
            selectors: {
                ':active,:hover': {
                    color: "#FFFFFF",
                    backgroundColor: errorColorPref ? errorColorPref : "#DC3C37",
                    filter: highlight
                }
            }
        },
        failureButtonDisabled: {         
            borderStyle: "hidden",
            color: dashboard.darktheme ? "#909398" : "#616E85 !important",
            backgroundColor: dashboard.darktheme ? "#1F2223" : "#f3f2f1"
        },
        defaultButton: {
            borderStyle: 'solid', borderWidth: 1,
            color: dashboard.darktheme ? "#909398" : "#616E85 !important",
            backgroundColor: dashboard.darktheme ? "#1F2223" : "#f3f2f1",
            borderColor: dashboard.darktheme ? "#4D4C4B" : "#6D6C6B",
            selectors: {
                ':active,:hover': {
                    filter: highlight
                }
            }
        },
        artifactLinks: {
            borderRadius: 6,
            padding: '2px 6px',
            backgroundColor: dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "rgba(0, 0, 0, 0.3)",
            selectors: {
                ':active,:hover': {
                    backgroundColor: dashboard.darktheme ? "rgba(255, 255, 255, 0.3)" : "rgba(0, 0, 0, 0.2)"
                }
            }
        },
        highlighted: {
            filter: strongHighlight
        }
    });

    return eventStyles;
}

/// Main event entry point
const Event: React.FC<{ entry: EventEntry, phase: TestPhaseStatus }> = memo(({entry, phase}) => {
    const eventStyles = getEventStyling();

    const isError = entry.level === EventLevel.Error || entry.level === EventLevel.Critical;
    const isWarning = entry.level === EventLevel.Warning;
    const style = isError ? eventStyles.eventError : (isWarning ? eventStyles.eventWarning : eventStyles.event);

    return <Stack className={style}>
                {BasicEvent(entry)}
                {!!entry.properties && Object.values(entry.properties).map(property => !!property && EventTypeChooser(property, phase))}
            </Stack>
});

const EventTypeChooser = (property: string | number | Record<string, string | number | undefined>, phase: TestPhaseStatus): JSX.Element | undefined => {
    switch (property["$type"]) {
        case LogPropertyType.ImageCompare:
            return <ImageComparer key={`ic-${property["$text"]}`} imageCompare={property as ImageCompare} phase={phase} />

        case LogPropertyType.EmbeddedHTML:
            return <EmbeddedHTMLWidget key={`e-html-${property["$text"]}`} embeddedHTML={property as EmbeddedHTML} phase={phase} />

        case LogPropertyType.Artifacts:
            return <ArtifactsWidget key={`a-${property["$text"]}`} artifacts={property as Artifacts} phase={phase} />

        case LogPropertyType.URLLink:
            return <URLLinkWidget key={`link-${property['$text']}`} url={property as URLLink} phase={phase} />
    }

    return;
}

const BasicEvent = (entry: EventEntry): JSX.Element => {
    return <pre style={{ margin: 0, whiteSpace: "pre-wrap" }}>[{entry.time}] {entry.level}: {entry.message}</pre>
}

/// Event pane
const OnDeviceCard = (device: DeviceRef): JSX.Element => {
    const meta = device.metadata;
    return <Stack style={{padding: 6}}>
            { meta && <Stack horizontal tokens={{childrenGap: 4}}>
                        <Stack tokens={{childrenGap: 4}}>
                            {Object.keys(meta).map((key, i) => <Stack style={{fontWeight: "bold"}} key={`device-${device.name}-key-${i}`}>{key}: </Stack>)}
                        </Stack>
                        <Stack tokens={{childrenGap: 4}}>
                            {Object.values(meta).map((value, i) => <Stack key={`device-${device.name}-value-${i}`}>{value}</Stack>)}
                        </Stack>
                </Stack>
            }
            { !meta && <Text>no metadata</Text> }
        </Stack>;
}

const DeviceItem = (device: DeviceRef, index: number): JSX.Element => {
    const deviceCardProps: IPlainCardProps = {
        renderData: device,
        onRenderPlainCard: OnDeviceCard,
        directionalHint: DirectionalHint.topRightEdge,
        gapSpace: 8,
        calloutProps: { isBeakVisible: true }
    };

    return <HoverCard plainCardProps={deviceCardProps} type={HoverCardType.plain} cardOpenDelay={20} key={`${device.name}#${index}`}>
                <Text style={{fontSize: 12, cursor: 'pointer', paddingLeft: 3, paddingRight: 3}} styles={{root:{selectors: {':hover': {backgroundColor: "rgba(125, 125, 125, 0.3)"}}}}}>{device.name}</Text>
            </HoverCard>
}

export const PhaseEventsPane: React.FC<{phase: TestPhaseStatus, onEventsLoaded?: (events: EventEntry[]) => void}> = ({phase, onEventsLoaded}) => {
    const [events, setEvents] = useState<EventEntry[] | undefined>(undefined);
    const [filterError, setFilterError] = useState<boolean>(true);
    const [filterWarning, setFilterWarning] = useState<boolean>(true);
    const [filterInfo, setFilterInfo] = useState<boolean>(false);
    const [artifacts, setArtifacts] = useState<string[] | undefined>(undefined);
    const [artifactModal, setArtifactModal] = useState<boolean>(false);

    useEffect(() => {
        // reset events and filtering
        if (events !== undefined) {
            setEvents(undefined);
            setArtifacts(undefined);
        }
        if (!filterInfo && phase.outcome !== TestPhaseOutcome.Failed) {
            setFilterInfo(true);
        }
        // fetch the events
        phase.fetchEventStream().then((value) => {
            setEvents(value ?? []);
        }).catch((reason) => setEvents([{level: EventLevel.Error, message: `Could not retrieve test events: ${reason}`, format: "", time: "", id: 0}]));
    }, [phase]);

    useEffect(() => {
        if (events) onEventsLoaded?.(events);
    }, [events]);

    const collectPhaseArtifacts = useCallback((): string[] | undefined => {
        if (!events) return;
        // collect artifact references
        const artifacts: Set<string> = new Set();
        // phase logs
        phase.devices.forEach(device => {
            if (device.logPath) artifacts.add(device.logPath);
        });
        // parse event properties
        events.forEach(entry => {
            if (!!entry.properties) {
                Object.values(entry.properties).forEach(property => {
                    if (!property) return;
                    switch (property["$type"]) {
                        case LogPropertyType.ImageCompare:
                            const imageCompare = property as ImageCompare;
                            if (imageCompare.approved) artifacts.add(imageCompare.approved);
                            if (imageCompare.approved_metadata) artifacts.add(imageCompare.approved_metadata);
                            if (imageCompare.unapproved) artifacts.add(imageCompare.unapproved);
                            if (imageCompare.unapproved_metadata) artifacts.add(imageCompare.unapproved_metadata);
                            if (imageCompare.difference) artifacts.add(imageCompare.difference);
                            if (imageCompare.difference_report) artifacts.add(imageCompare.difference_report);
                            break;

                        case LogPropertyType.EmbeddedHTML:
                            const embeddedHTML = property as EmbeddedHTML;
                            if (embeddedHTML.path) artifacts.add(embeddedHTML.path);
                            break;

                        case LogPropertyType.Artifacts:
                            const artifactsProp = property as Artifacts
                            Object.entries(artifactsProp).forEach(([key, value]) => {
                                if (!value || key.startsWith('$')) return;
                                artifacts.add(value);
                            });
                            break;
                    }
                });
            }
        });
        return [...artifacts];
    }, [events]);

    const isEntryNeedDisplay = useCallback((event: EventEntry): boolean => {
        if (filterError && (event.level === EventLevel.Error || event.level === EventLevel.Critical)) {
            return true;
        }
        if (filterWarning && event.level === EventLevel.Warning) {
            return true;
        }
        if (filterInfo && event.level === EventLevel.Information) {
            return true;
        }

        return false;
    }, [filterError, filterWarning, filterInfo]);

    const uniqueDevices = phase.devices.filter((device, index) => phase.devices.findIndex(other => device.isSame(other)) === index);

    const borderColor = dashboard.darktheme ? "#4D4C4B" : "#6D6C6B";

    return <Stack style={{borderWidth: 1, borderColor: borderColor, borderStyle: 'solid', padding: 4, overflow: 'hidden'}} grow>
                {!events &&
                    <Stack horizontalAlign='center' style={{ padding: 12}} grow>
                        <Spinner size={SpinnerSize.medium} />
                    </Stack>
                }
                {!! events && artifactModal && artifacts &&
                    <PhaseArtifactsModal phase={phase} artifactPaths={artifacts} onClose={() => setArtifactModal(false)}/>
                }
                {!!events &&
                    <Stack style={{overflow: 'hidden'}} tokens={{childrenGap: 3}} grow>
                        <Stack style={{borderWidth: '0px 0px 1px 0px', borderColor: borderColor, borderStyle: 'solid', paddingBottom: 2}} horizontal tokens={{childrenGap: 8}} disableShrink>
                            <Text><span style={styles.labelSmall}>Run on: </span><span>{getShortNiceTime(phase.start, true, true, true)}</span></Text>
                            <Text><span style={styles.labelSmall}> For: </span><span>{msecToElapsed(phase.duration * 1000, true, true)}</span></Text>
                            {!!uniqueDevices.length &&
                                <Stack horizontal tokens={{childrenGap: 4}} verticalAlign="end">
                                    <Text style={styles.labelSmall}> Devices: </Text>
                                    {uniqueDevices.splice(0, 5).map((d, index) => DeviceItem(d, index))}
                                    {uniqueDevices.length > 5 &&
                                        <Text style={{userSelect: 'none'}}>...</Text>
                                    }
                                </Stack>
                            }
                            <Stack verticalAlign="center" horizontal tokens={{ childrenGap: 5 }} style={{borderWidth: '0px 1px', paddingLeft: 9, paddingRight: 9, borderStyle: 'solid', borderColor: borderColor}}>
                                <FontIcon iconName="PageListFilter" title="Events filtering" style={styles.icon} />
                                <Checkbox styles={styles.defaultCheckbox} label="Error" checked={filterError} onChange={(_: any, checked?: boolean) => setFilterError(!!checked)} disabled={!events.length}/>
                                <Checkbox styles={styles.defaultCheckbox} label="Warning" checked={filterWarning} onChange={(_: any, checked?: boolean) => setFilterWarning(!!checked)} disabled={!events.length}/>
                                <Checkbox styles={styles.defaultCheckbox} label="Info" checked={filterInfo} onChange={(_: any, checked?: boolean) => setFilterInfo(!!checked)} disabled={!events.length}/>
                            </Stack>
                            <Stack horizontalAlign="end">
                                <DefaultButton disabled={artifacts && artifacts.length === 0} style={{ fontSize: 11, padding: 0, height: 18, color: 'white', backgroundColor: artifacts && artifacts.length === 0 ? undefined : styles.defaultButton.backgroundColor }}
                                    onClick={(ev) => {
                                        ev.stopPropagation();
                                        let collectedArtifacts = artifacts;
                                        if (!collectedArtifacts) {
                                            collectedArtifacts = collectPhaseArtifacts();
                                            setArtifacts(collectedArtifacts);
                                        }
                                        if (collectedArtifacts && collectedArtifacts.length > 0) setArtifactModal(true);
                                    }}>Artifacts</DefaultButton>
                            </Stack>
                        </Stack>
                        <Stack style={{ overflowX: 'hidden', overflowY: 'auto' }}>
                            { events.map((e, i) => isEntryNeedDisplay(e) && <Event key={`phase-${phase.key}-event-${i}`} entry={e} phase={phase}/>)}
                            { !events.length && <Text style={styles.textSmall}>No events</Text>}
                        </Stack>
                    </Stack>
                }
            </Stack>
}

/// Artifacts widget
type ArtifactLink = { name: string, href: string }
const getArtifactLinks = async (artifacts: Artifacts, factory: ArtifactFactory): Promise<ArtifactLink[]> => {
    const links: ArtifactLink[] = [];
    for (const [key, value] of Object.entries(artifacts)) {
        if (key.startsWith('$') || !value) continue;
        const href = await factory.getLink(value);
        links.push({name: key, href: href});
    }

    return links;
}

const ArtifactsWidget: React.FC<{artifacts: Artifacts, phase: TestPhaseStatus}> = memo(({artifacts, phase}) => {
    const [artifactsInfo, setArtifactsInfo] = useState<ArtifactLink[] | undefined>(undefined);

    const eventStyles = getEventStyling();

    useEffect(() => {
        const phaseArtifacts = phase.artifacts;
        !!phaseArtifacts && getArtifactLinks(artifacts, phaseArtifacts).then(info => setArtifactsInfo(info));
    }, [artifacts, phase])

    return  <Stack styles={{ root: { paddingLeft: 16, width: '100%' } }}>
                <Stack><Text variant="medium"><span style={{ fontWeight: "bold" }}>{artifacts['$type']}: </span>{artifacts['$text']}</Text></Stack>
                {artifactsInfo &&
                    <Stack horizontal tokens={{childrenGap: 6}}>
                        {artifactsInfo.map(item => 
                                <Stack className={eventStyles.artifactLinks}>
                                    <Link href={item.href} target="_blank">{item.name}</Link>
                                </Stack>
                            )
                        }
                    </Stack>
                }
            </Stack>
});

/// Embedded HTML widget
const getEmbeddedHTMLLink = async (artifact: EmbeddedHTML, factory: ArtifactFactory): Promise<string> => {
    return await factory.getLink(artifact.path);
}

const EmbeddedHTMLWidget: React.FC<{embeddedHTML: EmbeddedHTML, phase: TestPhaseStatus}> = memo(({embeddedHTML, phase}) => {
    const [artifactInfo, setArtifactInfo] = useState<string | undefined>(undefined);

    useEffect(() => {
        const phaseArtifacts = phase.artifacts;
        !!phaseArtifacts && getEmbeddedHTMLLink(embeddedHTML, phaseArtifacts).then(info => setArtifactInfo(info));
    }, [embeddedHTML, phase])

        const borderColor = dashboard.darktheme ? "#4D4C4B" : "#6D6C6B";

    return  <Stack styles={{ root: { paddingLeft: 16, width: '100%' } }}>
                {artifactInfo &&
                    <Stack style={{borderWidth: 1, borderColor: borderColor, borderStyle: 'solid', padding: 4, margin: 1}} grow>
                        <iframe style={{resize: 'vertical'}} src={artifactInfo}/>
                    </Stack>
                }
            </Stack>
});

/// URL link widget
const URLLinkWidget: React.FC<{url: URLLink, phase: TestPhaseStatus}> = memo(({url, phase}) => {
    const eventStyles = getEventStyling();
    return <Stack styles={{ root: { paddingLeft: 16 } }} horizontal>
                <Stack className={eventStyles.artifactLinks}>
                    <Link href={url.href} target="_blank">{url.$text}</Link>
                </Stack>
            </Stack>
})

/// Image comparison widget
const missingImage = "/images/missing-image.png";
const MissingImageLabel = (): JSX.Element => { return <span style={{ fontWeight: 'bold' }}> [missing image]</span> }
type ImageData = { link?: string, ref?: string }
type ImageLinks = {
    unapproved: ImageData;
    approved?: ImageData;
    difference?: ImageData;
}

const getImageLinks = async (imageCompare: ImageCompare, artifacts: ArtifactFactory): Promise<ImageLinks> => {
    let url = await artifacts.getLink(imageCompare.unapproved);
    const imageInfo: ImageLinks = {unapproved: { link: url, ref: imageCompare.unapproved }};
    if (imageCompare.approved) {
        url = await artifacts.getLink(imageCompare.approved);
        imageInfo.approved = { link: url, ref: imageCompare.approved };
    }
    if (imageCompare.difference) {
        url = await artifacts.getLink(imageCompare.difference);
        imageInfo.difference = { link: url, ref: imageCompare.difference };
    }

    return imageInfo;
}

const ImageComparer: React.FC<{imageCompare: ImageCompare, phase: TestPhaseStatus}> = memo(({imageCompare, phase}) => {
    const [imageInfo, setImageInfo] = useState<ImageLinks | undefined>(undefined);

    useEffect(() => {
        const phaseArtifacts = phase.artifacts;
        !!phaseArtifacts && getImageLinks(imageCompare, phaseArtifacts).then(info => setImageInfo(info));
    }, [imageCompare, phase])

    return  <Stack styles={{ root: { paddingLeft: 16, width: '100%' } }}>
                <Stack><Text variant="medium"><span style={{ fontWeight: "bold" }}>{imageCompare['$type']}: </span>{imageCompare['$text']}</Text></Stack>
                {imageInfo &&
                    <Stack horizontal>
                        {imageInfo.approved &&
                            <Stack styles={{ root: { padding: 5 } }}>
                                <a href={imageInfo.approved.link}>
                                    <Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo.approved.link || missingImage} alt={imageInfo.approved.ref} />
                                </a>
                                <Stack.Item align="center">Reference{!imageInfo.approved && MissingImageLabel()}</Stack.Item>
                            </Stack>
                        }
                        {imageInfo.difference &&
                            <Stack styles={{ root: { padding: 5 } }}>
                                <a href={imageInfo.difference.link}>
                                    <Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo.difference.link || missingImage} alt={imageInfo.difference.ref} />
                                </a>
                                <Stack.Item align="center">Difference{!imageInfo.difference && MissingImageLabel()}</Stack.Item>
                            </Stack>
                        }
                        <Stack styles={{ root: { padding: 5 } }}>
                            <a href={imageInfo.unapproved.link}>
                                <Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo.unapproved.link || missingImage} alt={imageInfo.unapproved.ref} />
                            </a>
                            <Stack.Item align="center">Produced{!imageInfo.unapproved && MissingImageLabel()}</Stack.Item>
                        </Stack>
                    </Stack>
                }
            </Stack>
});
