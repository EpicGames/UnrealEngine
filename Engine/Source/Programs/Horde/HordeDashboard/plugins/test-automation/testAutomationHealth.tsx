// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, FontIcon, IColumn, IconButton, Label, Modal, SelectionMode, Stack, Text, HoverCard, HoverCardType, IPlainCardProps, DirectionalHint, DetailsListLayoutMode, Spinner, SpinnerSize, Toggle, DatePicker, ComboBox, IComboBoxOption, FocusZone, FocusZoneTabbableElements, FocusTrapCallout, PrimaryButton, DefaultButton, TooltipHost, TooltipOverflowMode, TooltipDelay, Link, IRenderFunction, IDetailsHeaderProps, Sticky, StickyPositionType, ScrollablePane, IContextualMenuProps } from "@fluentui/react";
import { MetadataRef, TestDataHandler, TestNameRef, TestSessionResult } from "./testData";
import { useCallback, useEffect, useMemo, useState } from "react";
import { observer } from "mobx-react-lite";
import { TestOutcome } from "./api";
import { DomainNameColors, EditableTextField, getTestUniqueMetaIdentifiers, StreamSelector, tagColors, UserPickerModal, styles, getStatusColors } from "./testAutomationCommon";
import { TestHistoryGraphWidget } from "./testAutomationSummary";
import { StoreKey, userStorage } from "./testAutomationUserStorage";
import { projectStore } from "horde/backend/ProjectStore";
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { getShortNiceTime, msecToElapsed } from "horde/base/utilities/timeUtils";
import { booleanColors, cadenceColors, cadenceOrders, healthColors, healthStars, TestCadence, TestHealth, TestHealthExplain, TestHealthItem, TestHealthTiming, UsersInfoCache } from "./testHealthData";
import { GetUserResponse, DeepLinkResponse } from "horde/backend/Api";
import { errorDialogStore } from "horde/components/error/ErrorStore";
import backend from "horde/backend";
import { downloadCsv } from "horde/base/utilities/csvDownload";

/// Colors
const profileColors = new DomainNameColors();

const teamColors = new DomainNameColors();

const harnessColors = new DomainNameColors();

const healthBadgeBackground = () => dashboard.darktheme? "#67676748" : "#8f8f8f9e";
const textEmphasisColor = () => dashboard.darktheme? "#ffffff75" : "#00000075";
const borderCardColor = () => dashboard.darktheme ? "#2D2B29" : "#EDEBE9";

const healthStyles = {
    userTag: {
        backgroundColor: booleanColors.get(false),
        padding: '2px 9px',
        borderRadius: 11,
        height: 23
    },
    badge: {
        padding: '3px 12px',
        borderRadius: 4,
        cursor: 'default'
    }
}

enum UserType {
    Owner = "Owner",
    Customers = "Customers",
}

const onRenderIntentCard = (item: TestHealthItem): JSX.Element => {
    return <Stack style={{pointerEvents: 'none', padding: 10, maxWidth: 300, border: `1px solid ${borderCardColor()}`}} tokens={{childrenGap: 5}}>{item.intent}</Stack>
}

const renderTestTitle = (item: TestHealthItem): JSX.Element => {
    const maskMetaString = item.displayMeta;
    let title = item.name;
    if (maskMetaString) title += ` - ${maskMetaString}`;
    const DetailsCardProps: IPlainCardProps | undefined = item.intent? {
        onRenderPlainCard: onRenderIntentCard,
        renderData: item,
        directionalHint: DirectionalHint.topLeftEdge,
        calloutProps: { isBeakVisible: true }
    } : { onRenderPlainCard: () => null /** empty card */ };
    return <HoverCard plainCardProps={DetailsCardProps} type={HoverCardType.plain} cardOpenDelay={300} sticky={false} styles={{host: {width: '100%'}}}>
                <Text title={title} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap'}}>
                    {item.name}{maskMetaString && <span style={{fontSize: 11}}> - {maskMetaString}</span>}
                </Text>
            </HoverCard>
}

const onRenderExplainCard = (item: TestHealthItem): JSX.Element => {
    const explainData: TestHealthExplain = item.healthExplain!;
    return <Stack style={{pointerEvents: 'none', padding: 10}} tokens={{childrenGap: 5}}>
                {!!item.underAudit && <Stack style={{ padding: 4, borderRadius: 4, backgroundColor: booleanColors.get(true) }}><Text>Under Audit</Text></Stack>}
                <Stack style={{ padding: 4, borderRadius: 4, backgroundColor: healthBadgeBackground()}}><Text style={{color: healthColors.get(item.health), fontWeight: 800}}>{item.health}</Text></Stack>
                {!!explainData.successRate &&
                    <Stack horizontal>
                        <Stack grow style={{fontWeight: 600}}>Success &nbsp;</Stack><Stack horizontalAlign="end">{explainData.successRate}%</Stack>
                    </Stack>
                }
                {!!explainData.failureRate &&
                    <Stack>
                        <Stack horizontal><Stack grow style={{fontWeight: 600}}>Failure &nbsp;</Stack><Stack horizontalAlign="end">{explainData.failureRate}%</Stack></Stack>
                        {!!explainData.catastrophicFailureRate &&
                            <Stack style={{paddingLeft: 10, fontSize: 11}} horizontal>
                                <Stack grow>&bull; Catastrophic &nbsp;</Stack><Stack horizontalAlign="end">{explainData.catastrophicFailureRate}%</Stack>
                            </Stack>
                        }
                        {!!explainData.redundantErrorRate &&
                            <Stack style={{paddingLeft: 10, fontSize: 11}} horizontal>
                                <Stack grow>&bull; Redundant &nbsp;</Stack><Stack horizontalAlign="end">{explainData.redundantErrorRate}%</Stack>
                            </Stack>
                        }
                    </Stack>
                }
            </Stack>
}

const renderHealthBadge = (item: TestHealthItem, width?: number): JSX.Element => {
    const StatusCardProps: IPlainCardProps | undefined = item.healthExplain? {
        onRenderPlainCard: onRenderExplainCard,
        renderData: item,
        directionalHint: DirectionalHint.rightCenter,
        calloutProps: { isBeakVisible: true }
    } : { onRenderPlainCard: () => null /** empty card */ };
    return <HoverCard plainCardProps={StatusCardProps} type={HoverCardType.plain} cardOpenDelay={20} sticky={false}>
                <Stack style={{backgroundColor: item.underAudit? booleanColors.get(true) : healthBadgeBackground(), padding: 3, borderRadius: 4, width: width, height: 24, cursor: 'inherited'}} horizontal verticalAlign="center" tokens={{childrenGap: 3}} horizontalAlign="center">
                    {new Array(healthStars.get(item.health)).fill(1).map((_, i) => <FontIcon key={`star-${i}`} style={{ fontSize: 11, color: healthColors.get(item.health) }} iconName="Star" />)}
                </Stack>
            </HoverCard>
}

type TestHealthColumn = IColumn & {
    compare?: (a: TestHealthItem, b: TestHealthItem) => number;
}

type TestHealthListState = {
    items: TestHealthItem[];
    columns: TestHealthColumn[];
}

const getHealthColumns = (items: TestHealthItem[], columns: TestHealthColumn[], onClickItem: (item: TestHealthItem) => void, setListState: (state: TestHealthListState) => void): TestHealthColumn[] => {
    const innerState: TestHealthListState = {
        items: items,
        columns: []
    }
    
    const onClickColumn = (column: TestHealthColumn) => {
        const newColumns: TestHealthColumn[] = innerState.columns.slice();
        const cColumn: TestHealthColumn = newColumns.find(c => column.key === c.key)!;
        newColumns.forEach(newColumn => {
            if (newColumn === cColumn) {
                newColumn.isSortedDescending = !newColumn.isSortedDescending;
                newColumn.isSorted = true;
            } else {
                newColumn.isSorted = false;
                newColumn.isSortedDescending = !newColumn.isSortedDescending;
            }
        });

        const newItems = items.slice().sort((a, b) => (cColumn.isSortedDescending? 1 : -1) * (cColumn.compare?.(a, b) ?? 0));

        setListState({
            items: newItems,
            columns: newColumns
        })
    }

    const newColumns: TestHealthColumn[] = [
        {
            key: 'name',
            name: 'Test',
            minWidth: 32,
            maxWidth: 250,
            isResizable: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                return <Stack onClick={() => onClickItem(item)} style={{cursor: 'pointer'}} horizontal>{renderTestTitle(item)}</Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => a.name.localeCompare(b.name)
        },
        {
            key: 'health',
            name: 'Health',
            minWidth: 80,
            maxWidth: 80,
            isResizable: true,
            isCollapsible: true,
            isSorted: true,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => <Stack onClick={() => onClickItem(item)} style={{cursor: 'pointer'}}>{renderHealthBadge(item)}</Stack>,
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => healthStars.get(a.health)! - healthStars.get(b.health)!
        },
        {
            key: 'profile',
            name: 'Profile',
            minWidth: 90,
            maxWidth: 90,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                const profile = !!item.profile ? item.profile : "N/A";
                return <Stack style={{...healthStyles.badge, backgroundColor: profileColors.get(item.profile)}} horizontalAlign="center" horizontal>
                            <Text title={profile} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap'}}>{profile}</Text>
                        </Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => a.profile?.localeCompare(b.profile ?? "") ?? -1
        },
        {
            key: 'team',
            name: 'Team',
            minWidth: 90,
            maxWidth: 90,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                const team = !!item.team ? item.team : "N/A";
                return <Stack style={{...healthStyles.badge, backgroundColor: teamColors.get(item.team)}} horizontalAlign="center" horizontal>
                            <Text title={team} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap'}}>{team}</Text>
                        </Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => a.team?.localeCompare(b.team ?? "") ?? -1
        },
        {
            key: 'owner',
            name: 'Owner',
            minWidth: 70,
            maxWidth: 120,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                return <Stack style={{...healthStyles.userTag, backgroundColor: booleanColors.get(!item.owner), cursor: 'default', height: '100%'}} horizontalAlign="center" verticalAlign="center" horizontal>
                            <Text title={usersInfoCache.getInfo(item.owner)?.name} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap'}}>
                                {!!item.owner && (usersInfoCache.getInfo(item.owner)?.name ?? <Spinner size={SpinnerSize.small}/>)}
                                {!item.owner && "N/A"}
                            </Text>
                        </Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => a.owner?.localeCompare(b.owner ?? "") ?? -1
        },
        {
            key: 'hasNotification',
            name: 'Notification',
            minWidth: 32,
            maxWidth: 70,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                return <Stack style={{backgroundColor: booleanColors.get(!item.notification), padding: 3, borderRadius: 4, cursor: 'default'}} horizontalAlign="center" title={!!item.notification? "Enabled" : "Not set"}>
                            <FontIcon iconName={!!item.notification ? "MessageFill": "MuteChat"} style={{fontSize: 16, padding: 1}} />
                        </Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => a.notification?.localeCompare(b.notification ?? "") ?? -1
        },
        {
            key: 'tags',
            name: 'Tags',
            minWidth: 32,
            maxWidth: 300,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            onRender: (item: TestHealthItem) => {
                const content = <Stack horizontal verticalAlign="center" tokens={{childrenGap: 4}} wrap style={{maxWidth: 350}}>
                                    {item.nameRef?.tagRefs &&
                                        item.nameRef.tagRefs.keys().map(t => t.name).toArray().sort().map(tag =>
                                            <Stack key={`tag-${tag}`} style={{backgroundColor: tagColors.getTagColor(tag), borderRadius: 5, padding: '1px 4px', opacity: 0.8}}>
                                                <Text>#{tag}</Text>
                                            </Stack>
                                        )
                                    }
                                </Stack>;

                return <Stack style={{cursor: 'default', paddingTop: 2, height: 24, maskImage: 'linear-gradient(to right, white 70%, transparent)'}}>
                            <TooltipHost content={content} overflowMode={TooltipOverflowMode.Self}>{content}</TooltipHost>
                        </Stack>
            },
        },
        {
            key: 'cadence',
            name: 'Cadence',
            minWidth: 75,
            maxWidth: 120,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                return <Stack style={{...healthStyles.badge, backgroundColor: cadenceColors.get(item.cadence)}} horizontalAlign="center" horizontal>
                            <Text title={item.cadence} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap'}}>{item.cadence}</Text>
                        </Stack>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => cadenceOrders.get(a.cadence)! - cadenceOrders.get(b.cadence)!
        },
        {
            key: 'lastCompletedDate',
            name: 'Last Uninterrupted Run',
            minWidth: 75,
            maxWidth: 100,
            isResizable: true,
            isCollapsible: true,
            isSorted: false,
            isSortedDescending: true,
            onRender: (item: TestHealthItem) => {
                return <Text>{getShortNiceTime(item.timings?.latestCompletedRun, true, true)}</Text>
            },
            onColumnClick: (_, column) => onClickColumn(column),
            compare: (a: TestHealthItem, b: TestHealthItem) => (a.timings?.latestCompletedRun?.getTime() ?? 0) - (b.timings?.latestCompletedRun?.getTime() ?? 0)
        },
    ];
    
    newColumns.forEach(column => {
        innerState.columns.push(column);
        if (columns.length > 0) {
            // copy column states
            const oldColumn = columns.find(c => column.key === c.key);
            if (oldColumn) {
                column.isSorted = oldColumn.isSorted;
                column.isSortedDescending = oldColumn.isSortedDescending;
            }
        }
    });

    const sortedColumn = newColumns.find(item => item.isSorted);
    if (sortedColumn) {
        items.sort((a, b) => (sortedColumn.isSortedDescending? 1 : -1) * (sortedColumn.compare?.(a, b) ?? 0))
    }

    return innerState.columns;
}

const percRound = (value: number) => Math.round(value * 100);

const getTestHealthItems = (handler: TestDataHandler, stream?: string, test?: TestNameRef) => {
    if (!stream) {
        return [];
    }

    const testStatus = handler.getStatusStream(stream);
    if (!testStatus?.tests.size) {
        return [];
    }

    let filteredTests = handler.filteredTests;
    if (!filteredTests.length) {
        return [];
    }
    if (!!test) {
        if (!filteredTests.find(t => t === test)) {
            return [];
        }
        filteredTests = [test];
    }

    const filteredMetadata = handler.filteredMetadata;
    
    // collect data and reorganize it for computation
    const testHealths: TestHealthItem[] = [];
    const testMeta: Map<string, [TestNameRef, MetadataRef[]][]> = new Map();
    let testMetaKeys: Map<string, string[]> | undefined;

    filteredTests.forEach((test) => {
        const status = testStatus!.tests.get(test);
        if (!status) return;

        let totalPhaseCount = 0;
        let totalUndefinedCount = 0;
        let totalFailureCount = 0;
        let totalSuccessCount = 0;
        let latestCompletedRun: Date | undefined;
        const testCompleteRunDurations: number[] = [];
        const testCatastrophicDurations: number[] = [];
        const errorFingerprints: Set<string> = new Set<string>();
        const recipeIds: Set<string> = new Set<string>();

        const metaRefs: MetadataRef[] = [];
        const testRefMeta: [TestNameRef, MetadataRef[]][] = testMeta.get(test.name) ?? testMeta.set(test.name, []).get(test.name)!;
        testRefMeta.push([test, metaRefs]);

        filteredMetadata.forEach(meta => {
            const metaStatus = status.sessions.get(meta);
            if (!metaStatus) return;

            const session = metaStatus.getLastSession();
            if (!session) return;

            if (session.recipeId) recipeIds.add(session.recipeId);

            metaStatus.history.forEach(session => {
                const outcome = session.outcome;
                if ( outcome === TestOutcome.Skipped ) return;

                switch (outcome) {
                    case TestOutcome.Unspecified:
                        testCatastrophicDurations.push(session.duration);
                        break;
                    case TestOutcome.Failure:
                    case TestOutcome.Success:
                        testCompleteRunDurations.push(session.duration);
                        if (!latestCompletedRun || session.start > latestCompletedRun) {
                            latestCompletedRun = session.start;
                        }
                        break;
                    default:
                }

                totalPhaseCount += session.phasesSucceededCount + session.phasesUnspecifiedCount + session.phasesFailedCount;
                totalFailureCount += session.phasesUnspecifiedCount + session.phasesFailedCount;
                totalUndefinedCount += session.phasesUnspecifiedCount;
                totalSuccessCount += session.phasesSucceededCount;
                session.errorFingerprints?.keys().forEach(error => errorFingerprints.add(error));
            })

            metaRefs.push(meta);
        });

        // Compute Cadence
        let cadenceValue = TestCadence.NA;
        if (!!handler.filterState.weeks) {
            const highestRunCount = status.sessions.get(metaRefs.reduce((pmeta, meta) => {
                const pCount = !!pmeta? status.sessions.get(pmeta)?.history.length ?? 0 : 0;
                const cCount = status.sessions.get(meta)?.history.length ?? 0;
                return cCount >= pCount ? meta : pmeta;
            }, undefined)!)?.history.filter((s, i, history) => history.findIndex(h => h.commitId === s.commitId) === i).length ?? 0;

            if (highestRunCount > 0) {
                const rangeDayCount = handler.filterState.weeks * 7;
                const cadenceRate = highestRunCount / rangeDayCount;
                if (cadenceRate > 24) {
                    /** more than 24 runs / 1 day */
                    cadenceValue = TestCadence.Incremental;
                } else if (cadenceRate > 3) {
                    /** more than 3 runs / 1 day */
                    cadenceValue = TestCadence.Hourly;
                } else if (cadenceRate > 0.4) {
                    /** more than 3 runs / 7 days */
                    cadenceValue = TestCadence.Daily;
                } else if (cadenceRate > 0.14) {
                    /** more than 1 run / 7 days */
                    cadenceValue = TestCadence.Weekly;
                } else {
                    cadenceValue = TestCadence.Limited;
                }
            } else {
                cadenceValue = TestCadence.NotRun;
            }
        }

        // Compute Health Heuristic
        let healthValue = TestHealth.NA;
        let healthExplain: TestHealthExplain | undefined;
        if (totalPhaseCount > 0) {
            const catastrophicFailureRate = totalFailureCount === 0 ? 0: percRound(totalUndefinedCount/totalFailureCount);
            const failureRate = percRound(totalFailureCount/totalPhaseCount);
            const successRate = percRound(totalSuccessCount/totalPhaseCount);
            const redundantErrorRate = totalFailureCount === 0 ? 0: percRound(1 - errorFingerprints.size/totalFailureCount);
            let nStars = healthStars.get(TestHealth.Fair)!;
            if (failureRate > 15) {
                if (catastrophicFailureRate > 40 || successRate === 0) {
                    nStars -= 2;
                } else {
                    nStars -= 1;
                }
                if (redundantErrorRate < 80) {
                    nStars += redundantErrorRate < 60 ? 1 : 0;
                } else if (failureRate > 40) {
                    nStars -= 1;
                }
            }
            if (successRate > 60) {
                nStars += 1;
            }
            healthValue = healthStars.entries().find(([_, value]) => value === nStars)?.[0] ?? healthValue;
            healthExplain = {
                catastrophicFailureRate: catastrophicFailureRate,
                failureRate: failureRate,
                successRate: successRate,
                redundantErrorRate: redundantErrorRate
            }
        }

        // Compute timings
        testCatastrophicDurations.sort();
        testCompleteRunDurations.sort();
        const midIndex1 = Math.floor(testCatastrophicDurations.length / 2);
        const midIndex2 = Math.floor(testCompleteRunDurations.length / 2);
        const timings: TestHealthTiming = {
            medianToInterruptSecs: testCatastrophicDurations.length === 0?
                            0 : testCatastrophicDurations.length % 2?
                                testCatastrophicDurations[midIndex1] : (testCatastrophicDurations[midIndex1] + testCatastrophicDurations[midIndex1 - 1]) / 2,
            medianToCompleteRunSecs: testCompleteRunDurations.length === 0?
                            0 : testCompleteRunDurations.length % 2?
                                testCompleteRunDurations[midIndex2] : (testCompleteRunDurations[midIndex2] + testCompleteRunDurations[midIndex2 - 1]) / 2,
            latestCompletedRun: latestCompletedRun,
        }

        const testHealthItem = new TestHealthItem(test, handler, stream);
        testHealthItem.metaRefs = metaRefs;
        testHealthItem.health = healthValue;
        testHealthItem.healthExplain = healthExplain;
        testHealthItem.cadence = cadenceValue;
        testHealthItem.setLazyDisplayMeta(() => status.sessions.keys().find((meta) => !!meta)?.getCommonValues(testMetaKeys?.get(test.name))?.join(" / "));
        testHealthItem.timings = timings;
        testHealthItem.linkRecipeIds(recipeIds);

        testHealths.push(testHealthItem);
    });

    testMetaKeys = getTestUniqueMetaIdentifiers(testMeta);

    return testHealths;
}

const TestHealthTimes: React.FC<{ test: TestHealthItem }> = ({test}) => {
    const [init, setInit] = useState<boolean>(false);

    useEffect(() => {
        test.fetchRecipes().then(() => setInit(true));
    }, [test]);

    if (!test.timings) return null;

    const medianToCompleteRunString = useMemo(() => test.timings? msecToElapsed(test.timings.medianToCompleteRunSecs * 1000, true, true) : "", [test]);
    const medianToInterruptString = useMemo(() => test.timings? msecToElapsed(test.timings.medianToInterruptSecs * 1000, true, true) : "", [test]);
    const timeoutString = useMemo(() => test.recipes? msecToElapsed(test.recipes.map(r => r.timeoutMinutes).reduce((acc, value) => value > acc? value : acc, 0 ) * 60 * 1000, true, true) : "", [test, init]);
    const colors = getStatusColors();

    return <Stack horizontal tokens={{childrenGap: 15}} style={{cursor: 'default'}}>
                <TooltipHost content="Median time to Complete a run" delay={TooltipDelay.zero}>
                    <FontIcon iconName="Completed" style={{fontSize: 14, paddingRight: 5, color: colors.get(StatusColor.Success)}}/>
                    {!!test.timings.medianToCompleteRunSecs && <span style={{color: textEmphasisColor()}}>{medianToCompleteRunString}</span>}
                    {!test.timings.medianToCompleteRunSecs && <span style={{color: textEmphasisColor()}}>n/a</span> }
                </TooltipHost>
                {!!test.timings.medianToInterruptSecs &&
                    <TooltipHost content="Median time to Interrupted run" delay={TooltipDelay.zero}>
                        <FontIcon iconName="ErrorBadge" style={{fontSize: 14, paddingRight: 5, color: colors.get(StatusColor.Unspecified)}} />
                        <span style={{color: textEmphasisColor()}}>{medianToInterruptString}</span>
                    </TooltipHost>
                }
                {!!test.recipes &&
                    <TooltipHost content="Max timeout" delay={TooltipDelay.zero}>
                        <FontIcon iconName="Recent" style={{fontSize: 14, paddingRight: 5, color: colors.get(StatusColor.Waiting)}} />
                        <span  style={{color: textEmphasisColor()}}>{timeoutString}</span>
                    </TooltipHost>
                }
            </Stack> 
}


const usersInfoCache = new UsersInfoCache();

type DeepLinkState = {
    directMessage: {
        isFetching?: boolean;
        url?: string;
    };
    notification: {
        isFetching?: boolean;
        url?: string;
    };
}

const ToolsMenu: React.FC<{items: TestHealthItem[]}> = ({items}) => {

    const options: IContextualMenuProps = {
        items: [
            {
                key: 'download',
                text: 'Download as CSV',
                iconProps: {iconName: 'Download'},
                onClick: () => {
                    downloadCsv(
                        items.map(item => item.getAsArray(usersInfoCache)),
                        'test-health.csv',
                        TestHealthItem.getFields()
                    );
                }
            },
        ],
    }

    return  <IconButton
                iconProps={{iconName: "ColumnOptions"}}
                menuProps={options}
                style={{
                    fontSize:16, height: 28,
                    border: '1px solid', borderColor: dashboard.darktheme ? "#4D4C4B" : "#6D6C6B", borderRadius: 4,
                    backgroundColor: dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "",
                    color: dashboard.darktheme ? 'white' : 'grey'
                }}
            />
}

export const TestHealthView: React.FC<{ test: TestHealthItem, handler: TestDataHandler, onUpdateItem: (item: TestHealthItem) => void }> = observer(({ test, handler, onUpdateItem }) => {
    const [streams, setStreams] = useState<{current: string, toCompare?: string}>(
        {
            current: handler.stream!,
            toCompare: userStorage.getItem(StoreKey.HistoryToCompare, () => undefined)
        });
    const [testHealthToCompare, setTestHealthToCompare] = useState<TestHealthItem | undefined>(getTestHealthItems(handler, userStorage.getItem(StoreKey.HistoryToCompare), test.nameRef).at(0));
    const [userPicker, setUserPicker] = useState<{target?: UserType, defaultUser?: GetUserResponse}>({});
    const [deepLinkState, setDeepLinkState] = useState<DeepLinkState>({directMessage: {}, notification: {}});

    // reset deep link notification state when notification changes
    useEffect(() => {
        setDeepLinkState(prev => ({...prev, notification: {}}));
    }, [test.notification]);

    // fetch users info when test changes
    useEffect(() => {
        let userIds: string[] = [];
        if (test.user) {
            userIds.push(test.user);
        }
        if (test.owner) {
            userIds.push(test.owner);
        }
        if (test.customers) {
            userIds = userIds.concat(test.customers);
        }
        if (userIds.length > 0) {
            usersInfoCache.fetchUsersInfo(userIds).then((fetchedUsers) => fetchedUsers.length > 0 && test.setUpdated());
        }
        // reset deep link direct message state when users info changes
        setDeepLinkState(prev => ({...prev, directMessage: {}}));
    }, [test.user,test.owner, test.customers]);

    handler.subscribeToAuditsLoading();
    handler.subscribeToRecipesLoading();
    test.subscribeUpdate();

    // collecting metadata
    let metadataSet = new Set(handler.getStatusStream(streams.current)!.tests.get(test.nameRef)!.getMetadata());
    if (!!streams.toCompare) {
        const toCompareMetadataList = handler.getStatusStream(streams.toCompare!)?.tests.get(test.nameRef)?.getMetadata();
        if (!!toCompareMetadataList) {
            metadataSet = metadataSet.union(new Set(toCompareMetadataList!));
        }
    }

    const commonMetaKeys = metadataSet.size === 1? handler.filterState.metadata?.keys().toArray() : MetadataRef.identifyCommonKeys(Array.from(metadataSet));
    const availableStreams = handler.availableStreams.filter((s) => s !== streams.current);

    const onClickStream = useCallback((targetStream: string) => {
        setStreams(prev => ({...prev, toCompare: undefined}));
        handler.query(targetStream, true)
            .catch((reason) => console.error(reason))
            .finally(() => {
                setTestHealthToCompare(getTestHealthItems(handler, targetStream, test.nameRef).at(0));
                setStreams(prev => ({...prev, toCompare: targetStream}));
                userStorage.setItem(StoreKey.HistoryToCompare, targetStream);
            });
    }, [test]);

    const onDetailsClick = useCallback((session: TestSessionResult) => {
        if (session.testDataId) {
            handler.setSearchParam('session', session.testDataId);
        }
        else if (session.jobId && session.stepId) {
            window.open(`/job/${session.jobId}?step=${session.stepId}`, '_blank');
        }
    }, []);

    const profileOptions: IComboBoxOption[] = profileColors.getDomain().map(p => ({key: p.toLowerCase(), text: p, styles: {optionText: {backgroundColor: profileColors.get(p), padding: 3, borderRadius: 4, minWidth: 72}}}));
    const teamOptions: IComboBoxOption[] = teamColors.getDomain().map(p => ({key: p.toLowerCase(), text: p, styles: {optionText: {backgroundColor: teamColors.get(p), padding: 3, borderRadius: 4, minWidth: 72}}}));
    const harnessOptions: IComboBoxOption[] = harnessColors.getDomain().map(p => ({key: p.toLowerCase(), text: p}));
    const notificationOptions: IComboBoxOption[] | undefined = projectStore.streamById(test.stream)?.workflows?.filter(w => !!w.reportChannel).map(w => ({key: w.id, text: w.id, data: w.id}));
    if (!!notificationOptions) {
        notificationOptions.unshift({key: 'disable', text: '- disable notification -', data: undefined});
    }
    const notificationReportChannel = projectStore.streamById(test.stream)?.workflows?.find(w => w.id === test.notification)?.reportChannel;
    const getNotificationReportLinkAsync = useCallback(async () => {
        if (!!notificationReportChannel) {
            if (!!deepLinkState.notification.url) {
                return deepLinkState.notification.url;
            }
            setDeepLinkState(prev => ({...prev, notification: {isFetching: true}}));
            let response: DeepLinkResponse | undefined;
            try {
                response = await backend.getDeepLinkChannelLink(notificationReportChannel);
            } catch (error) { /* ignore */ } finally {
                setDeepLinkState(prev => ({...prev, notification: {isFetching: false, url: response?.url}}));
                if (!!response?.url) {
                    return response.url;
                } else {
                    errorDialogStore.set({
                        title: `Error Getting Report Channel Link ${notificationReportChannel}`,
                        message: "Failed to get a deep link to the report channel."
                    }, true);
                }
            }
        }
    }, [notificationReportChannel, deepLinkState.notification.url]);
    const getDirectMessageLinkAsync = useCallback(async () => {
        if (!!test.owner || !!test.customers) {
            if (!!deepLinkState.directMessage.url) {
                return deepLinkState.directMessage.url;
            }
            const userIds = [test.owner, ...(test.customers ?? [])].filter(id => !!id) as string[];
            setDeepLinkState(prev => ({...prev, directMessage: {isFetching: true}}));
            let response: DeepLinkResponse | undefined;
            try {
                response = await backend.getDeepLinkDirectMessageLink(userIds);
            } catch (error) { /* ignore */ } finally {
                setDeepLinkState(prev => ({...prev, directMessage: {isFetching: false, url: response?.url}}));
                if (!!response?.url) {
                    return response.url;
                } else {
                    errorDialogStore.set({
                        title: "Error Getting Direct Message Link",
                        message: "Failed to get a deep link to the direct message channel with the users."
                    }, true);
                }
            }
        }
    }, [test.owner, test.customers, deepLinkState.directMessage.url]);

    const tags = useMemo(() => test.nameRef?.tagRefs?.keys().map(t => t.name).toArray().sort(), [test]);

    return <Stack>
                {userPicker.target &&
                    <UserPickerModal
                        defaultUser={userPicker.defaultUser}
                        onSelect={(userId?: string) => {
                            switch(userPicker.target) {
                                case UserType.Owner :
                                    test.owner = userId;
                                    break;
                                case UserType.Customers :
                                    if (userId) {
                                        const customers = new Set(test.customers ?? []);
                                        customers.add(userId);
                                        test.customers = customers.keys().toArray();
                                    }
                                    break;
                            }
                        }}
                        onClose={() => setUserPicker({})}
                    />
                }
                <Stack tokens={{childrenGap: 16}} style={{position: 'relative'}}>
                    <Stack tokens={{childrenGap: 16}} style={{padding: 14}}>
                        <Stack horizontal tokens={{childrenGap: 12}}>
                            <Stack tokens={{childrenGap: 12}} grow style={{width: '100%'}}>

                                <Stack horizontal tokens={{childrenGap: 12}} style={{paddingLeft: 8, backgroundColor: booleanColors.get(false)}}>
                                    <EditableTextField
                                        key={`intent-${test.updated}`}
                                        label="Intent"
                                        text={test.intent}
                                        onChange={(value) => test.intent = value}
                                        placeholder="a short description"
                                        loaded={handler.queryAuditsLoading}
                                    />
                                </Stack>

                                <Stack horizontal tokens={{childrenGap: 8}}>
                                    <ComboBox
                                        key={`profile-${test.updated}`}
                                        styles={{input: {backgroundColor: profileColors.get(test.profile)}, root: {backgroundColor: profileColors.get(test.profile), width: 170, height: 24}}}
                                        placeholder="Profile"
                                        allowFreeform
                                        autoComplete="off"
                                        options={profileOptions}
                                        text={test.profile}
                                        onChange={(_1, _2, _3, value) => { const lowerValue = value?.toLowerCase(); test.profile = profileOptions.find(o => o.key === lowerValue)?.text ?? value; onUpdateItem(test); }}
                                    />
                                    <ComboBox
                                        key={`team-${test.updated}`}
                                        styles={{input: {backgroundColor: teamColors.get(test.team)}, root: {backgroundColor: teamColors.get(test.team), width: 170, height: 24}}}
                                        placeholder="Team"
                                        allowFreeform
                                        autoComplete="off"
                                        options={teamOptions}
                                        text={test.team}
                                        onChange={(_1, _2, _3, value) => { const lowerValue = value?.toLowerCase(); test.team = teamOptions.find(o => o.key === lowerValue)?.text ?? value; onUpdateItem(test); }}
                                    />
                                    <ComboBox
                                        key={`harness-${test.updated}`}
                                        styles={{root: {width: 170, height: 24}}}
                                        placeholder="Harness"
                                        allowFreeform
                                        autoComplete="off"
                                        options={harnessOptions}
                                        text={test.harness}
                                        onChange={(_1, _2, _3, value) => { const lowerValue = value?.toLowerCase(); test.harness = harnessOptions.find(o => o.key === lowerValue)?.text ?? value; }}
                                    />
                                </Stack>

                                {tags &&
                                    <Stack horizontal verticalAlign="start" wrap grow style={{paddingLeft: 8, maxHeight: 160, overflowY: 'auto', overflowX: 'hidden'}}>
                                        {tags.map(tag =>
                                                <Stack key={`tag-${tag}`} style={{backgroundColor: tagColors.getTagColor(tag), borderRadius: 5, padding: '1px 4px', margin: '0px 2px 2px 0px', opacity: 0.8}}>
                                                    <Text>#{tag}</Text>
                                                </Stack>
                                            )
                                        }
                                    </Stack>
                                }

                            </Stack>
                            <Stack tokens={{childrenGap: 8}} style={{width: '100%'}}>

                                <Stack horizontal verticalAlign="start" tokens={{childrenGap: 6}} style={{cursor: 'default'}}>
                                    <Label style={{whiteSpace: 'nowrap'}}>Degradation Notification: </Label>
                                    {!notificationOptions && <Text style={{color:textEmphasisColor(), cursor: 'default'}}>no workflow available in this stream</Text>}
                                    {!!notificationOptions &&
                                        <Stack horizontalAlign="end" horizontal>
                                            <ComboBox
                                                key={`notification-${test.updated}`}
                                                styles={{root: {width: 150, height: 24}}}
                                                placeholder="none"
                                                autoComplete="on"
                                                options={notificationOptions}
                                                text={test.notification}
                                                onChange={(_1, option) => { test.notification = option?.data as string | undefined; onUpdateItem(test); }}
                                                title={test.notification}
                                            />
                                            {!!notificationReportChannel &&
                                                <Link style={{padding: '0px 5px'}} title="open target channel" onClick={() => { getNotificationReportLinkAsync().then(link => link && window.open(link, '_blank')); }} disabled={deepLinkState.notification.isFetching}>
                                                    {!deepLinkState.notification.isFetching && <FontIcon iconName="MessageFill" style={{fontSize: 17, paddingRight: 4}} />}
                                                    {!!deepLinkState.notification.isFetching && <Spinner style={{width: 24}} size={SpinnerSize.small} />}
                                                </Link>
                                            }
                                            {!notificationReportChannel &&
                                                <Stack style={{padding: '0px 5px'}}><FontIcon iconName="MuteChat" style={{fontSize: 17, paddingRight: 4}} /></Stack>
                                            }
                                        </Stack>
                                    }
                                </Stack>

                                <Stack horizontal verticalAlign="start" tokens={{childrenGap: 12}} grow>
                                    <Stack tokens={{childrenGap: 6}} style={{borderRadius: 4}} verticalAlign="start">
                                        <Label>Owner: </Label>
                                        <Stack horizontal tokens={{childrenGap: 6}} wrap>
                                            {!!test.owner &&
                                                <Stack style={healthStyles.userTag} horizontal verticalAlign="center">
                                                    <Text style={{cursor: 'pointer', color:textEmphasisColor(), whiteSpace: 'nowrap'}} title="change owner"
                                                        onClick={() => setUserPicker({target: UserType.Owner, defaultUser: usersInfoCache.getInfo(test.owner)})}
                                                    >{usersInfoCache.getInfo(test.owner)?.name ?? <Spinner size={SpinnerSize.xSmall} />}</Text>
                                                    <FontIcon iconName="Cancel" style={{cursor: 'pointer', fontSize: 11, paddingLeft: 6}} title="remove" onClick={() => {test.owner = undefined}}/>
                                                </Stack>
                                            }
                                            {!test.owner &&
                                                <Stack style={{...healthStyles.userTag, width: 46}} horizontal verticalAlign="center" horizontalAlign="center">
                                                    <FontIcon iconName="AddTo" style={{cursor: 'pointer', fontSize: 15, padding: 2}} title="add an owner"
                                                        onClick={() => setUserPicker({target: UserType.Owner})}
                                                    />
                                                </Stack>
                                            }
                                        </Stack>
                                    </Stack>
                                    <Stack tokens={{childrenGap: 6}} style={{borderRadius: 4}} verticalAlign="start" grow>
                                        <Label>Customers: </Label>
                                        <Stack horizontal tokens={{childrenGap: 6}} wrap>
                                            {!!test.customers &&
                                                test.customers.map(user => {
                                                    return <Stack key={`c-${user}`} style={healthStyles.userTag} verticalAlign="center" horizontal>
                                                                <Text style={{color:textEmphasisColor(), whiteSpace: 'nowrap'}}>{usersInfoCache.getInfo(user)?.name ?? <Spinner size={SpinnerSize.xSmall} />}</Text>
                                                                <FontIcon iconName="Cancel" style={{cursor: 'pointer', fontSize: 11, paddingLeft: 6}} title="remove" onClick={() => {test.customers = test.customers?.filter(c => c !== user)}}/>
                                                            </Stack>
                                                })
                                            }
                                            <Stack style={{...healthStyles.userTag, width: 46}} verticalAlign="center" horizontalAlign="center">
                                                <FontIcon iconName="AddTo" style={{cursor: 'pointer', fontSize: 15, padding: 2}} title="add a customer"
                                                    onClick={() => setUserPicker({target: UserType.Customers})}
                                                />
                                            </Stack>
                                        </Stack>
                                    </Stack>
                                    {(!!test.owner || !!test.customers) &&
                                        <Link onClick={() => { getDirectMessageLinkAsync().then(link => link && window.open(link, '_blank')); }} title="start chat with owner and customers" disabled={deepLinkState.directMessage.isFetching}>
                                            {!deepLinkState.directMessage.isFetching && <FontIcon iconName="OfficeChatSolid" style={{cursor: 'pointer', fontSize: 17, padding: 2}}/>}
                                            {!!deepLinkState.directMessage.isFetching && <Spinner style={{width: 24}} size={SpinnerSize.small} />}
                                        </Link>
                                    }
                                </Stack>

                            </Stack>
                        </Stack>

                        <Stack tokens={{childrenGap: 4}}>
                            <Stack style={{padding: '4px 12px', backgroundColor: booleanColors.get(test.underAudit)}}>
                                <Stack horizontal tokens={{childrenGap: 6}} verticalAlign="center">
                                    <Toggle
                                        key={`under-audit-${test.updated}`}
                                        label="Under Audit"
                                        checked={test.underAudit}
                                        onChange={(_, checked) => {test.underAudit = !!checked}}
                                        inlineLabel
                                        onText="Yes" offText="No"
                                    />
                                    <Stack horizontal horizontalAlign="start" tokens={{childrenGap: 6}}>
                                        <Label>Last date: </Label>
                                        <DatePicker
                                            key={`last-audit-${test.updated}`}
                                            styles={{root: { width: 150 }}}
                                            placeholder="Select a date... "
                                            value={test.lastAuditDate}
                                            onSelectDate={(date) => test.lastAuditDate = (date ?? undefined)}
                                        />
                                    </Stack>
                                </Stack>
                                <EditableTextField
                                    key={`notes-${test.updated}`}
                                    label="Notes"
                                    text={test.notes}
                                    onChange={(value) => test.notes = value}
                                    multiline
                                    placeholder="add notes"
                                    loaded={handler.queryAuditsLoading}
                                />
                            </Stack>
                            
                            <Stack horizontal horizontalAlign="end" verticalAlign="center" tokens={{childrenGap: 6}} style={{cursor: 'default'}} grow>
                                {!!test.user &&
                                    <Stack horizontal verticalAlign="center" tokens={{childrenGap: 6}} >
                                        <Label style={{fontSize: 12}}>Last update by: </Label>
                                        <TooltipHost content={`on ${getShortNiceTime(test.userInputDate)}`} delay={TooltipDelay.zero}>
                                            <Stack style={{...healthStyles.userTag, padding: '1px 7px'}} verticalAlign="center" horizontal>
                                                <Text variant="small" style={{color:textEmphasisColor(), whiteSpace: 'nowrap'}}>{usersInfoCache.getInfo(test.user)?.name ?? <Spinner size={SpinnerSize.xSmall} />}</Text>
                                            </Stack>
                                        </TooltipHost>
                                    </Stack>
                                }
                                {!test.user && <Text style={{color: textEmphasisColor()}} variant="small">no user update</Text>}
                                <Link href={`/test-automation/log/${test.nameRef.id}`} target="_blank" title="open audit history">
                                    <FontIcon iconName="TimeEntry" style={{cursor: 'pointer', fontSize: 17, padding: 2}}/>
                                </Link>
                            </Stack>
                        </Stack>

                    </Stack>
                    <Stack style={{paddingLeft: 16}} tokens={{childrenGap: 16}}>
                        <Stack style={{paddingLeft: 12}} tokens={{childrenGap: 16}}>
                            <Stack horizontal tokens={{childrenGap: 12}} verticalAlign="center">
                                <Stack style={styles.streamBadge} horizontalAlign="center">
                                    <Text variant="small">{projectStore.streamById(streams.current)?.fullname ?? streams.current}</Text>
                                </Stack>
                                <Stack style={{cursor: 'default'}}>{renderHealthBadge(test, 75)}</Stack>
                                <Stack style={{...healthStyles.badge, backgroundColor: cadenceColors.get(test.cadence)}} verticalAlign="center">
                                    <Text title="Cadence">{test.cadence}</Text>
                                </Stack>
                                {test.timings && <TestHealthTimes test={test} />}
                            </Stack>
                            <Stack horizontal tokens={{childrenGap: 16}}>
                                <Stack style={{backgroundColor: booleanColors.get(false), width: 6, borderRadius: 3}}></Stack>
                                <TestHistoryGraphWidget test={test.nameRef} stream={streams.current} commonMetaKeys={commonMetaKeys} handler={handler} onClick={onDetailsClick} />
                            </Stack>
                        </Stack>
                        <Stack style={{paddingLeft: 12}} tokens={{childrenGap: 16}}>
                            <Stack horizontal tokens={{childrenGap: 12}} verticalAlign="center">
                                <Stack style={{width: 200}}>
                                    <StreamSelector streams={availableStreams} selected={streams.toCompare} disabled={availableStreams.length === 0} onClick={onClickStream} />
                                </Stack>
                                {!!testHealthToCompare && <Stack style={{cursor: 'default'}}>{renderHealthBadge(testHealthToCompare, 75)}</Stack>}
                                {!!testHealthToCompare && 
                                    <Stack style={{...healthStyles.badge, backgroundColor: cadenceColors.get(testHealthToCompare.cadence)}} verticalAlign="center">
                                        <Text title="Cadence">{testHealthToCompare.cadence}</Text>
                                    </Stack>
                                }
                                {!!testHealthToCompare && testHealthToCompare.timings && <TestHealthTimes test={testHealthToCompare} />}
                            </Stack>
                            <Stack horizontal tokens={{childrenGap: 16}}>
                                <Stack style={{backgroundColor: booleanColors.get(false), width: 6, borderRadius: 3}}></Stack>
                                {!streams.toCompare && !handler.subQueryLoading &&
                                    <Stack style={{minHeight: 40}} verticalAlign="center"><Text>select a stream</Text></Stack>
                                }
                                {!!streams.toCompare && !!testHealthToCompare && 
                                    <TestHistoryGraphWidget test={test.nameRef} stream={streams.toCompare} commonMetaKeys={commonMetaKeys} handler={handler} onClick={onDetailsClick} />
                                }
                                {!!streams.toCompare && !testHealthToCompare && 
                                    <Stack horizontalAlign='center' style={{ padding: 4 }}>
                                        <Text style={{ fontSize: 12 }}>No Results</Text>
                                    </Stack>
                                }
                                {handler.subQueryLoading &&
                                    <Stack horizontalAlign='center' style={{ padding: 12}} tokens={{ childrenGap: 10 }} horizontal>
                                        <Text style={{ fontSize: 12 }}>Loading Data</Text>
                                        <Spinner size={SpinnerSize.small} />
                                    </Stack>
                                }
                            </Stack>
                        </Stack>
                    </Stack>

                    { (handler.queryAuditsLoading || handler.queryRecipesLoading) &&
                        <Stack horizontalAlign='center' style={{ width: "100%", position: "absolute", top: 0 }} tokens={{childrenGap: 8}}>
                            <Spinner size={SpinnerSize.large} />
                        </Stack>
                    }
                </Stack>
        </Stack>
});

const TestHealthDetailsModal: React.FC<{ test: TestHealthItem, handler: TestDataHandler, onDismiss: () => void, onUpdateItem: (item: TestHealthItem) => void }> = observer(({ test, handler, onDismiss, onUpdateItem }) => {
    const [isCalloutVisible, setCalloutVisible] = useState(false);
    const [isCommitting, setCommitting] = useState(false);

    useEffect(() => {
        if (test.loadStashedTransaction()) {
            onUpdateItem(test);
        }
        // stash changes on clean up
        return () => { if(test.isTransient) { test.stashTransaction(); test.discardTransaction(); onUpdateItem(test); } }
    }, [test]);

    test.subscribeUpdate();

    let title = test.name;
    if (test.displayMeta) title += ` - ${test.displayMeta}`;

    return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1134, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onDismiss()} >
            <Stack grow>
                <Stack horizontal verticalAlign="center">
                    <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>Health Details - {title}</Text></Stack>
                    <Stack grow />
                    {test.isTransient &&
                        <Stack style={{ paddingRight: 6 }} title="Commit changes">
                            {!isCommitting &&
                                <IconButton
                                    iconProps={{ iconName: 'Save', style: { fontSize: 16 } }}
                                    ariaLabel="Commit changes"
                                    onClick={() => { setCommitting(true); test.commitTransaction().then(() => { setCommitting(false) }); }}
                                />
                            }
                            {isCommitting &&
                                <Spinner style={{width: 32}} size={SpinnerSize.small} />
                            }
                        </Stack>
                    }
                    {!isCommitting && test.isTransient &&
                        <Stack style={{ paddingRight: 6 }} title="Discard changes">
                            <IconButton
                                iconProps={{ iconName: 'History', style: { fontSize: 14 } }}
                                ariaLabel="Revert changes"
                                onClick={() => { test.discardTransaction(); onUpdateItem(test); }}
                            />
                        </Stack>
                    }
                    <Stack className="Close">
                        <IconButton
                            iconProps={{ iconName: 'Cancel' }}
                            ariaLabel="Close popup modal"
                            onClick={() => { if(test.isTransient) { setCalloutVisible(true) } else { onDismiss(); } }}
                        />
                        {isCalloutVisible &&
                            <FocusTrapCallout
                                target=".Close"
                                style={{padding: '20px 24px', width: 300}}
                                isBeakVisible
                                onDismiss={() => setCalloutVisible(false)}
                            >
                                {!isCommitting &&
                                    <Stack tokens={{childrenGap: 6}}>
                                        <Text block>You have uncommitted changes to this audit.</Text>
                                        <Text block>Do you want to commit them?</Text>
                                        <FocusZone handleTabKey={FocusZoneTabbableElements.all} isCircularNavigation>
                                            <Stack horizontal horizontalAlign="center" verticalAlign="center">
                                                <PrimaryButton 
                                                    iconProps={{ iconName: 'Save', style: { fontSize: 16 } }}
                                                    onClick={() => { setCommitting(true); test.commitTransaction().then(() => { setCommitting(false); setCalloutVisible(false); onDismiss(); }) }}
                                                >Commit</PrimaryButton>
                                                <DefaultButton
                                                    iconProps={{ iconName: 'History', style: { fontSize: 14 } }}
                                                    onClick={() => { test.discardTransaction(); onUpdateItem(test); setCalloutVisible(false); onDismiss(); }}
                                                >Discard</DefaultButton>
                                                <DefaultButton
                                                    onClick={() => { setCalloutVisible(false); }}
                                                >Cancel</DefaultButton>
                                            </Stack>
                                        </FocusZone>
                                    </Stack>
                                }
                                {isCommitting &&
                                    <Stack horizontal>
                                        <Text block>Committing changes...</Text>
                                        <Spinner style={{width: 32}} size={SpinnerSize.small} />
                                    </Stack>
                                }
                            </FocusTrapCallout>
                        }
                    </Stack>
                </Stack>
                <Stack tokens={{ childrenGap: 40 }} style={{ padding: 8, overflow: "auto", maxHeight: 'calc(100vh - 200px)'} }>
                    <TestHealthView test={test} handler={handler} onUpdateItem={onUpdateItem}/>
                </Stack>
            </Stack>
        </Modal>
});

const getTestHealthItemKey = (item: TestHealthItem) => `test-health-${item.nameRef.id}`;

const onRenderDetailsHeader: IRenderFunction<IDetailsHeaderProps> = (props, defaultRender) => {
    if (!defaultRender) {
        return null;
    }
    return (
        <Sticky stickyPosition={StickyPositionType.Header}>
            {defaultRender(props)}
        </Sticky>
    );
};

export const TestAutomationHealth: React.FC<{ handler: TestDataHandler}> = observer(({ handler }) => {
    const [healthDetailsState, setHealthDetailsState] = useState<{test?: TestHealthItem}>({});
    const [healthListState, setHealthListState] = useState<TestHealthListState>({items: [], columns: []});

    const updateColorDomains = useCallback((items: TestHealthItem[]) => {
        const profiles = new Set(items.map(i => i.profile).filter(i => i) as string[]);
        const teams = new Set(items.map(i => i.team).filter(i => i) as string[]);
        const harnesses = new Set(items.map(i => i.harness).filter(i => i) as string[]);
        profileColors.setDomain(Array.from(profiles).sort());
        teamColors.setDomain(Array.from(teams).sort());
        harnessColors.setDomain(Array.from(harnesses).sort())
    }, []);

    const initHealthListData = useCallback((): [TestHealthItem[], TestHealthColumn[]] => {
        const initItems = getTestHealthItems(handler, handler.stream);
        const initColumns = getHealthColumns(initItems, healthListState.columns, (item) => {
            setHealthDetailsState({test: item});
            handler.setSearchParam('health', item.nameRef.id);
        }, setHealthListState);

        updateColorDomains(initItems);

        setHealthListState({
            items: initItems,
            columns: initColumns
        });

        return [initItems, initColumns];
    }, [handler.searchUpdated]);

    const forceListRefresh = () => {
        updateColorDomains(healthListState.items);
        setHealthListState(prev => ({ ...prev, items: healthListState.items.slice() }));
    }

    // initiate data on filtering change
    useEffect(() => {

        const [items, columns] = initHealthListData();

        // load audit async
        handler.queryAudits(items.map(i => i.nameRef))
                    .then(() => usersInfoCache.fetchUsersInfo(items.map(i => i.owner).filter(i => !!i) as string[]))
                    .then((fetchedUsers) => fetchedUsers.length > 0 && setHealthListState({items: items.slice(), columns: columns}));

    }, [handler.searchUpdated]);

    // find health item if not loaded already
    const testId = handler.getSearchParam('health') as string | undefined;
    useEffect(() => {
        if ((!!testId && !healthDetailsState.test)
            || (!!healthDetailsState.test && healthDetailsState.test.nameRef.id !== testId)) {

            // if no data were generated for the stream, do it now
            if (healthListState.items.length === 0) initHealthListData();

            const testHealth = !!testId ? healthListState.items.find(i => i.nameRef.id === testId) : undefined;

            setHealthDetailsState({test: testHealth});
        }
    }, [handler.searchUpdated, testId, healthListState.items]);

    return  <Stack grow style={{position: "relative"}}>
                {!!healthDetailsState.test &&
                    <TestHealthDetailsModal
                        test={healthDetailsState.test}
                        handler={handler}
                        onDismiss={() => {setHealthDetailsState({}); handler.removeSearchParam('health')}}
                        onUpdateItem={(item: TestHealthItem) => { forceListRefresh(); }}
                    />
                }
                { !handler.queryAuditsLoading &&
                    <ScrollablePane>
                        <DetailsList
                            items={healthListState.items}
                            getKey={getTestHealthItemKey}
                            compact={true}
                            columns={healthListState.columns}
                            isHeaderVisible={true}
                            onRenderDetailsHeader={onRenderDetailsHeader}
                            selectionMode={SelectionMode.none}
                            layoutMode={DetailsListLayoutMode.justified}
                        />
                    </ScrollablePane>
                }
                { !handler.queryAuditsLoading && 
                    <Stack horizontalAlign='end' style={{ position: "absolute", right: 24, top: 8, zIndex: 1000 }} tokens={{childrenGap: 8}}>
                        <ToolsMenu items={healthListState.items} />
                    </Stack>
                }
                { handler.queryAuditsLoading &&
                    <Stack horizontalAlign='center' style={{ width: "100%", position: "absolute", top: 24 }} tokens={{childrenGap: 8}}>
                        <Text style={{ fontSize: 24 }}>Loading Data</Text>
                        <Spinner size={SpinnerSize.large} />
                    </Stack>
                }
            </Stack>
});
