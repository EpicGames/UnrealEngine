// Copyright Epic Games, Inc. All Rights Reserved.

import { getHordeTheme } from "horde/styles/theme";
import { getJobParams, PhaseComparisonContext } from "./phaseCompare";
import { MetadataRef, PhaseSessionResult, TestDataHandler, TestNameRef, TestPhaseStatus, TestSessionStatus } from "./testData";
import { observer } from "mobx-react-lite";
import { memo, useCallback, useEffect, useMemo } from "react";
import { ComboBox, DefaultButton, FontIcon, IComboBoxOption, IContextualMenuProps, Link, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { getPhaseSessionStatusColor, getStreamOptions, PhaseHistoryGraphWidget } from "./testAutomationCommon";
import { projectStore } from "horde/backend/ProjectStore";
import { ChangeButton } from "horde/components/ChangeButton";
import { PhaseEventsPane } from "./testAutomationEvents";

export const comparisonContext = new PhaseComparisonContext();

const StreamSelector: React.FC<{ streams: string[], onClick: (stream: string) => void, selected?: string, disabled?: boolean }> = memo(({ streams, onClick, selected, disabled }) => {
    const menuProps: IContextualMenuProps = getStreamOptions(streams, onClick, selected);
    return <DefaultButton
                style={{ height: 22, fontSize: 11, whiteSpace: 'nowrap', borderWidth: 0 }}
                menuProps={menuProps}
                text={selected? projectStore.streamById(selected)?.fullname ?? selected : "Compare stream"}
                disabled={disabled}
            />
});

const onClickItem = (session: PhaseSessionResult) => {
    comparisonContext.setTestSesssionId(session.sessionId);
    comparisonContext.setPhaseStatus(undefined);
}

export const PhaseComparisonPane: React.FC<{ test: TestNameRef, meta?: MetadataRef, phaseKey: string, streamId?: string, focusedSessionId?: string, handler: TestDataHandler, onDismiss: () => void }> = observer(({test, meta, phaseKey, streamId, focusedSessionId, handler, onDismiss}) => {
    const hordeTheme = getHordeTheme();
    const isFromGridView = !!handler.getSearchParam('grid');

    comparisonContext.subscribe();
    handler.subscribeToPhaseQueryLoading();
    handler.subscribeToSubPhaseQueryLoading();
    handler.subscribeToSubQueryLoading();
    handler.subscribeToQueryLoading();

    comparisonContext.setStreamId(comparisonContext.streamId ?? streamId);
    const streamStatus = useMemo(
        () => !!comparisonContext.streamId? handler.getStatusStream(comparisonContext.streamId) : undefined
        , [comparisonContext.streamId, handler.subQueryLoading, handler.queryLoading]);
    const phaseSessionMetaStatus = useMemo(
        () => new Map(streamStatus?.tests.get(test)?.sessions.entries().filter(([_, status]) => status.phaseHistories.get(phaseKey)?.length ?? 0 > 0))
        , [test, phaseKey, handler.phaseQueryLoading, handler.subPhaseQueryLoading, streamStatus]);
    const commonMetaKeys =  useMemo(
        () => phaseSessionMetaStatus.size === 1 ? undefined : MetadataRef.identifyCommonKeys(phaseSessionMetaStatus.keys().toArray())
        , [phaseSessionMetaStatus]);
    const metas = useMemo(
        () => new Map(
            phaseSessionMetaStatus.entries().map<[string, {ref: MetadataRef, session: PhaseSessionResult}]>(
                ([meta, status]) => [meta.getValues().join('/').toLowerCase(), {ref: meta, session: status.phaseHistories.get(phaseKey)![0]}]
            )
        )
        , [phaseSessionMetaStatus, phaseKey]);
    const options: IComboBoxOption[] = useMemo(
        () => metas.keys().toArray().sort(
            (a, b) => !meta? a.localeCompare(b): (metas.get(b)!.ref.affinityScale(meta) - metas.get(a)!.ref.affinityScale(meta)) || a.localeCompare(b)
        ).map(key => ({key: key, text: commonMetaKeys? metas.get(key)!.ref.getValuesExcept(commonMetaKeys).join(' / ') : metas.get(key)!.ref.getValues().slice(1).join(' / '), data: metas.get(key)}))
        , [metas, meta, commonMetaKeys]);

    const onRenderOption = useCallback((item: IComboBoxOption) => {
        if (!item) return <div/>;
        const color = getPhaseSessionStatusColor(item.data.session);
        return <Stack style={{ paddingLeft: 4 }}>
                    <Stack horizontal tokens={{ childrenGap: 8 }}>
                        <FontIcon style={{ fontSize: 12, color: color }} iconName="Square" />
                        <Text style={{ fontSize: 11, fontStyle: item.data!.ref === meta ? 'italic' : 'normal' }}>{item.text}</Text>
                    </Stack>
                </Stack>
    }, [meta]);

    const onClickSwap = useCallback(() => {
        const compareTestDataId = comparisonContext.phaseStatus?.testSession?.testDataId;
        if (!!compareTestDataId && !!streamId && !!meta && !!focusedSessionId) {
            comparisonContext.setMetaKey(meta.getValues().join('/').toLowerCase());
            const testSessionId = handler.getStatusStream(streamId)?.tests.get(test)?.getPhaseSessions(phaseKey, meta)?.find(p => p.id === focusedSessionId)?.sessionId;
            comparisonContext.setTestSesssionId(testSessionId);
            comparisonContext.setPhaseStatus(undefined);
            comparisonContext.setStreamId(streamId);
            handler.setSearchParam('session', compareTestDataId);
        }
    }, [phaseKey, test, streamId, meta, focusedSessionId])

    const autoSelectSession = useCallback(() => {
        const selectedMeta = options.find(o => o.key === comparisonContext.metaKey)!.data!.ref as MetadataRef;
        const selectedMetaHistory = handler.getStatusStream(comparisonContext.streamId!)?.tests.get(test)?.sessions.get(selectedMeta)?.getPhaseSessions(phaseKey);
        if (!!selectedMetaHistory && selectedMetaHistory.length > 0) {
            // select matching commit Id
            if (!!comparisonContext.commitId) {
                const commitIndex = selectedMetaHistory.findIndex(s => s.commitId === comparisonContext.commitId);
                if (commitIndex !== -1) {
                    comparisonContext.setTestSesssionId(selectedMetaHistory[commitIndex].sessionId);
                    comparisonContext.setPhaseStatus(undefined);
                    return;
                }
            }
            // select latest
            comparisonContext.setTestSesssionId(selectedMetaHistory[0].sessionId)
            return;
        }
        comparisonContext.setTestSesssionId(undefined);
    }, [test, phaseKey, options]);

    useEffect(() => {
        if (comparisonContext.isLoadingPhase) return;
        // handle loading target stream
        if (!options.length && comparisonContext.streamId && streamId !== comparisonContext.streamId) {
            comparisonContext.setLoadingPhase(true);
            handler.query(comparisonContext.streamId, true)
                .then(() => {
                    // Try to find a matching test session result to load
                    const testMetaStatus = handler.getStatusStream(comparisonContext.streamId!)?.tests.get(test);
                    let sessionStatus: TestSessionStatus | undefined = !!meta && testMetaStatus?.sessions.has(meta)? testMetaStatus?.sessions.get(meta) : undefined;
                    if (!sessionStatus) {
                        const searchedMetaKey = comparisonContext.metaKey ?? meta?.getValues().join('/').toLowerCase();
                        sessionStatus = testMetaStatus?.sessions.entries().find(
                            ([m, ]) => !searchedMetaKey || m.getValues().join('/').toLowerCase() === searchedMetaKey
                        )?.[1];
                        if (!sessionStatus && !!searchedMetaKey && !!testMetaStatus && testMetaStatus.sessions.size > 0) {
                            // if we still can't find a matching meta then take the first one
                            sessionStatus = testMetaStatus.sessions.values().find(() => true);
                        }
                    }
                    const testSession = sessionStatus?.getLastSession();
                    if (!testSession || !testSession.testDataId) {
                        // no session available, then nothing to load
                        comparisonContext.setLoadingPhase(false);
                        comparisonContext.setUpdated();
                        return;
                    }
                    handler.queryTestData(testSession.testDataId).then(
                        test => {
                            if (!!test) {
                                const phase = test.phases.find(p => p.key === phaseKey);
                                if (!!phase) {
                                    handler.queryPhaseSessions([phase], comparisonContext.streamId!, true)
                                        .then(() => {
                                            // force auto selection once the stream is loaded
                                            comparisonContext.setLoadingPhase(false);
                                            comparisonContext.setUpdated();
                                        });
                                    return;
                                }
                                // TODO: load another test session from another meta to try to find a matching phase key
                            }
                            // no session available, then nothing to load
                            comparisonContext.setLoadingPhase(false);
                            comparisonContext.setUpdated();
                        });
            });
        }
    }, [comparisonContext.streamId, options]);

    useEffect(() => {
        if (comparisonContext.isLoadingPhase || !!comparisonContext.phaseStatus) return;
        // handle loading target phase
        if (comparisonContext.testSessionId && comparisonContext.metaKey) {
            const meta = options.find(o => o.key === comparisonContext.metaKey)?.data.ref;
            const metaStatus = streamStatus?.tests.get(test)?.sessions.get(meta);
            const testSession = metaStatus?.history.find(s => s.id === comparisonContext.testSessionId);
            if (!!testSession?.testDataId) {
                comparisonContext.setLoadingPhase(true);
                handler.queryTestData(testSession.testDataId).then(
                    test => {
                        if (!!test) {
                            comparisonContext.jobParams = getJobParams(test, handler);
                            comparisonContext.commitId = test.commitId;
                            const phase = test.phases.find(p => p.key === phaseKey);
                            if (!!phase) {
                                handler.queryPhaseSessions([phase], comparisonContext.streamId!, true)
                                    .then(() => comparisonContext.setPhaseStatus(phase));
                                return;
                            }
                        }
                        // no session or phase available, then nothing to load
                        comparisonContext.setLoadingPhase(false);
                        comparisonContext.setUpdated();
                    });
            }
        }
    }, [comparisonContext.updated, options]);

    useEffect(() => {
        if (!options.length || comparisonContext.isLoadingPhase) {
            return;
        }
        // auto selection
        if (!!comparisonContext.metaKey) {
            if (!options.find(o => o.key === comparisonContext.metaKey)) {
                // invalid meta key
                const stream = comparisonContext.streamId;
                comparisonContext.reset();
                comparisonContext.setEnable();
                comparisonContext.setStreamId(stream);
            }
        }
        if (!!meta && !comparisonContext.metaKey) {
            const selectedMetaKey = meta.getValues().join('/').toLowerCase();
            if (metas.has(selectedMetaKey)) {
                comparisonContext.setMetaKey(selectedMetaKey);
            }
        }
        if (!!comparisonContext.streamId && !!comparisonContext.metaKey) {
            const selectedMeta = options.find(o => o.key === comparisonContext.metaKey)!.data!.ref as MetadataRef;
            const selectedMetaHistory = streamStatus?.tests.get(test)?.sessions.get(selectedMeta)?.getPhaseSessions(phaseKey);
            if (!!selectedMetaHistory && selectedMetaHistory.length > 0) {
                if (!!comparisonContext.commitId) {
                    // get matching commit
                    const commitIndex = selectedMetaHistory.findIndex(s => s.commitId === comparisonContext.commitId);
                    if (commitIndex !== -1) {
                        comparisonContext.setTestSesssionId(selectedMetaHistory[commitIndex].sessionId)
                        comparisonContext.setPhaseStatus(undefined);
                        return;
                    }
                }
                if (!!focusedSessionId) {
                    const focusedIndex = selectedMetaHistory.findIndex(s => s.id === focusedSessionId);
                    if (focusedIndex !== -1) {
                        const errorFingerPrint = selectedMetaHistory[focusedIndex].errorFingerprint;
                        if (!!errorFingerPrint) {
                            // get previous error
                            const previousErrorIndex = selectedMetaHistory.findIndex(s => s.errorFingerprint !== errorFingerPrint);
                            if (previousErrorIndex > focusedIndex) {
                                comparisonContext.setTestSesssionId(selectedMetaHistory[previousErrorIndex].sessionId)
                                comparisonContext.setPhaseStatus(undefined);
                                return;
                            }
                        }
                        // get previous available session
                        if (focusedIndex + 1 < selectedMetaHistory.length) {
                            comparisonContext.setTestSesssionId(selectedMetaHistory[focusedIndex + 1].sessionId)
                            comparisonContext.setPhaseStatus(undefined);
                            return;
                        }
                    }
                }
            }
            autoSelectSession();
            comparisonContext.setPhaseStatus(undefined);
        }
    }, [meta, test, phaseKey, focusedSessionId, options, streamStatus, comparisonContext.isLoadingPhase])

    const selectedMeta = !!comparisonContext.metaKey ? options.find(o => o.key === comparisonContext.metaKey)?.data?.ref as MetadataRef : undefined;

    return <Stack style={{overflow: 'hidden', borderWidth: '1px 0px 0px 0px', borderStyle: 'solid', borderColor: "#035ca1", paddingTop: 5}} grow>
                {!!comparisonContext.streamId &&
                    <Stack disableShrink tokens={{childrenGap: 3}}>
                        <Stack horizontal verticalAlign="center">
                            <ComboBox styles={{ root: { width: 210, height: 22, fontSize: 12, marginRight: 9 }, input: {color: hordeTheme.semanticColors.link, fontWeight: 600} }} placeholder={"Compare with"} allowFreeform={true} autoComplete="on" disabled={!options.length}                                
                                selectedKey={comparisonContext.metaKey}
                                onChange={(_1, option, _2, value) => {
                                    let selectedKey = option?.key as string;
                                    if (!option && !!value) {
                                        // find first partial match
                                        const lowerCaseValue = value.toLowerCase();
                                        selectedKey = options.find(o => (o.text as string).includes(lowerCaseValue))?.key as string;
                                    }
                                    comparisonContext.setMetaKey(selectedKey);
                                    comparisonContext.setPhaseStatus(undefined);
                                    comparisonContext.jobParams = undefined;
                                    autoSelectSession();
                                }}
                                options={options}
                                onRenderOption={onRenderOption}
                            />

                            <StreamSelector streams={handler.availableStreams} selected={comparisonContext.streamId} disabled={handler.availableStreams.length <= 1}
                                onClick={(stream) => {
                                    comparisonContext.setStreamId(stream);
                                    comparisonContext.setPhaseStatus(undefined);
                                    comparisonContext.setTestSesssionId(undefined);
                                    comparisonContext.jobParams = undefined;
                                }}/>

                            {!!comparisonContext.testSessionId && !!comparisonContext.phaseStatus?.testSession && 
                                <Stack style={{ paddingLeft: 8 }} horizontal tokens={{childrenGap: 8}} verticalAlign="center">
                                    <ChangeButton job={comparisonContext.jobParams?.job} rangeCL={comparisonContext.jobParams?.rangeCL} />
                                    <Link href={`/job/${comparisonContext.jobParams?.job?.id}?step=${comparisonContext.jobParams?.step?.id}`} target="_blank"><Text style={{ fontSize: 12, color: hordeTheme.semanticColors.link }}>View Job</Text></Link>
                                </Stack>
                            }

                            <Stack grow horizontalAlign="end">
                                <FontIcon
                                    style={{ fontSize: 13, cursor: "pointer", padding: 5 }}
                                    title="Close Comparison View"
                                    iconName={"Cancel"}
                                    onClick={onDismiss}
                                />
                            </Stack>
                        </Stack>
                        {!!comparisonContext.metaKey && !!selectedMeta &&
                            <Stack style={{ paddingLeft: 3 }} horizontal verticalAlign="end">
                                <Stack style={{ marginLeft: 30, marginBottom: 18, width: 10, cursor: "default" }} title="History">
                                    <FontIcon style={{ fontSize: 15 }} iconName="Timeline"/>
                                </Stack>
                                <PhaseHistoryGraphWidget
                                    test={test}
                                    streamId={comparisonContext.streamId}
                                    phaseKey={phaseKey}
                                    sessionId={comparisonContext.phaseStatus?.session?.id}
                                    meta={selectedMeta}
                                    onClick={onClickItem}
                                    handler={handler}
                                    reverse={isFromGridView}
                                />
                                <Stack grow horizontalAlign="end" style={{ marginBottom: 16, marginRight: 16}}>
                                    <FontIcon
                                        style={{ fontSize: 15, cursor: "pointer" , backgroundColor: "#035ca1", borderRadius: 4, padding: 7, color: 'white' }}
                                        title="Switch Comparison Source"
                                        iconName={"SwitcherStartEnd"}
                                        onClick={onClickSwap}
                                    />
                                </Stack>
                            </Stack>
                        }
                    </Stack>
                }
                {!!comparisonContext.testSessionId && !!comparisonContext.phaseStatus &&
                    <PhaseEventsPane phase={comparisonContext.phaseStatus}/>
                }
                {!comparisonContext.phaseStatus && (comparisonContext.isLoadingPhase || handler.queryLoading) &&
                    <Stack horizontalAlign='center' style={{ padding: 12}} grow>
                        <Spinner size={SpinnerSize.medium} />
                    </Stack>
                }
                {!comparisonContext.isLoadingPhase && !handler.queryLoading && !options.length &&  <Stack horizontalAlign='center' style={{ padding: 12}} grow><Text>No phase matched in session history</Text></Stack>}
            </Stack>
});
