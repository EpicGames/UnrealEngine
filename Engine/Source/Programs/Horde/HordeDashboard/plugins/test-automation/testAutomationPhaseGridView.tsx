// Copyright Epic Games, Inc. All Rights Reserved.

import { IconButton, Stack, Text, Spinner, SpinnerSize, Link, FontIcon, DefaultButton, TooltipHost, Icon, ITag, TagPicker, IPickerItemProps, List, IPage, IDetailsHeaderProps, IRenderFunction, Sticky, StickyPositionType, IColumn, DetailsList, SelectionMode, DetailsListLayoutMode, ScrollablePane, ScrollToMode } from "@fluentui/react";
import React, { useState, useEffect, useCallback, useRef, memo, useMemo } from "react";
import { observer } from "mobx-react-lite";
import { TestDataHandler, TestSessionDetails, MetadataRef, PhaseSessionResult } from "./testData";
import dashboard from "horde/backend/Dashboard";
import { CheckListOption, CheckListOptionSmall, getPhaseSessionStatusColor, ICheckListOption, phaseFiltersContext, PhaseHistoryGraphWidget, PhasesStatusPie, PhaseValues, StreamSelector, styles, tagColors, testViewIcons, TestViewType, userNavData } from "./testAutomationCommon";
import { getEventStyling } from "./testAutomationEvents";
import { MetaSelectorContext, PhaseStatusGroup, MetadataItem } from "./phaseGrid";

type PhaseNavInfo = { key: string, testId: string };

const metaWidth = 20;
const phaseRowHeight = 44;

const PhaseHeader: React.FC<{ group: PhaseStatusGroup, onClick: () => void }> = memo(({ group, onClick }) => {

    const tags = useMemo(() => {
        return group.phase.tagRefs?.values().toArray().map(t => t.name);
    }, [group]);    

    return <Stack disableShrink style={{cursor: "pointer", width: '100%', height: phaseRowHeight, paddingLeft: 16, paddingTop: 4}} verticalAlign="start" onClick={onClick}>
                <Stack horizontal verticalAlign="center" horizontalAlign="start" title={group.phase.name} style={{width: '100%', paddingRight: 3}}>
                    <Text style={{fontSize: 13, fontWeight: 600, whiteSpace: 'nowrap', overflow: 'hidden', textOverflow: 'ellipsis', direction: 'rtl', textAlign: 'right'}}>{group.phase.name}</Text>
                </Stack>
                <Stack horizontal verticalAlign="center" horizontalAlign="start" style={{overflow: 'hidden', textOverflow: 'ellipsis'}}>
                    {!!tags &&
                        <Stack horizontal verticalAlign="center" tokens={{childrenGap: 6}} style={{paddingLeft: 6, paddingRight: 3}}>
                            {tags.map(tag => <Stack key={`tag-${tag}`} style={{backgroundColor: tagColors.getTagColor(tag), borderRadius: 5, padding: '0px 4px', opacity: 0.8}}><Text style={{fontSize: 11, whiteSpace: 'nowrap'}}>#{tag}</Text></Stack>)}
                        </Stack>
                    }
                </Stack>
            </Stack>
});

const PhaseStatusWidget: React.FC<{ group: PhaseStatusGroup, context: MetaSelectorContext, onClick: (phaseKey: string, meta?: MetadataItem) => void }> = memo(observer(({ group, context, onClick }) => {
    const [errorsCaught, setErrorsCaught] = useState<Map<string, number> | undefined>(undefined);

    useEffect(() => {
        if (context.focusPhase === group.phase.key) {
            context.focusPhase = undefined;
        }
        return context.registerOnPhaseDataLoaded(group.phase.key, () => {
            if (group.errorFingerprints) {
                const today = Date.now();
                const errorsCaught = context.getFirstTimeErrorsCaught(group);
                setErrorsCaught(new Map(errorsCaught.entries().toArray().map(([key, value]) => [key, Math.round((today - value.getTime()) / (1000 * 60 * 60 * 24))])));
            }
        });
    }, [group, context]);

    context.subscribeFilteredPhases();
    
    return <div ref={(ref) => { group.componentRef = ref; }} style={{width: '100%', height: '100%' }}>
                <Stack horizontal verticalAlign="center" style={{cursor: "pointer", height: '100%' }}>                
                { context.metas.filter(m => m.visible).map(m => {
                    const status = group.getMetadataStatus(m);
                    const iconColor = status ? getPhaseSessionStatusColor(status) : undefined;
                    const errorDays = errorsCaught?.get(m.meta.id) ?? 0;
                    const title = `${status ? status.outcome : 'N/A'}${!!m.name ? ` - ${m.name}` : ''}`;
                    const iconSizeFactor = Math.min(errorDays/context.daysQueried, 1);
                    return <Stack key={`meta-${m.meta.id}`}
                                className={`${getMetaCellClassName(m.meta.id)} ${getPhaseCellClassName(group.phase.key)}`}
                                style={{ height: '100%', paddingLeft: 4, paddingRight: 4, width: metaWidth, position: 'relative' }}
                                horizontalAlign="center" verticalAlign="center"
                                title={title}
                                onClick={() => onClick(group.phase.key, m)}
                                >                                    
                                    { !!status && <FontIcon style={{ fontSize: 12 * (iconSizeFactor + 1), color: iconColor }} iconName="Square" /> }
                                    { !!status && !!status.errorFingerprint && errorDays > 0 &&
                                        <Text style={{ fontSize: 8, color: 'white', position: 'absolute', cursor: 'pointer', userSelect: "none" }}>{errorDays >= context.daysQueried ? `${context.daysQueried}+` : errorDays}</Text>
                                    }
                                    { !!status && !!status.errorFingerprint &&
                                        <Text style={{ fontSize: 9, color: iconColor, position: 'absolute', bottom: 2, right: 1, cursor: 'pointer', userSelect: "none", fontWeight: 800 }}>#{status.errorFingerprint.substring(0,2)}</Text>
                                    }
                                    { !status && <div style={{ minHeight: 12, minWidth: 12 }}/> }
                            </Stack>
                    })
                }
                </Stack>
            </div>
}));

const PhaseHistoryWidget: React.FC<{ group: PhaseStatusGroup, context: MetaSelectorContext, handler: TestDataHandler}> = memo(observer(({ group, context, handler }) => {
    const [loading, setLoading] = useState<boolean>(false);

    context.subscribeSelectedMeta();

    useEffect(() => {
        setLoading(true);
        handler.queryPhaseSessions(group.metadata.values().toArray(), handler.stream!, true).finally(() => {
            setLoading(false);
            context.notifyOnPhaseDataLoaded(group.phase.key);
        });
    }, [group, handler]);

    const onClickSession = useCallback((session: PhaseSessionResult) => {
        const testStatus = MetaSelectorContext.getTestStatus(handler, context.test);
        if (!context.selectedMeta || !testStatus) return;
        const metaStatus = testStatus.sessions.get(context.selectedMeta.meta);
        const testDataId = metaStatus?.history.find(s => s.id === session.sessionId)?.testDataId;
        if (!!testDataId) {
            handler.setSearchParam('session', testDataId);
            handler.setSearchParam('phase', group.phase.key);
            context.setSelectedPhase(group.phase.key);
        }
    }, [group, handler, context]);

    return <Stack grow horizontalAlign="start" verticalAlign="center" style={{height: phaseRowHeight}} onClick={(ev) => ev.stopPropagation()}>
                {!loading && !!context.test && !!context.selectedMeta && context.selectedMeta.visible && !!group.getMetadataStatus(context.selectedMeta) &&
                    <PhaseHistoryGraphWidget
                        test={context.test}
                        streamId={handler.stream!}
                        phaseKey={group.phase.key}
                        meta={context.selectedMeta.meta}
                        sessionId={group.metadata.get(context.selectedMeta)?.session?.id}
                        handler={handler}
                        withoutHeader={true}
                        reverse={true}
                        onClick={onClickSession}
                    />
                }
                {!!loading && !!context.selectedMeta && <Spinner size={SpinnerSize.small} style={{marginLeft: 32}} />}
            </Stack>
}));

const onRenderSuggestionsItem = (item: ITag) => {
    return <Stack style={{height: 24, padding: 4}}>
                <Text title={item.name} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', maxWidth: 300, fontSize: 12}}>{item.name}</Text>
            </Stack>
}

const onRenderItem = (props: IPickerItemProps<ITag>) => {
    const item = props.item;
    const isTag = item.name.startsWith("#");
    return <Stack key={`picker_item_${item.name}`} horizontal verticalAlign="center" verticalFill={true} style={{ marginRight: 2 }}>
                <TooltipHost content={item.name} delay={0} closeDelay={500}>
                    <FontIcon
                        style={{ fontSize: 13, padding: 4, paddingTop: 5, cursor: 'pointer', backgroundColor: isTag? tagColors.getTagColor(getTextFromTag(item)) : styles.defaultFilter.backgroundColor, borderRadius: 3, color: 'white'}}
                        iconName="Filter"
                        title="Remove"
                        onClick={props.onRemoveItem} />
                </TooltipHost>
            </Stack>
}

const getTextFromTag = (item: ITag) => item.name;

const PhaseGroupFilter: React.FC<{groups: PhaseStatusGroup[], selected?: string[], onChange: (items: string[]) => void, onFocus?: () => void, onBlur?: () => void}> = ({ groups, selected, onChange, onFocus, onBlur }) => {
    const [filters, setFilters] = useState<ITag[]>([]);
    const phaseTokens = useRef<ITag[]>();
    const phaseTags = useRef<ITag[]>();

    useEffect(() => {
        const tokens: ITag[] = (new Set(groups.map(g => (g.phase.name.split('.'))).flat())).keys().map(token => ({key: token.toLowerCase(), name: token})).toArray().sort((a, b) => a.key.length - b.key.length);
        const names: ITag[] = groups.map(g => ({key: `@${g.phase.name.toLowerCase()}`, name: `@${g.phase.name}`}));
        const tags: ITag[] = (new Set(groups.map(g => g.metadata.values().toArray().flatMap(status => status.tags ?? [])).flat())).keys().map(t => ({key: `#${t.toLowerCase()}`, name: `#${t}`})).toArray().sort((a, b) => a.key.length - b.key.length);
        phaseTokens.current = [...tokens, ...names];
        phaseTags.current = tags;
        selected && setFilters(selected.map(i => ({key: i, name: i})));
    }, [groups]);
    
    const filterSelectedTests = useCallback((filterText: string): ITag[] => {
        filterText = filterText.trim();
        if (filterText.length < 2) return [];
        const lowerText = filterText.toLowerCase();
        if (filterText.startsWith("#")) {
            const closestTags = phaseTags.current?.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray() ?? [];
            return closestTags;
        }
        const closestTokens = phaseTokens.current?.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray() ?? [];
        const isTokenExist = closestTokens.find((tag) => tag.key === lowerText);
        if (!isTokenExist) closestTokens.splice(0, 0, {key: lowerText, name: filterText});
        return closestTokens;
    }, [groups]);

    return <TagPicker
                styles={{
                    text: { minWidth: 210, maxWidth: 450, maxHeight: 28, display: 'inline-flex', flexWrap: 'nowrap' },
                    input: { height: 28, minWidth: 40, maxWidth: 200 },
                    itemsWrapper: { marginLeft: 3, marginTop: 2, marginRight: 3, marginBottom: 2, height: 28, maxWidth: 400, overflow: 'hidden' }
                }}
                onRenderItem={onRenderItem}
                onRenderSuggestionsItem={onRenderSuggestionsItem}
                removeButtonAriaLabel="Remove"
                selectionAriaLabel="Filter phases"
                selectedItems={filters}
                onResolveSuggestions={filterSelectedTests}
                getTextFromItem={getTextFromTag}
                onChange={(tokens) => {
                    let items = tokens ?? [];
                    const uniqueTokens = new Set(items.map(i => i.key));
                    items = uniqueTokens.keys().map(k => items.find(i => i.key === k)!).toArray();
                    setFilters(items);
                    onChange(items.map(t => t.name));
                }}
                inputProps={{ placeholder: "Filter names and #tags", onFocus: onFocus, onBlur: onBlur }}
            />
}

const MetaSelectorTitle: React.FC<{ stateContext: MetaSelectorContext, onFilterChange?: (items?: string[]) => void }> = observer(({ stateContext, onFilterChange }) => {
    const [hoveredMeta, setHoveredMeta] = useState<MetadataItem | undefined>(undefined);

    stateContext.subscribeMetaFilter();
    stateContext.subscribeFilteredPhases();
    stateContext.subscribeSelectedMeta();

    useEffect(() => {
        return stateContext.registerOnMetaHovered(setHoveredMeta);
    }, [stateContext]);

    const options = stateContext.filterOptions;

    useEffect(() => {
        const gridFilterItems = stateContext.getSearchGridFilterItems();
        let changed = false;
        options.forEach(o => {
            const newChecked = gridFilterItems?.includes(o.text!) ?? false;
            if (newChecked !== o.checked) {
                changed = true;
                o.checked = newChecked;
            }
        });
        if (changed) {
            onMetaFilterChange(options, true);
        }
    }, [options, stateContext.searchUpdated]);

    const onMetaFilterChange = useCallback((items: ICheckListOption[], forceUpdate?: boolean) => {
        stateContext.updateMetasVisibility(items);
        stateContext.updateMetaFilter();
        stateContext.findForkingMetaKey();
        const gridFilterItems = items.filter(i => i.checked).map(i => i.text!);
        stateContext.setSearchGridFilterItems(gridFilterItems, forceUpdate);
        onFilterChange?.(gridFilterItems);
    }, [stateContext, onFilterChange]);

    const onMetaValueClick = useCallback((value: string) => {
        const option = options.find(o => o.text === value);
        if (option) {
            option.checked = !option.checked;
            onMetaFilterChange(options);
        }
    }, [options, onMetaFilterChange]);

    const onMetaValueBeforeMainClick = useCallback((value: string) => {
        let changed = false;
        stateContext.getMetaValuesBeforeForkingKey().reverse().some(t => {
            const option = options.find(o => o.text === t);
            if (option) {
                option.checked = false;
                changed = true;
            }
            return t === value;
        });
        if (changed) {
            onMetaFilterChange(options);
        }
    }, [options, onMetaFilterChange, stateContext]);

    const latestCommit = useMemo(() => stateContext.filteredPhases?.at(0)?.metadata?.values().reduce((latest, s) => (s.session?.commitOrder ?? 0) > latest.order? {order: s.session!.commitOrder, id: s.session!.commitId} : latest, {order: 0, id: "NA"}), [stateContext.filteredPhases]);

    const phaseStatus = useMemo(() => new Map(stateContext.metas.map(m => [m.meta, stateContext.filteredPhases.map(p => p.getMetadataStatus(m)).filter(s => !!s)])), [stateContext.metas, stateContext.filteredPhases]);

    const displayedMeta = hoveredMeta ?? (stateContext.selectedMeta?.visible ? stateContext.selectedMeta : undefined) ?? (stateContext.metas.filter(m => m.visible).length === 1 ? stateContext.metas.find(m => m.visible) : undefined);
    const displayedStatus = displayedMeta ? phaseStatus.get(displayedMeta.meta) : undefined;

    return <Stack verticalAlign="center" grow horizontal tokens={{childrenGap: 10}} style={{height: '100%', anchorName: '--meta-values-container'}}>
                <Stack horizontal tokens={{childrenGap: 5}} style={{height: 21, position: 'fixed', borderLeft: `1px solid ${dashboard.darktheme ? '#4D4C4B' : '#edebe9'}`, left: 'calc(anchor(--meta-values left) - 0.48px)', bottom: 'anchor(--meta-values-container bottom)'}}>
                    <Stack horizontal>
                        {!!stateContext.metaKeysBeforeFork && stateContext.metaKeysBeforeFork.length > 0 &&
                            <Stack horizontal style={{ position: 'fixed', right: `calc(anchor(--meta-values left) + 2px)`}}>
                                {stateContext.getMetaValuesBeforeForkingKey().map(v => (
                                    <Stack horizontal key={`meta-value-${v}`}>
                                        <Stack style={{background: stateContext.metaValueColors?.get(v, "60"), cursor: "pointer"}}
                                            onClick={() => onMetaValueBeforeMainClick(v)}
                                        >
                                            <Text style={{whiteSpace: 'nowrap', padding: '0 10px 0 4px', position: "relative", height: 18}}>{v.substring(0, 16)}{v.length > 16 && '...'}
                                                {!!options.find(o => o.text === v)?.checked &&
                                                    <FontIcon iconName="Filter" style={{fontSize: 8, position: "absolute", right: 2, top: 2, color: styles.highlightFilter.backgroundColor}} />
                                                }
                                            </Text>
                                            <Stack style={{background: stateContext.metaValueColors?.get(v), height: 3}}></Stack>
                                        </Stack>
                                        <Stack style={{background: dashboard.darktheme ? "#ffffff3f" : "#0000003f", paddingBottom: 2}} verticalAlign="center">
                                            <FontIcon iconName="ChevronRight" style={{fontSize: 12, cursor: "default"}} />
                                        </Stack>
                                    </Stack>
                                ))}
                            </Stack>
                        }
                        {!!displayedMeta?.visible &&
                            <Stack horizontal>
                                {displayedMeta.meta.getValuesExcept(stateContext.getCommonKeysToFork()).map((v, index, self) =>
                                    <Stack horizontal key={`meta-value-${v}`}
                                        className={stateContext.getMetasContainingValue(v).map(m => getMetaCellClassName(m.meta.id)).join(' ')}
                                    >
                                        <Stack style={{background: stateContext.metaValueColors?.get(v, "60"), cursor: "pointer", position: "relative"}}
                                            onClick={() => onMetaValueClick(v)}
                                        >
                                            <Text style={{whiteSpace: 'nowrap', padding: '0 10px 0 4px', position: "relative", height: 18}}>{v.substring(0, 16)}{v.length > 16 && '...'}
                                                {!!options.find(o => o.text === v)?.checked &&
                                                    <FontIcon iconName="Filter" style={{fontSize: 8, position: "absolute", right: 2, top: 2, color: styles.highlightFilter.backgroundColor}} />
                                                }
                                            </Text>
                                            <Stack style={{background: stateContext.metaValueColors?.get(v), height: 3}}></Stack>
                                        </Stack>
                                        {index < self.length - 1 &&
                                            <Stack style={{background: dashboard.darktheme ? "#ffffff3f" : "#0000003f", paddingBottom: 2}} verticalAlign="center">
                                                <FontIcon iconName="ChevronRight" style={{fontSize: 12, cursor: "default"}} />
                                            </Stack>
                                        }
                                    </Stack>
                                )}
                            </Stack>
                        }
                        {!displayedMeta && !!stateContext.forkingMetaKey &&
                            <Stack horizontal>
                                {stateContext.getForkingMetaKeyValues().map(v => {
                                        const visibleMetasCount = stateContext.getVisibleMetasContainingForkingKeyValue(v).length;
                                        const isTooTight = v ? v.length / visibleMetasCount > 2 : false; // heuristic to avoid too tight spacing
                                        return (
                                            <Stack key={`meta-value-${v}`} title={v}
                                                style={{background: v ? stateContext.metaValueColors?.get(v, "60") : undefined, cursor: "pointer", minWidth: metaWidth*visibleMetasCount}}
                                                onClick={() => onMetaValueClick(v)}
                                            >
                                                <Stack className={stateContext.getMetasContainingValue(v).map(m => getMetaCellClassName(m.meta.id)).join(' ')} style={{height: 18, position: 'relative'}} horizontal>
                                                    <Text style={{
                                                        whiteSpace: 'nowrap',
                                                        overflow: 'hidden',
                                                        textOverflow: 'ellipsis',
                                                        width: isTooTight ? undefined : '100%',
                                                        transform: isTooTight ? 'rotate(-35deg)' : 'none',
                                                        transformOrigin: '0% 100%',
                                                        position: 'absolute',
                                                        left: isTooTight ? 8 : 4,
                                                    }}>{v?.substring(0, 16) ?? " "}{v?.length > 16 && '...'}</Text>
                                                </Stack>
                                                <Stack style={{background: v ? stateContext.metaValueColors?.get(v) : undefined, height: 3}}></Stack>
                                            </Stack>
                                        );
                                    })
                                }
                            </Stack>
                        }
                    </Stack>
                    {(!!displayedMeta?.visible && !!displayedStatus) &&
                        <Stack horizontal verticalAlign="start" tokens={{childrenGap: 5}}>
                            {PhaseValues(displayedStatus, {cursor: "default"})}
                            <Text style={{whiteSpace: 'nowrap', cursor: "default"}}>- CL: {displayedStatus.at(0)?.commitId ?? "N/A"}</Text>
                            {!!latestCommit && latestCommit.id === displayedStatus.at(0)?.commitId && <FontIcon style={{ fontSize: 11, cursor: "default" }} title="Latest commit" iconName="Star" />}
                        </Stack>
                    }
                </Stack>
                <Stack grow horizontalAlign="end" verticalAlign="center">
                    <TooltipHost content='Filter metadata'>
                        <CheckListOptionSmall key={`options-${stateContext.metaFilterUpdated}`} options={options} onChange={onMetaFilterChange}/>
                    </TooltipHost>
                </Stack>
            </Stack>
});

const MetaSelector: React.FC<{ onClickMeta?: (meta?: MetadataItem) => void, stateContext: MetaSelectorContext }> = observer(({ onClickMeta, stateContext }) => {

    stateContext.subscribeMetaFilter();
    stateContext.subscribeFilteredPhases();
    stateContext.subscribeSelectedMeta();
    
    const eventStyles = getEventStyling();
    const forkingMetaKey = stateContext.forkingMetaKey;
    const phaseStatus = useMemo(() => new Map(stateContext.metas.map(m => [m.meta, stateContext.filteredPhases.map(p => p.getMetadataStatus(m)).filter(s => !!s)])), [stateContext.metas, stateContext.filteredPhases]);

    const metaBackgroundColor = dashboard.darktheme ? "#1F2223" : "#f3f2f1";

    return <Stack verticalAlign="start" horizontal style={{paddingTop: 1, anchorName: '--meta-values'}} >
                {!!stateContext.metas &&
                    stateContext.metas.filter(m => m.visible).map(m => {
                        return <Stack key={`meta-${m.meta.id}`} verticalAlign="start"
                                    style={{ backgroundColor: stateContext.selectedMeta?.meta === m.meta ? metaBackgroundColor : undefined, cursor: onClickMeta ? 'pointer' : 'default' }}
                                    className={`${stateContext.selectedMeta?.meta === m.meta ? eventStyles.highlighted : undefined} ${getMetaCellClassName(m.meta.id)}`}
                                    onMouseOver={() => stateContext.onMetaHovered?.(m)}
                                    onMouseLeave={() => stateContext.onMetaHovered?.(undefined)}
                                    onClick={() => stateContext.selectedMeta?.meta === m.meta ? onClickMeta?.(undefined) : onClickMeta?.(m)}
                               >
                                    <Stack                                        
                                        verticalAlign="start"
                                        horizontalAlign="center"
                                        style={{ width: metaWidth, height: 24, padding: 2, background: !!forkingMetaKey ? stateContext.metaValueColors?.get(m.meta.getValue(forkingMetaKey), "60") : undefined }}
                                    >{PhasesStatusPie(phaseStatus.get(m.meta)!, 9)}</Stack>
                                </Stack>
                    })
                }
            </Stack>
});

const headerStyles = {
    root: {
        padding: 0,
        height: 26,
        background: 'transparent !important',
        // Remove hover background color on header cells
        selectors: {
            '.ms-DetailsHeader-cell:hover': {
                background: 'transparent',
            },
            '.ms-DetailsHeader-cell': {
                borderRight: `1px solid ${dashboard.darktheme ? '#4D4C4B' : '#edebe9'}`,
                padding: 0,
                height: 'fit-content',
            },
            '.ms-DetailsHeader-cell:last-child': {
                borderRight: 'none'
            }
        },
    }
};

const listStyles = {
    root: {
        selectors: {
            // Row background color
            '.ms-DetailsRow': {
                background: 'transparent !important'
            },
            '.ms-DetailsRow:hover': {
                background: dashboard.darktheme ? "#ffffff1f !important" : "#0000001f !important"
            },
            // Vertical borders between row cells
            '.ms-DetailsRow-cell': {
                borderRight: `1px solid ${dashboard.darktheme ? '#4D4C4B' : '#edebe9'}`,
                padding: 0
            },
            '.ms-DetailsRow-cell:last-child': {
                borderRight: 'none'
            }
        }
    }
}

const getPhaseCellClassName = (phaseKey: string) => `phase-cell-${phaseKey}`;
const getMetaCellClassName = (id: string) => `meta-cell-${id}`;

const getListCellHoverStyleSelectors = (ids: string[]) => {
    const selectors: any = {};
    ids.forEach(id => {
        const cellClassName = getMetaCellClassName(id);
        const selectorRule = `&:has(.${cellClassName}:hover) .${cellClassName}`;
        selectors[selectorRule] = {
            background: dashboard.darktheme ? "#ffffff1f !important" : "#0000001f !important"
        }
    });
    return selectors;
}

const getOnRenderDetailsHeader = (onRenderPhaseName: (column: PhaseColumn) => JSX.Element | null, onRenderMetaStatus: (column: PhaseColumn) => JSX.Element | null, onRenderHistory: (column: PhaseColumn) => JSX.Element | null) => {
    return (props: IDetailsHeaderProps, defaultRender: IRenderFunction<IDetailsHeaderProps>) => {
        if (!defaultRender) {
            return null;
        }
        return (
            <Sticky stickyPosition={StickyPositionType.Header}>
                {defaultRender({...props!,
                        styles: headerStyles,
                        onRenderColumnHeaderTooltip: (tooltipProps) => {
                            const column = tooltipProps?.column as PhaseColumn;
                            if (column.key === 'phase-name') {
                                return onRenderPhaseName(column);
                            }
                            if (column.key === 'phase-meta-status') {
                                return onRenderMetaStatus(column);
                            }
                            if (column.key === 'phase-selected-meta-history') {
                                return onRenderHistory(column);
                            }
                            return null;
                        }
                    })
                }
            </Sticky>
        );
    }
}

type PhaseColumn = IColumn & {
    focusedPhase?: string;
    handler?: TestDataHandler;
    context?: MetaSelectorContext;
}

const getPhaseColumn = (context: MetaSelectorContext, handler: TestDataHandler, onPhaseNameRender: (item: PhaseStatusGroup) => React.ReactNode, onMetaStatusRender: (item: PhaseStatusGroup) => React.ReactNode, onHistoryRender: (item: PhaseStatusGroup) => React.ReactNode): PhaseColumn[] => {
    const statusColumnWidth = metaWidth * (context.metas.filter(m => m.visible).length) - 20;
    return [
        {
            key: 'phase-name',
            name: 'Phase',
            minWidth: 200,
            isResizable: true,
            isCollapsible: false,
            isSorted: false,
            isSortedDescending: true,
            onRender: onPhaseNameRender,
            handler: handler,
            context: context,
        },
        {
            key: 'phase-meta-status',
            name: 'Status',
            minWidth: statusColumnWidth === 0 ? 0.5 : statusColumnWidth,
            maxWidth: statusColumnWidth === 0 ? 0.5 : statusColumnWidth,
            isResizable: false,
            isCollapsible: false,
            isSorted: false,
            onRender: onMetaStatusRender,
            context: context,
        },
        {
            key: 'phase-selected-meta-history',
            name: 'History',
            minWidth: 1000,
            isResizable: false,
            isCollapsible: false,
            isSorted: false,
            onRender: onHistoryRender,
            context: context,
        }
    ];
}

export const TestPhasesGridView: React.FC<{ testId: string, handler: TestDataHandler, onDismiss: () => void }> = observer(({ testId, handler, onDismiss }) => {
    const listBoundaryRef = useRef<HTMLDivElement | null>(null);
    const listRef = useRef<List>();
    const contextRef = useRef<MetaSelectorContext | undefined>(undefined);

    const test = useMemo(() => handler.testStreams.get(handler.stream!)?.tests.find(t => t.id === testId), [testId, handler.stream]);
    const testStatus = MetaSelectorContext.getTestStatus(handler, test);
    const metaSelectorContext = useMemo(() => new MetaSelectorContext(handler, test), [handler, test, testStatus]);
    contextRef.current = metaSelectorContext;

    const onPhaseNameRender = useCallback((item: PhaseStatusGroup) => {
        return <PhaseHeader group={item} onClick={() => { onPhaseSelected(item.phase.key); }} />
    }, [metaSelectorContext]);
    const onMetaStatusRender = useCallback((item: PhaseStatusGroup) => {
        return <PhaseStatusWidget group={item} context={metaSelectorContext} onClick={onPhaseSelected} />
    }, [metaSelectorContext]);
    const onHistoryRender = useCallback((item: PhaseStatusGroup) => {
        return !!testStatus? <PhaseHistoryWidget group={item} context={metaSelectorContext} handler={handler} /> : null;
    }, [metaSelectorContext, handler, testStatus]);

    const columns = useMemo(() => getPhaseColumn(metaSelectorContext, handler, onPhaseNameRender, onMetaStatusRender, onHistoryRender)
    , [metaSelectorContext, handler, onPhaseNameRender, onMetaStatusRender, onHistoryRender, metaSelectorContext.filteredPhasesUpdated]);

    const onPhaseNameHeaderRender = useCallback((column: PhaseColumn) => {
        const handler = column.handler!;
        const context = column.context!;
        const streamId = handler.stream!;
        return <Stack horizontal tokens={{childrenGap: 8}} verticalAlign="start" style={{height: '100%', padding: '0px 12px'}}>
                    <StreamSelector
                        streams={handler.availableStreams.filter((s) => s !== streamId)}
                        selected={streamId}
                        onClick={(stream) => {
                            handler.selectStream(stream);
                            context.setPhaseGroups([]);
                        }}
                    />
                    <Link style={{padding: '0px 5px'}} title="open test health view" onClick={() => {
                            handler.setSearchParam('view', TestViewType.Health);
                            handler.setSearchParam('health', handler.getSearchParam('grid') as string);
                            handler.removeSearchParam('grid');
                            handler.removeSearchParam('phase');
                            handler.removeSearchParam('metagrid');
                        }}>
                        <FontIcon iconName={testViewIcons.get(TestViewType.Health)} style={{fontSize: 15, paddingRight: 4}} />
                    </Link>
                </Stack>
    }, []);
    const onMetaStatusHeaderRender = useCallback((column: PhaseColumn) => {
        const context = column.context!;
        return <MetaSelector
                    stateContext={context}
                    onClickMeta={(item) => {
                        onPhaseSelected(undefined, item);
                    }}
                />
    }, [metaSelectorContext]);
    const onHistoryHeaderRender = useCallback((column: PhaseColumn) => {
        return <Stack horizontalAlign="start" verticalAlign="end">
                    { !!test && !!testStatus && <PhaseHistoryGraphWidget
                            test={test}
                            streamId={handler.stream!}
                            phaseKey={'fake'}
                            meta={new MetadataRef('fake', {}, 0)}
                            handler={handler}
                            reverse={true}
                            onClick={() => {}}
                        />
                    }
                </Stack>
    }, [test, handler, testStatus]);
    const onRenderDetailsHeader = useMemo(() => getOnRenderDetailsHeader(onPhaseNameHeaderRender
    , onMetaStatusHeaderRender, onHistoryHeaderRender), [onPhaseNameHeaderRender, onMetaStatusHeaderRender, onHistoryHeaderRender, handler.searchUpdated]);

    handler.subscribeToTestDataQueryLoading();
    handler.subscribeToPhaseQueryLoading();
    handler.subscribeToQueryLoading();
    handler.subscribeToSearch();
    metaSelectorContext.subscribePhaseGroups();
    metaSelectorContext.subscribeFilteredPhases();
    metaSelectorContext.subscribeSelectedPhase();
    metaSelectorContext.subscribeSelectedMeta();

    // collect test phase groups
    useEffect(() => {
        if (!!testStatus && !handler.phaseQueryLoading)
        {
            handler.queryTestPhases(testStatus).then(() => {
                const testDataRequests = testStatus.getMetadataLastSessions().map(([_, session]) => {
                    return handler.queryTestData(session.testDataId!);
                }).filter(request => !!request);
                if (testDataRequests && testDataRequests.length > 0) {
                    handler.setTestDataQueryLoading(true);
                    const tagRefMap = new Map(test!.tagRefs?.values().toArray().map(t => [t.name, t]));
                    Promise.all(testDataRequests).then((responses) => {
                        const sessionDetails = (responses.filter(response => !!response) as TestSessionDetails[]);
                        // reorg data by unique phase and keep order
                        const seen = new Map<string, PhaseStatusGroup>();
                        const phaseGroups: PhaseStatusGroup[] = [];
                        sessionDetails.forEach(session => {
                            if (!session.meta) return;
                            const metaItem = metaSelectorContext.metas.find(m => m.meta === session.meta);
                            if (!metaItem) return;
                            session.phases.forEach(phase => {
                                const phaseRef = test!.phases?.get(phase.key);
                                if (!phaseRef) return;
                                phase.phaseRef = phaseRef;
                                // update tags
                                const tags = phase.tags?.map(t => tagRefMap.get(t)).filter(t => !!t);
                                if (tags) phaseRef.updateTags(tags);
                                // populate phase group
                                let group = seen.get(phaseRef.key);
                                if (!group) {
                                    group = new PhaseStatusGroup(phaseRef);
                                    seen.set(phaseRef.key, group);
                                    phaseGroups.push(group);
                                };
                                group.addMetadata(metaItem, phase);
                            });
                        });
                        metaSelectorContext.setPhaseGroups(phaseGroups);
                        metaSelectorContext.setFilteredPhases(phaseFiltersContext.filterTestPhases(testId, phaseGroups) as PhaseStatusGroup[]);
                        handler.setTestDataQueryLoading(false);
                    });
                }
                });
        }
    }, [testStatus, metaSelectorContext, handler])

    const onPagesUpdated = useCallback((pages: IPage<PhaseStatusGroup>[]) => {
        // Extract all loaded items from all pages
        const loaded = pages.filter(page => page.items && !page.isSpacer).flatMap(page => page.items ?? []);
        //console.log('Loaded items:', loaded.length);
        if (loaded.length === 0) return;
        // scroll to focus phase if it is not visible
        if (!metaSelectorContext.focusPhase) return;
        const phaseIndex = metaSelectorContext.filteredPhases.findIndex((g) => g.phase.key === metaSelectorContext.focusPhase);
        if (phaseIndex === -1) return;
        if (!isPhaseVisible(phaseIndex)) {
            // force the list to load the page that contains the target item
            listRef.current?.scrollToIndex(phaseIndex, undefined, ScrollToMode.center);
        } else {
            // validate the scroll to index was achieved
            metaSelectorContext.focusPhase = undefined;
        }
    }, [metaSelectorContext]);

    const isPhaseVisible = useCallback((phaseIndex: number) => {
        // Check if the phase row is already visible by checking if the componentRef exists and is within visible bounds
        const phaseGroup = metaSelectorContext.filteredPhases[phaseIndex];
        if (phaseGroup?.componentRef && listBoundaryRef.current) {
            const rowRect = (phaseGroup.componentRef as HTMLElement).getBoundingClientRect();
            const listRect = listBoundaryRef.current.getBoundingClientRect();
            return rowRect.top > (listRect.top + headerStyles.root.height) && rowRect.bottom < listRect?.bottom;
        }
        return false;
    }, [metaSelectorContext]);

    const onPhaseSelected = useCallback((phaseKey?: string, metaItem?: MetadataItem) => {
        if (!!metaItem) {
            handler.setSearchParam('metagrid', metaItem.meta.id);
            metaSelectorContext.setSelectedMeta(metaItem);
        } else if (!phaseKey) {
            handler.removeSearchParam('metagrid');
            metaSelectorContext.setSelectedMeta(undefined);
        }

        if (!!phaseKey && metaSelectorContext.setSelectedPhase(phaseKey)) {
            if (listRef.current) {
                const phaseIndex = metaSelectorContext.filteredPhases.findIndex((g) => g.phase.key === phaseKey);
                if (phaseIndex === -1 || isPhaseVisible(phaseIndex))  return;
                listRef.current.scrollToIndex(phaseIndex, undefined, ScrollToMode.center);
            }
        }
    }, [handler, metaSelectorContext]);

    useEffect(() => {
        const metaParam = handler.getSearchParam('metagrid') as string | undefined;
        metaSelectorContext.setSelectedMetaById(metaParam);

        // check if there is a phase nav info and if it is, force select the phase to focus
        const phaseNavInfo = userNavData.getData('phaseNavInfo') as PhaseNavInfo | undefined;
        if (!!phaseNavInfo) {
            if (phaseNavInfo.testId === metaSelectorContext.test?.id) {
                userNavData.setData('phaseNavInfo', undefined);
                onPhaseSelected(phaseNavInfo.key);
            }
        }
    }, [metaSelectorContext, handler.searchUpdated]);

    useEffect(() => {
        // on unmount
        return () => {
            if (contextRef.current?.selectedPhase) {
                // save the selected phase to focus when the component is mounted again
                userNavData.setData('phaseNavInfo', { key: contextRef.current.selectedPhase, testId: contextRef.current.test?.id ?? '' });
            }
        }
    }, []);

    const eventStyles = getEventStyling();
    const buttonNoneTextColor = dashboard.darktheme ? "#616e85" : "#949898";
    const failureButtonTextColor = "#F9F9FB";
    const interruptedButtonTextColor = failureButtonTextColor;
    const warningButtonTextColor = failureButtonTextColor;

    const failureEnabled = !!metaSelectorContext.failureCursor?.length;
    const failureText = failureEnabled ? metaSelectorContext.failureCursor!.length.toString() : "No Failure";

    const interruptedEnabled = !!metaSelectorContext.interruptedCursor?.length;
    const warningEnabled = !!metaSelectorContext.warningCursor?.length;

    const localListStyles = useMemo(() => {
        const styles = structuredClone(listStyles);
        styles.root.selectors = {
                ...styles.root.selectors,
                ...getListCellHoverStyleSelectors(metaSelectorContext.metas.map(m => m.meta.id))
            };
        if (!!metaSelectorContext.selectedPhase) {
            styles.root.selectors[`.ms-DetailsRow:has(.${getPhaseCellClassName(metaSelectorContext.selectedPhase)})`] = {
                background: dashboard.darktheme ? "#ffffff1f !important" : "#0000001f !important"
            };
        }
        if (!!metaSelectorContext.selectedMeta) {
            styles.root.selectors[`.${getMetaCellClassName(metaSelectorContext.selectedMeta.meta.id)}`] = {
                background: dashboard.darktheme ? "#ffffff1f !important" : "#0000001f !important"
            };
        }
        return styles;
    }, [metaSelectorContext, metaSelectorContext.selectedPhaseUpdated, metaSelectorContext.selectedMetaUpdated]);

    return <Stack style={{width: '100%', height: '100%'}} styles={localListStyles}>
                {!testStatus && (handler.queryLoading || handler.phaseQueryLoading || handler.testDataQueryLoading) &&
                        <Stack horizontalAlign='center' style={{ padding: 12 }} tokens={{ childrenGap: 10 }}>
                            <Text style={styles.textLarge}>Loading Data</Text>
                            <Spinner size={SpinnerSize.large} />
                        </Stack>
                }
                {!handler.queryLoading && !handler.phaseQueryLoading && !handler.testDataQueryLoading &&
                    <Stack grow style={{height: '100%'}}>
                        <Stack horizontal verticalAlign="start" tokens={{childrenGap: 10}}>
                            <Stack style={{ paddingLeft: 8, minWidth: 550 }} tokens={{childrenGap: 5}}>
                                <Text style={{ fontSize: 16, fontWeight: 600, whiteSpace: 'nowrap', overflow: 'hidden', textOverflow: 'ellipsis', cursor: "default" }}>{test?.name ?? testId}{!!metaSelectorContext.commonMetaString && ` - ${metaSelectorContext.commonMetaString}`}</Text>
                            </Stack>

                            <Stack horizontal verticalAlign="center" style={{ height: '100%', paddingLeft: 6 }} grow>
                                <MetaSelectorTitle
                                    stateContext={metaSelectorContext}
                                    onFilterChange={(items) => {
                                        metaSelectorContext.setFilteredPhases(phaseFiltersContext.filterTestPhases(testId, metaSelectorContext.phaseGroups) as PhaseStatusGroup[]);
                                    }}
                                />
                            </Stack>

                            <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                    <TooltipHost content={failureEnabled ? 'Navigate to next failure': ''}>
                                        <DefaultButton
                                            disabled={!failureEnabled} className={failureEnabled ? eventStyles.failureButton : eventStyles.failureButtonDisabled}
                                            style={{ color: failureEnabled ? failureButtonTextColor : buttonNoneTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: failureEnabled ? '4px 0 0 4px' : '4px' }}
                                            text={failureText}
                                            onClick={() => {
                                                const [nextPhase, nextMetadata] = metaSelectorContext.getNextFailure();
                                                if (nextPhase) {
                                                    onPhaseSelected(nextPhase, nextMetadata);
                                                }
                                            }} >
                                            {failureEnabled &&
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            }
                                        </DefaultButton>
                                    </TooltipHost>
                                    {failureEnabled &&
                                        <div style={{backgroundColor: failureButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    }
                                </Stack>
                                {failureEnabled &&
                                    <TooltipHost content={'Navigate to previous failure'}>
                                        <IconButton
                                            disabled={!failureEnabled} className={failureEnabled ? eventStyles.failureButton : eventStyles.failureButtonDisabled}
                                            style={{ color: failureButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => {
                                                const [previousPhase, previousMetadata] = metaSelectorContext.getPreviousFailure();
                                                if (previousPhase) {
                                                    onPhaseSelected(previousPhase, previousMetadata);
                                                }
                                            }} />
                                    </TooltipHost>
                                }
                            </Stack>

                            {interruptedEnabled &&
                                <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                    <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                        <TooltipHost content='Navigate to next interrupted'>
                                            <DefaultButton
                                                className={eventStyles.interruptedButton}
                                                style={{ color: interruptedButtonTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: '4px 0 0 4px' }}
                                                text={metaSelectorContext.interruptedCursor?.length.toString()}
                                                onClick={() => {
                                                    const [nextPhase, nextMetadata] = metaSelectorContext.getNextInterrupted();
                                                    if (nextPhase) {
                                                        onPhaseSelected(nextPhase, nextMetadata);
                                                    }
                                                }} >
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            </DefaultButton>
                                        </TooltipHost>
                                        <div style={{backgroundColor: interruptedButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    </Stack>
                                    <TooltipHost content={'Navigate to previous interrupted'}>
                                        <IconButton
                                            className={eventStyles.interruptedButton}
                                            style={{ color: interruptedButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => {
                                                const [previousPhase, previousMetadata] = metaSelectorContext.getPreviousInterrupted();
                                                if (previousPhase) {
                                                    onPhaseSelected(previousPhase, previousMetadata);
                                                }
                                            }} />
                                    </TooltipHost>
                                </Stack>
                            }

                            {warningEnabled &&
                                <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                    <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                        <TooltipHost content='Navigate to next warning'>
                                            <DefaultButton
                                                className={eventStyles.warningButton}
                                                style={{ color: warningButtonTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: '4px 0 0 4px' }}
                                                text={metaSelectorContext.warningCursor?.length.toString()}
                                                onClick={() => {
                                                    const [nextPhase, nextMetadata] = metaSelectorContext.getNextWarning();
                                                    if (nextPhase) {
                                                        onPhaseSelected(nextPhase, nextMetadata);
                                                    }
                                                }} >
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            </DefaultButton>
                                        </TooltipHost>
                                        <div style={{backgroundColor: warningButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    </Stack>
                                    <TooltipHost content={'Navigate to previous warning'}>
                                        <IconButton
                                            className={eventStyles.warningButton}
                                            style={{ color: warningButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => {
                                                const [previousPhase, previousMetadata] = metaSelectorContext.getPreviousWarning();
                                                if (previousPhase) {
                                                    onPhaseSelected(previousPhase, previousMetadata);
                                                }
                                            }} />
                                    </TooltipHost>
                                </Stack>
                            }

                            <Stack verticalAlign='center' style={{ height: '100%' }}>
                                <TooltipHost content='Filter by outcome'>
                                    <CheckListOption
                                        options={phaseFiltersContext.outcomeOptions}
                                        onChange={() => {
                                            metaSelectorContext.setFilteredPhases(phaseFiltersContext.filterTestPhases(testId, metaSelectorContext.phaseGroups) as PhaseStatusGroup[]);
                                        }}
                                    />
                                </TooltipHost>
                            </Stack>

                            <Stack verticalAlign='center' style={{ height: '100%' }}>
                                <TooltipHost content='Filter phases by name and tag'>
                                    <PhaseGroupFilter
                                        groups={metaSelectorContext.phaseGroups}
                                        selected={[...phaseFiltersContext.getFilteredNames(testId), ...phaseFiltersContext.getFilteredTags(testId)?.map(t => `#${t}`) ?? []]}
                                        onChange={(items => {
                                            phaseFiltersContext.setFilteredNames(testId, items);
                                            metaSelectorContext.setFilteredPhases(phaseFiltersContext.filterTestPhases(testId, metaSelectorContext.phaseGroups) as PhaseStatusGroup[]);
                                        })}/>
                                </TooltipHost>
                            </Stack>

                            <Stack style={{ paddingBottom: 4 }}>
                                <IconButton
                                    iconProps={{ iconName: 'Cancel' }}
                                    ariaLabel="Close phase grid view"
                                    onClick={onDismiss}
                                />
                            </Stack>
                        </Stack>
                        <Stack style={{ paddingLeft: 16, paddingBottom: 3, overflow: 'hidden', height: '100%', position: 'relative'}} grow>
                            {(handler.queryLoading || handler.phaseQueryLoading || handler.testDataQueryLoading) &&
                                <Stack horizontalAlign='center' style={{ padding: 12 }} tokens={{ childrenGap: 10 }}>
                                    <Text style={styles.textLarge}>Loading Data</Text>
                                    <Spinner size={SpinnerSize.large} />
                                </Stack>
                            }
                            {!!test && !handler.queryLoading && !handler.phaseQueryLoading && !handler.testDataQueryLoading &&
                                <div ref={(ref) => { listBoundaryRef.current = ref; }} style={{ height: '100%' }}>
                                    <ScrollablePane>
                                        <DetailsList
                                            styles={{ root: {overflowX: 'hidden !important'}}}
                                            items={metaSelectorContext.filteredPhases}
                                            columns={columns}
                                            getKey={(group: PhaseStatusGroup) => `phase-${group.phase.key}`}
                                            listProps={{
                                                componentRef: (list: List) => { listRef.current = list; },
                                                onPagesUpdated: onPagesUpdated
                                            }}
                                            compact={true}
                                            isHeaderVisible={true}
                                            onRenderDetailsHeader={onRenderDetailsHeader}
                                            selectionMode={SelectionMode.none}
                                            layoutMode={DetailsListLayoutMode.justified}                                    
                                        />
                                    </ScrollablePane>
                                </div>
                            }
                            {!test && !handler.queryLoading && !handler.phaseQueryLoading && !handler.testDataQueryLoading &&
                                <Stack horizontalAlign='center' style={{ padding: 12 }} tokens={{ childrenGap: 10 }}>
                                    <Text style={styles.textLarge}>Test not found in stream</Text>
                                </Stack>
                            }
                        </Stack>
                    </Stack>
                }     
            </Stack>
});