// Copyright Epic Games, Inc. All Rights Reserved.

import { IconButton, Stack, Text, Spinner, SpinnerSize, Link, FontIcon, DefaultButton, TooltipHost, Icon, ITag, TagPicker, IPickerItemProps, Label, List } from "@fluentui/react";
import { useState, useEffect, useCallback, useRef, memo, useMemo } from "react";
import { observer } from "mobx-react-lite";
import { TestDataHandler, TestSessionDetails, TestPhaseStatus, MetadataRef, TestNameRef } from "./testData";
import { TestPhaseOutcome } from './api';
import dashboard from "horde/backend/Dashboard";
import { ChangeButton } from "horde/components/ChangeButton";
import { getShortNiceTime, msecToElapsed } from "horde/base/utilities/timeUtils";
import { getHordeTheme } from "horde/styles/theme";
import { GetArtifactResponse } from "horde/backend/Api";
import { CheckListOption, CircularList, getPhaseSessionStatusColor, phaseFiltersContext, PhaseHistoryGraphWidget, PhasesStatusPie, styles, tagColors, testViewIcons, TestViewType } from "./testAutomationCommon";
import { projectStore } from "horde/backend/ProjectStore";
import { JobArtifactsModal } from "horde/components/artifacts/ArtifactsModal";
import { getJobParams, JobContext } from "./phaseCompare";
import { getEventStyling, PhaseEventsPane } from "./testAutomationEvents";
import { comparisonContext, PhaseComparisonPane } from "./testAutomationPhaseCompare";
import { EventEntry } from "./testEventsModel";

const PhaseHistoryPane: React.FC<{ test: TestNameRef, meta?: MetadataRef, phaseKey: string, streamId?: string, sessionId?: string, handler: TestDataHandler, onClickCompare: () => void }> = memo(observer(({test, meta, phaseKey, streamId, sessionId, handler, onClickCompare}) => {

    const metaStatus = !!meta && !!streamId? handler.getStatusStream(streamId)?.tests.get(test)?.sessions.get(meta) : undefined;
    const isFromGridView = !!handler.getSearchParam('grid');
   
    return <Stack style={{ paddingLeft: 3 }} horizontal verticalAlign="end" disableShrink>
                <Stack style={{ marginLeft: 30, marginBottom: 18, width: 10, cursor: "default" }} title="History">
                    <FontIcon style={{ fontSize: 15 }} iconName="Timeline" />
                </Stack>
                {(!metaStatus || !sessionId) &&
                    <Stack style={{ marginLeft: 10, marginBottom: 21 }}>
                        {handler.queryLoading ?
                            <Stack horizontal verticalAlign="center" tokens={{childrenGap: 6}}><Spinner size={SpinnerSize.small} /><Text>Loading...</Text></Stack>
                            : <Text>No history available for this phase (for preflight, open the <FontIcon style={{ fontSize: 9 , backgroundColor: "#035ca1", borderRadius: 3, padding: 5, color: 'white' }} iconName="BranchCompare" title="comparison view"/> view <FontIcon style={{fontSize: 9}} iconName="ChromeBackMirrored"/> )</Text>
                        }
                    </Stack>
                }
                {!!metaStatus && !!sessionId && <PhaseHistoryGraphWidget
                    test={test}
                    streamId={streamId!}
                    sessionId={sessionId}
                    phaseKey={phaseKey}
                    meta={meta!}
                    handler={handler}
                    reverse={isFromGridView}
                    onClick={(session) => {
                        const testSession = metaStatus?.history.find(s => s.id === session.sessionId);
                        !!testSession?.testDataId && handler.setSearchParam('session', testSession.testDataId);
                    }}
                />}
                <Stack grow horizontalAlign="end" style={{ marginBottom: 16, marginRight: 16}}>
                    <FontIcon
                        style={{ fontSize: 15, cursor: "pointer" , backgroundColor: "#035ca1", borderRadius: 4, padding: 7, color: 'white' }}
                        title="Toggle Comparison View"
                        iconName={"BranchCompare"}
                        onClick={onClickCompare}
                    />
                </Stack>
            </Stack>
}));

const TestPhaseExpandPane: React.FC<{ phase: TestPhaseStatus, onEventsLoaded?: (events: EventEntry[]) => void, handler: TestDataHandler }> = memo(observer(({ phase, onEventsLoaded, handler }) => {
    const [loadingPhaseSession, setLoadingPhaseSession] = useState<boolean>(false);
    const [comparison, setComparison] = useState<boolean>(comparisonContext.isEnable);

    useEffect(() => {
        // query phase data
        if (!handler.queryLoading) {
            setLoadingPhaseSession(true);
            handler.queryPhaseSessions([phase], phase.testSession?.streamId ?? handler.stream!).then(() => {
                if (!comparisonContext.isEnable) {
                    // if no phase session associated with this phase in history, it should be a preflight.
                    // we are going to prefill the commitId in the comparison context with the preflight base commit id.
                    comparisonContext.commitId = !phase.session ? phase.commitId : undefined;
                }
                setComparison(comparisonContext.isEnable);
                setLoadingPhaseSession(false);
            });
        }
    }, [phase, phase.testSession, handler.queryLoading]);

    return <Stack style={{overflow: 'hidden'}} grow>
                <Stack style={{margin: 3, overflow: 'hidden', height: comparison? '47%' : 'auto'}} grow>
                    { (loadingPhaseSession || handler.queryLoading) &&
                        <Stack horizontal verticalAlign='center' style={{ marginLeft: 53, paddingBottom: 21, height: 50 }} tokens={{ childrenGap: 6 }}>
                            <Spinner size={SpinnerSize.small} /><Text>Loading...</Text>
                        </Stack>
                    }
                    { !loadingPhaseSession && !handler.queryLoading && !!phase.test && <PhaseHistoryPane
                            test={phase.test}
                            streamId={phase.testSession?.streamId ?? handler.stream}
                            meta={phase.testMetadata}
                            sessionId={phase.session?.id}
                            phaseKey={phase.key}
                            handler={handler}
                            onClickCompare={() => {
                                if (comparison) { comparisonContext.reset() }
                                    else { comparisonContext.setEnable() }
                                setComparison(!comparison)
                            }}
                        /> }
                    <PhaseEventsPane phase={phase} onEventsLoaded={onEventsLoaded}/>
                </Stack>
                {comparison && !!phase.test && 
                    <Stack style={{margin: 3, overflow: 'hidden', height: '53%'}} grow>
                        <PhaseComparisonPane
                            test={phase.test}
                            streamId={phase.testSession?.streamId ?? handler.stream}
                            meta={phase.testMetadata}
                            focusedSessionId={phase.session?.id}
                            phaseKey={phase.key}
                            handler={handler}
                            onDismiss={() => {comparisonContext.reset(); setComparison(false)}}
                        />
                    </Stack>
                }
            </Stack>
}));

const TestPhaseHeader: React.FC<{ phase: TestPhaseStatus, expanded: boolean, toggleView: (phaseKey: string) => void, onNext?: () => void, onBack?: () => void }> = memo(({ phase, expanded, toggleView, onNext, onBack }) => {
    const onClick = useCallback(() => {
        toggleView(phase.key);
    }, [phase]);

    const color = getPhaseSessionStatusColor(phase);
    const eventStyles = getEventStyling();
    return <Stack tokens={{childrenGap: 2}} disableShrink style={{cursor: "pointer" }} onClick={onClick} horizontal verticalAlign="center">
                <Stack>
                    <Stack horizontal verticalAlign="center">
                        <Stack style={{ paddingTop: 1, paddingRight: 4}}>
                            <FontIcon style={{ fontSize: 13 }} iconName={expanded ? "ChevronDown" : "ChevronRight"} />
                        </Stack>
                        <Text style={{width: 'calc(100vw - 150px)', fontSize: 13, fontWeight: 600, whiteSpace: 'nowrap', overflow: 'hidden', textOverflow: 'ellipsis', cursor: "text"}} onClick={(ev) => ev.stopPropagation()}>{phase.name}</Text>
                    </Stack>
                    <Stack horizontal verticalAlign="center">
                        <Stack style={{ paddingLeft: 5, paddingRight: 4 }}>
                            <FontIcon style={{ fontSize: 12, color: color }} iconName="Square" />
                        </Stack>
                        <Text style={{fontSize: 11}}>{phase.outcome}</Text>
                        {!!phase.tags &&
                            <Stack horizontal verticalAlign="center" tokens={{childrenGap: 6}} style={{paddingLeft: 6}}>
                                {phase.tags.map(tag => <Stack key={`tag-${tag}`} style={{backgroundColor: tagColors.getTagColor(tag), borderRadius: 5, padding: '0px 4px', opacity: 0.8}}><Text style={{fontSize: 11, whiteSpace: 'nowrap'}}>#{tag}</Text></Stack>)}
                            </Stack>
                        }
                    </Stack>
                </Stack>
                <Stack horizontalAlign="end" grow style={{paddingRight: 12}}>
                    {!!expanded &&
                        <Stack horizontal>
                            <DefaultButton
                                className={eventStyles.defaultButton} title="Navigate to next phase"
                                style={{ height: 24, minWidth: 24, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 4, paddingRight: 4, borderRadius: '4px 0px 0px 4px' }}
                                onClick={(ev) => { ev.stopPropagation(); onNext && onNext() }}
                                >
                                    <Icon style={{ fontSize: 12 }} iconName='ChevronDown' />
                            </DefaultButton>
                            <DefaultButton
                                className={eventStyles.defaultButton} title="Navigate to previous phase"
                                style={{ height: 24, minWidth: 24, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 4, paddingRight: 4, borderRadius: 0 }}
                                onClick={(ev) => { ev.stopPropagation(); onBack && onBack()}}
                                >
                                    <Icon style={{ fontSize: 12 }} iconName='ChevronUp' />
                            </DefaultButton>
                            <DefaultButton
                                className={eventStyles.defaultButton} title="Collapse phase details"
                                style={{ height: 24, minWidth: 24, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 4, paddingRight: 4, borderRadius: '0px 4px 4px 0px' }}>
                                    <Icon style={{ fontSize: 12, paddingTop: 1 }} iconName='ChromeMinimize' />
                            </DefaultButton>
                        </Stack>
                    }
                </Stack>
            </Stack>
});

const TestPhaseSection: React.FC<{ phase: TestPhaseStatus, handler: TestDataHandler, onCollapse: (phaseKey: string) => void, onNext: () => void, onBack: () => void }> = ({ phase, handler, onCollapse, onNext, onBack }) => {
    const toggleView = useCallback((phaseKey: string) => {
        comparisonContext.reset();
        handler.removeSearchParam('phase');
        onCollapse(phaseKey);
    }, []);

    const borderColor = dashboard.darktheme ? "#4D4C4B" : "#6D6C6B";
    return <Stack style={{ borderTopWidth: 1, borderTopColor: borderColor, borderTopStyle: 'solid', height: '100%', paddingTop: 3}}>
                <TestPhaseHeader phase={phase} expanded={true} toggleView={toggleView} onNext={onNext} onBack={onBack} />
                <TestPhaseExpandPane phase={phase} handler={handler}/>
            </Stack>
}

const TestPhaseListItem: React.FC<{ phase: TestPhaseStatus, focused: boolean, handler: TestDataHandler }> = memo(({ phase, focused, handler }) => {
    useEffect(() => {
        if (focused) {
            const component = phase.componentRef;
            if (!component) return;
            component.scrollIntoView();
       }
    }, [focused]);

    const registerRef = useCallback((ref: any) => {
        phase.componentRef = ref;
    }, []);

    const toggleView = useCallback((phaseKey: string) => {
        handler.setSearchParam("phase", phaseKey);
    }, []);
    
    const borderColor = dashboard.darktheme ? "#4D4C4B" : "#6D6C6B";
    const highLightColor = dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "rgba(0, 0, 0, 0.1)";

    return <Stack styles={{ root: { borderTopWidth: 1, borderTopColor: borderColor, borderTopStyle: 'solid', paddingTop: 3, paddingBottom: 6, selectors: {':hover' : { backgroundColor: highLightColor }} } }}>
                <div ref={registerRef}><TestPhaseHeader phase={phase} expanded={false} toggleView={toggleView} /></div>
            </Stack>
});


const onRenderSuggestionsItem = (item: ITag) => {
    return <Stack style={{height: 24, padding: 4}}>
                <Text title={item.name} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', maxWidth: 300, fontSize: 12}}>{item.name}</Text>
            </Stack>
}

const onRenderItem = (props: IPickerItemProps<ITag>) => {
    const item = props.item;
    const isTag = item.name.startsWith("#");
    return <Stack key={`picker_item_${item.name}`} horizontal verticalAlign="center" verticalFill={true} style={{ marginRight: 2 }}>
                <TooltipHost content={item.name} delay={0} closeDelay={500}>
                    <FontIcon
                        style={{ fontSize: 13, padding: 4, paddingTop: 5, cursor: 'pointer', backgroundColor: isTag? tagColors.getTagColor(getTextFromTag(item)) : styles.defaultFilter.backgroundColor, borderRadius: 3, color: 'white'}}
                        iconName="Filter"
                        title="Remove"
                        onClick={props.onRemoveItem} />
                </TooltipHost>
            </Stack>
}

const getTextFromTag = (item: ITag) => item.name;

const PhaseFilter: React.FC<{phases: TestPhaseStatus[], selected?: string[], onChange: (items: string[]) => void, onFocus?: () => void, onBlur?: () => void}> = ({ phases, selected, onChange, onFocus, onBlur }) => {
    const [filters, setFilters] = useState<ITag[]>([]);
    const phaseTokens = useRef<ITag[]>();
    const phaseTags = useRef<ITag[]>();

    useEffect(() => {
        const tokens: ITag[] = (new Set(phases.map(p => (p.name.split('.'))).flat())).keys().map(token => ({key: token.toLowerCase(), name: token})).toArray().sort((a, b) => a.key.length - b.key.length);
        const names: ITag[] = phases.map(p => ({key: `@${p.name.toLowerCase()}`, name: `@${p.name}`}));
        const tags: ITag[] = (new Set(phases.map(p => p.tags ?? []).flat())).keys().map(t => ({key: `#${t.toLowerCase()}`, name: `#${t}`})).toArray().sort((a, b) => a.key.length - b.key.length);
        phaseTokens.current = [...tokens, ...names];
        phaseTags.current = tags;
        selected && setFilters(selected.map(i => ({key: i, name: i})));
    }, [phases]);
    
    const filterSelectedTests = useCallback((filterText: string): ITag[] => {
        filterText = filterText.trim();
        if (filterText.length < 2) return [];
        const lowerText = filterText.toLowerCase();
        if (filterText.startsWith("#")) {
            const closestTags = phaseTags.current?.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray() ?? [];
            return closestTags;
        }
        const closestTokens = phaseTokens.current?.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray() ?? [];
        const isTokenExist = closestTokens.find((tag) => tag.key === lowerText);
        if (!isTokenExist) closestTokens.splice(0, 0, {key: lowerText, name: filterText});
        return closestTokens;
    }, [phases]);

    return <TagPicker
                styles={{
                    text: { minWidth: 210, maxWidth: 450, maxHeight: 28, display: 'inline-flex', flexWrap: 'nowrap' },
                    input: { height: 28, minWidth: 40, maxWidth: 200 },
                    itemsWrapper: { marginLeft: 3, marginTop: 2, marginRight: 3, marginBottom: 2, height: 28, maxWidth: 400, overflow: 'hidden' }
                }}
                onRenderItem={onRenderItem}
                onRenderSuggestionsItem={onRenderSuggestionsItem}
                removeButtonAriaLabel="Remove"
                selectionAriaLabel="Filter phases"
                selectedItems={filters}
                onResolveSuggestions={filterSelectedTests}
                getTextFromItem={getTextFromTag}
                onChange={(tokens) => {
                    let items = tokens ?? [];
                    const uniqueTokens = new Set(items.map(i => i.key));
                    items = uniqueTokens.keys().map(k => items.find(i => i.key === k)!).toArray();
                    setFilters(items);
                    onChange(items.map(t => t.name));
                }}
                inputProps={{ placeholder: "Filter names and #tags", onFocus: onFocus, onBlur: onBlur }}
            />
}

export const TestPhasesView: React.FC<{ testDataId: string, handler: TestDataHandler, onDismiss: () => void }> = observer(({ testDataId, handler, onDismiss }) => {
    const [sessionDetails, setSessionDetails] = useState<TestSessionDetails | undefined>(undefined);
    const [filteredPhases, setFilteredPhases] = useState<TestPhaseStatus[]>([]);
    const [focusPhase, setFocusPhase] = useState<string>();
    const [jobArtifacts, setJobArtifacts] = useState<GetArtifactResponse>();
    const selectedPhase = useRef<TestPhaseStatus>();
    const failureCursor = useRef<CircularList<string>>();
    const interruptedCursor = useRef<CircularList<string>>();
    const warningCursor = useRef<CircularList<string>>();
    const phaseCursor = useRef<CircularList<string>>();
    const listRef = useRef<List>();

    const phaseKey = handler.getSearchParam('phase') as string | undefined;

    handler.subscribeToTestDataQueryLoading();
    handler.subscribeToSearch();

    const updateFilter = useCallback((phases: TestPhaseStatus[]) => {
        const failures = phases.filter((p) => p.outcome !== TestPhaseOutcome.Success && p.outcome !== TestPhaseOutcome.NotRun && p.outcome !== TestPhaseOutcome.Skipped).map((p) => p.key) ?? [];
        failureCursor.current = new CircularList(failures);
        failureCursor.current.setCursor(selectedPhase.current?.key);
        const interrupted = phases.filter((p) => p.outcome === TestPhaseOutcome.Interrupted).map((p) => p.key) ?? [];
        interruptedCursor.current = new CircularList(interrupted);
        interruptedCursor.current.setCursor(selectedPhase.current?.key);
        phaseCursor.current = new CircularList(phases.map((p) => p.key));
        phaseCursor.current.setCursor(selectedPhase.current?.key);
        const warnings = phases.filter((p) => p.outcome === TestPhaseOutcome.Success && p.hasWarning).map((p) => p.key) ?? [];
        warningCursor.current = new CircularList(warnings);
        warningCursor.current.setCursor(selectedPhase.current?.key);
        setFilteredPhases(phases);
    }, [handler.searchUpdated]);

    const onRenderCell = useCallback(
        (phase: TestPhaseStatus) => <TestPhaseListItem key={phase.key} phase={phase} focused={focusPhase === phase.key} handler={handler}/>
        , [handler.searchUpdated]);

    useEffect(() => {
        return () => {
            // on unmount
            comparisonContext.reset();
        }
    }, [])

    useEffect(() => {
        if (!!sessionDetails) {
            const tagNames: string[] = (new Set(sessionDetails.phases.map(p => p.tags ?? []).flat())).keys().toArray().sort((a, b) => a.length - b.length);
            tagColors.setTagColors(tagNames);
        }
    }, [sessionDetails]);

    useEffect(() => {
        handler.queryTestData(testDataId).then((fetchedSessionDetails) => {
            if (fetchedSessionDetails) {
                if(!handler.filterState.stream) {
                    // Select stream only if none selected.
                    handler.selectStream(fetchedSessionDetails.streamId);
                }
                setSessionDetails(fetchedSessionDetails);
                const testId = fetchedSessionDetails.test?.id ?? testDataId;
                updateFilter(phaseFiltersContext.filterTestPhases(testId, fetchedSessionDetails.phases) as TestPhaseStatus[]);
            }
        });
        setSessionDetails(undefined);
    }, [testDataId]);

    const jobContext : JobContext | undefined = useMemo(() => sessionDetails ? getJobParams(sessionDetails, handler) : undefined, [sessionDetails, sessionDetails?.session]);

    const onPagesUpdated = useCallback(() => {
        if (!focusPhase) return;
        const phaseIndex = filteredPhases.findIndex((phase) => phase.key === focusPhase);
        if (phaseIndex === -1) return;

        if (!filteredPhases[phaseIndex].componentRef) {
            // force the list to load the page that contains the target item
            listRef.current?.scrollToIndex(phaseIndex);
        } else {
            // validate the scroll to index was achieved
            setFocusPhase(undefined);
        }
    }, [focusPhase]);

    selectedPhase.current = useMemo(() => {
        return sessionDetails?.phases.find((phase) => phase.key === phaseKey);
    }, [handler.searchUpdated, sessionDetails]);

    useEffect(() => {
        if (phaseKey !== failureCursor.current?.selected) {
            failureCursor.current?.setCursor(phaseKey);
        }
        if (phaseKey !== warningCursor.current?.selected) {
            warningCursor.current?.setCursor(phaseKey);
        }
        if (phaseKey !== phaseCursor.current?.selected) {
            phaseCursor.current?.setCursor(phaseKey);
        }
    }, [handler.searchUpdated, filteredPhases]);

    const StatusPie = useMemo(
        () => <Stack horizontal verticalAlign='center' style={{ height: '100%' }} tokens={{childrenGap: 3}}>
                <Label style={{whiteSpace: 'nowrap'}}>{filteredPhases.length} {filteredPhases.length > 1 ? "Phases" : "Phase"}</Label>
                {PhasesStatusPie(filteredPhases, 14)}
            </Stack>
        , [filteredPhases]);

    const metaString = useMemo(() => sessionDetails?.meta?.getValues().join(' / '), [sessionDetails]);

    const hordeTheme = getHordeTheme();
    const eventStyles = getEventStyling();
    const buttonNoneTextColor = dashboard.darktheme ? "#616e85" : "#949898";
    const failureButtonTextColor = "#F9F9FB";
    const interruptedButtonTextColor = failureButtonTextColor;
    const warningButtonTextColor = failureButtonTextColor;

    const failureEnabled = !!failureCursor.current?.length;
    const failureText = failureEnabled ? failureCursor.current!.length.toString() : "No";

    const interruptedEnabled = !!interruptedCursor.current?.length;
    const warningEnabled = !!warningCursor.current?.length;

    const testId = sessionDetails?.test?.id ?? testDataId;

    return <Stack style={{width: '100%', height: '100%'}}>
                {!!jobArtifacts && jobContext?.job?.id && jobContext?.step?.id &&
                    <JobArtifactsModal
                        jobId={jobContext.job.id} stepId={jobContext?.step?.id} contextType={jobArtifacts.type} artifacts={[jobArtifacts]}
                        onClose={() => {
                            setJobArtifacts(undefined);
                            handler.removeSearchParam('artifactPath');
                        }}/>
                }
                {!sessionDetails && handler.testDataQueryLoading &&
                        <Stack horizontalAlign='center' style={{ padding: 12 }} tokens={{ childrenGap: 10 }}>
                            <Text style={styles.textLarge}>Loading Data</Text>
                            <Spinner size={SpinnerSize.large} />
                        </Stack>
                }
                {!sessionDetails && !handler.testDataQueryLoading &&
                    <Stack grow tokens={{childrenGap: 3}} style={{height: '100%'}}>
                        <Stack horizontal verticalAlign="start" tokens={{childrenGap: 10}}>
                            <Stack grow />
                            <Stack style={{ paddingBottom: 4 }}>
                                <IconButton
                                    iconProps={{ iconName: 'Cancel' }}
                                    ariaLabel="Close phase view"
                                    onClick={onDismiss}
                                />
                            </Stack>
                        </Stack>
                        <Stack horizontalAlign='center' style={{ padding: 12 }}>
                            <Text style={styles.textLarge}>No Data</Text>
                        </Stack>
                    </Stack>
                }
                {!!sessionDetails &&
                    <Stack grow tokens={{childrenGap: 3}} style={{height: '100%'}}>
                        <Stack horizontal verticalAlign="start" tokens={{childrenGap: 10}}>
                            <Stack style={{ paddingLeft: 8}} tokens={{childrenGap: 5}}>
                                <Text style={{ fontSize: 16, fontWeight: 600, whiteSpace: 'nowrap', overflow: 'hidden', textOverflow: 'ellipsis' }}>{`${sessionDetails.name} - ${metaString}`}</Text>
                                <Stack style={{ paddingLeft: 8 }} horizontal tokens={{childrenGap: 8}}>
                                    <Text>{projectStore.streamById(sessionDetails.streamId)?.fullname ?? sessionDetails.streamId}</Text>
                                    <ChangeButton job={jobContext?.job} rangeCL={jobContext?.rangeCL} />
                                    <Link href={`/job/${jobContext?.job?.id}?step=${jobContext?.step?.id}`} target="_blank"><Text style={{ fontSize: 12, whiteSpace: 'nowrap', color: hordeTheme.semanticColors.link }}>View Job</Text></Link>
                                    <Text style={{whiteSpace: 'nowrap'}}><span style={styles.labelSmall}>Run on: </span><span>{getShortNiceTime(sessionDetails.start, true, true, true)}</span></Text>
                                    <Text style={{whiteSpace: 'nowrap'}}><span style={styles.labelSmall}> For: </span><span>{msecToElapsed(sessionDetails.duration * 1000, true, true)}</span></Text>
                                    <Stack horizontalAlign="end">
                                        <DefaultButton style={{ fontSize: 11, padding: 0, height: 18, backgroundColor: styles.defaultButton.backgroundColor, color: 'white' }}
                                            onClick={(ev) => {
                                                ev.stopPropagation();                                                
                                                sessionDetails.artifacts?.getJobStepSavedArtifacts().then(value => setJobArtifacts(value));
                                            }}>Artifacts</DefaultButton>
                                    </Stack>
                                    {!!sessionDetails.test && !!handler.stream &&
                                        <Link style={{padding: '0px 5px'}} title="open test health view" onClick={() => {
                                                handler.setSearchParam('view', TestViewType.Health);
                                                handler.setSearchParam('health', sessionDetails.test!.id);
                                                handler.removeSearchParam('session');
                                            }}>
                                            <FontIcon iconName={testViewIcons.get(TestViewType.Health)} style={{fontSize: 15, paddingRight: 4}} />
                                        </Link>
                                    }
                                </Stack>
                            </Stack>

                            <Stack grow />

                            {StatusPie}

                            <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                    <TooltipHost content={failureEnabled ? 'Navigate to next failure': ''}>
                                        <DefaultButton
                                            disabled={!failureEnabled} className={failureEnabled ? eventStyles.failureButton : eventStyles.failureButtonDisabled}
                                            style={{ color: failureEnabled ? failureButtonTextColor : buttonNoneTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: failureEnabled ? '4px 0 0 4px' : '4px' }}
                                            text={`${failureText} ${(failureCursor.current?.length ?? 0) > 1 ? "Failures" : "Failure"}`}
                                            onClick={() => failureCursor.current && handler.setSearchParam('phase', failureCursor.current.next())} >
                                            {failureEnabled &&
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            }
                                        </DefaultButton>
                                    </TooltipHost>
                                    {failureEnabled &&
                                        <div style={{backgroundColor: failureButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    }
                                </Stack>
                                {failureEnabled &&
                                    <TooltipHost content={'Navigate to previous failure'}>
                                        <IconButton
                                            disabled={!failureEnabled} className={failureEnabled ? eventStyles.failureButton : eventStyles.failureButtonDisabled}
                                            style={{ color: failureButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => failureCursor.current && handler.setSearchParam('phase', failureCursor.current.back())} />
                                    </TooltipHost>
                                }
                            </Stack>

                            {interruptedEnabled &&
                                <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                    <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                        <TooltipHost content='Navigate to next interrupted'>
                                            <DefaultButton
                                                className={eventStyles.interruptedButton}
                                                style={{ color: interruptedButtonTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: '4px 0 0 4px' }}
                                                text={`${interruptedCursor.current?.length}${(failureCursor.current?.length ?? 0) === 0 ? " Interrupted" : ""}`}
                                                onClick={() => interruptedCursor.current && handler.setSearchParam('phase', interruptedCursor.current.next())} >
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            </DefaultButton>
                                        </TooltipHost>
                                        <div style={{backgroundColor: interruptedButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    </Stack>
                                    <TooltipHost content={'Navigate to previous interrupted'}>
                                        <IconButton
                                            className={eventStyles.interruptedButton}
                                            style={{ color: interruptedButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => interruptedCursor.current && handler.setSearchParam('phase', interruptedCursor.current.back())} />
                                    </TooltipHost>
                                </Stack>
                            }

                            {warningEnabled &&
                                <Stack horizontal verticalAlign='center' style={{ height: '100%' }}>
                                    <Stack horizontal verticalAlign='center' style={{position: 'relative'}}>
                                        <TooltipHost content='Navigate to next warning'>
                                            <DefaultButton
                                                className={eventStyles.warningButton}
                                                style={{ color: warningButtonTextColor, height: 28, whiteSpace: 'nowrap', fontSize: 12, paddingLeft: 8, paddingRight: 8, borderRadius: '4px 0 0 4px' }}
                                                text={`${warningCursor.current?.length}${(failureCursor.current?.length ?? interruptedCursor.current?.length ?? 0) === 0 ? (warningCursor.current?.length ?? 0) > 1 ? " Warnings" : " Warning" : ""}`}
                                                onClick={() => warningCursor.current && handler.setSearchParam('phase', warningCursor.current.next())} >
                                                <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />
                                            </DefaultButton>
                                        </TooltipHost>
                                        <div style={{backgroundColor: warningButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />
                                    </Stack>
                                    <TooltipHost content={'Navigate to previous warning'}>
                                        <IconButton
                                            className={eventStyles.warningButton}
                                            style={{ color: warningButtonTextColor, height: 28, fontSize: 12, padding: 2, borderRadius: '0 4px 4px 0' }}
                                            iconProps={{ iconName: 'ChevronUp' }}
                                            onClick={() => warningCursor.current && handler.setSearchParam('phase', warningCursor.current.back())} />
                                    </TooltipHost>
                                </Stack>
                            }

                            <Stack verticalAlign='center' style={{ height: '100%' }}>
                                <TooltipHost content='Filter by outcome'>
                                    <CheckListOption
                                        options={phaseFiltersContext.outcomeOptions}
                                        onChange={() => {
                                            updateFilter(phaseFiltersContext.filterTestPhases(testId, sessionDetails.phases) as TestPhaseStatus[]);
                                        }}
                                    />
                                </TooltipHost>
                            </Stack>

                            <Stack verticalAlign='center' style={{ height: '100%' }}>
                                <TooltipHost content='Filter phases by name and tag'>
                                    <PhaseFilter
                                        phases={sessionDetails.phases}
                                        selected={[...phaseFiltersContext.getFilteredNames(testId), ...phaseFiltersContext.getFilteredTags(testId)?.map(t => `#${t}`) ?? []]}
                                        onChange={(items => {
                                            phaseFiltersContext.setFilteredNames(testId, items);
                                            updateFilter(phaseFiltersContext.filterTestPhases(testId, sessionDetails.phases) as TestPhaseStatus[]);
                                        })}/>
                                </TooltipHost>
                            </Stack>

                            <Stack style={{ paddingBottom: 4 }}>
                                <IconButton
                                    iconProps={{ iconName: 'Cancel' }}
                                    ariaLabel="Close phase view"
                                    onClick={onDismiss}
                                />
                            </Stack>
                        </Stack>
                        <Stack style={{ paddingTop: 3, paddingLeft: 16, paddingBottom: 3, paddingRight: 3, overflow: !selectedPhase.current? 'auto' : 'hidden', height: '100%'}} grow>
                            {!selectedPhase.current &&
                                <List
                                    items={filteredPhases}
                                    onRenderCell={onRenderCell}
                                    getKey={(phase: TestPhaseStatus) => `phase-${phase.key}`}
                                    ref={(list: List) => listRef.current = list}
                                    onPagesUpdated={onPagesUpdated}
                                />
                            }
                            {!!selectedPhase.current &&
                                <TestPhaseSection
                                    phase={selectedPhase.current}
                                    handler={handler}
                                    onCollapse={(phaseKey) => {setFocusPhase(phaseKey)}}
                                    onNext={() => phaseCursor.current && handler.setSearchParam('phase', phaseCursor.current.next())}
                                    onBack={() => phaseCursor.current && handler.setSearchParam('phase', phaseCursor.current.back())}
                                />
                            }
                        </Stack>
                    </Stack>
                }     
            </Stack>
});