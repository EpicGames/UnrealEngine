// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, FontIcon, IconButton, mergeStyleSets, Modal, Stack, Text, HoverCard, HoverCardType, IPlainCardProps, DirectionalHint, Spinner, SpinnerSize, IComboBoxOption, Link } from "@fluentui/react";
import { useState, useEffect, useCallback, memo } from "react";
import { observer } from "mobx-react-lite";
import { TestDataHandler, TestNameRef, MetadataRef, TestSessionStatus, TestSessionResult, TestStatus, TestTagRef } from "./testData";
import { TestOutcome } from './api';
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { getShortNiceTime } from "horde/base/utilities/timeUtils";
import { getHordeStyling } from "horde/styles/Styles";
import { projectStore } from 'horde/backend/ProjectStore';
import { SessionStatusBar, SessionValues, StatusBar, StatusBarStack, MultiOptionChooser, getTestSessionStatusColor, getStatusColors, statusTexts, getTestUniqueMetaIdentifiers, StreamSelector, styles, TestViewType, testViewIcons } from "./testAutomationCommon";
import { TestHistoryGraph } from "./testHistoryGraph";
import { StoreKey, userStorage } from "./testAutomationUserStorage";

type SessionCallback = (session: TestSessionResult) => void;
type HistorySessionCallback = (test: TestNameRef, onClick: SessionCallback) => void;

export const TestHistoryGraphWidget: React.FC<{ test: TestNameRef, stream: string, handler: TestDataHandler, onClick: SessionCallback, commonMetaKeys?: string[], selectedMetaKeys?: string[]}> = ({ test, stream, handler, onClick, commonMetaKeys, selectedMetaKeys }) => {
    const [container, setContainer] = useState<HTMLDivElement | null>(null);
    const [state, setState] = useState<{ graph?: TestHistoryGraph }>({});

    useEffect(() => {
        const graph = new TestHistoryGraph(test, stream, handler, commonMetaKeys, onClick);
        setState({ graph: graph });
        return () => graph.cleanupCallback();
    }, [commonMetaKeys]);

    useEffect(() => {
        // refresh on meta selection update
        container && state.graph?.render(container, true, selectedMetaKeys);
    }, [selectedMetaKeys])

    if (!state.graph) {
        return null;
    }

    if (container) {
        try {
            state.graph?.render(container, false, selectedMetaKeys);
        } catch (err) {
            console.error(err);
        }
    }

    const { hordeClasses } = getHordeStyling();
    const graph_container_id = `test_history_graph_container_${test.id}_${stream}`;

    return <Stack className={hordeClasses.horde}>
                <Stack style={{ paddingLeft: 8, paddingTop: 8 }}>
                    <div id={graph_container_id} className="horde-no-darktheme" style={{ userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)}/>
                </Stack>
            </Stack>
}

export const TestHistoryView: React.FC<{ test: TestNameRef, handler: TestDataHandler, onClick: SessionCallback  }> = observer(({ test, handler, onClick }) => {

    const { hordeClasses } = getHordeStyling();
    const [streams, setStreams] = useState<{current: string, toCompare?: string}>({current: handler.stream!, toCompare: userStorage.getItem(StoreKey.HistoryToCompare, () => undefined)});
    const [selectedMeta, setSelectedMeta] = useState<Set<string>>(new Set());

    handler.subscribeToSubQueryLoading();

    const onClickStream = useCallback((targetStream: string) => {
        setStreams(prev => ({...prev, toCompare: undefined}));
        handler.query(targetStream, true)
            .catch((reason) => console.error(reason))
            .finally(() => {
                setStreams(prev => ({...prev, toCompare: targetStream}));
                userStorage.setItem(StoreKey.HistoryToCompare, targetStream);
            });
    }, []);

    const availableStreams = handler.availableStreams.filter((s) => s !== streams.current);

    // collecting metadata
    let metadataSet = new Set(handler.getStatusStream(streams.current)!.tests.get(test)!.getMetadata());
    if (!!streams.toCompare) {
        const toCompareMetadataList = handler.getStatusStream(streams.toCompare!)?.tests.get(test)?.getMetadata();
        if (!!toCompareMetadataList) {
            metadataSet = metadataSet.union(new Set(toCompareMetadataList!));
        }
    }

    const commonMetaKeys = MetadataRef.identifyCommonKeys(metadataSet.values().toArray());
    const commonMetaValues = metadataSet.values().next().value?.getCommonValues(commonMetaKeys).join(" / ");

    const filteredMetadata = new Set(handler.filteredMetadata);
    const metaOptions: IComboBoxOption[] = metadataSet.keys()
        .filter(meta => filteredMetadata.has(meta))
        .map((meta) => ({key: meta.id, text: meta.getValuesExcept(commonMetaKeys).join(' / ')}))
        .toArray().sort((a, b) => a.text.localeCompare(b.text));

    const streamViews = (!!availableStreams.length? [streams.current, streams.toCompare] : [streams.current]).map(stream => {

        const streamName = projectStore.streamById(stream)?.fullname ?? stream;

        const status = stream && handler.getStatusStream(stream)?.tests.get(test);

        return <Stack key={`test_history_view_${stream}_${test.id}`}>
                { !!stream &&
                    <Stack style={{width: 200}}>
                        { stream === streams.current &&
                            <Stack style={styles.streamBadge} horizontalAlign="center">
                                <Text variant="small">{streamName}</Text>
                            </Stack>
                        }
                        { stream !== streams.current &&
                            <StreamSelector streams={availableStreams} selected={streams.toCompare} onClick={onClickStream} />
                        }
                    </Stack>
                }
                { !!status &&
                    <Stack style={{ paddingTop: 4 }}>
                        <TestHistoryGraphWidget test={test} stream={stream} commonMetaKeys={commonMetaKeys} selectedMetaKeys={selectedMeta.size? selectedMeta.keys().toArray() : undefined} handler={handler} onClick={onClick} />
                    </Stack>
                }
                { !status && !!stream &&
                    <Stack horizontalAlign='center' style={{ padding: 4 }}>
                        <Text style={{ fontSize: 12 }}>No Results</Text>
                    </Stack>
                }
            </Stack>
    });

    let name = test.name;
    if (commonMetaValues) {
        name += ` - ${commonMetaValues}`
    }

    return <Stack>
            <Stack className={hordeClasses.raised}>
                <Stack>
                    <Stack horizontal tokens={{childrenGap: 10}}>
                        <Stack>
                            <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>{name}</Text>
                        </Stack>
                        <Link style={{padding: '0px 5px'}} title="open test health view" onClick={() => {
                                handler.setSearchParam('view', TestViewType.Health);
                                handler.setSearchParam('health', test.id);
                            }}>
                            <FontIcon iconName={testViewIcons.get(TestViewType.Health)} style={{fontSize: 15, paddingRight: 4}} />
                        </Link>
                    </Stack>
                </Stack>
                <Stack horizontal tokens={{childrenGap: 15}} style={{ paddingBottom: 12, paddingTop: 5 }}>
                    <Stack style={{ width: 170 }}>
                        <MultiOptionChooser style={{height: 24, fontSize: 11}} placeholder="Filter by metadata"
                            options={metaOptions} initialSelection={selectedMeta.keys().toArray()} disabled={metaOptions.length <= 1}
                            updateKeys={(keys) => setSelectedMeta(new Set(keys))} />
                    </Stack>
                </Stack>
                <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
                    {streamViews}
                    {!handler.subQueryLoading && !streams.toCompare &&
                        <Stack style={{width: 200, margin: 0}}>
                            <StreamSelector streams={availableStreams} selected={streams.toCompare} disabled={availableStreams.length === 0} onClick={onClickStream} />
                        </Stack>
                    }
                </Stack>
                {handler.subQueryLoading &&
                    <Stack horizontalAlign='center' horizontal style={{...styles.streamBadge, marginLeft: 12, cursor: 'default' }} tokens={{ childrenGap: 10 }}>
                        <Text style={{ fontSize: 12 }}>Loading Data</Text>
                        <Spinner size={SpinnerSize.small} />
                    </Stack>
                }
            </Stack>
        </Stack>
});

const TestHistoryModal: React.FC<{ test: TestNameRef, handler: TestDataHandler, onDismiss: () => void, onDetailsClick: (test: TestSessionResult) => void }> = ({ test, handler, onDismiss, onDetailsClick }) => {
    return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1134, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onDismiss()} >
            <Stack grow>
                <Stack horizontal verticalAlign="center">
                    <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>{`${test.name} History`}</Text></Stack>
                    <Stack grow />
                    <Stack style={{ paddingBottom: 4 }}>
                        <IconButton
                            iconProps={{ iconName: 'Cancel' }}
                            ariaLabel="Close popup modal"
                            onClick={() => { onDismiss(); }}
                        />
                    </Stack>
                </Stack>
                <Stack tokens={{ childrenGap: 40 }} style={{ padding: 8, overflow: "auto", maxHeight: 'calc(100vh - 200px)'} }>
                    <TestHistoryView test={test} handler={handler} onClick={onDetailsClick} />
                </Stack>
            </Stack>
        </Modal>
}

const TestSummaryCard: React.FC<{ test: TestNameRef, sessions: [MetadataRef, TestSessionResult][], metaKeyMask: string[] | undefined, handler: TestDataHandler, onDetailsClick: SessionCallback, onHistoryClick: HistorySessionCallback }> = memo(({ test, sessions, metaKeyMask, handler, onDetailsClick, onHistoryClick }) => {
    const [expanded, setExpanded] = useState(userStorage.getItem(StoreKey.SummaryExpandedCards, () => new Map<string, boolean>()).get(test) ?? false);

    const expandCard = useCallback((bExpand) => {
        userStorage.getItem(StoreKey.SummaryExpandedCards).set(test, bExpand);
        setExpanded(bExpand);
    }, [test]);
 
    const { hordeClasses, modeColors } = getHordeStyling();
    const statusColors = getStatusColors();
 
    const styles = mergeStyleSets({
       metaitem: {
            selectors: {
                ':hover': {
                    filter: dashboard.darktheme ? "brightness(120%)" : "brightness(90%)"
                }
            }
       }
    });
 
    const colorA = dashboard.darktheme ? "#181A1B" : "#e8e8e8";
    const colorB = dashboard.darktheme ? "#242729" : "#f8f8f8";
 
    const metaElements: JSX.Element[] = [];
    const onRenderStatusCard = (session: TestSessionResult): JSX.Element => {
        return <Stack key={`test_summary_hover_${test.id}`} style={{pointerEvents: 'none'}}>
                    {SessionValues(session, {fontSize: 12, padding: '8px'})}
                </Stack>
    }
 
    let testFailed = 0;
    let testUnspecified = 0;
    let testTotal = 0;
 
    sessions.sort(([, asession], [, bsession]) => bsession.commitOrder - asession.commitOrder);
    const latestCommit = sessions.reduce((latest, [, session]) => session.commitOrder > latest.order? {order: session.commitOrder, id: session.commitId} : latest, {order: 0, id: "NA"});
 
    const commonKeys = MetadataRef.identifyCommonKeys(sessions.map(([meta,]) => meta));
    const commonMetaString  = sessions.map(([meta,]) => meta).find(() => true)?.getCommonValues(commonKeys)?.join(" / ");
    const maskMetaString  = sessions.map(([meta,]) => meta).find(() => true)?.getCommonValues(metaKeyMask)?.join(" / ");
 
    sessions.forEach(([meta, session]) => {
        const metaName = meta.getValuesExcept(commonKeys).join(" / ") || (meta.getValues().findLast(() => true) ?? test.name);
    
        const sessionTotalCount = session.phasesSucceededCount + session.phasesFailedCount + session.phasesUnspecifiedCount;
        testUnspecified += session.phasesUnspecifiedCount;
        testFailed += session.phasesFailedCount;
        testTotal += sessionTotalCount;
    
        const color = getTestSessionStatusColor(session);
    
        let dateString = "";
    
        let date = handler.commitIdDates.get(session.id);      
        if (!date) {
            date = session.start;
        }               
        if (date) {
            dateString = getShortNiceTime(date, false, false)
        }
    
        let fontWeight: number | undefined;
        if (session.commitId === latestCommit.id) {
            fontWeight = 600;
        }
    
        const StatusCardProps: IPlainCardProps = {
            onRenderPlainCard: onRenderStatusCard,
            renderData: session,
            directionalHint: DirectionalHint.rightCenter,
            calloutProps: { isBeakVisible: true }
        }
    
        metaElements.push(
            <Stack key={`test_summary_button_${session.id}`} className="horde-no-darktheme" style={{ cursor: "pointer", backgroundColor: metaElements.length % 2 ? colorA : colorB, paddingTop: 2, paddingBottom: 2 }}
                onClick={(ev) => {
                    ev.stopPropagation();
                    onDetailsClick(session);
                }}>
                <HoverCard plainCardProps={StatusCardProps} type={HoverCardType.plain} cardOpenDelay={20} sticky={false}>
                    <Stack className={styles.metaitem} horizontal verticalAlign="center">
                        <Stack className="horde-no-darktheme" style={{ paddingLeft: 0, paddingTop: 1, paddingRight: 4 }}>
                            <FontIcon style={{ fontSize: 11, color: color }} iconName="Square" />
                        </Stack>
                        <Stack horizontal>
                            <Stack style={{ width: 163 }}>
                                <Text variant="xSmall" style={{ fontWeight: fontWeight, textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap' }} title={metaName}>{metaName}</Text>
                            </Stack>
                            <Stack horizontal style={{ width: 50, paddingLeft: 3 }} verticalFill verticalAlign="center" tokens={{ childrenGap: 4 }}>
                                <Text variant="xSmall" style={{ fontWeight: fontWeight }}>{session.commitId}</Text>
                                {session.commitId === latestCommit.id && <FontIcon style={{ fontSize: 11, color: color }} iconName="Star" />}
                            </Stack>
                            <Stack style={{ width: 60 }} horizontalAlign="end">
                                <Text variant="xSmall" style={{ fontWeight: fontWeight }}>{dateString}</Text>
                            </Stack>
                        </Stack>
                    </Stack>
                    {SessionStatusBar(session, 294, 8)}
                </HoverCard>
            </Stack>
        );
    });
 
    const testFailedFactor = Math.ceil(testFailed / (testTotal || 1) * 50) / 50;
    const testUnspecifiedFactor = Math.ceil(testUnspecified / (testTotal || 1) * 50) / 50;
    const testStack: StatusBarStack[] = [
        {
            value: testUnspecifiedFactor * 100,
            title: statusTexts.get(TestOutcome.Unspecified),
            color: statusColors.get(StatusColor.Unspecified)!,
            stripes: true
        },
        {
            value: testFailedFactor * 100,
            title: statusTexts.get(TestOutcome.Failure),
            color: statusColors.get(StatusColor.Failure)!,
            stripes: true
        }
    ]
 
    return <Stack className={hordeClasses.raised} style={{ cursor: "pointer", height: "fit-content", backgroundColor: dashboard.darktheme ? "#242729" : modeColors.background, padding: 12, width: 335 }} onClick={() => expandCard(!expanded)}>
            <Stack horizontal verticalAlign="center" >
                <Stack className="horde-no-darktheme" style={{ paddingTop: 1, paddingRight: 4 }}>
                    <FontIcon style={{ fontSize: 13 }} iconName={expanded ? "ChevronDown" : "ChevronRight"} />
                </Stack>
                <Stack>
                    <Text style={{maxWidth: 240, fontWeight: 600, textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap' }}  title={`${test.name} - ${commonMetaString}`}>{test.name}{maskMetaString && <span style={{fontSize: 11}}> - {maskMetaString}</span>}</Text>
                </Stack>
        
                <Stack grow />        
        
                <Stack horizontalAlign="end">
                    <DefaultButton style={{ minWidth: 25, fontSize: 10, padding: 1, height: 20 }} title="Grid Phase view" onClick={(ev) => {
                        ev.stopPropagation();
                        handler.setSearchParam('grid', test.id);
                    }}><FontIcon style={{ fontSize: 15 }} iconName="TimelineMatrixView"/></DefaultButton>
                </Stack>

                <Stack horizontalAlign="end">
                    <DefaultButton style={{ minWidth: 25, fontSize: 10, padding: 1, height: 20 }} title="History view" onClick={(ev) => {
                        ev.stopPropagation();
                        onHistoryClick(test, onDetailsClick);
                    }}><FontIcon style={{ fontSize: 15 }} iconName="Timeline"/></DefaultButton>
                </Stack>
            </Stack>
            <Stack style={{ paddingTop: 4 }}>
                {StatusBar(testStack, 308, 10, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
                <Stack horizontal style={{marginLeft: '5px'}} tokens={{childrenGap: 3}}>                    
                    {!!testUnspecified && <Text variant="xSmall" style={{color: statusColors.get(StatusColor.Unspecified), minWidth: `${testUnspecifiedFactor*100}%`}} title={`${testUnspecified} ${statusTexts.get(TestOutcome.Unspecified)}`}> {testUnspecified}</Text>}
                    {!!testFailed && <Text variant="xSmall" style={{color: statusColors.get(StatusColor.Failure), minWidth: `${testFailedFactor*100}%`}} title={`${testFailed} ${statusTexts.get(TestOutcome.Failure)}`}> {testFailed}</Text>}
                </Stack>
            </Stack>        
            {!!expanded &&
                <Stack style={{ paddingLeft: 8 }} tokens={{ childrenGap: 4 }}>
                    {metaElements}
                </Stack>
            }
            </Stack>
});

const filterSessionsWithMetaMask = (testStatus: TestStatus, filteredTests: TestNameRef[], selectedTags: TestTagRef[] | undefined, handler: TestDataHandler): [[TestNameRef, [MetadataRef, TestSessionStatus][]][], [TestNameRef, [MetadataRef, TestSessionStatus][]][], Map<string, string[]>] => {
    const filteredMetadata = handler.filteredMetadata;

    const testMeta: Map<string, [TestNameRef, MetadataRef[]][]> = new Map();

    const failing: [TestNameRef, [MetadataRef, TestSessionStatus][]][] = [];
    const passing: [TestNameRef, [MetadataRef, TestSessionStatus][]][] = [];
    const testFailureCount: Map<TestNameRef, number> = new Map();

    filteredTests.forEach(test => {
        // sort by failed
        const status = testStatus!.tests.get(test)!;
        let totalFailureCount = 0;

        const metaRefs: MetadataRef[] = [];
        const testRefMeta: [TestNameRef, MetadataRef[]][] = testMeta.get(test.name) ?? testMeta.set(test.name, []).get(test.name)!;
        testRefMeta.push([test, metaRefs]);

        const metaResults: [MetadataRef, TestSessionStatus][] = [];

        filteredMetadata.forEach(meta => {
            const metaStatus = status.sessions.get(meta);
            if (!metaStatus) return;

            const session = metaStatus.getLastSession();
            if (!session) return;

            if (!!selectedTags && !metaStatus.includeTags(selectedTags)) {
                // skip if it filters by tags and no tag match
                return;
            }

            const outcome = session.outcome;
            if ( outcome === TestOutcome.Failure || outcome === TestOutcome.Unspecified ) {
                totalFailureCount += session.phasesFailedCount + session.phasesUnspecifiedCount;
            }

            metaResults.push([meta, metaStatus]);
            metaRefs.push(meta);
        });

        if (totalFailureCount > 0) {
            failing.push([test, metaResults]);
            testFailureCount.set(test, totalFailureCount);
        } else {
            passing.push([test, metaResults]);
        }
    });

    const testMetaKeys = getTestUniqueMetaIdentifiers(testMeta);

    failing.sort(([testA, ], [testB, ]) => testFailureCount.get(testB)! - testFailureCount.get(testA)!);

    return [failing, passing, testMetaKeys];
}

export const TestSummary: React.FC<{ handler: TestDataHandler }> = ({ handler }) => {
    const [historyState, setHistoryState] = useState<{test?: TestNameRef, onClick?: SessionCallback}>({});
    const searchTestId = handler.getSearchParam('history');
    if (!searchTestId) {
        historyState.test = historyState.onClick = undefined;
    }
    
    const onDetailsClick = useCallback((session: TestSessionResult) => {
        if (session.testDataId) {
            handler.setSearchParam('session', session.testDataId);
        }
        else if (session.jobId && session.stepId) {
            window.open(`/job/${session.jobId}?step=${session.stepId}`, '_blank');
        }
    }, []);

    const testStatus = handler.selectedStatusStream;
    if (!testStatus?.tests.size) {
        return null;
    }

    const filteredTests = handler.filteredTests;
    if (!filteredTests.length) {
        return null;
    }

    const selectedTags = handler.selectedTags;

    const [failingTestSessions, passingTestSessions, testMetaKeys] = filterSessionsWithMetaMask(testStatus, filteredTests, selectedTags, handler);

    const testSummaryTemplate =([test, sessions]) => {

        if (searchTestId && !historyState.test && test.id === searchTestId) {
            historyState.test = test;
            historyState.onClick = onDetailsClick;
        }

        return <TestSummaryCard key={`test_summary_${test.id}`}
                    test={test}
                    sessions={sessions.map(([meta, status]) => [meta, status.getLastSession()] as [MetadataRef, TestSessionResult])}
                    metaKeyMask={testMetaKeys.get(test.name)} handler={handler}
                    onHistoryClick={(test: TestNameRef, onClick: SessionCallback) => {
                        setHistoryState({test: test, onClick: onClick});
                        handler.setSearchParam('history', test.id);
                    }}
                    onDetailsClick={onDetailsClick}
                />

    }

    const failingTests = failingTestSessions.map(testSummaryTemplate);
    const passingTests = passingTestSessions.map(testSummaryTemplate);

    const { hordeClasses } = getHordeStyling();

    return <Stack>
            {!!historyState.test &&
                <TestHistoryModal
                    test={historyState.test}
                    handler={handler}
                    onDismiss={() => {
                        setHistoryState({});
                        handler.removeSearchParam('history');
                    }}
                    onDetailsClick={historyState.onClick!}
                />
            }
            <Stack tokens={{childrenGap: 12}}>
                {failingTests.length > 0 &&
                    <Stack className={hordeClasses.raised}>
                        <Stack style={{ paddingBottom: 12 }}>
                            <Stack>
                                <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>Latest Failing Test Results</Text>
                            </Stack>
                        </Stack>
                        <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
                            <Stack wrap horizontal style={{ width: "100%" }} tokens={{ childrenGap: 12 }}>
                                {failingTests}
                            </Stack>
                        </Stack>
                    </Stack>
                }
                {passingTests.length > 0 &&
                    <Stack className={hordeClasses.raised}>
                        <Stack style={{ paddingBottom: 12 }}>
                            <Stack>
                                <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>Latest Passing Test Results</Text>
                            </Stack>
                        </Stack>
                        <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
                            <Stack wrap horizontal style={{ width: "100%" }} tokens={{ childrenGap: 12 }}>
                                {passingTests}
                            </Stack>
                        </Stack>
                    </Stack>
                }
            </Stack>
        </Stack>
}