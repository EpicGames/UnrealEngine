// Copyright Epic Games, Inc. All Rights Reserved.

class TransientStorage {
    constructor() {
        this.blackboard = new  Map<string, any>();
    }

    getItem(key: string, initializer?: () => any): any {
        if (!this.blackboard.has(key) && !!initializer) {
            this.blackboard.set(key, initializer());
        }

        return this.blackboard.get(key);
    }

    setItem(key: string, value: any) {
        this.blackboard.set(key, value);
    }

    private blackboard: Map<string, any>;
}

export const userStorage = new TransientStorage();

export enum StoreKey {
    HistoryToCompare = "historyToCompare",
    SummaryExpandedCards = "summaryExpandedCards",
    HealthDetailsTransaction = "healthDetailsTransaction"
}

