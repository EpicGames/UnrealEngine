// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, FontIcon, IComboBoxOption, IContextualMenuItem, IContextualMenuProps, Text, Spinner, SpinnerSize, Label, DefaultButton, ITag, TagPicker, IPickerItemProps, PrimaryButton, Modal, IconButton, TooltipHost, DirectionalHint, PivotItem, Pivot } from "@fluentui/react";
import { Breadcrumbs } from "horde/components/Breadcrumbs";
import { TopNav } from "horde/components/TopNav";
import { getHordeStyling } from "horde/styles/Styles";
import { observer } from "mobx-react-lite";
import { useNavigate, useSearchParams } from "react-router-dom";
import React, { useEffect, useState, useCallback, useMemo } from "react";
import { projectStore } from 'horde/backend/ProjectStore';
import { TestDataHandler, TestDataVersionRegistrar, TestSessionDetails } from "./testData";
import { TestSummary } from "./testAutomationSummary";
import { getStatusColors, getStreamOptions, getTestSessionStatusColor, MultiOptionChooser, SessionStatusBar, tagColors, testViewIcons, TestViewType } from "./testAutomationCommon";
import { TestPhasesView } from "./testAutomationPhaseView";
import { JobStepTestDataItem } from "./api";

import { TestDataV2 } from "./testDataV2Fetcher";
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { TestAutomationHealth } from "./testAutomationHealth";
import { TestPhasesGridView } from "./testAutomationPhaseGridView";

const handler = new TestDataHandler();

// register TestDataV2 fetcher
TestDataVersionRegistrar.register(new TestDataV2);

const StreamChooser: React.FC = observer(() => {

    handler.subscribeToFilter();
    handler.subscribeToSearch();

    const stream = handler.filterState.stream;

    const menuProps: IContextualMenuProps = useMemo(() => getStreamOptions(handler.availableStreams, (stream) => handler.selectStream(stream), stream), [stream]);
    
    return <Stack style={{ paddingTop: 12, paddingBottom: 4 }} horizontal>
            <Label style={{ paddingTop: 5, paddingRight: 5 }}>Stream</Label>
            <DefaultButton style={{ width: 270, textAlign: "left", height: 30 }} menuProps={menuProps} text={stream ? projectStore.streamById(stream)?.fullname ?? stream : "Select a stream first"} />
        </Stack>   
});

const onRenderSuggestionsItem = (item: ITag) => {
    return <Stack style={{height: 24, padding: 4}}>
                <Text title={item.name} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', maxWidth: 300, fontSize: 12}}>{item.name}</Text>
            </Stack>
}

const onRenderItem = (props: IPickerItemProps<ITag>) => {
    const item = props.item;
    const isTag = item.name.startsWith("#");
    return <Stack style={{ marginTop: 2, marginLeft: 3 }} key={`picker_item_${item.name}`}>
                <PrimaryButton
                    iconProps={{ iconName: "Cancel", styles: { root: { fontSize: 12, margin: 0, padding: 0 } } }}
                    styles={{label: { textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', minWidth: 0, maxWidth: 150, margin: 0, paddingLeft: 3 }}}
                    style={{ padding: 2, paddingLeft: 5, paddingRight: 5, fontSize: 12, height: "unset", backgroundColor: isTag ? tagColors.getTagColor(getTextFromTag(item)) : undefined }}
                    text={item.name}
                    title={item.name}
                    onClick={props.onRemoveItem} />
            </Stack>;
}

const getTextFromTag = (item: ITag) => item.name;
const getTagNameFromTag = (item: ITag) => item.name.substring(1);

const TestChooser: React.FC = observer(() => {
    
    handler.subscribeToFilter();
    handler.subscribeToSearch();

    const ctests: Set<string> = new Set<string>(handler.filterState.tests ?? []);
    const streamTests = handler.allStreamTestNames;

    const [testsSelection, setTestsSelection] = useState<Set<string>>(ctests);

    useEffect(() => {
        // keep input selection in sync
        if (testsSelection.size !== ctests.size || testsSelection.keys().some(s => !ctests.has(s))) {
            setTestsSelection(ctests);
        }
    }, [handler.searchUpdated]);

    const testNames: ITag[] = streamTests.map(t => ({key: `@${t.toLowerCase()}`, name: `@${t}`}));
    const filterSelectedTests = useCallback((filterText: string, _: ITag[]): ITag[] => {
        filterText = filterText.trim();
        if (filterText.length < 2) return [];
        const lowerText = filterText.toLowerCase();
        const closestNames = testNames.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray();
        const isTagExist = closestNames.find((tag) => tag.key === lowerText);
        if (!isTagExist) closestNames.splice(0, 0, {key: lowerText, name: filterText});
        return closestNames;
    }, [handler.queryLoading, handler.searchUpdated]);

    const selectedItems: ITag[] = testsSelection.keys().toArray().map((key) => ({key: key.toLowerCase(), name: key} as ITag));

    return <Stack style={{ paddingTop: 12, paddingBottom: 4 }} horizontal>
            <Label style={{ paddingTop: 5, paddingRight: 5 }}>Tests</Label>
            <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }} grow>
                <TagPicker
                    disabled={!streamTests.length}
                    styles={{ input: { height: 28, width: 40 }, itemsWrapper: { marginRight: 3, marginBottom: 2 } }}
                    onRenderItem={onRenderItem}
                    onRenderSuggestionsItem={onRenderSuggestionsItem}
                    removeButtonAriaLabel="Remove"
                    selectionAriaLabel="Selected tests"
                    selectedItems={selectedItems}
                    onResolveSuggestions={filterSelectedTests}
                    getTextFromItem={getTextFromTag}
                    onChange={(tags) => {
                        if (!!tags) {
                            const keys = tags!.map(getTextFromTag);
                            keys.forEach((key) => handler.addTest(key));
                            testsSelection.difference(new Set(keys)).values().forEach(key => handler.removeTest(key));
                        } else {
                            testsSelection.values().forEach(key => handler.removeTest(key));
                        }
                        setTestsSelection(new Set(handler.filterState.tests ?? []));
                    }}
                />                
            </Stack>
        </Stack>
});

const TagChooser: React.FC = observer(() => {
    
    handler.subscribeToFilter();
    handler.subscribeToSearch();

    const ctags: Set<string> = new Set<string>(handler.filterState.tags ?? []);
    const streamTags = handler.allStreamTestTags;

    const [tagsSelection, setTagsSelection] = useState<Set<string>>(ctags);

    useEffect(() => {
        // keep input selection in sync
        if (tagsSelection.size !== ctags.size || tagsSelection.keys().some(s => !ctags.has(s))) {
            setTagsSelection(ctags);
        }
    }, [handler.searchUpdated]);

    const testTags: ITag[] = streamTags.map(t => ({key: t.toLowerCase(), name: `#${t}`}));
    const filterSelectedTags = useCallback((filterText: string, _: ITag[]): ITag[] => {
        filterText = filterText.trim();
        if (filterText.startsWith("#")) filterText = filterText.substring(1);
        if (filterText.length < 2) return [];
        const lowerText = filterText.toLowerCase();
        const closestTags = testTags.values().filter((tag) => (tag.key as string).indexOf(lowerText) >= 0).take(10).toArray();
        return closestTags;
    }, [handler.queryLoading, handler.searchUpdated]);

    const selectedItems: ITag[] = tagsSelection.keys().toArray().map((key) => ({key: key.toLowerCase(), name: `#${key}`} as ITag));

    return <Stack style={{ paddingTop: 12, paddingBottom: 4 }} horizontal>
            <Label style={{ paddingTop: 5, paddingRight: 5 }}>Tags</Label>
            <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }} grow>
                <TagPicker
                    disabled={!handler.stream}
                    styles={{ input: { height: 28, width: 40 }, itemsWrapper: { marginRight: 3, marginBottom: 2 } }}
                    onRenderItem={onRenderItem}
                    onRenderSuggestionsItem={onRenderSuggestionsItem}
                    removeButtonAriaLabel="Remove"
                    selectionAriaLabel="Selected tags"
                    selectedItems={selectedItems}
                    onResolveSuggestions={filterSelectedTags}
                    getTextFromItem={getTagNameFromTag}
                    onChange={(tags) => {
                        if (!!tags) {
                            const keys = tags!.map(getTagNameFromTag);
                            keys.forEach((key) => handler.addTag(key));
                            tagsSelection.difference(new Set(keys)).values().forEach(key => handler.removeTag(key));
                        } else {
                            tagsSelection.values().forEach(key => handler.removeTag(key));
                        }
                        setTagsSelection(new Set(handler.filterState.tags ?? []));
                    }}
                />                
            </Stack>
        </Stack>
});

enum MetaType {
    Session = "Session",
    Test = "Test",
}

const MetaKeyChooser: React.FC<{ onUpdateKey: (key: string) => void }> = observer(({ onUpdateKey }) => {

    handler.subscribeToFilter();
    handler.subscribeToSearch();

    const cmetakeys: Set<string> = new Set(handler.filterState.metadata?.keys() ?? []);
    const streamMetadataKeys = handler.allStreamMetadataKeys;

    const ctestmetakeys: Set<string> = new Set(handler.filterState.testmeta?.keys() ?? []);
    const streamTestMetaKeys = handler.allStreamTestMetaKeys;

    const options: IContextualMenuItem[] = [];
    streamMetadataKeys.forEach(a => {
        if (cmetakeys.has(a)) {
            return;
        }
        options.push({
            key: `metakey_${a}`, text: a, data: MetaType.Session, onClick: () => {
                onUpdateKey(a);
            }
        });
    });
    streamTestMetaKeys.forEach(a => {
        if (ctestmetakeys.has(a)) {
            return;
        }
        options.push({
            key: `tmetakey_${a}`, text: a.charAt(0).toUpperCase()+a.slice(1), data: MetaType.Test, onClick: () => {
                onUpdateKey(a);
            }
        });
    });
    options.sort((a, b) => a.text!.localeCompare(b.text!));

    const menuProps: IContextualMenuProps = {
        shouldFocusOnMount: true,
        subMenuHoverDelay: 0,
        items: options,
    };

    return <Stack style={{ paddingTop: 12, paddingBottom: 4 }} horizontal horizontalAlign="end" title="add a meta filter">
                <Stack style={{ padding: 5, fontSize: 18, cursor: "default" }} verticalAlign="center"><FontIcon iconName="AddTo" /></Stack>
                <DefaultButton style={{width: 150, textAlign: "left", height: 30 }} menuProps={menuProps} text="More filters" disabled={!streamMetadataKeys.length}/>
            </Stack>   
});

const MetaValueChooser: React.FC<{ metakey: string, onRemove: (key: string) => void }> = observer(({ metakey, onRemove }) => {

    handler.subscribeToFilter();
    handler.subscribeToSearch();

    const metaType = handler.allStreamTestMetaKeys.includes(metakey)? MetaType.Test : MetaType.Session;
    const state = metaType === MetaType.Test? handler.filterState.testmeta : handler.filterState.metadata;

    const cmetavalues: Set<string> = new Set(state?.get(metakey) ?? []);
    const metavalues = metaType === MetaType.Test? handler.getAllStreamTestMetaValues(metakey) : handler.getAllStreamMetadataValues(metakey);

    const addMeta = metaType === MetaType.Test? (key: string, value: string) => handler.addTestMeta(key, value) : (key: string, value: string) => handler.addMetadata(key,value);
    const removeMeta = metaType === MetaType.Test? (key: string, value?: string) => handler.removeTestMeta(key, value) : (key: string, value: string) => handler.removeMetadata(key, value);

    let options: IComboBoxOption[] = !!metavalues.length ? metavalues.map(t => { return { key: t, text: t } }) : cmetavalues.keys().toArray().map(t => { return { key: t, text: `! ${t}` } });
    const diffmetavalues = cmetavalues.difference(new Set(metavalues));
    if (!!metavalues.length && diffmetavalues.size > 0) {
        options = options.concat(diffmetavalues.keys().toArray().map(t => { return {key: t , text: `! ${t}`} })).sort((a, b) => (a.key as string).localeCompare(b.key as string));
    }
    const noMatch = !metavalues.length || (cmetavalues.size > 0 && diffmetavalues.size === cmetavalues.size);

    return <TooltipHost
            content={noMatch ? "meta key or value(s) missing in this stream" : undefined}
            directionalHint={DirectionalHint.rightCenter}>
                <Stack style={{ paddingTop: 12, paddingBottom: 4 }} horizontal>
                <Label style={{ paddingTop: 5, paddingRight: 5 }}>{metakey.charAt(0).toUpperCase()+metakey.slice(1)}</Label>
                <Stack style={{ width: 270, borderWidth: 1, borderStyle: "solid", borderColor: noMatch ? getStatusColors().get(StatusColor.Failure) : "transparent" }}>
                    <MultiOptionChooser style={{height: 28}} options={options} initialSelection={cmetavalues.keys().toArray()} disabled={!metavalues.length}
                        updateKeys={(keys) => {
                            keys.forEach(t => {
                                if (!cmetavalues.has(t)) addMeta(metakey, t);
                            });
                            cmetavalues.forEach(t => {
                                if (!keys.includes(t)) removeMeta(metakey, t);
                            });
                        }} />
                </Stack>
                <DefaultButton style={{ minWidth: 0, fontSize: 11, padding: 5, height: 30 }} title="remove filter"
                        onClick={() => {
                            if (state?.keys().some((key) => key === metakey)) {
                                removeMeta(metakey);
                            }
                            onRemove(metakey);
                        }}>
                    <FontIcon iconName="CalculatorMultiply" />
                </DefaultButton>
            </Stack>
        </TooltipHost>
});

const TestAutomationSidebarLeft: React.FC = observer(() => {

    handler.subscribeToFilter();
    let filterMetaKeys = handler.filterState.metadata?.keys().toArray() ?? [];
    filterMetaKeys = filterMetaKeys.concat(handler.filterState.testmeta?.keys().toArray() ?? []);
    filterMetaKeys.sort((a, b) => a.localeCompare(b));

    const view = (handler.getSearchParam("view") as string | undefined) ?? TestViewType.Summary;

    const [metaKeys, setMetaKeys] = useState<string[]>(filterMetaKeys);
    const { hordeClasses } = getHordeStyling();

    const updateMetaKey = useCallback((key: string) => {
        const idx = metaKeys.indexOf(key);
        if (idx < 0) {
            metaKeys.push(key);
        } else {
            metaKeys.splice(idx, 1);
        }
        setMetaKeys([...metaKeys]);
    }, [metaKeys]);

    useEffect(() => {
        // keep search filter in sync
        if (metaKeys.toString() !== filterMetaKeys.toString()) {
            setMetaKeys(filterMetaKeys);
        }
    }, [handler.searchUpdated]);

   return <Stack style={{ width: 300, paddingRight: 18 }} className={hordeClasses.modal}>
            <Pivot selectedKey={view} onLinkClick={(item) => {handler.setSearchParam("view", item?.props?.itemKey ?? TestViewType.Summary)}}>
                <PivotItem headerText="Latest Status" itemKey={TestViewType.Summary} itemIcon={testViewIcons.get(TestViewType.Summary)}>
                    <Stack key="chooser_stream">
                        <StreamChooser />
                    </Stack>
                    <Stack key="chooser_test">
                        <TestChooser />
                    </Stack>
                    <Stack key="chooser_tag">
                        <TagChooser />
                    </Stack>
                    {!!metaKeys.length && metaKeys.map((key) => <Stack key={`chooser_meta_${key}`}><MetaValueChooser metakey={key} onRemove={updateMetaKey} /></Stack>)}
                    <Stack>
                        <MetaKeyChooser key="chooser_meta" onUpdateKey={updateMetaKey} />
                    </Stack>
                </PivotItem>
                <PivotItem headerText="Health" itemKey={TestViewType.Health} itemIcon={testViewIcons.get(TestViewType.Health)}>
                    <Stack key="chooser_stream">
                        <StreamChooser />
                    </Stack>
                    <Stack key="chooser_test">
                        <TestChooser />
                    </Stack>
                    <Stack key="chooser_tag">
                        <TagChooser />
                    </Stack>
                    {!!metaKeys.length && metaKeys.map((key) => <Stack key={`chooser_meta_${key}`}><MetaValueChooser metakey={key} onRemove={updateMetaKey} /></Stack>)}
                    <Stack>
                        <MetaKeyChooser key="chooser_meta" onUpdateKey={updateMetaKey} />
                    </Stack>
                </PivotItem>
            </Pivot>
        </Stack>
});

const TestAutomationPanel: React.FC = observer(() => {

    if (handler.queryLoading) {
        return <Stack horizontalAlign='center' style={{ paddingTop: 24, width: "100%" }} tokens={{childrenGap: 8}}>
                    <Text style={{ fontSize: 24 }}>Loading Data</Text>
                    <Spinner size={SpinnerSize.large} />
                </Stack>
    }

    handler.subscribeToSearch();

    if (!handler.selectedStatusStream) {
        return null;
    }

    const status = handler.filteredTests;
    const view = handler.getSearchParam("view") as string | undefined ?? "0";

    return <Stack grow>
                { !status.length &&
                    <Stack grow horizontalAlign='center' style={{ paddingTop: 24 }}>
                        <Text style={{ fontSize: 24 }}>No Results</Text>
                    </Stack>
                }
                { !!status.length && view === "0" &&
                    <TestSummary handler={handler} />
                }
                { !!status.length && view === "1" &&
                    <TestAutomationHealth handler={handler} />
                }
            </Stack>

});

const JobStepTestsSelectionModal: React.FC<{ jobId: string, stepId: string }> = ({ jobId, stepId }) => {
    const navigate = useNavigate();
    const [tests, setTests] = useState<(JobStepTestDataItem & {details?: TestSessionDetails})[]>();

    const loadTestData = async (tests: JobStepTestDataItem[]) => {
        const testWithDetails: (JobStepTestDataItem & {details?: TestSessionDetails})[] = [];
        for (const test of tests) {
            const sessionDetails = await handler.queryTestData(test.testDataId);
            testWithDetails.push({...test, details: sessionDetails});
        }

        return testWithDetails;
    }

    useEffect(() => {
        handler.queryJobStepTestData(jobId, stepId).then(tests => {
            if (!!tests && tests.length === 1) {
                // if only one item, redirect automatically
                navigate(`?session=${tests[0].testDataId}`, {replace: true});
            } else {
                const selection = tests ?? [];
                if (selection.length > 0) {
                    loadTestData(selection).then(detailedList => setTests(detailedList));
                } else {
                    setTests(selection);
                }
            }
        })
    }, []);

    const color = dashboard.darktheme ? "rgba(255, 255, 255, 0.1)" : "rgba(0, 0, 0, 0.1)";

    return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1134, hasBeenOpened: false, top: "200px", position: "absolute" } }} onDismiss={() => navigate(-1)} >
            <Stack grow>
                <Stack horizontal verticalAlign="center">
                    <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>Test Reports</Text></Stack>
                    <Stack grow />
                    <Stack style={{ paddingBottom: 4 }}>
                        <IconButton
                            iconProps={{ iconName: 'Cancel' }}
                            ariaLabel="Close popup modal"
                            onClick={() => navigate(-1)}
                        />
                    </Stack>
                </Stack>
                <Stack tokens={{ childrenGap: 9 }} style={{ padding: 24, overflow: "auto", maxHeight: 'calc(100vh - 300px)'}}>
                    {!tests &&
                        <Stack horizontalAlign='center' style={{ paddingTop: 24, width: "100%" }}>
                            <Spinner size={SpinnerSize.large} />
                        </Stack>
                    }
                    {!!tests && tests.length &&
                        tests.map((test, i) =>  <Stack key={`${test.testKey}-${i}`} tokens={{childrenGap: 15}} horizontal verticalAlign="end"
                                                styles={{root: {borderBottom: `1px solid ${color}`, padding: '8px 2px', cursor: 'pointer', selectors: {':hover': {backgroundColor: color}}}}}
                                                onClick={() => navigate(`?session=${test.testDataId}`)}>
                                                <Stack horizontal verticalAlign="center">
                                                    {!!test.details &&
                                                        <Stack style={{ paddingLeft: 0, paddingTop: 1, paddingRight: 4 }}>
                                                            <FontIcon style={{ fontSize: 11, color: getTestSessionStatusColor(test.details) }} iconName="Square" />
                                                        </Stack>
                                                    }
                                                    <Stack>
                                                        <Text style={{ whiteSpace: 'nowrap' }} >{test.testName}{!!test.details?.meta && <span> - {test.details.meta.getValues().join(' / ')}</span>}</Text>                                                        
                                                    </Stack>
                                                </Stack>
                                                <Stack>{!!test.details && SessionStatusBar(test.details, 200, 10)}</Stack>
                                            </Stack>)
                    }
                    {!!tests && tests.length === 0 &&
                        <Text>No report found</Text>
                    }
                </Stack>
            </Stack>
        </Modal>
}

export const TestAutomationView: React.FC = () => {
   const navigate = useNavigate();

    const [searchParams, setSearchParams] = useSearchParams();
    const [initialized, setInitialized] = useState(false);

    const testDataId = searchParams.get('session');
    const testId = searchParams.get('grid');
    const jobId = searchParams.get('job');
    const stepId = searchParams.get('step');

    useEffect(() => {
        handler.initialize(searchParams.toString(), (search: string, replace?: boolean) => {setSearchParams(search, {replace: replace})}).then(() => {setInitialized(true)});
        return () => {
            handler.clear();
            setInitialized(false); // for dev env
        };
    }, []);

    useEffect(() => {
        initialized && handler.syncSearchParam(searchParams.toString());
    }, [searchParams]);

    useEffect(() => {
        const allTags = handler.allStreamTestTags;
        tagColors.setTagColors(allTags);
    }, [handler.stream]);

    const { hordeClasses, modeColors } = getHordeStyling();

    const testAutomationHubDocs = "/docs/Config/TestAutomationHub.md";

    return <Stack className={hordeClasses.horde} key="key_test_automation_hub">
                <TopNav />
                <Breadcrumbs items={[{ text: 'Test Automation Hub' }]} />
                <Stack grow style={{ padding: 12, backgroundColor: modeColors.background, height: 'calc(100vh - 136px)', overflow: 'auto'} } horizontalAlign='center'>
                    {initialized && !!jobId && !!stepId &&
                        <JobStepTestsSelectionModal jobId={jobId} stepId={stepId} />
                    }
                    {initialized && !!testDataId &&
                        <TestPhasesView testDataId={testDataId} handler={handler}
                            onDismiss={() => {
                                handler.removeSearchParam('session');
                                handler.removeSearchParam('phase');
                            }} />
                    }
                    {initialized && !!handler.stream && !!testId && !testDataId &&
                        <TestPhasesGridView testId={testId} handler={handler}
                            onDismiss={() => {
                                handler.removeSearchParam('grid');
                                handler.removeSearchParam('metagrid');
                                handler.removeSearchParam('gridfilter');
                            }} />
                    }
                    {initialized && !testDataId && !testId  && !!handler.availableStreams.length &&
                        <Stack horizontal grow>
                            <TestAutomationSidebarLeft/>
                            <Stack style={{ width: 'calc(100vw - 350px)', height: 'calc(100vh - 162px)', overflow: 'auto' }} grow>
                                <TestAutomationPanel/>
                            </Stack>
                        </Stack>
                    }
                    {initialized && handler.loaded && !testDataId && !testId && !handler.availableStreams.length &&
                        <Stack horizontal style={{ width: 1440, paddingTop: 30 }} tokens={{ childrenGap: 6 }} horizontalAlign="center">
                            <Text variant="mediumPlus">No test metadata found, for more information please see</Text>
                            <a href={testAutomationHubDocs} style={{ fontSize: "18px", "cursor": "pointer" }}
                                onClick={(ev) => {
                                    ev.preventDefault();
                                    ev.stopPropagation();
                                    navigate(testAutomationHubDocs);
                                }}> test automation hub documentation.</a>
                        </Stack>
                    }
                </Stack>
            </Stack>
}