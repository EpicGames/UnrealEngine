// Copyright Epic Games, Inc. All Rights Reserved.

import { isNumber } from '@datadog/browser-core';
import { action, makeObservable, observable, reaction, when } from 'mobx';
import { JobStepTestDataItem, TestOutcome, TestPhaseOutcome } from './api';
import { projectStore } from 'horde/backend/ProjectStore';
import backend from 'horde/backend';
import { GetArtifactResponse } from 'horde/backend/Api';
import { EventEntry } from './testEventsModel';
import { IPhaseFilter } from './testAutomationCommon';

export const defaultQueryWeeks = 2;
const filterRefreshMS = 200;
export const consecutivePassesThreshold = 4;

type StreamId = string;
type TestId = string;
type MetaId = string;
type TagId = string;
type TestSessionId = string;
type TestDataId = string;
type PhaseKey = string;
type TestRecipeId = string;
type ErrorFingerprintId = string;
type WorkflowId = string;

// TestData Version registration
export interface TestVersionFetcher {

    get version(): number;

    /// fetch stream information and populate stream map
    fetchStreamsInfo(streamMap: Map<string, TestStreamRef>): Promise<void>;

    /// fetch test sessions and populate test status session
    fetchTestSessions(streamRef: TestStreamRef, status: TestStatus, weeks: number, onAdd?: (session: TestSessionResult) => void): Promise<void>;

    /// fetch test data and run the test session details
    fetchTestData(testDataId: TestDataId): Promise<TestSessionDetails | undefined>;

    /// fetch test phase sessions and populate test status phase session
    fetchPhaseSessions(streamRef: TestStreamRef, status: TestMetaStatus, phaseKeys: string[], weeks: number): Promise<boolean>;

    /// fetch test phases and populate test status phase
    fetchTestPhases(status: TestMetaStatus, phaseKeys?: string[]): Promise<void>;

    /// fetch test data information from job and step id
    fetchJobStepTestData(jobId: string, stepId: string): Promise<JobStepTestDataItem[] | undefined>;

    /// fetch test recipes and attached them to their associated test
    fetchTestRecipes(testRef: TestNameRef, recipeIds: TestRecipeId[]): Promise<void>;

    /// fetch test audits associated with a list of tests
    fetchTestAudit(testRefs: TestNameRef[], auditMap: Map<TestNameRef, TestAuditRecords>, weeks?: number): Promise<void>;

}

export class TestDataVersionRegistrar {

    private static _manifest: TestVersionFetcher[] = [];

    static register(fetcher: TestVersionFetcher) {
        if (TestDataVersionRegistrar._manifest.find(f => f.version === fetcher.version)) {
            console.warn(`TestData fetcher version ${fetcher.version} already registered`);
        } else {
            TestDataVersionRegistrar._manifest.push(fetcher);
        }
    }

    static get manifest(): ReadonlyArray<TestVersionFetcher> {
        return TestDataVersionRegistrar._manifest;
    }

}

export class PhaseSessionResult {
    constructor(id: string, start: Date, duration: number, commitId: string, commitOrder: number, sessionId: string, phaseRef: string, metadataId: string, streamId: string, outcome: TestPhaseOutcome) {
        this.id = id;
        this.sessionId = sessionId;
        this.commitId = commitId;
        this.commitOrder = commitOrder;
        this.start = start;
        this.duration = duration;
        this.streamId = streamId;
        this.metadataId = metadataId;
        this.phaseRef = phaseRef;
        this.outcome = outcome;
    }

    id: string;
    streamId: string;
    metadataId: string;
    sessionId: string;
    commitId: string;
    commitOrder: number;
    duration: number;
    start: Date;
    phaseRef: string;
    outcome: TestPhaseOutcome;
    jobId?: string;
    stepId?: string;
    hasWarning?: boolean;
    errorFingerprint?: ErrorFingerprintId;
    tagIds?: TagId[];
}

export class TestSessionResult {
    constructor(id: string, start: Date, duration: number, commitId: string, commitOrder: number, nameRef: string, metadataId: string, streamId: string, outcome: TestOutcome, testDataId?: string) {
        this.id = id;
        this.start = start;
        this.duration = duration;
        this.commitId = commitId;
        this.commitOrder = commitOrder;
        this.nameRef = nameRef;
        this.metadataId = metadataId;
        this.streamId = streamId;
        this.outcome = outcome;
        this.testDataId = testDataId;
        this.phasesSucceededCount = 0;
        this.phasesFailedCount = 0;
        this.phasesUnspecifiedCount = 0;
    }

    get artifacts(): ArtifactFactory | undefined {
        if (!this._artifactFactory && this.jobId && this.stepId) {
            this._artifactFactory = new ArtifactFactory(this.jobId, this.stepId);
        }
        return this._artifactFactory;
    }

    id: TestSessionId;
    start: Date;
    duration: number;
    commitId: string;
    commitOrder: number;
    nameRef: TestId;
    metadataId: MetaId;
    streamId: StreamId
    outcome: TestOutcome;
    phasesSucceededCount: number;
    phasesFailedCount: number;
    phasesUnspecifiedCount: number;
    jobId?: string;
    stepId?: string;
    testDataId?: string;
    tagIds?: TagId[];
    recipeId?: TestRecipeId;
    errorFingerprints?: Map<ErrorFingerprintId, PhaseKey[]>;

    private _artifactFactory?: ArtifactFactory;
}

export class TestSessionStatus {
    constructor(test: TestNameRef) {
        this.test = test;
        this.history = [];
        this.phaseHistories = new Map();
    }

    addSession(session: TestSessionResult) {
        this.history.push(session);
    }

    getLastSession(): TestSessionResult | undefined {
        if (!this.history.length) {
            return;
        }
        return this.history[0];
    }

    getPreviousSession(session: TestSessionResult): TestSessionResult | undefined {
        const index = this.history.indexOf(session);
        if (index === -1) return;
        if (index + 1 === this.history.length) return;

        return this.history[index + 1];
    }

    sortSessions() {
        /// sort session history by commit order descending
        this.history.sort((a, b) => {
            if (b.commitOrder !== a.commitOrder) {
                return b.commitOrder - a.commitOrder;
            }
            // if commitOrder is equal, sort by date descending
            return b.start.getTime() - a.start.getTime();
        });
    }

    addPhaseSession(phaseKey: PhaseKey, session: PhaseSessionResult) {
        if (!this.phaseHistories.has(phaseKey)) {
            this.phaseHistories.set(phaseKey, []);
        }
        this.phaseHistories.get(phaseKey)!.push(session);
    }

    getPhaseSessions(phaseKey: PhaseKey): PhaseSessionResult[] | undefined {
        if (!this.phaseHistories.get(phaseKey)?.length) {
            return;
        }
        return this.phaseHistories.get(phaseKey)!;
    }

    sortPhaseSessions(phaseId: string) {
        /// sort phase session history by commit order descending, then by date descending
        this.phaseHistories.get(phaseId)?.sort((a, b) => {
            if (b.commitOrder !== a.commitOrder) {
                return b.commitOrder - a.commitOrder;
            }
            // if commitOrder is equal, sort by date descending
            return b.start.getTime() - a.start.getTime();
        });
    }

    includeTags(tags: TestTagRef[]): boolean {
        const sessionTagIds = this.getLastSession()?.tagIds;
        return !!sessionTagIds && tags.some(tag => sessionTagIds.some(tagId => tag.id === tagId));
    }

    history: TestSessionResult[];
    phaseHistories: Map<PhaseKey, PhaseSessionResult[]>;
    test: TestNameRef;
}

export type Metadata = { [Key in string]: string | undefined };
export type MetadataFilter = Map<string, string[]>;

// query states
type TestDataState = {
    stream?: string;
    tests?: string[];
    tags?: string[];
    metadata?: MetadataFilter;
    testmeta?: MetadataFilter;
    weeks?: number;
}

export class TestTagRef {
    constructor(id: string, name: string, version: number) {
        this.id = id;
        this.name = name;
        this.version = version;
        this.lowerName = this.name.toLowerCase();
    }

    isMatch(toMatch: string): boolean {
        return toMatch === this.lowerName;
    }

    id: TagId;
    name: string;
    version: number;

    private lowerName: string;
}

export class TestPhaseRef {
    constructor(id: string, key: string, name: string, version: number) {
        this.id = id;
        this.key = key;
        this.name = name;
        this.version = version;
        this.lowerName = this.name.toLowerCase();
    }

    isMatch(toMatch: string): boolean {
        return toMatch.charAt(0) === '@'? this.lowerName === toMatch.substring(1) : this.lowerName.includes(toMatch);
    }

    updateTags(tags: TestTagRef[]) {
        const newTags = new Set<TestTagRef>(tags);
        this.tagRefs = this.tagRefs? this.tagRefs.union(newTags) : newTags;
    }

    includesTags(toMatch: string[]): boolean {
        return toMatch.every(tag => this.tagRefs?.values().some(r => r.isMatch(tag)));
    }

    id: string;
    name: string;
    key: string;
    version: number;
    tagRefs?: Set<TestTagRef>;

    private lowerName: string;
}

export class DeviceRef {
    constructor(name: string, logPath?: string, metadata?: Metadata, version?: number) {
        this.name = name;
        this.version = version ?? 0;
        this.logPath = logPath;
        this.metadata = metadata ?? {};
    }

    isSame(other: DeviceRef): boolean {

        if (this.name === other.name)
        {
            const metaKeys = new Set(Object.keys(this.metadata));
            const otherMetaKeys = new Set(Object.keys(other.metadata));
            if (metaKeys.size === otherMetaKeys.size && metaKeys.difference(otherMetaKeys).size === 0) {
                const metaValues = new Set(Object.values(this.metadata));
                const otherMetaValues = new Set(Object.values(other.metadata));
                return metaValues.difference(otherMetaValues).size === 0;
            }
        }

        return false;
    }

    name: string;
    version: number;
    logPath?: string;
    metadata: Metadata;
}

export abstract class TestPhaseStatus implements IPhaseFilter {
    constructor (key: string, name: string, start: Date, duration: number, devices: DeviceRef[], outcome: TestPhaseOutcome, sessionDetails: TestSessionDetails) {
        this.key = key;
        this._name = name;
        this._start = start;
        this._duration = duration;
        this.devices = devices;
        this.outcome = outcome;
        this._sessionDetails = sessionDetails;
    }

    associateSession(phaseSessions?: PhaseSessionResult[]) {
        if (!this.testSession) return;
        this.session = phaseSessions?.find(s => s.sessionId === this.testSession!.id);
    }

    isMatch(names: string[], outcomes?: Set<TestPhaseOutcome>, tags?: string[]) {
        if (!!outcomes && !outcomes.has(this.outcome)
            && (!outcomes.has(TestPhaseOutcome.Warning) || this.outcome !== TestPhaseOutcome.Success || !this.hasWarning)) /** allow filtering of success but with warning */
                return false;
        const lowerTags = this.tags?.map(t => t.toLowerCase());
        if (!!tags && (!lowerTags || !lowerTags.some((t) => tags.includes(t)))) return false;
        if (names.length === 0) return true;

        const lowerName = this.name.toLowerCase();
        return names.some(i => i.charAt(0) === '@'? lowerName === i.substring(1) : lowerName.includes(i));
    }

    get name() {
        return this.phaseRef?.name ?? this._name;
    }

    get commitId() {
        return this._sessionDetails.commitId;
    }

    get testSession() {
        return this._sessionDetails.session;
    }

    get artifacts() {
        return this._sessionDetails.artifacts;
    }

    get testMetadata() {
        return this._sessionDetails.meta;
    }

    get test() {
        return this._sessionDetails.test;
    }
    get start() {
        return this.session?.start ?? this._start;
    }

    get duration() {
        return this.session?.duration ?? this._duration;
    }

    abstract fetchEventStream():  Promise<EventEntry[] | undefined>;

    devices: DeviceRef[];
    key: PhaseKey;
    outcome: TestPhaseOutcome;

    phaseRef?: TestPhaseRef;
    session?: PhaseSessionResult;
    componentRef?: any;
    hasWarning?: boolean;
    tags?: string[];
    errorFingerprint?: string;

    private _name: string;
    private _start: Date;
    private _duration: number;
    private _sessionDetails: TestSessionDetails;
}

export class TestSessionDetails {
    constructor(key: string, name: string, start: Date, duration: number, commitId: string, commitOrder: number, streamId: string, jobId: string, stepId: string) {
        this.key = key;
        this.name = name;
        this._start = start;
        this._duration = duration;
        this.commitId = commitId;
        this.commitOrder = commitOrder
        this.streamId = streamId;
        this.jobId = jobId;
        this.stepId = stepId;
        this.devices = new Map();
        this.phases = [];
        makeObservable(this);
    }

    get start() {
        return this.session?.start ?? this._start;
    }

    get duration() {
        return this.session?.duration ?? this._duration;
    }

    get artifacts(): ArtifactFactory | undefined {
        if (!this._artifactFactory && this.jobId && this.stepId) {
            this._artifactFactory = new ArtifactFactory(this.jobId, this.stepId);
        }
        return this._artifactFactory;
    }
    
    get phasesSucceededCount(): number {
        return this.session?.phasesSucceededCount ?? this._phasesSucceededCount;
    }
    set phasesSucceededCount(value: number) {
        this._phasesSucceededCount = value;
    }

    get phasesUnspecifiedCount(): number {
        return this.session?.phasesUnspecifiedCount ?? this._phasesUnspecifiedCount;
    }
    set phasesUnspecifiedCount(value: number) {
        this._phasesUnspecifiedCount = value;
    }

    get phasesFailedCount(): number {
        return this.session?.phasesFailedCount ?? this._phasesFailedCount;
    }
    set phasesFailedCount(value: number) {
        this._phasesFailedCount = value;
    }

    get outcome(): TestOutcome {
        return this.session?.outcome ?? this._outcome;
    }
    set outcome(value: TestOutcome) {
        this._outcome = value;
    }

    key: string;
    name: string;
    commitId: string;
    commitOrder: number;
    jobId: string;
    stepId: string;
    streamId: string;
    devices: Map<string, DeviceRef>;
    phases: TestPhaseStatus[];
    
    test?: TestNameRef;
    meta?: MetadataRef;
    tags?: string[];
    preflightCommitId?: string;
    commandline?: string;

    @observable
    session?: TestSessionResult;

    private _start: Date;
    private _duration: number;
    private _artifactFactory?: ArtifactFactory;
    private _phasesSucceededCount: number;
    private _phasesUnspecifiedCount: number;
    private _phasesFailedCount: number;
    private _outcome: TestOutcome;
}

export class TestNameRef {
    constructor(id: string, key: string, name: string, version: number) {
        this.id = id;
        this.key = key;
        this.name = name;
        this.version = version;
        this.lowerKey = this.key.toLowerCase();
        this.lowerName = this.name.toLowerCase();
    }

    isMatch(toMatch: string): boolean {
        return toMatch.charAt(0) === '@'? this.lowerName === toMatch.substring(1) || this.lowerKey === toMatch.substring(1) : this.lowerName.includes(toMatch) || this.lowerKey.includes(toMatch);
    }

    updateTags(tags: TestTagRef[]) {
        const newTags = new Set<TestTagRef>(tags);
        this.tagRefs = this.tagRefs? this.tagRefs.union(newTags) : newTags;
    }

    includesTags(toMatch: string[]): boolean {
        return toMatch.some(tag => this.tagRefs?.values().some(r => r.isMatch(tag)));
    }

    isMetaMatch(metaFilter: MetadataFilter): boolean {
        // match test metadata (not session metadata)
        for (const [key, values] of metaFilter.entries()) {
            const thisValue = this[key];
            if (!thisValue) {
                // missing key
                return false;
            }
            const toValidate = values.some((v) => v.indexOf('+') > -1) ? [thisValue.toLowerCase()] : thisValue.split('+').map((v) => v.toLowerCase());
            if (!toValidate.some((v) => values.includes(v))) {
                return false;
            }
        }
        return true;
    }

    updateRecipes(recipes: TestRecipeRef[]) {
        const newRecipes = new Set<TestRecipeRef>(recipes);
        this.recipeRefs = this.recipeRefs? this.recipeRefs.union(newRecipes) : newRecipes;
    }

    getRecipe(recipeId: TestRecipeId) {
        return this.recipeRefs?.keys().find(r => r.id === recipeId);
    }

    id: TestId;
    key: string;
    name: string;
    version: number;
    tagRefs?: Set<TestTagRef>;
    phases?: Map<PhaseKey, TestPhaseRef>;
    team?: string;
    harness?: string;
    profile?: string;
    intent?: string;
    recipeRefs?: Set<TestRecipeRef>;

    private lowerKey: string;
    private lowerName: string;
}

export class MetadataRef {

    constructor(id: string, entries: Metadata, version: number) {
        this.id =  id;
        this.version = version;
        this.entries = entries;
        this.orderedKeys = this.getKeys().sort((a, b) => {
            const indexa = MetadataRef.keysOrderPriority.indexOf(a);
            const indexb = MetadataRef.keysOrderPriority.indexOf(b);
            if (indexa === -1) return indexb === -1? a.localeCompare(b) : 1;
            if (indexb === -1) return -1;
            return indexa - indexb;
        });
        this.orderedValues = this.orderedKeys.map((k) => this.entries[k]!);
    }

    getKeys(): string[] {
        return Object.keys(this.entries);
    }

    getValues(): string[] {
        return this.orderedKeys.map((k) => this.entries[k]!).filter((v) => !MetadataRef.valuesToIgnore.has(v));
    }

    getValuesExcept(skipKeys: string[]): string[] {
        return this.orderedKeys.filter((k) => !skipKeys.includes(k)).map((k) => this.entries[k]!).filter((v) => !MetadataRef.valuesToIgnore.has(v));
    }

    getValue(key: string): string | undefined {
        return this.entries[key];
    }

    getCommonValues(commonKeys?: string[]): string[] | undefined {
        if (!commonKeys) {
            return undefined;
        }
        return this.orderedKeys.filter((k) => commonKeys.includes(k)).map((k) => this.entries[k]!).filter((v) => !MetadataRef.valuesToIgnore.has(v));
    }

    isMatch(metaFilter: MetadataFilter, extact?: boolean): boolean {
        if (extact && Object.keys(this.entries).length !== metaFilter.size) {
            return false;
        }
        for (const [key, values] of metaFilter.entries()) {
            const thisValue = this.entries[key];
            if (!thisValue) {
                // missing key
                return false;
            }
            const toValidate = values.some((v) => v.indexOf('+') > -1) ? [thisValue] : thisValue.split('+');
            if (!toValidate.some((v) => values.includes(v)) || (extact && values.length  !== toValidate.length)) {
                return false;
            }
        }
        return true;
    }

    affinityScale(ref: MetadataRef): number {
        return MetadataRef.identifyCommonKeys([this, ref]).length;
    }

    id: MetaId;
    version: number;
    entries: Metadata;
    orderedKeys: string[];
    orderedValues: string[];

    private static keysOrderPriority = ["Project", "Platform", "BuildTarget"]; // Should those be set from a config?
    private static valuesToIgnore = new Set(["default", "Default"]);

    static identifyCommonKeys(items: MetadataRef[]): string[] {
        if (!items.length) return [];
        const first = items[0].entries;
        const commonKeys = new Set(Object.keys(first));
        if (items.length > 1) {
            items.slice(1).forEach((item) => {
                commonKeys.values().forEach((key) => {
                    if (first[key] !== item.entries[key]) {
                        commonKeys.delete(key);
                    }
                });
            });
        }
        return Array.from(commonKeys);
    }
}

export class TestRecipeRef {
    constructor(id: string, streamId: string, jobTemplateId: string, jobStep: string, nodeClass: string, testVersion: string, timeoutMinutes: number, version: number) {
        this.id = id;
        this.streamId = streamId;
        this.jobTemplateId = jobTemplateId;
        this.jobStep = jobStep;
        this.nodeClass = nodeClass;
        this.testVersion = testVersion;
        this.timeoutMinutes = timeoutMinutes;
        this.version = version;
    }

    isMatch(recipeId: string) {
        return this.id === recipeId;
    }

    includeParams(paramKeys: string[]) {
        return paramKeys.some(key => this.params?.has(key));
    }

    isParamValueMatch(key: string, value: any) {
        return !!this.params && this.params.get(key) === value;
    }

    id: TestRecipeId;
    version: number;
    streamId: StreamId;
    jobTemplateId: string;
    jobStep: string;
    nodeClass: string;
    testVersion: string;
    timeoutMinutes: number;

    test?: TestNameRef;
    workflowId?: string;
    params?: Map<string, any>
}

export class TestAudit {
    constructor(testId: string, userId: string, userInputDate: Date, enableSchedule: boolean) {
        this.testId = testId;
        this.userId = userId;
        this.userInputDate = userInputDate;
        this.enableSchedule = enableSchedule;
    }

    clone(): TestAudit {
        return {
            ...this,
            customers: this.customers?.slice(),
            notifications: this.notifications? new Map(this.notifications.entries()) : undefined
        };
    }

    testId: TestId;
    userId: string;
    userInputDate: Date;
    enableSchedule: boolean;
    lastAuditDate?: Date;
    isUnderAudit?: boolean;
    owner?: string;
    customers?: string[];
    notifications?: Map<StreamId, WorkflowId>;
    team?: string;
    harness?: string;
    intent?: string;
    profile?: string;
    notes?: string;
}

export abstract class TestAuditRecords {
    constructor(test: TestNameRef) {
        this.history = [];
        this.test = test;
    }

    getLatest() {
        if (!this.history.length) {
            return;
        }

        return this.history[0];
    }

    appendRecord(audit: TestAudit) {
        this.history.push(audit);
    }

    sortRecords() {
        this.history.sort((a, b) => b.userInputDate.getTime() - a.userInputDate.getTime());
    }

    abstract post(entry: TestAudit): Promise<boolean>;

    test: TestNameRef;
    history: TestAudit[];
}

export class TestMetaStatus {
    constructor(test: TestNameRef) {
        this.sessions = new Map();
        this.test = test;
    }

    addSession(metaRef: MetadataRef, session: TestSessionResult) {
        if (!this.sessions.has(metaRef)) {
            this.sessions.set(metaRef, new TestSessionStatus(this.test));
        }

        this.sessions.get(metaRef)!.addSession(session);
    }

    getLastSession(metaRef: MetadataRef): TestSessionResult | undefined {
        return this.sessions.get(metaRef)?.getLastSession();
    }

    getPreviousSession(metaRef: MetadataRef, session: TestSessionResult): TestSessionResult | undefined {
        return this.sessions.get(metaRef)?.getPreviousSession(session);
    }

    addPhaseSession(metaRef: MetadataRef, phaseKey: PhaseKey, session: PhaseSessionResult) {
        if (!this.sessions.has(metaRef)) {
            this.sessions.set(metaRef, new TestSessionStatus(this.test));
        }

        this.sessions.get(metaRef)!.addPhaseSession(phaseKey, session);
    }

    getPhaseSessions(phaseKey: PhaseKey, metaRef?: MetadataRef): PhaseSessionResult[] | undefined {
        if (!metaRef) {
            return this.sessions.values().map(m => m.getPhaseSessions(phaseKey)).filter(m => m?.length).toArray().flat() as PhaseSessionResult[];
        }
        return this.sessions.get(metaRef)?.getPhaseSessions(phaseKey);
    }

    hasPhaseSession(phaseKey: PhaseKey) {
        return this.sessions.values().some((s) => !!s.phaseHistories.get(phaseKey));
    }

    getMetadata(): MetadataRef[] {
        return this.sessions.keys().toArray();
    }

    includesMetadata(metaRef: MetadataRef): boolean {
        return this.sessions.has(metaRef);
    }

    includesAnyMetadata(metaRefs: Set<MetadataRef>): boolean {
        return metaRefs.keys().some((meta) => this.sessions.has(meta));
    }

    getMetadataLastSessions(): [MetadataRef, TestSessionResult][] {
        return this.sessions.entries().filter(([meta, status]) => status.history.length > 0).map(([meta, status]) => [meta, status.getLastSession()!] as [MetadataRef, TestSessionResult]).toArray();
    }

    sortSessions() {
        for (const status of this.sessions.values()) {
            status.sortSessions();
        }
    }

    sortPhaseSessions(phaseKey: PhaseKey) {
        for (const status of this.sessions.values()) {
            status.sortPhaseSessions(phaseKey);
        }
    }

    sessions: Map<MetadataRef, TestSessionStatus>;
    test: TestNameRef;
}

export class TestStatus {
    constructor() {
        this.tests = new Map();
    }

    addSession(testNameRef: TestNameRef, metaRef: MetadataRef, session: TestSessionResult) {
        if (!this.tests.has(testNameRef)) {
            this.tests.set(testNameRef, new TestMetaStatus(testNameRef));
        }

        this.tests.get(testNameRef)!.addSession(metaRef, session);
    }

    getLastSession(testNameRef: TestNameRef, metaRef: MetadataRef): TestSessionResult | undefined {
        return this.tests.get(testNameRef)?.getLastSession(metaRef);
    }

    getPreviousSession(testNameRef: TestNameRef, metaRef: MetadataRef, session: TestSessionResult): TestSessionResult | undefined {
        return this.tests.get(testNameRef)?.getPreviousSession(metaRef, session);
    }

    addPhaseSession(testNameRef: TestNameRef, metaRef: MetadataRef, phaseKey: PhaseKey, session: PhaseSessionResult) {
        if (!this.tests.has(testNameRef)) {
            this.tests.set(testNameRef, new TestMetaStatus(testNameRef));
        }

        this.tests.get(testNameRef)!.addPhaseSession(metaRef, phaseKey, session);
    }

    getPhaseSessions(testNameRef: TestNameRef, phaseKey: PhaseKey, metaRef?: MetadataRef): PhaseSessionResult[] | undefined {
        return this.tests.get(testNameRef)?.getPhaseSessions(phaseKey, metaRef);
    }

    getMatchingTests(testNameRefs: Set<TestNameRef>, metaRefs: Set<MetadataRef>): TestNameRef[] {
        return this.tests.entries().filter(([test, status]) => testNameRefs.has(test) && status.includesAnyMetadata(metaRefs)).map(([test, status]) => test).toArray();
    }

    getTestNames(): TestNameRef[] {
        return this.tests.keys().toArray();
    }

    getTestNameMetadata(): [TestNameRef, TestMetaStatus][] {
        return this.tests.entries().toArray();
    }

    sortSessions() {
        for (const meta of this.tests.values()) {
            meta.sortSessions();
        }
    }

    tests: Map<TestNameRef, TestMetaStatus>;
}

export class TestStreamRef {

    constructor(streamId: string) {
        this.id = streamId;
        this.tests = [];
        this.meta = [];
        this.tags = [];

        this.testNames = new Set();
        this.metadataMap = new Map();
        this.tagNames = new Set();
        this.testmetaMap = new Map([
            ['team', new Set()],
            ['profile', new Set()],
            ['harness', new Set()]
        ]);
    }

    addTest(item: TestNameRef) {
        this.tests.push(item);
        this.testNames.add(item.name);
        // test metadata
        if (item.team) {
            this.testmetaMap.get('team')!.add(item.team);
        }
        if (item.harness) {
            this.testmetaMap.get('harness')!.add(item.harness);
        }
        if (item.profile) {
            this.testmetaMap.get('profile')!.add(item.profile);
        }
    }

    addMetadata(item: MetadataRef) {
        this.meta.push(item);
        item.getKeys().forEach((key) => {
            let meta = this.metadataMap.get(key);
            if (!meta) {
                meta = new Set<string>();
                this.metadataMap.set(key, meta!);
            }
            meta!.add(item.getValue(key)!);
        });
    }

    addTag(item: TestTagRef) {
        this.tags.push(item);
        this.tagNames.add(item.name);
    }

    getAllTestNames(): string[] {
        return  this.testNames.keys().toArray();
    }

    getAllMetadataKeys(): string[] {
        return  this.metadataMap.keys().toArray();
    }

    getAllMetadataValues(key: string): string[] | undefined {
        return this.metadataMap.get(key)?.keys().toArray();
    }

    getAllTestMetaKeys(): string[] {
        return this.testmetaMap.keys().toArray();
    }

    getAllTestMetaValues(key: string): string[] | undefined {
        return this.testmetaMap.get(key)?.keys().toArray();
    }

    getAllTagNames(): string[] {
        return  this.tagNames.keys().toArray();
    }

    getMatchingTests(toMatchNames?: string[], toMatchTags?: string[], toMatchMeta?: MetadataFilter): Set<TestNameRef> {
        const lowerToMatchNames = toMatchNames?.map((i) => i.toLowerCase());
        const lowerToMatchTags = toMatchTags?.map((i) => i.toLowerCase());
        const lowerToMatchMeta = toMatchMeta? new Map(toMatchMeta.entries().map(([key, values]) => [key, values.map((v) => v.toLowerCase())])) : undefined;
        const matches: Set<TestNameRef> = new Set();
        this.tests.forEach((test) => {
            const isMatchNames = !lowerToMatchNames || lowerToMatchNames.some((key) => test.isMatch(key));
            const isMatchTags = !lowerToMatchTags || test.includesTags(lowerToMatchTags);
            const isMatchMeta = !lowerToMatchMeta || test.isMetaMatch(lowerToMatchMeta);
            if (isMatchNames && isMatchTags && isMatchMeta) {
                matches.add(test);
            }
        })

        return matches;
    }

    getMatchingMeta(toMatchMeta?: MetadataFilter): Set<MetadataRef> {
        const matches: Set<MetadataRef> = new Set();
        this.meta.forEach((meta) => {
            if (!toMatchMeta || meta.isMatch(toMatchMeta)) {
                matches.add(meta);
            }
        });

        return matches;
    }

    getMatchingTags(toMatchTags?: string[]): Set<TestTagRef> {
        const lowerToMatchTags = toMatchTags?.map((i) => i.toLowerCase());
        const matches: Set<TestTagRef> = new Set();
        this.tags.forEach((tag) => {
            if (!lowerToMatchTags || lowerToMatchTags.some((t) => tag.isMatch(t))) {
                matches.add(tag);
            }
        });

        return matches;
    }

    id: StreamId;
    tests: TestNameRef[];
    meta: MetadataRef[];
    tags: TestTagRef[];

    private testNames: Set<string>;
    private metadataMap: Map<string, Set<string>>;
    private tagNames: Set<string>;
    private testmetaMap: Map<string, Set<string>>;
}

export class TestDataFilterSearchParams {

    constructor(options?: any) {
        this.search =  new URLSearchParams(options);
    }

    private encodeKey(target: string, key?: string | null): string {
        if (!!key) {
            return `${target}.${key}`;
        }

        return target;
    }

    private encodeValue(items: string | string[]): string {
        if (!Array.isArray(items)) {
            return items;
        }

        return items.join('~');
    }

    private decodeKey(key: string): string[] {
        return key.split('.', 2);
    }

    private decodeValue(value: string): string[] {
        return value.split('~')
    }

    append(target: string, key: string | null, items: string | string[]) {
        const actual_key = this.encodeKey(target, key);
        const append_items = typeof items === "string"? [items] : items;
        const merged_items = new Set<string>(
            !this.search.has(actual_key)? append_items : [...this.decodeValue(this.search.get(actual_key)!), ...append_items]
        );

        this.search.set(actual_key, this.encodeValue(Array.from(merged_items)));
    }

    set(target: string, key: string | null, items?: string | string[]) {
        if (!items || !items.length) {
            this.delete(target, key);
            return;
        }

        this.search.set(this.encodeKey(target, key), this.encodeValue(items));
    }

    delete(target: string, key?: string | null, items?: string | string[]) {
        const actual_key = this.encodeKey(target, key);
        if (!this.search.has(actual_key)) {
            return;
        }
        if (!items) {
            this.search.delete(actual_key)
            return;
        }
        const current_items = new Set<string>(this.decodeValue(this.search.get(actual_key)!));
        const remove_items = new Set<string>(typeof items === "string"? [items] : items);
        this.search.set(actual_key, this.encodeValue(Array.from(current_items.difference(remove_items))));
    }

    get(target: string, key?: string | null): string[] | null {
        const actual_key = this.encodeKey(target, key);
        return this.search.has(actual_key)? this.decodeValue(this.search.get(actual_key)!) : null;
    }

    getGroup(target: string): Map<string, string[]> | null {
        const entries = new Map<string, string[]>();
        const targetKey = `${target}.`;

        this.search.keys().forEach(longKey => {
            if (longKey.startsWith(targetKey)) {
                entries.set(longKey.substring(targetKey.length), this.decodeValue(this.search.get(longKey)!));
            }
        });

        return entries;
    }

    deleteGroup(target: string) {
        const targetKey = `${target}.`;

        this.search.keys().forEach(longKey => {
            if (longKey.startsWith(targetKey)) {
                this.search.delete(longKey);
            }
        });
    }

    has(target: string, key?: string | null, items?: string | string[]): boolean {
        const actual_key = this.encodeKey(target, key);
        if (!!items) {
            if (!this.search.has(actual_key)) {
                return false;
            }
            const current_items = new Set<string>(this.decodeValue(this.search.get(actual_key)!));
            return typeof items === "string"? current_items.has(items) : items.every((item) => current_items.has(item));
        } else {
            return this.search.has(actual_key);
        }
    }

    toString(): string {
        return this.search.toString();
    }

    private search: URLSearchParams;
}

export class TestDataHandler {

    constructor() {
        TestDataHandler.instance = this;
        makeObservable(this);
        // setup reactions to debounce the search and filter refresh
        reaction(
            () => this.filterUpdated,
            () => this.updateFilter(),
            { delay: filterRefreshMS }
        );
        reaction(
            () => this.checkSearchUpdated,
            () => this.onSearchUpdated(),
            { delay: 100 }
        );
    }

    async initialize(search: string, searchCallback?: (search: string, replace?: boolean) => void) {
        this.filterState = this.filterFromSearch(search);
        this.searchCallback = searchCallback;
        this.checkSearchCallback(search);
        await this.load();
    }

    clear() {
        this.searchCallback = undefined;
        this.filterState = {};
        this.search = new TestDataFilterSearchParams();
        this.testStreams = new Map();
        this.commitIdDates = new Map();
        this.testStatus = new Map();
        this.loaded = false;
    }

    syncSearchParam(search: string) {
        if (search != this.search.toString()) {
            this.filterState = this.filterFromSearch(search);
            this.setSearchUpdated();
        }
    }

    private checkSearchCallback(previous: string): boolean {
        if (previous != this.search.toString()) {
            this.setCheckSearchUpdated();
            return true;
        }
        return false;
    }

    setSearchParam(key: string, value: string | string[], replace?: boolean) {
        const previous = this.search.toString();
        this.search.set(key, null, value);
        this.checkSearchCallback(previous) && replace && this.queueHistoryReplace();
    }

    removeSearchParam(key: string, replace?: boolean) {
        const previous = this.search.toString();
        this.search.delete(key);
        this.checkSearchCallback(previous) && replace && this.queueHistoryReplace();
    }

    getSearchParam(key: string): string | string[] | null | undefined {
        const values = this.search.get(key);
        if (values?.length === 1) return values[0];
        return values;
    }

    setQueryWeeks(weeks: number) {
        if (weeks !== this.filterState.weeks) {
            this.filterState.weeks = weeks;
            if (weeks && weeks > defaultQueryWeeks) {
                // scrap the cache
                this.testStatus = new Map();
            }
            this.setFilterUpdated();
        }
    }

    addMetadata(key: string, value: string) {

        const state = this.filterState;
        const meta = state.metadata?.get(key);

        if (!!meta && meta!.includes(value)) {
            return;
        }

        if (!state.metadata) {
            state.metadata = new Map();
        }

        if (!meta) {
            state.metadata!.set(key, [value]);
        } else {
            state.metadata!.get(key)!.push(value);
        }

        this.setFilterUpdated();
    }

    removeMetadata(key: string, value?: string) {

        const state = this.filterState;
        const meta = state.metadata?.get(key);
        if (!!value) {
            const idx = meta?.indexOf(value!);

            if (!meta || !isNumber(idx) || idx! < 0) {
                return;
            }

            meta!.splice(idx!, 1);
        }

        if (!value || !meta!.length) {
            state.metadata!.delete(key);
        }

        this.setFilterUpdated();
    }

    addTestMeta(key: string, value: string) {

        const state = this.filterState;
        const meta = state.testmeta?.get(key);

        if (!!meta && meta!.includes(value)) {
            return;
        }

        if (!state.testmeta) {
            state.testmeta = new Map();
        }

        if (!meta) {
            state.testmeta!.set(key, [value]);
        } else {
            state.testmeta!.get(key)!.push(value);
        }

        this.setFilterUpdated();
    }

    removeTestMeta(key: string, value?: string) {

        const state = this.filterState;
        const meta = state.testmeta?.get(key);
        if (!!value) {
            const idx = meta?.indexOf(value!);

            if (!meta || !isNumber(idx) || idx! < 0) {
                return;
            }

            meta!.splice(idx!, 1);
        }

        if (!value || !meta!.length) {
            state.testmeta!.delete(key);
        }

        this.setFilterUpdated();
    }

    addTest(test: string) {

        const state = this.filterState;

        if (state.tests) {
            if (state.tests.includes(test)) {
                return;
            }
        }

        if (!state.tests) {
            state.tests = [];
        }

        state.tests.push(test);

        this.setFilterUpdated();
    }

    removeTest(test: string) {

        const state = this.filterState;

        const idx = state.tests?.indexOf(test);
        if (!isNumber(idx) || idx! < 0) {
            return;
        }

        state.tests!.splice(idx!, 1);

        if (!state.tests!.length) {
            state.tests = undefined;
        }

        this.setFilterUpdated();
    }

    addTag(tag: string) {

        const state = this.filterState;

        if (state.tags) {
            if (state.tags.includes(tag)) {
                return;
            }
        }

        if (!state.tags) {
            state.tags = [];
        }

        state.tags.push(tag);

        this.setFilterUpdated();
    }

    removeTag(tag: string) {

        const state = this.filterState;

        const idx = state.tags?.indexOf(tag);
        if (!isNumber(idx) || idx! < 0) {
            return;
        }

        state.tags!.splice(idx!, 1);

        if (!state.tags!.length) {
            state.tags = undefined;
        }

        this.setFilterUpdated();
    }

    selectStream(streamId?: StreamId) {

        const state = this.filterState;

        if (state.stream === streamId) {
            return;
        }

        if (!state.stream && !!this.search.get('session')) {
            // detect straight-to-session-view to then replace history instead of adding to it
            // as in the case selectStream is called only because we want to load the stream related test data for comparison
            this.queueHistoryReplace();
        }

        state.stream = streamId;

        this.setFilterUpdated();
    }

    /// All test names from current stream
    get allStreamTestNames(): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllTestNames().sort((a, b) => a.localeCompare(b));
    }

    /// All test tags from current stream
    get allStreamTestTags(): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllTagNames().sort((a, b) => a.localeCompare(b));
    }

    /// All metadata keys from current stream
    get allStreamMetadataKeys(): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllMetadataKeys().sort((a, b) => a.localeCompare(b));
    }

    /// All metadata values for a key from current stream
    getAllStreamMetadataValues(key: string): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllMetadataValues(key)?.sort((a, b) => a.localeCompare(b)) ?? [];
    }

    /// All test metadata keys from current stream
    get allStreamTestMetaKeys(): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllTestMetaKeys().sort((a, b) => a.localeCompare(b));
    }

    /// All test metadata values for a key from current stream
    getAllStreamTestMetaValues(key: string): string[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef.getAllTestMetaValues(key)?.sort((a, b) => a.localeCompare(b)) ?? [];
    }

    /// Tests that match current filters
    get filteredTests(): TestNameRef[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        const testStatus = this.testStatus.get(state.stream);
        if (!testStatus) {
            return [];
        }

        const metaRefs = testStreamRef!.getMatchingMeta(state.metadata);
        const testNameRefs = testStreamRef!.getMatchingTests(state.tests, state.tags, state.testmeta);

        return testStatus!.getMatchingTests(testNameRefs, metaRefs).sort((a, b) => a.name.localeCompare(b.name));
    }

    /// Metadata that match current filters
    get filteredMetadata(): MetadataRef[] {

        const state = this.filterState;
        if (!state.stream) {
            return [];
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return [];
        }

        return testStreamRef!.getMatchingMeta(state.metadata).keys().toArray().sort((a, b) => a.orderedValues.join().localeCompare(b.orderedValues.join()));
    }

    /// Tags that match current tag filters, return undefined if no tag is selected
    get selectedTags(): TestTagRef[] | undefined {

        const state = this.filterState;
        if (!state.stream) {
            return undefined;
        }

        const testStreamRef = this.testStreams.get(state.stream);
        if (!testStreamRef) {
            return undefined;
        }

        if (!state.tags) {
            return undefined;
        }

        const lowerTags = state.tags.map(t => t.toLowerCase());
        return testStreamRef!.getMatchingTags(lowerTags).keys().toArray().sort((a, b) => a.name.localeCompare(b.name));
    }

    private queueHistoryReplace() {
        this.nextHistoryReplace = true;
    }

    private popHistoryReplace(): boolean | undefined {
        const replace = this.nextHistoryReplace;
        this.nextHistoryReplace = undefined;
        return replace;
    }

    private updateFilter() {
        if (this.checkUpdateFilter()) {
            this.onSearchUpdated();
        }
    }

    private onSearchUpdated() {
        this.setSearchUpdated();
        if (this.searchCallback) {
            this.searchCallback(this.search.toString(), this.popHistoryReplace());
        }
    }

    private checkUpdateFilter(): boolean {
        const state = this.filterState;
        state.weeks = state.weeks ? state.weeks : undefined;
        state.stream = state.stream ? state.stream : undefined;
        state.metadata = state.metadata && state.metadata.size > 0 ? state.metadata : undefined;
        state.testmeta = state.testmeta && state.testmeta.size > 0 ? state.testmeta : undefined;
        state.tests = state.tests?.sort((a, b) => a.localeCompare(b));
        state.tags = state.tags?.sort((a, b) => a.localeCompare(b));

        let needUpdate = false;
        const search = this.search;

        needUpdate = needUpdate || search.get('weeks')?.[0] !== state.weeks?.toString();
        if (needUpdate) {
            search.set("weeks", null, state.weeks?.toString());
        }

        needUpdate = needUpdate || search.get('stream')?.[0] !== state.stream;
        if (needUpdate) {
            search.set("stream", null, state.stream);
        }

        needUpdate = needUpdate || search.get('tests')?.toString() !== state.tests?.toString();
        if (needUpdate) {
            search.set("tests", null, state.tests);
        }

        needUpdate = needUpdate || search.get('tags')?.toString() !== state.tags?.toString();
        if (needUpdate) {
            search.set("tags", null, state.tags);
        }

        const metaKeys = new Set<string>(search.getGroup("meta")?.keys());
        metaKeys.union(new Set(state.metadata?.keys())).keys().forEach(key => {
            needUpdate = needUpdate || search.get('meta', key)?.toString() !== state.metadata?.get(key)?.toString();
            search.set("meta", key, state.metadata?.get(key));
        });

        const testMetaKeys = new Set<string>(search.getGroup("tmeta")?.keys());
        testMetaKeys.union(new Set(state.testmeta?.keys())).keys().forEach(key => {
            needUpdate = needUpdate || search.get('tmeta', key)?.toString() !== state.testmeta?.get(key)?.toString();
            search.set("tmeta", key, state.testmeta?.get(key));
        });

        if (needUpdate) {
            this.query(state.stream);
            return true;
        }

        return false;
    }

    private filterFromSearch(searchInfo: string): TestDataState {
        const search = new TestDataFilterSearchParams(searchInfo);
        this.search = search;

        const state: TestDataState = {};

        const stream = search.get("stream") ?? undefined;
        const metadata = search.getGroup("meta");
        const testmeta = search.getGroup("tmeta");
        const tests = search.get("tests") ?? undefined;
        const tags = search.get("tags") ?? undefined;
        const weeks = search.get("weeks") ?? undefined;

        state.stream = stream && stream.length > 0? stream[0] : undefined;
        state.metadata = metadata && metadata.size > 0 ? metadata : undefined;
        state.testmeta = testmeta && testmeta.size > 0 ? testmeta : undefined;
        state.tests = tests;
        state.tags = tags;
        state.weeks = weeks && weeks.length > 0 ? parseInt(weeks[0]) : defaultQueryWeeks;
        if (typeof (state.weeks) !== "number") {
            state.weeks = defaultQueryWeeks;
        }

        return state;
    }

    async query(streamId?: StreamId, isSubQuery: boolean = false) {

        if (this.queryLoading || this.subQueryLoading) {
            await when(() => !this.queryLoading && !this.subQueryLoading);
        }

        if (!streamId || !this.testStreams.has(streamId) || this.testStatus.has(streamId)) {
            isSubQuery? this.setSubQueryLoading(false): this.setQueryLoading(false);
            return;
        }

        isSubQuery? this.setSubQueryLoading(true): this.setQueryLoading(true);

        const state = this.filterState;
        const commitIdDates = this.commitIdDates
        const testStreamRef: TestStreamRef = this.testStreams.get(streamId)!;
        const testStatus = new TestStatus();
        this.testStatus.set(streamId, testStatus);

        const aggregateCommitDates = (session: TestSessionResult) => {
            /// Update uniform commit dates
            const date = commitIdDates.get(session.commitId);
            const rdate = session.start;
            // Use the earliest date for summary, otherwise gets confusing when a CL spans days
            if (!date || date.getTime() > rdate.getTime()) {
                commitIdDates.set(session.commitId, rdate);
            }
        }

        const queryWeeks = state.weeks && state.weeks > defaultQueryWeeks ? state.weeks : defaultQueryWeeks;

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            console.debug(`Fetching from version ${fetcherVersion.version} test sessions`)
            await fetcherVersion.fetchTestSessions(testStreamRef, testStatus, queryWeeks, aggregateCommitDates);
        }

        /// sort session history by commit order descending
        testStatus.sortSessions();

        if (this.loaded) {
            isSubQuery? this.setSubQueryLoading(false): this.setQueryLoading(false);
            !isSubQuery && this.setSearchUpdated();
        }
    }

    async queryTestData(testDataId: string): Promise<TestSessionDetails | undefined> {
        if (this.testTestData.has(testDataId)) {
            this.setTestDataQueryLoading(false);
            return this.testTestData.get(testDataId);
        }

        this.setTestDataQueryLoading(true);

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            console.debug(`Fetching from version ${fetcherVersion.version} test data`)
            try {
                const sessionDetails = await fetcherVersion.fetchTestData(testDataId);
                if (sessionDetails) {
                    this.testTestData.set(testDataId, sessionDetails)
                    this.query(sessionDetails.streamId).then(() => {
                        const session = this.getStatusStream(sessionDetails.streamId)?.tests.get(sessionDetails.test!)?.sessions.get(sessionDetails.meta!)?.history.find(s => s.testDataId === testDataId);
                        sessionDetails.session = session;
                    });
                    /// stop fetching if a valid object was pulled
                    break;
                }
            }  catch(reason) {
                /** ignore failed attempt to try with next version */
                console.debug(`Failed to fetch from version ${fetcherVersion.version}: ${reason}`);
            }
        }

        if (this.loaded) {
            this.setTestDataQueryLoading(false);
        }

        return this.testTestData.get(testDataId);
    }

    async queryJobStepTestData(jobId: string, stepId: string): Promise<JobStepTestDataItem[] | undefined> {
        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            console.debug(`Fetching from version ${fetcherVersion.version} job step test data`)
            try {
                const response = await fetcherVersion.fetchJobStepTestData(jobId, stepId);
                if (response) {
                    return response
                }
            }  catch(reason) {
                /** ignore failed attempt to try with next version */
                console.debug(`Failed to fetch from version ${fetcherVersion.version}: ${reason}`);
            }
        }
    }

    async queryPhaseSessions(phases: TestPhaseStatus[], streamId: StreamId, isSubQuery: boolean = false) {

        if (phases.length === 0) {
            return;
        }

        const test = phases[0].test!;
        if (!phases.every(phase => phase.test === test)) {
            console.error('Phase sessions query is not for the same test');
            return;
        }

        const testStatus = this.testStatus.get(streamId)?.tests.get(test);
        const phaseKeys = Array.from(new Set(phases.map(phase => phase.key)));

        const associatePhaseSessions = () => {
            if (testStatus) {
                phases.forEach(phase => {
                    if (!phase.session) {
                        /// associate the corresponding phase session with the phase status
                        phase.associateSession(testStatus.getPhaseSessions(phase.key, phase.testMetadata));
                        /// associate the corresponding phase reference
                        phase.phaseRef = test.phases?.get(phase.key);
                    }
                });
            }
        }

        // Check if all phases already have sessions
        if (!testStatus || phaseKeys.every(phaseKey => testStatus.hasPhaseSession(phaseKey))) {
            associatePhaseSessions();
            isSubQuery? this.setSubPhaseQueryLoading(false): this.setPhaseQueryLoading(false);
            return;
        }

        isSubQuery? this.setSubPhaseQueryLoading(true): this.setPhaseQueryLoading(true);

        await this.query(streamId);

        const state = this.filterState;
        const queryWeeks = state.weeks && state.weeks > defaultQueryWeeks ? state.weeks : defaultQueryWeeks;

        const testStreamRef = this.testStreams.get(streamId)!;

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            /// fetch only from the matching version
            if (fetcherVersion.version === test.version) {
                console.debug(`Fetching from version ${fetcherVersion.version} test phase sessions`);
                try {
                    await fetcherVersion.fetchPhaseSessions(testStreamRef, testStatus, phaseKeys, queryWeeks);
                } catch (reason) {
                    /** ignore failure as getting the related phase sessions is optional to display content */
                    console.debug(`Failed to fetch from version ${fetcherVersion.version}: ${reason}`);
                }
                break;
            }
        }

        /// sort phase session history by commit order descending for all phases
        phaseKeys.forEach(phaseKey => {
            testStatus.sortPhaseSessions(phaseKey);
        });

        associatePhaseSessions();

        if (this.loaded) {
            isSubQuery? this.setSubPhaseQueryLoading(false): this.setPhaseQueryLoading(false);
        }
    }

    async queryTestPhases(test: TestMetaStatus, phaseKeys?: string[]) {

        if (!!phaseKeys && phaseKeys.every(phaseKey => test.test.phases?.get(phaseKey))) {
            this.setPhaseQueryLoading(false);
            return;
        }

        this.setPhaseQueryLoading(true);

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            /// fetch only from the matching version
            console.debug(`Fetching from version ${fetcherVersion.version} test phases`);
            try {
                await fetcherVersion.fetchTestPhases(test, phaseKeys);
                break;
            } catch (reason) {
                /** ignore failed attempt to try with next version */
                console.debug(`Failed to fetch phases from version ${fetcherVersion.version}: ${reason}`);
            }
        }

        this.setPhaseQueryLoading(false)
    }

    async queryRecipes(test: TestNameRef, recipeIds: TestRecipeId[]) {

        const missingRecipes = recipeIds.filter(r => !test.getRecipe(r));
        if (missingRecipes.length === 0) {
            this.setRecipeQueryLoading(false);
            return;
        }

        this.setRecipeQueryLoading(true);

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            /// fetch only from the matching version
            if (fetcherVersion.version === test.version) {
                console.debug(`Fetching from version ${fetcherVersion.version} test recipes`);
                try {
                    await fetcherVersion.fetchTestRecipes(test, missingRecipes);
                } catch (reason) {
                    console.debug(`Failed to fetch recipes from version ${fetcherVersion.version}: ${reason}`);
                }
                break;
            }
        }

        if (this.loaded) {
            this.setRecipeQueryLoading(false);
        }
    }

    async queryAudits(tests: TestNameRef[], withHistory?: boolean) {

        const missingTests = tests.filter(t => !this.testAudits.get(t) || (withHistory && this.testAudits.get(t)!.history.length === 1));

        if (missingTests.length === 0) {
            this.setAuditQueryLoading(false);
            return;
        }

        let queryWeeks: number | undefined;
        if (withHistory) {
            const state = this.filterState;
            queryWeeks = state.weeks && state.weeks > defaultQueryWeeks ? state.weeks : defaultQueryWeeks;
        }

        this.setAuditQueryLoading(true);

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            console.debug(`Fetching from version ${fetcherVersion.version} test audits`);
            try {
                await fetcherVersion.fetchTestAudit(missingTests, this.testAudits, queryWeeks);
            } catch (reason) {
                console.debug(`Failed to fetch audits from version ${fetcherVersion.version}: ${reason}`);
            }
        }

        if (withHistory) {
            missingTests.forEach(test => {
                const records = this.testAudits.get(test);
                records?.sortRecords();
            });
        }

        if (this.loaded) {
            this.setAuditQueryLoading(false);
        }
    }

    // load immutable data for this view session
    private async load() {

        const allStreams = projectStore.projects.map(p => p.streams).flat().filter(s => !!s).map(s => s!.id);
        const testStreams = this.testStreams = new Map(allStreams.map((s) => [s, new TestStreamRef(s)]));

        for (const fetcherVersion of TestDataVersionRegistrar.manifest) {
            console.debug(`Fetching from version ${fetcherVersion.version} stream info`)
            await fetcherVersion.fetchStreamsInfo(testStreams);
        }

        // filter out empty streams
        for(const stream of this.testStreams.entries().filter(([, ref]) => ref.tests.length === 0).map(([key,]) => key))
        {
            this.testStreams.delete(stream);
        }

        if (!this.loaded) {
            this.loaded = true;
            this.query(this.filterState.stream);
        }
    }

    @action
    setFilterUpdated() {
        this.filterUpdated++;
    }

    @action
    setSearchUpdated() {
        this.searchUpdated++;
    }

    @action
    setCheckSearchUpdated() {
        this.checkSearchUpdated++;
    }

    @action
    setQueryLoading(value: boolean) {
        this.queryLoading = value;
    }

    @action
    setSubQueryLoading(value: boolean) {
        this.subQueryLoading = value;
    }

    @action
    setTestDataQueryLoading(value: boolean) {
        this.testDataQueryLoading = value;
    }

    @action
    setPhaseQueryLoading(value: boolean) {
        this.phaseQueryLoading = value;
    }

    @action
    setSubPhaseQueryLoading(value: boolean) {
        this.subPhaseQueryLoading = value;
    }

    @action
    setRecipeQueryLoading(value: boolean) {
        this.queryRecipesLoading = value;
    }

    @action
    setAuditQueryLoading(value: boolean) {
        this.queryAuditsLoading = value;
    }

    get stream(): StreamId | undefined {
        return this.filterState.stream;
    }

    get availableStreams(): StreamId[] {
        return  this.testStreams.keys().toArray();
    }

    get availableStatusStreams(): StreamId[] {
        return this.testStatus.keys().toArray();
    }

    getStatusStream(streamId: string): TestStatus | undefined {
        return this.testStatus.get(streamId);
    }

    get selectedStatusStream(): TestStatus | undefined {
        if (!this.filterState.stream) {
            return;
        }
        return this.getStatusStream(this.filterState.stream!);
    }

    @observable
    filterUpdated: number = 0;

    @observable
    searchUpdated: number = 0;

    @observable
    checkSearchUpdated: number = 0;

    @observable
    queryLoading: boolean = false;

    @observable
    subQueryLoading: boolean = false;

    @observable
    testDataQueryLoading: boolean = false;

    @observable
    phaseQueryLoading: boolean = false;

    @observable
    subPhaseQueryLoading: boolean = false;

    @observable
    queryRecipesLoading: boolean = false;

    @observable
    queryAuditsLoading: boolean = false;

    subscribeToFilter() {
        if (this.filterUpdated) { }
    }

    subscribeToSearch() {
        if (this.searchUpdated) { }
    }

    subscribeToQueryLoading() {
        if (this.queryLoading) { }
    }

    subscribeToSubQueryLoading() {
        if (this.subQueryLoading) { }
    }

    subscribeToTestDataQueryLoading() {
        if (this.testDataQueryLoading) { }
    }

    subscribeToPhaseQueryLoading() {
        if (this.phaseQueryLoading) { }
    }

    subscribeToSubPhaseQueryLoading() {
        if (this.subPhaseQueryLoading) { }
    }

    subscribeToRecipesLoading() {
        if (this.queryRecipesLoading) { }
    }

    subscribeToAuditsLoading() {
        if (this.queryAuditsLoading) { }
    }

    private searchCallback?: (search: string, replace?: boolean) => void;
    private search: TestDataFilterSearchParams = new TestDataFilterSearchParams();
    filterState: TestDataState = {};

    private nextHistoryReplace?: boolean;

    testStreams: Map<StreamId, TestStreamRef> = new Map();

    testTestData: Map<TestDataId, TestSessionDetails> = new Map();

    testAudits: Map<TestNameRef, TestAuditRecords> = new Map();

    // quantized common commits to land at same time in the views
    commitIdDates = new Map<string, Date>();
    
    private testStatus: Map<StreamId, TestStatus> = new Map();

    loaded = false;

    static instance: TestDataHandler;
}

export class ArtifactFactory {
    constructor(jobId: string, stepId: string) {
        this.jobId = jobId;
        this.stepId = stepId;
    }

    async getJobStepSavedArtifacts(): Promise<GetArtifactResponse | undefined> {
        if (this.artifactV2Response) {
            return this.artifactV2Response;
        }

        const v = await backend.getJobArtifactsV2(undefined, [`job:${this.jobId}/step:${this.stepId}`]);
        const av2 = v?.artifacts.find(a => a.type === "step-saved")
        if (av2) {
            this.artifactV2Response = av2;
            if (av2.id) {
                this.artifactId = av2.id;
            }
        } else {
            console.error("Unable to get step-saved artifacts for test report");
        }

        return this.artifactV2Response;
    }

    async getLink(referencePath: string): Promise<string> {

        await this.getJobStepSavedArtifacts();
        
        return `${backend.serverUrl}/api/v2/artifacts/${this.artifactId}/file?path=${encodeURIComponent(referencePath)}&inline=true`;
    }

    async fetch(referencePath: string): Promise<any> {

        const cacheKey = `job:${this.jobId}/step:${this.stepId}/${referencePath}`;
        const cache = ArtifactFactory.cache.get(cacheKey);
        if (!!cache) {
            return cache;
        }

        const link = await this.getLink(referencePath);

        return new Promise<any>((resolve, reject) => {
            backend.fetch.get(link, {}).then((value) => {
                ArtifactFactory.cache.set(cacheKey, value.data);
                resolve(value.data);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    async download(referencePath: string): Promise<void> {
        await this.getJobStepSavedArtifacts();
        if (!!this.artifactId) backend.downloadArtifactV2(this.artifactId, referencePath);
    }

    async downloadZip(referencePaths: string[]) : Promise<void> {
        await this.getJobStepSavedArtifacts();
        if (!!this.artifactId) {
            backend.downloadArtifactZipV2(this.artifactId, {filter: referencePaths})
        }
    }

    private jobId: string;
    private stepId: string;
    private artifactV2Response?: GetArtifactResponse;
    private artifactId?: string;

    private static cache: Map<string, any> = new Map();
}
