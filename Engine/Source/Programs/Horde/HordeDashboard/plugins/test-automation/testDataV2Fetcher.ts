// Copyright Epic Games, Inc. All Rights Reserved.

import { getJobStepTestDataV2, GetPhaseResponse, getStreamTestsV2, getTestAuditsV2, getTestDataV2, GetTestMetadataResponse, GetTestNameResponse, getTestNamesV2, GetTestPhaseSessionResponse, getTestPhaseSessionsV2, getTestPhasesV2, getTestRecipesV2, GetTestSessionResponse, getTestSessionsV2, GetTestTagResponse, JobStepTestDataItem, PhaseSessionDetailsResponse, postTestAuditV2, TestAuditPost, TestAuditResponse, TestData, TestOutcome, TestRecipeResponse, TestSessionDetailsResponse } from "./api";
import { TestStreamRef, TestStatus, TestSessionResult, TestNameRef, MetadataRef, TestTagRef, PhaseSessionResult, TestPhaseRef, TestMetaStatus, TestSessionDetails, TestPhaseStatus, DeviceRef, TestVersionFetcher, TestRecipeRef, TestAudit, TestAuditRecords } from "./testData"
import { EventEntry } from "./testEventsModel";

const minuteInWeek = 10080;

class TestPhaseStatusV2 extends TestPhaseStatus {

    constructor(phase: PhaseSessionDetailsResponse, devices: DeviceRef[], parent: TestSessionDetails) {
        super(phase.key, phase.name ?? "<undefinied>", new Date(phase.dateTime), phase.timeElapseSec, devices, phase.outcome, parent)
        this._eventStreamPath = phase.eventStreamPath;
        this.hasWarning = phase.hasWarning;
        this.errorFingerprint = phase.errorFingerprint;
    }

    async fetchEventStream(): Promise<EventEntry[] | undefined> {
        if (!this._eventStreamPath) return;
        return await this.artifacts?.fetch(this._eventStreamPath);
    }

    private _eventStreamPath?: string;

}

class TestAuditRecordsV2 extends TestAuditRecords {

    async post(entry: TestAudit): Promise<boolean> {

        const post: TestAuditPost = {
            enableSchedule: entry.enableSchedule,
            lastAuditDate: entry.lastAuditDate?.toISOString(),
            isUnderAudit: entry.isUnderAudit,
            team: entry.team,
            harness: entry.harness,
            profile: entry.profile,
            intent: entry.intent,
            notes: entry.notes,
            owner: entry.owner,
            customers: entry.customers,
            notifications: entry.notifications?.entries().reduce((obj, [stream, workflow]) => { obj[stream] = workflow; return obj }, {})
        };

        try {
            const response = await postTestAuditV2(entry.testId, post);
            const newAuditEntry = TestDataV2.createTestAudit(response);
            this.history.unshift(newAuditEntry);

        } catch(reason) {
            console.debug(`Failed to post audit: ${reason}`);
            return false;
        }
        
        return true;
    }

}

export class TestDataV2 implements TestVersionFetcher {
    get version() { return 2 }

    async fetchStreamsInfo(testStreams: Map<string, TestStreamRef>) {

        const results = await getStreamTestsV2(testStreams.keys().toArray());
        results.forEach((streamInfo) => {
            const testStream = testStreams.get(streamInfo.streamId)!;

            /// Tests
            streamInfo.tests.forEach((test) => {
                testStream.addTest(this.createTestNameRef(test));
            });

            /// Metadata
            streamInfo.testMetadata.forEach((entry) => {
                testStream.addMetadata(this.createMetadataRef(entry));
            });

            /// Tags
            streamInfo.testTags.forEach((tag) => {
                testStream.addTag(this.createTagRef(tag));
            });
            
        });
   
    }

    async fetchTestSessions(streamRef: TestStreamRef, status: TestStatus, weeks: number, onAdd?: (session: TestSessionResult) => void) {

        const streamId = streamRef.id;
        const tests: Map<string, TestNameRef> = new Map(streamRef.tests.filter((t) => t.version >= this.version).map((t) => [t.id, t]));
        const tags: Map<string, TestTagRef> = new Map(streamRef.tags.filter((t) => t.version >= this.version).map((t) => [t.id, t]));
        const meta: Map<string, MetadataRef> = new Map(streamRef.meta.filter((m) => m.version >= this.version).map((m) => [m.id, m]));

        const minQueryDate = new Date(new Date().valueOf() - (minuteInWeek * weeks * 60000));
        const maxQueryDate = new Date();

        if (tests.size) {
            const responses = await getTestSessionsV2([streamId], Array.from(tests.keys()),  Array.from(meta.keys()), minQueryDate.toISOString(), maxQueryDate.toISOString());
            responses.forEach((session) => {
                if (session.streamId !== streamId) {
                    return;
                }
                const testSession = this.createTestSession(session);
                const testNameRef = tests.get(testSession.nameRef);
                const metadataRef = meta.get(testSession.metadataId);
                const tagRefs = testSession.tagIds?.map((id) => tags.get(id)).filter((ref) => !!ref);
                if (!!tagRefs && !!testNameRef) {
                    testNameRef.updateTags(tagRefs);
                }
                if (!!testNameRef && !!metadataRef) {
                    status.addSession(testNameRef, metadataRef, testSession);
                    if (onAdd) onAdd(testSession);
                }                
            });
        }
        
    }

    async fetchTestData(testDataId: string): Promise<TestSessionDetails | undefined> {

        const response = await getTestDataV2(testDataId);
        if (!!response) {
            const sessionDetails = this.createSessionDetails(response);
            return sessionDetails;
        }

        return undefined;

    }

    async fetchJobStepTestData(jobId: string, stepId: string): Promise<JobStepTestDataItem[] | undefined> {
        return await getJobStepTestDataV2(jobId, stepId)
    }

    async fetchPhaseSessions(streamRef: TestStreamRef, status: TestMetaStatus, phaseKeys: string[], weeks: number): Promise<boolean> {

        const streamId = streamRef.id;
        const tags: Map<string, TestTagRef> = new Map(streamRef.tags.filter((t) => t.version >= this.version).map((t) => [t.id, t]));
        const meta: Map<string, MetadataRef> = new Map(streamRef.meta.filter((m) => m.version >= this.version).map((m) => [m.id, m]));

        await this.fetchTestPhases(status, phaseKeys);        
        
        // Get phase refs for all requested phase keys and create a map from phase ID to phaseRef
        const phaseIdToPhaseRefMap = new Map<string, TestPhaseRef>();
        
        phaseKeys.forEach(phaseKey => {
            const phaseRef = status.test.phases?.get(phaseKey);
            if (phaseRef) {
                phaseIdToPhaseRefMap.set(phaseRef.id, phaseRef);
            }
        });

        if (phaseIdToPhaseRefMap.size > 0) {
            const minQueryDate = new Date(new Date().valueOf() - (minuteInWeek * weeks * 60000));
            const maxQueryDate = new Date();
    
            const phaseIds = Array.from(phaseIdToPhaseRefMap.keys());
            const responses = await getTestPhaseSessionsV2([streamId], phaseIds, minQueryDate.toISOString(), maxQueryDate.toISOString());
            responses.forEach((session) => {
                if (session.streamId !== streamId) {
                    return;
                }
                const phaseRef = phaseIdToPhaseRefMap.get(session.phaseRef);
                if (!phaseRef) {
                    return;
                }
                const phaseSession = this.createPhaseSession(session);
                const tagRefs = phaseSession.tagIds?.map((id) => tags.get(id)).filter((ref) => !!ref);
                if (!!tagRefs) {
                    phaseRef.updateTags(tagRefs);
                }
                const metadataRef = meta.get(phaseSession.metadataId);
                if (!!metadataRef) {
                    status.addPhaseSession(metadataRef, phaseRef.key, phaseSession);
                }
            });

            return true;
        }

        return false;

    }

    async fetchTestPhases(status: TestMetaStatus, phaseKeys?: string[]) {
        const missingPhaseKeys = phaseKeys?.filter(p => !status.test.phases?.get(p));
        if (!phaseKeys || missingPhaseKeys!.length > 0) {
            const responses = await getTestPhasesV2([status.test.id]); /** TODO: filter by phaseKeys seems to be not working */
            if (!status.test.phases) {
                status.test.phases = new Map();
            }
            responses.forEach((test) => {
                if (test.testId !== status.test.id) {
                    return;
                }
                test.phases.forEach((phase) => {
                    const phaseRef = this.createPhaseRef(phase);
                    status.test.phases!.set(phaseRef.key, phaseRef);
                });
            });
        }
    }

    async fetchTestRecipes(testRef: TestNameRef, recipeIds: string[]) {

        const responses = await getTestRecipesV2(recipeIds);
        const refs: TestRecipeRef[] = [];
        responses.forEach((recipe) => {
            if (recipe.nameRef !== testRef.id) {
                return;
            }

            const recipeRef = this.createRecipeRef(recipe);
            recipeRef.test = testRef;
            refs.push(recipeRef);
        });

        testRef.updateRecipes(refs);

    }

    async fetchTestAudit(testRefs: TestNameRef[], auditMap: Map<TestNameRef, TestAuditRecords>, weeks?: number) {

        const tests: Map<string, TestNameRef> = new Map(testRefs.filter((t) => t.version >= this.version).map((t) => [t.id, t]));
        if (tests.size) {

            // create an audit record collection for each test
            tests.forEach((test) => {
                if (!auditMap.get(test)) {
                    auditMap.set(test, new TestAuditRecordsV2(test));
                }
            });

            const testsAudited = new Set<string>();
            const minQueryDate = weeks? new Date(new Date().valueOf() - (minuteInWeek * weeks * 60000)) : undefined;
            // send the query
            const responses = await getTestAuditsV2(testRefs.map(t => t.id), minQueryDate?.toISOString());
            responses.forEach((audit) => {
                const test = tests.get(audit.testId);
                if (!!test) {
                    testsAudited.add(audit.testId);
                    const auditEntry = TestDataV2.createTestAudit(audit);
                    const auditRecords = auditMap.get(test)!;
                    auditRecords.appendRecord(auditEntry);
                }
            });

            const noAuditTests = (new Set<string>(tests.keys())).difference(testsAudited);
            if (noAuditTests.size) {
                // Make sure we have complete data from unaudit tests
                const responses2 = await getTestNamesV2(Array.from(noAuditTests));
                responses2.forEach((nameRef) => {
                    const test = tests.get(nameRef.id);
                    if (!!test) {
                        test.profile = nameRef.profile;
                        test.team = nameRef.team;
                        test.harness = nameRef.harness;
                        test.intent = nameRef.intent;
                    }
                });
            }
        }

    }

    private testNameRefMap: Map<string, TestNameRef> = new Map();
    private metadataRefMap: Map<string, MetadataRef> = new Map();
    private testTagRefMap: Map<string, TestTagRef> = new Map();
    private testRecipeRefMap: Map<string, TestRecipeRef> = new Map();

    private createTestNameRef(test: GetTestNameResponse): TestNameRef {
        const globalKey = test.id;
        let item = this.testNameRefMap.get(globalKey);
        if (!item) {
            item = new TestNameRef(test.id, test.key, test.name, this.version);
            item.team = test.team;
            item.harness = test.harness;
            item.profile = test.profile;
            this.testNameRefMap.set(globalKey, item);
        }

        return item;
    }

    private createMetadataRef(item: GetTestMetadataResponse): MetadataRef {
        const globalKey = item.id;
        let meta = this.metadataRefMap.get(globalKey);
        if (!meta) {
            meta = new MetadataRef(item.id, item.entries, this.version);
            this.metadataRefMap.set(globalKey, meta);
        }

        return meta;
    }

    private createTagRef(item: GetTestTagResponse): TestTagRef {
        const globalKey = item.id;
        let tag = this.testTagRefMap.get(globalKey);
        if (!tag) {
            tag = new TestTagRef(item.id, item.name, this.version);
            this.testTagRefMap.set(globalKey, tag);
        }

        return tag;
    }

    private createPhaseRef(item: GetPhaseResponse): TestPhaseRef {
        return new TestPhaseRef(item.id, item.key, item. name, this.version);
    }

    private createRecipeRef(item: TestRecipeResponse): TestRecipeRef {
        const globalKey = item.id;
        let recipe = this.testRecipeRefMap.get(globalKey);
        if (!recipe) {
            recipe = new TestRecipeRef(item.id, item.streamId, item.jobTemplateId, item.jobStep, item.nodeClass, item.version, item.timeoutMinutes, this.version);
            recipe.workflowId = item.workflowId;
            recipe.params = recipe.params;
            this.testRecipeRefMap.set(globalKey, recipe);
        }

        return recipe;
    }

    static createTestAudit(item: TestAuditResponse): TestAudit {

        const auditDate = new Date(item.userInputDate);
        const audit = new TestAudit(item.testId, item.userId, auditDate, item.enableSchedule);
        audit.isUnderAudit = item.isUnderAudit;
        audit.lastAuditDate = item.lastAuditDate? new Date(item.lastAuditDate) : undefined;
        audit.owner = item.owner;
        audit.customers = item.customers;
        audit.notifications = item.notifications ? new Map(Object.keys(item.notifications).map(stream => [stream, item.notifications![stream]])): undefined;
        audit.team = item.team;
        audit.harness = item.harness;
        audit.profile = item.profile;
        audit.intent = item.intent;
        audit.notes = item.notes;

        return audit;
    }

    private createTestSession(test: GetTestSessionResponse): TestSessionResult {
        const start = new Date(test.startDateTime);
        const result = new TestSessionResult(test.id, start, test.duration, test.commitId, test.commitOrder, test.nameRef, test.metadataId, test.streamId, test.outcome, test.testDataId);
        result.phasesSucceededCount = test.phasesSucceededCount;
        result.phasesFailedCount = test.phasesFailedCount;
        result.phasesUnspecifiedCount = test.phasesUndefinedCount;
        result.jobId = test.jobId;
        result.stepId = test.stepId;
        result.tagIds = test.tagIds;
        result.recipeId = test.recipeId;
        result.errorFingerprints = !test.errorFingerprints? undefined : new Map(test.errorFingerprints.map(i => [i.key, i.phases]));

        return result;
    }

    private createPhaseSession(phase: GetTestPhaseSessionResponse): PhaseSessionResult {
        const start = new Date(phase.startDateTime);
        const result = new PhaseSessionResult(phase.id, start, phase.duration, phase.commitId, phase.commitOrder, phase.sessionId, phase.phaseRef, phase.metadataId, phase.streamId, phase.outcome);
        result.hasWarning = phase.hasWarning;
        result.errorFingerprint = phase.errorFingerprint;
        result.jobId = phase.jobId;
        result.stepId = phase.stepId;
        result.tagIds = phase.tagIds;

        return result;
    }

    private createSessionDetails(testData: TestData): TestSessionDetails {
        const testDetails = testData.data as TestSessionDetailsResponse;
        const start = new Date(testDetails.summary.dateTime);
        const result = new TestSessionDetails(testData.key, testDetails.summary.testName, start, testDetails.summary.timeElapseSec, testData.commitId.name, testData.commitId.order, testData.streamId, testData.jobId, testData.stepId);
        Object.entries(testDetails.devices).forEach(([key, d]) => {
            result.devices.set(key, new DeviceRef(d.name, d.appInstanceLogPath, d.metadata, this.version));
        });
        result.phases.push(...testDetails.phases.map(p => this.createPhaseStatus(p, result.devices, result)));
        result.phasesSucceededCount = testDetails.summary.phasesSucceededCount;
        result.phasesFailedCount = testDetails.summary.phasesFailedCount;
        result.phasesUnspecifiedCount = testDetails.summary.phasesUndefinedCount;
        result.outcome = testDetails.summary.phasesFailedCount > 0 ?
                            TestOutcome.Failure
                            : testDetails.summary.phasesUndefinedCount > 0 ?
                                TestOutcome.Unspecified
                                : testDetails.summary.phasesTotalCount > 0 ?
                                    TestOutcome.Success : TestOutcome.Skipped;
        result.preflightCommitId = testData.preflightCommitId;
        result.tags = testDetails.tags;
        result.commandline = testDetails.summary.commandline;

        // search for known metadata
        const metaFilter = new Map(Object.entries(testDetails.metadata).map(([key, value]) => [key, [value]]));
        const metaRef = this.metadataRefMap.values().find((meta) => meta.isMatch(metaFilter, true)) ?? this.createMetadataRef({id: testData.id, entries: testDetails.metadata});
        result.meta = metaRef;

        // search for known test name
        const testNameRef = this.testNameRefMap.values().find((nameRef) => nameRef.key === testData.key) ?? this.createTestNameRef(
            {
                id: testData.id,
                key: testData.key,
                name: result.name,
                team: testDetails.description?.team,
                harness: testDetails.description?.harness,
                profile: testDetails.description?.profile,
                intent: testDetails.description?.intent
            });
        result.test = testNameRef;

        return result;
    }

    private createPhaseStatus(phase: PhaseSessionDetailsResponse, deviceRefs: Map<string, DeviceRef>, parent: TestSessionDetails) {
        const devices = phase.deviceKeys.map(key => deviceRefs.get(key) ?? new DeviceRef(key));
        const result = new TestPhaseStatusV2(phase, devices, parent);
        result.tags = phase.tags;

        return result;
    }
}

