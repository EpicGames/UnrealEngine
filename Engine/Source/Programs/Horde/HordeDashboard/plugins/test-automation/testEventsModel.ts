// Copyright Epic Games, Inc. All Rights Reserved.

/// An event entry
export type EventEntry = {

    time: string;

    id: number;

    level: EventLevel;

    message: string;

    format: string;

    properties?: Record<string, string | number | Record<string, string | number | undefined> | undefined>;
}

/// The event level
export enum EventLevel {

    Unspecified = "Unspecified",

    Information = "Information",

    Warning = "Warning",

    Error = "Error",

    Critical = "Critical"
}

/// Known property types
export enum LogPropertyType {
    ImageCompare = "Image Compare",
    Artifacts = "Artifacts",
    EmbeddedHTML = "Embedded HTML",
    URLLink = "URL"
}

/// Image Compare detailed properties
export type ImageCompare = {
    "$type": LogPropertyType.ImageCompare;
    "$text": string;

    //** The path to the unapproved image from the artifacts */
    unapproved: string;

    //** The path to the unapproved metadata associated with */
    //** the image from the artifacts */
    unapproved_metadata?: string;

    //** The path to the approved image from the artifacts */
    approved?: string;

    //** The path to the approved metadata associated with */
    //** the image from the artifacts */
    approved_metadata?: string;

    //** The path to the delta image from the artifacts */
    difference?: string;

    //** The path to the json report associated with the image from the artifacts */
    difference_report?: string;
}

/// Artifacts detailed properties
export type Artifacts = {
    [Key in string]: string | undefined;
}

/// Embedded html detailed properties
export type EmbeddedHTML = {
    "$type": LogPropertyType.EmbeddedHTML;
    "$text": string;

    //** The path to the html artifact */
    path: string;
}

/// URL Link detailed properties
export type URLLink = {
    "$type": LogPropertyType.URLLink;
    "$text": string;

    //** the url link  */
    href: string;
}
