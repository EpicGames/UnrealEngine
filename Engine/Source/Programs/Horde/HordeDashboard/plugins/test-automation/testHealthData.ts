// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { MetadataRef, TestAudit, TestDataHandler, TestNameRef, TestRecipeRef } from "./testData";
import { StoreKey, userStorage } from "./testAutomationUserStorage";
import { getUsersInfo } from "./api";
import { GetUserResponse } from "horde/backend/Api";

export enum TestHealth {
    Healthy = "Healthy",
    Fair = "Fair",
    Unstable = "Unstable",
    Bad = "Bad",
    Broken = "Broken",
    NA = "N/A",
}

export const healthStars: Map<TestHealth, number> = new Map([
    [TestHealth.Healthy, 5],
    [TestHealth.Fair, 4],
    [TestHealth.Unstable, 3],
    [TestHealth.Bad, 2],
    [TestHealth.Broken, 1],
    [TestHealth.NA, 0],
]);

export const healthColors: Map<TestHealth, string> = new Map([
    [TestHealth.Healthy, "#67bc54"],
    [TestHealth.Fair, "#cae120ff"],
    [TestHealth.Unstable, "#c89320"],
    [TestHealth.Bad, "#ce7253ff"],
    [TestHealth.Broken, "#ce5353"],
    [TestHealth.NA, "#b5b5b5"],
]);

export enum TestCadence {
    Nightly = "Nightly",
    Incremental = "Incrementally",
    Hourly = "Hourly",
    Daily = "Daily",
    Limited = "Limited",
    Weekly = "Weekly",
    NotRun = "Not Run",
    NA = "N/A"
}

export const cadenceOrders: Map<TestCadence, number> = new Map([
    [TestCadence.Incremental, 0],
    [TestCadence.Daily, 1],
    [TestCadence.Weekly, 2],
    [TestCadence.Limited, 3],
    [TestCadence.NotRun, 4],
    [TestCadence.NA, 5],
]);

export const cadenceColors: Map<TestCadence, string> = new Map([
    [TestCadence.Incremental, "#c8b72081"],
    [TestCadence.Hourly, "#9fce538d"],
    [TestCadence.Daily, "#53cea18d"],
    [TestCadence.Weekly, "#1f49a579"],
    [TestCadence.Limited, "#4a3e6c8b"],
    [TestCadence.NotRun, "#b5b5b516"],
    [TestCadence.NA, "#6e6e6e16"],
]);

export const booleanColors: Map<boolean, string> = new Map([
    [true, "#cbc82248"],
    [false, "#67676748"],
]);

class LazyString {
    constructor(getter: () => string | undefined) {
        this.getter = getter;
        this.cached = null;
    }

    get value() {
        if (this.cached === null) {
            this.cached = this.getter();
            this.getter = () => undefined;
        }
        return this.cached;
    }

    private getter: () => string | undefined;
    private cached: string | undefined | null;
}

export type TestHealthExplain = {
    catastrophicFailureRate: number;
    failureRate: number;
    successRate: number;
    redundantErrorRate: number;
}

export type TestHealthTiming = {
    medianToInterruptSecs: number;
    medianToCompleteRunSecs: number;
    latestCompletedRun: Date | undefined;
}

export class TestHealthItem {
    constructor(test: TestNameRef, handler: TestDataHandler, stream: string) {
        this.nameRef = test;
        this.handler = handler;
        this.metaRefs = [];
        this.health = TestHealth.NA;
        this.cadence = TestCadence.NA
        this.stream = stream;
        this.lazyDisplayMeta = new LazyString(() => undefined);
        makeObservable(this)
    }

    private handler: TestDataHandler;
    private lazyDisplayMeta: LazyString;
    private transientCopy?: TestAudit;

    nameRef: TestNameRef;
    metaRefs: MetadataRef[];
    health: TestHealth;
    cadence: TestCadence;
    stream: string;

    healthExplain?: TestHealthExplain;
    timings?: TestHealthTiming;

    @observable
    updated: number = 0;
    @action
    setUpdated() {
        this.updated += 1;
    }
    subscribeUpdate() {
        if (this.updated) { }
    }

    get displayMeta() {
        return this.lazyDisplayMeta.value;
    }
    setLazyDisplayMeta(getter: () => string | undefined) {
        this.lazyDisplayMeta = new LazyString(getter);
    }

    private getOrStartTransaction(): TestAudit {
        if (!this.transientCopy) {
            const auditRecord = this.handler.testAudits.get(this.nameRef);
            let latestClone = auditRecord?.getLatest()?.clone();
            if (!latestClone) {
                latestClone = new TestAudit(this.nameRef.id, "", new Date(Date.now()), true);
            }
            this.transientCopy = latestClone;
        }
        return this.transientCopy;
    }

    private get audit(): TestAudit | undefined {
        return this.transientCopy ?? this.handler.testAudits.get(this.nameRef)?.getLatest();
    }

    get isTransient(): boolean {
        return this.transientCopy !== undefined;
    }

    async commitTransaction() {
        if (!this.isTransient) return;

        await this.handler.testAudits.get(this.nameRef)?.post(this.transientCopy!);

        this.transientCopy = undefined;
        this.setUpdated();
    }

    discardTransaction() {
        if (this.isTransient) {
            this.transientCopy = undefined;
            this.setUpdated();
        }
    }

    stashTransaction() {
        if (this.isTransient) {
            userStorage.setItem(StoreKey.HealthDetailsTransaction, this.transientCopy);
        }
    }

    loadStashedTransaction() {
        const stash = userStorage.getItem(StoreKey.HealthDetailsTransaction) as TestAudit | undefined;
        if (stash && stash.testId === this.nameRef.id) {
            this.transientCopy = stash;
            userStorage.setItem(StoreKey.HealthDetailsTransaction, undefined);
            return true;
        }
        return false;
    }

    get name() {
        return this.nameRef.name;
    }

    get profile() {
        return this.audit?.profile ?? this.nameRef.profile;
    }
    
    set profile(value: string | undefined) {
        this.getOrStartTransaction().profile = value;
        this.setUpdated();
    }

    get team() {
        return this.audit?.team ?? this.nameRef.team;
    }

    set team(value: string | undefined) {
        this.getOrStartTransaction().team = value;
        this.setUpdated();
    }

    get harness() {
        return this.audit?.harness ?? this.nameRef.harness;
    }

    set harness(value: string | undefined) {
        this.getOrStartTransaction().harness = value;
        this.setUpdated();
    }

    get intent() {
        return this.audit?.intent ?? this.nameRef.intent;
    }

    set intent(value: string | undefined) {
        this.getOrStartTransaction().intent = value;
        this.setUpdated();
    }

    get underAudit() {
        return !!this.audit?.isUnderAudit;
    }

    set underAudit(value: boolean) {
        this.getOrStartTransaction().isUnderAudit = value;
        if (value) {
            this.getOrStartTransaction().lastAuditDate = new Date(Date.now());
        }
        this.setUpdated();
    }

    get lastAuditDate() {
        return this.audit?.lastAuditDate;
    }

    set lastAuditDate(value: Date | undefined) {
        this.getOrStartTransaction().lastAuditDate = value;
        this.setUpdated();
    }

    get notes() {
        return this.audit?.notes;
    }

    set notes(value: string | undefined) {
        this.getOrStartTransaction().notes = value;
        this.setUpdated();
    }

    get user() {
        return this.audit?.userId;
    }

    get userInputDate() {
        return this.audit?.userInputDate;
    }

    get owner() {
        return this.audit?.owner;
    }

    set owner(value: string | undefined) {
        this.getOrStartTransaction().owner = value;
        this.setUpdated()
    }

    get customers() {
        return this.audit?.customers;
    }

    set customers(value: string[] | undefined) {
        if (value && value.length === 0) value = undefined;
        this.getOrStartTransaction().customers = value;
        this.setUpdated()
    }

    get notification() {
        return this.audit?.notifications?.get(this.stream);
    }

    set notification(value: string | undefined) {
        const audit = this.getOrStartTransaction();
        if (!value) {
            if (audit.notifications) {
                audit.notifications.delete(this.stream);
                if (audit.notifications.size === 0) {
                    audit.notifications = undefined;
                }
            }
        } else {
            if (!audit.notifications) {
                audit.notifications = new Map();
            }            
            audit.notifications.set(this.stream, value);
        }
        this.setUpdated()
    }

    linkRecipeIds(ids: Iterable<string>) {
        this.recipeIds = Array.from(ids);
    }

    async fetchRecipes() {
        if (this.recipeIds && this.recipeIds.length) {
            await this.handler.queryRecipes(this.nameRef, this.recipeIds);
            this.cached_recipes = undefined;
        }
    }

    private cached_recipes?: TestRecipeRef[];
    private recipeIds?: string[];
    get recipes() {
        if (!this.recipeIds) return;
        if (!this.cached_recipes) {
            this.cached_recipes = this.nameRef.recipeRefs?.keys().filter(r => r.streamId === this.stream).toArray();
        }
        if (!this.cached_recipes || !this.cached_recipes.length) return;
        return this.cached_recipes;
    }

    get fetchingRecipes() {
        return this.handler.queryRecipesLoading;
    }

    getAsArray(usersInfo?: UsersInfoCache): string[] {
        return [
            this.name,
            this.stream,
            this.health.toString(),
            this.timings?.latestCompletedRun?.toISOString() ?? "",
            this.cadence.toString(),
            this.team ?? "",
            this.profile ?? "",
            this.harness ?? "",
            this.intent ?? "",
            this.notes ?? "",
            usersInfo?.getInfo(this.owner)?.name ?? this.owner ?? "",
            this.customers?.map(c => usersInfo?.getInfo(c)?.name ?? c).join(', ') ?? "",
            this.notification ?? "",
            this.underAudit.toString(),
            this.lastAuditDate?.toISOString() ?? "",
            usersInfo?.getInfo(this.user)?.name ?? this.user ?? "",
            this.userInputDate?.toISOString() ?? "",
        ];
    }
    
    static getFields() {
        return [
            'Name',
            'Stream',
            'Health',
            'Latest Completed Run',
            'Cadence',
            'Team',
            'Profile',
            'Harness',
            'Intent',
            'Notes',
            'Owner',
            'Customers',
            'Notification',
            'Under Audit',
            'Last Audit Date',
            'User',
            'User Input Date',
        ];
    }
}

export class UsersInfoCache {
    constructor() {
        this.cache = new Map();
    }

    async fetchUsersInfo(userIds: string[]) {
        const missingUserIds = (new Set(userIds)).keys().filter(id => !this.cache.has(id)).toArray();
        if (missingUserIds.length > 0) {
            await getUsersInfo(missingUserIds).then((users) => {
                users.forEach(user => this.cache.set(user.id, user));
            });
            missingUserIds.filter(id => !this.cache.has(id)).forEach(id => this.cache.set(id, {id: id, name: "unknown"}));
        }

        return missingUserIds.map(id => this.cache.get(id));
    }

    getInfo(userId?: string) {
        return this.cache.get(userId);
    }

    private cache : Map<string | undefined, GetUserResponse>;
}
