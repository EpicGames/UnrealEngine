// Copyright Epic Games, Inc. All Rights Reserved.

import * as d3 from "d3";
import { TestDataHandler, TestNameRef, MetadataRef, TestMetaStatus, TestSessionResult } from './testData';
import dashboard, { StatusColor } from 'horde/backend/Dashboard';
import { getShortNiceTime, msecToElapsed } from 'horde/base/utilities/timeUtils';
import { getHordeStyling } from 'horde/styles/Styles';
import { getStatusColors, statusTexts, SessionValues, SessionListSelector } from "./testAutomationCommon";
import { renderToString } from "react-dom/server"
import { TestOutcome } from "./api";
import { timelineHandler, Margin, constructTimeline, calculateAverageSessionFrequencyHours, getSessionsNearTimeFactory } from "./timelineGraph";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type DivSelectionType = d3.Selection<HTMLDivElement, unknown, HTMLElement, undefined>;
type SelectionBaseType = d3.Selection<d3.BaseType, unknown, null, undefined>;

const clampMin = (x: number) => x !== 0 && x <= 0.3? 0.3: x;

const width = 1000;
const circleSize = 5;
const highFrequencyThreshold = 4; // in hours

export class TestHistoryGraph {

    constructor(test: TestNameRef, stream: string, handler: TestDataHandler, commonMetaKeys?: string[], onClickCallback?: (session: TestSessionResult) => void) {
        this.test = test;

        this.handler = handler;
        this.onClickCallback = onClickCallback;
        this.commonMetaKeys = commonMetaKeys;

        this.status = handler.getStatusStream(stream)!.tests.get(test)!;

        this.margin = { top: 0, right: 32, bottom: 0, left: 160 };
        this.clipId = `test_history_clip_path_${test.key}_${stream}`;
    }

    initData(selectedMetaKeys?: string[]) {
        this.metaRefs = this.handler.filteredMetadata.filter((m) => this.status.includesMetadata(m) && (!selectedMetaKeys || selectedMetaKeys.includes(m.id)));
        const commonMetaKeys = this.commonMetaKeys = this.commonMetaKeys ?? MetadataRef.identifyCommonKeys(this.metaRefs);

        const selectedTagRefs = this.handler.selectedTags;

        this.sessions = [];
        this.metaInfo = new Map();

        this.metaRefs.forEach((meta, index) => {
            const metaLastSession = this.status.getLastSession(meta);
            if (!!metaLastSession && !!selectedTagRefs
                && !this.status.sessions.get(meta)!.includeTags(selectedTagRefs)) {
                // skip if it filters by tags and no tag match
                return;
            }

            const elements: string[] = meta.getValuesExcept(commonMetaKeys);
            this.metaNames.set(meta, elements.join(" / "));

            this.sessions.push(...this.status.sessions.get(meta)!.history);
            this.metaInfo.set(meta.id, {y: 0 , index: index});
        });
    }

    render(container: HTMLDivElement, forceRender?: boolean, selectedMetaKeys?: string[]) {

        if (this.hasRendered && !forceRender) {
            return;
        }

        const { modeColors } = getHordeStyling();

        this.clear(container);
        this.cleanupCallback();

        this.hasRendered = true;

        this.initData(selectedMetaKeys);

        // initialize timeline handle if not set
        timelineHandler.init(this.test.id, this.handler, width, this.margin, false);
        if (timelineHandler.isTimelineAtExtreme && this.averageFrequencyHours < highFrequencyThreshold) {
            timelineHandler.rescaleTimeline(this.averageFrequencyHours / highFrequencyThreshold);
        }

        const handler = this.handler;
        const sessions = this.sessions;

        const X = _d3.map(sessions, (r) => handler.commitIdDates.get(r.commitId)!.getTime() / 1000);
        const Y = _d3.map(sessions, (r) => this.metaInfo.get(r.metadataId)!.index);

        const yDomain = new _d3.InternSet(Y as any);

        const I = d3.range(X.length);

        const yPadding = 1;
        const height = Math.ceil((yDomain.size + yPadding) * 16) + this.margin.top + this.margin.bottom;

        const yRange = [this.margin.top, height - this.margin.bottom];

        const yScale = _d3.scalePoint(yDomain, yRange).round(true).padding(yPadding);

        const getSessionsNearTime = getSessionsNearTimeFactory(sessions, X, this.handler.filterState!.weeks! * 7, this.metaInfo.size);

        const svg = d3.select(container)
            .append("svg")
            .attr("width", width)
            .attr("height", height + 24)
            .attr("viewBox", [0, 0, width, height + 24] as any)

        this.timelineCleanupCallback = constructTimeline(
            svg,
            () => {
                // update meta groups
                metaGroups.call(drawMetaGroups);
            },
            height - this.margin.bottom
        );    

        const metaGroups = svg.append("g")
            .attr("id", "meta-groups");

        const drawMetaGroups = (groups: SelectionType) => {
            const group = groups.selectAll(".meta")
                .data(_d3.group(I, i => Y[i]), ([y]: any) => y)
                .join(
                    enter => enter.append("g")
                        .attr("class", "meta")
                        .attr("id", ([y]: any) => `meta${y as any}`)
                        .attr("transform", ([y]: any) => `translate(0,${(yScale(y) as any) + 16})`)
                        .each(([y]: any, i, nodes: any) => {
                            const node = d3.select(nodes[i]);
                            node.append("line")
                                .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
                                .attr("stroke-width", 1)
                                .attr("stroke-linecap", 4)
                                .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
                                .attr("x1", this.margin.left)
                                .attr("x2", width - this.margin.right);

                            node.append("text")
                                .attr("text-anchor", "start")
                                .style("alignment-baseline", "left")
                                .style("font-family", "Horde Open Sans Regular")
                                .style("font-size", 10)
                                .attr("dy", "0.15em") // center stream name
                                .attr("fill", dashboard.darktheme ? "#E0E0E0" : "#2D3F5F")
                                .text(([y]: any) => this.metaNames.get(this.metaRefs[y])!.substring(0, 32));
                        }),
                    update => update.attr("transform", ([y]: any) => `translate(0,${(yScale(y) as any) + 16})`),
                 );

            group.selectAll(".session-circle")
                .data(
                    ([, I]: any) => _d3.groups(I, i => X[i])
                        .filter(([, items]: any) => X[items[0] as any] >= timelineHandler.startDate! && X[items[0] as any] <= timelineHandler.endDate!),
                    ([, items]: any) => sessions[items[0]].id)
                .join(
                    enter => enter.append("g")
                        .attr("class", "session-circle")
                        .attr("id", ([, items]: any) => `circle${sessions[items[0]].id}`)
                        .attr("transform", ([, items]: any) => `translate(${timelineHandler.timelineXScale!(X[items[0] as any])},0)`)
                        .append("g")
                        .attr("class", "circleScale")
                        .style("cursor", "pointer")
                        .each(([, items]: any, i, nodes: any) => SessionConcentricCircle(nodes[i], items.map(i => sessions[i]), 0, circleSize)),
                    update => update.attr("transform", ([, items]: any) => `translate(${timelineHandler.timelineXScale!(X[items[0] as any])},0)`),
                 );
        }
        metaGroups.call(drawMetaGroups);

        const setCircleStatus = (selection: SelectionBaseType, items: TestSessionResult[], index: number) => {
            const circle = selection.select(".circleScale");
            circle.selectAll("*").remove();
            SessionConcentricCircle(circle.node() as any, items, index, circleSize);
        }

        // Attempt to select an existing div with id 'tooltip' from the body, or create one if it doesn't exist
        let tooltip: DivSelectionType = d3.select("body").select("div#tooltip");
        if (tooltip.empty()) {
            tooltip = d3.select("body")
                .append("div")
                .attr("id", "tooltip")
                .style("display", "none")
                .style("background-color", modeColors.background)
                .style("border", "solid")
                .style("border-width", "1px")
                .style("border-radius", "3px")
                .style("border-color", dashboard.darktheme ? "#413F3D" : "#2D3F5F")
                .style("padding", "6px")
                .style("position", "absolute")
                .style("pointer-events", "none");
        }
        this.tooltip = tooltip;

        // populate metaInfo y value
        this.metaInfo.forEach(meta => meta.y = yScale(meta.index as any)!);

        const closestData = (x: number, y: number): TestSessionResult[] | undefined => {

            if (x < this.margin.left - 16 || y < this.margin.top + 16) {
                return undefined;
            }

            y -= 16;

            const closest = getSessionsNearTime(timelineHandler.timelineXScale!.invert(x)).reduce((best, session: TestSessionResult) => {
                const absy = Math.abs(this.metaInfo.get(session.metadataId)!.y - y);
                if (absy > 9) return best; // skip if too far away from the mouse y position

                const sessionTime = handler.commitIdDates.get(session.commitId)!.getTime() / 1000;
                const absx = Math.abs(timelineHandler.timelineXScale!(sessionTime) - x);

                const lengthSqr = absy * absy + absx * absx;

                if (lengthSqr < best.value) {
                    return { sessions: [session], value: lengthSqr, meta: session.metadataId };
                } else {
                    if (lengthSqr === best.value && session.metadataId === best.meta) best.sessions.push(session);
                    return best;
                }
            }, { sessions: [] as TestSessionResult[], value: Number.MAX_SAFE_INTEGER, meta: '' });

            if (closest.sessions.length) {
                return closest.sessions;
            }

            return undefined;

        }

        const getInfoSession = (items: TestSessionResult[], index: number) => {
            const item = items[index];
            const metaIndex = this.metaInfo.get(item.metadataId)!.index;
            const cmeta = this.metaRefs[metaIndex].getValuesExcept(this.commonMetaKeys!).join(" / ").substring(0, 64);
            const date = getShortNiceTime(item.start, true, true);
            let desc = "";
            if (items.length > 1) {
                desc += renderToString(SessionListSelector(items, index, '(mouse wheel to select)'));
            }
            if (!!cmeta || items.length > 1) desc += (items.length > 1? `::${(index + 1)} ` : '') + `${cmeta} <br/>`;
            desc += `Commit ${item.commitId} <br/>`;
            desc += `Duration ${msecToElapsed(item.duration * 1000, true, true)} <br/>`;
            desc += `${date} <br/>`;

            if (item.phasesSucceededCount + item.phasesFailedCount + item.phasesUnspecifiedCount > 1)
                desc += renderToString(SessionValues(item, {fontSize: 9}));

            return `<span style="color: ${modeColors.text};">${desc}</span>`;;
        }

        const handleMouseMove = (event: any) => {

            const mouseX = _d3.pointer(event)[0];
            const mouseY = _d3.pointer(event)[1];

            const closestItems = closestData(mouseX, mouseY);
            if (!closestItems) {
                handleMouseLeave(undefined);
                return;
            }

            const closest = closestItems[0];
            if (this.tooltipItem?.sessionId !== closest.id) {
                handleMouseLeave(undefined);

                const circleSelection = svg.select(`#circle${closest.id}`);
                circleSelection.raise();
                circleSelection.select(".circleScale").attr("transform", `scale(2.5)`);

                this.tooltipItem = {sessionId: closest.id, index: 0, items: closestItems, delta: 0, circleSelection: circleSelection};
                const metaIndex = this.metaInfo.get(closest.metadataId)!.index;

                svg.select(`#meta${metaIndex}`).raise();

                const timeStamp = handler.commitIdDates.get(closest.commitId)!.getTime() / 1000;
                const tx = timelineHandler.timelineXScale!(timeStamp);
                const ty = yScale(metaIndex as any)!
                const desc = getInfoSession(closestItems, 0);
                // Compute the bounding rectangle of the container to get its absolute position
                const containerRect = container.getBoundingClientRect();
                const offsetX = containerRect.left + window.scrollX;
                const offsetY = containerRect.top + window.scrollY;
                const absTx = tx + offsetX;
                const absTy = ty + offsetY;

                this.updateTooltip(absTx, absTy, desc);
            }

        }

        const handleMouseLeave = (event: any) => {

            if (this.tooltipItem) {
                tooltip.style("display", "none");
                const circleSelection = this.tooltipItem.circleSelection;
                circleSelection.select(".circleScale").attr("transform", `scale(1)`);
                if (this.tooltipItem.items.length > 1) {
                    setCircleStatus(circleSelection, this.tooltipItem.items, 0);
                }
                this.tooltipItem = undefined;
            }

        }

        const handleMouseClick = (event: any) => {

            const mouseX = _d3.pointer(event)[0];
            const mouseY = _d3.pointer(event)[1];

            if (this.tooltipItem) {
                const closestItems = closestData(mouseX, mouseY);
                if (closestItems && this.onClickCallback) {
                    this.onClickCallback(closestItems[this.tooltipItem.index]);
                    handleMouseLeave(undefined);
                }
            }

        }

        const handleMouseWheel = (event: any) => {

            if (this.tooltipItem && this.tooltipItem.items.length > 1) {
                this.tooltipItem.delta += event.wheelDelta;
                if (Math.abs(this.tooltipItem.delta) >= 240) {
                    this.tooltipItem.index += this.tooltipItem.delta > 0? 1 : -1;
                    this.tooltipItem.delta = 0;

                    if (this.tooltipItem.index < 0) {
                        this.tooltipItem.index += this.tooltipItem.items.length;
                    } else if (this.tooltipItem.index >= this.tooltipItem.items.length) {
                        this.tooltipItem.index -= this.tooltipItem.items.length;
                    }

                    setCircleStatus(this.tooltipItem.circleSelection, this.tooltipItem.items, this.tooltipItem.index);
                    const desc = getInfoSession(this.tooltipItem.items, this.tooltipItem.index);
                    this.updateTooltip(undefined, undefined, desc);
                }
                event.preventDefault();
            }
            
        }

        svg.on("mousemove", (event) => handleMouseMove(event));
        svg.on("mouseleave", (event) => handleMouseLeave(event));
        svg.on("click", (event) => handleMouseClick(event));
        svg.on("wheel", (event) => handleMouseWheel(event));

    }

    updateTooltip(x?: number, y?: number, html?: string) {
        if (!this.tooltip) {
            return;
        }

        if (x !== undefined && y != undefined) {
            this.tooltip
                .style("top", `${y}px`)
                .style("left", `${x}px`);
        }

        this.tooltip
            .style("display", "block")
            .html(html ?? "")
            .style("position", `absolute`)
            .style("width", `max-content`)
            .style("transform", "translate(-108%, -97%)")
            .style("font-family", "Horde Open Sans Semibold")
            .style("font-size", "10px")
            .style("line-height", "16px")
            .style("shapeRendering", "crispEdges")
            .style("stroke", "none")
            .style("z-index", 2000000);

    }

    clear(container: HTMLDivElement) {
        d3.select(container).selectAll("*").remove();
    }

    cleanupCallback() {
        this.timelineCleanupCallback?.();
        this.timelineCleanupCallback = undefined;
    }

    get averageFrequencyHours() {
        if (this._averageSessionFrequencyHours !== undefined) {
            return this._averageSessionFrequencyHours;
        }
        this._averageSessionFrequencyHours = calculateAverageSessionFrequencyHours(this.handler, this.sessions);
        return this._averageSessionFrequencyHours;
    }

    // test ref
    test: TestNameRef;

    commonMetaKeys?: string[];
    onClickCallback?: (session: TestSessionResult) => void;
    timelineCleanupCallback?: () => void;

    handler: TestDataHandler;
    margin: Margin;

    hasRendered = false;

    status: TestMetaStatus;
    metaRefs: MetadataRef[] = [];
    metaInfo: Map<string, {y: number, index: number}> = new Map();
    sessions: TestSessionResult[] = [];
    // metaRef => meta string
    metaNames: Map<MetadataRef, string> = new Map();

    clipId: string;

    tooltip?: DivSelectionType;
    tooltipItem?: {index: number, items: TestSessionResult[], sessionId: string, delta: number, circleSelection: SelectionBaseType};

    private _averageSessionFrequencyHours?: number;
}

type ItemStack = {value: number, name: string, color: string, title?: boolean};

const PieChart = (container: SVGGElement, data: ItemStack[], radius: number, innerRadius: number = 0, padAngle: number = 0) => {
    const pie = d3.pie<ItemStack>().padAngle(padAngle).value((d) => d.value).sort(null);
    const arc = d3.arc().innerRadius(innerRadius).outerRadius(radius);
    const color = d3.scaleOrdinal()
        .domain(data.map(d => d.name))
        .range(data.map(d => d.color));
    
    const g = d3.select(container)
        .selectAll("pie")
        .data(pie(data))
        .join("path")
            .attr("class", "pie")
            .attr("fill", d => color(d.data.name) as any)
            .attr("d", arc as any)
        .append("title")
            .text(d => d.data.title? `${Math.ceil(d.data.value)}% ${d.data.name}` : "");

    return g;
}

const SessionPieChart = (container: SVGGElement, session: TestSessionResult, radius: number, innerRadius:number = 0, padAngle: number = 0) => {

    const sessionTotalCount = session.phasesSucceededCount + session.phasesFailedCount + session.phasesUnspecifiedCount;
    const metaFailedFactor = Math.ceil(session.phasesFailedCount / (sessionTotalCount || 1) * 10) / 10;
    const metaUnspecifiedFactor = Math.ceil(session.phasesUnspecifiedCount / (sessionTotalCount || 1) * 10) / 10;

    const statusColors = getStatusColors();

    const stack: ItemStack[] = [
        {
            value: metaUnspecifiedFactor * 100,
            name: statusTexts.get(TestOutcome.Unspecified)!,
            color: statusColors.get(StatusColor.Unspecified)!,
        },
        {
            value: metaFailedFactor * 100,
            name: statusTexts.get(TestOutcome.Failure)!,
            color: statusColors.get(StatusColor.Failure)!,
        },
        {
            value: (1 - (metaFailedFactor + metaUnspecifiedFactor)) * 100,
            name: statusTexts.get(TestOutcome.Success)!,
            color: statusColors.get(StatusColor.Success)!
        }
    ];

    return PieChart(container, stack, radius, innerRadius, padAngle);
}

const ConcentricCircle = (container: SVGGElement, data: ItemStack[], radius: number) => {
    const color = d3.scaleOrdinal()
        .domain(data.map(d => d.name))
        .range(data.map(d => d.color));
    
    const g = d3.select(container)
        .selectAll(".centric")
        .data(data)
        .join("circle")
            .attr("class", "centric")
            .attr("fill", d => color(d.name) as any)
            .attr("r", d => radius * d.value)
        .append("title")
            .text(d => d.title? `${Math.ceil(d.value)}% ${d.name}` : "");

    return g;
}

const SessionConcentricCircle = (container: SVGGElement, sessions: TestSessionResult[], index: number, radius: number) => {
    const latestSession = sessions[index];

    const sessionTotalCount = latestSession.phasesSucceededCount + latestSession.phasesFailedCount + latestSession.phasesUnspecifiedCount;
    const metaFailedFactor = Math.ceil(clampMin(latestSession.phasesFailedCount / (sessionTotalCount || 1)) * 5) / 5;
    const metaUnspecifiedFactor = Math.ceil(clampMin(latestSession.phasesUnspecifiedCount / (sessionTotalCount || 1)) * 5) / 5;
    let metaSuccessFactor = 1 - (metaFailedFactor + metaUnspecifiedFactor);
    metaSuccessFactor = metaSuccessFactor < 0? 0 : metaSuccessFactor

    const statusColors = getStatusColors();

    const stack: ItemStack[] = [
        {
            value: metaSuccessFactor && 1,
            name: statusTexts.get(TestOutcome.Success)!,
            color: statusColors.get(StatusColor.Success)!
        },
        {
            value: metaUnspecifiedFactor && ((metaSuccessFactor && (metaFailedFactor + metaUnspecifiedFactor)) || 1),
            name: statusTexts.get(TestOutcome.Unspecified)!,
            color: statusColors.get(StatusColor.Unspecified)!,
        },
        {
            value: metaFailedFactor,
            name: statusTexts.get(TestOutcome.Failure)!,
            color: statusColors.get(StatusColor.Failure)!,
        },
    ];

    const g = ConcentricCircle(container, stack, radius);

    if (sessions.length > 1) {
        d3.select(container)
            .append("text")
                .attr("text-anchor", "start")
                .style("alignment-baseline", "left")
                .style("font-family", "Horde Open Sans Regular")
                .style("font-size", 10)
                .attr("y", -2)
                .attr("x", 6)
                .attr("fill", dashboard.darktheme ? "#E0E0E0" : "#2D3F5F")
                .text(`${sessions.length}`);
    }

    return g;
}
