// Copyright Epic Games, Inc. All Rights Reserved.

import * as d3 from "d3";
import { BaseType } from "d3";
import { PhaseSessionResult, TestDataHandler, TestSessionResult } from "./testData";
import { getHordeStyling } from "horde/styles/Styles";
import { getHumanTime } from "horde/base/utilities/timeUtils";
import dashboard from "horde/backend/Dashboard";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type SelectionBaseType = d3.Selection<BaseType, unknown, SVGGElement, unknown>;
export type Margin = { top: number, right: number, bottom: number, left: number }

class TimelineHandler {
    constructor() {
        this._cachedTimelines = new Map();
    }

    init(testid: string, handler: TestDataHandler, width: number, margin: Margin, reverse?: boolean) {
        if (this._testid !== testid)
        {
            this._testid = testid;
            if (!this.minDate || !this.maxDate) {
                this.maxDate = Date.now() / 1000;
                this.minDate = this.maxDate - 60 * 60 * 24 * 7 * handler.filterState.weeks!;
                this.ticks = this.getTicks();
                this.axisXScale = _d3.scaleTime().domain([this.minDate, this.maxDate]);
            }
            if (!this._cachedTimelines.has(testid)) {
                this._cachedTimelines.set(testid, {startDate: this.minDate!, endDate: this.maxDate!});
            }
            this.timelineXScale = _d3.scaleTime();
            this.callbacks = new Set();
        }

        this.axisXScale!.range(reverse? [width - margin.right, margin.left] : [margin.left, width - margin.right]);
        this.timelineXScale!.range(this.axisXScale!.range());
        this.axisRangeWidth = Math.abs(this.axisXScale!.range()[1] - this.axisXScale!.range()[0]);
        this.updateTimelineXScale();
    }

    updateTimelineXScale() {
        const start = Math.min(this.startDate, this.endDate);
        const end = Math.max(this.startDate, this.endDate);
        this.startDate = start;
        this.endDate = end;
        this.timelineXScale.domain([start, end]);
    }

    updateDate(x: number, isStart: boolean) {
        const date = this.axisXScale!.invert(x);
        if (isStart) {
            this.startDate = date;
        } else {
            this.endDate = date;
        }
        this.updateTimelineXScale();
    }

    getXStart() {
        return this.axisXScale!(this.startDate!);
    }
    getXEnd() {
        return this.axisXScale!(this.endDate!);
    }

    getTimelineRangeWidth() {
        return Math.abs(this.getXStart() - this.getXEnd());
    }

    addCallback(callback: () => void) {
        this.callbacks!.add(callback);
    }
    removeCallback(callback: () => void) {
        this.callbacks!.delete(callback);
    }
    triggerCallbacks() {
        // Trigger callbacks by chunks using promises with cancellation.
        // This is to avoid blocking the main thread for too long.
        if (this._triggerCallbacksAbortController) {
            this._triggerCallbacksAbortController.abort();
        }
        this._triggerCallbacksAbortController = new AbortController();
        const localAbort = this._triggerCallbacksAbortController;
        const callbacks = Array.from(this.callbacks!);
        const MAX_DELTA_MS = 10; // Target max milliseconds processing before yielding
        const defer = () => new Promise(resolve => setTimeout(resolve, 0));

        const run = async () => {
            let i = 0;
            while (i < callbacks.length) {
                const startTime = performance.now();
                for (; i < callbacks.length; ++i) {
                    try { callbacks[i](); } catch (err) { /* ignore individual callback errors */ }
                    const now = performance.now();
                    if (now - startTime > MAX_DELTA_MS) {
                        break;
                    }
                }
                if (i < callbacks.length) {
                    await defer();
                    if (localAbort.signal.aborted) break;
                }
            }
        };

        run().finally(() => {
            if (this._triggerCallbacksAbortController === localAbort) {
                // Only clear if this is still most recent
                this._triggerCallbacksAbortController = undefined;
            }
        });
    }

    get isTimelineAtExtreme() {
        return this.endDate === this.maxDate && this.startDate === this.minDate;
    }

    get scaleFactor() {
        return this.getTimelineRangeWidth() / this.axisRangeWidth!;
    }

    rescaleTimeline(factor: number, endDate?: number) {
        if (!this.isTimelineAtExtreme) {
            // if timeline is not at the extremes, don't rescale
            return;
        }
        this.startDate = (endDate ?? this.endDate!) - ((this.endDate! - this.startDate!) * factor);
        this.updateTimelineXScale();
        this.triggerCallbacks(); // trigger callbacks to update previously rendered timeline
    }

    private _triggerCallbacksAbortController?: AbortController;

    private getTicks() {
        const dateMin = new Date(this.minDate! * 1000);
        const dateMax = new Date(this.maxDate! * 1000);

        let ticks: number[] = [];
        for (const date of d3.timeDays(dateMin, dateMax, 1).reverse()) {
            ticks.push(date.getTime() / 1000);
        }

        if (ticks.length > 14) {
            let nticks = [...ticks];
            // remove first and last, will be readded 
            const first = nticks.shift()!;
            const last = nticks.pop()!;

            const n = Math.floor(nticks.length / 12);

            const rticks: number[] = [];
            for (let i = 0; i < nticks.length; i = i + n) {
                rticks.push(nticks[i]);
            }

            rticks.unshift(first);
            rticks.push(last);
            ticks = rticks;
        }

        return ticks;
    }

    get startDate() {
        return this._cachedTimelines.get(this._testid!)!.startDate;
    }
    get endDate() {
        return this._cachedTimelines.get(this._testid!)!.endDate;
    }
    set startDate(date: number) {
        this._cachedTimelines.get(this._testid!)!.startDate = date;
    }
    set endDate(date: number) {
        this._cachedTimelines.get(this._testid!)!.endDate = date;
    }

    private _testid?: string;
    private _cachedTimelines: Map<string, {startDate: number, endDate: number}>;
    maxDate?: number;
    minDate?: number;
    axisXScale?: any;
    axisRangeWidth?: number;
    timelineXScale?: any;
    ticks?: number[];
    callbacks?: Set<() => void>;
}

export const timelineHandler = new TimelineHandler();

export const constructTimeline = (svg: SelectionType, onTimelineChange: () => void, height: number, reverse?: boolean, withoutHeader?: boolean) => {
    const { modeColors } = getHordeStyling();

    // top axis
    const headerAxis = (element: SelectionBaseType) => {
        element.selectAll(".tick")
            .data(timelineHandler.ticks!)
            .join(
                enter => enter.append("g")
                    .attr("class", "tick")
                    .attr("transform", (d: number) => `translate(${timelineHandler.axisXScale!(d)},0)`)
                    .each((d, i, nodes) => {
                        if (!withoutHeader) {
                            d3.select(nodes[i]).append("text")
                                .attr("y", -8)
                                .attr("class", "text")
                                .attr("fill", modeColors.text)
                                .attr("text-anchor", "middle")
                                .text(d => getHumanTime(new Date((d as number) * 1000)))
                        }
                    })
                    .append("line")
                        .attr("class", "line")
                        .attr("y1", -5)
                        .attr("stroke-width", 1)
                        .attr("stroke-linecap", "round")
                        .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B"),
                update => update.attr("transform", (d: number) => `translate(${timelineHandler.axisXScale!(d)},0)`),
                exit => exit.remove()
            );
    }
    const seriesAxis = (element: SelectionBaseType) => {
        element.selectAll(".tick")
            .data(timelineHandler.ticks!)
            .join(
                enter => enter.append("g")
                    .attr("class", "tick")
                    .attr("transform", (d: number) => `translate(${timelineHandler.timelineXScale!(d)},0)`)
                    .attr("visibility", (d: number) => d >= timelineHandler.startDate! && d <= timelineHandler.endDate! ? "visible" : "hidden")
                    .append("line")
                        .attr("class", "line")
                        .attr("y1", -3)
                        .attr("stroke-width", 1)
                        .attr("stroke-linecap", "round")
                        .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
                        .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25).clone()
                        .attr("y2", height),
                update => update.attr("transform", (d: number) => `translate(${timelineHandler.timelineXScale!(d)},0)`)
                    .attr("visibility", (d: number) => d >= timelineHandler.startDate! && d <= timelineHandler.endDate! ? "visible" : "hidden"),
                exit => exit.remove()
            );
    }
    const xAxis = (g: SelectionType, header: boolean, series: boolean) => {
        if (header) {
            g.selectAll("#header-axis").call(headerAxis);
        }
        if (series) {
            g.selectAll("#series-axis").call(seriesAxis);
        }
    }
    const axis = svg.append("g");
    axis.append("g")
            .attr("id", "header-axis")
            .attr("transform", 'translate(0,16)')
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "9px");
    axis.append("g")
            .attr("id", "series-axis")
            .attr("transform", 'translate(0,16)');
    axis.call(xAxis, !withoutHeader, true);

    const innerOnTimelineChange = () => {
        // update axis
        withoutHeader && axis.call(xAxis, false, true);
        // trigger onTimelineChange callback
        onTimelineChange();
    }
    timelineHandler.addCallback(innerOnTimelineChange);
    const registeredCallbacks = [innerOnTimelineChange];

    if (!withoutHeader) {
        // dragable timeline
        const dragHandle = (handle: SelectionType, updateHandle: (value: number, width?: number) => void, isAreaSelected?: boolean) => {
            const [axisRangeStart, axisRangeEnd] = reverse? timelineHandler.axisXScale!.range().reverse() : timelineHandler.axisXScale!.range();
            let startOffset = 0;
            let width = 0;
            return d3.drag()
            .on("start", (e: any) => {
                handle.attr("cursor", "grabbing");
                startOffset = isAreaSelected ? e.x - Math.min(timelineHandler.getXStart(), timelineHandler.getXEnd()) : 0;
                width = isAreaSelected ? timelineHandler.getTimelineRangeWidth() : 0;
            })
            .on("end", (e: any) => { isAreaSelected ? handle.call(checkHandleGrabCursor, width) : handle.attr("cursor", "grab") })
            .on("drag", (e: any) => {
                const x = Math.min(Math.max(e.x - startOffset, axisRangeStart), axisRangeEnd - width);
                handle.attr("transform", `translate(${x},0)`);
                // update date
                updateHandle(x, width);
                // update area selected
                !isAreaSelected && areaSelected.call(updateAreaSelected);
                // update axis
                axis.call(xAxis, false, true);
                // trigger callbacks
                updatingThisHeader = true;
                timelineHandler.triggerCallbacks();
                updatingThisHeader = false;
            });
        }
        const checkHandleGrabCursor = (handle: SelectionType, width: number) => {
            handle.attr("cursor", Math.round(timelineHandler.axisRangeWidth! - width) > 20 ? "grab" : "default");
        }

        let updatingThisHeader = false;
        const onHeaderTimelineChange = () => {
            if (!updatingThisHeader) {
                areaSelected.call(updateAreaSelected);
                updateHandleDates();
                // update axis
                axis.call(xAxis, false, true);
            }
        }
        timelineHandler.addCallback(onHeaderTimelineChange);
        registeredCallbacks.push(onHeaderTimelineChange);

        const areaSelected = svg.append("g");
        areaSelected.append("rect")
            .attr("id", "area-selected")
            .attr("fill", modeColors.text)
            .attr("opacity", 0.1)
            .attr("x", 3)
            .attr("y", 0)
            .attr("height", 16);
        const updateAreaSelected = (selection: SelectionType) => {
            selection.attr("transform", `translate(${Math.min(timelineHandler.getXStart(), timelineHandler.getXEnd())},0)`);
            selection.select("#area-selected").attr("width", Math.max(0, timelineHandler.getTimelineRangeWidth() - 6));
        }
        const updateHandleDates = () => {
            handleADate = timelineHandler.startDate!;
            handleA.attr("transform", `translate(${timelineHandler.getXStart()},0)`);
            handleBDate = timelineHandler.endDate!;
            handleB.attr("transform", `translate(${timelineHandler.getXEnd()},0)`);
        }
        areaSelected.call(updateAreaSelected);
        areaSelected.call(
            dragHandle(areaSelected, (v, width) => {
                timelineHandler.updateDate(v, !reverse);
                width !== undefined && timelineHandler.updateDate(v + width, !!reverse);
                updateHandleDates();
            }, true) as any
        );
        areaSelected.on("mouseover", (event) => {
            const width = timelineHandler.getTimelineRangeWidth();
            areaSelected.call(checkHandleGrabCursor, width);
        });
        areaSelected.on("wheel", (event: any) => {
            event.preventDefault();
            // Get the relative x position from the event target
            const rect = event.target.getBoundingClientRect();
            const relX = event.clientX - rect.left;
            const baseWidth = timelineHandler.getTimelineRangeWidth();
            const offsetFactor = relX / baseWidth;
            const scaleFactor = event.deltaY < 0 ? 1.08 : 0.92;
            let width = baseWidth * scaleFactor;
            if (width > timelineHandler.axisRangeWidth!) {
                width = timelineHandler.axisRangeWidth!;
            } else if (width < 20) {
                width = 20;
            }
            let xStart = Math.min(timelineHandler.getXStart()!, timelineHandler.getXEnd()!) + relX - offsetFactor * width;
            let xEnd = xStart + width;
            const [axisRangeStart, axisRangeEnd] = reverse? timelineHandler.axisXScale!.range().reverse() : timelineHandler.axisXScale!.range();
            if (xEnd > axisRangeEnd) {
                xEnd = axisRangeEnd;
                xStart = xEnd - width;
            }
            if (xStart < axisRangeStart) {
                xStart = axisRangeStart;
                xEnd = xStart + width;
            }
            timelineHandler.updateDate(xStart, !reverse);
            timelineHandler.updateDate(xEnd, !!reverse);
            // update handles
            areaSelected.call(updateAreaSelected);
            areaSelected.call(checkHandleGrabCursor, width);
            updateHandleDates();
            // update axis
            axis.call(xAxis, false, true);
            // trigger callbacks
            updatingThisHeader = true;
            timelineHandler.triggerCallbacks();
            updatingThisHeader = false;
        });

        const constructHandle = (handle) => {
            handle.append("rect")
                .attr("x", -3)
                .attr("y", 0)
                .attr("width", 6)
                .attr("height", 16)
                .attr("fill", `${modeColors.text}3f`);
        }
    
        const handleA = svg.append("g")
            .attr("cursor", "grab")
            .attr("transform", `translate(${timelineHandler.getXStart()},0)`);
        let handleADate = timelineHandler.startDate!;
        handleA.call(
                dragHandle(handleA, (v) => {
                timelineHandler.updateDate(v, !Math.round(handleADate - timelineHandler.startDate!));
                handleADate = timelineHandler.axisXScale!.invert(v);
            }) as any
            );
        constructHandle(handleA);
    
        const handleB = svg.append("g")
            .attr("cursor", "grab")
            .attr("transform", `translate(${timelineHandler.getXEnd()},0)`);
        let handleBDate = timelineHandler.endDate!;
        handleB.call(
                dragHandle(handleB, (v) => {
                timelineHandler.updateDate(v, !Math.round(handleBDate - timelineHandler.startDate!));
                handleBDate = timelineHandler.axisXScale!.invert(v);
            }) as any
            );
        constructHandle(handleB);       
    }

    return () => {
        for (const callback of registeredCallbacks) {
            timelineHandler.removeCallback(callback);
        }
    }
}

export const calculateAverageSessionFrequencyHours = (handler: TestDataHandler, sessions: TestSessionResult[] | PhaseSessionResult[]) => {
    let averageSessionFrequencyHours: number = Infinity;

    const thresholdNumberOfSessions = 48; // Consider only the last 48 sessions for the average frequency
    if (sessions.length > thresholdNumberOfSessions) {
        // Filter sessions to only unique commitIds (preserve order)
        const seenCommitIds = new Set<string>();
        const uniqueDates: number[] = [];
        for (let i = 0; i < Math.min(sessions.length, thresholdNumberOfSessions); i++) {
            const commitId = sessions[i].commitId;
            if (!seenCommitIds.has(commitId)) {
                const commitDate = handler.commitIdDates.get(commitId);
                if (!commitDate) continue; // no matching commit date found? should not happen
                seenCommitIds.add(commitId);
                uniqueDates.push(commitDate.getTime() / 1000);
            }
        }

        if (uniqueDates.length > 1) {
            const firstDate = uniqueDates[0];
            const lastDate = uniqueDates[uniqueDates.length - 1];
            const totalSeconds = Math.abs(lastDate - firstDate);
            averageSessionFrequencyHours = totalSeconds / ((uniqueDates.length - 1) * 3600);
        }
    }

    return averageSessionFrequencyHours;
}

export const getSessionsNearTimeFactory = (sessions: TestSessionResult[] | PhaseSessionResult[], sessionTimes: number[], days: number, metaCount: number = 1) => {
        // Organize sessions into timeline sections by sessionTime, to enable fast lookup of sessions near a specific time
        // We'll divide the entire timeline into bins of fixed size.
        const sectionSizeSeconds = Math.min(2, (days * 24 / 2) / (sessions.length / metaCount)) * 60 * 60; // 2 hours per section bin minimum, otherwise inversely proportional to session count
        const sessionSections: Map<number, any> = new Map();
        // Time reference for binning
        const minSessionTime = sessionTimes.length > 0 ? sessionTimes[sessionTimes.length - 1] : 0;
        // Organize sessions into bins
        sessions.forEach((session, idx) => {
            const sessionTime = sessionTimes[idx];
            const sectionIdx = Math.floor((sessionTime - minSessionTime) / sectionSizeSeconds);
            if (!sessionSections.has(sectionIdx)) {
                sessionSections.set(sectionIdx, []);
            }
            sessionSections.get(sectionIdx)!.push(session);
        });

        function getSessionsNearTime(targetTime: number, windowSections: number = 1): TestSessionResult[] | PhaseSessionResult[] {
            const sectionIdx = Math.floor((targetTime - minSessionTime) / sectionSizeSeconds);
            const results: TestSessionResult[] | PhaseSessionResult[] = [];
            const startSectionIdx = sectionIdx - windowSections;
            const endSectionIdx = sectionIdx + windowSections;
            for (let i = startSectionIdx; i <= endSectionIdx; ++i) {
                if (sessionSections.has(i)) {
                    results.push(...sessionSections.get(i)!);
                }
            }
            return results;
        }
        return getSessionsNearTime;
}