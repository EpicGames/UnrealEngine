// Copyright Epic Games, Inc. All Rights Reserved.

import { Image, Spinner, SpinnerSize, Stack, Text, ThemeProvider } from '@fluentui/react';
import React, { useState } from 'react';
import { Navigate, Outlet, RouteObject, RouterProvider, createBrowserRouter } from 'react-router-dom';
import { enableHordePlugins, getHordePlugins } from '../plugins';
import "hordePlugins/registry";
import backend from './backend';
import { getSiteConfig } from './backend/Config';
import dashboard from './backend/Dashboard';
import { ThemeTester } from './base/components/ThemeTester/ThemeTester';
import { AdminToken } from './components/AdminToken';
import { AgentView } from './components/agents/AgentView';
import { AuditLogView } from './components/AuditLog';
import { AutomationView } from './components/AutomationView';
import { DashboardView } from './components/DashboardView';
import { DebugView } from './components/DebugView';
import { DeviceView } from './components/devices/DeviceView';
import { ErrorDialog } from './components/error/ErrorDialog';
import { errorDialogStore } from './components/error/ErrorStore';
import { JobRedirector } from './components/JobRedirector';
import { LogView } from './components/logs/LogView';
import { NoticeView } from './components/NoticeView';
import { PerforceServerView } from './components/PerforceView';
import { PoolsView } from './components/PoolsView';
import { PreflightRedirector } from './components/Preflight';
import { ProjectHome } from './components/ProjectHome';
import { StreamView } from './components/StreamView';
import { TestReportView } from './components/TestReportView';
import { ToolView } from './components/ToolView';
import { UserHomeView } from './components/UserHome';
import { UtilizationReportView } from './components/UtilizationReportView';
import { AccountsView } from './components/accounts/AccountsView';
import { HordeLoginView } from './components/accounts/HordeLoginView';
import { ServiceAccountsView } from './components/accounts/ServiceAccountsView';
import { AgentRequestsView } from './components/agents/AgentRequestsView';
import { ArtifactRedirector } from './components/artifacts/ArtifactsRedirector';
import { DocView } from './components/docs/DocView';
import { JobDetailViewV2 } from './components/jobDetailsV2/JobDetailViewV2';
import { PreflightConfigRedirector } from './components/preflights/PreflightConfigCheckRedirector';
import { ServerStatusView } from './components/server/ServerStatus';
import { StorageView } from './components/storage/StorageView';
import { HordeSetupView } from './components/setup/HordeSetupView';
import { StepIssueReportTest } from './components/test/IssueStepReport';
import hordePlugins from './legacyPlugins';
import { preloadFonts } from './styles/Styles';
import { darkTheme } from './styles/darkTheme';
import { lightTheme } from './styles/lightTheme';
import { BuildHealthView } from './components/buildhealth/BuildhealthView';

let router: any;

const RouteError: React.FC = () => {
   return <Navigate to="/index" replace={true} />
}

const Main: React.FC = () => {

   const [init, setInit] = useState(false);

   document.body.setAttribute('style', `background: ${dashboard.darktheme ? "#0F0F0F" : "#FAF9F9"}`)   

   if (window.location.pathname === "/login") {
      return <HordeLoginView />
   }

   if (window.location.pathname === "/setup") {
      return <HordeSetupView />
   }

   const config = getSiteConfig();

   if (!init) {

      console.log("Initializing " + config.environment + " dashboard");

      backend.init().then(() => {

         backend.getCurrentUser().then(() => {

            setInit(true);
            return null;

         }).catch(reason => {
            errorDialogStore.set({ title: "Error initializing site, unable to get user", reason: reason }, true);
         })


      }).catch((reason) => {
         errorDialogStore.set({ title: "Error initializing site", reason: reason }, true);
      });

      return (<ThemeProvider applyTo='element' theme={dashboard.darktheme ? darkTheme : lightTheme}>
         <div style={{ position: 'absolute', left: '50%', top: '50%', transform: 'translate(-50%, -50%)' }}>
            <Stack horizontalAlign="center" styles={{ root: { padding: 20, minWidth: 200, minHeight: 100 } }}>
               <Stack horizontal>
                  <Stack styles={{ root: { paddingTop: 2, paddingRight: 6 } }}>
                     <Image shouldFadeIn={false} shouldStartVisible={true} width={48} src="/images/horde.svg" />
                  </Stack>
                  <Stack styles={{ root: { paddingTop: 12 } }}>
                     <Text styles={{ root: { fontFamily: "Horde Raleway Bold", fontSize: 24 } }}>HORDE</Text>
                  </Stack>
               </Stack>
               <Stack>
                  {preloadFonts.map(font => {
                     // preload fonts to avoid FOUT
                     return <Text key={`font_preload_${font}`} styles={{ root: { fontFamily: font, fontSize: 10 } }} />
                  })}
               </Stack>
               <Spinner styles={{ root: { paddingTop: 8, paddingLeft: 4 } }} size={SpinnerSize.large} />
            </Stack>
         </div>
      </ThemeProvider>);
   }

   // legacy plugins
   hordePlugins.loadPlugins(config.plugins);

   const enabledPlugins = backend.enabledPlugins;
   if (enabledPlugins.length) {
      console.log("Enabled Server Plugins:")
      enabledPlugins.forEach(p => {
         console.log(`    ${p.name} : ${p.version ?? ""}`)
      })
   } else {
      console.log("No Enabled Plugins")
   }

   enableHordePlugins(enabledPlugins);
   const plugins = getHordePlugins();

   plugins.forEach(p => {
      if (!!enabledPlugins.find(plugin => p.id === plugin.name)) {
         console.log(`Loading Dashboard Plugin: ${p.id}`);
      }
   })

   if (!router) {

      const routes: RouteObject[] = [
         {
            path: "/", element: <Root />, errorElement: <RouteError />, children: [
               { path: "index", element: <UserHomeView /> },
               { path: "project/:projectId", element: <ProjectHome /> },
               { path: "pools", element: <PoolsView /> },
               { path: "job/:jobId", element: <JobDetailViewV2 /> },
               { path: "job", element: <JobRedirector /> },
               { path: "log/:logId", element: <LogView /> },
               { path: "testreport/:testdataId", element: <TestReportView /> },
               { path: "stream/:streamId", element: <StreamView /> },
               { path: "agents", element: <AgentView /> },
               { path: "agents/registration", element: <AgentRequestsView /> },
               { path: "artifact/:artifactId", element: <ArtifactRedirector /> },
               { path: "admin/token", element: <AdminToken /> },
               { path: "reports/utilization", element: <UtilizationReportView /> },
               { path: "preflight", element: <PreflightRedirector /> },
               { path: "preflightconfig", element: <PreflightConfigRedirector /> },
               { path: "dashboard", element: <DashboardView /> },
               { path: "perforce/servers", element: <PerforceServerView /> },
               { path: "notices", element: <NoticeView /> },
               { path: "serverstatus", element: <ServerStatusView /> },
               { path: "storage", element: <StorageView /> },
               { path: "devices", element: <DeviceView /> },
               { path: "audit/agent/:agentId", element: <AuditLogView /> },
               { path: "audit/issue/:issueId", element: <AuditLogView /> },
               { path: "audit/device/:deviceId", element: <AuditLogView /> },
               { path: "audit/job/:jobId", element: <AuditLogView /> },
               { path: "audit/template/:streamId/:templateId", element: <AuditLogView /> },
               { path: "automation", element: <AutomationView /> },
               { path: "buildhealth", element: <BuildHealthView/> },
               { path: "tools", element: <ToolView /> },
               { path: "lease/:leaseId", element: <DebugView /> },
               { path: "docs", element: <DocView /> },
               { path: "docs/*", element: <DocView /> },
               { path: "accounts", element: <AccountsView /> },
               { path: "accounts/service", element: <ServiceAccountsView /> },
               { path: "test/stepissuereport", element: <StepIssueReportTest /> },
               { path: "test/theme", element: <ThemeTester /> }
            ]
         }
      ];

      const pluginRoutes = plugins.map(p => p.routes).flat();
      routes[0].children!.push(...pluginRoutes);

      // mount plugins
      const legacyPluginRoutes = hordePlugins.routes.map((route) => {
         return { path: route.path, element: <route.component /> };
      })

      routes[0].children!.push(...legacyPluginRoutes);

      router = createBrowserRouter(routes);
   }
   
   
   return (
      <ThemeProvider applyTo='element' theme={dashboard.darktheme ? darkTheme : lightTheme}>
         <RouterProvider router={router} />
      </ThemeProvider>
   );
};

const App: React.FC = () => {

   return (
      <React.Fragment>
         <ErrorDialog />
         <Main />
      </React.Fragment>
   );
};

export default App;

const HomeRedirect: React.FC = () => {
   if (window.location.pathname === "/" || !window.location.pathname) {
      if (dashboard.user?.dashboardFeatures?.showLandingPage) {
         let route = "/docs/Landing.md";
         if (dashboard.user?.dashboardFeatures?.landingPageRoute?.length) {
            route = dashboard.user?.dashboardFeatures?.landingPageRoute;
         }
         return <Navigate to={route} replace={true} />
      }
      return <Navigate to="/index" replace={true} />
   }
   return null;
}

const Root: React.FC = () => {
   return <Stack>
      <Outlet />
      <HomeRedirect />
   </Stack>
}

