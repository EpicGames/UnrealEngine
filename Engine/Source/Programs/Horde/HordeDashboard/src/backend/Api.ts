// Copyright Epic Games, Inc. All Rights Reserved.

//  Horde backend API types


export type CommitId = string;

// The session type
export enum SessionType {

	// Execute a command
	Job = 0,

	// Upgrade the agent software
	Upgrade = 1
}

export enum Priority {

	// Lowest priority
	Lowest = "Lowest",

	// Below normal priority
	BelowNormal = "BelowNormal",

	// Normal priority
	Normal = "Normal",

	// Above normal priority
	AboveNormal = "AboveNormal",

	// High priority
	High = "High",

	// Highest priority
	Highest = "Highest"
}

// The state of a jobstep
export enum JobStepState {

	// Waiting for dependencies of this step to complete (or paused)
	Waiting = "Waiting",

	// Ready to run, but has not been scheduled yet
	Ready = "Ready",

	// Dependencies of this step failed, so it cannot be executed
	Skipped = "Skipped",

	// There is an active instance of this step running
	Running = "Running",

	// This step has been run
	Completed = "Completed",

	// This step has been aborted
	Aborted = "Aborted"

}

export enum DeviceStatus {

	// Device encountered an error
	Error = "Error",

	// Device is operating normally
	Normal = "Normal"
}

// Outcome of a jobstep run
export enum JobStepOutcome {

	// Finished with errors
	Failure = "Failure",

	// Finished with warnings
	Warnings = "Warnings",

	// Finished Succesfully
	Success = "Success",

	// Unspecified, skipped, aborted, etc
	Unspecified = "Unspecified"
}

/// Systemic error codes for a job failing
export enum JobStepError {
	/// No systemic error
	None = "None",
	/// Step did not complete in the required amount of time
	TimedOut = "TimedOut",
	/// Step is in paused state so was skipped
	Paused = "Paused"
}

// The state of a particular run
export enum JobStepBatchState {

	// Waiting for dependencies of at least one jobstep to complete
	Waiting = "Waiting",

	// Getting ready to execute
	Starting = "Starting",

	// Ready to execute
	Ready = "Ready",

	// Currently running
	Running = "Running",

	// Getting ready to execute
	Stopping = "Stopping",

	// All steps have finished executing
	Complete = "Complete"
}

/**Error code for a batch not being executed */
export enum JobStepBatchError {

	/** No error */
	None = "None",

	/** The stream for this job is unknown */
	UnknownStream = "UnknownStream",

	/** The given agent type for this batch was not valid for this stream */
	UnknownAgentType = "UnknownAgentType",

	/** The pool id referenced by the agent type was not found */
	UnknownPool = "UnknownPool",

	/** There are no agents in the given pool currently online */
	NoAgentsInPool = "NoAgentsInPool",

	/** There are no agents in this pool that are onlinbe */
	NoAgentsOnline = "NoAgentsOnline",

	/** Unknown workspace referenced by the agent type */
	UnknownWorkspace = "UnknownWorkspace",

	/** Cancelled */
	Cancelled = "Cancelled",

	/** Lost connection with the agent machine */
	LostConnection = "LostConnection",

	/** Lease terminated prematurely */
	Incomplete = "Incomplete",

	/** An error ocurred while executing the lease. Cannot be retried. */
	ExecutionError = "ExecutionError",

	/** The change that the job is running against is invalid. */
	UnknownShelf = "UnknownShelf",

	/** No longer needed */
	NoLongerNeeded = "NoLongerNeeded",

	/** Sync Failed */
	SyncingFailed = "SyncingFailed"

}

// The type of a template parameter
export enum TemplateParameterType {

	// An arbitrary string
	String = "String"
}

export enum EventSeverity {

	Unspecified = "Unspecified",
	Information = "Information",
	Warning = "Warning",
	Error = "Error"
}

export enum AclActions {

	// No actions allowed
	None = "None",

	// Reading to the object
	Read = "Read",

	// Modifying the object
	Write = "Write",

	// Creating new objects
	Create = "Create",

	// Deleting the object
	Delete = "Delete",

	// Executing the object
	Execute = "Execute",

	// Change permissions on the object
	ChangePermissions = "ChangePermissions"

}

export enum SubscriptonNotificationType {
	// Slack notification
	Slack = "Slack",
}

// aliases and extensions
export type AgentData = GetAgentResponse
export type LeaseData = GetAgentLeaseResponse
export type SessionData = GetAgentSessionResponse
export type PoolData = GetPoolResponse
export type BatchData = GetBatchResponse
export type EventData = GetLogEventResponse
export type LogData = GetLogFileResponse
export type AclData = GetAclResponse
export type AclEntryData = GetAclEntryResponse
export type SoftwareData = GetSoftwareResponse
export type ScheduleData = GetScheduleResponse
export type ChangeSummaryData = GetChangeSummaryResponse
export type JobsTabData = GetJobsTabResponse
export type JobsTabColumnData = GetJobsTabColumnResponse
export type TimingInfo = GetTimingInfoResponse
export type StepTimingInfo = GetStepTimingInfoResponse
export type SubscriptionData = GetSubscriptionResponse
export type ArtifactData = GetArtifactResponse;

export type JobLabel = {
    stateResponse: GetLabelStateResponse;
    timing?: GetLabelTimingInfoResponse;
    default: boolean;
}

export type IssueData = GetIssueResponse & {
	events?: GetLogEventResponse[];
}

export type ProjectData = GetProjectResponse & {
	streams?: StreamData[];
}

export type StreamData = GetStreamResponse & {
	project?: ProjectData;
	// full path as returned by server
	fullname?: string;

}

export type JobData = GetJobResponse;

export type StepData = GetStepResponse & {

	timing?: GetStepTimingInfoResponse;
}

export type TestData = GetTestDataResponse & {
	data: object;
}

export enum LogType {
	JSON = "JSON",
	TEXT = "TEXT"
}

export enum LogLevel {
	Informational = "Informational",
	Warning = "Warning",
	Error = "Error",
	Critical = "Critical"
}

export type LogLine = {
	time: string;
	level: LogLevel;
	message: string;
	id?: number;
	format?: string;
	properties?: Record<string, string | Record<string, string | number>>;
}

export type LogLineData = {
	format: LogType;
	index: number;
	count: number;
	maxLineIndex: number;
	lines?: LogLine[];
};

export type JobStreamQuery = {
	filter?: string;
	template?: string[];
	t?: string[];
	preflightStartedByUserId?: string,
	includePreflight?: boolean,
	maxCreateTime?: string;
	modifiedAfter?: string;
	index?: number;
	count?: number;
	consistentRead?: boolean;

};

export type AgentQuery = {
	modifiedAfter?: string;
	poolId?: string;
	includeDeleted?: boolean;
	invalidateCache?: boolean;
	condition?: string;
	filter?: string;
}

export type JobQuery = {
	id?: string[],
	filter?: string;
	streamId?: string;
	name?: string;
	template?: string[];
	t?: string[];
	state?: string[];
	outcome?: string[];
	minChange?: number;
	maxChange?: number;
	preflightChange?: number;
	preflightOnly?: boolean;
	includePreflight?: boolean;
	preflightStartedByUserId?: string;
	startedByUserId?: string;
	minCreateTime?: string;
	maxCreateTime?: string;
	modifiedAfter?: string;
	index?: number;
	count?: number;
};

export type JobTimingsQuery = {
	streamId?: string;
	template?: string[];
	filter?: string;
	count?: number;
};


export type IssueQuery = {
	jobId?: string;
	batchId?: string;
	stepId?: string;
	label?: number;
	streamId?: string;
	change?: number;
	minChange?: number;
	maxChange?: number;
	index?: number;
	count?: number;
	ownerId?: string;
	resolved?: boolean;
	promoted?: boolean;
}

export type IssueQueryV2 = {
	id?: string[];
	streamId?: string;
	minChange?: number;
	maxChange?: number;
	resolved?: boolean;
	index?: number;
	count?: number;
	filter?: string;
}

export type LogEventQuery = {
	index?: number;
	count?: number;
}

export type GetBatchLogEventsRequest = {
	logIds: string[];
	count?: number;
	severity?: EventSeverity;
}

export type BatchLogEventsEntry = {
	total: number;
	events: EventData[];
}

export type GetBatchLogEventsResponse = {
	logEvents: Record<string, BatchLogEventsEntry>;
}

export type UsersQuery = {
	ids?: string[];
	nameRegex?: string;
	index?: number;
	count?: number;
	includeAvatar?: boolean;
	includeClaims?: boolean;
}


export type ScheduleQuery = {
	streamId?: string;
	filter?: string;
}


/* The severity of an issue */
export enum IssueSeverity {
	/** Unspecified severity */
	Unspecified = "Unspecified",

	/** This error represents a warning */
	Warning = "Warning",

	/** This issue represents an error */
	Error = "Error"
}

export type CategoryAgents = {
	ids: string[];
	lastPoll: Date;
	polling?: boolean;
}

/** Describes a category for the agents page */
export type GetDashboardAgentCategoryResponse = {

	/** Title for the tab */
	name: string;

	/* Condition for agents to be included in this category */
	condition?: string;
}

/** Describes a category for the pools page */
export type GetDashboardPoolCategoryResponse = {

	/** Title for the tab */
	name: string;

	/* Condition for pools to be included in this category */
	condition?: string;
}

export enum AuthMethod {
	Anonymous = "Anonymous",
	Okta = "Okta",
	OpenIdConnect = "OpenIdConnect",
	Horde = "Horde"
}

/// Artifact type information
export type GetDashboardArtifactTypeResponse = {
	type: string;
	keepDays?: number;
}

/** Setting information required by dashboard */
export type GetDashboardConfigResponse = {

	// Authorization method in use
	authMethod?: AuthMethod;

	/** The name of the external issue service */
	externalIssueServiceName?: string;

	/** The url of the external issue service */
	externalIssueServiceUrl?: string;

	/** The url of the perforce swarm installation */
	perforceSwarmUrl?: string;

	/** Help email address that users can contact with issues */
	helpEmailAddress?: string;

	/** Help slack channel that users can use for issues */
	helpSlackChannel?: string;

	/** Device problem cooldown in minutes */
	deviceProblemCooldownMinutes?: number

	/** Categories to display on the agents page */
	agentCategories: GetDashboardAgentCategoryResponse[];

	/** Categories to display on the pools page */
	poolCategories: GetDashboardPoolCategoryResponse[];

	/** Categories to display on the pools page */
	artifactTypes: string[];

	/** Artifact type information */
	artifactTypeInfo: GetDashboardArtifactTypeResponse[];

	/** Telemetry views */
	telemetryViews: any[];

	/** The server's global config version number */
	configVersion?: number;
}

/**Parameters to register a new agent */
export type CreateAgentRequest = {

	/**Friendly name for the agent */
	name: string;

	/**Whether the agent is currently enabled */
	enabled: boolean;

	/**Whether the agent is ephemeral (ie. should not be shown when inactive) */
	ephemeral: boolean;

	/**Per-agent override for the desired client version */
	forceVersion?: string;

	/**Pools for this agent */
	pools?: string[];
}

/**Response from creating an agent */
export type CreateAgentResponse = {

	/**Unique id for the new agent */
	id: string;

	/**Bearer token for this agent to authorize itself with the server */
	token: string;
}

/**Parameters to update an agent */
export type UpdateAgentRequest = {

	/**New name of the agent */
	name?: string;

	/**Whether the agent is currently enabled */
	enabled?: boolean;

	/**comment for the agent */
	comment?: string;

	/**Whether the agent is ephemeral */
	ephemeral?: boolean;

	/**Boolean to request a conform */
	requestConform?: boolean;

	/** Request that a full conform be performed, removing all intermediate files */
	requestFullConform?: boolean;

	requestRestart?: boolean;

	requestForceRestart?: boolean;

	requestShutdown?: boolean;

	/**Per-agent override for the desired client version */
	forceVersion?: string;

	/**Pools for this agent */
	pools?: string[];

	/**New ACL for this agent */
	acl?: UpdateAclRequest;

}

// Agent Registration

/// Updates an existing lease
export type GetPendingAgentsResponse = {
	agents: GetPendingAgentResponse[];
}


/// Information about an agent pending admission to the farm
export type GetPendingAgentResponse = {
	key: string;
	hostName: string;
	description: string;
}

/// Approve an agent for admission to the farm
export type ApproveAgentsRequest = {
	agents: ApproveAgentRequest[];
}

/// Approve an agent for admission to the farm
export type ApproveAgentRequest = {
	key: string;
	agentId?: string;
}


export type AuditLogQuery = {
	minTime?: string;
	maxTime?: string;
	index?: number;
	count?: number;
}

export enum AuditLogLevel {
	Information = "Information"
}

export type AuditLogEntry = {
	time: string;
	level: AuditLogLevel;
	message: string;
	format: string;
	properties?: Record<string, string | number | Record<string, string | number>>;
}


/**Parameters to create a pool */
export type CreatePoolRequest = {

	/**New name of the pool */
	name: string;

	/**Arbitrary properties associated with this event */
	properties?: { [key: string]: string };
}


/**Parameters to update a pool */
export type UpdatePoolRequest = {

	/**New name of the pool */
	name?: string;

	/** Whether to enable autoscaling for this pool */
	enableAutoscaling?: boolean;

	/// Frequency to run conforms, in hours, set 0 to disable
	conformInterval?: number;

	/** Pool sizing strategy */
	sizeStrategy?: PoolSizeStrategy;

	/** The minimum nunmber of agents to retain in this pool */
	minAgents?: number;

	/** The minimum number of idle agents in this pool, if autoscaling is enabled */
	numReserveAgents?: number;

	/**Arbitrary properties associated with this event */
	properties?: { [key: string]: string };
}

/**Parameters to update a pool */
export type BatchUpdatePoolRequest = {

	/**ID of the pool to update  */
	id: string;

	/**New name of the pool */
	name?: string;

	/**Arbitrary properties associated with this event */
	properties?: { [key: string]: string };

	/**Soft-delete for this pool */
	deleted?: boolean;
}

/** Lease state for agent leases */
export enum LeaseState {

	Unspecified = "Unspecified",

	Pending = "Pending",

	Active = "Active",

	Completed = "Completed",

	Cancelled = "Cancelled"

}


export enum LeaseOutcome {

	/** Default value. */
	Unspecified = "Unspecified",

	/** The lease was executed successfully */
	Success = "Success",

	/** The lease was not executed succesfully, but cannot be run again. */
	Failed = "Failed",

	/** The lease was cancelled by request */
	Cancelled = "Cancelled"

}


/**Response for queries to find a particular lease within an agent */
export type GetAgentLeaseResponse = {

	/**Identifier for the lease */
	id: string;

	/** parent lease id */
	parentId?: string;

	/**Name of the lease */
	name?: string;

	/**Time at which the lease started (UTC) */
	startTime: Date | string;

	/**Time at which the lease ended (UTC) */
	finishTime?: Date | string;

	/**Whether this lease has started executing on the agent yet */
	executing: boolean;

	/**Details for this lease */
	details?: { [key: string]: string };

	/**agentId of lease */
	agentId?: string;

	/**What type of lease this is (parsed from Details array) */
	type: string;

	/**BatchId of this lease (parsed from Details array) */
	batchId: string;

	/**JobId of this lease (parsed from Details array) */
	jobId: string;

	/**logId of this lease (parsed from Details array) */
	logId: string;

	/**Batch data for this lease, queried from backend */
	batch?: GetBatchResponse;

	/** Outcome of the lease */
	outcome?: LeaseOutcome;

	state?: LeaseState;

}

/**Information about an agent session */
export type GetAgentSessionResponse = {

	/**Unique id for this session */
	id: string;

	/**Start time for this session */
	startTime: Date | string;

	/**Finishing time for this session */
	finishTime?: Date | string;

	/**Capabilities of this agent */
	capabilities?: GetAgentCapabilitiesResponse;

	/**Version of the software running during this session */
	version?: string;

}

export type GetAgentWorkspaceResponse = {

	/**Server and port */
	serverAndPort?: string;

	/**username */
	userName?: string;

	/**password */
	password?: string;

	/**identifier */
	identifier: string;

	/**stream */
	stream: string;

	/**view */
	view?: string[];

	/**incremental */
	bIncremental: boolean;
}

/**Information about an agent */
export type GetAgentResponse = {

	/**The agent's unique ID */
	id: string;

	/**Friendly name of the agent */
	name: string;

	/**Session ID */
	sessionId: string;

	/**Session expiry time */
	sessionExpiresAt: Date | string;

	/**Status of the agent */
	online: boolean;

	/**Whether the agent is currently enabled */
	enabled: boolean;

	/**Whether the agent is ephemeral */
	ephemeral: boolean;

	/**comment for the agent */
	comment?: string;

	/**The current client version */
	version?: string;

	/**Per-agent override for the desired client version */
	forceVersion?: string;

	/**Pools for this agent */
	pools?: string[];

	/**Status for this agent */
	status?: string;

	/**Capabilities of this agent */
	capabilities?: GetAgentCapabilitiesResponse;

	/**Array of active leases. */
	leases?: GetAgentLeaseResponse[];

	/**Per-object permissions */
	acl?: GetAclResponse;

	/**Last update time of the agent */
	updateTime: Date | string;

	/**Whether this agent is deleted */
	deleted: boolean;

	/** Whether agent is pending conform */
	pendingConform: boolean;

	/** Whether a full conform job is pending */
	pendingFullConform: boolean;

	/** Conform attempt count */
	conformAttemptCount?: number;

	/** Last conform time */
	lastConformTime?: Date | string;

	/** Next conform attempt time */
	nextConformTime?: Date | string;

	/** The reason for the last shutdown */
	lastShutdownReason: string;

	/** Whether a shutdown is pending */
	pendingShutdown: boolean;

	/** agent workspaces */
	workspaces?: GetAgentWorkspaceResponse[];

}

export type Condition = {

	/// The condition text
	text: string;

	/// Error produced when parsing the condition
	error?: string;

}

/// Available pool sizing strategies
export enum PoolSizeStrategy {
	/// Strategy based on lease utilization
	LeaseUtilization = "LeaseUtilization",

	/// Strategy based on size of job build queue
	JobQueue = "JobQueue",

	/// No-op strategy used as fallback/default behavior
	NoOp = "NoOp"
}

/** Here for API completeness, though empty class on server */
export type LeaseUtilizationSettings = {

}

export type JobQueueSettings = {

	/// Factor translating queue size to additional agents to grow the pool with
	/// The result is always rounded up to nearest integer.
	/// Example: if there are 20 jobs in queue, a factor 0.25 will result in 5 new agents being added (20 * 0.25)
	scaleOutFactor: number;

	/// Factor by which to shrink the pool size with when queue is empty
	/// The result is always rounded up to nearest integer.
	/// Example: when the queue size is zero, a default value of 0.9 will shrink the pool by 10% (current agent count * 0.9)
	scaleInFactor: number;
}

/**Information about an agent */
export type GetPoolResponse = {

	/**The agent's unique ID */
	id: string;

	/**Friendly name of the agent */
	name: string;

	/// Condition for agents to be auto-added to the pool
	condition?: Condition;

	/// Color for the pool
	colorValue: string;

	/// Whether to enable autoscaling for this pool
	enableAutoscaling: boolean;

	/// Frequency to run conforms, in hours, set 0 to disable
	conformInterval?: number;

	/// Cooldown time between scale-out events in seconds
	scaleOutCooldown?: number;

	/// Cooldown time between scale-in events in seconds
	scaleInCooldown?: number;

	/// Pool sizing strategy to be used for this pool
	sizeStrategy?: PoolSizeStrategy;

	/// Settings for lease utilization pool sizing strategy (if used)
	leaseUtilizationSettings?: LeaseUtilizationSettings;

	/// Settings for job queue pool sizing strategy (if used)
	jobQueueSettings?: JobQueueSettings;

	/// The minimum nunmber of agents to retain in this pool
	minAgents?: number;

	/// The minimum number of idle agents in this pool, if autoscaling is enabled
	numReserveAgents?: number;

	/// List of workspaces that this agent contains
	workspaces: GetAgentWorkspaceResponse[];

	/**Arbitrary properties associated with this event */
	properties?: { [key: string]: string };

	/** True if pool is managed via config files (read-only in dashboard). Null on pre-V3 servers. */
	configManaged?: boolean;

	/** True if this config-managed pool also has a database entry (shadow pool) that can be deleted. */
	hasDatabaseEntry?: boolean;

}

/// Response describing a pool
export type GetPoolSummaryResponse = {
	/// Identifier for the pool
	id: string;
	/// Name of the pool
	name: string;
	/// Color to render the pool label
	colorValue: string;
	/// Whether autoscaling is enabled for this pool
	autoscaled: boolean;
	/// Counts for agents in differrent states
	stats?: GetPoolStatsResponse;
	/// Truncated list of agents
	agents?: GetPoolAgentSummaryResponse[];
	/// Utilization samples for the pool, from zero to one, with one sample for each hour
	utilization?: number[];
	/** True if pool is managed via config files (read-only in dashboard). Null on pre-V3 servers. */
	configManaged?: boolean;
	/** True if this config-managed pool also has a database entry (shadow pool) that can be deleted. */
	hasDatabaseEntry?: boolean;
}

/// Numbers of agents matching various criteria
export type GetPoolStatsResponse = {
	/// Number of agents in the pool
	numAgents: number;
	/// Number of agents that are ready
	numIdle: number;
	/// Number of agents offline
	numOffline: number;
	/// Number of agents that are disabled
	numDisabled: number;
}

/// Response describing an agent in a pool
export type GetPoolAgentSummaryResponse = {
	/// Identifier for the pool
	agentId: string;
	/// Whether the agent is idle
	idle?: boolean;
	/// Whether the agent is online
	offline?: boolean;
	/// Whether the agent is disabled
	disabled?: boolean;
}

export type PoolQuery = {
	/// Condition to select which pools to include
	condition?: any;
	/// Whether to include stats in the response
	stats?: boolean;
	/// Number of agents to include for each pool in the response
	numAgents?: number;
	///Number of utilization samples to include for each pool
	numUtilizationSamples?: number;
}

/**Parameters for creating a new event */
export type CreateEventRequest = {

	/**Time at which the event ocurred */
	time: Date | string;

	/**Severity of this event */
	severity: EventSeverity;

	/**Diagnostic text */
	message: string;

	/**Unique id of the log containing this event */
	logId: string;

	/**Minimum offset within the log containing this event */
	minOffset: number;

	/**Maximum offset within the log containing this event */
	maxOffset: number;

	/**Unique id of the agent associated with this event */
	agentId?: string;

	/**Unique id of the job associated with this event */
	jobId?: string;

	/**Unique id of the jobstep associated with this event */
	jobStepId?: string;

	/**Unique id of the jobstep associated with this event */
	jobBatchId?: string;

	/*The structured message data */
	data?: { [key: string]: string };

}

/**Information about an uploaded event */
export type GetLogEventResponse = {

	/**Unique id of the log containing this event */
	logId: string;

	/**Severity of this event */
	severity: EventSeverity;

	/**Index of the first line for this event */
	lineIndex: number;

	/**Number of lines in the event */
	lineCount: number;

	/**The issue id associated with this event */
	issueId: number | null;

	/**The structured message data for this event */
	lines: LogLine[];

}



/** Stats for a search */
export type LogSearchStats = {

	/// Number of blocks that were scanned
	numScannedBlocks: number;

	/// Number of bytes that had to be scanned for results
	numScannedBytes: number;

	/// Number of blocks that were skipped
	numSkippedBlocks: number;

	/// Number of blocks that had to be decompressed
	numDecompressedBlocks: number;

	/// Number of blocks that were searched but did not contain the search term
	numFalsePositiveBlocks: number;
}

/** Response describing a log file */
export type SearchLogFileResponse = {
	/// List of line numbers containing the search text
	lines: number[];

	/// Stats for the search
	stats?: LogSearchStats;
}


/**  Information about a group of nodes */
export type CreateAggregateRequest = {

	/** Name of the aggregate */
	name: string;

	/** Nodes which must be part of the job for the aggregate to be valid */
	nodes: string[];

}


/** Response from creating a new aggregate */
export type CreateAggregateResponse = {

	/** Index of the first aggregate that was added	*/
	firstIndex: number;
}

/**Information about a group of nodes */
export type CreateLabelRequest = {

	/**Category for this label */
	category: string;

	/**Name of the label */
	name: string;

	/**Nodes which must be part of the job for the label to be valid */
	requiredNodes: string[];

	/**Nodes which must be part of the job for the label to be valid */
	includedNodes: string[];
}


/**Parameters required to create a job */
export type CreateJobRequest = {

	/** The stream that this job belongs to */
	streamId: string;

	/** The template for this job */
	templateId: string;

	/** Name of the job */
	name?: string;

	/** The changelist number to build. Can be null for latest. */
	change?: number;

	/** The commit id to build. Can be null for latest. */
	commitId?: string;

	/// List of change queries to evaluate
	changeQueries?: ChangeQueryConfig[];

	/** The preflight changelist number */
	preflightChange?: number;

	/** Commit hash for branch-based preflights */
	preflightCommitId?: string;

	/** Priority for the job */
	priority?: Priority;

	/** Whether to automatically submit the preflighted change on completion */
	autoSubmit?: boolean;

	/** Whether to update issues based on the outcome of this job */
	updateIssues?: boolean;

	/** Whether to run the job as the scheduler*/
	runAsScheduler?: boolean;

	/** Nodes for the new job */
	groups?: CreateGroupRequest[];

	/** Arguments for the job */
	arguments?: string[];

	/** Parameters for the job */
	parameters?: Record<string, string>

	/** Additional arguments for the job, job dialog v2 */
	additionalArguments?: string[];

	/** Custom list of targets for the job, will take precedence over all argument and parameter targets */
	targets?: string[];
}

/**Response from creating a new job */
export type CreateJobResponse = {

	/**Unique id for the new job */
	id: string;
}

/**Updates an existing job */
export type UpdateJobRequest = {

	/** New name for the job */
	name?: string;

	/** New priority for the job */
	priority?: Priority;

	/**  Set whether the job should be automatically submitted or not */
	autoSubmit?: boolean;

	/** Mark this job as aborted */
	aborted?: boolean;

	/** Reason job was canceled */
	cancellationReason?: string;

	/** New list of arguments for the job. Only -Target= arguments can be modified after the job has started.  */
	arguments?: string[];

	/**  Custom permissions for this object */
	acl?: UpdateAclRequest;
}

export enum ReportPlacement {
	/** On a panel of its own */
	Panel = "Panel",
	/** In the summary panel */
	Summary = "Summary"
}

/**Information about a report associated with a job*/
export type GetReportResponse = {

	// Report placement
	placement: ReportPlacement;

	/** Name of the report */
	name: string;

	/** The artifact id */
	// artifactId?: string;

	/** The report markdown content */
	content?: string;

	/** The job or step message */
	message?: string;

	/** The job or step severity */
	severity?: string;

	/** Color to display for message severity */
	color?: string;
}

export type GetJobArtifactResponse = {
	/// Identifier for this artifact
	id: string;
	/// Name of the artifact
	name: string;
	/// Artifact type
	type: string;
	/// Description to display for the artifact on the dashboard
	description?: string;
	/// Step producing the artifact
	stepId: string;
	/// Keys for the artifact	
	keys: string[];
	/// Metadata for the artifact
	metadata: string[];
}

/**Information about a job */
export type GetJobResponse = {

	/**Unique Id for the job */
	id: string;

	/**Unique id of the stream containing this job */
	streamId: string;

	/**Name of the job */
	name: string;

	/** The changelist number to build */
	change?: number;

	/** The preflight changelist number */
	preflightChange?: number;

	/** Description of the preflight */
	preflightDescription?: string;

	/** Commit hash for branch-based preflights */
	preflightCommitId?: string;

	/** The template type */
	templateId?: string;

	/** Hash of the actual template data */
	templateHash?: string;

	/** Hash of the graph for this job */
	// graphHash?: string;

	/** The user that started this job */
	startedByUserInfo?: GetThinUserInfoResponse;

	/** The user that started this job */
	abortedByUserInfo?: GetThinUserInfoResponse;

	/** The reason the job was canceled */
	cancellationReason?: string;

	/** Whether job was created by a bisect task */
	startedByBisectTaskId?: string;

	/** The roles to impersonate when executing this job */
	roles: string[];

	/** Priority of the job */
	priority: Priority;

	/** Whether the change will automatically be submitted or not */
	autoSubmit?: boolean;

	/** The submitted changelist number */
	autoSubmitChange?: number;

	/**  Message produced by trying to auto-submit the change */
	autoSubmitMessage?: string;

	/**Time that the job was created */
	createTime: Date | string;

	/** The global job state */
	state: JobState;

	/** Array of jobstep batches */
	batches?: GetBatchResponse[];

	/**  List of labels */
	labels?: GetLabelStateResponse[];

	/** The default label, containing the state of all steps that are otherwise not matched. */
	defaultLabel?: GetLabelStateResponse;

	/** List of reports */
	reports?: GetReportResponse[];

	/**  Parameters for the job */
	arguments: string[];

	/** Parameters for the job */
	parameters: Record<string, string>;

	/**The last update time for this job*/
	updateTime: Date | string;

	/** Whether to update issues based on the outcome of this job */
	updateIssues?: boolean;

	artifacts?: GetJobArtifactResponse[];

	/**  Custom permissions for this object */
	acl?: GetAclResponse;

	/** Additional arguments for the job, job dialog v2 */
	additionalArguments?: string[];

	/** Custom list of targets for the job, will take precedence over all argument and parameter targets */
	targets?: string[];

	/**  Whether the current user is allowed to update this job */
	canUpdate?: boolean;

	/// The parent job id, if any
	parentJobId?: string;

	/// The parent job step id, if any
	parentJobStepId?: string;

}

/**Request used to update a jobstep */
export type UpdateStepRequest = {

	/**The new jobstep state */
	state?: JobStepState;

	/**Outcome from the jobstep */
	outcome?: JobStepOutcome;

	/**If the step has been requested to abort */
	abortRequested?: boolean;

	/** The reason the step was canceled */
	cancellationReason?: string;

	/**Specifies the log file id for this step */
	logId?: string;

	/**Whether the step should be re-run */
	retry?: boolean;

	/**New priority for this step */
	priority?: Priority;

	/**Properties to set. Any entries with a null value will be removed. */
	properties?: { [key: string]: string | null };
}


/** Reference to the output of a step within the job */
export type JobStepOutputRef = {
	// Step producing the output
	stepId: string;
	// <param name="OutputIdx">Index of the output from this step</param>
	outputIdx: number;
}

export type JobStepId = string;

/**Returns information about a jobstep */
export type GetStepResponse = {

	/**The unique id of the step */
	id: string;

	/**Index of the node which this jobstep is to execute */
	nodeIdx: number;

	/**Current state of the job step. This is updated automatically when runs complete. */
	state: JobStepState;

	/**Current outcome of the jobstep */
	outcome: JobStepOutcome;

	/**Error describing additional context for why a step failed to complete*/
	error: JobStepError;

	/** If the step has been requested to abort	*/
	abortRequested?: boolean;

	/* The user that requested the abort of this step */
	abortedByUserInfo?: GetThinUserInfoResponse;

	/* The user that retried this step */
	retriedByUserInfo?: GetThinUserInfoResponse;

	/** The reason the job step canceled */
	cancellationReason?: string;

	/**The log id for this step */
	logId?: string;

	/** Time at which the batch was ready (UTC). */
	readyTime?: Date | string;

	/**Time at which the run started (UTC). */
	startTime?: Date | string;

	/**Time at which the run finished (UTC) */
	finishTime?: Date | string;

	/** List of reports */
	reports?: GetReportResponse[];

	/**User-defined properties for this jobstep. */
	properties: { [key: string]: string };

	/// The name of this node 
	name: string;

	/// References to inputs for this node
	inputs?: JobStepOutputRef[];

	/// List of output names
	outputNames?: string[];

	/// Indices of nodes which must have succeeded for this node to run
	inputDependencies?: JobStepId[];

	/// Indices of nodes which may have succeeded for this node to run
	optionalInputDependencies?: JobStepId[];

	/// Indices of nodes which must have completed for this node to run
	orderDependencies?: JobStepId[];

	/// Whether this node can be run multiple times
	allowRetry: boolean;

	/// This node can start running early, before dependencies of other nodes in the same group are complete
	runEarly: boolean;

	/// Whether to include warnings in the output (defaults to true)
	warnings: boolean;

	/// List of credentials required for this node. Each entry maps an environment variable name to a credential in the form "CredentialName.PropertyName".
	credentials?: Record<string, string>;

	/// Annotations for this node
	annotations?: Record<string, string>;

	/// Spawned jobs
	spawnedJobs?: string[];
}

//**Returns information about test data */
export type GetTestDataResponse = {

	/**The unique id of the test data */
	id: string;

	/**The key associated with this test data */
	key: string;

	/**The change id related to the job */
	change: number;

	/**The job id of this test data */
	jobId: string;

	/**The step id of this test data */
	stepId: string;

	/**The stream id of this test data */
	streamId: string;

	/**The template ref id of the related job */
	templateRefId: string;
}

export type UpdateStepResponse = {
	/** If a new step is created (due to specifying the retry flag), specifies the batch id */
	batchId?: string;

	/** If a step is retried, includes the new step id */
	stepId?: string
}

/**Request to update a jobstep batch */
export type UpdateBatchRequest = {

	/**The new log file id */
	logId?: string;

	/**The state of this batch */
	state?: JobStepBatchState;
}

/**Information about a jobstep batch */
export type GetBatchResponse = {

	/**Unique id for this batch */
	id: string;

	/**The unique log file id */
	logId?: string;

	/**Index of the group being executed */
	groupIdx: number;

	/**The state of this batch */
	state: JobStepBatchState;

	/**Error code for this batch */
	error: JobStepBatchError;

	/**Steps within this run */
	steps: GetStepResponse[];

	/**The agent assigned to execute this group */
	agentId?: string;

	/// The agent type
	agentType: string;

	/** The USD rate of an agent hour */
	agentRate?: number;

	/**The agent session holding this lease */
	sessionId?: string;

	/**The lease that's executing this group */
	leaseId?: string;

	/**The priority of this batch */
	weightedPriority: number;

	/**Time at which the group became ready (UTC) */
	readyTime?: Date | string;

	/**Time at which the group started (UTC). */
	startTime?: Date | string;

	/**Time at which the group finished (UTC) */
	finishTime?: Date | string;

	/** Human-readable names of lease outcome reasons (only populated when batch has an error) */
	leaseOutcomeReasons?: string[];
}

/**Describes the history of a step */
export interface GetJobStepRefResponse {
	/**The job id */
	jobId: string;

	/**The step name */
	stepName?: string;

	/**The batch containing the step */
	batchId: string;

	/**The step identifier */
	stepId: string;

	/**The change number being built */
	change: number;

	/**The step log id */
	logId?: string;

	/**The pool id */
	poolId?: string;

	/** The agent id */
	agentId?: string;

	/**State of the step. */
	state?: JobStepState;

	/**Outcome of the step, once complete. */
	outcome?: JobStepOutcome;

	jobStartTime: Date | string;

	/**Time at which the step started. */
	startTime: Date | string;

	/**Time at which the step finished. */
	finishTime?: Date | string;

	/** Issue ids affecting this step ref */
	issueIds?: number[];

	/** Step metadata */
	metadata?: string[];

}

export interface GetJobStepTraceResponse {

	Name: string;

	Start: number

	Finish: number

	Service?: string;

	Resource?: string;

	Children: GetJobStepTraceResponse[];
}

/**Parameters required to create log file */
export type CreateLogFileRequest = {
	/**Job Id this log file belongs to */
	jobId: string;

}

/**Response from creating a log file */
export type CreateLogFileResponse = {
	/**Unique id for this log file */
	id: string;

}

/**Response describing a log file */
export type GetLogFileResponse = {

	/**Unique id of the log file */
	id: string;

	/** Unique id of the job for this log file */
	jobId: string;

	/** Unique id of the lease for this log file */
	leaseId?: string;

	/** Unique id of the session for this log file */
	sessionId?: string;

	/** Type of events stored in this log */
	type: LogType;

	/** Length of the log, in bytes */
	length: number;

	/** Number of lines in the file	*/
	lineCount: number;

	/**Per-object permissions */
	acl?: GetAclResponse;

}

/**Parameters request a zip file for artifacts */
export type GetArtifactZipRequest = {

	/**JobId to get all artifacts for */
	jobId?: string;

	/**StepId to get all artifacts for */
	stepId?: string;

	/**Individual file name if we've just got one file */
	fileName?: string;

	/** Whether or not we're a forced single file download (versus a link click for one file) */
	isForcedDownload?: boolean;

	/** Whether or not to open the resulting link in place or in a new tab */
	isOpenNewTab?: boolean;

	/**List of arbitrary artifact Ids to generate a zip for */
	artifactIds?: string[];
}


// Artifacts V2

export enum ArtifactBrowserType {
	Directory,
	File,
	NavigateUp
}

export type ArtifactBrowserItem = {
	key: string;
	text: string;
	icon?: string;
	size?: number;
	type: ArtifactBrowserType;
	dirResponse?: GetArtifactDirectoryEntryResponse;
	fileResponse?: GetArtifactFileEntryResponse;
}

export type ArtifactSelection = {
	filesSelected: number;
	directoriesSelected: number;
	size: number;
	items: ArtifactBrowserItem[];
}

export type ArtifactContextType = "step-trace" | "step-output" | "step-saved" | string;

/// Request to create a zip file with artifact data
export type CreateZipRequest = {
	/// Filter lines for the zip. Uses standard <see cref="FileFilter"/> syntax.
	filter?: string[];
}


/** Describes an artifact */
export type GetArtifactResponse = {
	id: string;
	type: ArtifactContextType;
	keys: string[]
	name: string;
	description?: string;
	streamId?: string;
	change?: number;
	metadata?: string[];
}

/** Result of an artifact search */
export type FindArtifactsResponse = {
	/** List of artifacts matching the search criteria*/
	artifacts: GetArtifactResponse[];
}

/** Describes a file within an artifact */
export type GetArtifactFileEntryResponse = {

	name: string;
	length: number;
	hash: string;
}

/** Describes a directory within an artifact */
export type GetArtifactDirectoryResponse = {

	/** Names of sub-directories */
	directories?: GetArtifactDirectoryEntryResponse[];

	/** Files within the directory */
	files?: GetArtifactFileEntryResponse[];
}


/** Describes a directory within an artifact */
export type GetArtifactDirectoryEntryResponse = GetArtifactDirectoryResponse & {
	name: string;
	length: number;
	hash: string;
}

/**Parameters required to update a log file */
export type UpdateLogFileRequest = {

	/** New permissions */

	acl?: UpdateAclRequest;
}


export type CreateNodeRequest = {

	/**The name of this node  */
	name: string;

	/**Indices of nodes which must have succeeded for this node to run */
	inputDependencies?: string[];

	/**Indices of nodes which must have completed for this node to run */
	orderDependencies?: string[];

	/**The priority of this node */
	priority?: Priority;

	/**This node can be run multiple times */
	allowRetry?: boolean;

	/**This node can start running early, before dependencies of other nodes in the same group are complete */
	runEarly?: boolean;

	/**Credentials required for this node to run. This dictionary maps from environment variable names to a credential property in the format 'CredentialName.PropertyName' */
	credentials?: { [key: string]: string };

	/**Properties for this node */
	properties?: { [key: string]: string };
}

/**Information about a group of nodes */
export type CreateGroupRequest = {

	/**The executor to use for this group */
	executor: string;

	/**The type of agent to execute this group */
	agentType: string;

	/**Nodes in the group */
	nodes: CreateNodeRequest[];

}

/**Request to update a node */
export type UpdateNodeRequest = {

	/**The priority of this node */
	priority?: Priority;

}

/**Parameters to create a new project */
export type CreateProjectRequest = {

	/**Name for the new project */
	name: string;

	/**Properties for the new project */
	properties?: { [key: string]: string };
}

/**Response from creating a new project */
export type CreateProjectResponse = {

	/**Unique id for the new project */
	id: string;
}

/**Parameters to update a project */
export type UpdateProjectRequest = {

	/**Optional new name for the project */
	name?: string;

	/**Properties to update for the project. Properties set to null will be removed. */
	properties?: { [key: string]: string };


	/**Custom permissions for this object */
	acl?: UpdateAclRequest;
}

/**Information about a stream within a project */
export interface GetProjectStreamResponse {
	/**The stream id */
	id: string;

	/**The stream name */
	name: string;
}


/**Information about a category to display for a stream */
export type GetProjectCategoryResponse = {
	/**Heading for this column */
	name: string;

	/**Index of the row to display this category on */
	row: number;

	/**Whether to show this category on the nav menu */
	showOnNavMenu: boolean;

	/**Patterns for stream names to include */
	includePatterns: string[];

	/**Patterns for stream names to exclude */
	excludePatterns: string[];

	/**Streams to include in this category */
	streams: string[];
}

/**Response describing a project */
export type GetProjectResponse = {
	/**Unique id of the project */
	id: string;

	/**Name of the project */
	name: string;

	/**Order to display this project on the dashboard */
	order: number;

	/**List of streams that are in this project */
	// streams?: ProjectStreamData[];

	/**List of stream categories to display */
	categories?: GetProjectCategoryResponse[];

	/**Properties for this project. */
	properties: { [key: string]: string };

	/**Custom permissions for this object */
	acl?: GetAclResponse;
}

/**Parameters to create a new schedule */
export type CreateSchedulePatternRequest = {

	/**Days of the week to run this schedule on. If null, the schedule will run every day. */
	daysOfWeek?: string[];

	/**Time during the day for the first schedule to trigger. Measured in minutes from midnight. */
	minTime: number;

	/**Time during the day for the last schedule to trigger. Measured in minutes from midnight. */
	maxTime?: number;

	/**Interval between each schedule triggering */
	interval?: number;
}

/**Parameters to create a new schedule */
export type CreateScheduleRequest = {

	/**Name for the new schedule */
	name: string;

	/**Whether the schedule should be enabled */
	enabled: boolean;

	/**Maximum number of builds that can be active at once */
	maxActive: number;

	/**The stream to run this schedule in */
	streamId: string;

	/**The template job to execute */
	templateId: string;

	/**Parameters for the template */
	templateParameters: { [key: string]: string };

	/**New patterns for the schedule */
	patterns: CreateSchedulePatternRequest[];

}

/**Response from creating a new schedule */
export type CreateScheduleResponse = {

	/**Unique id for the new schedule */
	id: string;
}

/**Parameters to update a schedule */
export type UpdateScheduleRequest = {

	/**Optional new name for the schedule */
	name?: string;

	/**Whether to enable the schedule */
	enabled?: boolean;

	/**Maximum number of builds that can be active at once */
	maxActive: number;

	/**The template job to execute */
	templateHash?: string;

	/**Parameters for the template */
	templateParameters?: { [key: string]: string };

	/**New patterns for the schedule */
	patterns?: CreateSchedulePatternRequest[];

	/** Custom permissions for this object */
	acl?: UpdateAclRequest;
}

/**Information about a schedule pattern */
export type GetSchedulePatternResponse = {

	/**Days of the week to run this schedule on. If null, the schedule will run every day. */
	daysOfWeek?: string[];

	/**Time during the day for the first schedule to trigger. Measured in minutes from midnight. */
	minTime: number;

	/**Time during the day for the last schedule to trigger. Measured in minutes from midnight. */
	maxTime?: number;

	/**Interval between each schedule triggering */
	interval?: number;

}

/**Gate allowing a schedule to trigger.*/
export type ScheduleGateConfig = {
	/**The template containing the dependency*/
	templateId: string;

	/**Target to wait for*/
	target: string;
}

/**Response describing a schedule */
export type GetScheduleResponse = {

	/**Whether the schedule is currently enabled */
	enabled: boolean;

	/**Maximum number of scheduled jobs at once */
	maxActive: number;

	/**Maximum number of changes the schedule can fall behind head revision. If greater than zero, builds will be triggered for every submitted changelist until the backlog is this size. */
	maxChanges: number;

	/** Gate for this schedule to trigger */
	gate?: ScheduleGateConfig;

	/**The template job to execute */
	templateId: string;

	/**Parameters for the template */
	templateParameters: { [key: string]: string };

	/**New patterns for the schedule */
	patterns: GetSchedulePatternResponse[];

	/* Last changelist number that this was triggered for */
	lastTriggerChange: number;

	/** Last time that the schedule was triggered */
	lastTriggerTime: Date | string;

	/// Next trigger times for schedule
	nextTriggerTimesUTC: Date[];

	/** List of active jobs */
	activeJobs: string[];



}

/**Response describing when a schedule is expected to trigger */
export type GetScheduleForecastResponse = {

	/**Next trigger times */
	times: Date | string[];

}


/**Parameters for creating a new software archive */
export type CreateSoftwareRequest = {

	/**Whether this software should be the default */
	default: boolean;
}

/**Information about a client version */
export type CreateSoftwareResponse = {

	/**The software id */
	id: string;

}

/**Parameters for updating a software archive */
export type UpdateSoftwareRequest = {

	/**Whether this software should be the default */
	default: boolean;
}

/**Information about an uploaded software archive */
export type GetSoftwareResponse = {

	/**Unique id for this enty */
	id: string;

	/**Name of the user that created this software */
	uploadedByUser?: string;

	/**Time at which the client was created */
	uploadedAtTime: Date | string;

	/**Name of the user that created this software */
	madeDefaultByUser?: string;

	/**Time at which the client was made default. */
	madeDefaultAtTime?: Date | string;

}


/**Parameters to create a new stream */
export type CreateStreamRequest = {

	/**Unique id for the project */
	projectId: string;

	/**Name for the new stream */
	name: string;

	/**Properties for the new stream */
	properties?: { [key: string]: string };



}

/**Response from creating a new stream */
export type CreateStreamResponse = {

	/**Unique id for the new stream */
	id: string;

}

/**Parameters to update a stream */
export type UpdateStreamRequest = {

	/**Optional new name for the stream */
	name?: string;

	/**Properties to update on the stream. Properties with a value of null will be removed. */
	properties?: { [key: string]: string | null };

	/** Custom permissions for this object */
	acl?: UpdateAclRequest;
}

/**Mapping from a BuildGraph agent type to a se t of machines on the farm */
export type GetAgentTypeResponse = {

	/**Pool of agents to use for this agent type */
	pool: string;

	/**Name of the workspace to sync */
	workspace?: string;

	/**Pool of agents to use to execute this work */
	requirements?: GetAgentRequirementsResponse;

	/**Path to the temporary storage dir */
	tempStorageDir?: string;

	/**Environment variables to be set when executing the job */
	environment?: { [key: string]: string };

}

export type GetAgentTelemetrySampleResponse = {
	time: Date;
	userCpu: number;
	idleCpu: number;
	systemCpu: number;
	freeRam: number;
	usedRam: number;
	totalRam: number;
	workingFreeDisk: number;
	workingTotalDisk: number;
	systemFreeDisk: number;
	systemTotalDisk: number;
}

/**Information about a workspace type */
export type GetWorkspaceTypeResponse = {

	/**The Perforce server and port (eg. perforce:1666) */
	serverAndPort?: string;

	/**User to log into Perforce with (defaults to buildmachine) */
	userName?: string;

	/**Identifier to distinguish this workspace from other workspaces. Defaults to the workspace type name. */
	identifier?: string;

	/**Override for the stream to sync */
	stream?: string;

	/**Custom view for the workspace */
	view?: string[];

	/**Whether to use an incrementally synced workspace */
	incremental: boolean;
}


/** State information for a step in the stream */
export type GetTemplateStepStateResponse = {

	/**Name of the step */
	name: string;

	/**User who paused the step */
	pausedByUserInfo?: GetThinUserInfoResponse;

	/**The UTC time when the step was paused*/
	pauseTimeUtc?: Date | string;
}

/**  Updates an existing stream template ref */
export type UpdateTemplateRefRequest = {

	/** Step states to update */
	stepStates?: UpdateStepStateRequest[];

}

/** Step state update request */
export type UpdateStepStateRequest = {

	/** Name of the step */
	name: string;

	/** User who paused the step */
	pausedByUserId?: string;

}


/// Response describing a template
export type GetTemplateResponseBase = {

	/// Name of the template
	name: string;

	/// Default priority for this job
	priority?: Priority;

	/// Whether to allow preflights of this template
	allowPreflights: boolean;

	/// Whether to always update issues on jobs using this template
	updateIssues: boolean;

	/// The initial agent type to parse the BuildGraph script on
	initialAgentType?: string;

	/// Path to a file within the stream to submit to generate a new changelist for jobs
	submitNewChange?: string;

	/// Parameters for the job.
	arguments: string[];

	/// List of parameters for this template
	parameters: ParameterData[];

	/// Description for the template, supports markdown
	description?: string;
}

/// Query selecting the base changelist to use
export type ChangeQueryConfig = {

	/// Name of this query, for display on the dashboard.
	name?: string;

	/// Condition to evaluate before deciding to use this query. May query tags in a preflight.
	condition?: any;

	/// The template id to query
	templateId?: string;

	/// The target to query
	target?: string;

	/// Whether to match a job that produced warnings
	outcomes?: JobStepOutcome[];

	/// Finds the last commit with this tag
	commitTag?: any;
}

/**Information about a template in this stream */
export type GetTemplateRefResponse = GetTemplateResponseBase & {

	/**Unique id of this template ref, (sanitized name) */
	id: string;

	/**Hash of the template definition */
	hash: string;

	/// The schedule for this ref
	schedule?: GetScheduleResponse;

	/** Step state for template in stream */
	stepStates?: GetTemplateStepStateResponse[];

	/** List of queries for the default changelist */
	defaultChange?: ChangeQueryConfig[];

	/** Whether the current user can run the template or not */
	canRun?: boolean;

}

/** Specifies defaults for running a preflight */
export type DefaultPreflightConfig = {

	/** The template id to query */
	templateId?: string;

	change?: ChangeQueryConfig;
}


/**  Configuration for an issue workflow */
export type WorkflowConfig = {

	/** Identifier for this workflow */
	id: string;

	/** Name of the tab to post summary data to */
	summaryTab?: string;

	/** Channel where reports are sent */
	reportChannel?: string;

	/** Channel where items to triage are sent */
	triageChannel?: string;
}

/**Response describing a stream alias */
export type GetStreamAliasResponse = {

	/**The unique name/identifier for this alias */
	name: string;

	/**The subset of template IDs available through this alias */
	templates: string[];

}

/**Response describing a stream */
export type GetStreamResponse = {

	/**Unique id of the stream */
	id: string;

	/**Unique id of the project containing this stream */
	projectId: string;

	/**Name of the stream */
	name: string;

	/**The config file path on the server*/
	configPath?: string;

	/**Revision of the config file */
	configRevision?: string;

	/**List of tabs to display for this stream*/
	tabs: GetStreamTabResponse[];

	/** Default template for running preflights */
	defaultPreflight?: DefaultPreflightConfig;

	/**Map of agent name to type */
	agentTypes: { [key: string]: GetAgentTypeResponse };

	/**Map of workspace name to type */
	workspaceTypes?: { [key: string]: GetWorkspaceTypeResponse };

	/**Templates for jobs in this stream */
	templates: GetTemplateRefResponse[];

	/**Properties for this stream */
	properties: { [key: string]: string };

	/** Workflows for this stream */
	workflows: WorkflowConfig[];

	/** Stream aliases for this stream */
	aliases?: GetStreamAliasResponse[];

	/** Custom permissions for this object */
	acl?: GetAclResponse;

}

/**Information about a template parameter */
export type CreateTemplateParameterRequest = {

	/**Name of the parameter */
	name: string;

	/**Type of the template parameter */
	type: TemplateParameterType;

	/**Default value for this parameter */
	default: string;

	/**Whether this parameter is required */
	required: boolean;
}

/**Parameters to create a new template */
export type CreateTemplateRequest = {

	/**Name for the new template */
	name: string;

	/**Default priority for this job */
	priority: Priority;

	/**Whether to allow preflights of this template */
	allowPreflights: boolean;

	/**Whether always update issues regardless of how job was created */
	updateIssues: boolean;

	/**Array of nodes for this job */
	groups: CreateGroupRequest[];

	/**List of aggregates for this template */
	aggregates: CreateAggregateRequest[];

	/**List of labels  for this template */
	labels: CreateLabelRequest[];

	/**Parameter names for this template. These will be assigned to properties of the job at startup. */
	parameters: CreateTemplateParameterRequest[];

	/**Properties for the new template */
	properties: { [key: string]: string };
}

/**Describes how to render a group parameter */
export enum GroupParameterStyle {
	/**Separate tab on the form */
	Tab = "Tab",

	/**Section with heading */
	Section = "Section"
}

/**Style of list parameter */
export enum ListParameterStyle {
	/**Regular drop-down list. One item is always selected. */
	List = "List",

	/**Drop-down list with checkboxes */
	MultiList = "MultiList",

	/**Tag picker from list of options */
	TagPicker = "TagPicker"
}

export enum ParameterType {
	Bool = "Bool",
	List = "List",
	Text = "Text",
	Group = "Group"
}


/**Base class for template parameters */
export type ParameterData = {
	type: ParameterType;

	// client side key, with group encoding
	parameterKey?: string;
}

/**Used to group a number of other parameters */
export type GroupParameterData = ParameterData & {

	/**Label to display next to this parameter */
	label: string;

	/**How to display this group */
	style: GroupParameterStyle;

	/**List of child parameters */
	children: ParameterData[];
}

/**Free-form text entry parameter */
export type TextParameterData = ParameterData & {

	// Unique id for this parameter
	id: string;

	/**Name of the parameter associated with this parameter. */
	label: string;

	/**Argument to pass to the executor */
	argument: string;

	/**Default value for this argument */
	default: string;

	/**Hint text for this parameter */
	hint?: string;

	/**Regex used to validate this parameter */
	validation?: string;

	/**Message displayed if validation fails, informing user of valid values. */
	validationError?: string;

	/**Tool-tip text to display */
	toolTip?: string;

}

/**Possible option for a list parameter */
export type ListParameterItemData = ParameterData & {

	// Unique id for this parameter
	id: string;

	/**Optional group heading to display this entry under, if the picker style supports it. */
	group?: string;

	/**Name of the parameter associated with this list. */
	text: string;

	/// <summary>
	/// Argument to pass with this parameter, if enabled
	/// </summary>
	argumentIfEnabled?: string;

	/// <summary>
	/// Argument to pass with this parameter, if disabled
	/// </summary>
	argumentIfDisabled?: string;

	/// <summary>
	/// Arguments to pass with this parameter, if enabled
	/// </summary>
	argumentsIfEnabled?: string[];

	/// <summary>
	/// Arguments to pass with this parameter, if disabled
	/// </summary>
	argumentsIfDisabled?: string[];

	/**Whether this item is selected by default */
	default: boolean;

}

/**Allows the user to select a value from a constrained list of choices */
export type ListParameterData = ParameterData & {
	/**Label to display next to this parameter. Defaults to the parameter name. */
	label: string;

	/**The type of list parameter */
	style: ListParameterStyle;

	/**List of values to display in the list */
	items: ListParameterItemData[];

	/**Tool-tip text to display */
	toolTip?: string;
}

/**Allows the user to toggle an option on or off */
export type BoolParameterData = ParameterData & {

	// Unique id for this parameter
	id: string;

	/**Name of the parameter associated with this parameter. */
	label: string;

	/**Value if enabled */
	argumentIfEnabled?: string;

	/**Value if disabled */
	argumentIfDisabled?: string;

	/**Arguments if enabled */
	argumentsIfEnabled?: string[];

	/**Arguments if disabled */
	argumentsIfDisabled?: string[];

	/**Whether this argument is enabled by default */
	default: boolean;

	/**Tool-tip text to display */
	toolTip?: string;

}

/** Information about a commit */
export type GetChangeSummaryResponse = {

	/**  The source changelist number */
	number: number;

	/**  The description text */
	description: string;

	/** The submit date of the changelist */
	dateUtc: Date | string;

	/**  Information about the change author */
	authorInfo: GetThinUserInfoResponse;

}

/**Information about a device attached to this agent */
export type GetDeviceCapabilitiesResponse = {

	/**Logical name of this device */
	name: string;

	/**Required properties for this device, in the form "KEY=VALUE" */
	properties?: string[];

	/**Required resources for this node. If null, the node will assume exclusive access to the device. */
	resources?: { [key: string]: number };

}

/**Information about the capabilities of this agent */
export type GetAgentCapabilitiesResponse = {

	/**Information about the devices required for this node to run */
	devices?: GetDeviceCapabilitiesResponse[];

	/**Global agent properties for this node */
	properties?: string[];

}

/**Information about the device requirements of a node */
export type CreateDeviceRequirementsRequest = {

	/**Logical name of this device */
	name: string;

	/**Required properties for this device, in the form "KEY=VALUE" */
	properties?: string[];

	/**Required resources for this node. If null, the node will assume exclusive access to the device. */
	resources?: { [key: string]: number };

}

/**Information about the agent requirements of node */
export type CreateAgentRequirementsRequest = {

	/**Information about the devices required for this node to run */
	devices?: CreateDeviceRequirementsRequest[];

	/**Global agent properties for this node */
	properties?: string[];

	/**Whether the agent can be shared with another job */
	shared: boolean;
}

/**Information about the device requirements of a node */
export type GetDeviceRequirementsResponse = {

	/**Logical name of this device */
	name: string;

	/**Required properties for this device, in the form "KEY=VALUE" */
	properties?: string[];

	/**Required resources for this node. If null, the node will assume exclusive access to the device. */
	resources?: { [key: string]: number };

}

/**Information about the agent requirements of node */
export type GetAgentRequirementsResponse = {

	/**Information about the devices required for this node to run */
	devices?: GetDeviceRequirementsResponse[];

	/**Global agent properties for this node */
	properties?: string[];

	/**Whether the agent can be shared with another job */
	shared: boolean;

}

/**Individual entry in an ACL */
export type UpdateAclEntryRequest = {
	/**Name of the user or group */
	roles: string[];

	/**Array of actions to allow */
	allow: string[];
}

/**Parameters to update an ACL */
export type UpdateAclRequest = {

	/**Entries to replace the existing ACL */
	entries: UpdateAclEntryRequest[] | null;

	/**Whether to inherit permissions from the parent ACL */
	inheritPermissions: boolean | null;


	/** List of exceptions to the inherited setting */
	exceptions?: string[];
}

/**Individual entry in an ACL */
export type GetAclEntryResponse = {
	/**Names of the user or group */
	roles: string[];

	/**Array of actions to allow */
	allow: string[];
}

/**Information about an ACL */
export type GetAclResponse = {
	/**Entries to replace the existing ACL */
	entries: GetAclEntryResponse[];

	/**Whether to inherit permissions from the parent entity */
	inheritPermissions: boolean;
}

// Labels

/**Type of a column in a jobs tab */
export enum JobsTabColumnType {
	/**Contains labels */
	Labels = "Labels",

	/**Contains parameters */
	Parameter = "Parameter"
}

/**Describes a column to display on the jobs page */
export type GetJobsTabColumnResponse = {

	type: JobsTabColumnType;

	/**Heading for this column */
	heading: string;

	/**Relative width of this column. */
	relativeWidth?: number;
}

export type GetJobsTabLabelColumnResponse = GetJobsTabColumnResponse & {

	/**Category of labels to display in this column. If null, includes any label not matched by another column. */
	category?: string;

}

export type GetJobsTabParameterColumnResponse = GetJobsTabColumnResponse & {
	/** Parameter to show in this column */
	parameter?: string;
}

export enum TabType {

	Jobs = "Jobs"

}
/** Style for rendering a tab */
export enum TabStyle {

	/// Regular job list	
	Normal = "Normal",

	/// Omit job names, show condensed view	
	Compact = "Compact"
}

/**Information about a page to display in the dashboard for a stream */
export type GetStreamTabResponse = {

	/**Title of this page */
	title: string;

	type: TabType;

	style: TabStyle;

	showNames?: boolean;
};

/**Describes a job page */
export type GetJobsTabResponse = GetStreamTabResponse & {

	/** Whether to show names on the page */
	showNames: boolean;

	/** Whether to show preflights */
	showPreflights?: boolean;

	/** List of templates to show on the page */
	templates?: string[];

	/**Columns to display for different types of labels */
	columns?: GetJobsTabColumnResponse[];

};

/** State of an label */
export enum LabelState {

	/** label is not currently being built (no required nodes are present)*/
	Unspecified = "Unspecified",

	/** Steps are still running */
	Running = "Running",

	/** All steps are complete */
	Complete = "Complete"
}

/// Outcome of an aggregate
export enum LabelOutcome {

	/** Aggregate is not currently being built */
	Unspecified = "Unspecified",

	/** A step dependency failed */
	Failure = "Failure",

	/** A dependency finished with warnings */
	Warnings = "Warnings",

	/** Successful */
	Success = "Success"
}


/** State of the job */
export enum JobState {

	/** Waiting for resources*/
	Waiting = "Waiting",

	/** Currently running one or more steps*/
	Running = "Running",

	/** All steps have completed */
	Complete = "Complete"
}


/**The timing info for a job*/
export type GetJobTimingResponse = {

	/** The job response */
	jobResponse: JobData;

	/**Timing info for each step */
	steps: { [key: string]: GetStepTimingInfoResponse };

	/**Timing information for each label */
	labels: GetLabelTimingInfoResponse[];
}
/** batch timing info */
export type FindJobTimingsResponse = {
	timings: { [jobId: string]: GetJobTimingResponse };
}

/**Information about the timing info for a label */
export type GetLabelTimingInfoResponse = GetTimingInfoResponse &
{
	/**Category for the label */
	category: string;

	/**Name of the label */
	name: string;
}

/**State of an label within a job */
export type GetLabelStateResponse = {

	// Name to show for this label on the dashboard
	dashboardName?: string;

	// Category to show this label in on the dashboard
	dashboardCategory?: string;

	// Name to show for this label in UGS
	ugsName?: string;

	// Project to display this label for in UGS
	ugsProject?: string;

	// Steps to include in the status of this label
	steps: JobStepId[];

	/**State of the label */
	state?: LabelState;

	/**Outcome of the label */
	outcome?: LabelOutcome;
}

// --- Label Details (hover card) ---

/** Aggregated label details for hover card display */
export interface GetLabelDetailsResponse {
	dashboardName?: string;
	dashboardCategory?: string;
	state: LabelState;
	outcome: LabelOutcome;
	steps: GetLabelStepResponse[];
	logEvents: Record<string, GetLabelLogEventsResponse>;
	totalStepCount: number;
	issues: GetLabelIssueResponse[];
	previousBuild?: GetLabelPreviousBuildResponse;
	crossStreamChain?: GetLabelCrossStreamChainResponse;
}

/** Step summary within a label */
export interface GetLabelStepResponse {
	id: string;
	name: string;
	state: JobStepState;
	outcome: JobStepOutcome;
	logId?: string;
	startTime?: string;
	finishTime?: string;
	diagnosticMessage?: string;
}

/** Log events for a single step's log */
export interface GetLabelLogEventsResponse {
	total: number;
	events: GetLogEventResponse[];
}

/** Build health issue summary for label hover */
export interface GetLabelIssueResponse {
	id: number;
	summary: string;
	severity: IssueSeverity;
	owner?: GetThinUserInfoResponse;
	acknowledged: boolean;
	resolvedInStream: boolean;
	externalIssueKey?: string;
	workflowThreadUrl?: string;
}

/** Previous build's label status */
export interface GetLabelPreviousBuildResponse {
	jobId: string;
	change: number;
	labelIndex: number;
	state: LabelState;
	outcome: LabelOutcome;
}

/** Cross-stream label status ordered by merge chain */
export interface GetLabelCrossStreamChainResponse {
	botName: string;
	entries: GetLabelCrossStreamEntry[];
}

/** Single entry in the cross-stream merge chain */
export interface GetLabelCrossStreamEntry {
	streamId: string;
	branchName: string;
	jobId?: string;
	state?: LabelState;
	outcome?: LabelOutcome;
	hasApprovalGateBefore: boolean;
	isCurrent: boolean;
}

/**Information about the timing info for a particular target */
export type GetTimingInfoResponse = {

	/**Wait time on the critical path */
	totalWaitTime?: number;

	/**Sync time on the critical path */
	totalInitTime?: number;

	/**Duration to this point */
	totalTimeToComplete?: number;

	/**Average wait time by the time the job reaches this point */
	averageTotalWaitTime?: number;

	/**Average sync time to this point */
	averageTotalInitTime?: number;

	/**Average duration to this point */
	averageTotalTimeToComplete?: number;

}

/**Information about the timing info for a particular target */

export type GetStepTimingInfoResponse = GetTimingInfoResponse & {

	/** Name of this node */
	name?: string;

	/**Average wait time for this step */
	averageStepWaitTime?: number;

	/**Average init time for this step */
	averageStepInitTime?: number;

	/**Average duration for this step */
	averageStepDuration?: number;

}


/**Request used to update notifications */
export type UpdateNotificationsRequest = {

	/** Notify via email */
	email?: boolean;

	/** Notify via Slack */
	slack?: boolean;
}

/**Response describing notifications */
export type GetNotificationResponse = {

	/** Notify via email */
	email?: boolean;

	/** Notify via Slack */
	slack?: boolean;
}

export type JobCompleteEventRecord = {
	outcome: string;

	type: string;

	streamId: string;

	templateId: string;
}

export type LabelCompleteEventRecord = {

	type: string;

	streamId: string;

	templateId: string;

	labelName: string;

	categoryName: string;

	outcome: string;
}

export type StepCompleteEventRecord = {

	type: string;

	streamId: string;

	templateId: string;

	stepName: string;

	outcome: string;
}

/**Request used to create subscriptions */
export type CreateSubscriptionRequest = {

	event: JobCompleteEventRecord | LabelCompleteEventRecord | StepCompleteEventRecord;

	userId?: string;

	notificationType: SubscriptonNotificationType
}

/**Request used to create subscriptions */
export type CreateSubscriptionResponse = {

	id: string;
}

/**Request used to create subscriptions */
export type GetSubscriptionResponse = {

	id: string;

	event: JobCompleteEventRecord | LabelCompleteEventRecord | StepCompleteEventRecord;

	userId?: string;

	notificationType: SubscriptonNotificationType
}

/**Identifies a particular changelist and job */
export type GetIssueStepResponse = {

	/**The changelist number */
	change: number;

	/** Severity of the issue in this step */
	severity: IssueSeverity;

	/** Name of the job containing this step */
	jobName: string;

	/**The unique job id */
	jobId: string;

	/**The unique batch id */
	batchId: string;

	/**The unique step id */
	stepId: string;

	/**  Time at which the step ran */
	stepTime: Date | string;

	/// The unique log id
	logId?: string;

}

/** Information about a template affected by an issue */
export type GetIssueAffectedTemplateResponse = {

	/** The template id */
	templateId: string;

	/**  The template name */
	templateName: string;

	/**  Whether it has been resolved or not */
	resolved: boolean;

	/** Severity of this template */
	severity: IssueSeverity;

}

/**Trace of a set of node failures across multiple steps */
export type GetIssueSpanResponse = {

	/** Unique id of this span */
	id: string;

	/** The template containing this step */
	templateId: string;

	/**Name of the step */
	name: string;

	/**The previous build  */
	lastSuccess?: GetIssueStepResponse;

	/// Workflow that this span belongs to
	workflowId?: string;

	/**The failing builds for a particular event */
	steps: GetIssueStepResponse[];

	/**The following successful build */
	nextSuccess?: GetIssueStepResponse;

}

/**Information about a particular step */
export type GetIssueStreamResponse = {
	/**Unique id of the stream */
	streamId: string;

	/**Minimum changelist affected by this issue (ie. last successful build) */
	minChange?: number;

	/**Maximum changelist affected by this issue (ie. next successful build) */
	maxChange?: number;

	/**Map of steps to (event signature id -> trace id) */
	nodes: GetIssueSpanResponse[];
}

/**Outcome of a particular build */
export enum IssueBuildOutcome {
	/**Unknown outcome */
	Unknown = "Unknown",

	/**Build succeeded */
	Success = "Success",

	/**Build failed */
	Error = "Error",

	/**Build finished with warnings */
	Warning = "Warning"
}

/**Information about a suspect changelist that may have caused an issue */
export type GetIssueSuspectResponse = {
	/**Number of the changelist that was submitted */
	change: number;

	/**Author of the changelist */
	author: string;

	/**The originating change */
	originatingChange?: number;

	/** Time at which the user declined this issue */
	declinedAt?: Date | string;
}

/**Information about a diagnostic */
export type GetIssueDiagnosticResponse = {
	/**The corresponding build id */
	buildId?: number;

	/**Message for the diagnostic */
	message: string;

	/**Link to the error */
	url: string;
}

/**Summary for the state of a stream in an issue */
export interface GetIssueAffectedStreamResponse {
	/**Id of the stream */
	streamId: string;

	/**Name of the stream */
	streamName: string;

	/**Whether the issue has been resolved in this stream */
	resolved: boolean;

	/** The affected templates */
	affectedTemplates: GetIssueAffectedTemplateResponse[];

	/**List of affected template ids */
	templateIds: string[];

	/** List of resolved template ids */
	resolvedTemplateIds: string[];

	/** List of resolved template ids */
	unresolvedTemplateIds: string[];
}


/**Stores information about a build health issue */
export type GetIssueResponse = {
	/**The unique object id */
	id: number;

	/**Time at which the issue was created */
	createdAt: Date | string;

	/**Time at which the issue was retrieved */
	retrievedAt: Date | string;

	/**Time at which the issue was retrieved */
	lastSeenAt: Date | string;

	/**The associated project for the issue */
	project?: string;

	/**The summary text for this issue */
	summary: string;

	/**Description of the issue*/
	description?: string;

	/**Description of the fingerprint used to identify this issue*/
	fingerprintDescription?: string;

	/** Severity of this issue	*/
	severity: IssueSeverity;

	/**Whether the issue is promoted */
	promoted: boolean;

	/** Owner of the issue */
	ownerInfo?: GetThinUserInfoResponse;

	/** Use that nominated the current owner */
	nominatedByInfo: GetThinUserInfoResponse;

	/**Time that the issue was acknowledged */
	acknowledgedAt?: Date | string;

	/**Changelist that fixed this issue */
	fixChange?: number;

	/**Root cause commit id */
	rootCommitId?: CommitId;

	/**Root cause owner info*/
	rootCauseOwnerInfo?: GetThinUserInfoResponse;

	/**Root cause category*/
	rootCauseCategory?: string;

	/**Root cause summary*/
	rootCauseSummary?: string;

	/**Duplicate issue id*/
	duplicateIssueId?: number;

	/**Time at which the issue was resolved */
	resolvedAt?: Date | string;

	/** Use info for the person that resolved the issue */
	resolvedByInfo?: GetThinUserInfoResponse;

	/**  List of stream paths affected by this issue */
	streams: string[];

	/** List of affected stream ids */
	resolvedStreams: string[];

	/** List of unresolved streams */
	unresolvedStreams: string[];

	affectedStreams: GetIssueAffectedStreamResponse[];

	/** Use info for the person that resolved the issue */
	primarySuspectsInfo: GetThinUserInfoResponse[];

	/** Whether to show alerts for this issue */
	showDesktopAlerts: boolean;

	/** External issue tracking */
	externalIssueKey?: string;

	/** User info for who quarantined issue */
	quarantinedByUserInfo?: GetThinUserInfoResponse;

	/** The UTC time when the issue was quarantined */
	quarantineTimeUtc?: Date | string;

	/** User info for who force closed the issue */
	forceClosedByUserInfo?: GetThinUserInfoResponse;

	/** Workflow thread url (Slack, etc) */
	workflowThreadUrl?: string;
}

/**Request an issue to be updated */
export type UpdateIssueRequest = {

	/** Summary of the issue */
	summary?: string;

	/**New owner of the issue, pass empty string to clear current owner  */
	ownerId?: string;

	/**User than nominates the new owner */
	nominatedById?: string | null;

	/**Whether the issue has been acknowledged */
	acknowledged?: boolean;

	/** Whether the user has declined this issue */
	declined?: boolean;

	/**The change at which the issue is claimed fixed */
	fixChange?: number | null;

	/**Root cause commit id */
	rootCommitId?: CommitId | null;

	/**Root cause owner id*/
	rootCauseOwnerId?: string | null;

	/**Root cause category*/
	rootCauseCategory?: string;

	/**Root cause summary*/
	rootCauseSummary?: string;

	/**Duplicate issue id*/
	duplicateIssueId?: number;

	/**Whether the issue should be marked as resolved */
	resolved?: boolean;

	/**Description of the issue*/
	description?: string;

	/**Whether the issue is promoted */
	promoted?: boolean;

	/**  List of spans to add to this issue	 */
	addSpans?: string[];

	/** List of spans to remove from this issue */
	removeSpans?: string[];

	/** An external issue key*/
	externalIssueKey?: string;

	/** Id of user quarantining issue */
	quarantinedById?: string;

	/** Id of user force closing the issue */
	forceClosedById?: string;

}

export type GetUtilizationTelemetryStream = {

	streamId: string;

	time: number;
}

export type GetUtilizationTelemetryPool = {

	poolId: string;

	numAgents: number;

	adminTime: number;

	otherTime: number;

	hibernatingTime?: number | undefined;

	streams: GetUtilizationTelemetryStream[];
}

export type GetUtilizationTelemetryResponse = {

	startTime: Date;

	finishTime: Date;

	pools: GetUtilizationTelemetryPool[];

	adminTime: number;

	numAgents: number;
}

export type UserClaim = {

	type: string;
	value: string;
}

export enum DownloadFormat {
	Zip = "Zip",
	UGS = "UGS",
	UTOOLBOX = "UTOOLBOX"
}

export enum DashboardPreference {
	Darktheme = "Darktheme",
	DisplayUTC = "DisplayUTC",
	DisplayClock = "DisplayClock",
	ColorSuccess = "ColorSuccess",
	ColorWarning = "ColorWarning",
	ColorError = "ColorError",
	ColorRunning = "ColorRunning",
	ColorUser1 = "ColorUser1",
	ColorUser2 = "ColorUser2",
	ColorUser3 = "ColorUser3",
	LocalCache = "LocalCache",
	LeftAlignLog = "LeftAlignLog",
	CompactViews = "CompactViews",
	ShowPreflights = "ShowPreflights",
	DownloadFormat = "DownloadFormat"

}

export type DashboardSettings = {

	preferences: Map<DashboardPreference, string>;

}

export type UpdateDashboardSettings = {

	preferences?: Record<DashboardPreference, string>;

}

/** Settings for whether various features should be enabled on the dashboard */
export type GetDashboardFeaturesResponse = {

	/** Whether the notice editor should be listed in the server menu */
	showNoticeEditor?: boolean;

	/** Whether controls for modifying pools should be shown */
	showPoolEditor?: boolean;

	/** Whether the remote desktop button should be shown on the agent modal */
	showRemoteDesktop?: boolean;

	/** Show the landing page by default */
	showLandingPage?: boolean;

	/** Custom landing page route to direct users to */
	landingPageRoute?: string;

	/** Enable CI functionality */
	showCI?: boolean;

	/** Whether to show functionality related to agents, pools, and utilization on the dashboard. */
	showAgents?: boolean;

	/** Whether to show the agent registration page. When using registration tokens from elsewhere this is not needed. */
	showAgentRegistration?: boolean;

	/** Show the Perforce server option on the server menu */
	showPerforceServers?: boolean;

	/** Show the device manager on the server menu */
	showDeviceManager?: boolean;

	/** Show automated tests (legacy) on the server menu */
	showTests?: boolean;

	/** Whether to show accounts on the server menu*/
	showAccounts?: boolean;
}

/// Job template settings for the current user
export type GetJobTemplateSettingsResponse = {

	/// The stream the job was run in	
	streamId: string;

	/// The template id of the job	
	templateId: string;

	/// The hash of the template definition	
	templateHash: string;

	/// The arguments defined when creating the job	
	arguments: string[];

	/// The last update time of the job template	
	updateTimeUtc: Date | string;

}

/**  Response describing the current user */
export type GetUserResponse = {

	/** Id of the user */
	id: string;

	/** Name of the user */
	name: string;

	/** Email of the user */
	email?: string;

	/** Avatar image URL (24px) */
	image24?: string;

	/** Avatar image URL (32px) */
	image32?: string;

	/** Avatar image URL (48px) */
	image48?: string;

	/** Avatar image URL (72px) */
	image72?: string;

	/** Claims for the user */
	claims?: UserClaim[];

	/** Whether to enable experimental features for this user */
	enableExperimentalFeatures?: boolean;

	/** Whether to always tag preflight changelists */
	alwaysTagPreflightCL?: boolean;

	/**  Settings for the dashboard */
	dashboardSettings?: DashboardSettings;

	/// <summary>
	/// Settings for whether various dashboard features should be shown for the current user
	/// </summary>
	dashboardFeatures?: GetDashboardFeaturesResponse;

	// array of user job templates settings
	jobTemplateSettings?: GetJobTemplateSettingsResponse[];

	/** List of pinned job ids */
	pinnedJobIds?: string[];

	/** List of pinned bisect task ids */
	pinnedBisectTaskIds?: string[];

}

/** Basic information about a user. May be embedded in other responses.*/
export type GetThinUserInfoResponse = {

	/**  Id of the user */
	id: string;

	/** Name of the user */
	name: string;

	/**  The user's email address */
	email: string;

}


/// <summary>
/// Request to update settings for a user
/// </summary>
export type UpdateUserRequest = {

	/** New dashboard settings */
	dashboardSettings?: UpdateDashboardSettings;

	/** Job ids to add to the pinned list */
	addPinnedJobIds?: string[];

	/** Job ids to remove from the pinned list */
	removePinnedJobIds?: string[];

	/** Bisect ids to add to the pinned list */
	addPinnedBisectTaskIds?: string[];

	/** Bisect ids to remove from the pinned list */
	removePinnedBisectTaskIds?: string[];

	/** Whether to enable experimental features for this user */
	enableExperimentalFeatures?: boolean;

	/** Whether to always tag preflight changelists */
	alwaysTagPreflightCL?: boolean;

}

// Server Status

/// Status for a subsystem within Hord
export type ServerStatusSubsystem = {
	/// Category of this subsystem
	category: string;

	/// Name of the subsystem
	name: string;

	/// List of updates
	updates: ServerStatusUpdate[];
}

/// Type of status result for a single updat
export enum ServerStatusResult {
	/// Indicates that the health check determined that the subsystem was unhealthy
	Unhealthy = "Unhealthy",

	/// Indicates that the health check determined that the component was in a subsystem state
	Degraded = "Degraded",

	/// Indicates that the health check determined that the subsystem was healthy
	Healthy = "Healthy"
}

/// A single status updat
export type ServerStatusUpdate = {
	/// Result of status update
	result: ServerStatusResult;

	/// Optional message describing the result
	message?: string;

	/// Time this update was created
	updatedAt: Date;
}

/// Response from server status controller
export type ServerStatusResponse = {
	/// List of subsystem statuses
	statuses: ServerStatusSubsystem[];
}

export type GetPerforceServerStatusResponse = {
	serverAndPort: string;
	baseServerAndPort: string;
	cluster: string;
	numLeases: number;
	status: string;
	detail: string;
}

/** Request to validate server configuration with the given files replacing their checked-in counterparts. */
export type PreflightConfigRequest = {

	/**  Change to test	*/
	shelvedChange: number;

	/**  Perforce cluster to retrieve from */
	cluster?: string;

}

/**  Response from validating config files */
export type PreflightConfigResponse = {

	/** Whether the files were validated successfully */
	result: boolean;

	/** Output message from validation */
	message?: string;

	/** Detailed response */
	detail?: string;

	/** Warnings generated during config processing (e.g. diamond include dependencies) */
	warnings?: string[];
}


/** Get object response which describes a device platform */
export type GetDevicePlatformResponse = {

	/** Unique id of device platform */
	id: string;

	/** Friendly name of device platform */
	name: string;

	/** Platform vendor models */
	modelIds: string[];
}


export enum DevicePoolType {

	/** Used in CIS jobs */
	Automation = "Automation",

	/** User devices which can be checked out and in */
	Shared = "Shared"
}

/** Device pool response object */
export type GetDevicePoolResponse = {

	/**  Id of  the device pool */
	id: string;

	/** The type of the pool */
	poolType: DevicePoolType;

	/**  Name of the device pool */
	name: string;

	/** Whether there is write access to the pool */
	writeAccess: boolean;

}


export type CreateDeviceRequest = {

	/**  The platform of the device */
	platformId: string;

	/**  The pool to assign the device */
	poolId: string;

	/**  The friendly name of the device */
	name: string;

	/**  Whether to create the device in enabled state */
	enabled?: boolean;

	/**  The network address of the device */
	address?: string;

	/**  The vendor model id of the device */
	modelId?: string;

	/** List of metadata tags on the device */
	tags?: string[];
}

export type CreateDeviceResponse = {

	/** id of created device */
	id: string;

}

/// Get response object which describes a device
export type GetDeviceUtilizationResponse = {
	/// The job id which utilized device
	jobId?: string;

	/// The job's step id
	stepId?: string;

	/// The time device was reserved
	reservationStartUtc: Date | string;

	/// The time device was freed
	reservationFinishUtc?: Date | string;
}


/** Get response object which describes a device */
export type GetDeviceResponse = {
	/** Make this type indexable by property name */
	[key: string]: any;

	/** The unique id of the device */
	id: string;

	/** The platform of the device */
	platformId: string;

	/** The pool the device belongs to */
	poolId: string;

	/** The friendly name of the device */
	name: string;

	/** Whether the device is currently enabled */
	enabled: boolean;

	/**  The address of the device (if it allows network connections) */
	address?: string;

	/** The vendor model id of the device */
	modelId?: string;

	/** Any notes provided for the device */
	notes?: string;

	/** List of metadata tags on the device */
	tags?: string[];

	/** The UTC time when a device problem was reported */
	problemTime?: Date | string;

	/** The UTC time when a device was set for maintenance */
	maintenanceTime?: Date | string;

	/// The user id that has the device checked out, pass "" to clear checkout
	checkedOutByUserId?: string;

	/// The last time the device was checked out
	checkOutTime?: Date | string;

	/// The time the device checkout will expire
	checkOutExpirationTime?: Date | string;

	/** Id of the user which last modified this device */
	modifiedByUser?: string;

	/** Utilization info */
	utilization?: GetDeviceUtilizationResponse[];

	/** The time the cleaning started */
	cleanStartTime?: Date | string;

	/** The time the cleaning finished */
	cleanFinishTime?: Date | string;

	/** Whether the device requires cleaning */
	cleanRequird?: boolean;

}

/** Device checkout request object */
export type CheckoutDeviceRequest = {

	/** Whether to checkout or in the device */
	checkout: boolean;
}



/** Request object for updating a device */
export type UpdateDeviceRequest = {

	/** The pool to assign device to */
	poolId?: string;

	/** The new name of the device */
	name?: string;

	/** Whether the device is enabled */
	enabled?: boolean;

	/** Whether to clear any problem state */
	problem?: boolean;

	/** Whether the device should be put into maintenance mode */
	maintenance?: boolean;

	/** New address or hostname of device */
	address?: string;

	/** The new model id of device */
	modelId?: string;

	/** Markdown notes */
	notes?: string;

	/** List of metadata tags on the device */
	tags?: string[];
}


/** A reservation containing one or more devices */

export type GetDeviceReservationResponse = {

	/** Randomly generated unique id for this reservation */
	id: string;

	/** Which device pool the reservation is in	*/
	poolId: string;

	/** The reserved device ids	*/
	devices: string[];

	/** JobID holding reservation */
	jobId?: string;

	/** Job step id holding reservation	*/
	stepId?: string;

	/** Reservations held by a user	*/
	userId?: string;

	/** The hostname of machine holding reservation	*/
	hostname?: string;

	/** The optional details of the reservation	*/
	reservationDetails?: string;

	/** The UTC time when the reservation was created */
	createTimeUtc: Date | string;

	/** The legacy reservation system guid, to be removed once can update Gauntlet client in all streams */
	legacyGuid: string;
}

/** Updates an existing lease */
export type UpdateLeaseRequest = {
	/** Mark this lease as aborted */
	aborted?: boolean

}

/** Server settings */
export type GetServerSettingsResponse = {

	/** The number of live server setting updates */
	numServerUpdates?: number;

	globalConfigPath: string;

	/** The server settings on local storage */
	userServerSettingsPath: string;

	/** MongoDB connection string */
	databaseConnectionString?: string;

	/** MongoDB database name */
	databaseName: string;

	/** The claim type for administrators */
	adminClaimType: string;

	/** Value of the claim type for administrators */
	adminClaimValue: string;

	/** Optional certificate to trust in order to access the database (eg. AWS cert for TLS) */
	databasePublicCert?: string;

	/** Access the database in read-only mode (avoids creating indices or updating content)
		Useful for debugging a local instance of HordeServer against a production database. */
	databaseReadOnlyMode: boolean;

	/** Optional PFX certificate to use for encryting agent SSL traffic. This can be a self-signed certificate, as long as it's trusted by agents.	*/
	serverPrivateCert?: string;

	/** Issuer for tokens from the auth provider */
	oidcAuthority?: string;

	/** Client id for the OIDC authority */
	oidcClientId?: string;

	/** Optional redirect url provided to OIDC login */
	oidcSigninRedirect?: string;

	/** Name of the issuer in bearer tokens from the server*/
	jwtIssuer?: string;

	/** Secret key used to sign JWTs. This setting is typically only used for development. In prod, a unique secret key will be generated and stored in the DB for each unique server instance.*/
	jwtSecret?: string;

	/** Length of time before JWT tokens expire, in hourse*/
	jwtExpiryTimeHours: number;

	/** Disable authentication for debugging purposes*/
	disableAuth: boolean;

	/** Whether to enable Cors, generally for development purposes*/
	corsEnabled: boolean;

	/** Allowed Cors origin */
	corsOrigin: string;

	/** Whether to enable a schedule in test data (false by default for development builds)*/
	enableScheduleInTestData: boolean;

	/** Interval between rebuilding the schedule queue with a DB query.*/
	schedulePollingInterval: any;

	/** Interval between polling for new jobs*/
	noResourceBackOffTime: any;

	/** Interval between attempting to assign agents to take on jobs*/
	initiateJobBackOffTime: any;

	/** Interval between scheduling jobs when an unknown error occurs*/
	unknownErrorBackOffTime: any;

	/** Config for connecting to Redis server(s).
		Setting it to null will disable Redis use and connection
		See format at https://stackexchange.github.io/StackExchange.Redis/Configuration.html */
	redisConnectionConfig?: string

	/** Type of write cache to use in log service, currently Supported: "InMemory" or "Redis" */
	logServiceWriteCacheType: string

	/** Provider Type, currently Supported: "S3" or "FileSystem" */
	externalStorageProviderType: any

	/** Local log/artifact storage directory, if using type filesystem */
	localLogsDir: string;

	/** Local blob storage directory, if using type filesystem */
	localBlobsDir: string;

	/** Local artifact storage directory, if using type filesystem */
	localArtifactsDir: string;

	/** S3 bucket region for logfile storage */
	s3BucketRegion: string;

	/** Arn to assume for s3.  "Basic", "AssumeRole", "AssumeRoleWebIdentity" only */
	s3CredentialType: string;

	/** S3 Client username (used in Basic auth type only) */
	s3ClientKeyId: string;

	/** S3 client password (used in Basic auth type only) */
	s3ClientSecret: string;

	/** Arn to assume for s3 */
	s3AssumeArn: string;

	/** S3 log bucket name */
	s3LogBucketName: string;

	/** S3 artifact bucket name */
	s3ArtifactBucketName: string;

	/** When using a relay storage provider, specifies the remote server to use */
	logRelayServer?: string;

	/** Authentication token for using a relay server */
	logRelayBearerToken?: string;

	/** Whether to log json to stdout */
	logJsonToStdOut: boolean;

	/** Which fleet manager service to use */
	fleetManager: any;

	/** Whether to run scheduled jobs. Not wanted for development. */
	disableSchedules: boolean;

	/** Timezone for evaluating schedules */
	scheduleTimeZone?: string;

	/** Token for interacting with Slack */
	slackToken?: string;

	/** Token for opening a socket to slack */
	slackSocketToken?: string;

	/** Channel to send stream notification update failures to */
	updateStreamsNotificationChannel?: string;

	/** URI to the SmtpServer to use for sending email notifications */
	smtpServer?: string;

	/** The email address to send email notifications from */
	emailSenderAddress?: string;

	/** The name for the sender when sending email notifications */
	emailSenderName?: string;

	/** The p4 bridge service username */
	p4BridgeServiceUsername?: string;

	/** The p4 bridge service password */
	p4BridgeServicePassword?: string;

	/** Whether the p4 bridge service account can impersonate other users */
	p4BridgeCanImpersonate: boolean;

	/** Set the minimum size of the global thread pool
		This value has been found in need of tweaking to avoid timeouts with the Redis client during bursts
		of traffic. Default is 16 for .NET Core CLR. The correct value is dependent on the traffic the Horde Server
		is receiving. For Epic's internal deployment, this is set to 40. */
	globalThreadPoolMinSize?: number;

	/** Path to the root config file */
	configPath?: string;

	/** Lazily computed timezone value */
	timeZoneInfo: any;

}

export type UpdateServerSettingsRequest = {

	settings: Record<string, string | boolean | number>;
}

export type ServerUpdateResponse = {

	errors: string[];

	restartRequired: boolean;

}


// Config

/// References a project configuration
export type ProjectConfigRef = {

	/// Unique id for the project
	id: string;

	/// Config path for the project
	path: string;
}

/// How frequently the maintence window repeats
export enum ScheduledDowntimeFrequency {
	/// Once
	Once,

	/// Every day
	Daily,

	/// Every week
	Weekly
}


/// Settings for the maintenance window
export type ScheduledDowntime = {

	/// Start time
	dateTimeOffset: any;

	/// Finish time
	finishTime: any;

	/// Frequency that the window repeats\
	frequency: ScheduledDowntimeFrequency;
}



/// User notice
export type Notice = {

	/// Unique id for this notice
	id: string;

	/// Start time to display this message
	startTime?: Date | string;

	/// Finish time to display this message
	finishTime?: Date | string;

	/// Message to display
	message: string;
}

/// Path to a platform and stream to use for syncing AutoSDK
export type AutoSdkWorkspace = {

	/// Name of this workspace
	name?: string;

	/// The agent properties to check (eg. "OSFamily=Windows")
	properties: string[];

	/// Username for logging in to the server
	userName?: string;

	/// Stream to use
	stream?: string;
}


/// Information about an individual Perforce server
export type PerforceServer = {

	/// The server and port. The server may be a DNS entry with multiple records, in which case it will be actively load balanced.
	serverAndPort: string;

	/// Whether to query the healthcheck address under each server
	healthCheck?: boolean;

	/// Whether to resolve the DNS entries and load balance between different hosts
	resolveDns?: boolean;

	/// Maximum number of simultaneous conforms on this server
	maxConformCount: number;

	/// List of properties for an agent to be eligable to use this server
	properties?: string[];
}


/// Credentials for a Perforce user
export type PerforceCredentials = {
	/// The username
	userName: string;

	/// Password for the user
	password: string;
}


/// Information about a cluster of Perforce servers.
export type PerforceCluster = {

	/// The default cluster name
	defaultName?: string;

	/// Name of the cluster
	name: string;

	/// Username for Horde to log in to this server
	serviceAccount: string;

	/// Whether the service account can impersonate other users
	canImpersonate: boolean;

	/// List of servers
	servers: PerforceServer[];

	/// List of server credentials
	credentials: PerforceCredentials[];

	/// List of autosdk streams
	autoSdk: AutoSdkWorkspace[];

}


/// Configuration for storage system
export type StorageConfig = {

	/// List of storage namespaces
	namespaces: NamespaceConfig[];
}

/// Configuration for a storage namespace
export type NamespaceConfig = {

	/// Identifier for this namespace
	id: string;

	/// Buckets within this namespace
	buckets: BucketConfig[];

	/// Access control for this namespace
	//UpdateAclRequest? Acl;
}

/// Configuration for a bucket
export type BucketConfig = {
	/// Identifier for the bucket
	id: string;
}

/** Global configuration */
export type GlobalConfig = {

	/// List of projects
	projects: ProjectConfigRef[];

	/// Manually added status messages
	notices?: Notice[];

	/// List of scheduled downtime
	downtime?: ScheduledDowntime[];

	/// List of Perforce clusters
	perforceClusters: PerforceCluster[];

	/// Maximum number of conforms to run at once
	maxConformCount?: number;

	/// List of storage namespaces
	storage?: StorageConfig;

	/// Access control list
	// public UpdateAclRequest ? Acl;
}


export type CreateProjectCategoryRequest = {
	/// <summary>
	/// Name of this category
	/// </summary>

	name: string;

	/// <summary>
	/// Index of the row to display this category on
	/// </summary>
	row: number;

	/// <summary>
	/// Whether to show this category on the nav menu
	/// </summary>
	showOnNavMenu: boolean;

	/// <summary>
	/// Patterns for stream names to include
	/// </summary>
	includePatterns: string[];

	/// <summary>
	/// Patterns for stream names to exclude
	/// </summary>
	excludePatterns: string[];

}


// Project Config
/// Stores configuration for a project
export type ProjectConfig = {

	/// Name for the new project
	name: string;

	/// Path to the project logo
	logo?: string;

	/// Categories to include in this project
	categories: CreateProjectCategoryRequest[];

	/// List of streams
	streams: StreamConfigRef[];

	/// Acl entries
	// public UpdateAclRequest? Acl;
}


/// <summary>
/// Reference to configuration for a stream
/// </summary>
export type StreamConfigRef = {

	/// <summary>
	/// Unique id for the stream
	/// </summary>
	id: string;

	/// <summary>
	/// Path to the configuration file
	/// </summary>
	path: string;
}



export type CreateAgentTypeRequest = {
	/// <summary>
	/// Pool of agents to use for this agent type
	/// </summary>
	pool: string;

	/// <summary>
	/// Name of the workspace to sync
	/// </summary>
	workspace?: string;

	/// <summary>
	/// Path to the temporary storage dir
	/// </summary>
	tempStorageDir?: string;

	/// <summary>
	/// Environment variables to be set when executing the job
	/// </summary>
	environment?: Record<string, string>;
}


/// Information about a workspace type
export type CreateWorkspaceTypeRequest = {

	/// <summary>
	/// Name of the Perforce server cluster to use
	/// </summary>
	cluster?: string;

	/// <summary>
	/// The Perforce server and port (eg. perforce:1666)
	/// </summary>
	serverAndPort?: string;

	/// <summary>
	/// User to log into Perforce with (defaults to buildmachine)
	/// </summary>
	userName?: string;

	/// <summary>
	/// Password to use to log into the workspace
	/// </summary>
	password?: string;

	/// <summary>
	/// Identifier to distinguish this workspace from other workspaces. Defaults to the workspace type name.
	/// </summary>
	identifier?: string;

	/// <summary>
	/// Override for the stream to sync
	/// </summary>
	stream?: string;

	/// <summary>
	/// Custom view for the workspace
	/// </summary>
	view?: string[];

	/// <summary>
	/// Whether to use an incrementally synced workspace
	/// </summary>
	incremental: boolean;
}


/// Config for a stream
export type StreamConfig = {

	/// <summary>
	/// Name of the stream
	/// </summary>
	name: string;

	/// <summary>
	/// The perforce cluster containing the stream
	/// </summary>
	clusterName?: string;

	/// <summary>
	/// Order for this stream
	/// </summary>
	order?: number;

	/// Notification channel for all jobs in this stream
	notificationChannel?: string;

	/// Notification channel filter for this template. Can be Success|Failure|Warnings
	notificationChannelFilter?: string;

	/// <summary>
	/// Channel to post issue triage notifications
	/// </summary>
	triageChannel?: string;

	/// <summary>
	/// Default template for running preflights
	/// </summary>
	defaultPreflightTemplate?: string;

	/// Default template for running preflights
	defaultPreflight?: DefaultPreflightConfig;

	/// <summary>
	/// List of tabs to show for the new stream
	/// </summary>
	tabs: any[]/*CreateStreamTabRequest[]*/;

	/// <summary>
	/// Map of agent name to type
	/// </summary>
	agentTypes: Record<string, CreateAgentTypeRequest>;

	/// <summary>
	/// Map of workspace name to type
	/// </summary>
	workspaceTypes: Record<string, CreateWorkspaceTypeRequest>;

	/// <summary>
	/// List of templates to create
	/// </summary>
	templates: any[];

	/// <summary>
	/// Custom permissions for this object
	/// </summary>
	// public UpdateAclRequest ? Acl;

	/// <summary>
	/// Pause stream builds until specified date
	/// </summary>
	/// public DateTime ? PausedUntil;

	/// <summary>
	/// Reason for pausing builds of the stream
	/// </summary>
	// public string ? PauseComment;
}

/// Parameters to update server settings
export type UpdateGlobalConfigRequest = {

	/// Delta updates for global config
	globalsJson?: string;

	/// projects json
	projectsJson?: Record<string, string>;

	/// streams json
	streamsJson?: Record<string, string>;

	/// default pool name
	defaultPoolName?: string;

	/// Delta updates for server settings from dashboard
	serverSettingJson?: string;

	/// Base64 encoded project logo
	projectLogo?: string;

}

export type ServerPluginInfoResponse = {
	name: string;
	description?: string;
	loaded: boolean,
	version?: string;
	/// List of ACL actions the current user has for this plugin
	userAcls?: string[];
};

export type GetServerInfoResponse = {

	/// Server version info
	serverVersion: string;

	/// The current agent version
	agentVersion?: string;

	/// The plugins on the server
	plugins: ServerPluginInfoResponse[];
}

/// Information about a span within an issue
export type FindIssueSpanResponse = {

	/// Unique id of this span
	id: string;

	/// The template containing this step
	templateId: string;

	/// Name of the step
	name: string;

	/// Workflow for this span
	workflowId?: string;

	/// The previous build
	lastSuccess?: GetIssueStepResponse;

	/// The following successful build
	nextSuccess?: GetIssueStepResponse;

}

/// Stores information about a build health issue
export type FindIssueResponse = {
	/// The unique object id
	id: number;

	/// Time at which the issue was created
	createdAt: Date | string;

	/// Time at which the issue was retrieved
	RetrievedAt: Date | string;

	/// The associated project for the issue
	project?: string;

	/// The summary text for this issue
	summary: string;

	/// Detailed description text
	description?: string;

	/// Severity of this issue
	severity: IssueSeverity;

	/// Current severity in the queried stream
	streamSeverity?: IssueSeverity;

	/// Whether the issue is promoted
	promoted: boolean;

	/// Owner of the issue
	owner?: GetThinUserInfoResponse;

	/// Owner of the issue
	mominatedBy?: GetThinUserInfoResponse;

	/// Time that the issue was acknowledged
	acknowledgedAt?: Date | string;

	/// Changelist that fixed this issue
	fixChange?: number;

	/**Root cause commit id */
	rootCommitId?: CommitId;

	/**Root cause owner id*/
	rootCauseOwner?: GetThinUserInfoResponse;

	/**Root cause category*/
	rootCauseCategory?: string;

	/**Root cause summary*/
	rootCauseSummary?: string;

	/**Duplicate issue id*/
	duplicateIssueId?: number;

	/// Time at which the issue was resolved
	resolvedAt?: Date | string;

	/// User that resolved the issue
	resolvedBy?: GetThinUserInfoResponse;

	/// Time at which the issue was verified
	verifiedAt?: Date | string;

	/// Time that the issue was last seen
	lastSeenAt: Date | string;

	/// Spans for this issue
	spans: FindIssueSpanResponse[];

	/** External issue tracking */
	externalIssueKey?: string;

	/** User who quarantined the issue */
	quarantinedBy?: GetThinUserInfoResponse;

	/** The UTC time when the issue was quarantined */
	quarantineTimeUtc?: Date | string;

	/** Workflows for which this issue is open */
	openWorkflows: string[];

	/** Workflow thread url (Slack, etc) */
	workflowThreadUrl?: string;
}

export type GetAgentSoftwareChannelResponse = {
	name?: string;
	modifiedBy?: string;
	modifiedTime: string;
	version?: string;
}

/// External Issue Response
export type GetExternalIssueResponse = {

	/** The external issue key */
	key: string;

	/** The issue link on external tracking site */
	link?: string;

	/** The issue status name, "To Do", "In Progress", etc*/
	statusName?: string;

	/** The issue resolution name, "Fixed", "Closed", etc*/
	resolutionName?: string;

	/** The issue priority name, "1 - Critical", "2 - Major", etc*/
	priorityName?: string;

	/** The current assignee's user name*/
	assigneeName?: string;

	/** The current assignee's display name*/
	assigneeDisplayName?: string;

	/** The current assignee's email address*/
	assigneeEmailAddress?: string;
}

/** Request an issue to be created on external issue tracking system */
export type CreateExternalIssueRequest = {

	/** Horde issue which is linked to external issue */
	issueId: number;

	/** Summary text for external issue */
	summary: string;

	/** A stream this this issue */
	streamId: string;

	/** External issue project id */
	projectId: string;

	/** External issue component id */
	componentId: string;

	/** External issue type id */
	issueTypeId: string;

	/** Optional description text for external issue */
	description?: string;

	/** Optional link to Horde issue */
	hordeIssueLink?: string;
}

/** Response for externally created issue */
export type CreateExternalIssueResponse = {

	/** External issue tracking key	 */
	key: string;

	/**  Link to issue on external tracking site */
	link?: string;
}

/// External issue project information
export type GetExternalIssueProjectResponse = {

	/// The project key
	projectKey: string;

	/// The name of the project
	name: string;

	/// The id of the project
	id: string;

	/// component id => name
	components: Record<string, string>;

	/// IssueType id => name
	issueTypes: Record<string, string>;
}

/// Create a notice which will display on the dashboard
export type CreateNoticeRequest = {

	/**  Message to display	*/
	message: string;
}

/// Parameters required to update a notice
export type UpdateNoticeRequest = {

	/** The id of the notice to update */
	id: string;

	/** Start time to display this message */
	startTime?: Date | string;

	/** Finish time to display this message */
	finishTime?: Date | String;

	/** Message to display */
	message?: string;
}


export type GetNoticeResponse = {

	/** The id of the notice for user created notices */
	id?: string;

	/** Start time to display this message */
	startTime?: Date | String;

	/** Finish time to display this message */
	finishTime?: Date | String;

	/** Whether this notice is for scheduled downtime */
	scheduledDowntime: boolean;

	/** Whether the notice is currently active */
	active: boolean;

	/** Message to display */
	message?: string;

	/** User id who created the notice, otherwise null if a system message */
	createdByUser?: GetThinUserInfoResponse;

}

// Device telemetry

export type DevicePoolTelemetryQuery = {
	minCreateTime?: string;
	maxCreateTime?: string;
	index?: number;
	count?: number;
};

export type DeviceTelemetryQuery = {
	id?: string[];
	poolId?: string;
	platformId?: string;
	minCreateTime?: string;
	maxCreateTime?: string;
	index?: number;
	count?: number;
};

export type GetDevicePoolReservationTelemetryResponse = {

	/** Device id for reservation */
	deviceId: string;

	/** Job id associated with reservation */
	jobId?: string;

	/** The step id of reservation */
	stepId?: string;

	/** The name of the job holding reservation */
	jobName?: string;

	/** The name of the step holding reservation */
	stepName?: string;

}

/** Device pool telemetry respponse */
export type GetDevicePlatformTelemetryResponse = {

	/** The corresponding platform id */
	platformId: string;

	/** Available device ids of this platform */
	available?: string[];

	/** Device ids in maintenance state */
	maintenance?: string[];

	/** Device ids in problem state */
	problem?: string[];

	/** Device ids in disabled state */
	disabled?: string[];

	/** Stream id => reserved devices of this platform */
	reserved?: Record<string, GetDevicePoolReservationTelemetryResponse[]>;
}


/** Device telemetry respponse */
export type GetDevicePoolTelemetryResponse = {

	/** The UTC time the telemetry data was created */
	createTimeUtc: Date | string;


	/** Individual pool id -> telemetry data points */
	telemetry: Record<string, GetDevicePlatformTelemetryResponse[]>;
}

/// Device telemetry respponse
export type GetTelemetryInfoResponse = {

	/// The UTC time the telemetry data was created
	createTimeUtc: Date | string;

	/// The stream id which utilized device
	streamId?: string;

	/// The job id which utilized device
	jobId?: string;

	/// The job's step id
	stepId?: string;

	/// The job name which utilized device
	jobName?: string;

	/// The job's step name
	stepName?: string;

	/// If this telemetry has a reservation, the start time of the reservation
	reservationStartUtc?: Date | string;

	/// If this telemetry has a reservation, the finish time of the reservation
	reservationFinishUtc?: Date | string;

	/// If this telemetry marks a detected device issue, the time of the issue
	problemTimeUtc?: Date | string;
}


/// Device telemetry respponse
export type GetDeviceTelemetryResponse = {

	/// The device id for the telemetry data
	deviceId: string;

	/// Individual telemetry data points
	telemetry: GetTelemetryInfoResponse[];

}

// Test Data ----------------------------

/// A test emvironment running in a stream
export type GetTestMetaResponse = {

	/// Meta unique id for environment
	id: string;

	/// The platforms in the environment
	platforms: string[];

	/// The build configurations being tested
	configurations: string[];

	/// The build targets being tested
	buildTargets: string[];

	/// The test project name
	projectName: string;

	/// The rendering hardware interface being used with the test
	rhi: string;

	/// The test variation identifier (or "default")
	variation: string;

}


/// A test that runs in a stream
export type GetTestResponse = {

	/// The id of the test
	id: string;

	/// The name of the test
	name: string;

	/// The display name of the test, if any
	displayName?: string;

	/// The suite the test belongs to, if any
	suiteName?: string;

	/// The environments the test runs in
	metadata: string[];
}

/// A test that runs in a stream
export type GetTestsRequest = {
	/// The id of the test
	testIds: string[];
}


/// A test suite that runs in a stream, contain subtests
export type GetTestSuiteResponse = {

	/// The id of the test suite
	id: string;

	/// The name of the test suite
	name: string;

	/// The environments the suite runs in
	metadata: string[];
}

/// Describes tests running in a stream
export type GetTestStreamResponse = {

	/// The stream id
	streamId: string;

	/// Individual tests which run in the stream
	tests: GetTestResponse[];

	/// Test suites that run in the stream
	testSuites: GetTestSuiteResponse[];

	/// Test suites that run in the stream
	testMetadata: GetTestMetaResponse[];
}

/// Test outcome
export enum TestOutcome {
	/// The test was successful
	Success = "Success",
	/// The test failed
	Failure = "Failure",
	/// The test was skipped
	Skipped = "Skipped",
	/// The test had an unspecified result
	Unspecified = "Unspecified",
	// Warnings
	Warning = "Warning"
}


/// Suite test data
export type GetSuiteTestDataResponse = {

	/// The test id
	testId: string;

	/// The ourcome of the suite test
	outcome: TestOutcome;

	/// How long the suite test ran	 (TimeSpan)
	duration: string;

	/// Test UID for looking up in test details
	uid: string;

	// The number of test warnings generated
	warningCount?: number;

	// The number of test warnings generated
	errorCount?: number;
}

/// Test details
export type GetTestDataDetailsResponse = {

	/// The corresponding test ref
	id: string;

	/// The test documents for this ref
	testDataIds: string[];

	/// Suite test data
	suiteTests?: GetSuiteTestDataResponse[];

}

/// Testt data ref
export type GetTestDataRefResponse = {
	/// ref id
	id: string;

	/// The associated stream
	streamId: string;

	/// How long the test ran (TimeSpan)
	duration: string;

	/// The build changelist upon which the test ran, may not correspond to the job changelist
	buildChangeList: number;

	/// The environment the test ran on
	metaId: string;

	/// The test id in stream
	testId?: string;

	/// The outcome of the test
	outcome?: TestOutcome;

	/// The if of the stream test suite
	suiteId?: string;

	/// The number of suite tests skipped
	suiteSkipCount?: number;

	/// The number of suite tests with warnings
	suiteWarningCount?: number;

	/// The number of suite tests swith errors
	suiteErrorCount?: number;

	/// The number of suite tests swith errors
	suiteSuccessCount?: number;
}

/** Current state of a tool's deployment */
export enum ToolDeploymentState {
	/** The deployment is ongoing */
	Active = "Active",

	/** The deployment should be paused at its current state */
	Paused = "Paused",

	/** Deployment of this version is complete */
	Complete = "Complete",

	/** The deployment has been cancelled. */
	Cancelled = "Cancelled"
}

/** Summary for a particular tool */
export type GetToolSummaryResponse = {

	/** Unique identifier for the tool */
	id: string;

	/** Name of the tool */
	name: string;

	/** Description of the tool */
	description: string;

	/** Category to display the tool in on the dashboard */
	category?: string;

	/** Grouping key to control how different tools should be merged on the dashboard */
	group?: string;

	/** List of platforms that this tool supports, as NET runtime identifiers */
	platforms?: string;

	/** Version of tool */
	version?: string;

	/** Identifier for the current deployment */
	deploymentId?: string;

	/** State of the current deployment */
	deploymentState?: ToolDeploymentState;

	/** Progress of the current deployment */
	deploymentProgress?: number;

	/** Whether the tool is bundled with the server */
	bundled?: boolean;

	/** Whether to show this tool for download inside UGS */
	showInUGS: boolean;

	/**Whether to show this tool for download on the dashboard */
	showInDashboard: boolean;
}

export type GetToolDeploymentResponse = {
	id: string;
	version: string;
	state: ToolDeploymentState;
	progress: number;
	startedAt: Date | string;
	duration: any;
	refName: string;
	locator: any;
}

export type GetToolResponse = {
	id: string;
	name: string;
	description: string;
	category?: string;
	group?: string;
	platforms?: string[];
	deployments: GetToolDeploymentResponse[];
	public: boolean;
	bundled?: boolean;
	showInUgs: boolean;
	showInDashboard: boolean;
};

/**  Update an existing deployment */
export type UpdateDeploymentRequest = {

	/** New state for the deployment */
	state?: ToolDeploymentState;
}


/** Job Bisect */

/// State of a bisect task
export enum BisectTaskState {
	/// Currently running	
	Running = "Running",

	/// Cancelled by a user	
	Cancelled = "Cancelled",

	/// Finished running. The first job/change identifies the first failure.	
	Succeeded = "Succeeded",

	/// Task failed due to not having a job before the first failure.	
	MissingHistory = "MissingHistory",

	/// Task failed due to the stream no longer existing.	
	MissingStream = "MissingStream",

	/// Task failed due to the first job no longer existing.	
	MissingJob = "MissingJob",

	/// Task failed due to template no longer existing.	
	MissingTemplate = "MissingTemplate"
}


export type CommitTag = {
	text: string;
}

/// Request to create a new bisect task
export type CreateBisectTaskRequest = {

	/// Job containing the node to check	
	jobId: string;

	/// Name of the node to query	
	nodeName: string;

	/// Commit tag to filter possible changes against	
	commitTags?: CommitTag[];

	/// Set of changes to ignore. Can be modified later through UpdateBisectTaskRequest
	ignoreChanges?: number[];

	/// Set of job ids to ignore. Can be modified later through UpdateBisectTaskRequest"
	ignoreJobs?: string[];
}

/// Response from creating a bisect task
export type CreateBisectTaskResponse = {

	/// Identifier for the new bisect task	
	bisectTaskId: string;
}

/// Information about a bisect task
export type GetBisectTaskResponse = {

	/// Identifier for this task
	id: string;

	/// Current task state
	state: BisectTaskState;

	/// User that initiated the search
	owner: GetThinUserInfoResponse;

	/// Stream being searched
	streamId: string;

	/// Template within the stream to execute
	templateId: string;

	/// Name of the step to search for
	nodeName: string;

	/// Outcome to search for
	outcome: JobStepOutcome;

	/// Starting job id for the bisect
	initialJobId: string;

	/// Starting job batch id for the bisect
	initialBatchId: string;

	/// Starting job step id for the bisect
	initialStepId: string;

	/// Starting change for the bisect
	initialChange: number;

	/// First known job id that is broken
	currentJobId: string;

	/// First known job batch id that is broken
	currentBatchId: string;

	/// First known job step id that is broken
	currentStepId: string;

	/// Changelist number of the first broken job id
	currentChange: number;

	/// Next step id of a running bisect
	nextJobId?: string;

	/// Next step id of a running bisect
	nextJobChange?: number;

	/// Lower job id bounds
	minJobId?: string;

	/// Lower step id bounds
	minBatchId?: string;

	/// Lower step id bounds
	minStepId?: string;

	/// Lower change id bounds
	minChange?: number;

	/// The steps that have been run on bisect
	steps?: GetJobStepRefResponse[];

}

/// Updates the state of a bisect task
export type UpdateBisectTaskRequest = {
	/// Cancels the current task	
	cancel?: boolean;

	/// List of change numbers to include in the search. 	
	includeChanges?: number[];

	/// List of change numbers to exclude from the search.	
	encludeChanges?: number[];

	/// List of jobs to include in the search.	
	includeJobs?: string[];

	/// List of jobs to exclude from the search.	
	excludeJobs?: string[];
}

// Accounts

export type GetDashboardChallengeResponse = {
	needsFirstTimeSetup?: boolean;
	needsAuthorization: boolean;
}

export type DashboardLoginRequest = {
	username: string;
	password?: string;
	returnUrl?: string;
}

/// Update request for the current user account
export type UpdateCurrentAccountRequest = {
	oldPassword?: string;
	newPassword?: string;
}

/// Message describing a claim for an account	
export type AccountClaimMessage = {
	type: string;
	value: string;
}

/// Creates a new user account
export type CreateAccountRequest = {

	/// Name of the user
	name: string;

	/// Perforce login identifier
	login: string;

	/// Claims for the user
	claims: AccountClaimMessage[];

	/// Description for the account
	description?: string;

	/// User's email address
	email?: string;

	/// Optional secret token for API access
	secretToken?: string;

	/// Password for the user
	password?: string;

	/// Whether the account is enabled
	enabled?: boolean;
}

/// Response from the request to create a new user account	
export type CreateAccountResponse = { id: string }

/// Update request for a user account	
export type UpdateAccountRequest = {
	name?: string;
	login?: string;
	claims?: AccountClaimMessage[];
	description?: string;
	email?: string;
	secretToken?: string;
	password?: string;
	enabled?: boolean;
};

/// Gets an existing user 
export type GetAccountResponse = {
	id: string;
	name: string;
	login: string;
	claims: AccountClaimMessage[];
	description?: string;
	email?: string;
	enabled?: boolean;
}

// Service Accounts

/// Creates a new user account
export type CreateServiceAccountRequest = {
	description: string;
	claims: AccountClaimMessage[],
	enabled?: boolean;
}

/// Response from the request to create a new user account
export type CreateServiceAccountResponse = {
	id: string;
	secretToken: string;
}

/// Update request for a user account
export type UpdateServiceAccountRequest = {
	description?: string;
	claims?: AccountClaimMessage[];
	resetToken?: boolean;
	enabled?: boolean;
}

/// Response from updating a user account
export type UpdateServiceAccountResponse = {
	newSecretToken?: string
}

/// Creates a new user account
export type GetServiceAccountResponse = {
	id: string;
	claims: AccountClaimMessage[];
	description: string;
	enabled: boolean;
}

// #region -- Build Health Filters --

/// Build Health Filters

/**
 * Datastructure that describes an add request for Build Health Filters.
 */
export type AddBuildHealthFiltersRequest = {
	/// The project id.
	filterProject: string;

	/// The filter name.
	filterName: string;

	/// The filter description.
	filterDescription?: string;

	/// The filter query.
	filterQuery: string;
}

/**
 * Datastructure that describes an update request for Build Health Filters.
 */
export type UpdateBuildHealthFiltersRequest = {
	/// The filter name.
	filterName: string;

	/// The filter description.
	filterDescription?: string;

	/// The filter query.
	filterQuery: string;
}

/** Parameters to request project Build Health Filters */
export type GetBuildHealthFiltersRequest = {
	/// The project id to obtain filters for.
	projectId: string;
}

/** Response object representing a Build Health Filter. */
export type GetBuildHealthFilterResponse = {
	/// The filter id.
	id: string;

	/// The owner of the filter.
	owner: GetThinUserInfoResponse;

	/// The project the filter belongs to.
	filterProject: string;

	/// The name of the filter.
	filterName: string;

	/// The filter description.
	filterDescription?: string;

	/// The query of the filter.
	filterQuery: string;

	/// The last updated time of the filter.
	updateTimeUtc: Date | string;
}

// #endregion -- Build Health Filters --

/** Deep Link Response */
export type DeepLinkResponse = {
	/// The deep link url
	url?: string;
}

// #region -- Storage Dashboard --

/** Consolidated storage dashboard response */
export type GetStorageDashboardResponse = {
	status: StorageDashboardStatus;
	gcMetrics: StorageDashboardGcMetrics;
	collectionStats: StorageDashboardCollectionStats;
}

/** Live GC configuration and namespace status */
export type StorageDashboardStatus = {
	enableGc: boolean;
	enableGcVerification: boolean;
	gcWorkerCount: number;
	gcQueueLimit: number;
	enableReliableGcEnqueue: boolean;
	enablePerNamespaceQueueIsolation: boolean;
	gcBlobIngestionBatchSize: number;
	gcBatchSize: number;
	bundleCacheDir: string;
	bundleCacheSize: string;
	namespaces: StorageNamespaceStatus[];
	backends: StorageBackendInfo[];
}

/** Per-namespace status */
export type StorageNamespaceStatus = {
	id: string;
	backend: string;
	prefix: string;
	gcFrequencyHrs: number;
	gcDelayHrs: number;
	gcQueueDepth: number;
	isPaused: boolean;
	lastGcTime?: string;
	enableAliases: boolean;
}

/** Storage backend configuration */
export type StorageBackendInfo = {
	id: string;
	type: string;
	bucketName?: string;
	bucketPath?: string;
	region?: string;
	containerName?: string;
	gcsBucketName?: string;
	gcsBucketPath?: string;
	baseDir?: string;
	secondaryBackend?: string;
}

/** GC metric entries container */
export type StorageDashboardGcMetrics = {
	entries: StorageGcMetricEntry[];
}

/** Single GC metric entry (per-tick snapshot) */
export type StorageGcMetricEntry = {
	time: string;
	namespaceId: string;
	blobsDeleted: number;
	blobsChecked: number;
	bytesFreed: number;
	blobsIngested: number;
	queueDepth: number;
	sweepDurationMs: number;
	wasPaused: boolean;
	enqueueFailures: number;
	throttleEvents: number;
	refsExpired: number;
}

/** MongoDB collection stats container */
export type StorageDashboardCollectionStats = {
	entries: CollectionStatsSnapshot[];
}

/** A single collection stats snapshot in time */
export type CollectionStatsSnapshot = {
	time: string;
	collections: CollectionStat[];
}

/** Stats for a single MongoDB collection */
export type CollectionStat = {
	name: string;
	documentCount: number;
	dataSizeBytes: number;
	indexSizeBytes: number;
	storageSizeBytes: number;
	avgDocSizeBytes: number;
	fragmentationRatio: number;
}

// #endregion -- Storage Dashboard --