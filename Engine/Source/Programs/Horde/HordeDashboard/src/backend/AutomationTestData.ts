// Copyright Epic Games, Inc. All Rights Reserved.

import { isNumber } from '@datadog/browser-core';
import { action, makeObservable, observable } from 'mobx';
import backend from '../backend';
import { GetTestDataRefResponse, GetTestMetaResponse, GetTestResponse, GetTestStreamResponse, GetTestSuiteResponse } from '../backend/Api';
import { projectStore } from '../backend/ProjectStore';

export type FilterState = "Success" | "Failed" | "Consecutive Failures" | "Skipped";

// some aliases
type StreamId = string;
type TestId = string;
type SuiteId = string;
type MetaId = string;

export type TestStatus = {
    suite: boolean;
    refs: GetTestDataRefResponse[];
    success?: number;
    error?: number;
    skip?: number;
}


const defaultQueryWeeks = 2;
const minuteInWeek = 10080;
const queryRefreshMS = 2500;

// query states
type TestDataState = {
    streams?: string[];
    automation?: string;
    platforms?: string[];
    configurations?: string[];
    targets?: string[];
    rhi?: string[];
    variation?: string[];
    tests?: string[];
    suites?: string[];
    weeks?: number;
    autoExpand?: boolean;
}

export class TestDataHandler {

    constructor(search: URLSearchParams) {
        TestDataHandler.instance = this;
        makeObservable(this);
        this.state = this.stateFromSearch(search);
    }

    setAutomation(automation: string, initStreams: boolean = false) {
        this.state.automation = automation;

        if (automation) {
            const streams = this.getAutomationStreams(automation);
            this.state.streams = this.state.streams?.filter(s => streams.indexOf(s) !== -1);
            if (!this.state.streams?.length) {
                this.state.streams = undefined;
            }

            if (initStreams) {
                const streams = this.getAutomationStreams(automation);
                this.state.streams = streams.map(s => s);
            }

            this.state.tests = undefined;
            this.state.suites = undefined;

            const tests = this.streamTests;
            if (tests?.length) {
                this.state.tests = tests.map(t => t.name);
            }

            const suites = this.streamSuites;
            if (suites?.length) {
                this.state.suites = suites.map(t => t.name);
            }


        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    setAutoExpand(value: boolean) {
        this.state.autoExpand = value;
        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    setQueryWeeks(weeks: number) {
        this.state.weeks = weeks;
        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    setSuiteRef(suiteRef?: string) {
        if (this.suiteRef === suiteRef) {
            return;
        }

        console.log(`Setting suite ref to ${suiteRef}}`)

        this.suiteRef = suiteRef;
        this.setUpdated();
    }

    addPlatform(platform: string) {

        const state = this.state;

        if (state.platforms) {
            if (state.platforms.indexOf(platform) !== -1) {
                return;
            }
        }

        if (!state.platforms) {
            state.platforms = [];
        }

        state.platforms.push(platform);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removePlatform(platform: string) {

        const state = this.state;

        if (state.platforms?.indexOf(platform) === -1) {
            return;
        }

        const idx = state.platforms?.indexOf(platform);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.platforms!.splice(idx, 1);

        if (!state.platforms!.length) {
            state.platforms = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    addConfiguration(config: string) {

        const state = this.state;

        if (state.configurations) {
            if (state.configurations.indexOf(config) !== -1) {
                return;
            }
        }

        if (!state.configurations) {
            state.configurations = [];
        }

        state.configurations.push(config);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeConfiguration(config: string) {

        const state = this.state;

        if (state.configurations?.indexOf(config) === -1) {
            return;
        }

        const idx = state.configurations?.indexOf(config);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.configurations!.splice(idx, 1);

        if (!state.configurations!.length) {
            state.configurations = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    addTarget(target: string) {

        const state = this.state;

        if (state.targets) {
            if (state.targets.indexOf(target) !== -1) {
                return;
            }
        }

        if (!state.targets) {
            state.targets = [];
        }

        state.targets.push(target);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeTarget(target: string) {

        const state = this.state;

        if (state.targets?.indexOf(target) === -1) {
            return;
        }

        const idx = state.targets?.indexOf(target);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.targets!.splice(idx, 1);

        if (!state.targets!.length) {
            state.targets = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    addRHI(rhi: string) {

        const state = this.state;

        if (state.rhi) {
            if (state.rhi.indexOf(rhi) !== -1) {
                return;
            }
        }

        if (!state.rhi) {
            state.rhi = [];
        }

        state.rhi.push(rhi);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeRHI(rhi: string) {

        const state = this.state;

        if (state.rhi?.indexOf(rhi) === -1) {
            return;
        }

        const idx = state.rhi?.indexOf(rhi);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.rhi!.splice(idx, 1);

        if (!state.rhi!.length) {
            state.rhi = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    addVariation(variation: string) {

        const state = this.state;

        if (state.variation) {
            if (state.variation.indexOf(variation) !== -1) {
                return;
            }
        }

        if (!state.variation) {
            state.variation = [];
        }

        state.variation.push(variation);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeVariation(variation: string) {

        const state = this.state;

        if (state.variation?.indexOf(variation) === -1) {
            return;
        }

        const idx = state.variation?.indexOf(variation);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.variation!.splice(idx, 1);

        if (!state.variation!.length) {
            state.variation = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }


    addTest(test: string) {

        const state = this.state;

        if (state.tests) {
            if (state.tests.indexOf(test) !== -1) {
                return;
            }
        }

        if (!state.tests) {
            state.tests = [];
        }

        state.tests.push(test);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeTest(test: string) {

        const state = this.state;

        if (state.tests?.indexOf(test) === -1) {
            return;
        }

        const idx = state.tests?.indexOf(test);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.tests!.splice(idx, 1);

        if (!state.tests!.length) {
            state.tests = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    addStream(streamId: StreamId, queryQuery: boolean = true) {

        const state = this.state;

        if (state.streams) {
            if (state.streams.indexOf(streamId) !== -1) {
                return;
            }
        }

        if (!state.streams) {
            state.streams = [];
        }

        state.streams.push(streamId);

        if (this.updateSearch(queryQuery)) {
            this.setUpdated();
        }
    }

    removeStream(streamId: string) {

        const state = this.state;

        if (state.streams?.indexOf(streamId) === -1) {
            return;
        }

        const idx = state.streams?.indexOf(streamId);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.streams!.splice(idx, 1);

        if (!state.streams!.length) {
            state.streams = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }


    addSuite(suite: string) {

        const state = this.state;

        if (state.suites) {
            if (state.suites.indexOf(suite) !== -1) {
                return;
            }
        }

        if (!state.suites) {
            state.suites = [];
        }

        state.suites.push(suite);

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    removeSuite(suite: string) {

        const state = this.state;

        if (state.suites?.indexOf(suite) === -1) {
            return;
        }

        const idx = state.suites?.indexOf(suite);
        if (!isNumber(idx) || idx < 0) {
            return;
        }

        state.suites!.splice(idx, 1);

        if (!state.suites!.length) {
            state.suites = undefined;
        }

        if (this.updateSearch()) {
            this.setUpdated();
        }
    }

    get currentMeta(): GetTestMetaResponse[] {

        const state = this.state;

        if (!state.targets?.length || !state.platforms?.length || !state.configurations?.length || !state.rhi?.length || !state.variation?.length) {
            return [];
        }

        if (!state.automation || !state.streams?.length) {
            return [];
        }

        const meta = new Map<string, GetTestMetaResponse>();

        state.streams.forEach(streamId => {

            const stream = this.testStreams.get(streamId);

            if (!stream) {
                return;
            }

            stream.testMetadata.forEach(m => {

                if (m.projectName !== state.automation) {
                    return;
                }

                if (state.targets?.length) {
                    if (!m.buildTargets.find(v => !!state.targets?.find(v2 => v === v2))) {
                        return;
                    }
                }

                if (state.platforms?.length) {
                    if (!m.platforms.find(v => !!state.platforms?.find(v2 => v === v2))) {
                        return;
                    }
                }

                if (state.configurations?.length) {
                    if (!m.configurations.find(v => !!state.configurations?.find(v2 => v === v2))) {
                        return;
                    }
                }

                if (state.rhi?.length) {
                    if (!state.rhi.find(r => m.rhi === r)) {
                        return;
                    }
                }

                if (state.variation?.length) {
                    if (!state.variation.find(r => m.variation === r)) {
                        return;
                    }
                }

                meta.set(m.id, m);

            });

        });

        return Array.from(meta.values());
    };

    /// Tests which match current stream
    get streamTests(): GetTestResponse[] {

        const state = this.state;

        if (!state.automation || !state.streams?.length) {
            return [];
        }

        const tests = new Map<string, GetTestResponse>();

        state.streams.forEach(streamId => {

            const stream = this.testStreams.get(streamId);
            if (!stream) {
                return;
            }

            stream.tests.forEach(test => {
                if (!!test.metadata.find(m => this.metaData.get(m)?.projectName === state.automation)) {
                    tests.set(test.id, test);
                }
            });

        });

        return Array.from(tests.values()).sort((a, b) => a.name.localeCompare(b.name));
    }

    /// Tests that match current meta
    get metaTests(): GetTestResponse[] {

        const state = this.state;

        if (!state.automation || !state.streams?.length) {
            return [];
        }

        const meta = this.currentMeta;
        const metaIds = new Set<string>();
        meta.forEach(m => metaIds.add(m.id));

        const tests = new Map<string, GetTestResponse>();
        const streamTests = this.streamTests;

        streamTests.forEach(test => {

            if (test.metadata.find(id => metaIds.has(id))) {
                tests.set(test.id, test);
            }
        });

        return Array.from(tests.values()).sort((a, b) => a.name.localeCompare(b.name));
    }

    /// Suites which match current stream
    get streamSuites(): GetTestSuiteResponse[] {

        const state = this.state;

        if (!state.automation || !state.streams?.length) {
            return [];
        }

        const suites = new Map<string, GetTestSuiteResponse>();

        state.streams.forEach(streamId => {

            const stream = this.testStreams.get(streamId);
            if (!stream) {
                return;
            }

            stream.testSuites.forEach(suite => {
                if (!!suite.metadata.find(m => this.metaData.get(m)?.projectName === state.automation)) {
                    suites.set(suite.id, suite);
                }
            });

        });

        return Array.from(suites.values()).sort((a, b) => a.name.localeCompare(b.name));
    }

    /// Suites that match current meta
    get metaSuites(): GetTestSuiteResponse[] {

        const state = this.state;

        if (!state.automation || !state.streams?.length) {
            return [];
        }

        const meta = this.currentMeta;
        const metaIds = new Set<string>();
        meta.forEach(m => metaIds.add(m.id));

        const suites = new Map<string, GetTestResponse>();
        const streamSuites = this.streamSuites;

        streamSuites.forEach(suite => {
            if (suite.metadata.find(id => metaIds.has(id))) {
                suites.set(suite.id, suite);
            }
        });


        return Array.from(suites.values()).sort((a, b) => a.name.localeCompare(b.name));
    }


    updateSearch(queueQuery: boolean = true): boolean {

        const state = this.state;

        state.weeks = state.weeks ? state.weeks : undefined;

        state.streams = state.streams?.sort((a, b) => a.localeCompare(b));
        state.automation = state.automation?.length ? state.automation : undefined;
        state.platforms = state.platforms?.sort((a, b) => a.localeCompare(b));

        state.configurations = state.configurations?.sort((a, b) => a.localeCompare(b));
        state.targets = state.targets?.sort((a, b) => a.localeCompare(b));
        state.rhi = state.rhi?.sort((a, b) => a.localeCompare(b));
        state.variation = state.variation?.sort((a, b) => a.localeCompare(b));

        state.tests = state.tests?.sort((a, b) => a.localeCompare(b));
        state.suites = state.suites?.sort((a, b) => a.localeCompare(b));

        state.autoExpand = state.autoExpand ? state.autoExpand : undefined;

        const search = new URLSearchParams();

        const csearch = this.search.toString();

        if (state.automation) {
            search.append("automation", state.automation);
        }

        if (state.weeks) {
            search.append("weeks", state.weeks.toString());
        }

        if (state.autoExpand) {
            search.append("autoexpand", "true");
        }

        state.streams?.forEach(s => {
            search.append("stream", s);
        });

        state.platforms?.forEach(p => {
            search.append("platform", p);
        });

        state.configurations?.forEach(c => {
            search.append("configurations", c);
        });

        state.targets?.forEach(t => {
            search.append("targets", t);
        });

        state.rhi?.forEach(r => {
            search.append("rhi", r);
        });

        state.variation?.forEach(v => {
            search.append("var", v);
        });

        state.tests?.forEach(r => {
            search.append("test", r);
        });

        state.suites?.forEach(r => {
            search.append("suite", r);
        });

        if (search.toString() !== csearch) {

            if (this.queryTimeoutId) {
                clearTimeout(this.queryTimeoutId);
            }

            this.queryTimeoutId = undefined;

            if (queueQuery) {

                this.queryTimeoutId = setTimeout(() => {
                    this.queryTimeoutId = undefined;
                    this.query();
                }, queryRefreshMS);
            }

            this.search = search;
            return true;
        }

        return false;
    }

    private stateFromSearch(search: URLSearchParams): TestDataState {

        const state: TestDataState = {};

        const streams = search.getAll("stream") ?? undefined;
        const automation = search.get("automation") ?? undefined;
        const platforms = search.getAll("platform") ?? undefined;
        const configurations = search.getAll("configurations") ?? undefined;
        const targets = search.getAll("targets") ?? undefined;
        const rhi = search.getAll("rhi") ?? undefined;
        const variation = search.getAll("var") ?? undefined;

        const tests = search.getAll("test") ?? undefined;
        const suites = search.getAll("suite") ?? undefined;

        const weeks = search.get("weeks") ?? undefined;

        const autoExpand = search.get("autoexpand") ?? undefined;

        state.streams = streams?.sort((a, b) => a.localeCompare(b));
        state.automation = automation?.length ? automation : undefined;
        state.platforms = platforms?.sort((a, b) => a.localeCompare(b));

        state.configurations = configurations?.sort((a, b) => a.localeCompare(b));
        state.targets = targets?.sort((a, b) => a.localeCompare(b));
        state.rhi = rhi?.sort((a, b) => a.localeCompare(b));
        state.variation = variation?.sort((a, b) => a.localeCompare(b));

        state.tests = tests?.sort((a, b) => a.localeCompare(b));
        state.suites = suites?.sort((a, b) => a.localeCompare(b));

        state.weeks = weeks ? parseInt(weeks) : defaultQueryWeeks;
        if (typeof (state.weeks) !== "number") {
            state.weeks = defaultQueryWeeks;
        }

        state.autoExpand = autoExpand === "true" ? true : undefined;

        return state;
    }

    getMetaString(metaId: string, platform = true, config = true, target = true, rhi = true, variation = true) {

        const meta = this.metaData.get(metaId);
        if (!meta) {
            return "";
        }

        const elements: string[] = [];
        if (platform) {
            elements.push(`${meta.platforms.join(" - ")}`);
        }
        if (config) {
            elements.push(`${meta.configurations.join(" - ")}`);
        }

        if (target) {
            elements.push(`${meta.buildTargets.join(" - ")}`);
        }

        if (rhi) {
            elements.push(meta.rhi === "default" ? "Default" : meta.rhi.toUpperCase());
        }

        if (variation) {
            elements.push(meta.variation === "default" ? "Default" : meta.variation.toUpperCase());
        }

        if (elements.length) {
            return elements.join(" / ");
        }

        return "";

    }

    getCommonMeta(metaIds: string[], config = true, target = true, rhi = true, variation = true) {

        let cconfigs = "";
        let uniqueConfigs = false;

        let ctargets = "";
        let uniqueTargets = false;

        let crhi = "";
        let uniqueRHI = false;

        let cvariation = "";
        let uniqueVariation = false;

        metaIds.forEach(m => {

            const meta = this.metaData.get(m)!;

            if (config) {
                const configs = meta.configurations.join(" - ");
                if (cconfigs && cconfigs !== configs) {
                    uniqueConfigs = true;
                } else {
                    cconfigs = configs;
                }
            }

            if (target) {
                const targets = meta.buildTargets.join(" - ");
                if (ctargets && ctargets !== targets) {
                    uniqueTargets = true;
                } else {
                    ctargets = targets;
                }
            }

            if (rhi) {
                const rhi = meta.rhi === "default" ? "Default" : meta.rhi.toUpperCase()

                if (crhi && crhi !== rhi) {
                    uniqueRHI = true;
                } else {
                    crhi = rhi;
                }
            }

            if (variation) {
                const variation = meta.variation === "default" ? "Default" : meta.variation.toUpperCase()

                if (cvariation && cvariation !== variation) {
                    uniqueVariation = true;
                } else {
                    cvariation = variation;
                }
            }

        });

        const elements: string[] = [];
        if (!uniqueConfigs || !uniqueTargets || !uniqueRHI) {
            if (config && !uniqueConfigs) {
                elements.push(cconfigs);
            }
            if (target && !uniqueTargets) {
                elements.push(ctargets);
            }
            if (rhi && !uniqueRHI) {
                elements.push(crhi);
            }
            if (variation && !uniqueVariation) {
                elements.push(cvariation);
            }
        }

        return {
            commonConfigs: !uniqueConfigs,
            commonTargets: !uniqueTargets,
            commonRHI: !uniqueRHI,
            commonVariation: !uniqueVariation,
            commonMetaString: elements.length ? elements.join(" / ") : ""
        }
    }

    async query() {

        const state = this.state;

        if (!state.automation || !state.streams?.length || (!state.tests?.length && !state.suites?.length)) {
            this.refs = [];
            this.streamStatus = new Map();
            this.setQueryLoading(false);
            this.setQueryUpdated();
            return;
        }

        this.setQueryLoading(true);

        const testIds: Set<string> = new Set();
        const suiteIds: Set<string> = new Set();
        const metaIds: Set<string> = new Set(this.currentMeta.map(m => m.id));

        const streamTestMap: Map<string, GetTestResponse> = new Map();
        this.streamTests.forEach(t => {
            streamTestMap.set(t.id, t);
        });

        const streamSuiteMap: Map<string, GetTestResponse> = new Map();
        this.streamSuites.forEach(s => {
            streamSuiteMap.set(s.id, s);
        });

        state.tests?.forEach(name => {
            streamTestMap.forEach((t, id) => {
                if (t.name === name || t.displayName === name) {
                    testIds.add(id);
                }
            });
        });

        state.suites?.forEach(name => {
            streamSuiteMap.forEach((s, id) => {
                if (s.name === name) {
                    suiteIds.add(id);
                }
            });
        });


        // notify user?
        if (!metaIds.size || (!testIds.size && !suiteIds.size)) {
            this.refs = [];
            this.streamStatus = new Map();
            this.setQueryLoading(false);
            this.setQueryUpdated();
            return;
        }

        let queryWeeks = this.state.weeks;
        if (typeof (queryWeeks) !== "number") {
            queryWeeks = defaultQueryWeeks;
        }

        const minQueryDate = new Date(new Date().valueOf() - (minuteInWeek * queryWeeks * 60000));
        const maxQueryDate = new Date();

        let refs = await backend.getTestRefs(state.streams, Array.from(metaIds), minQueryDate.toISOString(), maxQueryDate.toISOString(), undefined, undefined, testIds.size ? Array.from(testIds) : undefined, suiteIds.size ? Array.from(suiteIds) : undefined);

        // deduplicate and sort intro stream -> test -> meta -> refs descending by CL
        const streamStatus = this.streamStatus;
        streamStatus.clear();
        const dedupedRefs: GetTestDataRefResponse[] = [];
        const dupeWarnings = new Set<string>();

        this.changeDates = new Map();

        refs.forEach(r => {

            if (!this.idDates.get(r.id)) {
                const timestamp = r.id.substring(0, 8)
                this.idDates.set(r.id, new Date(parseInt(timestamp, 16) * 1000));
            }

            const date = this.changeDates.get(r.buildChangeList);
            const rdate = this.idDates.get(r.id)!;
            // Use the latest date for summary, otherwise gets confusing when a CL spans days
            if (!date || date.getTime() < rdate.getTime()) {
                this.changeDates.set(r.buildChangeList, rdate);
            }

            let streamRefs = streamStatus.get(r.streamId);
            if (!streamRefs) {
                streamRefs = new Map();
                streamStatus.set(r.streamId, streamRefs);
            }

            const id = r.testId ?? r.suiteId!;
            let testRefs = streamRefs.get(id);
            if (!testRefs) {
                testRefs = new Map();
                streamRefs.set(id, testRefs);
            }

            let metaStatus = testRefs.get(r.metaId);
            if (!metaStatus) {
                metaStatus = { refs: [], suite: !!r.suiteId };
                testRefs.set(r.metaId, metaStatus);
            }

            let cref = metaStatus.refs.find(tr => r.buildChangeList === tr.buildChangeList);
            if (cref) {
                // todo: better replace logic, use latest, should elevate errors/skips/etc
                dupeWarnings.add(id);
            } else {
                metaStatus.refs.push(r);
                dedupedRefs.push(r);
            }
        });

        this.minRefDate = new Date();
        this.maxRefDate = new Date();

        const timeRefs = refs.sort((a, b) => this.idDates.get(a.id)!.getTime() - this.idDates.get(b.id)!.getTime());
        if (timeRefs.length === 1) {
            this.minRefDate = this.maxRefDate = this.idDates.get(timeRefs[0].id)!;
        } else if (timeRefs.length > 1) {
            this.minRefDate = this.idDates.get(timeRefs[0].id)!;
            this.maxRefDate = this.idDates.get(timeRefs[timeRefs.length - 1].id)!;
        }

        for (const [, testMap] of streamStatus) {
            for (const [, metaMap] of testMap) {
                for (const [, metaStatus] of metaMap) {

                    const suite = metaStatus.suite;
                    metaStatus.refs = metaStatus.refs.sort((a, b) => a.buildChangeList - b.buildChangeList).reverse();

                    // errors
                    for (const ref of metaStatus.refs) {
                        const error = suite ? !!ref.suiteErrorCount : ref.outcome === "Failure";
                        if (!error) {
                            break;
                        }
                        metaStatus.error = metaStatus.error ? metaStatus.error + 1 : 1;
                    }

                    // skip
                    for (const ref of metaStatus.refs) {
                        const skip = suite ? !!ref.suiteSkipCount : ref.outcome === "Skipped";
                        if (!skip) {
                            break;
                        }
                        metaStatus.skip = metaStatus.skip ? metaStatus.skip + 1 : 1;
                    }

                    // success
                    for (const ref of metaStatus.refs) {
                        const success = suite ? (!ref.suiteSkipCount && !ref.suiteErrorCount) : (ref.outcome !== "Failure" && ref.outcome !== "Skipped");
                        if (!success) {
                            break;
                        }
                        metaStatus.success = metaStatus.success ? metaStatus.success + 1 : 1;
                    }
                }
            }
        }

        // generate warnings for non unique tests in stream
        for (let id of dupeWarnings) {
            if (this.testMap.get(id)) {
                console.warn(`Test ${this.testMap.get(id)!.name} has duplicate refs for stream changelists and may require a more specific name`);
            }
            if (this.suiteMap.get(id)) {
                console.warn(`Suite ${this.suiteMap.get(id)!.name} has duplicate refs for stream changelists and may require a more specific name`);
            }
        }

        this.refs = dedupedRefs;

        const changelists = this.refs.map(r => r.buildChangeList);

        this.minCL = this.maxCL = 0;

        this.minCL = Math.min(...changelists);
        this.maxCL = Math.max(...changelists);

        this.hasQueried = true;

        this.setQueryLoading(false);
        this.setQueryUpdated();

        // debug
        /*
        const debugRef = `63e3f1685c0ce8f11b01b7a5`;
        const ref = this.refs.find(r => r.id === debugRef);
        if (ref) {
            this.setSuiteRef(ref.id);
        } 
        */
    }

    // Loads immutable data for this view session
    async load() {

        const allStreams = projectStore.projects.map(p => p.streams).flat().filter(s => !!s).map(s => s!.id);
        //const streams = testingStreams;
        const testStreams = await backend.getTestStreams(allStreams);

        // get unique meta, tests, and suites
        const metaMap: Map<string, GetTestMetaResponse> = new Map();

        const testMap = this.testMap;
        const suiteMap = this.suiteMap;

        testMap.clear();
        suiteMap.clear();

        testStreams.forEach(stream => {

            // it looks like there was a misconfiguration, need a way to remove or age out tests
            stream.tests = stream.tests.filter(t => !t.name.startsWith("EditorAutomation") && !t.name.startsWith("TargetAutomation"));

            this.testStreams.set(stream.streamId, stream);

            const desktop = ["Win64", "Mac", "Linux"];
            stream.testMetadata.forEach(m => {

                m.platforms = m.platforms.sort((a, b) => {
                    if (desktop.indexOf(a) !== -1) {
                        return 1;
                    }
                    if (desktop.indexOf(b) !== -1) {
                        return -1;
                    }

                    return a.localeCompare(b);

                });

                if (!metaMap.has(m.id)) {
                    metaMap.set(m.id, m);
                }
            });

            stream.tests.forEach(t => {
                if (!testMap.has(t.id)) {
                    testMap.set(t.id, t);
                }
            });

            stream.testSuites.forEach(s => {
                if (!suiteMap.has(s.id)) {
                    suiteMap.set(s.id, s);
                }
            });
        });

        const metaData = Array.from(metaMap.values());
        this.tests = Array.from(testMap.values()).map(t => t.displayName ?? t.name).sort((a, b) => a.localeCompare(b));
        this.suites = Array.from(suiteMap.values()).map(s => s.name).sort((a, b) => a.localeCompare(b));

        this.metaData = metaMap;
        metaMap.forEach((meta, id) => {
            if (this.metaNames.has(id)) {
                return;
            }

            const metaName = `${meta.platforms.join(" - ")} / ${meta.configurations.join(" - ")} / ${meta.buildTargets.join(" - ")} / ${meta.rhi} / ${meta.variation}`;

            this.metaNames.set(id, metaName);

        });

        const automation = new Set<string>();
        const platforms = new Set<string>();
        const configurations = new Set<string>();
        const targets = new Set<string>();
        const rhi = new Set<string>();
        const variation = new Set<string>();

        metaData.forEach(m => {
            automation.add(m.projectName);
            m.platforms.forEach(p => platforms.add(p));
            m.buildTargets.forEach(t => targets.add(t));
            m.configurations.forEach(c => configurations.add(c));
            rhi.add(m.rhi);
            variation.add(m.variation)
        });

        this.automation = Array.from(automation).sort((a, b) => a.localeCompare(b));
        this.platforms = Array.from(platforms).sort((a, b) => a.localeCompare(b));
        this.configurations = Array.from(configurations).sort((a, b) => a.localeCompare(b));
        this.targets = Array.from(targets).sort((a, b) => a.localeCompare(b));
        this.rhi = Array.from(rhi).sort((a, b) => a.localeCompare(b));
        this.variation = Array.from(variation).sort((a, b) => a.localeCompare(b));

        this.loaded = true;

        const state = this.state;
        if (!state.platforms?.length && this.platforms.length) {
            this.platforms.forEach(p => this.addPlatform(p));
        }
        if (!state.configurations?.length && this.configurations.length) {
            this.configurations.forEach(c => this.addConfiguration(c));
        }
        if (!state.targets?.length && this.targets.length) {
            this.targets.forEach(t => this.addTarget(t));
        }
        if (!state.rhi?.length && this.rhi.length) {
            this.rhi.forEach(r => this.addRHI(r));
        }
        if (!state.variation?.length && this.variation.length) {
            this.variation.forEach(v => this.addVariation(v));
        }

        this.setUpdated();

        this.query()
    }

    @action
    setUpdated() {
        this.updated++;
    }

    @action
    setQueryUpdated() {
        this.queryUpdated++;
    }

    @action
    setQueryLoading(value: boolean) {
        this.queryLoading = value;
    }


    setFilterState(nstate?: FilterState) {
        if (this.filterState !== nstate) {
            this.filterState = nstate;
            this.setUpdated();
            // need to update the center view
            this.setQueryUpdated();
        }
    }

    getAutomationStreams(automation: string): StreamId[] {

        const streams: Set<StreamId> = new Set();

        this.testStreams.forEach(testStream => {
            if (testStream.testMetadata.find(t => t.projectName === automation)) {
                streams.add(testStream.streamId);
            }

        });

        return Array.from(streams);
    }

    getFilterState(): FilterState | undefined {
        return this.filterState;
    }

    getRef(refId: string) {
        return this.refs.find(r => r.id === refId);
    }

    getSuiteRefs(suiteId: string, metaId: string) {
        return this.refs.filter(r => r.suiteId === suiteId && r.metaId === metaId);
    }

    getFilteredRefs(streamIdIn?: StreamId, idIn?: TestId | SuiteId, metaIdIn?: MetaId): GetTestDataRefResponse[] {

        let status: TestStatus[] = [];

        for (const [streamId, testMap] of this.streamStatus) {
            if (streamIdIn && streamId !== streamIdIn) {
                continue;
            }
            for (const [id, metaMap] of testMap) {
                if (idIn && id !== idIn) {
                    continue;
                }
                for (const [metaId, metaStatus] of metaMap) {
                    if (metaIdIn && metaId !== metaIdIn) {
                        continue;
                    }
                    status.push(metaStatus);
                }
            }
        }

        if (!this.filterState) {
            return status.map(m => m.refs).flat();
        }

        if (this.filterState === "Failed") {
            return status.filter(s => !!s.error).map(m => m.refs).flat();
        }

        if (this.filterState === "Consecutive Failures") {
            return status.filter(s => s.error && s.error > 1).map(m => m.refs).flat();
        }

        if (this.filterState === "Skipped") {
            return status.filter(s => !!s.skip).map(m => m.refs).flat();
        }

        if (this.filterState === "Success") {
            return status.filter(s => !!s.success).map(m => m.refs).flat();
        }

        return [];

    }

    getStatusStreams(): StreamId[] {
        return Array.from(this.streamStatus.keys());
    }

    getStatusStream(streamId: string): Map<TestId | SuiteId, Map<MetaId, TestStatus>> | undefined {
        return this.streamStatus.get(streamId);
    }

    getStatusTests(streamIdIn?: string): GetTestResponse[] {

        const testIds = new Set<TestId>();

        for (let [streamId, streamMap] of this.streamStatus) {
            if (streamIdIn && streamIdIn !== streamId) {
                continue;
            }

            for (let id of streamMap.keys()) {
                const test = this.testMap.get(id);
                if (test) {
                    testIds.add(test.id);
                }
            }
        }

        return Array.from(testIds).map(id => this.testMap.get(id)!).sort((a, b) => (a.displayName ?? a.name).localeCompare((b.displayName ?? b.name)));
    }

    getStatusSuites(streamIdIn?: string): GetTestSuiteResponse[] {

        const suiteIds = new Set<TestId>();

        for (let [streamId, streamMap] of this.streamStatus) {
            if (streamIdIn && streamIdIn !== streamId) {
                continue;
            }

            for (let id of streamMap.keys()) {
                const suite = this.suiteMap.get(id);
                if (suite) {
                    suiteIds.add(suite.id);
                }
            }
        }

        return Array.from(suiteIds).map(id => this.suiteMap.get(id)!).sort((a, b) => a.name.localeCompare(b.name));
    }

    getStatus(id: TestId | SuiteId): Map<StreamId, Map<MetaId, TestStatus>> | undefined {

        if (!this.testMap.get(id) && !this.suiteMap.get(id)) {
            return undefined;
        }

        const streamMap = new Map<StreamId, Map<MetaId, TestStatus>>();

        for (let [streamId, testMap] of this.streamStatus) {
            const metaMap = testMap.get(id);
            if (metaMap && metaMap.size) {
                streamMap.set(streamId, metaMap);
            }
        }

        if (streamMap.size) {
            return streamMap;
        }

        return undefined;
    }


    @observable
    updated: number = 0;

    @observable
    queryUpdated: number = 0;

    @observable
    queryLoading: boolean = false;

    state: TestDataState = {};
    search: URLSearchParams = new URLSearchParams();

    testStreams: Map<string, GetTestStreamResponse> = new Map();

    metaData: Map<MetaId, GetTestMetaResponse> = new Map();
    metaNames: Map<MetaId, string> = new Map();

    testMap: Map<TestId, GetTestResponse> = new Map();
    suiteMap: Map<SuiteId, GetTestSuiteResponse> = new Map();

    automation: string[] = [];
    platforms: string[] = [];
    configurations: string[] = [];
    targets: string[] = [];
    rhi: string[] = [];
    variation: string[] = [];

    tests: string[] = [];
    suites: string[] = [];

    private refs: GetTestDataRefResponse[] = [];
    minCL: number = 0;
    maxCL: number = 0;
    minRefDate: Date = new Date();
    maxRefDate: Date = new Date();
    idDates = new Map<string, Date>();
    // quantized so common CL -> lands at same time in views
    changeDates = new Map<number, Date>();
    
    private filterState?: FilterState;
    private streamStatus: Map<StreamId, Map<TestId | SuiteId, Map<MetaId, TestStatus>>> = new Map();

    loaded = false;

    hasQueried = false;

    suiteRef?: string;

    static instance: TestDataHandler;

    private queryTimeoutId: any = undefined;
}