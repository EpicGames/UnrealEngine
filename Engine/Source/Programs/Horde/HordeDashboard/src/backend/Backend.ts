// Copyright Epic Games, Inc. All Rights Reserved.

import templateCache from '../backend/TemplateCache';
import { AccountClaimMessage, AddBuildHealthFiltersRequest, AgentData, AgentQuery, ApproveAgentsRequest, AuditLogEntry, AuditLogQuery, BatchUpdatePoolRequest, ChangeSummaryData, CreateAccountRequest, CreateBisectTaskRequest, CreateBisectTaskResponse, CreateDeviceRequest, CreateDeviceResponse, CreateExternalIssueRequest, CreateExternalIssueResponse, CreateJobRequest, CreateJobResponse, CreateNoticeRequest, CreatePoolRequest, CreateServiceAccountRequest, CreateServiceAccountResponse, CreateSoftwareResponse, CreateSubscriptionRequest, CreateSubscriptionResponse, CreateZipRequest, DashboardPreference, DeepLinkResponse, DevicePoolTelemetryQuery, DeviceTelemetryQuery, EventData, FindArtifactsResponse, FindIssueResponse, FindJobTimingsResponse, GetAccountResponse, GetAgentLeaseResponse, GetAgentTelemetrySampleResponse, GetArtifactDirectoryResponse, GetArtifactResponse, GetBatchLogEventsRequest, GetBatchLogEventsResponse, GetBisectTaskResponse, GetBuildHealthFilterResponse, GetBuildHealthFiltersRequest, GetDashboardConfigResponse, GetDevicePlatformResponse, GetDevicePoolResponse, GetDevicePoolTelemetryResponse, GetDeviceReservationResponse, GetDeviceResponse, GetDeviceTelemetryResponse, GetExternalIssueProjectResponse, GetExternalIssueResponse, GetIssueStreamResponse, GetJobsTabResponse, GetJobStepRefResponse, GetJobStepTraceResponse, GetJobTimingResponse, GetLabelDetailsResponse, GetLogEventResponse, GetLogFileResponse, GetNoticeResponse, GetNotificationResponse, GetPendingAgentsResponse, GetPerforceServerStatusResponse, GetPoolResponse, GetPoolSummaryResponse, GetServerInfoResponse, GetServerSettingsResponse, GetServiceAccountResponse, GetStorageDashboardResponse, GetSoftwareResponse, GetSubscriptionResponse, GetTestDataDetailsResponse, GetTestDataRefResponse, GetTestMetaResponse, GetTestResponse, GetTestsRequest, GetTestStreamResponse, GetToolResponse, GetToolSummaryResponse, GetUserResponse, GetUtilizationTelemetryResponse, GlobalConfig, IssueData, IssueQuery, IssueQueryV2, JobData, JobQuery, JobsTabColumnType, JobStepOutcome, JobStreamQuery, JobTimingsQuery, LeaseData, LogData, LogEventQuery, LogLineData, PoolData, PoolQuery, PreflightConfigResponse, ProjectData, ScheduleData, ScheduleQuery, SearchLogFileResponse, ServerPluginInfoResponse, ServerStatusResponse, ServerUpdateResponse, SessionData, StreamData, TabType, TestData, UpdateAccountRequest, UpdateAgentRequest, UpdateBisectTaskRequest, UpdateBuildHealthFiltersRequest, UpdateCurrentAccountRequest, UpdateDeploymentRequest, UpdateDeviceRequest, UpdateGlobalConfigRequest, UpdateIssueRequest, UpdateJobRequest, UpdateLeaseRequest, UpdateNoticeRequest, UpdateNotificationsRequest, UpdatePoolRequest, UpdateServerSettingsRequest, UpdateServiceAccountRequest, UpdateServiceAccountResponse, UpdateStepRequest, UpdateStepResponse, UpdateTemplateRefRequest, UpdateUserRequest, UsersQuery } from './Api';
import dashboard, { Dashboard } from './Dashboard';
import { ChallengeStatus, Fetch } from './Fetch';
import { projectStore } from './ProjectStore';


// Update interval for relatively static global data such as projects, schedules, templates
const updateInterval = 900 * 1000;

export class Backend {


    getDashboardConfig(): Promise<GetDashboardConfigResponse> {
        return new Promise<GetDashboardConfigResponse>((resolve, reject) => {
            this._fetch.get("/api/v1/dashboard/config").then((response) => {
                resolve(response.data as GetDashboardConfigResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getProjects(): Promise<ProjectData[]> {

        return new Promise<ProjectData[]>((resolve, reject) => {

            this._fetch.get("/api/v1/projects", { params: { categories: true } }).then(async (response) => {

                const projects = response.data as ProjectData[];

                await this._fetch.get("/api/v1/streams").then(response => {

                    const streams = (response.data as StreamData[]);

                    projects.forEach(p => {
                        p.streams = streams.filter(stream => stream.projectId === p.id);
                        p.streams!.forEach(s => {

                            // create a default columbn if none exist
                            s.tabs?.forEach(tab => {
                                if (tab.type === TabType.Jobs) {
                                    const jobTab = tab as GetJobsTabResponse;
                                    if (!jobTab.columns) {
                                        jobTab.columns = [{
                                            type: JobsTabColumnType.Labels,
                                            heading: "Other",
                                            relativeWidth: 1
                                        }]
                                    }
                                }
                            });

                            s.fullname = s.name;
                            s.name = s.name.substr(s.name.lastIndexOf("/") + 1);
                            s.project = p;
                        })
                    });
                });

                resolve(projects);

            }).catch(reason => { reject(reason); });

        });

    }

    getProject(project: ProjectData): Promise<ProjectData> {

        return new Promise<ProjectData>((resolve, reject) => {

            this._fetch.get(`/api/v1/projects/${project.id}`).then((response) => {

                const project = response.data as ProjectData;
                resolve(project);

            }).catch(reason => { reject(reason); });

        });

    }

    updateTemplateRef(streamId: string, templateRefId: string, request: UpdateTemplateRefRequest): Promise<boolean> {

        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`/api/v1/streams/${streamId}/templates/${templateRefId}`, request).then((value) => {
                resolve(true);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getPool(poolId: string): Promise<GetPoolResponse> {
        return new Promise<GetPoolResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/pools/${poolId}`).then((response) => {
                resolve(response.data as GetPoolResponse);
            }).catch(reason => { reject(reason); });
        });
    }

    getPools(filter?: string): Promise<PoolData[]> {

        const params: any = {
            filter: filter
        };

        return new Promise<PoolData[]>((resolve, reject) => {
            this._fetch.get("/api/v1/pools", { params: params }).then((response) => {
                const pools = response.data as PoolData[];
                resolve(pools);
            }).catch(reason => { reject(reason); });
        });
    }

    getPoolsV2(query: PoolQuery): Promise<GetPoolSummaryResponse[]> {

        return new Promise<GetPoolSummaryResponse[]>((resolve, reject) => {
            this._fetch.get("/api/v2/pools", { params: query }).then((response) => {
                const pools = response.data as GetPoolSummaryResponse[];
                resolve(pools);
            }).catch(reason => { reject(reason); });
        });
    }

    getAgentRegistrationRequests(): Promise<GetPendingAgentsResponse> {
        return new Promise<GetPendingAgentsResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/enrollment`).then((response) => {
                const agent = response.data as GetPendingAgentsResponse;
                resolve(agent);
            }).catch(reason => { reject(reason); });
        });
    }

    // create a new account
    registerAgents(request: ApproveAgentsRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.post(`/api/v1/enrollment`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }


    getAgent(id: string): Promise<AgentData> {
        return new Promise<AgentData>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${id}`).then((response) => {
                const agent = response.data as AgentData;
                resolve(agent);
            }).catch(reason => { reject(reason); });
        });
    }

    getAgentHistory(id: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
        return new Promise<AuditLogEntry[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${encodeURIComponent(id)}/history`, { params: query }).then((response) => {
                const history = (response.data?.entries ?? []) as AuditLogEntry[];
                resolve(history);
            }).catch(reason => { reject(reason); });
        });
    }

    getAgents(query: AgentQuery): Promise<AgentData[]> {
        return new Promise<AgentData[]>((resolve, reject) => {
            // for whenever we bring back modifiedDate, swap to this line
            this._fetch.get(`/api/v1/agents`, { params: query }).then((response) => {
                //this.backend.get(`/api/v1/agents`).then((response) => {
                const agents = response.data as AgentData[];
                resolve(agents);
            }).catch(reason => { reject(reason); });
        });
    }

    async getLeaseLog(leaseId: string): Promise<GetLogFileResponse> {
        return new Promise<GetLogFileResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/leases/${leaseId}/log`).then((response) => {
                resolve(response.data as GetLogFileResponse);
            }).catch(reason => { reject(reason); });
        });
    }

    getLease(leaseId: string, agentId?: string): Promise<LeaseData> {

        if (agentId) {
            return new Promise<LeaseData>((resolve, reject) => {
                this._fetch.get(`/api/v1/agents/${agentId}/leases/${leaseId}`).then((response) => {
                    resolve(response.data as LeaseData);
                }).catch(reason => { reject(reason); });
            });
        }

        return new Promise<LeaseData>((resolve, reject) => {
            this._fetch.get(`/api/v1/leases/${leaseId}`).then((response) => {
                resolve(response.data as LeaseData);
            }).catch(reason => { reject(reason); });
        });

    }

    getLeases(agentId: string, index: number, count: number, includeBatches?: boolean): Promise<LeaseData[]> {
        const params: string[] = [];
        params.push("Index=" + index);
        params.push("Count=" + count);
        const paramString = params.join("&");
        return new Promise<LeaseData[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${agentId}/leases?${paramString}`).then(async (response) => {
                const leases = response.data as LeaseData[];
                leases.forEach(function (lease) {
                    lease.startTime = new Date(Date.parse(lease.startTime as string));
                    if (lease.finishTime) {
                        lease.finishTime = new Date(Date.parse(lease.finishTime as string));
                    }
                    if (lease.details) {
                        lease.batchId = lease.details.batchId;
                        lease.jobId = lease.details.jobId;
                        lease.type = lease.details.type;
                    }
                });

                // get batch states
                if (includeBatches) {

                    const jobIds: Set<string> = new Set();
                    leases.filter(lease => lease.jobId && lease.batchId).forEach(lease => jobIds.add(lease.jobId));
                    if (jobIds.size) {
                        const jobs = await this.getJobsByIds(Array.from(jobIds), { filter: "id,batches" });

                        leases.forEach(lease => {
                            const job = jobs.find(j => lease.jobId === j.id);
                            if (job) {
                                lease.batch = job.batches?.find(b => lease.batchId === b.id);
                            }
                        });
                    }
                }

                resolve(leases);
            }).catch(reason => { reject(reason); });
        });
    }

    getSessions(agentId: string, index: number, count: number): Promise<SessionData[]> {
        const params: string[] = [];
        params.push("Index=" + index);
        params.push("Count=" + count);
        const paramString = params.join("&");
        return new Promise<SessionData[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${agentId}/sessions?${paramString}`).then((response) => {
                const sessions = response.data as SessionData[];
                sessions.forEach(function (session) {
                    session.startTime = new Date(Date.parse(session.startTime as string));
                    if (session.finishTime) {
                        session.finishTime = new Date(Date.parse(session.finishTime as string));
                    }
                });
                resolve(sessions);
            }).catch(reason => { reject(reason); });
        });
    }

    getAgentLeases(agentId: string, minTime: Date, maxTime: Date): Promise<GetAgentLeaseResponse[]> {
        const params: string[] = [];
        params.push("startTime=" + minTime.toISOString());
        params.push("finishTime=" + maxTime.toISOString());
        const paramString = params.join("&");
        return new Promise<GetAgentLeaseResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${agentId}/leases?${paramString}`).then((response) => {
                (response.data as GetAgentLeaseResponse[]).forEach(s => {
                    s.startTime = s.startTime ? new Date(s.startTime) : new Date();
                    s.finishTime = s.finishTime ? new Date(s.finishTime) : undefined;
                })
                resolve(response.data as GetAgentLeaseResponse[]);
            }).catch(reason => { reject(reason); });
        });
    }


    getAgentTelemetry(agentId: string, minTime: Date, maxTime: Date): Promise<GetAgentTelemetrySampleResponse[]> {
        const params: string[] = [];
        params.push("minTime=" + minTime.toISOString());
        params.push("maxTime=" + maxTime.toISOString());
        const paramString = params.join("&");
        return new Promise<GetAgentTelemetrySampleResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/agents/${agentId}/telemetry?${paramString}`).then((response) => {
                const data = response.data.samples as GetAgentTelemetrySampleResponse[];
                data.forEach((d) => {
                    d.time = new Date(d.time);
                });
                resolve(data);
            }).catch(reason => { reject(reason); });
        });
    }

    retryJobStep(jobId: string, batchId: string, stepId: string): Promise<any> {

        return new Promise<any>((resolve, reject) => {
            this._fetch.put(`/api/v1/jobs/${jobId}/batches/${batchId}/steps/${stepId}`, {
                Retry: true
            }).then((value) => {
                resolve(value.data as JobData[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getJob(id: string, query?: JobQuery, show404Error = false): Promise<JobData> {

        return new Promise<JobData>((resolve, reject) => {

            this._fetch.get(`/api/v1/jobs/${id}`, {
                params: query,
                show404Error: show404Error
            }).then((value) => {

                const response = value.data as JobData;
                resolve(response);

            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getJobsByIds(id: string[], query?: JobQuery): Promise<JobData[]> {

        if (!query) {
            (query as any) = {
                id: id
            }
        } else {
            (query as any).id = id;
        }

        return new Promise<JobData[]>((resolve, reject) => {

            this._fetch.get(`/api/v1/jobs`, {
                params: query
            }).then((value) => {

                const response = value.data as JobData[];

                resolve(response);

            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getStreamJobs(streamId: string, query: JobStreamQuery): Promise<JobData[]> {

        if (typeof query.index === 'number') {
            query.index = 0;
        }

        if (typeof query.count !== 'number') {
            query.count = 100;
        }

        return new Promise<JobData[]>((resolve, reject) => {

            this._fetch.get(`/api/v1/jobs/streams/${streamId}`, {
                params: query
            }).then((value) => {
                const jobs = value.data as JobData[];
                resolve(jobs);
                return;
            }).catch((reason) => {
                reject(reason);
            });
        });

    }


    getJobs(query: JobQuery): Promise<JobData[]> {

        if (typeof query.index === 'number') {
            query.index = 0;
        }

        if (typeof query.count !== 'number') {
            query.count = 100;
        }

        return new Promise<JobData[]>((resolve, reject) => {

            this._fetch.get("/api/v1/jobs", {
                params: query
            }).then((value) => {
                const jobs = value.data as JobData[];

                resolve(jobs);
                return;

            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    getJobTiming(jobId: string): Promise<GetJobTimingResponse> {

        return new Promise<GetJobTimingResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/jobs/${jobId}/timing`).then((value) => {
                resolve(value.data as GetJobTimingResponse);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getLabelDetails(jobId: string, labelIndex: number, config?: { signal?: AbortSignal }): Promise<GetLabelDetailsResponse> {
        return new Promise<GetLabelDetailsResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/jobs/${jobId}/labels/${labelIndex}/details`, { signal: config?.signal }).then((response) => {
                resolve(response.data as GetLabelDetailsResponse);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getBatchJobTiming(query: JobTimingsQuery): Promise<FindJobTimingsResponse> {

        return new Promise<FindJobTimingsResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/jobs/timing`, {
                params: query
            }).then((response) => {
                let timingsWrapper = response.data as FindJobTimingsResponse;
                resolve(timingsWrapper);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getLogEvents(logId: string, query?: LogEventQuery): Promise<EventData[]> {

        return new Promise<EventData[]>((resolve, reject) => {

            this._fetch.get(`/api/v1/logs/${logId}/events`, { params: query }).then((value) => {
                resolve(value.data as EventData[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getBatchLogEvents(request: GetBatchLogEventsRequest): Promise<GetBatchLogEventsResponse> {
        return new Promise<GetBatchLogEventsResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/logs/events`, request).then((value) => {
                resolve(value.data as GetBatchLogEventsResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    searchLog(logId: string, text: string, firstLine?: number, count?: number) {

        return new Promise<SearchLogFileResponse>((resolve, reject) => {

            const params: any = {
                text: text,
                firstLine: firstLine,
                count: count
            };

            this._fetch.get(`/api/v1/logs/${logId}/search`, {
                params: params
            }).then((value) => {
                resolve(value.data as SearchLogFileResponse);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getJobStepHistory(streamId: string, stepName: string, count: number = 10, templateId: string) {
        return new Promise<GetJobStepRefResponse[]>((resolve, reject) => {

            const params: any = {
                step: stepName,
                count: count,
                templateId: templateId
            };

            this._fetch.get(`/api/v1/streams/${streamId}/history`, {
                params: params
            }).then((value) => {
                let results = (value.data ?? []) as GetJobStepRefResponse[];
                results = results.filter(r => r.outcome !== JobStepOutcome.Unspecified);
                resolve(results);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getJobStepHistoryBatch(streamId: string, stepNames: string[], count: number = 10, templateId: string) {
        return new Promise<GetJobStepRefResponse[]>((resolve, reject) => {

            const params: any = {
                steps: stepNames,
                count: count,
                templateId: templateId
            };

            this._fetch.post(`/api/v1/streams/${streamId}/history`, {}, {
                params: params
            }).then((value) => {
                let results = (value.data ?? []) as GetJobStepRefResponse[];
                results = results.filter(r => r.outcome !== JobStepOutcome.Unspecified);
                resolve(results);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getJobStepTrace(jobId: string, batchId: string, stepId: string) {
        return new Promise<GetJobStepTraceResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/jobs/${jobId}/batches/${batchId}/steps/${stepId}/trace`, {
                suppress404: true
            }).then((value) => {
                resolve(value.data as GetJobStepTraceResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getArtifacts(streamId?: string, minChange?: number, maxChange?: number, name?: string, type?: string): Promise<FindArtifactsResponse> {

        return new Promise<FindArtifactsResponse>((resolve, reject) => {
            this._fetch.get(`/api/v2/artifacts`, { params: { streamId: streamId, minChange: minChange, maxChange: maxChange, name: name, type: type, maxResults: 512 } })
                .then(response => { resolve(response.data); })
                .catch(reason => reject(reason));
        })
    }


    getJobArtifactsV2(ids?: string[], keys?: string[]): Promise<FindArtifactsResponse> {

        const uniqueIds = Array.from(new Set(ids ?? []));
        const uniqueKeys = Array.from(new Set(keys ?? []));

        if (!uniqueIds.length && !uniqueKeys.length) {
            throw new Error(`Must provide at least 1 id or key`);
        }

        return new Promise<FindArtifactsResponse>((resolve, reject) => {

            this._fetch.get(`/api/v2/artifacts`, { params: { id: uniqueIds, key: uniqueKeys } })
                .then(response => { resolve(response.data); })
                .catch(reason => reject(reason));
        })
    }


    getBrowseArtifacts(id: string, path?: string): Promise<GetArtifactDirectoryResponse> {

        return new Promise<GetArtifactDirectoryResponse>((resolve, reject) => {

            this._fetch.get(`/api/v2/artifacts/${id}/browse`, { params: { path: path } })
                .then(response => { resolve(response.data); })
                .catch(reason => reject(reason));
        })
    }

    getArtifactData(artifactId: string): Promise<GetArtifactResponse> {
        const url = `/api/v2/artifacts/${artifactId}`;
        return new Promise<GetArtifactResponse>((resolve, reject) => {
            this._fetch.get(url).then((value) => {
                resolve(value.data as GetArtifactResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getArtifactV2(artifactId: string, path: string): Promise<object> {
        const url = `/api/v2/artifacts/${artifactId}/file?path=${encodeURIComponent(path)}`;
        return new Promise<object>((resolve, reject) => {
            this._fetch.get(url).then((value) => {
                resolve(value.data as object);
            }).catch(reason => {
                resolve([]);
                if (reason !== "Not Found") {
                    console.error(reason);
                }
            });
        });

    }

    downloadArtifactV2(artifactId: string, path: string) {
        const url = `/api/v2/artifacts/${artifactId}/file?path=${encodeURIComponent(path)}`;
        window.location.assign(url);
    }

    downloadArtifactZipV2(artifactId: string, request?: CreateZipRequest) {
        const url = `/api/v2/artifacts/${artifactId}/zip`;

        const filter = request?.filter?.map(f => `filter=${encodeURIComponent(f)}`).join("&") ?? '';

        if (filter.length < 2000) {
            // 2'048 characters is the usual limit enforced by browsers for the whole url
            window.location.assign(`${url}?${filter}`);
            return;
        }

        // the disadvantage of the POST method using fetch is the lack of progression indicator for large download size
        // this method wait for the whole data to be downloaded before passing the data blob to the new window
        return new Promise<object>(() => {
            this._fetch.post(url, request, { responseBlob: true }).then((response) => {
                // Create a temp link and activate it to save the incoming data
                const blobURL = URL.createObjectURL(response.data);
                const link = document.createElement('a');
                link.href = blobURL;
                document.body.appendChild(link);
                link.click();
                URL.revokeObjectURL(blobURL);
                document.body.removeChild(link);
            }).catch(reason => {
                if (reason !== "Not Found") {
                    console.error(reason);
                }
            });
        });
    }

    getLogData(logId: string): Promise<LogData> {

        return new Promise<LogData>((resolve, reject) => {

            this._fetch.get(`/api/v1/logs/${logId}`).then((value) => {
                resolve(value.data as LogData);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getLogLines(logId: string, index?: number, count?: number): Promise<LogLineData> {

        const params: any = {};

        if (Number.isInteger(index)) {
            params.index = index;
        }

        if (Number.isInteger(count)) {
            params.Count = count;
        }

        return new Promise<LogLineData>((resolve, reject) => {

            this._fetch.get(`/api/v1/logs/${logId}/lines`, {
                params: params
            }).then((value) => {
                resolve(value.data as LogLineData);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    testException(): Promise<void> {

        return new Promise<void>((resolve, reject) => {

            this._fetch.get("/api/v1/debug/exception").then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }


    createJob(request: CreateJobRequest): Promise<CreateJobResponse> {

        return new Promise<CreateJobResponse>((resolve, reject) => {

            this._fetch.post("/api/v1/jobs", request).then((value) => {
                resolve(value.data as CreateJobResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    updateJob(jobId: string, request: UpdateJobRequest): Promise<boolean> {

        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`/api/v1/jobs/${jobId}`, request).then((value) => {
                resolve(true);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    updateAgent(agentId: string, request: UpdateAgentRequest): Promise<boolean> {
        if (request.enabled === false) {
            let comment = request.comment ?? "";
            const userName = dashboard.hordeUser ?? dashboard.user?.name;
            if (userName) {
                request.comment = comment + ` (${userName})`;
            }
        }
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`api/v1/agents/${agentId}`, request).then((value) => {
                resolve(value.data as boolean);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    deleteAgent(agentId: string): Promise<any> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.delete(`api/v1/agents/${agentId}`).then((value) => {
                resolve(value.data);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    createPool(request: CreatePoolRequest): Promise<string> {
        return new Promise<string>((resolve, reject) => {
            this._fetch.post(`api/v1/pools`, request).then((value) => {
                resolve(value.data as string);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    updatePool(poolId: string, request: UpdatePoolRequest): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`api/v1/pools/${poolId}`, request).then((value) => {
                resolve(value.data as boolean);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    deletePool(poolId: string): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.delete(`api/v1/pools/${poolId}`).then((value) => {
                resolve(value.data as boolean);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    batchUpdatePools(request: BatchUpdatePoolRequest[]): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`api/v1/pools`, request).then((value) => {
                resolve(value.data as boolean);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    uploadSoftware(formData: FormData): Promise<CreateSoftwareResponse> {

        return new Promise<CreateSoftwareResponse>((resolve, reject) => {

            this._fetch.post("/api/v1/software", formData, { formData: true }).then((value) => {
                resolve(value.data as CreateSoftwareResponse);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    updateSoftware(id: string, isDefault: boolean): Promise<any> {

        return new Promise<any>((resolve, reject) => {
            this._fetch.put(`/api/v1/software/${id}`, {
                Default: isDefault
            }).then((value) => {
                resolve(value.data as any);
            }).catch(reason => {
                reject(reason);
            });
        });

    }


    getSchedule(id: string): Promise<ScheduleData> {

        return new Promise<ScheduleData>((resolve, reject) => {

            this._fetch.get(`/api/v1/schedules/${id}`).then((value) => {
                const data = value.data as ScheduleData;
                data.nextTriggerTimesUTC.forEach((d, index) => {
                    data.nextTriggerTimesUTC[index] = new Date(d as any as string);
                });
                resolve(value.data as ScheduleData);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getSchedules(query: ScheduleQuery): Promise<ScheduleData[]> {

        return new Promise<ScheduleData[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/schedules`, {
                params: query
            }).then((value) => {
                resolve(value.data as ScheduleData[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }


    getSoftware(): Promise<GetSoftwareResponse[]> {

        return new Promise<GetSoftwareResponse[]>((resolve, reject) => {

            this._fetch.get(`/api/v1/software`).then((value) => {
                resolve(value.data as GetSoftwareResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    deleteSoftware(id: string): Promise<any> {

        return new Promise<any>((resolve, reject) => {

            this._fetch.delete(`/api/v1/software/${id}`).then((value) => {
                resolve(value.data);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getCommit(streamId?: string, changelist?: number, filter?: string): Promise<ChangeSummaryData | undefined> {

        return new Promise<ChangeSummaryData | undefined>((resolve) => {
            const params: any = {};

            if (!streamId || !changelist) {
                resolve(undefined);
                return;
            }

            if (filter) {
                params.filter = filter
            }

            this._fetch.get(`/api/v1/streams/${streamId}/changes/${changelist}`, {
                params: params
            }).then((value) => {
                resolve(value.data as ChangeSummaryData);
            }).catch(reason => {
                console.error(reason);
                resolve(undefined);
            });
        });
    }

    getChangeSummaries(streamId: string, minChange?: number, maxChange?: number, changeNumbers?: number[], filter?: string, maxResults = 100): Promise<ChangeSummaryData[]> {

        return new Promise<ChangeSummaryData[]>((resolve, reject) => {

            const params: any = {
                results: maxResults
            };

            if (minChange) {
                params.min = minChange;
            }

            if (maxChange) {
                params.max = maxChange;
            }

            if (filter) {
                params.filter = filter;
            }

            if (changeNumbers) {
                params.changeNumbers = changeNumbers;
            }

            this._fetch.get(`/api/v1/streams/${streamId}/changes`, {
                params: params
            }).then((value) => {
                resolve(value.data as ChangeSummaryData[]);
            }).catch(reason => {
                reject(reason);
            });

        });

    }

    getBatchChangeSummaries(streamId: string, changeNumbers?: number[], filter?: string): Promise<ChangeSummaryData[]> {
        return new Promise<ChangeSummaryData[]>((resolve, reject) => {
            const body = changeNumbers ?? [];
            const params: any = {
            };

            if (filter) {
                params.filter = filter;
            }

            this._fetch.post(`/api/v1/streams/${streamId}/changes`, body, { params }
            ).then((value) => {
                resolve(value.data as ChangeSummaryData[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getAdminToken(): Promise<string> {

        return new Promise<string>((resolve, reject) => {

            this._fetch.get("/api/v1/admin/token").then((response) => {
                resolve(response.data as string);
            }).catch(reason => reject(reason));

        });

    }

    updateUser(request: UpdateUserRequest): Promise<void> {

        return new Promise<void>((resolve, reject) => {
            this._fetch.put("/api/v1/user", request).then((response) => {
                resolve();
            }).catch(reason => reject(reason));

        });

    }

    getUsers(query?: UsersQuery): Promise<GetUserResponse[]> {

        query = query ?? {};

        return new Promise<GetUserResponse[]>((resolve, reject) => {
            this._fetch.get("/api/v1/users", {
                params: query as any
            }).then((response) => {
                resolve(response.data as GetUserResponse[]);
            }).catch(reason => reject(reason));
        });

    }

    getCurrentUser(): Promise<GetUserResponse> {

        return new Promise<GetUserResponse>((resolve, reject) => {

            this._fetch.get("/api/v1/user", { suppress404: true }).then((response) => {

                let data = response.data as GetUserResponse;

                // convert from an object to a map
                const prefs = Object.values(DashboardPreference);
                const preferences = new Map<DashboardPreference, string>();

                if (!data.dashboardSettings?.preferences) {
                    if (!data.dashboardSettings) {
                        data.dashboardSettings = { preferences: preferences };
                    } else {
                        data.dashboardSettings.preferences = preferences;
                    }

                } else {
                    for (let key of Object.keys(data.dashboardSettings.preferences)) {

                        if (prefs.indexOf(key as DashboardPreference) === -1) {
                            continue;
                        }

                        preferences.set(key as DashboardPreference, (data.dashboardSettings.preferences as any)[key])
                    }

                    data.dashboardSettings.preferences = preferences;
                }

                // apply defaults
                const current = data.dashboardSettings.preferences.get(DashboardPreference.Darktheme);
                if (current !== "true" && current !== "false") {
                    const value = Dashboard.userPrefersDarkTheme ? "true" : "false";
                    data.dashboardSettings.preferences.set(DashboardPreference.Darktheme, value);
                    localStorage.setItem("horde_darktheme", value);
                } else {
                    localStorage.setItem("horde_darktheme", current);
                }

                resolve(data);

            }).catch(reason => reject(reason));

        });

    }

    getIssue(issueId: number): Promise<IssueData> {

        return new Promise<IssueData>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/${issueId}`).then((value) => {
                resolve(value.data as IssueData);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    getIssueHistory(id: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
        return new Promise<AuditLogEntry[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/${encodeURIComponent(id)}/history`, { params: query }).then((response) => {
                const history = (response.data?.entries ?? []) as AuditLogEntry[];
                resolve(history);
            }).catch(reason => { reject(reason); });
        });
    }

    getTemplateHistory(streamId: string, templateId: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
        return new Promise<AuditLogEntry[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/streams/${encodeURIComponent(streamId)}/templates/${encodeURIComponent(templateId)}/history`, { params: query }).then((response) => {
                const history = (response.data?.entries ?? []) as AuditLogEntry[];
                resolve(history);
            }).catch(reason => { reject(reason); });
        });
    }

    getIssuesV2(queryIn?: IssueQueryV2): Promise<FindIssueResponse[]> {

        const query = queryIn ?? {};

        return new Promise<FindIssueResponse[]>((resolve, reject) => {

            this._fetch.get("/api/v2/issues", {
                params: query
            }).then((value) => {
                resolve(value.data as FindIssueResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    getIssues(queryIn?: IssueQuery): Promise<IssueData[]> {

        const query = queryIn ?? {};

        return new Promise<IssueData[]>((resolve, reject) => {

            this._fetch.get("/api/v1/issues", {
                params: query
            }).then((value) => {
                resolve(value.data as IssueData[]);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    getIssuesByIds(ids: number[]): Promise<IssueData[]> {

        const unique = Array.from(new Set(ids));

        return new Promise<IssueData[]>((resolve, reject) => {

            this._fetch.get(`/api/v1/issues`, { params: { id: unique.map(id => id.toString()) } })
                .then(response => { resolve(response.data); })
                .catch(reason => reject(reason));
        })
    }

    getIssueStreams(issueId: number): Promise<GetIssueStreamResponse[]> {
        return new Promise<GetIssueStreamResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/${issueId}/streams`).then((value) => {
                resolve(value.data as GetIssueStreamResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    getIssueEvents(issueId: number, jobId?: string, logIds?: string[], count?: number): Promise<GetLogEventResponse[]> {

        if (logIds && logIds.length) {
            jobId = undefined;
        }

        const params = {
            jobId: jobId,
            logIds: logIds,
            count: count
        };

        return new Promise<GetLogEventResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/${issueId}/events`, { params: params }).then((value) => {
                resolve(value.data as GetLogEventResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });

    }

    updateIssue(id: number, request: UpdateIssueRequest): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`/api/v1/issues/${id}`, request).then((value) => {
                resolve(true);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getExternalIssues(streamId: string, keys: string[]): Promise<GetExternalIssueResponse[]> {

        const params = {
            streamId: streamId,
            keys: keys
        };

        return new Promise<GetExternalIssueResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/external`, { params: params }).then((value) => {
                resolve(value.data as GetExternalIssueResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getExternalIssueProjects(streamId: string): Promise<GetExternalIssueProjectResponse[]> {

        const params = {
            streamId: streamId,
        };

        return new Promise<GetExternalIssueProjectResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/issues/external/projects`, { params: params }).then((value) => {
                resolve(value.data as GetExternalIssueProjectResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }


    createExternalIssue(request: CreateExternalIssueRequest): Promise<CreateExternalIssueResponse> {
        return new Promise<CreateExternalIssueResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/issues/external`, request).then((value) => {
                resolve(value.data as CreateExternalIssueResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getTestData(id: string, filter?: string): Promise<TestData> {
        return new Promise<TestData>((resolve, reject) => {
            const params: any = {
                filter: 'id,key,change,jobId,stepId,streamId,' + (filter ? filter : 'data')
            };

            this._fetch.get(`/api/v1/testdata/${id}`, {
                params: params
            }).then((value) => {
                resolve(value.data as TestData);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTestMetadata(automationProjects?: string[], platforms?: string[], targets?: string[], configurations?: string[]): Promise<GetTestMetaResponse[]> {
        return new Promise<GetTestMetaResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/testdata/metadata`, {
                params: { project: automationProjects, platform: platforms, target: targets, configuration: configurations }
            }).then((value) => {
                resolve(value.data as GetTestMetaResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTestRefs(streamIds: string[], metaIds: string[], minCreateTime?: string, maxCreateTime?: string, minChange?: number, maxChange?: number, testIds?: string[], suiteIds?: string[]): Promise<GetTestDataRefResponse[]> {
        return new Promise<GetTestDataRefResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/testdata/refs`, {
                params: { id: streamIds, mid: metaIds, tid: testIds, sid: suiteIds, minCreateTime: minCreateTime, maxCreateTime: maxCreateTime, minChange: minChange, maxChange: maxChange }
            }).then((value) => {
                resolve(value.data as GetTestDataRefResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTestDetails(refIds: string[]): Promise<GetTestDataDetailsResponse[]> {
        return new Promise<GetTestDataDetailsResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/testdata/details`, {
                params: { id: refIds }
            }).then((value) => {
                resolve(value.data as GetTestDataDetailsResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTests(testIds: string[]): Promise<GetTestResponse[]> {
        const request: GetTestsRequest = { testIds: testIds };
        return new Promise<GetTestResponse[]>((resolve, reject) => {
            this._fetch.post(`/api/v2/testdata/tests`, request).then((value) => {
                resolve(value.data as GetTestResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTestStreams(streamIds: string[]): Promise<GetTestStreamResponse[]> {
        return new Promise<GetTestStreamResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/testdata/streams`, {
                params: { id: streamIds }
            }).then((value) => {
                resolve(value.data as GetTestStreamResponse[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getJobTestData(jobId: string, stepId?: string): Promise<TestData[]> {
        return new Promise<TestData[]>((resolve, reject) => {
            const params: any = {
                jobId: jobId,
                filter: 'id,key,change,jobId,stepId,streamId',
                count: 500,
            };

            if (stepId) {
                params.jobStepId = stepId;
            }

            this._fetch.get(`/api/v1/testdata`, {
                params: params
            }).then((value) => {
                resolve(value.data as TestData[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getTestDataHistory(streamId: string, key: string, maxChange?: number, count: number = 30, index?: number, filter?: string): Promise<TestData[]> {
        return new Promise<TestData[]>((resolve, reject) => {
            const params: any = {
                streamId: streamId,
                key: key,
                count: count,
                filter: 'id,key,change,jobId,stepId,streamId,' + (filter ? filter : 'data')
            };

            if (maxChange) {
                params.maxChange = maxChange;
            }
            if (index) {
                params.index = index;
            }

            this._fetch.get(`/api/v1/testdata`, {
                params: params
            }).then((value) => {
                resolve(value.data as TestData[]);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    downloadLog(logId: string, filename: string, json = false) {

        try {
            const url = `${this.serverUrl}/api/v1/logs/${logId}/data?download=true&format=${json ? 'raw' : 'text'}&filename=${encodeURIComponent(filename)}`;
            const link = document.createElement('a');
            link.href = url;
            document.body.appendChild(link);
            link.click();
        } catch (reason) {
            console.error(reason);
        }

    }

    updateJobStep(jobId: string, batchId: string, stepId: string, request: UpdateStepRequest): Promise<UpdateStepResponse> {
        return new Promise<UpdateStepResponse>((resolve, reject) => {
            this._fetch.put(`api/v1/jobs/${jobId}/batches/${batchId}/steps/${stepId}`, request).then((value) => {
                resolve(value.data as UpdateStepResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getNotification(type: string, jobId: string, labelIdx?: string, batchId?: string, stepId?: string): Promise<GetNotificationResponse> {

        let url = `api/v1/jobs/${jobId}`;
        if (type === "label" && labelIdx) {
            url = `${url}/labels/${labelIdx}`;
        }
        else if (type === "step" && batchId && stepId) {
            url = `${url}/batches/${batchId}/steps/${stepId}`;
        }
        return new Promise<GetNotificationResponse>((resolve, reject) => {
            this._fetch.get(`${url}/notifications`).then((value) => {
                resolve(value.data as GetNotificationResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    updateNotification(request: UpdateNotificationsRequest, type: string, jobId: string, labelIdx?: string, batchId?: string, stepId?: string | null): Promise<boolean> {

        let url = `api/v1/jobs/${jobId}`;

        if (type === "label" && labelIdx) {
            url = `${url}/labels/${labelIdx}`;
        }
        else if (type === "step" && batchId && stepId) {
            url = `${url}/batches/${batchId}/steps/${stepId}`;
        }
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`${url}/notifications`, request).then((value) => {
                resolve(true);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    createSubscription(requests: CreateSubscriptionRequest[]): Promise<CreateSubscriptionResponse[]> {
        requests.forEach(request => {
            request.userId = dashboard.userId;
        });
        let url = `api/v1/subscriptions`;
        return new Promise<CreateSubscriptionResponse[]>((resolve, reject) => {
            this._fetch.post(`${url}`, requests).then((value) => {
                let created = value.data as CreateSubscriptionResponse[];
                resolve(created);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getSubscriptions(): Promise<GetSubscriptionResponse[]> {
        let url = `api/v1/subscriptions?userId=${dashboard.userId}`;
        return new Promise<GetSubscriptionResponse[]>((resolve, reject) => {
            this._fetch.get(`${url}`, { suppress404: true }).then((value) => {
                resolve(value.data as GetSubscriptionResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    deleteSubscription(id: string): Promise<boolean> {
        let url = `api/v1/subscriptions/${id}`;
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.delete(`${url}`).then((value) => {
                resolve(true);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getUtilizationData(endDate: string, range: number, tzOffset: number): Promise<GetUtilizationTelemetryResponse[]> {
        let url = `api/v1/reports/utilization/${endDate}?range=${range}&tzOffset=${tzOffset}`;
        return new Promise<GetUtilizationTelemetryResponse[]>((resolve, reject) => {
            this._fetch.get(`${url}`).then((value) => {
                resolve(value.data as GetUtilizationTelemetryResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getPerforceServerStatus(): Promise<GetPerforceServerStatusResponse[]> {

        return new Promise<GetPerforceServerStatusResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/perforce/status`).then((value) => {
                resolve(value.data as GetPerforceServerStatusResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getServerStatus(): Promise<ServerStatusResponse> {

        return new Promise<ServerStatusResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/server/status`).then((value) => {
                const result = value.data as ServerStatusResponse;
                // convert from string date to Date
                result.statuses.forEach(s => s.updates.forEach(u => u.updatedAt = new Date(u.updatedAt)))
                resolve(result);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getStorageDashboard(minTime?: Date, maxTime?: Date, trendCount?: number): Promise<GetStorageDashboardResponse> {
        const params = new URLSearchParams();
        if (minTime) params.append("minTime", minTime.toISOString());
        if (maxTime) params.append("maxTime", maxTime.toISOString());
        if (trendCount) params.append("trendCount", trendCount.toString());

        return new Promise<GetStorageDashboardResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/storage/dashboard?${params.toString()}`).then((response) => {
                const data = response.data as GetStorageDashboardResponse;
                resolve(data);
            }).catch(reason => { reject(reason); });
        });
    }

    getDevices(): Promise<GetDeviceResponse[]> {

        return new Promise<GetDeviceResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices`).then((value) => {
                resolve(value.data as GetDeviceResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getDevicePoolTelemetry(query?: DevicePoolTelemetryQuery): Promise<GetDevicePoolTelemetryResponse[]> {

        return new Promise<GetDevicePoolTelemetryResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices/pools/telemetry`, { params: query }).then((value) => {
                resolve(value.data as GetDevicePoolTelemetryResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getDeviceTelemetry(query?: DeviceTelemetryQuery): Promise<GetDeviceTelemetryResponse[]> {

        return new Promise<GetDeviceTelemetryResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices/telemetry`, { params: query }).then((value) => {
                resolve(value.data as GetDeviceTelemetryResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    addDevice(request: CreateDeviceRequest): Promise<CreateDeviceResponse> {
        return new Promise<CreateDeviceResponse>((resolve, reject) => {
            this._fetch.post(`/api/v2/devices`, request).then((value) => {
                resolve(value.data as CreateDeviceResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    checkoutDevice(deviceId: string, checkout: boolean): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v2/devices/${deviceId}/checkout`, { checkout: checkout }).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }


    modifyDevice(deviceId: string, request: UpdateDeviceRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v2/devices/${deviceId}`, request).then((value) => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getDeviceHistory(id: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
        return new Promise<AuditLogEntry[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/devices/${encodeURIComponent(id)}/history`, { params: query }).then((response) => {
                const history = (response.data?.entries ?? []) as AuditLogEntry[];
                resolve(history);
            }).catch(reason => { reject(reason); });
        });
    }


    updateLease(leaseId: string, update: UpdateLeaseRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v1/leases/${leaseId}`, update).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }


    deleteDevice(deviceId: string): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.delete(`/api/v2/devices/${deviceId}`).then((response) => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getDevicePlatforms(): Promise<GetDevicePlatformResponse[]> {

        return new Promise<GetDevicePlatformResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices/platforms`).then((value) => {
                resolve(value.data as GetDevicePlatformResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getDevicePools(): Promise<GetDevicePoolResponse[]> {

        return new Promise<GetDevicePoolResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices/pools`).then((value) => {
                resolve(value.data as GetDevicePoolResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getDeviceReservations(): Promise<GetDeviceReservationResponse[]> {

        return new Promise<GetDeviceReservationResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v2/devices/reservations`).then((value) => {
                resolve(value.data as GetDeviceReservationResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    getServerSettings(): Promise<GetServerSettingsResponse> {
        return new Promise<GetServerSettingsResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/config/serversettings`).then((value) => {
                resolve(value.data as GetServerSettingsResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    updateServerSettings(request: UpdateServerSettingsRequest): Promise<ServerUpdateResponse> {
        return new Promise<ServerUpdateResponse>((resolve, reject) => {
            this._fetch.put(`/api/v1/config/serversettings`, request).then((value) => {
                resolve(value.data as ServerUpdateResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getGlobalConfig(): Promise<GlobalConfig> {
        return new Promise<GlobalConfig>((resolve, reject) => {
            this._fetch.get(`/api/v1/config/global`).then((value) => {
                resolve(value.data as GlobalConfig);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // updates global configuation
    updateGlobalConfig(request: UpdateGlobalConfigRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v1/config/global`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getServerInfo(): Promise<GetServerInfoResponse> {
        return new Promise<GetServerInfoResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/server/info`).then((value) => {
                resolve(value.data as GetServerInfoResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // create a new notice
    createNotice(request: CreateNoticeRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.post(`/api/v1/notices`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // get all notices
    getNotices(): Promise<GetNoticeResponse[]> {
        return new Promise<GetNoticeResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/notices`).then((value) => {
                resolve((value.data as GetNoticeResponse[]).map(notice => {
                    notice.startTime = notice.startTime ? new Date(notice.startTime as string) : undefined;
                    notice.finishTime = notice.finishTime ? new Date(notice.finishTime as string) : undefined;
                    return notice;
                }));
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // get tools
    getTools(): Promise<GetToolSummaryResponse[]> {
        return new Promise<GetToolSummaryResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/tools`).then((value) => {
                resolve(value.data?.tools as GetToolSummaryResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getTool(toolId: string): Promise<GetToolResponse> {
        return new Promise<GetToolResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/tools/${toolId}`).then((value) => {
                resolve(value.data as GetToolResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    /// Updates the state of an active deployment.
    updateDeployment(toolId: string, deploymentId: string, update: UpdateDeploymentRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.patch(`/api/v1/tools/${toolId}/deployments/${deploymentId}`, update).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }
    // update a notice
    updateNotice(request: UpdateNoticeRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v1/notices`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // delete a notice
    deleteNotice(id: string): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.delete(`/api/v1/notices/${id}`).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // create job bisection
    createBisectTask(create: CreateBisectTaskRequest): Promise<CreateBisectTaskResponse> {
        return new Promise<CreateBisectTaskResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/bisect`, create).then((response) => {
                resolve(response.data);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // get a bisection task
    getBisectTask(id: string): Promise<GetBisectTaskResponse> {
        return new Promise<GetBisectTaskResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/bisect/${id}`).then((value) => {
                resolve(value.data as GetBisectTaskResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getBisections(query: { id?: string[], ownerId?: string, jobId?: string, minCreateTime?: string, maxCreateTime?: string, index?: number, count?: number }) {

        return new Promise<GetBisectTaskResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/bisect`, { params: query }).then((value) => {
                resolve(value.data as GetBisectTaskResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // update a visection task
    updateBisectTask(id: string, request: UpdateBisectTaskRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.patch(`/api/v1/bisect/${id}`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getJobHistory(id: string, query: AuditLogQuery): Promise<AuditLogEntry[]> {
        return new Promise<AuditLogEntry[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/jobs/${encodeURIComponent(id)}/history`, { params: query }).then((response) => {
                const history = (response.data?.entries ?? []) as AuditLogEntry[];
                resolve(history);
            }).catch(reason => { reject(reason); });
        });
    }

    // get a job's bisection tasks
    getJobBisectTasks(jobId: string): Promise<GetBisectTaskResponse[]> {
        return new Promise<GetBisectTaskResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/bisect/job/${jobId}`).then((value) => {
                resolve(value.data as GetBisectTaskResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    checkPreflightConfig(shelvedChange: number, cluster?: string): Promise<PreflightConfigResponse> {

        return new Promise<PreflightConfigResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/server/preflightconfig`, { shelvedChange: shelvedChange, cluster: cluster }).then((value) => {
                resolve(value.data as PreflightConfigResponse);
            }).catch(reason => {
                reject(reason);
            });
        });

    }

    // Accounts

    getAccountEntitlements(): Promise<any> {
        return new Promise<any>((resolve, reject) => {
            this._fetch.get(`/account/entitlements`).then((value) => {
                resolve(value.data as any);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // get account access for a specific scope and action
    getAccountAccess(scope: string, action: string): Promise<any> {
        return new Promise<any>((resolve, reject) => {
            this._fetch.get(`/account/access`, { params: { scope: scope, action: action } }).then((value) => {
                resolve(value.data as any);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // update current account 
    updateCurrentAccount(request: UpdateCurrentAccountRequest): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.put(`/api/v1/accounts/current`, request).then(() => {
                resolve(true);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    getAccounts(): Promise<GetAccountResponse[]> {
        return new Promise<GetAccountResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/accounts`).then((value) => {
                resolve(value.data as GetAccountResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    getAccountGroups(): Promise<AccountClaimMessage[]> {
        return new Promise<AccountClaimMessage[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/dashboard/accountgroups`).then((value) => {
                resolve(value.data as AccountClaimMessage[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // create a new account
    createAccount(request: CreateAccountRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.post(`/api/v1/accounts`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // delete an account
    deleteAccount(id: string): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.delete(`/api/v1/accounts/${id}`).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // update an account
    updateAccount(id: string, request: UpdateAccountRequest): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.put(`/api/v1/accounts/${id}`, request).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // Service Accounts

    getServiceAccounts(): Promise<GetServiceAccountResponse[]> {
        return new Promise<GetServiceAccountResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/serviceaccounts`).then((value) => {
                resolve(value.data as GetServiceAccountResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // create a new account
    createServiceAccount(request: CreateServiceAccountRequest): Promise<CreateServiceAccountResponse> {
        return new Promise<CreateServiceAccountResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/serviceaccounts`, request).then((value) => {
                resolve(value.data as CreateServiceAccountResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // delete an account
    deleteServiceAccount(id: string): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            this._fetch.delete(`/api/v1/serviceaccounts/${id}`).then(() => {
                resolve();
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // update an account
    updateServiceAccount(id: string, request: UpdateServiceAccountRequest): Promise<UpdateServiceAccountResponse> {
        return new Promise<UpdateServiceAccountResponse>((resolve, reject) => {
            this._fetch.put(`/api/v1/serviceaccounts/${id}`, request).then((value) => {
                resolve(value.data as UpdateServiceAccountResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    //#region -- Build Health Filters --

    // add a build health project filter
    addBuildHealthProjectFilter(request: AddBuildHealthFiltersRequest): Promise<GetBuildHealthFilterResponse> {
        return new Promise<GetBuildHealthFilterResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/buildhealth/filters`, request).then((value) => {
                resolve(value.data as GetBuildHealthFilterResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // update a build health project filter
    updateBuildHealthProjectFilter(filterId: string, request: UpdateBuildHealthFiltersRequest): Promise<GetBuildHealthFilterResponse> {
        return new Promise<GetBuildHealthFilterResponse>((resolve, reject) => {
            this._fetch.put(`/api/v1/buildhealth/filters/${filterId}`, request).then((value) => {
                resolve(value.data as GetBuildHealthFilterResponse);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // delete a build health project filter
    deleteBuildHealthProjectFilter(filterId: string): Promise<boolean> {
        return new Promise<boolean>((resolve, reject) => {
            this._fetch.delete(`/api/v1/buildhealth/filters/${filterId}`).then(() => {
                resolve(true);
            }).catch(() => {
                reject(false);
            });
        });
    }

    // get build health project filters
    getBuildHealthProjectFilters(query: GetBuildHealthFiltersRequest): Promise<GetBuildHealthFilterResponse[]> {
        return new Promise<GetBuildHealthFilterResponse[]>((resolve, reject) => {
            this._fetch.get(`/api/v1/buildhealth/filters`, { params: query }).then((value) => {
                resolve(value.data as GetBuildHealthFilterResponse[]);
            }).catch(reason => {
                reject(reason);
            });
        });
    }

    // #endregion -- Build Health Filters --

    // #region -- Deep Links --

    /// Get a deep link to a channel
    getDeepLinkChannelLink(channelId: string): Promise<DeepLinkResponse> {
        return new Promise<DeepLinkResponse>((resolve, reject) => {
            this._fetch.get(`/api/v1/deeplink/channel`, {
                params: {channelId: channelId}
            }).then((value) => {
                resolve(value.data as DeepLinkResponse);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }
    
    /// Get a deep link to a direct message
    getDeepLinkDirectMessageLink(userIds: string[]): Promise<DeepLinkResponse> {
        const request = { userIds: userIds };
        return new Promise<DeepLinkResponse>((resolve, reject) => {
            this._fetch.post(`/api/v1/deeplink/directmessage`, request).then((value) => {
                resolve(value.data as DeepLinkResponse);
            }).catch((reason) => {
                reject(reason);
            });
        });
    }

    // #endregion -- Deep Links --

    private async update() {

        if (this.updateID === "updating") {
            return;
        }

        this.updateID = "updating";

        await Promise.all([projectStore.update()]).then(values => {

        }).catch(reason => {

        }).finally(() => {
            this.updateID = setTimeout(() => { this.update(); }, updateInterval);
        });

    }

    async serverLogout(redirect: string) {
        try {
            this._fetch.logout = true;
            await this._fetch.get("/api/v1/dashboard/logout", { params: { dashboard: true } });
            window.location.assign(redirect);

        } catch (err) {
            console.error(err);
        }
    }

    // server url if on a separate origin from dashboard, as with local development and a debug token
    get serverUrl(): string {

        const url = import.meta.env.VITE_HORDE_BACKEND;

        if (!url) {
            return "";
        }

        return url;
    }

    get debugToken(): string {

        const token = import.meta.env.VITE_HORDE_DEBUG_TOKEN;

        if (!token) {
            return "";
        }

        return token;

    }

    get enabledPlugins(): ServerPluginInfoResponse[] {
        return this.serverInfo?.plugins?.filter(p => p.loaded) ?? [];
    }

    init() {


        return new Promise<boolean>(async (resolve, reject) => {

            this._fetch.setBaseUrl(this.serverUrl);
            this._fetch.setDebugToken(this.debugToken);

            const challenge = await this._fetch.challenge();

            if (challenge === ChallengeStatus.Unauthorized) {
                this._fetch.login(window.location.toString());
                return;
            }

            this.serverInfo = await this.getServerInfo();

            await dashboard.update();

            if (dashboard.localCache) {
                templateCache.initialize();
            }

            this.update().then(() => {
                resolve(true);
            }).catch((reason) => {
                reject(reason);
            });

        });
    }

    get fetch() {
        return this._fetch;
    }

    constructor() {

        this._fetch = new Fetch();
    }

    serverInfo?: GetServerInfoResponse;

    updateID?: any;
    logout: boolean = false;

    private _fetch: Fetch;

}
