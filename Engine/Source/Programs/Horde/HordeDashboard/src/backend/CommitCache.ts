// Copyright Epic Games, Inc. All Rights Reserved.
import { action, makeObservable, observable } from 'mobx';
import backend from '.';
import { GetChangeSummaryResponse } from "./Api";

export class CommitCache {

    constructor() {
        makeObservable(this);
    }

    @action
    setUpdated() {
        this.updated++;
    }

    @observable
    updated: number = 0;

    getCommit(streamId: string, change: number): GetChangeSummaryResponse | undefined {

        let map = this.summaries.get(streamId); 
                
        if (!map) {
            return undefined;
        }
        
        const commit = map.get(change);

        if (!commit || !commit.authorInfo?.id) {
            return undefined;
        }

        return commit;

    }

    async set(streamId: string, changes: number[]) {

        let map = this.summaries.get(streamId);
        if (!map) {
            map = new Map();
            this.summaries.set(streamId, map);
        }

        if (!changes.length) {
            return;
        }

        // Filter to only changes we don't already have cached
        const uncachedChanges = changes.filter(c => !map!.get(c));

        if (!uncachedChanges.length) {
            return;
        }

        // Set placeholders for changes we're about to fetch
        uncachedChanges.forEach(c => {
            map!.set(c, {
                number: 0, description: "", dateUtc: "", authorInfo: { id: "", email: "", name: "" }
            });
        });

        try {
            const commits = await backend.getBatchChangeSummaries(streamId, uncachedChanges);
            commits.forEach(c => map!.set(c.number, c));
            this.setUpdated();
        } catch (reason) {
            console.error(reason);
            // @todo: we could clear out the place holders on error
        }

    }

    private summaries: Map<string, Map<number, GetChangeSummaryResponse>> = new Map();
}
