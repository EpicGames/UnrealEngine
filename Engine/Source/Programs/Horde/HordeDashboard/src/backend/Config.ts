// Copyright Epic Games, Inc. All Rights Reserved.

// import { initDatadog} from './Datadog';

export type SiteConfig = {
    environment: "dev" | "production" | "dev-local" | "";
    plugins?: string[];
    datadogClient?: string;
}

const dev = {
	"environment": "dev",
	"datadogClient" : "",
	"plugins": [
		"AutomatedTestSession",
		"UnrealAutomatedTests",
		"SimpleTestReport"
	]
} 

const prod = {
	"environment": "production",
	"datadogClient" : "",
	"plugins": [
		"AutomatedTestSession",
		"UnrealAutomatedTests",
		"SimpleTestReport"
	]
} 

export function getSiteConfig() {

	let isProd = false;
	
	try {
		// note: this must be exactly `import.meta.env.VITE_DASHBOARD_CONFIG`
		// as webpack does a simple find and replace to the value in the .env
		// so import?.meta?.env>.VITE_DASHBOARD_CONFIG for example is invalid
		isProd = import.meta.env.VITE_DASHBOARD_CONFIG !== "Development";
	 } catch (reason) {
		console.log("Process env error on VITE_DASHBOARD_CONFIG:", reason);
	 }
          
    return isProd ? prod : dev;
}
