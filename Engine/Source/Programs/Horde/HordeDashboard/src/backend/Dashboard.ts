// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import backend from '.';
import { AuthMethod, DashboardPreference, DownloadFormat, GetDashboardAgentCategoryResponse, GetDashboardArtifactTypeResponse, GetDashboardConfigResponse, GetDashboardPoolCategoryResponse, GetJobTemplateSettingsResponse, GetUserResponse, UserClaim } from './Api';
import { getSiteConfig } from './Config';

export enum StatusColor {
    Success,
    Warnings,
    Failure,
    Waiting,
    Ready,
    Skipped,
    Aborted,
    Running,
    User1,
    User2,
    User3,
    Unspecified
}

export enum WebBrowser {
    Chromium = "Chromium",
    Safari = "Safari",
    Other = "Other"
}

export enum UserPlatform {
    Windows = "Windows",
    Mac = "Mac",
    Linux = "Linux",
    Other = "Other"
}

export class Dashboard {

    constructor() {
        makeObservable(this);
    }

    startPolling() {
        this.polling = true;
    }

    stopPolling() {
        this.polling = false;
    }

    jobPinned(id: string | undefined) {
        return !!this.pinnedJobsIds.find(j => j === id);
    }

    pinJob(id: string) {

        if (this.data.pinnedJobIds!.find(j => j === id)) {
            return;
        }

        this.data.pinnedJobIds!.push(id);

        backend.updateUser({ addPinnedJobIds: [id] });

        this.setUpdated();
    }

    unpinJob(id: string) {

        if (!this.data.pinnedJobIds!.find(j => j === id)) {
            return;
        }

        this.data.pinnedJobIds = this.data.pinnedJobIds!.filter(j => j !== id);

        backend.updateUser({ removePinnedJobIds: [id] });

        this.setUpdated();
    }

    bisectPinned(id: string | undefined) {
        return !!this.pinnedBisectTaskIds.find(j => j === id);
    }

    pinBisect(id: string) {

        if (this.data.pinnedBisectTaskIds!.find(j => j === id)) {
            return;
        }

        this.data.pinnedBisectTaskIds!.push(id);

        backend.updateUser({ addPinnedBisectTaskIds: [id] });

        this.setUpdated();
    }

    unpinBisect(id: string) {

        if (!this.data.pinnedBisectTaskIds!.find(j => j === id)) {
            return;
        }

        this.data.pinnedBisectTaskIds = this.data.pinnedBisectTaskIds!.filter(j => j !== id);

        backend.updateUser({ removePinnedBisectTaskIds: [id] });

        this.setUpdated();
    }

    clearPinnedJobs() {

        if (!this.data.pinnedJobIds?.length) {
            return;
        }

        const jobs = this.data.pinnedJobIds;

        this.data.pinnedJobIds = [];

        backend.updateUser({ removePinnedJobIds: jobs });

        this.setUpdated();
    }

    get username(): string {
        return this.data.name;
    }

    get userImage48(): string | undefined {
        return this.data.image48;
    }

    get userImage32(): string | undefined {
        return this.data.image32;
    }

    get email(): string {

        if (this.data?.email) {
            return this.data?.email;
        }

        const email = this.claims.find(c => c.type.endsWith("/emailaddress"));

        return email ? email.value : "???";
    }

    get authMethod(): AuthMethod | undefined {
        return this.config?.authMethod;
    }

    get p4user(): string {
        const claims = this.claims;
        const user = claims.filter(c => c.type.endsWith("/perforce-user"));
        if (!user.length) {
            return "";
        }
        return user[0].value;

    }

    get hordeUser(): string | undefined {
        const claims = this.claims;
        const user = claims.filter(c => c.type.endsWith("/horde/user"));
        if (!user.length) {
            return undefined;
        }

        return user[0].value?.length ? user[0].value : undefined;
    }

    get hordeAdmin(): boolean {
        return !!this.claims?.find(c => c.type === "groups" && c.value === "app-horde-admins");
    }

    get hordeGenericAdmin(): boolean {
        return this.hordeAdmin || !!this.claims?.find(c => c.type === "http://epicgames.com/ue/horde/role" && c.value === "admin");
    }

    get preview(): boolean {

        try {
            return !!(import.meta.env.VITE_HORDE_DEBUG_PREVIEW) || window?.location?.hostname?.indexOf("preview") !== -1;
        } catch (reason) {
            console.error(reason);
        }
        return false;
    }

    get development(): boolean {

        try {
            return getSiteConfig().environment === "dev";
        } catch (reason) {
            console.error(reason);
        }
        return false;
    }

    get browser(): WebBrowser {

        const agent = window.navigator.userAgent.toLowerCase();

        switch (true) {
            case agent.indexOf("edge") > -1: return WebBrowser.Other;
            case agent.indexOf("edg") > -1: return WebBrowser.Chromium;
            case agent.indexOf("opr") > -1: return WebBrowser.Other;
            case agent.indexOf("chrome") > -1 && !!(window as any).chrome: return WebBrowser.Chromium;
            case agent.indexOf("trident") > -1: return WebBrowser.Other;
            case agent.indexOf("firefox") > -1: return WebBrowser.Other;
            case agent.indexOf("safari") > -1: return WebBrowser.Safari;
            default: return WebBrowser.Other;
        }

    }

    get platform(): UserPlatform {

        const uaData = (window.navigator as any).userAgentData?.platform?.toLowerCase() ?? "";
        if (uaData) {
            if (uaData === "windows") return UserPlatform.Windows;
            if (uaData === "macos") return UserPlatform.Mac;
            if (uaData === "linux") return UserPlatform.Linux;
        }

        const userAgent = window.navigator.userAgent.toLowerCase();
        if (userAgent.indexOf("windows") > -1) return UserPlatform.Windows;
        if (userAgent.indexOf("macintosh") > -1 || userAgent.indexOf("mac os") > -1) return UserPlatform.Mac;
        if (userAgent.indexOf("linux") > -1) return UserPlatform.Linux;
        return UserPlatform.Other;

    }


    get helpEmail(): string | undefined {
        return this.config?.helpEmailAddress;
    }

    get helpSlack(): string | undefined {
        return this.config?.helpSlackChannel;
    }

    get agentCategories(): GetDashboardAgentCategoryResponse[] {
        return this.config?.agentCategories ?? [];
    }

    get poolCategories(): GetDashboardPoolCategoryResponse[] {
        return this.config?.poolCategories ?? [];
    }

    get artifactTypes(): string[] {
        return this.config?.artifactTypes ?? [];
    }

    getArtifactTypeInfo(type: string): GetDashboardArtifactTypeResponse | undefined {
        return this.config?.artifactTypeInfo?.find(a => a.type === type);
    }

    get deviceProblemCooldownMinutes(): number {
        return this.config?.deviceProblemCooldownMinutes ?? 30;
    }

    get configVersion(): number | undefined {
        return this.config?.configVersion;
    }

    get swarmUrl(): string | undefined {
        return this.config?.perforceSwarmUrl;
    }

    get externalIssueService(): { name: string, url: string } | undefined {
        if (!this.config?.externalIssueServiceUrl) {
            return undefined;
        }
        return { name: this.config.externalIssueServiceName ?? "???", url: this.config.externalIssueServiceUrl };
    }

    get userId(): string {
        return this.data.id;
    }

    get pinnedJobsIds(): string[] {

        return this.data.pinnedJobIds ?? [];
    }

    get pinnedBisectTaskIds(): string[] {

        return this.data.pinnedBisectTaskIds ?? [];
    }


    getLastJobTemplateSettings(streamId: string, templateIds: string[]): GetJobTemplateSettingsResponse | undefined {

        try {

            const streamTemplates = this.data?.jobTemplateSettings?.filter(t => t.streamId === streamId && templateIds.indexOf(t.templateId) !== -1)

            if (!streamTemplates?.length) {
                return undefined;
            }

            const sorted = streamTemplates.sort((a, b) => {
                return new Date(b.updateTimeUtc).getTime() - new Date(a.updateTimeUtc).getTime();
            });

            return sorted[0];
        } catch (error) {
            console.error(error);
        }

        return undefined;
    }

    set jobTemplateSettings(settings: GetJobTemplateSettingsResponse[]) {
        if (this.data) {
            this.data.jobTemplateSettings = settings;
        }
    }

    get experimentalFeatures(): boolean {

        return this.data.enableExperimentalFeatures!;
    }

    set experimentalFeatures(value: boolean) {

        backend.updateUser({ enableExperimentalFeatures: value }).then(() => {

            this.data.enableExperimentalFeatures = value;
            this.setUpdated();

        });
    }

    get alwaysTagPreflightCL(): boolean {

        return this.data.alwaysTagPreflightCL!;
    }

    set alwaysTagPreflightCL(value: boolean) {

        backend.updateUser({ alwaysTagPreflightCL: value }).then(() => {

            this.data.alwaysTagPreflightCL = value;
            this.setUpdated();

        });
    }

    get roles(): UserClaim[] {
        const claims = this.claims;
        return claims.filter(c => c.type.endsWith("/role"));
    }


    get claims(): UserClaim[] {
        return this.data.claims ? this.data.claims : [];
    }

    get displayUTC(): boolean {

        return this.preferences.get(DashboardPreference.DisplayUTC) === 'true';

    }

    static get userPrefersDarkTheme(): boolean {

        try {
            if (window.matchMedia && window.matchMedia('(prefers-color-scheme: dark)').matches) {
                return true;
            }
        } catch (reason) {
            if (!Dashboard.hasLoggedDarkThemePref) {
                Dashboard.hasLoggedDarkThemePref = true;
                console.error(reason);
            }
        }

        return false;
    }

    private static hasLoggedDarkThemePref = false;

    get darktheme(): boolean {

        if (!this.available) {

            let local = localStorage.getItem("horde_darktheme");
            if (local === "true") {
                return true;
            }
            if (local === "false") {
                return false;
            }

            return Dashboard.userPrefersDarkTheme;
        }

        const pref = this.preferences.get(DashboardPreference.Darktheme);

        if (pref !== "true" && pref !== "false") {
            console.error("No theme preference set, should be defaulted in getCurrentUser");
            return Dashboard.userPrefersDarkTheme;;
        }

        return this.preferences.get(DashboardPreference.Darktheme) !== 'false';

    }

    setDarkTheme(value: boolean | undefined) {
        this.setPreference(DashboardPreference.Darktheme, value ? "true" : "false");
    }

    setDisplayUTC(value: boolean | undefined) {
        this.setPreference(DashboardPreference.DisplayUTC, value ? "true" : "false");
    }

    setLeftAlignLog(value: boolean | undefined) {
        this.setPreference(DashboardPreference.LeftAlignLog, value ? "true" : "false");
    }

    get leftAlignLog(): boolean {
        return this.preferences.get(DashboardPreference.LeftAlignLog) === 'true';
    }

    setShowPreflights(value: boolean | undefined) {
        this.setPreference(DashboardPreference.ShowPreflights, value ? "true" : "false");
    }

    get showPreflights(): boolean {
        return this.preferences.get(DashboardPreference.ShowPreflights) === 'true';
    }

    setCompactViews(value: boolean | undefined) {
        this.setPreference(DashboardPreference.CompactViews, value ? "true" : "false");
    }

    get compactViews(): boolean {
        return this.preferences.get(DashboardPreference.CompactViews) === 'true';
    }

    setDownloadFormat(value: DownloadFormat | undefined) {
        this.setPreference(DashboardPreference.DownloadFormat, value);
    }

    get downloadFormat(): DownloadFormat {
        return this.preferences.get(DashboardPreference.DownloadFormat) as DownloadFormat ?? DownloadFormat.Zip;
    }

    private hasLoggedLocalCache = false;

    get localCache(): boolean {

        /*
        if (this.browser === WebBrowser.Chromium) {
            if (!this.hasLoggedLocalCache) {
                this.hasLoggedLocalCache = true;
                console.log("Chromium browser detected, local caching is enabled")
            }

            return true;
        }
        */

        const value = this.preferences.get(DashboardPreference.LocalCache) !== 'false';

        if (value && !this.hasLoggedLocalCache) {
            this.hasLoggedLocalCache = true;
            console.log("Local graph and template caching is enabled")
        }

        return value;

    }

    setLocalCache(value: boolean | undefined) {
        this.setPreference(DashboardPreference.LocalCache, value ? "true" : "false");
    }


    get display24HourClock(): boolean {

        return this.preferences.get(DashboardPreference.DisplayClock) === '24';

    }

    private resetStatusColors() {
        this.setStatusColor(DashboardPreference.ColorRunning, undefined);
        this.setStatusColor(DashboardPreference.ColorWarning, undefined);
        this.setStatusColor(DashboardPreference.ColorError, undefined);
        this.setStatusColor(DashboardPreference.ColorSuccess, undefined);
    }

    setStatusColor(pref: DashboardPreference, value: string | undefined) {

        const defaultColors = this.getDefaultStatusColors();

        let defaultColor = "";
        switch (pref) {
            case DashboardPreference.ColorRunning:
                defaultColor = defaultColors.get(StatusColor.Running)!;
                break;
            case DashboardPreference.ColorWarning:
                defaultColor = defaultColors.get(StatusColor.Warnings)!;
                break;
            case DashboardPreference.ColorError:
                defaultColor = defaultColors.get(StatusColor.Failure)!;
                break;
            case DashboardPreference.ColorSuccess:
                defaultColor = defaultColors.get(StatusColor.Success)!;
                break;
        }

        if (defaultColor.toLowerCase() === value?.toLowerCase()) {
            value = undefined;
        }

        if (value && !value.startsWith("#")) {
            console.error("Status preference color must be in hex format with preceding #")
        }

        this.setPreference(pref, value);
    }

    getDefaultStatusColors = (): Map<StatusColor, string> => {

        const dark = this.darktheme;

        const colors = new Map<StatusColor, string>([
            [StatusColor.Success, dark ? "#48940f" : "#52C705"],
            [StatusColor.Warnings, dark ? "#d6b445" : "#EDC74A"],
            [StatusColor.Failure, dark ? "#c44525" : "#DE4522"],
            [StatusColor.Running, dark ? "#35a8c4" : "#00BCF2"],
            [StatusColor.Waiting, dark ? "#474542" : "#A19F9D"],
            [StatusColor.Ready, dark ? "#474542" : "#A19F9D"],
            [StatusColor.Skipped, dark ? "#63625c" : "#C3C2C1"],
            [StatusColor.User1, dark ? "#8e4747" : "#8e4747"],
            [StatusColor.User2, dark ? "#785EF0" : "#785EF0"],
            [StatusColor.User3, dark ? "#FE6100" : "#FE6100"],            
            [StatusColor.Unspecified, "#646464"]
        ]);

        colors.set(StatusColor.Aborted, colors.get(StatusColor.Failure)!);

        return colors;

    }



    getStatusColors = (): Map<StatusColor, string> => {

        const defaultStatusColors = this.getDefaultStatusColors();

        const success = this.getPreference(DashboardPreference.ColorSuccess);
        const warning = this.getPreference(DashboardPreference.ColorWarning);
        const error = this.getPreference(DashboardPreference.ColorError);
        const running = this.getPreference(DashboardPreference.ColorRunning);
        const user1 = this.getPreference(DashboardPreference.ColorUser1);
        const user2 = this.getPreference(DashboardPreference.ColorUser2);
        const user3 = this.getPreference(DashboardPreference.ColorUser3 );

        const colors = new Map<StatusColor, string>([
            [StatusColor.Success, success ? success : defaultStatusColors.get(StatusColor.Success)!],
            [StatusColor.Warnings, warning ? warning : defaultStatusColors.get(StatusColor.Warnings)!],
            [StatusColor.Failure, error ? error : defaultStatusColors.get(StatusColor.Failure)!],
            [StatusColor.Running, running ? running : defaultStatusColors.get(StatusColor.Running)!],
            [StatusColor.User1, user1 ? user1 : defaultStatusColors.get(StatusColor.User1)!],
            [StatusColor.User2, user2 ? user2 : defaultStatusColors.get(StatusColor.User2)!],
            [StatusColor.User3, user3 ? user3 : defaultStatusColors.get(StatusColor.User3)!],
            [StatusColor.Waiting, defaultStatusColors.get(StatusColor.Waiting)!],
            [StatusColor.Ready, defaultStatusColors.get(StatusColor.Ready)!],
            [StatusColor.Skipped, defaultStatusColors.get(StatusColor.Skipped)!],
            [StatusColor.Unspecified, defaultStatusColors.get(StatusColor.Unspecified)!]
        ]);

        colors.set(StatusColor.Aborted, colors.get(StatusColor.Failure)!);

        return colors;
    }



    setDisplay24HourClock(value: boolean | undefined) {
        this.setPreference(DashboardPreference.DisplayClock, value ? "24" : "");
    }

    getPreference(pref: DashboardPreference): string | undefined {


        if (!this.available) {
            return undefined;
        }

        if (!this.preferences) {
            return undefined;
        }

        return this.preferences.get(pref);
    }

    private get preferences() {
        return this.data.dashboardSettings!.preferences;
    }

    async update() {

        try {

            if (this.updating) {
                clearTimeout(this.updateTimeoutId);
                this.updateTimeoutId = setTimeout(() => { this.update(); }, this.pollMS);
                return;
            }

            this.updating = true;

            if (!this.config) {
                try {
                    this.config = await backend.getDashboardConfig();
                } catch (reason) {
                    console.error("Error getting dashboard config, defaults used: " + reason);
                    this.config = { agentCategories: [], poolCategories: [], telemetryViews: [], artifactTypes: [], artifactTypeInfo: [] };
                }
            }

            if (this.polling || !this.available) {

                const cancelId = this.cancelId++;

                const responses = await Promise.all([backend.getCurrentUser()]);

                const response = responses[0] as GetUserResponse;

                // check for canceled during request
                if (!this.canceled.has(cancelId)) {

                    this.data = response;

                    if (this.data.claims) {

                        const set = new Set<string>();
                        this.data.claims = this.data.claims.filter(c => {
                            const key = c.type + c.value;
                            if (set.has(key)) {
                                return false;
                            }
                            set.add(key);
                            return true;
                        })

                    }

                    // @todo: detect changed
                    this.setUpdated();

                }

            }

        } catch (reason) {
            if (!this.available) {
                // this is being added 1/25/21 for changes to how User's are handled on backend
                // ie. not created until logged in via Okta
                // if this is still an error in the future, this may be changed
                this.requestLogout = true;
            }
            console.error("Error updating user dashboard settings", reason)
        } finally {
            this.updating = false;
            clearTimeout(this.updateTimeoutId);
            this.updateTimeoutId = setTimeout(() => { this.update(); }, this.pollMS);
        }
    }

    private setPreference(pref: DashboardPreference, value: string | undefined): void {

        if (!this.available) {
            return;
        }

        if (this.preferences.get(pref) === value) {
            return;
        }

        if (value === undefined) {
            this.preferences.delete(pref);
        } else {
            this.preferences.set(pref, value);
        }

        if (pref === DashboardPreference.Darktheme) {
            if (value === "true") {
                localStorage.setItem("horde_darktheme", "true");
            } else if (value === "true") {
                localStorage.setItem("horde_darktheme", "false");
            }
        }

        this.postPreferences(pref === DashboardPreference.Darktheme);

    }

    setServerSettingsChanged(value: boolean | undefined) {
        this.serverSettingsChanged = value ?? false;
        this.setUpdated();
    }

    subscribe() {
        if (this.updated) { }
    }

    @observable
    updated: number = 0;

    @action
    private setUpdated() {
        this.updated++;
    }

    get available(): boolean {
        return this.data.id !== "";
    }

    get user(): GetUserResponse {
        return this.data;
    }

    private async postPreferences(reload?: boolean): Promise<boolean> {

        // cancel any pending        
        for (let i = 0; i < this.cancelId; i++) {
            this.canceled.add(i);
        }

        const data: any = {};

        for (const key of Object.keys(DashboardPreference)) {
            data[key] = this.data.dashboardSettings!.preferences?.get(key as DashboardPreference);
        }

        let success = true;
        try {
            await backend.updateUser({ dashboardSettings: { preferences: data } });
        } catch (reason) {
            success = false;
            console.error("Error posting user preferences", reason)
        } finally {
            if (reload) {
                window.location.reload();
            }

        }

        return success;

    }

    requestLogout = false;

    serverSettingsChanged: boolean = false;

    private data: GetUserResponse = { id: "", name: "", enableExperimentalFeatures: false, alwaysTagPreflightCL: false, claims: [], pinnedJobIds: [], dashboardSettings: { preferences: new Map() } };

    private updateTimeoutId: any = undefined;

    private updating = false;

    private polling = false;

    private canceled = new Set<number>();
    private cancelId = 0;

    private pollMS = 4 * 1000;

    private config?: GetDashboardConfigResponse;

    alertSquelch?: string;

}

const dashboard = new Dashboard();

export default dashboard;

