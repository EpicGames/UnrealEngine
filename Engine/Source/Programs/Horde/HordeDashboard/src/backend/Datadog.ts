// Copyright Epic Games, Inc. All Rights Reserved.

import { datadogLogs, StatusType } from '@datadog/browser-logs';
import { getSiteConfig } from '../backend/Config';

export const initDatadog = () => {

	const siteConfig = getSiteConfig();

	if (!siteConfig.datadogClient) {
		return;
	}	

	datadogLogs.init({
		clientToken: siteConfig.datadogClient,
		//datacenter: Datacenter.US,
		env: siteConfig.environment === "production" ? "prod" : "dev",
		forwardErrorsToLogs: true,
		sampleRate: 100

	});

	datadogLogs.logger.setLevel(StatusType.error);
	datadogLogs.addLoggerGlobalContext('hordedashboard', { env: siteConfig.environment, username: siteConfig.environment === 'dev-local' ? 'local.developer' : ""});	

};

export const setDatadogUser = (username: string) => {

	const siteConfig = getSiteConfig();

	if (!siteConfig.datadogClient) {
		return;
	}	

	datadogLogs.addLoggerGlobalContext('hordedashboard', { username: username, env: siteConfig.environment });
};



