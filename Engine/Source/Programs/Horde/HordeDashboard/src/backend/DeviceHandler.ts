// Copyright Epic Games, Inc. All Rights Reserved.

import moment from "moment";
import backend from "../backend";
import { GetDevicePlatformResponse, GetDevicePoolResponse, GetDeviceReservationResponse, GetDevicePoolTelemetryResponse, GetDevicePlatformTelemetryResponse, GetDeviceResponse, GetUserResponse, DevicePoolTelemetryQuery } from "../backend/Api";
import dashboard from "../backend/Dashboard";
import { PollBase } from "../backend/PollBase";

// in sort order
export enum DeviceStatus {
   Available = 0,
   Reserved = 1,
   Maintenance = 2,
   Problem = 3,
   Disabled = 4
}

export const DeviceStatusLabels = new Map<DeviceStatus, string>([
   [DeviceStatus.Available, "Available"],
   [DeviceStatus.Problem, "Problem"],
   [DeviceStatus.Reserved, "Reserved"],
   [DeviceStatus.Disabled, "Disabled"],
   [DeviceStatus.Maintenance, "Maintenance"]
]);

export enum DeviceHealthStatus {
   Unknown = 0,
   Healthy = 1,
   Unreliable = 2,
   Problematic = 3
}

export const DeviceHealthLabels: Map<DeviceHealthStatus, string> = new Map<DeviceHealthStatus, string>([
   [DeviceHealthStatus.Healthy, 'Healthy'],
   [DeviceHealthStatus.Unreliable, 'Unreliable'],
   [DeviceHealthStatus.Problematic, 'Problematic'],
]);

type DeviceTelemetry = { reservations: number, available: number, problems: number, disabled: number, deviceId: string };
type DeviceUsageTimeSerie = { data: DeviceTelemetry[], date: Date };
export type FilteredDeviceTelemetry = { reservations: number, available: number, problems: number, disabled: number, date: Date };
type DeviceHealth = { reservations: number, problems: number, isAvailable: boolean };

export class DeviceHandler extends PollBase {

   constructor(pollTime = 5000) {
      super(pollTime);
   }

   clear() {
      super.stop();
      this.loaded = false;
      this.devices = [];
      this.reservations = [];
      this.pools.clear();
      this.platforms.clear();
      this.poolTelemetry = [];
      this.poolTelemetryLastUpdate = undefined;
      this.deviceHealth.clear();
   }

   async poll(): Promise<void> {

      try {

         this.devices = await backend.getDevices();
         this.reservations = await backend.getDeviceReservations();

         // Get telemetry from the last 3 days using a different frequency than the poll frequency
         if (!this.poolTelemetryLastUpdate || Date.now() - this.poolTelemetryLastUpdate > this.poolTelemetryUpdateFrequency)
         {
            this.poolTelemetryLastUpdate = Date.now();
            const date = moment().subtract(this.telemetryHoursRange, 'hours');
            const deviceQuery: DevicePoolTelemetryQuery = { minCreateTime: date.toISOString(), count: 65536 };
            this.processTelemetry(await backend.getDevicePoolTelemetry(deviceQuery));
         }

         // filter out utilization to entries which contain job id (legacy reservations don't)
         this.devices.forEach(d => d.utilization = d.utilization?.filter(u => !!u.jobId));

         const userIds = new Set<string>();
         this.devices.forEach(d => {
            if (d.checkedOutByUserId && !this.users.has(d.checkedOutByUserId)) {
               userIds.add(d.checkedOutByUserId);
            }
         });

         if (!this.users.get(dashboard.userId)) {
            userIds.add(dashboard.userId);
         }

         if (userIds.size) {
            const users = await backend.getUsers({ ids: Array.from(userIds) });
            users.forEach(u => this.users.set(u.id, u));
         }

         if (!this.devices.length || this.devices.find(d => !this.platforms.has(d.platformId))) {
            // we need to get platforms
            const platforms = await backend.getDevicePlatforms();
            this.platforms.clear();
            platforms.forEach(p => {
               this.platforms.set(p.id, p);
            });

            const pools = await backend.getDevicePools();
            this.pools.clear();
            pools.forEach(p => {
               this.pools.set(p.id, p);
            });

         }

         const uniqueTags = new Set<string>(); 
         this.devices.forEach(device => device.tags?.forEach(tag => uniqueTags.add(tag)))
         this.tags = Array.from(uniqueTags);
         
         this.loaded = true;         
         this.setUpdated();

      } catch (err) {

      }

   }

   getUserDeviceCheckouts(userId: string): GetDeviceResponse[] {
      return this.devices.filter(d => d.checkedOutByUserId === userId);
   }

   getReservation(device: GetDeviceResponse): GetDeviceReservationResponse | undefined {
      return this.reservations.find(r => !!r.devices.find(d => d === device.id));
   }

   getDeviceWriteAccess(device: GetDeviceResponse): boolean {

      const pool = this.pools.get(device.poolId);

      if (!pool) {
         return false;
      }

      return pool.writeAccess;

   }

   getDeviceStatus(device: GetDeviceResponse): DeviceStatus {

      if (this.getReservation(device)) {
         return DeviceStatus.Reserved;
      }

      if (!device.enabled) {
         return DeviceStatus.Disabled;
      }

      if (device.maintenanceTime) {
         return DeviceStatus.Maintenance;
      }

      if (device.cleanStartTime && !device.cleanFinishTime) {
         return DeviceStatus.Maintenance;
      }

      if (device.problemTime) {

         const end = moment.utc();
         const d = moment.duration(end.diff(moment(device.problemTime)));

         // note this must match reservation selection for problem time in backend
         if (d.asMinutes() < dashboard.deviceProblemCooldownMinutes) {
            return DeviceStatus.Problem;
         }
      }

      return DeviceStatus.Available;
   }

   getDevices(platformId?: string) {
      return this.devices.filter(d => !platformId || d.platformId === platformId);
   }

   processTelemetry(telemetryData: GetDevicePoolTelemetryResponse[]) {
      const timeSeries: DeviceUsageTimeSerie[] = [];
      const healthAggregates: Map<string, DeviceHealth> = new Map();
      const getHealthDevice = (id) => {
         let deviceHealth = healthAggregates.get(id);
         if (!deviceHealth) {
            deviceHealth = {
               reservations: 0, problems: 0,
               isAvailable: true // contiguous reservation flag
            };
            healthAggregates.set(id, deviceHealth);
         }
         return deviceHealth!;
      }

      const cleanedData = telemetryData.map(t => {
         const date = new Date(t.createTimeUtc);
         return { date: date, pools: t.telemetry }
      }).sort((a, b) => a.date.getTime() - b.date.getTime());

      cleanedData.forEach(d => {
         const tserie: DeviceUsageTimeSerie = { data: [], date: d.date };
         timeSeries.push(tserie);
         Object.values(d.pools).forEach(telemetry => {
            telemetry.forEach(platformData => {
               const devices: Map<string, DeviceTelemetry> = new Map();
               const getDevice = (id) => {
                  let tdata = devices.get(id);
                  if (!tdata) {
                     tdata = {
                        reservations: 0,
                        available: 0,
                        problems: 0,
                        disabled: 0,
                        deviceId: id
                     };
                     tserie.data.push(tdata);
                  }
                  return tdata!;
               }
               // update device health aggregates
               Object.values(platformData.reserved ?? 0).forEach(stream => {
                  stream.forEach(d => {
                     const deviceHealth = getHealthDevice(d.deviceId);
                     if (deviceHealth.isAvailable) {
                        deviceHealth.reservations += 1;
                        deviceHealth.isAvailable = false;
                     }
                     const device = getDevice(d.deviceId);
                     device.reservations += 1;
                  });
               });
               if (platformData.problem) {
                  platformData.problem.forEach(deviceId => {
                     const deviceHealth = getHealthDevice(deviceId);
                     deviceHealth.problems += 1;
                     deviceHealth.isAvailable = true; // reset contiguous reservation flag
                     const device = getDevice(deviceId);
                     device.problems += 1;
                  });
               }
               if (platformData.available) {
                  platformData.available.forEach(deviceId => {
                     const deviceHealth = getHealthDevice(deviceId);
                     deviceHealth.isAvailable = true;
                     const device = getDevice(deviceId);
                     device.available += 1;
                  });
               }
            });
         });
      });

      this.deviceHealth = new Map();
      healthAggregates.forEach((value, key, map) => {
         this.deviceHealth.set(key, this.computeDeviceHealth(value));
      });

      this.poolTelemetry = timeSeries;
   }

   getDevicesTelemetry(devices?: string[]) {
      if (!devices) {
         return undefined;
      }

      return this.poolTelemetry.map(telemetry => {
         const tdata: FilteredDeviceTelemetry = {
            reservations: 0,
            available: 0,
            problems: 0,
            disabled: 0,
            date: telemetry.date
         };

         return telemetry.data.filter(
               t => devices.includes(t.deviceId)
            ).reduce((acc, t) => {
               acc.reservations += t.reservations;
               acc.available += t.available;
               acc.problems += t.problems;
               acc.disabled += t.disabled;
               return acc;
            }, tdata);
      });
   }

   computeDeviceHealth(health: DeviceHealth) {
      if (health.reservations === 0 && health.problems === 0 ) {
         return DeviceHealthStatus.Unknown;
      }
      const issueRate = (health.problems / (health.reservations + health.problems)) * 100;
      if (issueRate > 60) return DeviceHealthStatus.Problematic;
      if (issueRate > 40) return DeviceHealthStatus.Unreliable;
      return DeviceHealthStatus.Healthy;
   }

   getDeviceHealth(deviceId: string) {
      return this.deviceHealth.get(deviceId) ?? DeviceHealthStatus.Unknown;
   }

   loaded = false;

   tags: string[] = [];

   private devices: GetDeviceResponse[] = [];

   platforms: Map<string, GetDevicePlatformResponse> = new Map();

   pools: Map<string, GetDevicePoolResponse> = new Map();

   users: Map<string, GetUserResponse> = new Map();

   private reservations: GetDeviceReservationResponse[] = [];

   private poolTelemetry: DeviceUsageTimeSerie[] = [];
   poolTelemetryLastUpdate?: number;
   private poolTelemetryUpdateFrequency = 5 * 60 * 1000; // 5 minutes
   private deviceHealth: Map<string, DeviceHealthStatus> = new Map();

   telemetryHoursRange = 72;

}
