// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import moment from "moment";
import backend from "../backend";
import { projectStore } from "../backend/ProjectStore";
import { DevicePoolTelemetryQuery, DevicePoolType, DeviceTelemetryQuery, GetDevicePlatformResponse, GetDevicePlatformTelemetryResponse, GetDevicePoolResponse, GetDevicePoolTelemetryResponse, GetDeviceResponse, GetDeviceTelemetryResponse, GetTelemetryInfoResponse } from "../backend/Api";
import { msecToElapsed } from "../base/utilities/timeUtils";

type PoolId = string;

type PoolTelemetryData = {
   date: Date;
   dateString: string;
   pools: Record<PoolId, GetDevicePlatformTelemetryResponse[]>;
};

export type PlatformTelemetry = GetDevicePlatformTelemetryResponse & {
   date: Date;
   dateString: string;
};

type DeviceTelemetry = GetTelemetryInfoResponse & {
   createTime: Date;
   reservationStart?: Date;
   reservationFinish?: Date;
   problemTime?: Date;
}

export type UsagePlatformTelemetry = { reservations: number, date: Date }

export type TelemetryData = {
   pool: GetDevicePoolResponse;
   platform: GetDevicePlatformResponse;
   streamIds: string[];
   // stream id => full stream name
   streamNames: Record<string, string>;
   // stream full stream name => stream id
   streamNamesReverse: Record<string, string>;
   // stream name => valid selector
   streamSelectors: Record<string, string>;
   // selector => stream name
   streamSelectorsReverse: Record<string, string>;
   deviceIds: string[];
   devices: Record<string, GetDeviceResponse>;
   telemetry: PlatformTelemetry[];
}

type StepMetrics = {
   numKits: number;
   numProblems: number;
   duration: string;
}

type ProblemStep = {
   streamId?: string;
   jobId?: string;
   jobName?: string;
   stepId?: string;
   stepName?: string;
}

export type DeviceProblem = { deviceId: string, deviceName: string, poolName: string, problems: number, problemsRate: number, reservations: number, problemsDesc: string, latestProblem?: DeviceTelemetry, reservable: boolean };

export class PoolTelemetryHandler {

   constructor() {
      makeObservable(this);
      this.set();
   }

   getProblemSteps(deviceIds: string[], time: Date): Record<string, ProblemStep> {
      const problems: Record<string, ProblemStep> = {};

      deviceIds.forEach(id => {
         const telemetry = this.deviceTelemetry.get(id);
         if (!telemetry) {
            return;
         }

         let best: DeviceTelemetry | undefined;
         let bestMS = Number.MAX_VALUE;

         telemetry.forEach(t => {

            if (!t.problemTime) {
               return;
            }

            const ctime = Math.abs(t.problemTime.getTime() - time.getTime());

            if (ctime > 60 * 1000 * 30) {
               return;
            }

            if (bestMS > ctime) {
               best = t;
               bestMS = ctime;
            }

         });

         if (!best) {
            return;
         }

         problems[id] = {
            streamId: best.streamId,
            jobId: best.jobId,
            jobName: best.jobName,
            stepId: best.stepId,
            stepName: best.stepName
         };

      });

      return problems;
   }

   getSteps(platformId: string, startDate: Date | undefined, endDate: Date | undefined) {

      const stepData: Map<string, { stream: string, pool: string, stepName: string, reservations: number, problems: number, durationMS: number }> = new Map();

      if(!!startDate && !!endDate) {
         // reorder if inverted
         if (startDate > endDate) {
            const tempDate = endDate;
            endDate = startDate;
            startDate = tempDate;
         }
      }

      this.deviceTelemetry.forEach((telemetry, deviceId) => {

         const device = this.devices.get(deviceId);
         if (!device || device.platformId !== platformId) {
            return;
         }

         const pool = this.pools.get(device.poolId)!.name ?? "Unknown Pool"

         telemetry.forEach(t => {

            if (!!t.reservationStart) {
               if (!!startDate && t.reservationStart < startDate) {
                  return;
               }

               if (!!endDate && t.reservationStart > endDate) {
                  return;
               }
            }

            const stepName = t.stepName ?? "Unknown Step";
            const stream = projectStore.streamById(t.streamId)?.fullname ?? `${t.streamId}`;
            const key = stepName + stream + pool;

            if (!stepData.has(key)) {
               stepData.set(key, { stream: stream, pool: pool, stepName: stepName, reservations: 0, problems: 0, durationMS: 0 });
            }

            const step = stepData.get(key)!;

            if (t.problemTimeUtc) {
               step.problems++;
            } else {
               step.reservations++;
               if (t.reservationStart && t.reservationFinish) {
                  step.durationMS += (t.reservationFinish.getTime() - t.reservationStart.getTime());
               }
            }
         });
      });

      return stepData;
   }

   getPlatformTelemetry(platformId: string) {
      if (!this.data) {
         return [];
      }
      const timeSeries: UsagePlatformTelemetry[] = [];

      this.data.forEach(d => {
         const tdata: UsagePlatformTelemetry = { reservations: 0, date: d.date };
         Object.keys(d.pools).map(key => d.pools[key]).forEach(p => {
            let platformData: GetDevicePlatformTelemetryResponse | undefined;
            platformData = p.find(pd => pd.platformId === platformId);
            if (!platformData || !platformData.reserved) {
               return;
            }
            // platform 'reserved' property is a map of streams, each item contains an array of the reserved devices
            tdata.reservations += Object.values(platformData.reserved).reduce((acc, item) => acc + item.length, 0);
         });
         timeSeries.push(tdata);
      });

      return timeSeries
   }

   getProblemDevices() {

      const platforms: Map<string, DeviceProblem[]> = new Map();

      this.deviceTelemetry.forEach((telemetry, deviceId) => {

         const device = this.devices.get(deviceId);
         if (!device) {
            return;
         }

         const platform = this.platforms.get(device.platformId);
         if (!platform) {
            return;
         }

         let count = 0;
         let latest: DeviceTelemetry | undefined;
         telemetry.filter(t => t.problemTime).forEach(t => {
            count++;
            if (!latest || latest.problemTime! < t.problemTime!) {
               latest = t;
            }

            if (t.jobName?.indexOf("- Kicked By") !== -1) {
               t.jobName = t.jobName?.split("- Kicked By")[0];
            }

         });

         if (!count) {
            return;
         }

         if (!platforms.has(platform.id)) {
            platforms.set(platform.id, []);
         }

         const poolName = this.pools.get(device.poolId)?.name ?? "No Pool";
         const ratePercent = Math.floor(count / telemetry.length * 100);
         const reservable = device.enabled && !device.maintenanceTime;

         platforms.get(platform.id)!.push({ deviceId: deviceId, deviceName: device.name, poolName: poolName, problems: count, problemsRate: ratePercent, reservations: telemetry.length, problemsDesc: `${count} / ${telemetry.length} (${ratePercent}%)`, latestProblem: latest, reservable: reservable });

      });

      platforms.forEach((devices, key) => {
         devices = devices.sort((a, b) => b.problems - a.problems);
         platforms.set(key, devices);
      });

      return platforms;

   }

   getStepMetrics(jobId: string, stepId: string): StepMetrics | undefined {

      const telemetry: DeviceTelemetry[] = [];

      const devices = new Set<string>();

      this.deviceTelemetry.forEach((t, id) => {

         const stepTelemetry = t.filter(td => { return (td.jobId === jobId) && (td.stepId === stepId) });

         if (stepTelemetry.length) {
            devices.add(id);
            telemetry.push(...stepTelemetry);
         }

      });

      if (!telemetry.length) {
         return undefined;
      }

      let minDate: Date | undefined;
      let maxDate: Date | undefined;

      telemetry.forEach(t => {

         if (t.reservationStart) {
            if (!minDate || minDate > t.reservationStart) {
               minDate = t.reservationStart;
            }
         }

         if (t.reservationFinish) {
            if (!maxDate || maxDate < t.reservationFinish) {
               maxDate = t.reservationFinish;
            }
         }
      });

      let duration = "";
      if (minDate && maxDate) {
         duration = msecToElapsed(maxDate.getTime() - minDate.getTime(), true, false);
      }

      let numProblems = 0;

      telemetry.forEach(t => {
         if (t.problemTime) {
            numProblems++;
         }
      });

      return {
         numKits: devices.size,
         numProblems: numProblems,
         duration: duration
      }
   }

   async set() {

      const requests: any = [];

      // get 2 weeks worth
      const date = moment().subtract(14, 'days');

      const poolQuery: DevicePoolTelemetryQuery = { minCreateTime: date.toISOString(), count: 65536 };
      const deviceQuery: DeviceTelemetryQuery = { minCreateTime: date.toISOString(), count: 65536 };

      requests.push(backend.getDevicePlatforms());
      requests.push(backend.getDevicePools());
      requests.push(backend.getDevices());
      requests.push(backend.getDevicePoolTelemetry(poolQuery));
      requests.push(backend.getDeviceTelemetry(deviceQuery));

      const responses = await Promise.all(requests);

      const platforms = responses[0] as GetDevicePlatformResponse[];
      const pools = responses[1] as GetDevicePoolResponse[];
      const devices = responses[2] as GetDeviceResponse[];
      const telemetry = responses[3] as GetDevicePoolTelemetryResponse[];
      const deviceTelemetry = responses[4] as GetDeviceTelemetryResponse[];

      deviceTelemetry.forEach(d => {
         const telemetry: DeviceTelemetry[] = [];
         d.telemetry.forEach(t => {

            const t2: DeviceTelemetry = { ...t, createTime: new Date(t.createTimeUtc) };

            if (t.reservationStartUtc) {
               t2.reservationStart = new Date(t.reservationStartUtc);
            }

            if (t.problemTimeUtc) {
               t2.problemTime = new Date(t.problemTimeUtc);
            }

            if (t.reservationFinishUtc) {
               t2.reservationFinish = new Date(t.reservationFinishUtc);
            }

            telemetry.push(t2);
         });

         this.deviceTelemetry.set(d.deviceId, telemetry);
      });

      for (const platform of platforms) {
         this.platforms.set(platform.id, platform);
      }

      for (const pool of pools) {
         if (pool.poolType !== DevicePoolType.Automation) {
            continue;
         }
         this.pools.set(pool.id, pool);
      }

      for (const device of devices) {
         this.devices.set(device.id, device);
      }

      // patch up telemetry data 
      telemetry.forEach(t => {
         for (const poolId in t.telemetry) {
            const values = t.telemetry[poolId];
            values.forEach(v => {

               if (!v.reserved) {
                  return;
               }

               for (const rkey in v.reserved) {
                  const rd = v.reserved[rkey];
                  rd.forEach(rdv => {

                     if (!rdv.jobName) {
                        rdv.jobName = `Unknown Job in stream ${rkey}, ${rdv.jobId}`;
                     }
                     if (!rdv.stepName) {
                        rdv.stepName = `Unknown Step in stream ${rkey}`;
                     }

                     if (rdv.jobName.indexOf("- Kicked By") !== -1) {
                        rdv.jobName = rdv.jobName.split("- Kicked By")[0];
                     }
                  })
               }
            });
         };
      });

      this.data = telemetry.map(t => {
         const date = new Date(t.createTimeUtc);
         return { date: date, dateString: date.toDateString(), pools: t.telemetry }
      }).sort((a, b) => a.date.getTime() - b.date.getTime());

      this.setUpdated();

   }

   @action
   setUpdated() {
      this.updated++;
   }

   getData(poolId: string, platformId: string): TelemetryData | undefined {

      if (!this.data) {
         return undefined;
      }

      const pool = this.pools.get(poolId);
      const platform = this.platforms.get(platformId);

      if (!pool || !platform) {
         return undefined;
      }

      const telemetry: PlatformTelemetry[] = [];
      const streamIds = new Set<string>();
      const deviceIds = new Set<string>();

      const streamNames: Record<string, string> = {};
      const streamNamesReverse: Record<string, string> = {};
      const streamSelectors: Record<string, string> = {};
      const streamSelectorsReverse: Record<string, string> = {};

      this.data.forEach(d => {

         const poolData = d.pools[poolId];

         let platformData: GetDevicePlatformTelemetryResponse | undefined;

         if (poolData) {
            platformData = poolData.find(pd => pd.platformId === platformId);
         }

         if (!platformData) {
            return;
         }

         const tdata = { ...platformData, date: d.date, dateString: d.dateString } as any;

         tdata["Available"] = platformData.available?.length ?? 0;
         tdata["Disabled"] = platformData.disabled?.length ?? 0;
         tdata["Maintenance"] = platformData.maintenance?.length ?? 0;
         tdata["Problem"] = platformData.problem?.length ?? 0;

         platformData.available?.forEach(d => { deviceIds.add(d); });
         platformData.maintenance?.forEach(d => { deviceIds.add(d); });
         platformData.disabled?.forEach(d => { deviceIds.add(d); });
         platformData.problem?.forEach(d => { deviceIds.add(d); });

         if (platformData.reserved) {
            for (const streamId in platformData.reserved) {
               if (!streamIds.has(streamId)) {

                  streamIds.add(streamId);

                  streamNames[streamId] = streamId;
                  const stream = projectStore.streamById(streamId);
                  if (stream) {
                     streamNames[streamId] = stream.fullname ?? streamId;
                  }

                  const name = streamNames[streamId];
                  streamNamesReverse[name] = streamId;

                  streamSelectors[name] = name.replaceAll(".", "_").replaceAll("/", "_");
                  streamSelectorsReverse[streamSelectors[name]] = name;
               }

               // make stream data accessible by id and fullname
               tdata[streamId] = platformData.reserved[streamId].length;
               tdata[streamNames[streamId]] = platformData.reserved[streamId].length;

               platformData.reserved[streamId].forEach(d => { deviceIds.add(d.deviceId); })
            }
         }

         telemetry.push(tdata);
      });

      const deviceIdArray = Array.from(deviceIds).sort((a, b) => a.localeCompare(b));
      const deviceResponses = deviceIdArray.map(id => this.devices.get(id)).filter(d => !!d);
      const devices: Record<string, GetDeviceResponse> = {};

      deviceResponses.forEach(d => devices[d!.id] = d!);

      const streamIdArray = Array.from(streamIds).sort((a, b) => a.localeCompare(b))

      for (const streamId of streamIdArray) {
         telemetry.forEach((t: any) => {
            if (!t[streamId]) {
               t[streamId] = 0;
               t[streamNames[streamId]] = 0;
            }
         });
      }

      return {
         pool: pool,
         platform: platform,
         streamIds: streamIdArray,
         streamNames: streamNames,
         streamNamesReverse: streamNamesReverse,
         streamSelectors: streamSelectors,
         streamSelectorsReverse: streamSelectorsReverse,
         telemetry: telemetry,
         deviceIds: deviceIdArray,
         devices: devices
      }
   }

   get loaded(): boolean { return this.data ? true : false; }

   @observable
   updated: number = 0;

   data?: PoolTelemetryData[];

   platforms = new Map<string, GetDevicePlatformResponse>();
   pools = new Map<string, GetDevicePoolResponse>();
   devices = new Map<string, GetDeviceResponse>();

   deviceTelemetry = new Map<string, DeviceTelemetry[]>();
}
