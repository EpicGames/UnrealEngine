// Copyright Epic Games, Inc. All Rights Reserved.

import { errorDialogStore, ErrorInfo, errorBarStore, ErrorReportType } from "../components/error/ErrorStore";
import { GetDashboardChallengeResponse } from "./Api";
import userInactivity from "./UserInactivity";
//import { setDatadogUser } from './Datadog';

export enum ChallengeStatus {
    Ok,
    Unauthorized,
    Error
}

export interface FetchRequestConfig {
    formData?: boolean;
    responseBlob?: boolean;
    params?: Record<string, string | string[] | number | boolean | undefined>;
    // does not report 404 error
    suppress404?: boolean;
    // shows error dialog for 404, in addition to logging it
    show404Error?: boolean;
    signal?: AbortSignal;
}

export type FetchResponse = {
    data: any;
    status: number;
}

function handleError(
    errorIn: ErrorInfo,
    force?: boolean,
    reportType?: ErrorReportType,
    retryCallback?: () => boolean | Promise<boolean>
) {

    const e = { ...errorIn, hordeError: true } as ErrorInfo;

    switch (reportType) {
        case ErrorReportType.Dialog:
            errorDialogStore.set(e, !!force, retryCallback);
            break;
        case ErrorReportType.Bar:
            errorBarStore.set(e, !!force, retryCallback);
            break;
        default:
            errorBarStore.set(e, !!force, retryCallback);
            break;
    }
}

/** Fetch convenience wrapper */
export class Fetch {

    get(url: string, config?: FetchRequestConfig) {

        url = this.buildUrl(url, config);

        if (this.checkDebugMockStatus(url)) {
            return Promise.reject(new Error("Vite endpoint status debugging"))
        }

        return new Promise<FetchResponse>(async (resolve, reject) => {

            try {

                const response = await fetch(url, this.buildRequest("GET", undefined, config)).then(response => {
                    // Internal Server Error Response
                    if (!response.ok && response.status === 500) {
                        response.text().then(text => {
                            handleError({
                                response: response,
                                reason: "Internal Server Error",
                                mode: "GET",
                                url: url,
                                message: text
                            }, true, ErrorReportType.Dialog);
                        });
                        return null;
                    }

                    // 404 Not Found Response
                    if (!response.ok && response.status === 404 && config?.show404Error) {
                        response.json().then(o => {
                            handleError({
                                response: response,
                                title: "404 Not Found",
                                reason: "404 Not Found",
                                mode: "GET",
                                url: url,
                                message: o?.message ?? "Malformed json on response object"
                            }, true, ErrorReportType.Dialog);
                        }).catch(reason => {
                            console.error("Unable to parse response json on 404: ", reason)
                        });
                        return null;
                    }

                    // 401 Unauthorized Response - No Credentials
                    if (!response.ok && response.status === 401) {
                        handleError({
                            response,
                            title: "401 Unauthorized",
                            reason: "401 Unauthorized",
                            mode: "GET",
                            url,
                            message: "You don't have access because you are not logged in. Try logging in again."
                        }, true)
                        return null;
                    }

                    // 403 Forbidden Response - Insufficient Permissions
                    if (!response.ok && response.status === 403) {
                        handleError({
                            response,
                            title: "403 Forbidden",
                            reason: "403 Forbidden",
                            mode: "GET",
                            url,
                            message: "You don't have access to this page or some of its features"
                        }, true)
                        return null;
                    }

                    return response;
                });

                if (!response) {
                    reject('');
                    return;
                }

                if (response.ok) {
                    this.handleResponse(response, url, "GET", resolve, reject, config);
                    return;
                }

                if (response.status === 404) {
                    if (config?.suppress404) {
                        reject(`Received suppressed 404 on ${url}`);
                        return;
                    }
                }

                const challenge = await this.challenge();

                if (challenge === ChallengeStatus.Unauthorized) {

                    this.login(window.location.toString());
                    return;
                }

                if (response.status === 502) {
                    throw new Error(`502 Gateway error connecting to server`);
                }

                throw response.statusText ? response.statusText : response.status;

            } catch (reason) {

                // Note: the fetch request sets error for redirects, this is opaque as per the spec
                // so all we get is an error string and can't detect AccessDenied, etc in redirect :/
                let message = `Error in request, ${reason}`;

                handleError({
                    reason: message,
                    mode: "GET",
                    url: url
                });

                reject(message);
            };
        });
    }

    // DEBUG code, used to test response status handling.
    checkDebugMockStatus(url: string): boolean {
        if (!import.meta.env.VITE_HORDE_DEBUG_MOCK_STATUS_ENDPOINTS) {
            // No debug settings found, return false to continue the calling function
            return false;
        }

        const mockStatusEndpoints: Record<string, [number, ErrorReportType]> = {};
        // Parse the vite mock status envrionment variable into a record map-like object
        import.meta.env.VITE_HORDE_DEBUG_MOCK_STATUS_ENDPOINTS
            .split(",")
            .forEach((triplet, index) => {
                try {
                    const [endpointRaw, statusRaw, reportTypeRaw] = triplet.split(":");

                    // Endpoint parsing
                    if (!endpointRaw) {
                        throw new Error(`Missing endpoint in triplet "${triplet}" at index ${index}`);
                    }
                    const endpoint = endpointRaw.trim();

                    // Status parsing
                    let status = 401;
                    if (statusRaw) {
                        const parsedStatus = Number(statusRaw.trim());
                        if (isNaN(parsedStatus) || parsedStatus < 100 || parsedStatus > 599) {
                            console.warn(`Invalid status "${statusRaw}" in triplet "${triplet}" at index ${index}. Defaulting to status 401.`);
                        }
                        else {
                            status = parsedStatus;
                        }
                    }
                    else {
                        console.warn(`No status provided in triplet "${triplet}" at index ${index}. Defaulting to status 401. If this was unintended, please use the following format: "endpoint:status:reportType".`);
                    }

                    // Report type parsing
                    let reportType = ErrorReportType.Bar;
                    if (reportTypeRaw) {
                        const parsedReportType = ErrorReportType[reportTypeRaw.trim() as keyof ErrorReportType];
                        if (!parsedReportType) {
                            console.warn(`Invalid report type "${reportTypeRaw}" in triplet "${triplet}" at index ${index}. Defaulting to ErrorReportType "Bar".`);
                        }
                        else {
                            reportType = parsedReportType;
                        }
                    }
                    else {
                        console.warn(`No report type provided in triplet "${triplet}" at index ${index}. Defaulting to ErrorReportType "Bar". If this was unintended, please use the following format: "endpoint:status:reportType".`);
                    }

                    mockStatusEndpoints[endpoint] = [status, reportType];
                } catch (e) {
                    console.error("Issue with VITE Debug Env Var: VITE_HORDE_DEBUG_MOCK_STATUS_ENDPOINTS", e)
                }
            }); // No need for super careful list construction

        // Determine if any entries match the current url
        const match = Object.entries(mockStatusEndpoints).find(([endpoint]) => url.includes(endpoint));
        // If a match is found, simulate a bad response with the given status and mock an error with the given report type
        if (match) {
            const [endpoint, [status, reportType]] = match;

            const response = new Response(JSON.stringify({ error: `Error ${status} (Mock)` }), {
                status,
                headers: { "Content-Type": "application/json" }
            });

            if (!response.ok) {
                handleError({
                    response,
                    title: `Error ${status}`,
                    reason: "Endpoint Status Debugging",
                    mode: "GET",
                    url,
                }, true, reportType);

                // A match was found, return true to return from the calling function early
                return true;
            }
        }

        // A match was not found, return false to continue calling function
        return false;
    }

    post(url: string, data?: any, config?: FetchRequestConfig) {

        url = this.buildUrl(url, config);

        return new Promise<FetchResponse>((resolve, reject) => {

            fetch(url, this.buildRequest("POST", data, config)).then(response => {

                this.handleResponse(response, url, "POST", resolve, reject, config);

            }).catch(reason => {

                // Note: the fetch request sets error for redirects, this is opaque as per the spec
                // so all we get is an error string and can't detect AccessDenied, etc in redirect :/
                let message = `Possible permission issue, ${reason}`;

                handleError({
                    reason: message,
                    mode: "POST",
                    url: url
                }, true);

                reject(message);
            });
        });

    }

    put(url: string, data?: any, config?: FetchRequestConfig) {

        url = this.buildUrl(url, config);

        return new Promise<FetchResponse>((resolve, reject) => {

            fetch(url, this.buildRequest("PUT", data, config)).then(response => {

                this.handleResponse(response, url, "PUT", resolve, reject, config);

            }).catch(reason => {

                handleError({
                    reason: reason,
                    mode: "PUT",
                    url: url
                }, true);

                reject(reason);
            });
        });

    }

    patch(url: string, data?: any, config?: FetchRequestConfig) {

        url = this.buildUrl(url, config);

        return new Promise<FetchResponse>((resolve, reject) => {

            fetch(url, this.buildRequest("PATCH", data, config)).then(response => {

                this.handleResponse(response, url, "PATCH", resolve, reject, config);

            }).catch(reason => {

                handleError({
                    reason: reason,
                    mode: "PATCH",
                    url: url
                }, true);

                reject(reason);
            });
        });

    }


    delete(url: string, config?: FetchRequestConfig) {

        url = this.buildUrl(url, config);

        return new Promise<FetchResponse>((resolve, reject) => {

            fetch(url, this.buildRequest("DELETE", undefined, config)).then(response => {

                this.handleResponse(response, url, "DELETE", resolve, reject, config);

            }).catch(reason => {

                handleError({
                    reason: reason,
                    mode: "DELETE",
                    url: url
                }, true);

                reject(reason);
            });
        });
    }

    login(redirect?: string) {

        if (this.debugToken || this.logout) {
            return;
        }

        window.location.assign("/api/v2/dashboard/login?redirect=" + btoa(redirect ?? "/index"));
    }

    private async handleResponse(response: Response, url: string, mode: string, resolve: (value: FetchResponse | PromiseLike<FetchResponse>) => void, reject: (reason?: any) => void, config?: FetchRequestConfig) {

        if (!response.ok && response.status === 500) {

            response.text().then(text => {
                handleError({
                    mode: mode,
                    response: response,
                    url: url,
                    title: "Internal Server Error",
                    message: text
                }, true, ErrorReportType.Dialog);
            });
            return reject("Internal Server Error");
        }

        if (response.status === 401) {

            return reject(response.statusText);
        }

        if (!response.ok) {

            let message = response.statusText;

            if (response.url?.indexOf("AccessDenied") !== -1) {

                handleError({
                    mode: mode,
                    response: response,
                    url: url,
                    title: "Access Denied"
                }, true, ErrorReportType.Dialog);

            } else {

                if (response.status !== 404 || !config?.suppress404) {

                    let errorInfo: ErrorInfo = {
                        mode: mode,
                        response: response,
                        url: url
                    }

                    let json: any = undefined;

                    try {
                        json = await response.json();
                    } catch {

                    }

                    // dynamic detection of horde formatted error
                    if (json && json.time && json.message && json.level) {
                        errorInfo.format = json;
                        message = json.message;
                        if (json.id) {
                            message = `(Error ${json.id}) - ${message}`;
                        }
                    }

                    handleError(errorInfo);

                }
            }

            reject(message);
            return;
        }

        if (config?.responseBlob) {
            response.blob().then(blob => {
                resolve({
                    data: blob,
                    status: response.status
                });
            }).catch(reason => {

                handleError({
                    mode: mode,
                    response: response,
                    url: url,
                    reason: reason
                });

                reject(reason);
            });
        }
        else {
            response.clone().json().then(data => {
                resolve({
                    data: data,
                    status: response.status
                });
            }).catch((reason) => {
                response.clone().text().then(text => {
                    resolve({
                        data: text,
                        status: response.status
                    });
                }).catch(reason => {

                    handleError({
                        mode: mode,
                        response: response,
                        url: url,
                        reason: reason
                    });

                    reject(reason);
                });
            });
        }

    }

    private buildRequest(method: "GET" | "POST" | "PUT" | "DELETE" | "PATCH", data?: any, config?: FetchRequestConfig): RequestInit {

        const headers = this.buildHeaders();

        let body: any = undefined;

        if (method === "POST" || method === "PUT" || method === "PATCH") {
            if (data) {
                if (config?.formData) {
                    body = data;
                } else {
                    headers['Content-Type'] = 'application/json';
                    body = JSON.stringify(data);
                }
            }
        }

        // no-cors doesn't allow authorization header

        const requestInit: RequestInit = {
            method: method,
            mode: "cors",
            cache: "no-cache",
            credentials: this.debugToken ? "same-origin" : 'include',
            redirect: "error",
            headers: headers,
            body: body,
            signal: config?.signal
        };

        return requestInit;
    }

    private get withCredentials(): boolean {
        return !!this.authorization && !!this.authorization.length;
    }

    private buildHeaders(): Record<string, string> {

        const headers: Record<string, string> = {};

        if (this.withCredentials) {
            headers["Authorization"] = this.authorization!;
        }

        // Report how idle the user has been on current page in each HTTP request
        // This helps identify how much load potentially unused pages generate on the Horde server
        headers["X-Horde-LastUserActivity"] = userInactivity.getSecondsSinceLastActivity().toString();

        return headers;
    }


    private buildUrl(url: string, config?: FetchRequestConfig): string {

        while (url.startsWith("/")) {
            url = url.slice(1);
        }

        while (url.endsWith("/")) {
            url = url.slice(0, url.length - 1);
        }

        url = `${this.baseUrl}/${url}`;

        if (!config?.params) {
            return url;
        }

        const keys = Object.keys(config.params).filter((key) => {
            return config.params![key] !== undefined;
        });

        const query = keys.map((key) => {
            const value = config.params![key]!;
            if (Array.isArray(value)) {
                return (value as string[]).map(v => {
                    return encodeURIComponent(key) + '=' + encodeURIComponent(v);
                });
            } else {
                return encodeURIComponent(key) + '=' + encodeURIComponent(value as any);
            }

        }).flat().join('&');

        return query.length ? `${url}?${query}` : url;

    }

    async challenge(): Promise<ChallengeStatus> {

        if (this.debugToken) {
            return ChallengeStatus.Ok;
        }

        try {
            const url = this.buildUrl("/api/v1/dashboard/challenge");

            const result = await fetch(url, this.buildRequest("GET"));
            const response = await result.json() as GetDashboardChallengeResponse;

            if (response.needsFirstTimeSetup) {
                window.location.assign("/setup");
                // give the window assignment a couple seconds, so we don't continue down challenge route
                await new Promise(r => setTimeout(r, 2000));
                return ChallengeStatus.Ok;
            }

            if (!response.needsAuthorization) {
                return ChallengeStatus.Ok;
            }

            return ChallengeStatus.Unauthorized;

        } catch (reason) {
            console.error(reason);
        }

        return ChallengeStatus.Error;

    }

    // debug token
    private setAuthorization(authorization: string | undefined) {
        this.authorization = authorization;
    }

    setDebugToken(token?: string) {

        if (!token) {

            this.debugToken = undefined;
            this.setAuthorization(undefined);
            return;
        }

        this.debugToken = token;
        this.setAuthorization(`Bearer ${this.debugToken}`);

    }

    setBaseUrl(url?: string) {

        if (!url) {
            this.baseUrl = "";
            return;
        }

        this.baseUrl = url;
    }

    logout: boolean = false;

    private baseUrl = "";
    private debugToken?: string;
    private authorization?: string;

}