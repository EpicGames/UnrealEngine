// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import { IssueData } from './Api';
import backend from '.';

const updateMS = 5000;

// This class is intended for high level, cross job stream issue views, for job specific issues see JobDetails
class IssueStore {

    constructor() {
        makeObservable(this);
    }

    @observable
    issuesUpdated: number = 0;

    get issues(): IssueData[] {
        // subscribe in any observers
        if (this.issuesUpdated) { }
        return this._issues;
    }
        
    private _issues: IssueData[] = [];

    getIssues(jobId: string, stepId: string): IssueData[] {
        // @todo: refactor for new issue API
        return [];
    }

    setStream(streamId: string) {

        if (this.streamId === streamId) {
            return;
        }

        this.streamId = streamId;

        if (this._issues.length) {
            this.setData([]);
        }

        clearTimeout(this.timeoutID);
        this.timeoutID = undefined;
        this.updating = false;

        if (this.streamId) {
            this.update();
        }
    }

    @action
    private setData(data: IssueData[]) {
        // @todo: do a deep compare here to avoid redundant updates
        this._issues = data;
        this.issuesUpdated++;
    }

    private update(): void {

        if (!this.streamId) {
            return;
        }

        clearTimeout(this.timeoutID);
        this.timeoutID = setTimeout(() => { this.update(); }, updateMS);

        if (this.updating) {
            return;
        }

        this.updating = true;

        const updateStream = this.streamId;

        new Promise<void>(() => {
            backend.getIssues().then(data => {
                if (this.streamId !== updateStream) {
                    return;
                }
                this.setData(data);
            }).catch(reason => {
                console.error(`Error getting issues: ${reason}`);
            }).finally(() => {
                this.updating = false;
            });
        });
    }

    private streamId = '';

    private timeoutID?: any;

    private updating = false;
}

export const issueStore = new IssueStore();