// Copyright Epic Games, Inc. All Rights Reserved.

import { GetIssueResponse, GetLogEventResponse, IssueSeverity } from "./Api";

enum IssueType {
    Unknown,
    Compiler
}

const getCompilerIssueSummary = (issue: GetIssueResponse, events: GetLogEventResponse[]): string => {

    let summary = issue.summary;

    let sourceFiles: string[] = [];

    events.forEach(event => {

        event.lines?.forEach(line => {

            let props = line.properties;

            if (props && props["file"]) {

                const file = props["file"] as Record<string, string>;

                if (file["type"] === "SourceFile") {

                    let relativePath = file["relativePath"];

                    if (relativePath) {

                        const sourceFile = relativePath.replace(/^.*[\\]/, '');
                        if (!sourceFiles.find(s => s === sourceFile)) {
                            sourceFiles.push(sourceFile);
                        }
                    }
                }
            }
        })
    })

    if (!sourceFiles.length) {
        return summary;
    }

    summary = (issue.severity === IssueSeverity.Warning) ? "Compile warnings" : "Compile errors";

    if (sourceFiles.length === 1) {
        return summary.slice(0, summary.length - 1) + " in " + sourceFiles[0];
    }

    if (sourceFiles.length === 2) {
        return summary + " in " + sourceFiles[0] + " and " + sourceFiles[1];
    }

    return `${summary} in ${sourceFiles[0]} and ${sourceFiles.length - 1} others`;

}

export const getIssueSummary = (issue?: GetIssueResponse, events?: GetLogEventResponse[]): string => {

    if (!issue) {
        return "Issue is undefined";
    }

    let summary = issue.summary;

    let type = IssueType.Unknown;

    // @todo: would be better to have type meta data in issue
    if (summary.startsWith("Compile warning") || summary.startsWith("Compile error")) {
        type = IssueType.Compiler;    
    }

    if (type === IssueType.Unknown || !summary || !events || !events.length) {
        return summary ? summary : "Issue has no summary or details";
    }

    if (type === IssueType.Compiler) {
        return getCompilerIssueSummary(issue, events);
    }

    return "";
}