// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import moment from 'moment';
import backend from '../backend';
import { JobData, JobState, JobStepOutcome, JobStreamQuery, StreamData } from '../backend/Api';
import { CommitCache } from './CommitCache';

export type FilterStatus = "Running" | "Complete" | "Succeeded" | "Failed" | "Cancelled" | "Waiting";

const jobPageSize = 50;
const jobsRefreshTime = 10000;

export class JobHandler {

    constructor(includeBatches: boolean = false, jobLimit?: number, commitCache?: CommitCache) {
        makeObservable(this);
        this.includeBatches = includeBatches;
        this.jobLimit = jobLimit;
        this.commitCache = commitCache;
    }


    filter(stream: StreamData, templateNamesIn?: string[], preflightStartedByUserId?: string): boolean {

        const names = templateNamesIn ?? [];
        const sameNames = (this.templateNames.length === names.length && this.templateNames.every((value, index) => value === names[index]));

        if (stream.id === this.stream?.id && sameNames && this.preflightStartedByUserId === preflightStartedByUserId) {
            return false;
        }

        this.clear();
        this.stream = stream;
        this.templateNames = names;
        this.preflightStartedByUserId = preflightStartedByUserId;
        this.setUpdated();
        this.update();
        return true;
    }

    clear() {
        clearTimeout(this.timeoutId);
        this.timeoutId = undefined;
        this.modifiedAfter = undefined;
        this.stream = undefined;
        this.templateNames = [];
        this.jobs = [];
        this.initial = true;
        this.bumpCount = false;
        this.count = jobPageSize * 2;
        this.includePreflights = true;
        this.preflightStartedByUserId = undefined;

        // cancel any pending        
        for (let i = 0; i < this.cancelId; i++) {
            this.canceled.add(i);
        }
    }

    async update() {

        clearTimeout(this.timeoutId);
        this.timeoutId = setTimeout(() => { this.update(); }, jobsRefreshTime);

        if (this.updating) {
            return;
        }

        this.updating = true;

        // cancel any pending        
        for (let i = 0; i < this.cancelId; i++) {
            this.canceled.add(i);
        }

        let wasUpdated = false;

        try {

            // discover new/updated jobs
            let filter = "id,streamId,name,change,preflightChange,preflightCommitId,templateId,templateHash,graphHash,startedByUserInfo,abortedByUserInfo,createTime,state,arguments,updateTime,labels,defaultLabel,preflightDescription,cancellationReason";

            if (this.includeBatches) {
                filter += ",batches";
            }

            // needs modified time
            const query: JobStreamQuery = {
                filter: filter,
                count: this.count,
                t: this.templateNames,
                includePreflight: this.includePreflights,
                preflightStartedByUserId: this.preflightStartedByUserId
            };

            // instance property can mutate externally as we're async
            const bumpCount = this.bumpCount;

            if (bumpCount) {

                this.modifiedAfter = undefined;

                let maxCreate: Date | undefined;

                // find max create time
                this.jobs.forEach(j => {
                    const time = new Date(j.createTime);
                    if (!maxCreate || (time < maxCreate)) {
                        maxCreate = time;
                    }
                })

                if (maxCreate) {
                    maxCreate = new Date(maxCreate.getTime() - 1);
                    query.maxCreateTime = maxCreate.toISOString();
                    query.count = jobPageSize;
                }

            } else {

                query.modifiedAfter = this.modifiedAfter;

            }

            const cancelId = this.cancelId++;

            const mjobs = await backend.getStreamJobs(this.stream!.id, query);

            // check for canceled after modified test
            if (this.canceled.has(cancelId)) {
                return;
            }

            if (bumpCount) {

                let njob = false;
                mjobs.forEach(j => {
                    if (!this.jobs.find(j2 => j.id === j2.id)) {
                        njob = true;
                    }
                })

                // add page size if we found any new jobs
                if (njob) {
                    this.count += jobPageSize
                }

            }

            const jobs: JobData[] = [];

            mjobs.forEach(j1 => {
                jobs.push(j1);
            })

            jobs.push(...this.jobs.filter(j1 => !jobs.find(j2 => j1.id === j2.id)));

            this.jobs = jobs.sort((a, b) => {
                const timeA = new Date(a.createTime).getTime();
                const timeB = new Date(b.createTime).getTime();
                if (timeA === timeB) return 0;
                return timeA < timeB ? 1 : -1;
            });


            // check for canceled during graph request
            if (this.canceled.has(cancelId)) {
                return;
            }

            if (mjobs.length || this.initial) {
                this.initial = false;
                wasUpdated = true;
            }

            // Populate commit cache with change numbers from jobs
            if (this.commitCache && this.stream) {
                const changes = [...new Set(this.jobs.map(j => j.change).filter(change => !!change))] as number[];
                await this.commitCache.set(this.stream.id, changes);
            }

        } catch (reason) {
            console.log(reason);
        } finally {

            this.bumpCount = false;

            if (this.jobLimit) {

                if (this.count === this.jobs.length && this.count < this.jobLimit) {
                    this.bumpCount = true;
                }
            }

            if (this.jobs.length) {
                let job = this.jobs[0];
                this.jobs.forEach(j => {
                    const date1 = new Date(job.updateTime);
                    const date2 = new Date(j.updateTime);
                    if (date2.getTime() > date1.getTime()) {
                        job = j;
                    }
                });

                this.modifiedAfter = moment(job.updateTime).add(1, 'milliseconds').toDate().toISOString();
            } else {
                this.modifiedAfter = undefined;
            }


            if (wasUpdated) {
                this.setUpdated();
            }

            this.updating = false;
        }

    }

    getFilteredJobs(status?: FilterStatus[], minCL?: number, maxCL?: number) {
        if ((!status || !status.length) && !minCL && !maxCL) {
            return this.jobs;
        }

        return this.jobs.filter(job => {
            if((minCL || maxCL) && !job.change) return false;
            
            let withinBounds = true;
            if((minCL && job.change! < minCL) || (maxCL && job.change! > maxCL)) withinBounds = false;
            if((minCL || maxCL) && !withinBounds) return false;

            if(!status || !status.length) return true;

            return status.some(status => {

                if ((status === "Complete" || status === "Succeeded") && job.state !== JobState.Complete) {
                    return false;
                }

                if (status === "Running" && job.state !== JobState.Running) {
                    return false;
                }

                if (status === "Waiting" && job.state !== JobState.Waiting) {
                    return false;
                }

                if (status === "Succeeded") {
                    if (!job.batches?.every(b => b.steps.every(s => s.outcome === JobStepOutcome.Success || s.outcome === JobStepOutcome.Warnings))) {
                        return false;
                    }
                }

                if (status === "Failed") {
                    if (job.batches?.every(b => b.steps.every(s => s.outcome === JobStepOutcome.Success || s.outcome === JobStepOutcome.Warnings))) {
                        return false;
                    }
                }

                if(status === "Cancelled" && !job.abortedByUserInfo) {
                    return false;
                }

                return true;
            })

        });
    }


    @action
    setUpdated() {
        this.updated++;
    }

    @observable
    updated: number = 0;

    updating = false;

    jobs: JobData[] = [];

    addJobs() {

        if (this.count < 1000) {
            this.bumpCount = true;
        }

    }

    private modifiedAfter?: string;
    private stream?: StreamData;
    private templateNames: string[] = [];

    count = jobPageSize * 2;
    bumpCount = false;

    private timeoutId: any;

    private canceled = new Set<number>();
    private cancelId = 0;

    includeBatches = false;

    includePreflights = true;
    private preflightStartedByUserId?: string;

    private jobLimit?: number;
    private commitCache?: CommitCache;

    initial = true;

}
