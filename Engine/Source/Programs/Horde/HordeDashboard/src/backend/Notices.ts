// Copyright Epic Games, Inc. All Rights Reserved.

import moment from "moment";
import backend from ".";
import { displayTimeZone } from "../base/utilities/timeUtils";
import { GetNoticeResponse } from "./Api";
import dashboard from "./Dashboard";
import { PollBase } from "./PollBase";


class Notices extends PollBase {

    constructor(pollTime = 1000) {
        super(pollTime);
    }

    clear() {
        super.stop();
    }

    async poll(): Promise<void> {


        try {

            if (!dashboard.available) {
                return;
            }

            this.notices = await backend.getNotices();

            this.pollTime = 60 * 1000;

            this.setUpdated();

        } catch (err) {

        }

    }

    get alertText(): string | undefined {
        
        // first one created by a user
        let notice = this.notices.find(n => (n.active && !!n.createdByUser));
        if (!notice) {
            notice = this.notices.find(n => n.active);
        }

        if (!notice) {
            return undefined;
        }

        if (notice.scheduledDowntime) {            

            let time = "";
            if (notice.finishTime) {
                time = moment(notice.finishTime as Date).tz(displayTimeZone()).format(dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z");
                return `Horde is currently in scheduled downtime. Jobs will resume execution at ${time}`;
            }
                    
            return `Horde is currently in scheduled downtime`
        }

        return notice?.message;

    }


    get allNotices(): GetNoticeResponse[] {

        return this.notices ?? [];
    
    }

    private notices: GetNoticeResponse[] = [];

}

const notices = new Notices();
notices.start();

export default notices;
