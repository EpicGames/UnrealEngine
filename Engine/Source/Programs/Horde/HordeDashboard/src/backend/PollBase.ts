// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';

// base class for polling behavior
export abstract class PollBase {

    constructor(pollTime = 5000) {
        makeObservable(this);
        this.pollTime = pollTime;
    }

    stop() {

        clearTimeout(this.timeoutId);
        this.timeoutId = undefined;

        // cancel any pending        
        for (let i = 0; i < this.cancelId; i++) {
            this.canceled.add(i);
        }

        this.updating = false;
    }

    start() {

        if (this.timeoutId || this.updating) {
            return;
        }

        this.update();
    }

    abstract poll(): Promise<void>;

    async forceUpdate() {        
        this.stop(); 
        this.update();
    }

    async update() {

        clearTimeout(this.timeoutId);
        this.timeoutId = setTimeout(() => { this.update(); }, this.pollTime);

        if (this.updating) {
            return;
        }

        this.updating = true;

        // cancel any pending        
        for (let i = 0; i < this.cancelId; i++) {
            this.canceled.add(i);
        }

        try {

            const cancelId = this.cancelId++;

            await this.poll();

            // check for canceled during graph request
            if (this.canceled.has(cancelId)) {
                return;
            }


        } catch (reason) {

            console.log(reason);

        } finally {

            this.updating = false;
        }

    }



    @action
    setUpdated() {
        this.updated++;
    }

    @observable
    updated: number = 0;

    subscribe() {
        if (this.updated) { }
    }

    updating = false;

    protected timeoutId: any;

    protected pollTime: number;

    private canceled = new Set<number>();
    private cancelId = 0;

}