// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import { ProjectData, StreamData } from './Api';
import backend from '.';

export class ProjectStore {

    constructor() {
        makeObservable(this);
    }

    @observable
    projectsUpdated: number = 0;

    get projects(): ProjectData[] {
        // subscribe in any observers
        if (this.projectsUpdated) { }
        return this._projects;
    }

    @observable
    activeId?: string;

    @action
    setProjects(data: ProjectData[]) {
        // @todo: compare to avoid update
        this._projects = data;
        this.projectsUpdated++;
    }

    @action
    setActive(projectId: string | undefined): ProjectData | undefined {
        if (this.activeId === projectId) {
            return this.byId(this.activeId);
        }

        this.activeId = projectId;
        return this.byId(this.activeId);
    }

    getActive(): ProjectData | undefined {
        return this._projects.find(p => p.id === this.activeId);
    }

    byId(id: string | undefined): ProjectData | undefined {
        return this._projects.find(p => p.id === id);
    }

    byName(name: string | undefined): ProjectData | undefined {
        return this._projects.find(p => p.name === name);
    }

    firstProject(): ProjectData | undefined {
        return this._projects.length ? this._projects[0] : undefined;
    }

    projectByStreamId(streamId: string): ProjectData | undefined {
        return this._projects.find(p => p.streams?.find(stream => stream.id === streamId))
    }

    streamById(id: string | undefined): StreamData | undefined {
        if (!id) {
            return undefined;
        }
        let stream: StreamData | undefined;
        this._projects.forEach(p => {
            if (stream) {
                return;
            }
            stream = p.streams?.find(s => s.id === id);

        });
        return stream;
    }

    streamByFullname(name: string): StreamData | undefined {

        let stream: StreamData | undefined;

        this._projects.forEach(p => {
            if (stream) {
                return;
            }
            stream = p.streams?.find(s => s.fullname?.toLowerCase() === name.toLowerCase());

        });
        return stream;
    }

    streamByAliasName(aliasName: string): StreamData | undefined {

        let stream: StreamData | undefined;

        this._projects.forEach(p => {
            if (stream) {
                return;
            }
            stream = p.streams?.find(s => s.aliases?.find(a => a.name.toLowerCase() === aliasName.toLowerCase()));

        });
        return stream;
    }


    streamByName(project: ProjectData, streamName: string | undefined): StreamData | undefined {
        if (!project || !project.streams) {
            return undefined;
        }
        return project.streams.find(s => s.name.toLowerCase() === streamName?.toLowerCase());
    }

    firstStream(project: ProjectData): StreamData | undefined {
        if (!project.streams || !project.streams.length) {
            return undefined;
        }
        return project.streams[0];
    }

    async update(): Promise<void> {
        return new Promise<void>((resolve, reject) => {
            backend.getProjects().then(projects => {

                this.setProjects(projects);
                this.setActive(this.byId(this.activeId)?.id);

                resolve();
            }).catch(reason => {
                reject(`Error getting projects: ${reason}`);
            });
        });
    }


    private _projects: ProjectData[] = [];

}

export const projectStore = new ProjectStore();