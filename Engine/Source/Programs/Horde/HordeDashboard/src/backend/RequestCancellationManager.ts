
// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Cancellation reason when a subsequent request to the same @see AbortController has occurred.
 */
export const REASON_REENTRANT_REQUEST = "REENTRANT_REQUEST";

/**
 * Lightweight class to help with managing cancellations for various endpoints.
 */
export class RequestCancellationManager {

    private controllers: Map<string, AbortController> = new Map<string, AbortController>();

    /**
     * Obtains a new cancellation signal for API requests, cancelling any before that with the @see REASON_REENTRANT_REQUEST reason.
     * @param identifier The identifier associated with the @see AbortController
     * @returns The signal of the active abort controller assigned to the identifier.
     */
    getSignal(identifier: string): AbortSignal {
        this.controllers.get(identifier)?.abort(REASON_REENTRANT_REQUEST);
        let newController: AbortController = new AbortController();
        this.controllers.set(identifier, newController);

        return newController.signal;
    }

    /**
     * Cancels an existing @see AbortController given the identifier, should it exist.
     * @param identifier The identifier associated with the @see AbortController
     * @param reason The reason to issue the cancel.
     */
    cancel(identifier: string, reason?: string): void {
        this.controllers.get(identifier)?.abort(reason);
        this.controllers.delete(identifier);
    }

    /**
     * Cancels all managed @see AbortController .
     * @param reason The reason to issue a cancel all.
     */
    cancelAll(reason?: string): void {
        this.controllers.forEach((value: AbortController) => {
            value.abort(reason);
        });
        this.controllers.clear();
    }
}