// Copyright Epic Games, Inc. All Rights Reserved.

import { GetTemplateRefResponse, StreamData } from './Api';

class TemplateCache {

	getStreamTemplates(stream: StreamData): Promise<GetTemplateRefResponse[]> {

		// if this seems odd, it is, part of cleanup to solely use GetTemplateRefResponse and effectively ignpre legacy GetTemplateResponse
		return new Promise<GetTemplateRefResponse[]>(resolve => {
			resolve(stream.templates ?? []);
		});
	}

	initialize() {
	}
}

const templateCache = new TemplateCache();
export default templateCache;