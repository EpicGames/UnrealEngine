// Copyright Epic Games, Inc. All Rights Reserved.

import { observable, action, when, makeObservable } from "mobx";
import backend from '../backend';
import { JobData, TestData } from './Api';

export type DataWrapper = {
    postPropertiesUpdate(): void
    isAllPropertiesLoaded(): boolean
    getMissingProperties(): string[]
}

const updateObject = (target: any, version: any, skipKeys?: string[]) => {
    const properties = Object.getOwnPropertyDescriptors(version);
    Object.getOwnPropertyNames(version).forEach((key) => {
        if (!properties[key].value || skipKeys?.includes(key)) {
            delete properties[key];
        }
    });
    return Object.defineProperties(target, properties);
}

export class TestDataWrapper implements TestData {
    get id(): string { return this.getTestDataProperty('id'); }
    get key(): string { return this.getTestDataProperty('key'); }
    get change(): number { return this.getTestDataProperty('change'); }
    get jobId(): string { return this.getTestDataProperty('jobId'); }
    get stepId(): string { return this.getTestDataProperty('stepId'); }
    get streamId(): string { return this.getTestDataProperty('streamId'); }
    get templateRefId(): string { return this.getTestDataProperty('templateRefId'); }
    get data(): any { return this.getTestDataProperty('data'); }

    get type(): string { return this.key.split('::', 2)[0]; }

    private artifactV2Map: Map<string, object | undefined> = new Map();
    private artifactV2Id?: string;
    private jobdata?: JobData;
    private stepName?: string;

    private _testdata: TestData;
    private _datahandler?: DataWrapper;

    constructor(item: TestData) {
        this._testdata = item;
    }

    setDataHandler(handler: DataWrapper) {
        if (this.data) {
            this._datahandler = handler;
        }
    }

    getDataHandler(): DataWrapper | undefined {
        return this._datahandler;
    }

    getTestDataProperty<T>(key: string): T {
        return Object.getOwnPropertyDescriptor(this._testdata, key)?.value as T;
    }

    updateProperties(update: TestData) {
        updateObject(this._testdata, update, ['data']);
        if (update.data) {
            if (!this._testdata.data) {
                this._testdata.data = update.data;
            } else {
                updateObject(this._testdata.data, update.data);
            }
            this._datahandler?.postPropertiesUpdate();
        }
    }

    isPropertiesDefined(...properties: string[]) {
        const checks = properties.map((key) => {
            let obj = this._testdata;
            let tailKey = key;
            let headKey = key;
            while (tailKey.indexOf('.') > 0) {
                [headKey, tailKey] = tailKey.split('.', 2);
                const nextProp = Object.getOwnPropertyDescriptor(obj, headKey);
                if (!nextProp) {
                    return false;
                }
                obj = nextProp?.value;
            }
            return !!Object.getOwnPropertyDescriptor(obj, tailKey);
        });

        return !checks.find((check) => !check);
    }

    async getArtifacts() {

        if (this.artifactV2Id) {
            return;
        }

        if (!this.jobdata) {
            this.jobdata = await backend.getJob(this.jobId);
        }

        const v = await backend.getJobArtifactsV2(undefined, [`job:${this.jobId}/step:${this.stepId}`]);
        const av2 = v?.artifacts.find(a => a.type === "step-saved")
        if (av2?.id) {
            this.artifactV2Id = av2.id;
        } else {
            console.error("Unable to get step-saved artifacts v2 for report");
        }

    }

    async getArtifactImageLink(referencePath: string) {

        await this.getArtifacts();

        return `${backend.serverUrl}/api/v2/artifacts/${this.artifactV2Id}/file?path=Engine/Programs/AutomationTool/Saved/Logs/RunUnreal/${encodeURIComponent(referencePath)}&inline=true`;

    }

    async findArtifactData(artifactName: string) {

        if (!artifactName) {
            return undefined;
        }

        await this.getArtifacts();

        artifactName = artifactName.replace(/\\/g, '/');

        if (this.artifactV2Id) {

            if (this.artifactV2Id) {
                const path = `Engine/Programs/AutomationTool/Saved/Logs/RunUnreal/${encodeURIComponent(artifactName)}`;
                let result = this.artifactV2Map.get(path);
                if (result) {
                    return result;
                }
                const data = await backend.getArtifactV2(this.artifactV2Id, path);
                this.artifactV2Map.set(path, result);
                return data;
            }
        }

        return undefined;
        
    }

    async getJobStepName() {
        if (this.stepName) {
            return this.stepName;
        }
        if (!this.jobdata) {
            this.jobdata = await backend.getJob(this.jobId);
        }

        let stepName = "";
        const batch = this.jobdata.batches?.find(b => b.steps.find(s => s.id === this.stepId));
        const stepNode = batch?.steps.find(s => s.id === this.stepId);
        stepName = stepNode?.name ?? "";

        return this.stepName;
    }

    getJobStepLink() {
        return `/job/${this.jobId}?step=${this.stepId}`;
    }

    async getJobName() {
        if (!this.jobdata) {
            this.jobdata = await backend.getJob(this.jobId);
        }

        return this.jobdata.name;
    }

    getJobLink() {
        return `/job/${this.jobId}`;
    }

    async preflighChange() {
        if (!this.jobdata) {
            this.jobdata = await backend.getJob(this.jobId);
        }

        return this.jobdata.preflightChange;
    }

    async getChangeName() {
        if (!this.jobdata) {
            this.jobdata = await backend.getJob(this.jobId);
        }

        const data = this.jobdata;

        let changeName = "";
        if (data.preflightChange) {
            changeName = `Preflight ${data.preflightChange} `;
            changeName += ` - Base ${data.change ? data.change : "Latest"}`;

        } else {
            changeName = `${data.change ? "CL " + data.change : "Latest CL"}`;
        }

        return changeName;
    }
}

export type OnFetchTestData = (newChunk: (TestDataWrapper | undefined)[]) => void;
export type SelectorTestData = (item: TestDataWrapper) => boolean;

export class TestDataCollection {

    constructor() {
        makeObservable(this);
    }

    activated = false;
    setActive(activated: boolean) {
        this.activated = activated;
    }

    // loadingNotice is used to mark discreet items that are being fetched.
    // That is to avoid doing multiple fetch requests of the same items.
    @observable
    loadingNotice: Set<string> = new Set();
    @action
    acquireLoadingNotice(key: string) {
        this.loadingNotice.add(key);
    }
    @action
    releaseLoadingNotice(key: string) {
        this.loadingNotice.delete(key);
    }
    @action
    releaseAllLoadingNotice() {
        this.loadingNotice.clear();
    }
    isLoading(key: string) {
        return this.loadingNotice.has(key);
    }

    key?: string;
    streamId?: string;
    items?: Map<string, TestDataWrapper>;
    itemsByChange?: Map<number, TestDataWrapper[]>;
    cursor?: TestDataWrapper;
    private nextIndex: number = 0;
    private fetchCount: number = 20;
    private fetchMaxCount: number = 1000;

    desactivate() {
        this.setActive(false);
        this.releaseAllLoadingNotice();
    }

    findById(id: string) {
        if (!this.items) {
            return undefined;
        }
        return this.items.get(id);
    }

    forEach(func: (item: TestDataWrapper) => void) {
        if (!this.items) {
            return;
        }
        this.items.forEach(func);
    }

    *iterItems() {
        const items = this.items;
        if (items) {
            const iterator = items.values();
            let result = iterator.next();
            while (!result.done) {
                yield result.value;
                result = iterator.next();
            }
        }
    }

    *iterItemsByChange() {
        if (this.itemsByChange) {
            const changes = this.getChanges();
            for (const change of changes) {
                const items = this.itemsByChange.get(change);
                if (items) {
                    yield items;
                }
            }
        }
    }

    getChanges() {
        if (this.itemsByChange) {
            const changes = Array.from(this.itemsByChange.keys());
            changes.sort((a, b) => a > b ? -1 : (a < b ? 1 : 0)); // in reversed order
            return changes;
        }
        return [];
    }

    async setFromStream(streamId: string, key: string, onFetch?: OnFetchTestData, filter?: string, maxCount?: number) {
        this.setActive(true);

        if (key === this.key && streamId === this.streamId && !this.cursor) {
            // reprocessed what was loaded already, and then continue loading if max count was not reached
            if (this.items) {
                if (onFetch) {
                    onFetch(Array.from(this.items.values()));
                }
                if (this.items.size % this.fetchCount !== 0 || this.items.size >= this.fetchMaxCount) {
                    return;
                }
            }
        } else {
            this.cleanCache();
            this.key = key;
            this.streamId = streamId;
        }

        await this.queueFetchItems(onFetch, filter, maxCount);
    }

    async setFromId(id: string, onFetch?: OnFetchTestData, filter?: string) {
        this.setActive(true);

        if (this.items) {
            const item = this.items.get(id);
            if (item) {
                const handler = item.getDataHandler();
                if (handler) {
                    if (!handler.isAllPropertiesLoaded()) {
                        const missing = handler.getMissingProperties();
                        const filter = missing.map((item) => `data.${item}`).join(',');
                        const count = await this.fetchUpdateItem(id, undefined, filter);
                        if (count === 0) {
                            return undefined;
                        }
                    }
                }
                this.cursor = item;
                return item;
            }
        }

        // pre fetch minimal data to get the context and see if we need to clean the cache
        const item = await backend.getTestData(id, "nodata");
        if (!item) {
            return undefined;
        }
        if (item.key !== this.key || item.streamId !== this.streamId) {
            this.cleanCache();
            this.key = item.key;
            this.streamId = item.streamId;
        }

        // actually fetching the item
        const count = await this.fetchUpdateItem(id, onFetch, filter);
        if (count === 1) {
            this.cursor = this.items?.get(id);
            return this.cursor;
        }

        return undefined;
    }

    async getCursorHistory(onFetch?: OnFetchTestData, filter?: string, maxCount?: number, selector?: SelectorTestData) {
        if (!this.cursor) {
            return [];
        }

        await this.fetchCursorHistory(onFetch, filter, maxCount, selector);

        const mapper = selector ? (items: TestDataWrapper[]) => items.find(selector) as TestDataWrapper : (items: TestDataWrapper[]) => items[0];

        return Array.from(this.iterItemsByChange()).map(mapper);
    }

    async fetchCursorHistory(onFetch?: OnFetchTestData, filter?: string, maxCount?: number, selector?: SelectorTestData) {
        if (!this.cursor) {
            return;
        }

        if (!this.itemsByChange || this.itemsByChange.size === 1) {
            const key = `cursorHistory-${this.key}`;
            if (this.isLoading(key)) {
                await when(() => !this.loadingNotice.has(key));
                return;
            }
            this.acquireLoadingNotice(key);
            await this.queueFetchItems(onFetch, filter, maxCount ?? this.fetchCount);
            this.releaseLoadingNotice(key);
        }
    }

    async queueFetchItems(onFetch?: OnFetchTestData, filter?: string, maxCount?: number) {
        if (maxCount) {
            this.fetchMaxCount = maxCount;
        }

        let totalCount = await this.fetchItems(onFetch, filter);
        let count = totalCount;
        while (this.activated && count > 0 && count % this.fetchCount === 0 && this.items && this.items.size < this.fetchMaxCount) {
            count = await this.fetchItems(onFetch, filter);
            totalCount += count;
        }

        return totalCount;
    }

    private cleanCache() {
        if (this.items) {
            this.items = undefined;
            this.itemsByChange = undefined;
            this.cursor = undefined;
            this.nextIndex = 0;
            this.fetchMaxCount = 1000; // the default
            this.releaseAllLoadingNotice();
        }
    }

    private initiateCache() {
        if (!this.items) {
            this.items = new Map();
            this.itemsByChange = new Map();
        }
    }

    private updateItemsByChange(item: TestDataWrapper) {
        if (!this.itemsByChange) {
            throw new Error("Trying to update itemsByChange before the object was initiated.")
        }
        const change = item.change;
        const changeCollection = this.itemsByChange.get(change);
        if (changeCollection) {
            changeCollection.push(item);
        } else {
            this.itemsByChange.set(change, [item])
        }
    }

    private async fetchItems(onFetch?: OnFetchTestData, filter?: string) {
        if (!this.key || !this.streamId) {
            return 0;
        }
        const key = this.key;
        const streamId = this.streamId;
        let count = 0;
        try {
            const pageData = await backend.getTestDataHistory(streamId, key, undefined, this.fetchCount, this.nextIndex, filter);
            this.nextIndex += this.fetchCount;

            // Are we still fetching for the same collection?
            if (key === this.key && streamId === this.streamId) {

                this.initiateCache();

                const wrappedData = pageData.map((item) => {
                    if (!this.items?.has(item.id)) {
                        const itemWrapped = new TestDataWrapper(item);
                        this.items?.set(item.id, itemWrapped);
                        this.updateItemsByChange(itemWrapped);
                        return itemWrapped;
                    }
                    return this.items?.get(item.id);
                });

                //process data
                if (onFetch) {
                    onFetch(wrappedData);
                }

                count = pageData.length;
            }

        } catch (reason) {
            console.error(`Failed to fetch TestData history from ${streamId} on key '${key}' !`);
            console.error(reason);
        }

        return count;
    }

    async fetchUpdateItems(ids: string[], onFetch?: OnFetchTestData, filter?: string, parallelCount: number = 30) {
        let count = 0;
        for (let i = 0; i < ids.length; i += parallelCount) {
            if (!this.activated) {
                break;
            }
            const chunk = ids.slice(i, i + parallelCount);
            count += (
                await Promise.all(chunk.map((id) => this.fetchUpdateItem(id, onFetch, filter)))
            ).reduce((a, b) => a + b);
        }
        return count;
    }

    async fetchUpdateItem(id: string, onFetch?: OnFetchTestData, filter?: string) {
        if (!this.key || !this.streamId) {
            return 0;
        }
        if (this.loadingNotice.has(id)) {
            await when(() => !this.loadingNotice.has(id));
            return this.activated ? 1 : 0;
        }
        const key = this.key;
        const streamId = this.streamId;
        let count = 0;
        this.acquireLoadingNotice(id);
        try {
            const update = await backend.getTestData(id, filter);

            // Are we still fetching for the same collection?
            if (key === this.key && streamId === this.streamId) {

                this.initiateCache();

                let item = this.items?.get(id);
                if (item) {
                    item.updateProperties(update);
                } else {
                    item = new TestDataWrapper(update);
                    this.items?.set(id, item);
                    this.updateItemsByChange(item);
                }

                //process data
                if (onFetch) {
                    onFetch([item]);
                }

                count = 1;
            }
        } catch (reason) {
            console.error(`Failed to fetch updated TestData item ${id} from ${streamId} on key '${key}' !`);
        } finally { this.releaseLoadingNotice(id) }

        return count;
    }

    async fetchItemsFromAlternateKey(altKey: string, maxChange?: number, count: number = 1, page: number = 0, filter?: string) {
        if (!this.key || !this.streamId) {
            return [];
        }
        if (this.loadingNotice.has(altKey)) {
            await when(() => !this.loadingNotice.has(altKey));
            return []; // We have no way to know what is the results... :/
        }

        const streamId = this.streamId;
        let results: TestData[] = [];

        this.acquireLoadingNotice(altKey);
        try {
            results = await backend.getTestDataHistory(streamId, altKey, maxChange, count, page, filter);
        } catch (reason) {
            console.error(`Failed to fetch Test Data for ${altKey} from ${streamId} !`);
        } finally { this.releaseLoadingNotice(altKey) }

        return results;

    }
}