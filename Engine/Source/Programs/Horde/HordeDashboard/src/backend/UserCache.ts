// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import backend from '.';
import { GetUserResponse } from './Api';


class UserCache {

    constructor() {
        makeObservable(this);
    }

    @action
    setUpdated() {
        this.updated++;
    }

    @observable
    updated: number = 0;

    async getUsers(partialMatch: string) {

        partialMatch = partialMatch.toLowerCase();

        let users = this.userMap.get(partialMatch);

        if (users !== undefined) {
            return users;
        }

        users = [];

        try {
            const regex = partialMatch.length === 1 ? `^${partialMatch}` : `${partialMatch}`;
            users = await backend.getUsers({ count: 128, includeAvatar: true, nameRegex: regex });
            console.log(`Found ${users.length} users with partial match: ${partialMatch}`);

            users = users.sort((a, b) => {
                const aname = a.name.toLowerCase();
                const bname = b.name.toLowerCase();

                if (aname < bname) return -1;
                if (aname > bname) return 1;
                return 0;
            })
            this.userMap.set(partialMatch, users);
            this.setUpdated();
        } catch (reason) {
            console.error("Unable to get users", reason);
        }

        return users;
    }

    private userMap: Map<string, GetUserResponse[]> = new Map();

}

const userCache = new UserCache()
export default userCache;