// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Tracks a timestamp for when the user last performed a UI activity
 */
class UserInactivity {
    private lastActivityTime: number;
    private readonly eventHandler: () => void;
    
    constructor() {
        this.lastActivityTime = Date.now();
        this.eventHandler = () => this.resetLastActivityTime();
    }

    start() {
        
        window.addEventListener('mousemove', this.eventHandler);
        window.addEventListener('mouseenter', this.eventHandler);
        window.addEventListener('mouseleave', this.eventHandler);
        window.addEventListener('touchstart', this.eventHandler);
        window.addEventListener('keydown', this.eventHandler);
        window.addEventListener('scroll', this.eventHandler);
        window.addEventListener('click', this.eventHandler);
        window.addEventListener('pointerdown', this.eventHandler);
    }

    public getSecondsSinceLastActivity(): number {
        return Math.floor((Date.now() - this.lastActivityTime) / 1000);
    }

    private resetLastActivityTime() {
        this.lastActivityTime = Date.now();
    }
}

const userInactivity = new UserInactivity()
userInactivity.start();

export default userInactivity;