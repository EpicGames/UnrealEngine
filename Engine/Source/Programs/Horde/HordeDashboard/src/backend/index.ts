// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { Backend } from './Backend';
import { projectStore } from './ProjectStore';
import { issueStore } from './IssueStore';

const backend = new Backend();
export default backend;

const backendContext = React.createContext({
  projectStore: projectStore,
  issueStore: issueStore  
});

export const useBackend = () => React.useContext(backendContext);
