// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import type MarkdownComponentType from 'markdown-to-jsx';
import type { MarkdownToJSX } from 'markdown-to-jsx';
import * as MarkdownModule from 'markdown-to-jsx';
import { IImageStyles, classNamesFunction, IStyleFunction, styled } from '@fluentui/react';
import { DefaultButton } from '@fluentui/react/lib/Button';
import * as MDTable from '../MarkdownTable/index';
import { MarkdownHeader } from './MarkdownHeader';
import { MarkdownParagraph } from './MarkdownParagraph';
import { IMarkdownProps, IMarkdownSubComponentStyles, IMarkdownStyleProps, IMarkdownStyles } from './Markdown.types';
import { MarkdownLink } from './MarkdownLink';
import { MarkdownPre } from './MarkdownPre';
import { PropsWithChildren } from 'react';
import dashboard from '../../../backend/Dashboard';
import { getHordeStyling } from '../../../styles/Styles';
import { MarkdownImage } from './MarkdownImage';

// This is to work around inconsistency between the way markdown-to-jsx declares its types
// (as having a default export) and the way it actually builds its files (for its cjs `main` file,
// assigning to module.exports and not setting a default export)
const MarkdownComponent: typeof MarkdownComponentType =
   // eslint-disable-next-line @typescript-eslint/no-explicit-any
   (MarkdownModule as any).default || (MarkdownModule as any);

const getStyles: IStyleFunction<IMarkdownStyleProps, IMarkdownStyles> = () => {

   const { modeColors } = getHordeStyling();

   const imageStyles: Partial<IImageStyles> = {
      root: {
         maxWidth: '100%',
         margin: '8px 0',
      },
   };

   return {
      root: {
         fontSize: "13px", fontFamily: "Horde Open Sans Regular", selectors: {
            'li': {
               margin: "4px 0px 8px 0px",
               lineHeight: "1.6"
            },
            'ul': {
               margin: "8px 0px 16px 0px"
            },
            'pre': {
               whiteSpace: "pre-wrap",
               padding: 16,
               backgroundColor: dashboard.darktheme ? modeColors.header : modeColors.crumbs
            },
            'p > code': {

               padding: "3px 6px !important"

            },
            'a > code': {
               
            },
            'td > a': {
               fontFamily: "Horde Cousine Regular",
               fontSize: "12px",
               backgroundColor: dashboard.darktheme ? "#25282A" : "#f2f2f2",
               padding: "3px 0px 2px 6px !important"

            },
            'td > code': {

               padding: "3px 6px !important"

            },
            'code': {
               whiteSpace: "pre-wrap",
               fontFamily: "Horde Cousine Regular",
               fontSize: "12px",
               backgroundColor: dashboard.darktheme ? "#25282A" : "#f2f2f2"
            }

         }
      },
      subComponentStyles: { image: imageStyles } as IMarkdownSubComponentStyles,
   };
};

const getClassNames = classNamesFunction<IMarkdownStyleProps, IMarkdownStyles>();

const MarkdownBase: React.FunctionComponent<PropsWithChildren<IMarkdownProps>> = props => {
   const { styles, theme, children } = props;
   const classNames = getClassNames(styles, { theme: theme! });

   return (
      <div className={classNames.root}>
         <MarkdownComponent options={{ overrides: getOverrides(classNames.subComponentStyles, props) }}>
            {children as string}
         </MarkdownComponent>
      </div>
   );
};
MarkdownBase.displayName = 'Markdown';

function getOverrides(subComponentStyles: IMarkdownSubComponentStyles, props: IMarkdownProps): MarkdownToJSX.Overrides {
   return {
      h1: {
         component: MarkdownHeader,
         props: { styles: subComponentStyles.header },
      },
      h2: {
         component: MarkdownHeader,
         props: { as: 'h2', styles: subComponentStyles.header },
      },
      h3: {
         component: MarkdownHeader,
         props: { as: 'h3', styles: subComponentStyles.header },
      },
      h4: {
         component: MarkdownHeader,
         props: { as: 'h4', styles: subComponentStyles.header },
      },
      h5: {
         component: MarkdownHeader,
         props: { as: 'h5', styles: subComponentStyles.header },
      },
      h6: {
         component: MarkdownHeader,
         props: { as: 'h6', styles: subComponentStyles.header },
      },
      p: {
         component: MarkdownParagraph,
         props: { styles: subComponentStyles.paragraph },
      },
      pre: {
         component: MarkdownPre,
         props: { enableRenderHtmlBlock: props.enableRenderHtmlBlock },
      },
      a: {
         component: MarkdownLink,
         props: { className: 'ms-mdLink', styles: subComponentStyles.link },
      },
      img: {
         component: MarkdownImage,
         props: { className: 'ms-mdImage', styles: subComponentStyles.image },
      },
      button: {
         component: DefaultButton,
         props: { className: 'ms-mdButton', styles: subComponentStyles.button },
      },
      table: {
         component: MDTable.MarkdownTable,
         props: { styles: subComponentStyles.table },
      },
      thead: {
         component: MDTable.MarkdownTHead,
         props: { styles: subComponentStyles.table },
      },
      tbody: {
         component: MDTable.MarkdownTBody,
         props: { styles: subComponentStyles.table },
      },
      tr: {
         component: MDTable.MarkdownTr,
         props: { styles: subComponentStyles.table },
      },
      th: {
         component: MDTable.MarkdownCell,
         props: { as: 'th', styles: subComponentStyles.table },
      },
      td: {
         component: MDTable.MarkdownCell,
         props: { as: 'td', styles: subComponentStyles.table },
      },
      ...props.overrides,
   };
}

export const Markdown: React.FunctionComponent<PropsWithChildren<IMarkdownProps>> = styled<
   IMarkdownProps,
   IMarkdownStyleProps,
   IMarkdownStyles
>(MarkdownBase, getStyles, undefined, { scope: 'Markdown' });
