// Copyright Epic Games, Inc. All Rights Reserved.

import type { ITheme, IStyleFunctionOrObject, IStyle, ILinkStyleProps, IImageStyleProps } from '@fluentui/react';
import type { IMarkdownHeaderStyleProps } from './MarkdownHeader';
import type { IMarkdownParagraphStyleProps } from './MarkdownParagraph';
import type { IMarkdownTableStyleProps } from '../MarkdownTable/index';
import type { MarkdownToJSX } from 'markdown-to-jsx';

export interface IMarkdownProps {
  /** CSS class to apply to the component root */
  className?: string;

  /**
   * If true, using a code block with language name `renderhtml` will render the contents as HTML.
   * This is to work around markdown-to-jsx's limited support for nested HTML elements.
   */
  enableRenderHtmlBlock?: boolean;

  /** Additional component overrides for markdown rendering */
  overrides?: MarkdownToJSX.Overrides;

  /** Theme provided by higher-order component. */
  theme?: ITheme;

  /** Optional override styles */
  styles?: IStyleFunctionOrObject<IMarkdownStyleProps, IMarkdownStyles>;
}

export type IMarkdownStyleProps = Required<Pick<IMarkdownProps, 'theme'>>;

export interface IMarkdownStyles {
  root: IStyle;
  subComponentStyles: IMarkdownSubComponentStyles;
}

export interface IMarkdownSubComponentStyles {
  // TODO: remove anys
  /* eslint-disable @typescript-eslint/no-explicit-any */
  header: IStyleFunctionOrObject<IMarkdownHeaderStyleProps, any>;
  paragraph: IStyleFunctionOrObject<IMarkdownParagraphStyleProps, any>;  
  link: IStyleFunctionOrObject<ILinkStyleProps, any>;
  image: IStyleFunctionOrObject<IImageStyleProps, any>;
  button: IStyleFunctionOrObject<any, any>;
  table: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
  thead: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
  tbody: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
  tr: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
  th: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
  td: IStyleFunctionOrObject<IMarkdownTableStyleProps, any>;
}
