// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { classNamesFunction, IStyleFunctionOrObject, styled, IStyleFunction } from '@fluentui/react/lib/Utilities';
import { ITheme, IStyle } from '@fluentui/react/lib/Styling';
import { FontSizes, NeutralColors } from '@fluentui/theme';
import dashboard from '../../../backend/Dashboard';

/**
 * The component props.
 */
export interface IMarkdownHeaderProps {
   as?: keyof React.ReactHTML;
   children?: React.ReactNode;
   className?: string;
   id?: string;

   theme?: ITheme;
   styles?: IStyleFunctionOrObject<IMarkdownHeaderStyleProps, IMarkdownHeaderStyles>;
}

/**
 * The getStyles props contract.
 */
export type IMarkdownHeaderStyleProps = {
   as: string;
   className?: string;
};

/**
 * The styles produced by getStyles.
 */
export interface IMarkdownHeaderStyles {
   root: IStyle;
}

const getStyles: IStyleFunction<IMarkdownHeaderStyleProps, IMarkdownHeaderStyles> = props => {
   const { className } = props;
   return {
      root: [
         {
            fontSize: FontSizes.size16,
            fontWeight: 600,
            padding: 0,
            margin: 0,
            marginTop: '32px',
            marginBottom: '12px',

            selectors: {
               '&:first-child': {
                  marginTop: 0,
               },
            },
         },

         props.as === 'h1' && [
            {
               fontSize: FontSizes.size32,
               marginBottom: '28px',
               color: dashboard.darktheme ? NeutralColors.white : NeutralColors.gray160,
            },
         ],

         props.as === 'h2' && [
            {
               fontSize: FontSizes.size24,
               marginTop: '24px',
               marginRight: '0px',
               marginBottom: '8px',
               marginLeft: '0px',
               color: dashboard.darktheme ? NeutralColors.white : NeutralColors.gray160,


            },
         ],

         props.as === 'h3' && [
            {
               fontSize: FontSizes.size20,
               marginBottom: '8px',
               color: dashboard.darktheme ? NeutralColors.gray30 : NeutralColors.gray130,
            },
         ],
         props.as === 'h4' && [
            {
               fontSize: FontSizes.size16,
               marginBottom: '6px',
               color: dashboard.darktheme ? NeutralColors.white : NeutralColors.gray160,
            },
         ],
         className,
      ],
   };
};

const getClassNames = classNamesFunction<IMarkdownHeaderStyleProps, IMarkdownHeaderStyles>();

const MarkdownHeaderBase: React.FunctionComponent<IMarkdownHeaderProps> = props => {
   const { as: RootType = 'h1', children, id, styles, className } = props;

   const classNames = getClassNames(styles, { as: RootType, className });
   return (
      <RootType className={classNames.root} id={id}>
         {children}
      </RootType>
   );
};

export const MarkdownHeader: React.FunctionComponent<IMarkdownHeaderProps> = styled<
   IMarkdownHeaderProps,
   IMarkdownHeaderStyleProps,
   IMarkdownHeaderStyles
>(MarkdownHeaderBase, getStyles);
