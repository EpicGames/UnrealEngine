// Copyright Epic Games, Inc. All Rights Reserved.

import { IImageProps, Image } from "@fluentui/react"
import dashboard from "../../../backend/Dashboard";


function absolute(base: string, relative: string) {
   var stack = base.split("/"),
      parts = relative.split("/");
   stack.pop();

   for (var i = 0; i < parts.length; i++) {
      if (parts[i] === ".")
         continue;
      if (parts[i] === "..")
         stack.pop();
      else
         stack.push(parts[i]);
   }
   return stack.join("/");
}

export const MarkdownImage: React.FunctionComponent<IImageProps> = props => {

   const location = window.location

   if (props?.src?.endsWith("#gh-light-mode-only") && dashboard.darktheme) {
      return null;
   }

   if (props?.src?.endsWith("#gh-dark-mode-only") && !dashboard.darktheme) {
      return null;
   }

   if (props?.src && location.pathname.startsWith("/docs")) {
      const src = absolute(location.pathname.replace("/docs/", ""), props.src);
      return <Image style={{
         maxWidth: '100%',
         margin: '8px 0'
      }} shouldFadeIn={false} shouldStartVisible={true} src={`/documentation/Docs/${src}`} />
   }

   return <Image style={{
      maxWidth: '100%',
      margin: '8px 0'
   }} shouldFadeIn={false} shouldStartVisible={true} src={props.src} />
}