// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { Link } from 'react-router-dom';
import { ILinkProps } from '@fluentui/react/lib/Link';

const SCROLL_DISTANCE = 52;

/**
 * Given a URL containing a route path (first hash) and possibly an anchor (second hash),
 * returns only the anchor part, if it exists. (Does not include the query string, if any.)
 * If `url` has no hash, or only has a single hash (for the route path), returns an empty string.
 * @param url - Full or partial URL. Just the hash section is valid, as long as it's prepended with #.
 */
export function extractAnchorLink(url: string): string {
   // URLs containing anchors:
   // #/components/checkbox#Overview
   // http://whatever#/components/checkbox#Overview
 
   // URLs NOT containing anchors, by this function's definition:
   // #/components/checkbox
   // http://whatever#/components/checkbox
   // #Overview
   // http://whatever#Overview
   const split = url.split('#');
   if (split.length === 3) {
     // Also remove the query if present
     // (technically the query can't be after the hash, but this is likely with hash routing)
     return split[2].split('?')[0];
   }
   return '';
 }
 

function jumpToAnchor(anchor?: string, scrollDistance: number = SCROLL_DISTANCE): void {
  const hash = anchor || extractAnchorLink(window.location.hash);
  const el = hash && document.getElementById(hash);
  if (hash && el) {
    const elRect = el.getBoundingClientRect();
    const windowY = window.scrollY || window.pageYOffset;
    const currentScrollPosition = windowY + elRect.top;
    const top = currentScrollPosition - scrollDistance;
    if (window.scrollTo) {
      if (window.navigator.userAgent.indexOf('rv:11.0') > -1 || window.navigator.userAgent.indexOf('Edge') > -1) {
        // Edge currently has a bug that jumps to the top of the page if window.scrollTo is passed an oject.
        window.scrollTo(0, top);
      } else {
        window.scrollTo({
          top,
          behavior: 'smooth'
        });
      }
    }
  }
}

function absolute(base: string, relative: string) {
   var stack = base.split("/"),
      parts = relative.split("/");
   stack.pop();

   for (var i = 0; i < parts.length; i++) {
      if (parts[i] === ".")
         continue;
      if (parts[i] === "..")
         stack.pop();
      else
         stack.push(parts[i]);
   }
   return stack.join("/");
}

export const MarkdownLink: React.FunctionComponent<ILinkProps> = props => {
   let href = props.href;

   const inDocs = window.location.pathname.startsWith("/docs");   

   if (inDocs && href?.indexOf("README.md") !== -1) {
      href = "/docs/"
   }

   if (href && href[0] === '#' && href.indexOf('/') === -1) {
      // This is an anchor link within this page. We need to prepend the current route.
      return <a children={props.children} href={ href} onClick={() => jumpToAnchor(href)} />;
   }

   if (href?.startsWith("file://") || href?.startsWith("/api/v1")) {
      return <a children={props.children} href={href!} />;
   }

   if (href) {

      try {

         if (href.startsWith("?")) {
            const search = new URLSearchParams(href);
            const csearch = new URLSearchParams(window.location.search);
            search.forEach((v, k) => {
               csearch.set(k, v);
            });

            href = "?" + csearch.toString();
         }

         if (inDocs && !href.startsWith("/") && !href.startsWith("http")) {            
            href = absolute(window.location.pathname, href);
            if (!href.startsWith("/docs/")) {
               let base = "/docs";
               if (!href.startsWith("/")) {
                  base += "/";
               }
               href = base + href;
            }
         }
         
         const url = new URL(href);
         if (url.hostname === window.location.hostname) {
            href = url.pathname + url.search
         } else {
            return <a children={props.children} href={href} target="_blank" rel="noreferrer" />;
         }
      } catch {

      }
   }

   return <Link children={props.children} to={href!} />;
};
