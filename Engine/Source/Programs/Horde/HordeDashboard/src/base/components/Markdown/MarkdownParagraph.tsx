// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import {
   IClassNames,
   IStyleFunction,
   classNamesFunction,
   styled,
   IStyleFunctionOrObject,
} from '@fluentui/react/lib/Utilities';
import { ITheme, IStyle } from '@fluentui/react/lib/Styling';
import { PropsWithChildren } from 'react';

export interface IMarkdownParagraphProps {
   styles?: IStyleFunctionOrObject<IMarkdownParagraphStyleProps, IMarkdownParagraphStyles>;
   theme?: ITheme;
}

export interface IMarkdownParagraphStyles {
   root: IStyle;
}

export interface IMarkdownParagraphStyleProps {
   theme: ITheme;
   isTodo: boolean;
}

const getStyles: IStyleFunction<IMarkdownParagraphStyleProps, IMarkdownParagraphStyles> = props => {
   const { theme, isTodo } = props;
   return {
      root: [
         theme.fonts.medium,
         {
            lineHeight: "1.6",
            marginBottom: 4,
         },
         isTodo && {
            padding: 8,
            background: theme.semanticColors.warningBackground,
         },
      ],
   };
};

const getClassNames = classNamesFunction<IMarkdownParagraphStyleProps, IMarkdownParagraphStyles>();

const MarkdownParagraphBase: React.FunctionComponent<PropsWithChildren<IMarkdownParagraphProps>> = props => {
   const { children, theme } = props;
   const classNames: IClassNames<IMarkdownParagraphStyles> = getClassNames(props.styles, {
      theme: theme!,
      isTodo: typeof children === 'string' && children.indexOf('TODO') === 0,
   });

   return <p className={classNames.root}>{children}</p>;
};

export const MarkdownParagraph: React.FunctionComponent<PropsWithChildren<IMarkdownParagraphProps>> = styled<
   IMarkdownParagraphProps,
   IMarkdownParagraphStyleProps,
   IMarkdownParagraphStyles
>(MarkdownParagraphBase, getStyles);
