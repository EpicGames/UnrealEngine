// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import dashboard from '../../../backend/Dashboard';

export interface IMarkdownPreProps extends React.HTMLAttributes<HTMLPreElement> {
   /**
    * If true, using a code block with language name `renderhtml` will render the contents as HTML.
    * This is to work around markdown-to-jsx's limited support for nested HTML elements.
    */
   enableRenderHtmlBlock?: boolean;
}

export const MarkdownPre: React.FunctionComponent<IMarkdownPreProps> = props => {
   const { children, enableRenderHtmlBlock, ...rest } = props;

   // markdown-to-jsx will render code blocks as <pre><code>code here</code></pre>.
   // If we're using the code block as a hack to render raw HTML, we need to strip the wrapping <pre>
   // and just render the <code>'s children in a div.
   // eslint-disable-next-line @typescript-eslint/no-explicit-any
   const childrenDisplayName: string | undefined = (children as any)?.type?.displayName;
   // eslint-disable-next-line @typescript-eslint/no-explicit-any
   const childrenProps: React.PropsWithChildren<any> | undefined = (children as any)?.props;
   if (
      enableRenderHtmlBlock &&
      typeof childrenDisplayName === 'string' &&
      childrenDisplayName.indexOf('MarkdownCode') !== -1 &&
      childrenProps.className === 'lang-renderhtml' &&
      typeof childrenProps?.children === 'string'
   ) {
      // eslint-disable-next-line react/no-danger
      return <div style={{ backgroundColor: dashboard.darktheme ? "#25282A" : "#f2f2f2" }} dangerouslySetInnerHTML={{ __html: childrenProps.children }} />;
   }

   return <pre {...rest} style={{ backgroundColor: dashboard.darktheme ? "#25282A" : "#f2f2f2" }}>{children}</pre>;
};
