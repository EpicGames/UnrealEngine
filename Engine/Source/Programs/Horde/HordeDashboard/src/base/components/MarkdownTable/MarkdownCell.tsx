// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { classNamesFunction, styled } from '@fluentui/react';
import { IMarkdownTableCellProps, IMarkdownTableStyleProps, IMarkdownTableStyles } from './MarkdownTable.types';
import { getStyles } from './MarkdownTable.styles';
import { PropsWithChildren } from 'react';

const getClassNames = classNamesFunction<IMarkdownTableStyleProps, IMarkdownTableStyles>();

class MarkdownCellBase extends React.PureComponent<PropsWithChildren<IMarkdownTableCellProps>> {
  public render(): JSX.Element {
    const { as = 'td', children, styles, theme } = this.props;

    const classNames = getClassNames(styles, {
      theme: theme!,
    });

    const Tag = as;
    const cellClassName = (as === 'td' && classNames.td) || (as === 'th' && classNames.th) || '';

    return (
      <Tag {...this.props} className={cellClassName}>
        {children}
      </Tag>
    );
  }
}

export const MarkdownCell = styled<IMarkdownTableCellProps, IMarkdownTableStyleProps, IMarkdownTableStyles>(
  MarkdownCellBase,
  getStyles,
  undefined,
  { scope: 'MarkdownTable' },
);
