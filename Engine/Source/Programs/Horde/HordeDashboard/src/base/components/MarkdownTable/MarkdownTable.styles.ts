// Copyright Epic Games, Inc. All Rights Reserved.

import { getGlobalClassNames, IStyle } from '@fluentui/react';
import { IMarkdownTableStyleProps, IMarkdownTableStyles } from './MarkdownTable.types';

const GlobalClassNames = {
  root: 'ms-MarkdownTable-wrapper',
  table: 'ms-MarkdownTable-table',
  thead: 'ms-MarkdownTable-thead',
  tbody: 'ms-MarkdownTable-tbody',
  tr: 'ms-MarkdownTable-tr',
  td: 'ms-MarkdownTable-td',
  th: 'ms-MarkdownTable-th',
};

const cellStyles: IStyle = {
  fontSize: '14px',
  padding: '14px',
  verticalAlign: 'top',

  selectors: {
    '&:first-of-type': {
      paddingLeft: 0,
    },

    '&[style*="text-align"] .ms-mdImage img': {
      display: 'inline-block',
    },
  },
};

export const getStyles = (props: IMarkdownTableStyleProps): IMarkdownTableStyles => {
  const { className, theme } = props;

  const { palette } = theme;

  const classNames = getGlobalClassNames(GlobalClassNames, theme);

  return {
    root: [
      classNames.root,
      className,
      {
        width: '100%',
        overflowX: 'auto',
        marginBottom: 20,
        // TODO: add background gradient styles (see Page.module.scss)
      },
    ],

    table: [
      classNames.root,
      {
        width: '100%',
        borderCollapse: 'collapse',
        borderSpacing: '0',

        selectors: {
          '.ms-mdImage img': {
            maxWidth: '100%',
          },
        },
      },
    ],

    tr: [
      classNames.tr,
      {
        borderBottom: `1px solid ${palette.neutralTertiary}`,
      },
    ],

    thead: [
      classNames.thead,
      {
        selectors: {
          [`.${classNames.tr}`]: {
            borderColor: palette.neutralPrimary,
          },
        },
      },
    ],

    tbody: [
      classNames.tbody,
      {
        selectors: {
          [`.${classNames.tr}:last-of-type`]: {
            borderBottom: 'none',
          },
        },
      },
    ],

    td: [
      classNames.td,
      {
        color: palette.neutralSecondary,
        ...cellStyles,
      },
    ],

    th: [
      classNames.th,
      {
        fontWeight: 600,
        color: palette.neutralPrimary,
        whiteSpace: 'nowrap',
        textAlign: 'left',
        ...cellStyles,
      },
    ],
  };
};
