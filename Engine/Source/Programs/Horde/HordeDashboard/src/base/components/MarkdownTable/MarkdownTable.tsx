// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { classNamesFunction, styled } from '@fluentui/react';
import { IMarkdownTableProps, IMarkdownTableStyleProps, IMarkdownTableStyles } from './MarkdownTable.types';
import { getStyles } from './MarkdownTable.styles';
import { PropsWithChildren } from 'react';

const getClassNames = classNamesFunction<IMarkdownTableStyleProps, IMarkdownTableStyles>();

class MarkdownTableBase extends React.PureComponent<PropsWithChildren<IMarkdownTableProps>> {
  public render(): JSX.Element {
    const { children, className, styles, theme } = this.props;

    const classNames = getClassNames(styles, {
      theme: theme!,
      className,
    });

    return (
      <div className={classNames.root}>
        <table {...this.props} className={classNames.table}>
          {children}
        </table>
      </div>
    );
  }
}

export const MarkdownTable = styled<IMarkdownTableProps, IMarkdownTableStyleProps, IMarkdownTableStyles>(
  MarkdownTableBase,
  getStyles,
  undefined,
  { scope: 'MarkdownTable' },
);
