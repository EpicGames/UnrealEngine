// Copyright Epic Games, Inc. All Rights Reserved.

import { IStyle, ITheme, IStyleFunctionOrObject } from '@fluentui/react';

export interface IMarkdownTableProps {
  className?: string;

  /** Theme provided by higher-order component. */
  theme?: ITheme;

  /** Call to provide customized styling that will layer on top of the variant rules. */
  styles?: IStyleFunctionOrObject<IMarkdownTableStyleProps, IMarkdownTableStyles>;
}

export interface IMarkdownTableCellProps extends IMarkdownTableProps {
  /**
   * Render the table cell as a th or td.
   * @default 'td'
   */
  as?: 'th' | 'td';
}

export type IMarkdownTableStyleProps = Required<Pick<IMarkdownTableProps, 'theme'>> &
  Pick<IMarkdownTableProps, 'className'>;

export interface IMarkdownTableStyles {
  root: IStyle;
  table: IStyle;
  thead: IStyle;
  tbody: IStyle;
  tr: IStyle;
  td: IStyle;
  th: IStyle;
}
