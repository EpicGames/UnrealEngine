// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { classNamesFunction, styled } from '@fluentui/react';
import { IMarkdownTableProps, IMarkdownTableStyleProps, IMarkdownTableStyles } from './MarkdownTable.types';
import { getStyles } from './MarkdownTable.styles';
import { PropsWithChildren } from 'react';

const getClassNames = classNamesFunction<IMarkdownTableStyleProps, IMarkdownTableStyles>();

class MarkdownTrBase extends React.PureComponent<PropsWithChildren<IMarkdownTableProps>> {
  public render(): JSX.Element {
    const { children, styles, theme } = this.props;

    const classNames = getClassNames(styles, {
      theme: theme!,
    });

    return (
      <tr {...this.props} className={classNames.tr}>
        {children}
      </tr>
    );
  }
}

export const MarkdownTr = styled<IMarkdownTableProps, IMarkdownTableStyleProps, IMarkdownTableStyles>(
  MarkdownTrBase,
  getStyles,
  undefined,
  { scope: 'MarkdownTable' },
);
