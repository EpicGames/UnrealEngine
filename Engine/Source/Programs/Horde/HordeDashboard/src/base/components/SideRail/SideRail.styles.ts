// Copyright Epic Games, Inc. All Rights Reserved.

//import { FontWeights, getFocusOutlineStyle, IStyleFunction } from '@fluentui/react/lib/index';
import { FontWeights, getFocusOutlineStyle, IStyleFunction } from '@fluentui/react';
import { ISideRailStyleProps, ISideRailStyles } from './SideRail.types';
import { getHordeStyling } from '../../../styles/Styles';


const appPaddingSm = 28;

export const sideRailClassNames = {
  isActive: 'SideRail-isActive'
};

export const getStyles: IStyleFunction<ISideRailStyleProps, ISideRailStyles> = props => {

  const { modeColors } = getHordeStyling();

  const theme = props.theme!;
  return {
    root: {},
    section: {
      marginBottom: appPaddingSm,
      selectors: {
        '&:last-child': {
          marginBottom: 0,
          'a:active,a:visited,a:hover' : {color: modeColors.text, textDecoration: "none"}
        }
      }
    },
    sectionTitle: {
      fontSize: theme.fonts.mediumPlus.fontSize,
      fontWeight: FontWeights.semibold,
      color: modeColors.text,
      marginTop: 0,
      paddingLeft: 8
    },
    links: {
      margin: 0,
      padding: 0      
    },
    linkWrapper: {
      display: 'flex',
      fontSize: theme.fonts.medium.fontSize,
      selectors: {
        a: [
          {
            display: 'block',
            flex: '1',
            padding: '4px 0px',
            selectors: {              
              ':active,:visited,:hover': { color: modeColors.text, textDecoration: "none", background: theme.palette.neutralLight }
            }
          },
          getFocusOutlineStyle(theme, 1)
        ]
      }
    },
    markdownList: {
      selectors: {
        'ul li': [
          {
            fontSize: theme.fonts.medium.fontSize,
            padding: '4px 0px',
            selectors: {
              '&:hover': { background: theme.palette.neutralLight }
            }
          },
          getFocusOutlineStyle(theme, 1)
        ]
      }
    },
    jumpLinkWrapper: {
      position: 'relative'
    },
    jumpLink: {
      color: theme.palette.neutralPrimary,
      borderLeft: '2px solid transparent',
      paddingLeft: 0, // 8px - 2px border
      selectors: {
        '&:focus': {
          color: modeColors.text
        }, 
        '&:active,&:visited,&:hover' : {color: modeColors.text, textDecoration: "none"}
      }
    },
    jumpLinkActive: {
      borderLeftColor: theme!.palette.themePrimary
    },
    jumpLinkSection: {
      selectors: {
        '@media screen and (max-width: 1360px)': {
          display: 'none'
        }
      }
    }
  };
};
