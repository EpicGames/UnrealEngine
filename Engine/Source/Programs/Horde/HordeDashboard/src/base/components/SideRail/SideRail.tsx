// Copyright Epic Games, Inc. All Rights Reserved.

import { classNamesFunction, css, FocusZone, FocusZoneDirection, IProcessedStyleSet, Link, Stack, styled, Text } from '@fluentui/react';
import * as React from 'react';
import { Link as ReactRouterLink } from 'react-router-dom';
import { jumpToAnchor, /*isPageActive,*/ removeAnchorLink } from '../../utilities';
import { getStyles } from './SideRail.styles';
import { ISideRailLink, ISideRailProps, ISideRailStyleProps, ISideRailStyles } from './SideRail.types';

export interface ISideRailState {
   activeLink?: string;
}

const getClassNames = classNamesFunction<ISideRailStyleProps, ISideRailStyles>();

class SideRailBase extends React.Component<ISideRailProps, ISideRailState> {
   public readonly state: ISideRailState = {};
   private _classNames?: IProcessedStyleSet<ISideRailStyles>;
   private _observer?: IntersectionObserver;

   public componentDidMount(): void {
      if (typeof IntersectionObserver !== 'undefined') {
         const { observe, jumpLinks } = this.props;
         if (observe && jumpLinks) {
            this._observer = new IntersectionObserver(this._handleObserver, {
               threshold: [0.5]
            });

            jumpLinks.forEach((jumpLink: ISideRailLink) => {
               const element = document.getElementById(jumpLink.url);
               if (element) {
                  this._observer!.observe(element);
               }
            });
         }
      }
   }

   public componentWillUnmount() {
      if (this._observer) {
         this._observer.disconnect();
      }
   }

   public render(): JSX.Element | null {
      this._classNames = getClassNames(this.props.styles, { theme: this.props.theme });

      const jumpLinkList = this._renderJumpLinkList();
      const relatedLinkList = this._renderLinkList(this.props.relatedLinks, 'Reference');
      const contactLinkList = this._renderLinkList(this.props.contactLinks, 'Contacts');

      return jumpLinkList || relatedLinkList || contactLinkList ? (
         <FocusZone direction={FocusZoneDirection.vertical} className={this._classNames.root}>
            {relatedLinkList}
            {jumpLinkList}
            {contactLinkList}
         </FocusZone>
      ) : null;
   }

   private _handleObserver = (entries: IntersectionObserverEntry[]) => {
      for (const entry of entries) {
         const { intersectionRatio, target } = entry;
         if (intersectionRatio > 0.5) {
            this.setState({
               activeLink: target.id
            });
            break;
         }
      }
   };

   private _renderJumpLinkList = (): JSX.Element | null => {
      const { activeLink } = this.state;
      const { jumpLinks } = this.props;
      const classNames = this._classNames!;

      if (!jumpLinks || !jumpLinks.length) {
         return null;
      }
      const links = jumpLinks.map((jumpLink: ISideRailLink) => (
         <li key={jumpLink.url} className={css(classNames.linkWrapper, classNames.jumpLinkWrapper)}>
            {!jumpLink.reactRouterLink && <Link
               href={this._getJumpLinkUrl(jumpLink.url)}
               onClick={this._onJumpLinkClick}
               styles={{
                  root: [classNames.jumpLink, activeLink === jumpLink.url && classNames.jumpLinkActive]
               }}
            >
               {jumpLink.text}
            </Link>}
            {!!jumpLink.reactRouterLink && <ReactRouterLink to={jumpLink.url} target={jumpLink.target}>{jumpLink.text}</ReactRouterLink>}
         </li>
      ));
      return (
         <div className={css(classNames.section, classNames.jumpLinkSection)}>
            <Stack style={{ paddingBottom: 8 }}>
               <Text variant='medium' style={{ fontWeight: 600 }}>On this page</Text>
            </Stack>
            <ul className={classNames.links}>{links}</ul>
         </div>
      );
   };

   private _renderLinkList(linksFromProps: ISideRailLink[] | JSX.Element | undefined, title: string): JSX.Element | null {
      const classNames = this._classNames!;

      let links: JSX.Element | undefined;
      if (_isElement(linksFromProps)) {
         links = <div className={classNames.markdownList}>{linksFromProps}</div>;
      } else if (Array.isArray(linksFromProps)) {
         const linksToRender = linksFromProps.filter(link => true/*!isPageActive(link.url)*/);
         if (linksToRender.length) {
            links = (
               <ul className={classNames.links}>
                  {linksToRender.map(link => (
                     <li key={link.url} className={classNames.linkWrapper}>
                        <ReactRouterLink to={link.url}>{link.text}</ReactRouterLink>
                     </li>
                  ))}
               </ul>
            );
         }
      }

      if (links) {
         return (
            <div className={css(classNames.section)}>
               <Stack style={{ paddingBottom: 8 }}>
                  <Text variant='medium' style={{ fontWeight: 600 }}>{title}</Text>
               </Stack>
               <Stack>
                  {links}
               </Stack>
            </div>
         );
      }
      return null;
   }

   // tslint:disable-next-line:no-any
   private _onJumpLinkClick = (ev?: React.MouseEvent<any>): void => {
      const target = ev && (ev.target as HTMLAnchorElement);
      if (target && target.href === window.location.href) {
         // If this link is already in the URL, scroll back to it on click
         // (otherwise, scrolling will be handled elsewhere)
         jumpToAnchor();
      }
   };

   private _getJumpLinkUrl(anchor: string): string {
      // This makes sure that location hash changes don't append
      return `${removeAnchorLink(window.location.hash)}#${anchor}`;
   }
}

// tslint:disable-next-line:no-any
function _isElement(x: any): x is JSX.Element {
   return !!(x && (x as JSX.Element).props && (x as JSX.Element).type);
}

export const SideRail: React.FunctionComponent<ISideRailProps> = styled<ISideRailProps, ISideRailStyleProps, ISideRailStyles>(
   SideRailBase,
   getStyles,
   undefined,
   { scope: 'SideRail' }
);
