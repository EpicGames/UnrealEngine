// Copyright Epic Games, Inc. All Rights Reserved.

import { ITheme, IStyleFunctionOrObject, IStyle } from '@fluentui/react';

export interface ISideRailProps {
  /**
   * List of hash links to places within the current page.
   */
  jumpLinks?: ISideRailLink[];

  /**
   * List of links to related pages, as link objects or pre-rendered.
   */
  relatedLinks?: ISideRailLink[] | JSX.Element;

  /**
   * Mailto links for people related to the page, as link objects or pre-rendered.
   */
  contactLinks?: ISideRailLink[] | JSX.Element;

  /**
   * If the page has elements to observe.
   */
  observe?: boolean;

  /**
   * Override styles.
   */
  styles?: IStyleFunctionOrObject<ISideRailStyleProps, ISideRailStyles>;

  /**
   * Theme provided by higher-order component.
   */
  theme?: ITheme;
}

export interface ISideRailLink {
  /** Link text to display */
  text: string;
  /**
   * URL for the link:
   * - For related or contact links, a complete URL
   * - For jump links, just an element ID (no #)
   */
  url: string;

  // url target
  target?: string;

  /** Whether to use react router for the link */
  reactRouterLink?: boolean;
}

export type ISideRailStyleProps = Pick<ISideRailProps, 'theme'>;

export interface ISideRailStyles {
  root: IStyle;
  section: IStyle;
  sectionTitle: IStyle;
  links: IStyle;
  linkWrapper: IStyle;
  markdownList: IStyle;
  jumpLinkWrapper: IStyle;
  jumpLink: IStyle;
  jumpLinkActive: IStyle;
  jumpLinkSection: IStyle;
}
