// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, IContextualMenuProps, ITextStyles, Pivot, PivotItem, PrimaryButton, SearchBox, Separator, SpinButton, Stack, Text, TextField, Toggle } from "@fluentui/react";
import { useState } from "react";
import { Breadcrumbs } from "../../../components/Breadcrumbs";
import { TopNav } from "../../../components/TopNav";
import { getHordeStyling } from "../../../styles/Styles";
import { useWindowSize } from "../../utilities/hooks";
import { ComboBoxSticker } from "./stickers/ComboBox";
import { ContextualMenuSticker } from "./stickers/ContextualMenu";
import { DropdownSticker } from "./stickers/Dropdown";
import { ProgressIndicatorSticker } from "./stickers/Progress";
import { SliderSticker } from "./stickers/Slider";
import { SpinnerSticker } from "./stickers/Spinner";

const mediumTextStyle: ITextStyles = {
   root: {
      fontSize: "14px",
      fontWeight: 600
   }
}

/*
const CommandsMenusNavsPanel: React.FC = () => {

   return <Stack>
      <Stack horizontal style={{ width: 1400 }} tokens={{ childrenGap: 120 }}>
         <Stack>
            <Stack>
               <Text styles={largeTextStyle} >Commands & Nav</Text>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Command Bar</Text>
               </Stack>
               <Stack style={{ width: 512 }} tokens={{ childrenGap: 40 }}>
                  <CommandBarSticker />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Contextual Menu</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 10 }}>
                  <ContextualMenuSticker />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
}
*/

const BasicInputsPanel: React.FC = () => {

   const splitButtonMenuProps: IContextualMenuProps = {
      items: [
         {
            key: 'emailMessage',
            text: 'Email message',
            iconProps: { iconName: 'Mail' },
         },
         {
            key: 'calendarEvent',
            text: 'Calendar event',
            iconProps: { iconName: 'Calendar' },
         },
      ],
   };

   return <Stack>
      <Stack horizontal style={{ width: 1400 }} tokens={{ childrenGap: 64 }}>
         <Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Buttons</Text>
               </Stack>
               <Stack horizontal style={{ width: 420 }} tokens={{ childrenGap: 40 }}>
                  <DefaultButton text="Standard" />
                  <PrimaryButton text="Primary" />
                  <DefaultButton
                     text="Search"
                     split
                     menuProps={splitButtonMenuProps}
                  />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Checkbox</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 10 }}>
                  <Checkbox label="Unchecked checkbox" />
                  <Checkbox label="Checked checkbox" defaultChecked />
                  <Checkbox label="Disabled checkbox" disabled />
                  <Checkbox label="Disabled checked checkbox" disabled defaultChecked />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Combo Box</Text>
               </Stack>
               <Stack>
                  <ComboBoxSticker />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Dropdown</Text>
               </Stack>
               <Stack>
                  <DropdownSticker />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Contextual Menu</Text>
               </Stack>
               <Stack>
                  <ContextualMenuSticker />
               </Stack>
            </Stack>
         </Stack>
         <Stack>
            <Stack style={{ width: 300 }}>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Search Box</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 20 }}>
                  <SearchBox
                     placeholder="Search"
                     disableAnimation
                  />
               </Stack>
            </Stack>

            <Stack style={{ width: 300 }}>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Slider</Text>
               </Stack>
               <Stack>
                  <SliderSticker />
               </Stack>
            </Stack>
            <Stack style={{ width: 300 }}>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Spin Button</Text>
               </Stack>
               <Stack>
                  <SpinButton
                     label="Basic SpinButton"
                     defaultValue="0"
                     min={0}
                     max={100}
                     step={1}
                     styles={{ spinButtonWrapper: { width: 75 } }}
                  />
               </Stack>
            </Stack>

            <Stack style={{ width: 300 }}>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Text Field</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 10 }}>
                  <TextField label="Standard" />
                  <TextField label="Disabled" disabled defaultValue="I am disabled" />
                  <TextField label="Read-only" readOnly defaultValue="I am read-only" />
               </Stack>
            </Stack>

            <Stack style={{ width: 300 }}>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Toggle</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 10 }}>
                  <Toggle label="Enabled and checked" defaultChecked onText="On" offText="Off" />
                  <Toggle label="Enabled and unchecked" onText="On" offText="Off" />
                  <Toggle label="Disabled and checked" defaultChecked disabled onText="On" offText="Off" />
                  <Toggle label="Disabled and unchecked" disabled onText="On" offText="Off" />
               </Stack>
            </Stack>
         </Stack>
         <Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Progress</Text>
               </Stack>
               <Stack >
                  <ProgressIndicatorSticker />
               </Stack>
            </Stack>
            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Spinner</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 10 }}>
                  <SpinnerSticker />
               </Stack>
            </Stack>

            <Stack>
               <Stack style={{ paddingTop: 24, paddingBottom: 12 }} >
                  <Text styles={mediumTextStyle} >Separator</Text>
               </Stack>
               <Stack>
                  <Separator />
               </Stack>
            </Stack>

         </Stack>
      </Stack>
   </Stack>
}

const StickerPanel: React.FC = () => {
   const [state, setState] = useState("Basic");

   return <Stack>
      <Stack>
         <Pivot defaultSelectedKey={state} linkSize="normal"
            linkFormat="links"
            onLinkClick={(item => {
               setState(item!.props.itemKey!)
            })}>
            <PivotItem headerText="Basic" itemKey="Basic" />
         </Pivot>
      </Stack>
      <Stack style={{ paddingLeft: 24, paddingBottom: 24 }}>
         {state === "Basic" && <BasicInputsPanel />}
      </Stack>
   </Stack>
}

export const ThemeTester: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Theme Tester' }]} />
      <Stack horizontal>
         <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
            <Stack style={{ maxWidth: 1440, paddingTop: 6, marginLeft: 4, height: 'calc(100vh - 8px)' }}>
               <Stack horizontal className={hordeClasses.raised}>
                  <Stack style={{ width: "100%", height: 'calc(100vh - 228px)' }} tokens={{ childrenGap: 18 }}>
                     <StickerPanel />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

