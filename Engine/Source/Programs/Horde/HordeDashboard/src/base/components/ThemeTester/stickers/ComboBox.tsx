// Copyright Epic Games, Inc. All Rights Reserved.

import {
   ComboBox,
   IComboBoxOption,
   IComboBoxStyles,
   SelectableOptionMenuItemType
} from '@fluentui/react';
import * as React from 'react';

const options: IComboBoxOption[] = [
   { key: 'Header1', text: 'First heading', itemType: SelectableOptionMenuItemType.Header },
   { key: 'A', text: 'Option A' },
   { key: 'B', text: 'Option B' },
   { key: 'C', text: 'Option C' },
   { key: 'D', text: 'Option D' },
   { key: 'divider', text: '-', itemType: SelectableOptionMenuItemType.Divider },
   { key: 'Header2', text: 'Second heading', itemType: SelectableOptionMenuItemType.Header },
   { key: 'E', text: 'Option E' },
   { key: 'F', text: 'Option F', disabled: true },
   { key: 'G', text: 'Option G' },
   { key: 'H', text: 'Option H' },
   { key: 'I', text: 'Option I' },
   { key: 'J', text: 'Option J' },
];
// Optional styling to make the example look nicer
const comboBoxStyles: Partial<IComboBoxStyles> = { root: { maxWidth: 300 } };
//const buttonStyles: Partial<IButtonStyles> = { root: { display: 'block', margin: '10px 0 20px' } };

export const ComboBoxSticker: React.FunctionComponent = () => {

   return (
      <div>
         <ComboBox
            defaultSelectedKey="C"
            label="Basic single-select ComboBox"
            options={options}
            styles={comboBoxStyles}
         />

         <ComboBox
            defaultSelectedKey="C"
            label="Basic multi-select ComboBox"
            multiSelect
            options={options}
            styles={comboBoxStyles}
         />
      </div>
   );
};
