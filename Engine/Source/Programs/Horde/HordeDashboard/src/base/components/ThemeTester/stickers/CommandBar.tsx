// Copyright Epic Games, Inc. All Rights Reserved.

import { setVirtualParent } from '@fluentui/dom-utilities';
import { IButtonProps } from '@fluentui/react/lib/Button';
import { CommandBar, ICommandBarItemProps } from '@fluentui/react/lib/CommandBar';
import * as React from 'react';

const overflowProps: IButtonProps = { ariaLabel: 'More commands' };

export const CommandBarSticker: React.FunctionComponent = () => {


   return (<CommandBar
      items={_items}
      overflowItems={_overflowItems}
      overflowButtonProps={overflowProps}
      farItems={_farItems}
      ariaLabel="Inbox actions"
      primaryGroupAriaLabel="Email actions"
      farItemsGroupAriaLabel="More actions"
   />
   );
};

const _items: ICommandBarItemProps[] = [
   {
      key: 'newItem',
      text: 'New',
      cacheKey: 'myCacheKey', // changing this key will invalidate this item's cache
      iconProps: { iconName: 'Add' },
      subMenuProps: {
         items: [
            {
               key: 'emailMessage',
               text: 'Email message',
               iconProps: { iconName: 'Mail' }               
            },
            {
               key: 'calendarEvent',
               text: 'Calendar event',
               iconProps: { iconName: 'Calendar' },
            },
         ],
      },
   },
   {
      key: 'upload',
      text: 'Upload',
      iconProps: { iconName: 'Upload' },
      subMenuProps: {
         items: [
            {
               key: 'uploadfile',
               text: 'File',
               preferMenuTargetAsEventTarget: true,
               onClick: (ev?: React.MouseEvent<HTMLElement, MouseEvent> | React.KeyboardEvent<HTMLElement> | undefined) => {
                  ev?.persist();

                  Promise.resolve().then(() => {
                     const inputElement = document.createElement('input');
                     inputElement.style.visibility = 'hidden';
                     inputElement.setAttribute('type', 'file');

                     document.body.appendChild(inputElement);

                     const target = ev?.target as HTMLElement | undefined;

                     if (target) {
                        setVirtualParent(inputElement, target);
                     }

                     inputElement.click();

                     if (target) {
                        setVirtualParent(inputElement, null);
                     }

                     setTimeout(() => {
                        inputElement.remove();
                     }, 10000);
                  });
               },
            },
            {
               key: 'uploadfolder',
               text: 'Folder',
               preferMenuTargetAsEventTarget: true,
               onClick: (ev?: React.MouseEvent<HTMLElement, MouseEvent> | React.KeyboardEvent<HTMLElement> | undefined) => {
                  ev?.persist();

                  Promise.resolve().then(() => {
                     const inputElement = document.createElement('input');
                     inputElement.style.visibility = 'hidden';
                     inputElement.setAttribute('type', 'file');

                     (inputElement as { webkitdirectory?: boolean }).webkitdirectory = true;

                     document.body.appendChild(inputElement);

                     const target = ev?.target as HTMLElement | undefined;

                     if (target) {
                        setVirtualParent(inputElement, target);
                     }

                     inputElement.click();

                     if (target) {
                        setVirtualParent(inputElement, null);
                     }

                     setTimeout(() => {
                        inputElement.remove();
                     }, 10000);
                  });
               },
            },
         ],
      },
   },
   {
      key: 'share',
      text: 'Share',
      iconProps: { iconName: 'Share' },
      onClick: () => console.log('Share'),
   },
   {
      key: 'download',
      text: 'Download',
      iconProps: { iconName: 'Download' },
      onClick: () => console.log('Download'),
   },
];

const _overflowItems: ICommandBarItemProps[] = [
   { key: 'move', text: 'Move to...', onClick: () => console.log('Move to'), iconProps: { iconName: 'MoveToFolder' } },
   { key: 'copy', text: 'Copy to...', onClick: () => console.log('Copy to'), iconProps: { iconName: 'Copy' } },
   { key: 'rename', text: 'Rename...', onClick: () => console.log('Rename'), iconProps: { iconName: 'Edit' } },
];

const _farItems: ICommandBarItemProps[] = [
   {
      key: 'tile',
      text: 'Grid view',
      // This needs an ariaLabel since it's icon-only
      ariaLabel: 'Grid view',
      iconOnly: true,
      iconProps: { iconName: 'Tiles' },
      onClick: () => console.log('Tiles'),
   },
   {
      key: 'info',
      text: 'Info',
      // This needs an ariaLabel since it's icon-only
      ariaLabel: 'Info',
      iconOnly: true,
      iconProps: { iconName: 'Info' },
      onClick: () => console.log('Info'),
   },
];
