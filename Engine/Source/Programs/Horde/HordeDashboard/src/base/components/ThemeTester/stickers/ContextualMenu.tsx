// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { ContextualMenuItemType, IContextualMenuProps } from '@fluentui/react/lib/ContextualMenu';
import { DefaultButton } from '@fluentui/react/lib/Button';
import { useConst } from '@fluentui/react-hooks';

export const ContextualMenuSticker: React.FunctionComponent = () => {
  const menuProps = useConst<IContextualMenuProps>(() => ({
    items: [
      {
        key: 'section1',
        itemType: ContextualMenuItemType.Section,
        sectionProps: {
          topDivider: true,
          bottomDivider: true,
          title: 'Actions',
          items: [
            { key: 'newItem', text: 'New' },
            { key: 'deleteItem', text: 'Delete' },
          ],
        },
      },
      {
        key: 'section2',
        itemType: ContextualMenuItemType.Section,
        sectionProps: {
          title: 'Social',
          items: [
            { key: 'share', text: 'Share' },
            { key: 'print', text: 'Print' },
            { key: 'music', text: 'Music' },
          ],
        },
      },
      {
        key: 'section3',
        itemType: ContextualMenuItemType.Section,
        sectionProps: {
          title: 'Navigation',
          items: [{ key: 'Epic Games', text: 'Go to Epic Games', href: 'https://www.epicgames.com', target: '_blank' }],
        },
      },
    ],
  }));

   return <DefaultButton text="Click for ContextualMenu" menuProps={menuProps} style={{width: 300}} />;
};
