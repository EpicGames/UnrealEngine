// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { IStackTokens, Stack } from '@fluentui/react/lib/Stack';
import { Dropdown, DropdownMenuItemType, IDropdownStyles, IDropdownOption } from '@fluentui/react/lib/Dropdown';

const dropdownStyles: Partial<IDropdownStyles> = {
   dropdown: { width: 300 },
};

const options: IDropdownOption[] = [
   { key: 'fruitsHeader', text: 'Fruits', itemType: DropdownMenuItemType.Header },
   { key: 'apple', text: 'Apple' },
   { key: 'banana', text: 'Banana' },
   { key: 'orange', text: 'Orange', disabled: true },
   { key: 'grape', text: 'Grape' },
   { key: 'divider_1', text: '-', itemType: DropdownMenuItemType.Divider },
   { key: 'vegetablesHeader', text: 'Vegetables', itemType: DropdownMenuItemType.Header },
   { key: 'broccoli', text: 'Broccoli' },
   { key: 'carrot', text: 'Carrot' },
   { key: 'lettuce', text: 'Lettuce' },
];

const stackTokens: IStackTokens = { childrenGap: 20 };

export const DropdownSticker: React.FunctionComponent = () => {
   return (
      <Stack tokens={stackTokens}>
         <Dropdown
            placeholder="Select an option"
            label="Basic uncontrolled example"
            options={options}
            styles={dropdownStyles}
         />
         <Dropdown
            placeholder="Select options"
            label="Multi-select uncontrolled example"
            defaultSelectedKeys={['apple', 'banana', 'grape']}
            multiSelect
            options={options}
            styles={dropdownStyles}
         />
      </Stack>
   );
};
