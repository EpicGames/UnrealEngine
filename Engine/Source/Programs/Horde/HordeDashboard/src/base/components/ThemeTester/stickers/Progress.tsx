// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { ProgressIndicator } from '@fluentui/react/lib/ProgressIndicator';

const intervalDelay = 100;
const intervalIncrement = 0.01;

export const ProgressIndicatorSticker: React.FunctionComponent = () => {
  const [percentComplete, setPercentComplete] = React.useState(0);

  React.useEffect(() => {
    const id = setInterval(() => {
      setPercentComplete((intervalIncrement + percentComplete) % 1);
    }, intervalDelay);
    return () => {
      clearInterval(id);
    };
  });

  return (
     <ProgressIndicator styles={{ root: { width: 180 } }} percentComplete={percentComplete} />
  );
};
