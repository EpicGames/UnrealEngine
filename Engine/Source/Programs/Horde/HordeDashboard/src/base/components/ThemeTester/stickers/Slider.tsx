// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { Slider, IStackTokens, Stack, IStackStyles } from '@fluentui/react';

const stackStyles: Partial<IStackStyles> = { root: { maxWidth: 300 } };
const stackTokens: IStackTokens = { childrenGap: 20 };

export const SliderSticker: React.FunctionComponent = () => {
  return (
    <Stack tokens={stackTokens} styles={stackStyles}>
      <Slider aria-label="Basic example" />
      <Slider label="Disabled example" min={50} max={500} step={50} defaultValue={300} showValue disabled />
    </Stack>
  );
};
