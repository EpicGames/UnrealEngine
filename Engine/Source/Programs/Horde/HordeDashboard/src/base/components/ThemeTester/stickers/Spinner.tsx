// Copyright Epic Games, Inc. All Rights Reserved.

import * as React from 'react';
import { Spinner, SpinnerSize } from '@fluentui/react/lib/Spinner';
import { Label } from '@fluentui/react/lib/Label';
import { IStackProps, Stack } from '@fluentui/react/lib/Stack';

export const SpinnerSticker: React.FunctionComponent = () => {
  // This is just for laying out the label and spinner (spinners don't have to be inside a Stack)
  const rowProps: IStackProps = { horizontal: true, verticalAlign: 'center' };

  const tokens = {
    sectionStack: {
      childrenGap: 10,
    },
    spinnerStack: {
      childrenGap: 20,
    },
  };

  return (
    <Stack tokens={tokens.sectionStack}>
      <Stack {...rowProps} tokens={tokens.spinnerStack}>
        <Label>Extra small spinner</Label>
        <Spinner size={SpinnerSize.xSmall} />
      </Stack>

      <Stack {...rowProps} tokens={tokens.spinnerStack}>
        <Label>Small spinner</Label>
        <Spinner size={SpinnerSize.small} />
      </Stack>

      <Stack {...rowProps} tokens={tokens.spinnerStack}>
        <Label>Medium spinner</Label>
        <Spinner size={SpinnerSize.medium} />
      </Stack>

      <Stack {...rowProps} tokens={tokens.spinnerStack}>
        <Label>Large spinner</Label>
        <Spinner size={SpinnerSize.large} />
      </Stack>
    </Stack>
  );
};
