// Copyright Epic Games, Inc. All Rights Reserved.

export const copyToClipboard = (value: string | undefined) => {

    if (!value) {
        return;
    }

    const el = document.createElement('textarea');
    el.value = value;
    document.body.appendChild(el);
    el.select();
    document.execCommand('copy');
    document.body.removeChild(el);
}

export const copyToClipboardAsync = async (value: string | undefined): Promise<boolean> => {

    if (!value) {
        return false;
    }

    if (navigator.clipboard?.writeText) {
        try {
            await navigator.clipboard.writeText(value);
            return true;
        } catch (reason) {
            console.warn(`navigator.clipboard.writeText failed, falling back to execCommand: ${reason}`);
        }
    }

    return copyToClipboardLegacy(value);
}

// Legacy fallback for non-secure contexts. Uses a hidden, off-screen textarea so the page
// doesn't visibly jump or steal focus while the copy is staged.
const copyToClipboardLegacy = (value: string): boolean => {
    const el = document.createElement('textarea');
    el.value = value;
    el.setAttribute('readonly', '');
    el.style.position = 'fixed';
    el.style.top = '0';
    el.style.left = '0';
    el.style.opacity = '0';
    document.body.appendChild(el);
    el.select();
    try {
        return document.execCommand('copy');
    } catch (reason) {
        console.warn(`Legacy clipboard fallback failed: ${reason}`);
        return false;
    } finally {
        document.body.removeChild(el);
    }
}
