// Copyright Epic Games, Inc. All Rights Reserved.

export const hexToRgb = (hex: string): string => {

    hex = hex.replace("#", "");

    let hexInt = parseInt(hex, 16);
    let r = (hexInt >> 16) & 255;
    let g = (hexInt >> 8) & 255;
    let b = hexInt & 255;

    return r + "," + g + "," + b;
}


export const isBright = (hex: string): boolean => {

    let rgbColor = hexToRgb(hex);
    let [r, g, b] = rgbColor.split(",").map(n => parseInt(n));

    let luminosity: number = ((r * 0.2126) + (g * 0.7152) + (b * 0.0722)) / 255

    return luminosity > 0.65
}
