// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Resolves a preflight display string from the available fields.
 * Prefers preflightCommitId; returns the numeric value if the entire string is digits,
 * otherwise an abbreviated 8-char hash. Falls back to preflightChange for legacy data.
 * Returns undefined if no preflight info is available.
 */
export function getPreflightDisplay(preflightCommitId?: string, preflightChange?: number): string | undefined {
	if (preflightCommitId) {
		return /^\d+$/.test(preflightCommitId) ? preflightCommitId : preflightCommitId.substring(0, 8);
	}
	if (preflightChange && preflightChange > 0) {
		return preflightChange.toString();
	}
	return undefined;
}

/**
 * Returns the numeric CL for a preflight, or undefined if the preflight uses a non-numeric commit ID.
 * Prefers preflightCommitId; falls back to preflightChange for legacy data.
 */
export function getPreflightCL(preflightCommitId?: string, preflightChange?: number): number | undefined {
	if (preflightCommitId) {
		return /^\d+$/.test(preflightCommitId) ? Number(preflightCommitId) : undefined;
	}
	if (preflightChange && preflightChange > 0) {
		return preflightChange;
	}
	return undefined;
}
