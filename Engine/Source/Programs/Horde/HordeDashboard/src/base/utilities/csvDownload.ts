// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Escapes a CSV cell value (wraps in quotes if needed, escapes internal quotes).
 */
function escapeCsvCell(value: string): string {
	const needsQuotes = /[",\r\n]/.test(value) || value.trim() !== value;
	const escaped = value.replace(/"/g, '""');
	return needsQuotes ? `"${escaped}"` : escaped;
}

/**
 * Builds a CSV string from an array of rows. Each row is an array of cell values.
 * The first row is treated as the header if headers are not provided separately.
 */
function buildCsv(rows: string[][], header?: string[]): string {
	const bodyRows = header ? rows : rows.slice(1);
	const firstRow = header ?? rows[0];
	if (!firstRow?.length) {
		return '';
	}
	const toLine = (row: string[]) => row.map(cell => escapeCsvCell(String(cell ?? ''))).join(',');
	const lines = [toLine(firstRow), ...bodyRows.map(toLine)];
	return lines.join('\n');
}

/**
 * Triggers a file download in the browser using a Blob and a temporary anchor.
 * Use this for client-generated content (e.g. CSV) so the user gets a named file.
 */
function downloadBlob(blob: Blob, filename: string): void {
	const url = URL.createObjectURL(blob);
	const link = document.createElement('a');
	link.href = url;
	link.download = filename;
	link.style.display = 'none';
	document.body.appendChild(link);
	link.click();
	document.body.removeChild(link);
	URL.revokeObjectURL(url);
}

/**
 * Generates a CSV file and triggers a download in the browser.
 *
 * @param rows - Array of rows; each row is an array of cell values (strings or numbers).
 * @param filename - Name of the downloaded file (e.g. "export.csv").
 * @param header - Optional. Pass a separate header row; otherwise the first row is used as header.
 *
 * @example
 * // Header + data rows
 * downloadCsv([
 *   ['Name', 'Status', 'Date'],
 *   ['Test A', 'Pass', '2025-02-17'],
 *   ['Test B', 'Fail', '2025-02-17'],
 * ], 'test-results.csv');
 *
 * @example
 * // From array of objects (build rows yourself)
 * const rows = items.map(i => [i.name, i.status, i.date]);
 * downloadCsv(rows, 'export.csv', ['Name', 'Status', 'Date']);
 */
export function downloadCsv(rows: string[][], filename: string, header?: string[]): void {
	const csv = buildCsv(rows, header);
	// UTF-8 BOM so Excel and other tools recognize encoding
	const bom = '\uFEFF';
	const blob = new Blob([bom + csv], { type: 'text/csv;charset=utf-8' });
	downloadBlob(blob, filename.endsWith('.csv') ? filename : `${filename}.csv`);
}

/**
 * Triggers a download of a raw CSV string (e.g. if you built the CSV yourself).
 *
 * @param csvContent - Full CSV string (can include header line).
 * @param filename - Name of the downloaded file (e.g. "export.csv").
 */
export function downloadCsvContent(csvContent: string, filename: string): void {
	const bom = '\uFEFF';
	const blob = new Blob([bom + csvContent], { type: 'text/csv;charset=utf-8' });
	downloadBlob(blob, filename.endsWith('.csv') ? filename : `${filename}.csv`);
}
