// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Given a URL containing a route path (first hash) and possibly an anchor (second hash),
 * returns only the anchor part, if it exists. (Does not include the query string, if any.)
 * If `url` has no hash, or only has a single hash (for the route path), returns an empty string.
 * @param url - Full or partial URL. Just the hash section is valid, as long as it's prepended with #.
 */
export function extractAnchorLink(url: string): string {
  // URLs containing anchors:
  // #/components/checkbox#Overview
  // http://whatever#/components/checkbox#Overview

  // URLs NOT containing anchors, by this function's definition:
  // #/components/checkbox
  // http://whatever#/components/checkbox
  // #Overview
  // http://whatever#Overview

  if (url.startsWith("#")) {
    return url.trim().split('?')[0];
  }

  const split = url.split('#');
  if (split.length === 3) {
    // Also remove the query if present
    // (technically the query can't be after the hash, but this is likely with hash routing)
    return split[2].split('?')[0];
  }
  return '';
}
