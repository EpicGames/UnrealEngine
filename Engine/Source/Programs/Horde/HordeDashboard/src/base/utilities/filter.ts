// Copyright Epic Games, Inc. All Rights Reserved.

import { GetJobResponse } from "../../backend/Api";

export type JobFilterSimple = {
    filterKeyword?: string;
    showOthersPreflights: boolean;
}

const getJobKeywords = (job: GetJobResponse): string[] => {

    let keywords: string[] = [];

    if (job.name) {
        keywords.push(job.name);
    }

    if (job.change) {
        keywords.push(job.change.toString());
    }

    if (job.preflightChange) {
        keywords.push(job.preflightChange.toString());
    }

    if (job.startedByUserInfo) {
        keywords.push(job.startedByUserInfo.name);
    } else {
        keywords.push("Scheduler");
    }

    job.batches?.forEach(batch => {
        batch.steps.forEach(step => {
            keywords.push(step.name);
        })
    })

    keywords.push(...job.arguments);

    return keywords;

}

export const filterJob = (job: GetJobResponse, keywordIn?: string, additionalKeywords?: string[]): boolean => {

    const keyword = keywordIn?.toLowerCase();

    if (!keyword) {
        return true;
    }

    let keywords = getJobKeywords(job);


    if (additionalKeywords) {
        keywords.push(...additionalKeywords);
    }

    keywords = keywords.map(k => k.toLowerCase());

    const keys = keyword.indexOf(";") !== -1 ? keyword.split(";").map(k => k.trim()).filter(k => k.length) : [keyword];

    let j = 0;
    for (let i = 0; i < keys.length; i++) {
        if (keywords.find(k => k.indexOf(keys[i]) !== -1)) {
            j++;
        }
    }

    return j === keys.length;
}