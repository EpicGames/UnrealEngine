// Copyright Epic Games, Inc. All Rights Reserved.

import { useState, useEffect } from 'react';
import { useSearchParams } from 'react-router-dom';
import LZString from "lz-string"

export function useWindowSize():{width:number, height:number} {

	function getSize() {
		return {
			width: window.innerWidth,
			height: window.innerHeight
		};
	}

	const [windowSize, setWindowSize] = useState(getSize);

	useEffect(() => {

		function handleResize() {
			setWindowSize(getSize());
		}

		window.addEventListener('resize', handleResize);
		return () => window.removeEventListener('resize', handleResize);
	}, []);


	return windowSize;
}

export function useQuery() {
	const [searchParams, setSearchParams] = useSearchParams();

	const get = (key: string): string | null => {
		return searchParams.get(key);
	}

	const getAll = (key: string): string[] => {
		return searchParams.getAll(key);
	}

	const getCopy = (): URLSearchParams => {
		return new URLSearchParams(searchParams);
	}

	const set = (key: string, value: string) => {
		const newParams = new URLSearchParams(searchParams);
		newParams.set(key, value);
		setSearchParams(newParams);
	}

	const has = (key: string, value?: string): boolean => {
		return searchParams.has(key, value);
	}

	const clear = (keepList: string[] = []) => {
		const newParams = new URLSearchParams();

		for(const key of keepList){
			const value = get(key);
			if(value) newParams.set(key, value);
		}

		setSearchParams(newParams);
	}

	const setCompressed = (key: string, value: string) => {
		const compressedValue = LZString.compressToEncodedURIComponent(value);
		set(key, compressedValue);
	}

	const getCompressed = (key: string): string | null => {
		const compressedValue = get(key);
		if(compressedValue){
			return LZString.decompressFromEncodedURIComponent(compressedValue);
		} else {
			console.error("Could not find key \'" + key + "\' to decompress.");
			return null;
		}
	}

	return {get, getAll, getCopy, set, has, clear, setCompressed, getCompressed, raw: {searchParams, setSearchParams}}
}