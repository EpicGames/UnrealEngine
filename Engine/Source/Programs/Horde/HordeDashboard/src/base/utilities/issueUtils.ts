// Copyright Epic Games, Inc. All Rights Reserved.

import moment from "moment";
import { GetIssueResponse, StreamData } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { projectStore } from "../../backend/ProjectStore";
import { displayTimeZone } from "./timeUtils";

export function getIssueStatus(issue: GetIssueResponse, showResolveTime?: boolean): string {

	let text = "";

	if (issue.resolvedAt) {

		let resolvedText = "Resolved";

		if (issue.fixChange) {
			resolvedText += ` in CL ${issue.fixChange}`;
		}

		resolvedText += ` by ${issue.resolvedByInfo?.name ?? "Horde"}`;

		if (!showResolveTime) {
			return resolvedText;
		}

		const displayTime = moment(issue.resolvedAt).tz(displayTimeZone());

		const format = dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z";

		resolvedText += ` on ${displayTime.format(format)}`;

		return resolvedText;
	}

	if (!issue.ownerInfo) {
		text = "Currently unassigned.";
	} else {
		if (issue.ownerInfo.id === dashboard.userId) {
			if (issue.nominatedByInfo) {
				text = `You have been nominated to fix this issue by ${issue.nominatedByInfo.name}.`;
			} else {
				text = `Assigned to ${issue.ownerInfo.name}.`;
			}
		} else {
			text = `Assigned to ${issue.ownerInfo.name}`;
			if (issue.nominatedByInfo) {
				text += ` by ${issue.nominatedByInfo.name}`;
			}
			if (!issue.acknowledgedAt) {
				text += ` (Unacknowledged)`;
			} else {
				text += ` (Acknowledged)`;
			}
			text += ".";
		}
	}

	return text;

}

export function generateStreamSummary(issue: GetIssueResponse): string {

	const streams: StreamData[] = [];

	projectStore.projects.forEach(p => {
		if (!p.streams) {
			return;
		}

		p.streams.filter(s => issue.unresolvedStreams.indexOf(s.id) !== -1).forEach(stream => {
			if (!streams.find(s => s.id === stream.id)) {
				streams.push(stream);
			}
		});
	});

	if (streams.length === 0) {
		return "";
	}

	const present = streams.slice(0, 3);
	const others = streams.slice(3);

	let text = "Affects " + present.map(s => `//${s.project?.name}/${s.name}`).join(", ");

	if (others.length) {
		text += ` and ${others.length} other streams.`;
	}

	return text;

}

