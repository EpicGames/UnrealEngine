// Copyright Epic Games, Inc. All Rights Reserved.

import { extractAnchorLink } from './extractAnchorLink';

const SCROLL_DISTANCE = 52;

export function jumpToAnchor(anchor?: string, scrollDistance: number = SCROLL_DISTANCE): void {
  const hash = anchor || extractAnchorLink(window.location.hash);
  const el = hash && document.getElementById(hash);
  if (hash && el) {
    const elRect = el.getBoundingClientRect();
    const windowY = window.scrollY || window.pageYOffset;
    const currentScrollPosition = windowY + elRect.top;
    const top = currentScrollPosition - scrollDistance;
    if (window.scrollTo) {
      if (window.navigator.userAgent.indexOf('rv:11.0') > -1 || window.navigator.userAgent.indexOf('Edge') > -1) {
        // Edge currently has a bug that jumps to the top of the page if window.scrollTo is passed an oject.
        window.scrollTo(0, top);
      } else {
        window.scrollTo({
          top,
          behavior: 'smooth'
        });
      }
    }
  }
}
