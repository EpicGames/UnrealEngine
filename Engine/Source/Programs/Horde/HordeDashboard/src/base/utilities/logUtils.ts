// Copyright Epic Games, Inc. All Rights Reserved.

import { JobStepBatchState, JobStepState, JobStepError, JobStepOutcome } from "horde/backend/Api";
import { getBatchText } from "horde/components/JobDetailCommon";
import { JobDetailsV2 } from "horde/components/jobDetailsV2/JobDetailsViewCommon";
import { getBatchInitElapsed, getNiceTime, getStepElapsed, getStepETA, getStepTimingDelta, getStepFinishTime } from "./timeUtils";
import { NavigateFunction } from "react-router-dom";

/**
 * Returns batch summary markdown including information like start times, run times, current state, etc.
 * @param jobDetails Job details related to the batch.
 * @param batchId The ID of the batch to summarize.
 */
export const getBatchSummaryMarkdown = (jobDetails: JobDetailsV2, batchId: string): string => {

    const batch = jobDetails.batches.find(b => b.id === batchId);

    if (!batch) {
        return "";
    }

    const agentType = batch?.agentType ?? "";
    const agentPool = jobDetails.stream?.agentTypes[agentType!]?.pool ?? "";

    let summaryText = "";
    if (batch && batch.agentId) {

        if (batch.state === JobStepBatchState.Running || batch.state === JobStepBatchState.Complete) {
            let duration = "";
            if (batch.startTime) {
                duration = getBatchInitElapsed(batch);
                summaryText = `Batch started ${getNiceTime(batch.startTime)} and ${batch.finishTime ? "ran" : "is running"} on [${batch.agentId}](?batch=${batchId}&agentId=${encodeURIComponent(batch.agentId)}) with a setup time of ${duration}.`;
            }
        }
    }

    if (!summaryText) {
        summaryText = getBatchText({ batch: batch, agentId: batch?.agentId, agentType: agentType, agentPool: agentPool }) ?? "";

        if (summaryText && batch.state === JobStepBatchState.Starting) {
            summaryText += " batch is the starting state";
        }

        if (summaryText && batch.state === JobStepBatchState.Stopping) {
            summaryText += " batch is the stopping state";
        }

        if (!summaryText) {
            summaryText = batch?.agentId ?? (batch?.state ?? "Batch is unassigned");
        }
    }

    return summaryText;

}

/**
 * Returns step summary markdown including information about creation, retries, run times, etc.
 * @param jobDetails Job details related to the step.
 * @param stepId The id of the step to summarize.
 * @todo Resolve redundancy with near-identical function in StepDetailSummary.tsx.
 */
export const getStepSummaryMarkdown = (jobDetails: JobDetailsV2, stepId: string): string => {

    const step = jobDetails.stepById(stepId)!;
    const batch = jobDetails.batchByStepId(stepId);

    if (!step || !batch) {
        return "";
    }

    const duration = getStepElapsed(step);
    let eta = getStepETA(step, jobDetails.jobData!);

    const text: string[] = [];

    if (jobDetails.jobData) {
        text.push(`Job created by ${jobDetails.jobData.startedByUserInfo ? jobDetails.jobData.startedByUserInfo.name : "scheduler"}`);
    }

    const batchText = () => {

        if (!batch) {
            return undefined;
        }

        const agentType = batch?.agentType;
        const agentPool = jobDetails.stream?.agentTypes[agentType!]?.pool;
        return getBatchText({ batch: batch, agentType: agentType, agentPool: agentPool });

    };

    if (step.retriedByUserInfo) {
        const retryId = jobDetails.getRetryStepId(step.id);
        if (retryId) {
            text.push(`[Step was retried by ${step.retriedByUserInfo.name}](/job/${jobDetails.jobId!}?step=${retryId})`);
        } else {
            text.push(`Step was retried by ${step.retriedByUserInfo.name}`);
        }
    }

    if (step.abortRequested || step.state === JobStepState.Aborted) {
        eta.display = eta.server = "";
        let aborted = "";
        if (step.abortedByUserInfo) {
            aborted = "This step was canceled";
            aborted += ` by ${step.abortedByUserInfo.name}.`;
        } else if (jobDetails.jobData?.abortedByUserInfo) {
            aborted = "The job was canceled";
            aborted += ` by ${jobDetails.jobData?.abortedByUserInfo.name}.`;
        } else {
            aborted = "This step was canceled by Horde";

            if (step.error === JobStepError.TimedOut) {
                aborted = "The step was canceled due to reaching the maximum run time limit";
            }
        }
        text.push(aborted);
    } else if (step.state === JobStepState.Skipped) {
        eta.display = eta.server = "";
        text.push("The step was skipped");
    } else if (step.state === JobStepState.Ready || step.state === JobStepState.Waiting) {

        text.push(batchText() ?? `The step is pending in ${step.state} state`);
    }

    if (batch?.agentId) {

        if (step.startTime) {
            const str = getNiceTime(step.startTime);
            text.push(`Step started on ${str}`);
        }

        let runningText = `${step.finishTime ? "Ran" : "Running"} on [${batch.agentId}](?step=${stepId}&agentId=${encodeURIComponent(batch.agentId)})`;

        if (duration) {
            runningText += ` for ${duration.trim()}`;
        }

        if (step.finishTime && (step.outcome === JobStepOutcome.Success || step.outcome === JobStepOutcome.Warnings)) {
            const delta = getStepTimingDelta(step);
            if (delta) {
                runningText += " " + delta;
            }
        }

        if (runningText) {
            text.push(runningText);
        }

    } else {
        if (!step.abortRequested) {
            text.push(batchText() ?? "Step does not have a batch.");
        }
    }

    if (eta.display) {
        text.push(`Expected to finish around ${eta.display}.`);
    }

    const finish = getStepFinishTime(step).display;

    if (finish && step.state !== JobStepState.Aborted) {

        let outcome = "";
        if (step.outcome === JobStepOutcome.Success) {
            outcome += `Completed at ${finish}`;
        }
        if (step.outcome === JobStepOutcome.Failure)
            outcome += `Completed with errors at ${finish}.`;
        if (step.outcome === JobStepOutcome.Warnings)
            outcome += `Completed with warnings at ${finish}.`;

        if (outcome) {
            text.push(outcome);
        }
    }

    if (!text.length) {
        text.push(`Step is in ${step.state} state.`);
    }

    return text.join(".&nbsp;&nbsp;");

}

/**
 * Returns the value of any url parameter matching lineIndex or lineindex. Returns undefined if none found.
 */
export function getQueryLine(): number | undefined {

   const search = new URLSearchParams(window.location.search);

   if (search.get("lineindex")) {
      return parseInt(search.get("lineindex")!) + 1;
   }

   if (search.get("lineIndex")) {
      return parseInt(search.get("lineIndex")!);
   }

   return undefined;

}

/**
 * Updates the lineIndex query parameter in the url.
 * @param line The line number to set the query parameter with.
 * @param navigate The navigate function from react router should be passed to this function.
 */
export function updateLineQuery(line: number, navigate: NavigateFunction) {

   const search = new URLSearchParams(window.location.search);
   search.set("lineIndex", line.toString());
   search.delete("lineindex");
   const url = `${window.location.pathname}?` + search.toString();

   navigate(url, { replace: true })

}