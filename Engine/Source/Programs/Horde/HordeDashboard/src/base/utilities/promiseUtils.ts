// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Waits for a potentially asynchronous value to not be null or undefined.
 * @param accessor A simple function that returns the asynchronous value to keep track of.
 * @param interval How often to check the asynchronous value.
 * @returns The value of the property when it is anything but undefined or null.
 */
export const waitFor = async <T>(accessor: () => T, interval = 100): Promise<T> => {
    let value: T;
    while((value = accessor()) == null) {
        await new Promise(resolve => setTimeout(resolve, interval));
    }
    return value;
}