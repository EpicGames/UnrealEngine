// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Remove the anchor link from a full page URL or a hash.
 * Preserves any route path specified in the URL.
 * Behavior with more than two # in the URL is unspecified.
 * If there's a query string after the hash (not technically valid but possible with hash routing),
 * also removes that.
 */
export function removeAnchorLink(url: string): string {
  // First group: most of the URL
  // Second group: optional last hash with only word or dash characters following (an anchor)
  // Third group: optional post-hash query string
  const match = url.match(/^(.*?)((#[\w-]*)(\?.*)?)$/);
  if (match) {
    return match[1];
  }
  return url;
}
