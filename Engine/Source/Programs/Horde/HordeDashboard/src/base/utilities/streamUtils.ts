// Copyright Epic Games, Inc. All Rights Reserved.

import { StreamData } from "../../backend/Api";
import { projectStore } from "../../backend/ProjectStore";

export const getActiveStreamId = (): string | undefined => {

    const path = window.location.pathname?.split("/").filter(c => !!c.trim());
    if (path.length < 2) {
        return undefined;
    }

    if (path[0] !== "stream") {
        return undefined;
    }

    if (!projectStore.streamById(path[1])) {
        return undefined;
    }

    return path[1];

}

/**
 * Extracts the display name (last path component) from an alias name.
 * e.g., "//Project/Stream/AliasName" -> "AliasName"
 */
export const getAliasDisplayName = (aliasName: string): string => {
    const lastSlash = aliasName.lastIndexOf("/");
    return lastSlash >= 0 ? aliasName.substring(lastSlash + 1) : aliasName;
}

/**
 * Returns the display name for a stream in breadcrumbs.
 * When aliasParam is present and matches a stream alias,
 * returns "AliasDisplayName (StreamName)", otherwise returns stream.name
 */
export const getStreamDisplayName = (
    stream: StreamData | undefined,
    aliasParam: string | null
): string => {
    if (!stream) return "Unknown Stream";

    if (aliasParam) {
        const alias = stream.aliases?.find(a => a.name === aliasParam);
        if (alias) {
            return `${getAliasDisplayName(alias.name)} (${stream.name})`;
        }
    }

    return stream.name;
}

/**
 * Builds a stream link, preserving alias parameter if present.
 */
export const getStreamLink = (streamId: string, aliasParam: string | null): string => {
    if (aliasParam) {
        return `/stream/${streamId}?alias=${encodeURIComponent(aliasParam)}`;
    }
    return `/stream/${streamId}`;
}