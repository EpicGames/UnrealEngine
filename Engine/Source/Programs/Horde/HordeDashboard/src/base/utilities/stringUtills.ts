// Copyright Epic Games, Inc. All Rights Reserved.

export function formatBytes(bytes: number, decimals = 2) {
    if (!+bytes) return '0 Bytes'
 
    const k = 1024
    const dm = decimals < 0 ? 0 : decimals
    const sizes = ['Bytes', 'KB', 'MB', 'GB', 'TB', 'PB', 'EB', 'ZB', 'YB']
 
    const i = Math.floor(Math.log(bytes) / Math.log(k))
 
    return `${parseFloat((bytes / Math.pow(k, i)).toFixed(dm))} ${sizes[i]}`
 }
 