// Copyright Epic Games, Inc. All Rights Reserved.

import moment, { Moment } from "moment-timezone";
import { BatchData, GetAgentLeaseResponse, JobData, JobLabel, LabelState, StepData, TimingInfo } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";

export const serverTimeZone = "UTC";
const localTimeZone = moment.tz.guess();

export const displayTimeZone = () => {
    return dashboard.displayUTC ? "UTC" : localTimeZone;
}

export type HordeTime = {
    display: string;
    server: string;
    displayNice?: string;
    serverNice?: string;
}

export const msecToElapsed = (millisec: number, includeMinutes: boolean = true, includeSeconds: boolean = true): string => {

    let duration = "";
    const d = moment.duration(millisec);

    let aggregatedHours = 0;

    if (d.years()) {
        aggregatedHours += d.years() * 365 * 24;
    }

    if (d.months()) {
        aggregatedHours += d.months() * 31 * 24;
    }

    if (d.days()) {
        aggregatedHours += d.days() * 24;
    }

    if (d.hours()) {
        aggregatedHours += d.hours();
    }

    if (aggregatedHours) {
        duration += `${aggregatedHours}h `;
    }

    if (includeMinutes) {

        if (d.minutes()) {
            duration += `${d.minutes()}m `;
        }

        if (d.seconds() && (!duration || includeSeconds)) {
            duration += `${d.seconds()}s `;
        }
    }


    if (duration === "") {
        duration = "0s";
    }

    return duration;
};

function roundTime(date: moment.Moment, duration: moment.Duration) {
    return moment(Math.ceil((+date) / (+duration)) * (+duration));
}

export const getTargetTimingDelta = (timing: TimingInfo | undefined): string => {

    if (!timing) {
        return "";
    }

    const avg = timing.averageTotalTimeToComplete;
    const time = timing.totalTimeToComplete;
    if (avg && time) {

        const seconds = time - avg;
        const tm = msecToElapsed(Math.abs(seconds) * 1000);

        if (seconds > 5) {
            return `(${tm} longer than normal)`;
        } else if (seconds < -5) {
            return `(${tm} shorter than normal)`;
        }

        return `(taking the normal time).`;
    }

    return "";

};

export const getStepTimingDelta = (step: StepData): string => {

    const avg = step.timing?.averageStepDuration;

    if (avg && step.startTime && step.finishTime) {

        const start = moment(step.startTime);
        const finish = moment(step.finishTime);

        const seconds = finish.diff(start, "seconds") - avg;
        const tm = msecToElapsed(Math.abs(seconds) * 1000);


        if (seconds > 5) {
            return `(${tm} longer than normal)`;
        } else if (seconds < -5) {
            return `(${tm} shorter than normal)`;
        }

        return `(taking the normal time)`;
    }

    return "";

};


export const getStepETA = (step: StepData, job: JobData): HordeTime => {

    if (step.finishTime || !step.timing || !step.timing.totalTimeToComplete) {
        return {
            display: "",
            server: ""
        };
    }

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    let end = moment(job.createTime).add(step.timing.totalTimeToComplete, "seconds");

    let now = moment(Date.now());

    const seconds = end.diff(now, "seconds");
    
    // if less than a minute, push out
    if (seconds < 60) {
        end = now.add(5, "minutes");
    }

    end = roundTime(end, moment.duration(5, "minutes"));

    const serverETA = moment.utc(end).tz(serverTimeZone);
    const displayETA = moment.utc(end).tz(displayTimeZone());

    return {
        display: displayETA.format(format),
        server: serverETA.format(format),
        displayNice: getNiceTime(displayETA.toDate()),
        serverNice: getNiceTime(serverETA.toDate())
    };

};

export const getStepFinishTime = (step: StepData): HordeTime => {

    let displayFinish = "";
    let serverFinish = "";
    let displayNice: string | undefined;
    let serverNice: string | undefined;

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    if (step.finishTime) {
        const end = moment(step.finishTime);
        const dfinish = moment.utc(end).tz(displayTimeZone());
        const sfinish = moment.utc(end).tz(serverTimeZone);

        displayFinish = dfinish.format(format);
        serverFinish = sfinish.format(format);
        displayNice = getNiceTime(dfinish.toDate());
        serverNice = getNiceTime(sfinish.toDate());
    }

    return {
        display: displayFinish,
        server: serverFinish,
        displayNice: displayNice,
        serverNice: serverFinish
    };
};

export const getStepStartTime = (step: StepData): HordeTime => {

    let displayStart = "";
    let serverStart = "";
    let displayNice: string | undefined;
    let serverNice: string | undefined;

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    if (step.startTime) {
        const end = moment(step.startTime);
        const dstart = moment.utc(end).tz(displayTimeZone());
        const sstart = moment.utc(end).tz(serverTimeZone);

        displayStart = dstart.format(format);
        serverStart = sstart.format(format);
        displayNice = getNiceTime(dstart.toDate());
        serverNice = getNiceTime(sstart.toDate());

    }

    return {
        display: displayStart,
        server: serverStart,
        displayNice: displayNice,
        serverNice: serverNice
    };
};


export const getLabelFinishTime = (label: JobLabel, job: JobData): HordeTime => {

    let displayFinish = "";
    let serverFinish = "";

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    if (label.stateResponse.state === LabelState.Complete && label.timing) {
        const end = moment(job.createTime).add(label.timing.totalTimeToComplete, "seconds");
        displayFinish = moment.utc(end).tz(displayTimeZone()).format(format);
        serverFinish = moment.utc(end).tz(serverTimeZone).format(format);
    }

    return {
        display: displayFinish,
        server: serverFinish
    };
};

export const getLabelETA = (label: JobLabel, job: JobData): HordeTime => {

    let displayEta = "";
    let serverEta = "";

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    if (label.timing && label.timing.totalTimeToComplete && label.stateResponse.state === LabelState.Running) {
        const end = moment(job.createTime).add(label.timing.totalTimeToComplete, "seconds");
        serverEta = moment.utc(end).tz(serverTimeZone).format(format);
        displayEta = moment.utc(end).tz(displayTimeZone()).format(format);
    }

    return {
        display: displayEta,
        server: serverEta
    };

};


export const getStepPercent = (step: StepData): number | undefined => {

    const timing = step.timing;

    if (!step.startTime || !timing?.averageStepDuration) {
        return undefined;
    }

    if (step.finishTime) {
        return 1;
    }

    const start = moment(step.startTime);
    const end = moment(Date.now());
    const duration = moment.duration(end.diff(start)).asSeconds();

    // 90% weighted from estimate, meaning if finish on time will jump 90% => done
    let percent = duration / timing.averageStepDuration * 0.9;
    if (percent >= 0.8) {

        // sigmoid that hits 99% at 1.5x duration
        percent = Math.min((2 / (1 + Math.pow(40, -percent)) - 1), 0.99);
    }

    return percent;
};

export const getElapsedString = (start: Moment, end: Moment, includeSeconds: boolean = true): string => {

    let duration = "";
    const d = moment.duration(end.diff(start));

    if (d.years()) {
        duration = `${d.years()} year`;
        if (d.years() > 1) {
            duration += "s";
        }
    }
    if (d.months()) {
        duration += ` ${d.months()} month`;
        if (d.months() > 1) {
            duration += "s";
        }
        return duration;
    }

    if (!d.years() && !d.months()) {

        if (d.days()) {
            duration += `${d.days()}d `;
        }

        if (d.hours()) {
            duration += `${d.hours()}h `;
        }

        if (d.minutes()) {
            duration += `${d.minutes()}m `;
        }

        if (includeSeconds || !duration) {
            if (d.seconds() > 0) {
                duration += `${d.seconds()}s `;
            } else {
                duration += `0s `;
            }
        }
    }

    return duration?.trim();

};

export const getBatchInitElapsed = (batch: BatchData | undefined): string => {

    if (!batch || !batch.startTime) {
        return "";
    }

    const start = moment(batch.startTime);
    let end = moment(Date.now());

    if (!batch.steps.find(step => step.startTime) && batch.finishTime) {
        end = moment(batch.finishTime);
    } else {
        batch.steps.forEach(step => {

            if (!step.startTime) {
                return;
            }

            const time = moment(step.startTime);

            if (time.unix() < end.unix()) {
                end = time;
            }
        });
    }


    return getElapsedString(start, end);
};

export const getStepElapsed = (step: StepData | BatchData | undefined): string => {

    if (!step) {
        return "";
    }

    const start = moment(step.startTime);
    let end = moment(Date.now());

    if (step.finishTime) {
        end = moment(step.finishTime);
    }

    return getElapsedString(start, end);

};

export const getLeaseElapsed = (lease: GetAgentLeaseResponse | undefined): string => {

    if (!lease) {
        return "";
    }

    const start = moment(lease.startTime);
    let end = moment(Date.now());

    if (lease.finishTime) {
        end = moment(lease.finishTime);
    }

    return getElapsedString(start, end);

};


export const getHumanTime = (timeIn: Date | string | undefined): string => {

    if (!timeIn) {
        return "";
    }

    const now = moment.utc().tz(displayTimeZone());
    const time = moment(timeIn).tz(displayTimeZone());

    const nowDay = now.dayOfYear();
    const timeDay = time.dayOfYear();

    const delta = nowDay - timeDay;

    if (delta > 2 || now.year() !== time.year()) {
        return time.format('MMM Do');
    }

    if (delta === 1) {
        return 'Yesterday';
    }

    if (delta === 0) {
        return 'Today';
    }

    if (time.day() === 1) {
        return "Monday";
    }

    let timeStr =  time.format('MMM Do');

    if (time.year() !== now.year()) {
        timeStr += `, ${time.year()}`;
    }

    return timeStr;

};


export const getShortNiceTime = (timeIn: Date | string | undefined, relative: boolean = false, includeHour: boolean = false, includeDay: boolean = true): string => {

    if (!timeIn) {
        return "";
    }

    const now = moment.utc().tz(displayTimeZone());
    const nowTimeStr = now.format('MMM Do');

    const time = moment(timeIn).tz(displayTimeZone());

    let timeStr = time.format('MMM Do');
    if (relative && timeStr === nowTimeStr) {
        timeStr += " (Today)";
    } else if (relative && time.calendar().toLowerCase().indexOf("yesterday") !== -1) {
        timeStr += " (Yesterday)";
    } else {
        timeStr = time.format('MMM Do');
    }

    if (includeHour) {
        const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";
        if (includeDay)
            timeStr += ` at ${time.format(format)}`;
        else
            timeStr = time.format(format);
    }

    if (time.year() !== now.year()) {
        timeStr += `, ${time.year()}`;
    }

    return timeStr;

};

export const getNiceTime = (timeIn: Date | string | undefined, relative: boolean = true): string => {

    if (!timeIn) {
        return "";
    }

    const now = moment.utc().tz(displayTimeZone());
    const nowTimeStr = now.format('dddd, MMMM Do');

    const time = moment(timeIn).tz(displayTimeZone());

    let timeStr = time.format('dddd, MMMM Do');
    if (relative && timeStr === nowTimeStr) {
        timeStr += " (Today)";
    } else if (relative && time.calendar().toLowerCase().indexOf("yesterday") !== -1) {
        timeStr += " (Yesterday)";
    } else {
        timeStr = time.format('dddd, MMMM Do');
    }

    const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

    timeStr += ` at ${time.format(format)}`;

    if (time.year() !== now.year()) {
        timeStr += `, ${time.year()}`;
    }

    return timeStr;

};

export const getMongoIdDate = (mongoId: string): Date | undefined => {

    if (!mongoId || mongoId.length < 8) {
        return undefined;
    }

    const timestamp = mongoId.substring(0, 8)
    return new Date(parseInt(timestamp, 16) * 1000);
}