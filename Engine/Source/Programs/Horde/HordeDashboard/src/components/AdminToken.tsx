// Copyright Epic Games, Inc. All Rights Reserved.

import React, { useState } from 'react';
import {Text, Stack, Spinner, SpinnerSize, PrimaryButton} from '@fluentui/react';
import backend from '../backend';
import {copyToClipboard} from "../base/utilities/clipboard";


export const AdminToken: React.FC = () => {

	const [token, setToken] = useState("");
	const [tokenError, setTokenError] = useState("");	

	
	if (!token && !tokenError) {
		backend.getAdminToken().then(token => setToken(token)).catch(reason => setTokenError(reason));
		return <Stack horizontal verticalAlign="center" tokens={{childrenGap:24}} styles={{root:{padding:24}}}>
			<Text variant="mediumPlus">Getting Token...</Text>
			<Spinner size={SpinnerSize.large} />
		</Stack>;
	}

	return (
		<Stack horizontalAlign="center" styles={{ root: { width: "100%", padding: 12 } }}>
			<Stack styles={{ root: { width: 1440 } }} tokens={{ childrenGap: 18 }}>
				<h1>Horde Access Token</h1>
				{token && <PrimaryButton onClick={() => copyToClipboard(token)} text="Copy to clipboard" style={{ width: 200 }} />}
				{!token && <div>Error getting token</div>}
				<textarea readOnly={true} style={{ wordBreak: "break-word", height: 400 }} value={token ? token : tokenError}></textarea>
			</Stack>
		</Stack>
	);
};