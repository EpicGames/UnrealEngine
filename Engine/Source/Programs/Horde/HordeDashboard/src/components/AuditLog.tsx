// Copyright Epic Games, Inc. All Rights Reserved.
import { ComboBox, DefaultButton, Dropdown, FocusZone, FocusZoneDirection, IComboBoxOption, List, mergeStyleSets, SearchBox, SelectableOptionMenuItemType, Selection, SelectionMode, SelectionZone, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { action, makeObservable, observable } from 'mobx';
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useEffect, useMemo, useRef, useState } from "react";
import Highlight from "react-highlighter";
import { useParams } from "react-router";
import { Link } from "react-router-dom";
import backend from "../backend";
import { AuditLogEntry } from "../backend/Api";
import dashboard from "../backend/Dashboard";
import { useWindowSize } from "../base/utilities/hooks";
import { displayTimeZone } from "../base/utilities/timeUtils";
import { BreadcrumbItem, Breadcrumbs } from "./Breadcrumbs";
import { DateTimeRange } from "./DateTimeRange";
import { HistoryModal } from "./agents/HistoryModal";
import { IssueModalV2 } from "./IssueViewV2";
import { getLogStyles, logMetricNormal } from "./logs/LogStyle";
import { TopNav } from "./TopNav";
import { getHordeStyling } from "../styles/Styles";
import { getHordeTheme } from "../styles/theme";
import { projectStore } from "horde/backend/ProjectStore";


let _auditStyleNormal: any;

const getAuditStyleNormal = () => {

   const theme = getHordeTheme();
   const { logStyleNormal } = getLogStyles();

   const auditStyleNormal = _auditStyleNormal ?? mergeStyleSets(logStyleNormal, {

      logLine: [
         {
            selectors: {
               '&:hover': { background: theme.palette.neutralLight }
            },
         }
      ],
      container: {
         selectors: {
            '.ms-List-cell': {
               height: "auto",
            }
         }
      }

   });

   _auditStyleNormal = auditStyleNormal;

   return auditStyleNormal;
}

export type TemplateAudit = {
   streamId: string;
   templateId: string;
}

export type AuditLogItem = {
      entry: AuditLogEntry;
      type: string;
   }


class AuditLogHandler {

   constructor() {
      makeObservable(this);
      this.clear();
   }


   clear() {

      this.agentId = undefined;
      this.issueId = undefined;
      this.deviceId = undefined;
      this.templateAudit = undefined;
      this.jobId = undefined;
      this.itemAudit = undefined;

      this.timeSelectKey = "time_live_tail";

      this.typedEntries = [];
      this.typeOptions = [];

      clearTimeout(this.timeoutId);
      this.timeoutId = undefined;

      // cancel any pending
      for (let i = 0; i < this.cancelId; i++) {
         this.canceled.add(i);
      }

      this.updating = false;
      this.haveUpdated = false;

   }

   get tailing(): boolean {
      return this.timeSelectKey === "time_live_tail";
   }

   setAgent(agentId?: string) {

      if (!agentId) {
         this.clear();
         return;
      }

      if (this.agentId === agentId) {
         return;
      }

      this.agentId = agentId;
      this.update();
   }

   setIssue(issueId?: string) {

      if (!issueId) {
         this.clear();
         return;
      }

      if (this.issueId === issueId) {
         return;
      }

      this.issueId = issueId;
      this.update();
   }

   setDevice(deviceId?: string) {

      if (!deviceId) {
         this.clear();
         return;
      }

      if (this.deviceId === deviceId) {
         return;
      }

      this.deviceId = deviceId;
      this.update();
   }

   setJob(jobId?: string) {

      if (!jobId) {
         this.clear();
         return;
      }

      if (this.jobId === jobId) {
         return;
      }

      this.jobId = jobId;
      this.update();
   }

   setTemplateAudit(templateAudit?: TemplateAudit) {

      if (!templateAudit) {
         this.clear();
         return;
      }

      if (this.templateAudit?.streamId === templateAudit.streamId && this.templateAudit.templateId === templateAudit.templateId) {
         return;
      }

      this.templateAudit = templateAudit;
      this.update();
   }

   setItem(itemAudit?: ItemAuditLogConfig) {

      if (!itemAudit) {
         this.clear();
         return;
      }

      if (this.itemAudit?.id === itemAudit.id) {
         return;
      }

      this.itemAudit = itemAudit;
      this.update();
   }

   get isAgentLog(): boolean {
      return !!this.agentId;
   }

   get isIssueLog(): boolean {
      return !!this.issueId;
   }

   get isDeviceLog(): boolean {
      return !!this.deviceId;
   }

   get isJobLog(): boolean {
      return !!this.jobId;
   }

   get isTemplateAuditLog(): boolean {
      return !!this.templateAudit;
   }

   get isGenericItemLog(): boolean {
      return !!this.itemAudit;
   }

   async update() {

      clearTimeout(this.timeoutId);

      if (this.tailing) {

         this.timeoutId = setTimeout(() => { this.update(); }, this.pollTime);

         if (this.updating) {
            return;
         }

      }

      // cancel any pending
      for (let i = 0; i < this.cancelId; i++) {
         this.canceled.add(i);
      }

      if (!this.agentId && !this.issueId && !this.templateAudit && !this.deviceId && !this.jobId && !this.itemAudit) {
         return;
      }


      try {

         this.updating = true;

         const cancelId = this.cancelId++;

         let minTime = this.minDate;
         let maxTime = this.maxDate;

         if (this.tailing) {

            if (this.issueId) {
               minTime = undefined;
            } else if (this.agentId) {
               // live tail of the last 4 days
               // We need to optimize the agent history endpoint, this was an attempt to limit results though doesn't work 
               minTime = undefined; //new Date(new Date().valueOf() - (60 * 24 * 4 * 60000));
            } else if (this.deviceId) {
               minTime = undefined;
            } else if (this.templateAudit) {
               minTime = undefined;
            }

            maxTime = new Date();
         }

         let entries: AuditLogEntry[] = [];

         if (this.agentId) {
            entries = await backend.getAgentHistory(this.agentId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 });
         }

         if (this.deviceId) {
            entries = await backend.getDeviceHistory(this.deviceId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 });
         }

         if (this.issueId) {
            entries = await backend.getIssueHistory(this.issueId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 });
         }

         if (this.templateAudit) {
            entries = await backend.getTemplateHistory(this.templateAudit.streamId, this.templateAudit.templateId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 });
         }

         if (this.jobId) {
            entries = await backend.getJobHistory(this.jobId, { minTime: minTime?.toISOString(), maxTime: maxTime?.toISOString(), count: 8192 * 2 });
         }

         if (this.itemAudit) {
            entries = await this.itemAudit.getHistory(this.itemAudit.id, minTime, maxTime);
         }

         // check for canceled during graph request
         if (this.canceled.has(cancelId)) {
            return;
         }

         this.entries = entries;
         this.typedEntries = this.deriveTypedEntries();

         this.haveUpdated = true;

         this.setUpdated();

      } catch (reason) {

         console.error(reason);

      } finally {

         this.updating = false;
      }

   }

   /**
    * Creates an array of @see AuditLogItem objects and calls @see this.updateTypesIfChanged.
    * @returns Sorted array of typed entries for the current audit log.
    */
   deriveTypedEntries() {
      let types: Set<string> = new Set<string>();
      let typedEntries = this.entries.map((entry) => {

         let type = entry.level.toString();
         const TYPE_PROPERTY = "Type";
         const COMPUTE_TYPE = "Compute";
         if (entry.properties && entry.properties[TYPE_PROPERTY]) {
            type = entry.properties[TYPE_PROPERTY] as string;
         } else {
            // @todo:
            if (entry.message.startsWith(COMPUTE_TYPE)) {
               type = COMPUTE_TYPE;
            }
         }

         types.add(type);

         return { entry: entry, type: type }
      })

      this.updateTypesIfChanged(types);

      // Display issue and job audit logs in chronological order, entries in all other audit logs are sorted from most recent to least recent to better surface information
      if (this.issueId || this.jobId) {
         typedEntries.reverse();
      }

      return typedEntries
   }

    /**
    * Helper function to update the types present in the audit log, if they have changed since the last handler update.
    * @param newTypes The set of new types found in the latest fetch of audit log entries.
    * @returns Sorted array of typed entries for the current audit log.
    * @remark The reassignment of @see this.typeOptions is intentional to update the ref and let the @see AuditLogPanel component know the set of types has changed.
    */
   @action
   updateTypesIfChanged(newTypes: Set<string>) {
      // Types unchanged, no need to update
      if (this.typeOptions.length === newTypes.size && this.typeOptions.every(type => newTypes.has(type))) {
        return;
      }

      this.typeOptions = [...newTypes];
   }

   setTimeSelection(time: TimeSelection) {

      const live = time.key === "time_live_tail";
      const calendar = time.key === "time_select_calendar";

      if (calendar) {
         console.error("Should not be setting time_select_calendar in setTimeSelection");
      }

      // @todo
      if (live) {
         this.timeSelectKey = time.key;
         this.minDate = this.maxDate = undefined;
         this.haveUpdated = false;
         this.update();
         this.setUpdated();
         return;
      }

      this.minDate = new Date(new Date().valueOf() - (time.minutes * 60000));
      this.maxDate = new Date();
      this.timeSelectKey = time.key;
      this.haveUpdated = false;
      this.update();
      this.setUpdated();

   }

   setTimeRange(minDate: Date, maxDate: Date) {

      this.minDate = minDate;
      this.maxDate = maxDate;

      this.timeSelectKey = "time_select_calendar";

      this.haveUpdated = false;
      this.update();

      this.setUpdated();

   }

   agentId?: string;
   issueId?: string;
   deviceId?: string;
   jobId?: string;
   templateAudit?: TemplateAudit;
   itemAudit?: ItemAuditLogConfig;

   timeSelectKey?: string;

   typedEntries: AuditLogItem[];

   @observable
   typeOptions: string[] = [];

   minDate?: Date;
   maxDate?: Date;

   scroll?: number;

   entries: AuditLogEntry[] = [];

   haveUpdated?: boolean;

   @action
   setUpdated() {
      this.updated++;
   }

   @observable
   updated: number = 0;

   updating = false;
   private timeoutId: any;

   private canceled = new Set<number>();
   private cancelId = 0;

   private pollTime = 15000;

}

const handler = new AuditLogHandler();

type TimeSelection = {
   text: string;
   key: string;
   minutes: number;
}

const timeSelections: TimeSelection[] = [
   {
      text: "Live Tail", key: "time_live_tail", minutes: 0
   },
   {
      text: "Past 15 Minutes", key: "time_15_minutes", minutes: 15
   },
   {
      text: "Past 1 Hour", key: "time_1_hour", minutes: 60
   },
   {
      text: "Past 4 Hours", key: "time_4_hours", minutes: 60 * 4
   },
   {
      text: "Past 1 Day", key: "time_1_day", minutes: 60 * 24
   },
   {
      text: "Past 2 Days", key: "time_2_days", minutes: 60 * 24 * 2
   },
   {
      text: "Past 1 Week", key: "time_1_week", minutes: 60 * 24 * 7
   },
   {
      text: "Past 1 Month", key: "time_1_month", minutes: 60 * 24 * 31 // yeah, not all months have 31 days, sheesh 
   },
   {
      text: "Select from Calendar", key: "time_select_calendar", minutes: 0
   }
]

const selection = new Selection({ selectionMode: SelectionMode.multiple });

export const AuditLogPanel: React.FC<{ agentId?: string, deviceId?: string, jobId?: string, issueId?: string, templateAudit?: TemplateAudit, itemAudit?: ItemAuditLogConfig }> = observer(({ agentId, deviceId, jobId, issueId, templateAudit, itemAudit }) => {

   const windowSize = useWindowSize();
   const [showDatePicker, setShowDatePicker] = useState(false);
   const [tsFormat, setTSFormat] = useState(dashboard.displayUTC ? 'UTC' : 'Local');
   const [search, setSearch] = useState<string | undefined>(undefined);
   const [selectedTypes, setSelectedTypes] = useState<string[]>([]);
   const [typeModifiedByUser, setTypeModifiedByUser] = useState<boolean>(false);
   const [viewAgent, setViewAgent] = useState(false);
   const [viewIssue, setViewIssue] = useState(false);


   useEffect(() => {

      return () => {
         handler.clear();
      };

   }, []);

   const auditStyleNormal = getAuditStyleNormal();
   const { modeColors } = getHordeStyling();

   // Read handler.updated so MobX knows to re-ender the component if it has changed
   if (handler.updated) { }

   if (agentId && handler.agentId !== agentId) {
      handler.setAgent(agentId);
   }

   if (deviceId && handler.deviceId !== deviceId) {
      handler.setDevice(deviceId);
   }

   if (jobId && handler.jobId !== jobId) {
      handler.setJob(jobId);
   }

   if (issueId && handler.issueId !== issueId) {
      handler.setIssue(issueId);
   }

   if (templateAudit) {
      handler.setTemplateAudit(templateAudit);
   }

   if (itemAudit) {
      handler.setItem(itemAudit);
   }

   if (!agentId && !deviceId && !issueId && !templateAudit && !jobId && !itemAudit) {
      return null;
   }

   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const searchTerm = search?.toLowerCase();

   // filter
   const logItems: AuditLogItem[] = handler.typedEntries?.filter(item => {

      if (selectedTypes.indexOf("All") === -1) {
         if (selectedTypes.indexOf(item.type) === -1) {
            return false;
         }
      }

      if (!searchTerm) {
         return true;
      }

      if (item.entry.message.toLowerCase().indexOf(searchTerm) === -1) {
         return false;
      }

      return true;

   });

   // keep type data and selection logic in this component to avoid re-rendering log items on changes to selection
   const typeOptions: IComboBoxOption[] = useMemo(() => {
      
      let options: IComboBoxOption[]  = Array.from(handler.typeOptions).map(type => {
         return {
            key: type,
            text: type
         };
      }).sort((a, b) => { if (a.text === b.text) return 0; return a.text < b.text ? -1 : 1 })
         
      if (options.length > 1) {
         options.unshift({
            key: 'All',
            text: 'Select All',
            itemType: SelectableOptionMenuItemType.SelectAll
         });
      }

      return options;

   }, [handler.typeOptions]);

   const prevAllTypesSelected = useRef(false);
   const allTypesSelected =  selectedTypes.length > 1 && selectedTypes.length === handler.typeOptions.length;

   // derive selected keys at render time to drive selectAll behaviour and avoid adding sentinel selectAll node to array of actual types
   let selectedTypesDisplayed: string[] = [...selectedTypes];
   if (allTypesSelected) {
      selectedTypesDisplayed.push('All');
   }

   // types supplied from the backend, so we need to maintain which types should be disbaled by default across all audit logs
   const disabledByDefaultTypes = ["Trace", "Debug"];

   useEffect(() => {

      // set defaults only if user hasn't interacteds
      if (!typeModifiedByUser){
         const defaultTypes: string[] = Array.from(handler.typeOptions).filter(option => !disabledByDefaultTypes.includes(option));
         setSelectedTypes(defaultTypes);
      }

      // automatically select new options if a new type is added and all options were selected previously
      if (prevAllTypesSelected.current) {
         setSelectedTypes([...handler.typeOptions]);
      }

   }, [handler.typeOptions]);

   // keep track of allTypesSelected value from previous render
   useEffect(() => {
      prevAllTypesSelected.current = allTypesSelected;
   });

   let crumbItems: BreadcrumbItem[] = [];
   if (agentId) {
      crumbItems = [
         {
            text: `Agents`,
            link: '/agents'
         },
         {
            text: `Agent Audit - ${agentId}`,
            link: `/agents?agentId=${encodeURIComponent(agentId)}`
         }
      ];
   }

   if (deviceId) {
      crumbItems = [
         {
            text: `Devices`,
            link: '/devices'
         },
         {
            text: `Device Audit - ${deviceId.toUpperCase()}`,
            link: `/devices?deviceId=${encodeURIComponent(deviceId)}`
         }
      ];
   }

   if (jobId) {
      crumbItems = [
         {
            text: `Job`,
            link: `/job/${jobId}`
         },
         {
            text: `Audit - ${jobId}`,
            link: `/job/${jobId}`
         }
      ];
   }


   if (issueId) {
      crumbItems = [
         {
            text: `Issues`,
         },
         {
            text: `Audit - Issue ${issueId}`,
         }
      ];

   }

   if (templateAudit) {
      const stream = projectStore.streamById(templateAudit.streamId)!;
      const templateName = stream?.templates.find(t => t.id === templateAudit.templateId)?.name;

      crumbItems = [
         {
            text: `Template`,
         },
         {
            text: `Audit - ${stream.fullname ?? templateAudit.streamId} - ${templateName ?? templateAudit.templateId}`,
         }
      ];
   }

   if (itemAudit) {
      crumbItems = itemAudit.breadCrumbs;
   }

   const onRenderCell = (item?: AuditLogItem, index?: number, isScrolling?: boolean): JSX.Element => {
      const entry = item!.entry;

      let tm = moment.utc(entry.time);
      if (tsFormat === 'Local') {
         tm = tm.local();
      }

      const format = dashboard.display24HourClock ? "MMM DD HH:mm:ss" : "MMM DD hh:mm:ss A";

      let timestamp = `[${tm.format(format)}]`;

      return (
         <Stack className={auditStyleNormal.logLine} key={`key_log_line_${item?.entry.time}`} style={{ width: "100%" }}>
            <div style={{ position: "relative" }}>
               <Stack tokens={{ childrenGap: 8 }} horizontal disableShrink={true}>
                  <Stack horizontal disableShrink={true} >
                     <Stack styles={{ root: { width: 140, whiteSpace: "nowrap", fontSize: logMetricNormal.fontSize, userSelect: "none" } }}> {timestamp}</Stack>
                     <Stack styles={{ root: { color: "#8a8a8a", width: 92, paddingRight: 8, whiteSpace: "nowrap", textAlign: "right", fontSize: logMetricNormal.fontSize, userSelect: "none" } }}> [{item!.type}]</Stack>
                     <div className={auditStyleNormal.logLineOuter}> <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8 } }}> {renderAuditEntry(entry, search, itemAudit?.lineFactory)}</Stack></div>
                  </Stack>
               </Stack>
            </div>
         </Stack>
      );
   }

   let timeComboText: string | undefined;
   let timeComboWidth = 180;

   if (handler.timeSelectKey === "time_select_calendar") {

      let format = dashboard.display24HourClock ? "MMM DD H:mm z" : "MMM DD h:mm A z";
      timeComboText = moment(handler.minDate).tz(displayTimeZone()).format(format);
      timeComboText += " / " + moment(handler.maxDate).tz(displayTimeZone()).format(format);
      timeComboWidth = 350;
   }

   return <Stack>
      <Breadcrumbs items={crumbItems} />
      <Stack tokens={{ childrenGap: 12 }} style={{ backgroundColor: modeColors.background }}>
         <Stack horizontal>
            <div key={`windowsize_logview1_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2) - 990, flexShrink: 0, backgroundColor: modeColors.background }} />
            <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, margin: "auto", paddingTop: 12, paddingRight: 10 } }}>
               <Stack horizontal styles={{ root: { paddingLeft: 0, paddingBottom: 4, paddingRight: 12, width: 1440 } }}>
                  <Stack horizontal tokens={{ childrenGap: 12 }}>

                        <SearchBox style={{ width: 400 }} onEscape={() => setSearch(undefined)} autoComplete="off" disableAnimation={true} spellCheck={false} onChange={(ev, value) => setSearch(value)} />

                        <ComboBox style={{ width: 240 }} multiSelect options={typeOptions} selectedKey={selectedTypesDisplayed}
                        onChange={(event, option, index) => { 
                           setTypeModifiedByUser(true);
                           const selected = option?.selected;

                           if (option) {
                              // toggle selectAll
                              if (option?.itemType === SelectableOptionMenuItemType.SelectAll) {
                                 allTypesSelected ? setSelectedTypes([]) : setSelectedTypes([...handler.typeOptions]);
                              } else {
                              // toggle individual type
                              let updatedKeys = selected ? [...selectedTypes, option!.key as string] : selectedTypes.filter(k => k !== option.key);
                              if (updatedKeys.length !== selectedTypes.length) {
                                 updatedKeys = updatedKeys.filter(k => k!== 'All');
                              }
                              setSelectedTypes(updatedKeys);
                              }
                           }
                        }}
                        />
                  </Stack>

                  <Stack grow />
                  <Stack horizontalAlign={"end"}>
                     <Stack>
                        <Stack verticalAlign="center" horizontal tokens={{ childrenGap: 24 }} styles={{ root: { paddingRight: 4, paddingTop: 4 } }}>
                           <Stack horizontal tokens={{ childrenGap: 24 }}>
                              <Stack>
                                 <ComboBox
                                    styles={{ root: { width: timeComboWidth } }}
                                    options={timeSelections}
                                    text={timeComboText}
                                    selectedKey={handler.timeSelectKey}
                                    onItemClick={(ev, option, index) => {
                                       if (!option) {
                                          return;
                                       }

                                       if (option.key === "time_select_calendar") {
                                          setShowDatePicker(true);
                                       }
                                    }}
                                    onChange={(ev, option, index, value) => {
                                       const select = option as TimeSelection;

                                       if (select.key === "time_select_calendar") {

                                       } else {
                                          handler.setTimeSelection(select);
                                       }

                                    }}
                                 />
                              </Stack>
                              <Stack>
                                 <Dropdown
                                    styles={{ root: { width: 92 } }}
                                    options={[{ key: 'Local', text: 'Local' }, { key: 'UTC', text: 'UTC' }]}
                                    defaultSelectedKey={tsFormat}
                                    onChanged={(value) => {
                                       setTSFormat(value.key as string);
                                       // Audit fix me
                                       //listRef?.forceUpdate();
                                    }}
                                 />
                              </Stack>
                              <Stack>
                                 {handler.isAgentLog && <DefaultButton style={{ fontFamily: "Horde Open Sans SemiBold" }} text="View Agent" onClick={() => setViewAgent(true)} />}
                                 {handler.isIssueLog && <DefaultButton style={{ fontFamily: "Horde Open Sans SemiBold" }} text="View Issue" onClick={() => setViewIssue(true)} />}
                              </Stack>

                           </Stack>
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>

         <Stack style={{ backgroundColor: modeColors.background, paddingLeft: "24px", paddingRight: "24px" }}>
            <Stack tokens={{ childrenGap: 0 }}>
               {showDatePicker && < DateTimeRange onChange={(minDate, maxDate) => { handler.setTimeRange(minDate, maxDate); setShowDatePicker(false) }} onDismiss={() => { handler.setTimeSelection({ text: "Live Tail", key: "time_live_tail", minutes: 0 }); setShowDatePicker(false) }} />}
               {viewAgent && <HistoryModal agentId={agentId} onDismiss={() => { setViewAgent(false) }} />}
               {viewIssue && <IssueModalV2 issueId={issueId} onCloseExternal={() => { setViewIssue(false); }} popHistoryOnClose={false} />}
               <FocusZone direction={FocusZoneDirection.vertical} isInnerZoneKeystroke={() => { return true; }} defaultActiveElement="#LogList" style={{ padding: 0, margin: 0 }} >
                  <div className={auditStyleNormal.container} style={{ height: 'calc(100vh - 260px)', position: 'relative' }} data-is-scrollable={true}>
                     <Stack horizontal>
                        <div key={`windowsize_logview2_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2) - 24, flexShrink: 0 }} />
                        <Stack styles={{ root: { backgroundColor: modeColors.background, paddingLeft: "0px", paddingRight: "0px" } }}>
                           {!handler.haveUpdated && <Spinner size={SpinnerSize.large} />}
                           {!!handler.haveUpdated && !logItems.length && <Stack style={{ paddingLeft: 0 }}><Text variant="mediumPlus">No audit entries found</Text></Stack>}

                           {!!handler.haveUpdated && !!logItems.length &&
                              <SelectionZone selection={selection} selectionMode={SelectionMode.multiple}>
                                 <List key={`audit_log_list_key`} id="LogList"
                                    items={logItems}
                                    // NOTE: getPageSpecification breaks initial scrollToIndex when query contains lineIndex!
                                    getPageHeight={() => 10 * (logMetricNormal.lineHeight)}
                                    onShouldVirtualize={() => { return true; }}
                                    onRenderCell={onRenderCell}
                                    data-is-focusable={true} />
                              </SelectionZone>
                           }
                        </Stack>
                     </Stack>
                  </div>
               </FocusZone>
            </Stack>
         </Stack>
      </Stack>
   </Stack>

});

export const AuditLogView: React.FC = () => {

   const { hordeClasses } = getHordeStyling();

   const { agentId, deviceId, jobId, issueId, streamId, templateId } = useParams<{ agentId: string, deviceId: string, jobId, issueId: string, streamId: string, templateId: string }>();

   let templateAudit: TemplateAudit | undefined;
   if (!!streamId && !!templateId) {
      templateAudit = {
         streamId: streamId,
         templateId: templateId
      }
   }

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Stack>
         {!!agentId && <AuditLogPanel agentId={agentId} />}
         {!!deviceId && <AuditLogPanel deviceId={deviceId} />}
         {!!issueId && <AuditLogPanel issueId={issueId} />}
         {!!jobId && <AuditLogPanel jobId={jobId} />}
         {!!templateAudit && <AuditLogPanel templateAudit={templateAudit} />}
      </Stack>
   </Stack>

}

// Audit log rendering generally means creating a link from a tag 
// some tags depend on other tags to generate the route includig query parameters, and need to have knowledge of the routes themselves,
// which means can be somewhat data/tag driven, though not entirely

type AuditProperty = {
   tag: string; // without enclosing {}
   value: string | number;
   type?: string;
}

export type AuditLineFactory = (entry: AuditLogEntry, search?: string) => AuditLine;
export type GetItemHistory = (itemId: string, minTime?: Date, maxTime?: Date) => Promise<AuditLogEntry[]>;
export type ItemAuditLogConfig = {
   id: string;
   lineFactory: AuditLineFactory;
   getHistory: GetItemHistory;
   breadCrumbs: BreadcrumbItem[];
}

export class AuditLine {

   constructor(entry: AuditLogEntry, search?: string) {
      this.entry = entry;
      this.search = search;
      this.process();
   }

   private process() {

      const entry = this.entry;
      const properties = entry.properties;

      const match = entry.format.match(AuditLine.tagRegex)?.map(m => {
         m = m.replaceAll("{", "");
         m = m.replaceAll("}", "");
         return m;
      });

      if (!match?.length || !properties) {
         return;
      }

      match.forEach(m => {

         let property = properties[m];

         if (property === null) {

            this.properties.set(m, {
               tag: m,
               value: "null",
            })

            return;

         }

         if (typeof (property) === "string" || typeof (property) === "number" || typeof (property) === "boolean") {

            this.properties.set(m, {
               tag: m,
               value: property as any,
            })

            return;
         }

         const record = property as Record<string, string | number>;

         const type = record["$type"] as string | undefined;
         const value = record["$text"];

         this.properties.set(m, {
            tag: m,
            value: value,
            type: type ? type : undefined
         })
      });

   }

   protected highlight(key: string, text: string) {
      return <Highlight key={`${key}_hightlight`} search={this.search ? this.search : ""} >{text}</Highlight>
   }


   renderLine(): JSX.Element | null {

      const entry = this.entry;
      const format = entry.format;

      let tags: string[] = [];

      const match = format.match(AuditLine.tagRegex)?.map(m => {
         return m;
      });

      if (match?.length) {
         tags = match;
      }

      let renderedTags = tags.map((tag, index) => {

         tag = tag.replaceAll("{", "");
         tag = tag.replaceAll("}", "");

         const key = `audit_entry_${index}`;
         const property = this.properties.get(tag);

         if (!property) {
            return <span key={key}>{this.highlight(key, "!==> Missing Property <==!")}</span>;
         }

         // Common recognizers

         let element: JSX.Element | null = null;

         // @todo: switch these over to use types, with fallback to tag

         // LeaseId
         if (tag === "LeaseId") {

            const leaseId = property.value as string;
            const logId = this.properties.get("LogId")?.value;

            if (logId && leaseId) {
               let to = `/log/${logId}`;
               element = <Link to={to} key={key}>{this.highlight(key, leaseId)}</Link>
            }

         }

         // LogId
         if (tag === "LogId") {

            const logId = property.value as string;

            if (logId) {
               let to = `/log/${logId}`;
               element = <Link to={to} key={key}>{this.highlight(key, logId)}</Link>
            }

         }


         // JobId
         if (tag === "JobId") {

            const jobId = property.value as string;

            if (jobId) {
               let to = `/job/${jobId}`;
               element = <Link to={to} key={key}>{this.highlight(key, jobId)}</Link>
            }

         }

         // StepId
         if (tag === "StepId") {

            const stepId = property.value as string;
            const jobId = this.properties.get("JobId")?.value;

            if (stepId && jobId) {
               let to = `/job/${jobId}?step=${stepId}`;
               element = <Link to={to} key={key}>{this.highlight(key, stepId)}</Link>
            }

         }

         // BatchId
         if (tag === "BatchId") {

            const batchId = property.value as string;
            const jobId = this.properties.get("JobId")?.value;

            if (batchId && jobId) {
               let to = `/job/${jobId}?batch=${batchId}`;
               element = <Link to={to} key={key}>{this.highlight(key, batchId)}</Link>
            }

         }

         // Change
         if (tag === "Change") {

            const change = property.value?.toString();

            if (change) {
               let to = `${dashboard.swarmUrl}/changes/${change}`;
               element = <a href={to} rel="noreferrer" target="_blank" key={key}>{this.highlight(key, change)}</a>
            }

         }

         // Message
         if (tag === "Message") {
            const message = property.value?.toString();
            if (message && message.includes("\n")) {
               element = <pre style={{margin: 0}} key={key}>{this.highlight(key, message)}</pre>;
            }
         }

         if (element) {
            return element;
         }

         return <span key={key}>{this.highlight(key, property.value?.toString())}</span>;


      });

      let remaining = entry.format;

      renderedTags = renderedTags.map((t, idx) => {

         let current = remaining;

         const tag = tags[idx];
         const index = remaining.indexOf(tag);

         remaining = remaining.slice(tag.length + (index > 0 ? index : 0));

         if (index < 0) {
            console.error("not able to find tag in format");
            return <Text>Error, unable to find tag</Text>;
         }

         if (index === 0) {
            return t;
         }

         const rtags: any = [];

         const key = `log_line_${idx}_${index}_fragment`;

         rtags.push(<Highlight key={key} search={this.search ? this.search : ""} >{current.slice(0, index)}</Highlight>);
         rtags.push(t);

         return rtags;

      }).flat();

      if (remaining) {
         const key = `log_line_remaining_fragment`;
         renderedTags.push(<Highlight key={key} search={this.search ? this.search : ""} >{remaining}</Highlight>)
      }

      return <div>
         {renderedTags}
      </div>;

   }

   entry: AuditLogEntry;

   properties: Map<string, AuditProperty> = new Map();

   search?: string;

   private static tagRegex = /{[^{}]+}/g
}

class AgentAuditLine extends AuditLine {

}

class IssueAuditLine extends AuditLine {

}

class TemplateAuditLine extends AuditLine {

}

class DeviceAuditLine extends AuditLine {

}

class JobAuditLine extends AuditLine {

}

// Line Rendering -------------------------------------------------------------------------------------------------------------------


const renderAuditEntry = (entry: AuditLogEntry, search?: string, factory?: AuditLineFactory) => {

   if (handler.isAgentLog) {
      factory = (entry: AuditLogEntry, search?: string) => new AgentAuditLine(entry, search);
   }

   if (handler.isIssueLog) {
     factory = (entry: AuditLogEntry, search?: string) => new IssueAuditLine(entry, search);
   }

   if (handler.isTemplateAuditLog) {
      factory = (entry: AuditLogEntry, search?: string) => new TemplateAuditLine(entry, search);
   }

   if (handler.isDeviceLog) {
      factory = (entry: AuditLogEntry, search?: string) => new DeviceAuditLine(entry, search);
   }

   if (handler.isJobLog) {
      factory = (entry: AuditLogEntry, search?: string) => new JobAuditLine(entry, search);
   }

   if (!factory) {
      return <div>Unknown Log Type</div>
   }
   const line = factory(entry, search);
   return line.renderLine();
}

