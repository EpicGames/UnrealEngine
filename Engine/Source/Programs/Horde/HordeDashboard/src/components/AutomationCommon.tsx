// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyles, mergeStyleSets } from "@fluentui/react";

const styles = mergeStyleSets({

   stripes: {
      backgroundImage: 'repeating-linear-gradient(-45deg, rgba(255, 255, 255, .2) 25%, transparent 25%, transparent 50%, rgba(255, 255, 255, .2) 50%, rgba(255, 255, 255, .2) 75%, transparent 75%, transparent)',
   }

});

export type StatusBarStack = {
   value: number,
   title?: string,
   titleValue?: number,
   color?: string,
   onClick?: () => void,
   stripes?: boolean,

}

export const StatusBar = (stack: StatusBarStack[], width: number, height: number, basecolor?: string, style?: any): JSX.Element => {

   stack = stack.filter(s => s.value > 0);

   const mainTitle = stack.map((item) => {
      return item.titleValue === undefined ? `${Math.ceil(item.value)}% ${item.title}` : `${item.titleValue} ${item.title}`
   }).join(' ');

   return (
      <div className={mergeStyles({ backgroundColor: basecolor, width: width, height: height, verticalAlign: 'middle', display: "flex" }, style)} title={mainTitle}>
         {stack.map((item) => <span key={item.title!}
            onClick={item.onClick}
            className={item.stripes ? styles.stripes : undefined}
            style={{
               width: `${item.value}%`, height: '100%',
               backgroundColor: item.color,
               display: 'block',
               cursor: item.onClick ? 'pointer' : 'inherit',
               backgroundSize: `${height * 2}px ${height * 2}px`
            }} />)}
      </div>
   );
}
