// Copyright Epic Games, Inc. All Rights Reserved.

import { IconButton, Label, List, Modal, ScrollablePane, ScrollbarVisibility, Spinner, SpinnerSize, Stack, Text, getFocusStyle, mergeStyleSets } from "@fluentui/react";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useEffect, useState } from "react";
import { Link, useNavigate } from 'react-router-dom';
import backend from "../backend";
import { EventSeverity, GetLogEventResponse, GetTestDataDetailsResponse, GetTestDataRefResponse, GetTestResponse } from "../backend/Api";
import { TestDataHandler } from "../backend/AutomationTestData";
import dashboard from "../backend/Dashboard";
import { projectStore } from "../backend/ProjectStore";
import { getNiceTime, msecToElapsed } from "../base/utilities/timeUtils";
import { getHordeStyling } from "../styles/Styles";
import { getHordeTheme } from "../styles/theme";
import { renderLine } from "./logs/LogRender";


let _styles: any;
const getStyles = () => {
   const theme = getHordeTheme();

   const styles = _styles ?? mergeStyleSets({
      gutter: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: "#EC4C47",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      gutterWarning: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: "rgb(247, 209, 84)",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      itemCell: [
         getFocusStyle(theme, { inset: -1 }),
         {
            selectors: {
               '&:hover': { background: dashboard.darktheme ? undefined : theme.palette.neutralLight }
            }
         }
      ],
   
   });

   _styles = styles;
   return styles;
}



type TestFailureEvent = {
   // last failure ref id
   ref: GetTestDataRefResponse;
   time: Date;
   details?: GetTestDataDetailsResponse;
   jobId?: string;
   stepId?: string;
   logId?: string;
   events?: GetLogEventResponse[];
}

type TestFailureStream = {
   streamId: string;

   // metaId => failure events
   metaEvents: Map<string, TestFailureEvent>;
}

type TestFailureSummary = {
   test: GetTestResponse;
   // streamId => stream meta events
   streams: Map<string, TestFailureStream>;
}

class TestReportGenerator {

   constructor(handler: TestDataHandler, test: GetTestResponse) {
      makeObservable(this);
      this.handler = handler;
      this.test = test;

      this.init();
   }

   @action
   setUpdated() {
      this.updated++;
   }

   async init() {

      const handler = this.handler!;

      const ctests = handler.getStatusTests();

      // test Id => TestFailureSummary
      const testFailures = new Map<string, TestFailureSummary>();

      const errorRefIds: string[] = [];

      const allMetaEvents: TestFailureEvent[] = [];

      ctests.forEach(ctest => {

         if (ctest.id !== this.test.id) {
            return;
         }

         const testStatus = handler.getStatus(ctest.id);
         if (!testStatus) {
            return;
         }

         for (let [streamId, metaMap] of testStatus) {

            for (let [metaId, status] of metaMap) {

               if (status.error && status.refs.length) {

                  // got one
                  let failure = testFailures.get(ctest.id);
                  if (!failure) {
                     failure = {
                        test: ctest,
                        streams: new Map()
                     }
                     testFailures.set(ctest.id, failure);
                  }

                  let stream = failure.streams.get(streamId);
                  if (!stream) {
                     stream = {
                        streamId: streamId,
                        metaEvents: new Map()
                     }
                     failure.streams.set(streamId, stream);
                  }

                  const ref = status.refs[0];

                  errorRefIds.push(ref.id);

                  const timestamp = ref.id.substring(0, 8)


                  const metaEvent = {
                     ref: ref,
                     time: new Date(parseInt(timestamp, 16) * 1000)
                  }

                  allMetaEvents.push(metaEvent);

                  stream.metaEvents.set(metaId, metaEvent);
               }
            }
         }
      });

      console.log("query started");

      let details = await backend.getTestDetails(errorRefIds);

      details.forEach((d, index) => {
         const refId = d.id;
         const metaEvent = allMetaEvents.find(e => e.ref.id === refId);
         if (!metaEvent) {
            console.error("Unable to find meta event");
         } else {
            metaEvent.details = d;
         }
      });

      while (details.length) {

         const batch = details.slice(0, 10).map(detail => {
            return backend.getTestData(detail.testDataIds[0], "jobId,stepId");
         });

         // eslint-disable-next-line
         await Promise.all(batch).then((result) => {
            result.forEach((r, index) => {
               const detail = details[index];

               const metaEvent = allMetaEvents.find(e => e.details?.id === detail.id);
               if (!metaEvent) {
                  console.error("can't find meta event for details");
               } else {
                  metaEvent.jobId = r.jobId;
                  metaEvent.stepId = r.stepId;
               }
            });
         }).catch((errors) => {
            console.log(errors);
            // eslint-disable-next-line
         }).finally(() => {

            details = details.slice(10);
         });
      }

      const jobIds = new Set<string>();
      allMetaEvents.forEach(m => {
         if (m.jobId) {
            jobIds.add(m.jobId);
         }
      });

      if (jobIds.size) {
         const jobs = await backend.getJobsByIds(Array.from(jobIds), { filter: "id,batches" });
         jobs.forEach(j => {
            const metas = allMetaEvents.filter(m => m.jobId === j.id);
            metas.forEach(m => {
               const batch = j.batches?.find(b => b.steps.find(s => s.id === m.stepId));
               const step = batch?.steps.find(s => s.id === m.stepId);
               const logId = step?.logId;
               if (logId) {
                  m.logId = logId;
               }
            })
         });
      }


      const logIds = new Set<string>();
      allMetaEvents.forEach(m => {
         if (m.logId) {
            logIds.add(m.logId);
         }
      })


      if (logIds.size) {

         let logBatches = Array.from(logIds);

         while (logBatches.length) {

            const batch = logBatches.slice(0, 10).map(logId => {
               return backend.getLogEvents(logId);
            });

            // eslint-disable-next-line
            await Promise.all(batch).then((result) => {
               result.forEach((r, index) => {
                  const logId = logBatches[index];
                  const m = allMetaEvents.find(m => m.logId === logId);
                  if (m) {
                     m.events = r;
                  }
               });
            }).catch((errors) => {
               console.log(errors);
               // eslint-disable-next-line
            }).finally(() => {
               logBatches = logBatches.slice(10);
            });
         }

      }

      console.log("done");
      let count = 0;
      allMetaEvents.forEach(m => {
         if (!m.events) {
            count++;
         }
      })

      if (count) {
         console.error(`${count} meta failures with no events`);
      }

      this.failures = testFailures;
      this.loading = false;
      this.setUpdated();

   }

   @observable
   updated: number = 0;


   clear() {
      //this.handler = undefined;
   }

   handler?: TestDataHandler;
   test: GetTestResponse;
   loading = true;

   // test id => summary
   failures = new Map<string, TestFailureSummary>();

}

const ErrorPane: React.FC<{ failure: TestFailureEvent }> = ({ failure }) => {

   const { modeColors } = getHordeStyling();
   const navigate = useNavigate();
   const styles = getStyles();

   const events = failure.events;
   if (!events) {
      return null;
   }

   let errors = events.filter(e => e.severity === EventSeverity.Error).sort((a, b) => {
      return a.lineIndex < b.lineIndex ? -1 : 1;
   });

   const onRenderCell = (event?: GetLogEventResponse, index?: number, isScrolling?: boolean): JSX.Element => {

      if (!event) {
         return <div>???</div>;
      }

      const url = `/log/${failure.logId}?lineIndex=${event.lineIndex + 1}`;

      const lines = event.lines.filter(line => line.message?.trim().length).map(line => <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8, lineBreak: "anywhere", whiteSpace: "pre-wrap", lineHeight: 18, fontSize: 10, fontFamily: "Horde Cousine Regular, monospace, monospace" } }}> <Link className="log-link" target="_blank" to={url}>{renderLine(navigate, line, undefined, {})}</Link></Stack>);

      return (<Stack className={styles.itemCell} styles={{ root: { padding: 8, marginRight: 8 } }}><Stack className={event.severity === EventSeverity.Warning ? styles.gutterWarning : styles.gutter} styles={{ root: { padding: 0, margin: 0 } }}>
         <Stack styles={{ root: { paddingLeft: 14 } }}>
            {lines}
         </Stack>
      </Stack>
      </Stack>
      );
   };

   return (<Stack>
      {!!errors.length && <Stack>
         <div style={{ overflowY: 'auto', overflowX: 'visible', maxHeight: "800px", backgroundColor: modeColors.content, border: "2px solid #EDEBE9" }} data-is-scrollable={true}>
            <List
               items={errors}
               data-is-focusable={false}
               onRenderCell={onRenderCell} />
            <div style={{ padding: 8 }} />
         </div>
      </Stack>}
   </Stack>)
};


let idCounter = 0;

const AutomationFailureInner: React.FC<{ generator: TestReportGenerator, test: GetTestResponse, }> = ({ generator, test }) => {

   const { hordeClasses, modeColors } = getHordeStyling();

   const failure = generator.failures.get(test.id);
   if (!failure?.streams.size) {
      return <Stack>No Failure Data</Stack>
   }

   const handler = generator.handler!;

   const streamIds = Array.from(failure.streams.keys()).sort((a, b) => a.localeCompare(b));

   const streamElements = streamIds.map(s => {

      const streamName = projectStore.streamById(s)?.fullname ?? "Unknown Stream";

      const sf = failure.streams.get(s)!;
      const meta = Array.from(sf.metaEvents.keys()).sort((a, b) => {
         return handler.metaNames.get(a)!.localeCompare(handler.metaNames.get(b)!);
      })

      const metaElements = meta.map(m => {

         const labelWidth = 64;
         const textWidth = 400;

         const infoItem = (label: string, textIn: any, link?: string) => {
            const text = textIn.toString();
            return <Stack key={`test_info_item_${idCounter++}`} horizontal verticalAlign="center" style={{ width: labelWidth + textWidth + 12, height: 18 }}>
               <Stack style={{ width: labelWidth }}>
                  <Label style={{ fontSize: 11 }}>{label}:</Label>
               </Stack>
               <Stack style={{ fontSize: 11, width: textWidth }}>
                  {!link &&
                     <Text style={{ fontSize: 11 }}>{text}</Text>}
                  {!!link &&
                     <Link style={{ fontSize: 11 }} to={link} target="_blank">{text}</Link>}
               </Stack>
            </Stack>
         }

         const infoItems: JSX.Element[] = [];

         const metaName = handler.getMetaString(m);
         const event = sf.metaEvents.get(m)!;
         const duration = `${msecToElapsed(moment.duration(event.ref.duration).asMilliseconds(), true, true)}`;

         if (event.jobId && event.stepId) {
            infoItems.push(infoItem("Horde", "Job Step", `/job/${event.jobId!}?step=${event.stepId}`));
         }

         infoItems.push(infoItem("Date", getNiceTime(event.time)));
         infoItems.push(infoItem("Duration", duration));
         infoItems.push(infoItem("Build CL", event.ref.buildChangeList));

         return <Stack tokens={{ childrenGap: 4 }}>
            <Stack>
               <Text variant="smallPlus" style={{ fontWeight: 600 }}>{metaName}</Text>
            </Stack>
            <Stack style={{ width: 600 }}>
               {infoItems}
            </Stack>
            <Stack style={{ paddingTop: 8 }}>
               <ErrorPane failure={sf.metaEvents.get(m)!} />
            </Stack>
         </Stack>
      })

      return <Stack className={hordeClasses.raised} style={{ cursor: "pointer", height: "fit-content", backgroundColor: dashboard.darktheme ? modeColors.header : modeColors.background, padding: 12, paddingLeft: 24, paddingRight: 48, paddingBottom: 24 }}>
         <Stack style={{ paddingBottom: 8 }}>
            <Text variant="medium" style={{ fontWeight: 600 }}>{streamName}</Text>
         </Stack>
         <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 12 }}>
            {metaElements}
         </Stack>
      </Stack>
   });

   return <Stack style={{ paddingLeft: 12, paddingRight: 12 }}>
      <div style={{ marginTop: 8, height: 'calc(100vh - 258px)', position: 'relative' }} data-is-scrollable={true}>
         <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} onScroll={() => { }}>
            <Stack tokens={{ childrenGap: 24 }}>
               {streamElements}
            </Stack>
         </ScrollablePane>
      </div>
   </Stack>


};

export const AutomationFailureModal: React.FC<{ handler: TestDataHandler, test: GetTestResponse, onDismiss: () => void }> = observer(({ handler, test, onDismiss }) => {

   const [generator] = useState(new TestReportGenerator(handler, test));

   useEffect(() => {
      return () => {
         generator?.clear();
      };
   }, [generator]);

   const { hordeClasses } = getHordeStyling();

   // subscribe
   if (generator.updated) { }



   const title = `Current ${test.displayName ?? test.name} Failures`

   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ scrollableContent: { overflow: "auto", height: "calc(100vh - 180px)" }, main: { padding: 8, width: 1320, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onDismiss()} >
      <Stack>
         <Stack horizontal verticalAlign="center">
            <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>{title}</Text></Stack>
            <Stack grow />
            <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onDismiss(); }}
               />
            </Stack>
         </Stack>
         <Stack>
            {generator.loading && <Stack>
               <Stack>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>}

            {!generator.loading && <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
               <AutomationFailureInner generator={generator} test={test} />
            </Stack>}

         </Stack>
      </Stack>

   </Modal>
});

