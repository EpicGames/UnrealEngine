// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, DetailsRow, Dropdown, FontIcon, IColumn, IconButton, IDetailsGroupRenderProps, IDetailsListProps, IDropdownOption, IGroup, mergeStyleSets, Modal, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import * as d3 from "d3";
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import React, { useState } from "react";
import backend from "../backend";
import { GetSuiteTestDataResponse, GetTestDataDetailsResponse, GetTestDataRefResponse, GetTestMetaResponse, GetTestResponse, GetTestSuiteResponse, TestOutcome } from "../backend/Api";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { projectStore } from "../backend/ProjectStore";
import { getHumanTime, getShortNiceTime } from "../base/utilities/timeUtils";
import { StatusBar, StatusBarStack } from "./AutomationCommon";
import { AutomationSuiteTest } from "./AutomationSuiteTest";
import { getHordeStyling } from "../styles/Styles";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type TestId = string;
type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type DivSelectionType = d3.Selection<HTMLDivElement, unknown, null, undefined>;
type TestState = "Success" | "Failed" | "Consecutive Failures" | "Skipped" | "Unspecified";
type FilterState = {
   testState?: TestState;
   text?: string;
}

const suiteGraphWidth = 900;

type TestStatus = {
   testId: string,
   success?: number;
   error?: number;
   skip?: number;
   unspecified?: number;
   warning?: number;
}

type TestSelection = {
   ref: GetTestDataRefResponse;
   testId: string;
   details: GetTestDataDetailsResponse;
}

type TestMetrics = {
   total: number;
   error: number;
   success: number;
   skipped: number;
   unspecified: number;
   warning: number;
}

const suiteStyles = mergeStyleSets({
   testModal: {
      selectors: {
         ".ms-Modal-scrollableContent": {
            height: "100% !important"
         }
      }
   },
   group: {
      selectors: {
         '.ms-GroupSpacer': {
            width: "0px !important",
            height: "18px !important"
         },
         '.ms-List-cell .ms-DetailsRow': {
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 0,
            paddingLeft: 4,
            minHeight: 0
         }
      }
   }
});


/*
         '.ms-List-cell': {
            padding: "0 !important",
            height: "18px !important"
         },
         '.ms-DetailsRow-cell': {
            padding: "0 !important",
            height: "18px !important",
            minHeight: "18px !important"
         },
         '.ms-DetailsRow-fields': {
            height: "18px !important"
         },
         '.ms-DetailsRow.root': {
            height: "18px !important"
         },

*/

type GroupStatus = {
   error: number;
   skip: number;
   unspecified: number;
   success: number;
   warning: number;
   totalError: number;
   totalSkip: number;
   totalUnspecified: number;
   totalSuccess: number;
   totalWarning: number;
};

type SuiteGroup = IGroup & {
   status: GroupStatus;
   color: string;
}


class SuiteHandler {

   constructor(suite: GetTestSuiteResponse, suiteRefs: GetTestDataRefResponse[], metaData: GetTestMetaResponse) {

      makeObservable(this);

      this.suite = suite;

      const dedupe = new Set<number>();

      this.suiteRefs = suiteRefs.filter(r => {
         if (!dedupe.has(r.buildChangeList)) {
            dedupe.add(r.buildChangeList);
            return true;
         }
         return false;
      });

      if (this.suiteRefs.length !== suiteRefs.length) {
         console.warn("Warning: Test suite details had duplicate ref changelists");
      }

      this.metaData = metaData;

      this.metaName = `${metaData.platforms.join(" - ")} / ${metaData.configurations.join(" - ")} / ${metaData.buildTargets.join(" - ")} / ${metaData.rhi === "default" ? "Default" : metaData.rhi?.toUpperCase()} / ${metaData.variation === "default" ? "Default" : metaData.variation?.toUpperCase()}`;

      this.load();
   }

   setSelection(selection?: TestSelection) {

      if ((this.selection?.ref.id === selection?.ref.id) && (this.selection?.testId === selection?.testId)) {
         return;
      }

      this.selection = selection;
      this.setSelectionUpdated();
   }

   @action
   setUpdated() {
      this.updated++;
   }

   @action
   setSelectionUpdated() {
      this.selectionUpdated++;
   }

   async load() {

      console.log(`Loading ${this.suiteRefs.length} suite ref details for metaData: ${JSON.stringify(this.metaData)}`);

      this.changeDates = new Map();
      this.suiteRefs.forEach(r => {

         this.refs.set(r.id, r);

         if (!this.refDates.get(r.id)) {
            const timestamp = r.id.substring(0, 8)
            this.refDates.set(r.id, new Date(parseInt(timestamp, 16) * 1000));
         }

         const date = this.changeDates.get(r.buildChangeList);
         const rdate = this.refDates.get(r.id)!;
         if (!date || date.getTime() > rdate.getTime()) {
            this.changeDates.set(r.buildChangeList, rdate);
         }
      });


      let details = await backend.getTestDetails(this.suiteRefs.map(r => r.id));
      details = details.filter(d => !!d.suiteTests);
      details.forEach(d => this.testDetails.set(d.id, d));

      const uniqueTests = new Set<string>(details.map(d => d.suiteTests!.map(t => t.testId)).flat())
      const request = Array.from(uniqueTests).filter(tid => !SuiteHandler.tests.has(tid));

      if (request.length) {
         const tests = await backend.getTests(Array.from(uniqueTests).filter(tid => !SuiteHandler.tests.has(tid)));

         const replace = ["Project.Functional Tests.Tests.", "Project.Maps.AllInPIE."];

         function fixName(name: string) {

            if (!name) {
               return "???"
            }

            for (let r of replace) {
               if (name.startsWith(r)) {
                  name = name.substring(r.length)
               }
            }
            return name;
         }

         tests.forEach(t => {
            SuiteHandler.tests.set(t.id, t);
            const fixed = fixName(t.name) ?? "???";
            SuiteHandler.fixedTestNames.set(t.id, fixed);            
            SuiteHandler.lowerCaseTestNames.set(t.id, fixed.toLowerCase());
         });
      }

      const streams = Array.from(new Set(this.suiteRefs.map(s => s.streamId))).sort((a, b) => a.localeCompare(b));

      streams.forEach(streamId => {

         const testStream: TestStream = {
            testStatus: new Map(),
            testData: new Map()
         }

         this.testStreams.set(streamId, testStream);

         const refs = this.suiteRefs.filter(r => r.streamId === streamId).sort((a, b) => a.buildChangeList - b.buildChangeList).reverse();

         for (const ref of refs) {
            const details = this.testDetails.get(ref.id);
            const suiteTests = details?.suiteTests;
            if (!suiteTests) {
               return;
            }

            for (const test of suiteTests) {

               let testData = testStream.testData.get(test.testId);

               if (!testData) {
                  testData = [];
                  testStream.testData.set(test.testId, testData);
               }

               testData.push({ ...test, refId: ref.id });

               let status = testStream.testStatus.get(test.testId);
               if (!status) {
                  status = { testId: test.testId };
                  testStream.testStatus.set(test.testId, status);
               }
            }
         }

         for (const [, testData] of testStream.testData) {

            if (!testData.length) {
               continue;
            }

            const test = testData[0];

            const status = testStream.testStatus.get(test.testId)!;

            if (test.outcome === "Failure") {
               status.error = 1;
               continue;
            }

            if (test.outcome === "Skipped") {
               status.skip = 1;
               continue;
            }

            if (test.outcome === "Unspecified") {
               status.unspecified = 1;
               continue;
            }

            if (test.warningCount) {
               status.warning = 1;
            } else {
               status.success = 1;
            }

         }

         const metrics = this.metrics = {
            total: 0,
            error: 0,
            success: 0,
            skipped: 0,
            warning: 0,
            unspecified: 0
         };

         testStream.testStatus.forEach((v, k) => {

            let gstatus = this.globalStatus.get(k);

            if (!gstatus) {
               gstatus = { ...v };
               this.globalStatus.set(k, gstatus);
            } else {
               gstatus.error = (gstatus.error ?? 0) + (v.error ?? 0);
               gstatus.skip = (gstatus.skip ?? 0) + (v.skip ?? 0);
               gstatus.unspecified = (gstatus.unspecified ?? 0) + (v.unspecified ?? 0);
               gstatus.success = (gstatus.success ?? 0) + (v.success ?? 0);
               gstatus.warning = (gstatus.warning ?? 0) + (v.warning ?? 0);
            }

            metrics.total++;
            if (v.error) {
               metrics.error++;
            }
            if (v.success) {
               metrics.success++;
            }
            if (v.warning) {
               metrics.warning++;
            }

            if (v.skip) {
               metrics.skipped++;
            }
            if (v.unspecified) {
               metrics.unspecified++;
            }
         });
      });

      this.loaded = true;

      this.setUpdated();

      /*
      const debugRef = "63d9a44acf52968117e67b39";
      const debugTestId = "6376b26ee30d43884952c55d";

      const sdetails = this.testDetails.get(debugRef)!;
      this.setSelection({ ref: this.suiteRefs.find(s => s.id === debugRef)!, details: sdetails, testId: debugTestId })
      */
   }

   setFilterState(nstate?: FilterState) {

      if ((this.filterState?.testState === nstate?.testState) && (this.filterState?.text === nstate?.text)) {
         return;
      }

      this.filterState = (nstate?.testState || nstate?.text) ? nstate : undefined;

      this.setUpdated();
   }

   getFilterState(): FilterState | undefined {
      return this.filterState;
   }


   getFilteredTestData(streamId: string, testId?: string): TestData[] {

      const testStream = this.testStreams.get(streamId);
      if (!testStream) {
         return [];
      }


      if (!this.filterState) {
         if (testId) {
            return testStream.testData.get(testId)?.flat() ?? [];
         }
         return Array.from(testStream.testData.values()).flat();
      }

      const testData: TestData[] = [];

      const textLowercase = this.filterState.text?.toLowerCase();

      let testStatus = testStream.testStatus;

      if (testId) {
         const ts = testStream.testStatus.get(testId);
         testStatus = new Map();
         if (ts) {
            testStatus.set(testId, ts);
         }
      }

      testStatus.forEach(ts => {

         const sdata = testStream.testData.get(ts.testId);
         if (!sdata) {
            return;
         }

         const test = SuiteHandler.tests.get(ts.testId);

         if (!test) {
            return;
         }

         if (textLowercase) {
            const name = SuiteHandler.lowerCaseTestNames.get(test.id)!;
            if (name.indexOf(textLowercase) === -1) {
               return;
            }
         }

         if (!this.filterState!.testState) {
            testData.push(...sdata);
            return;
         }

         const testState = this.filterState!.testState;

         if (testState === "Failed") {
            if (ts.error) {
               testData.push(...sdata);
            }
         }

         if (testState === "Consecutive Failures") {
            if (ts.error && ts.error > 1) {
               testData.push(...sdata);
            }
         }

         if (testState === "Success") {
            if (ts.success) {
               testData.push(...sdata);
            }
         }

         if (testState === "Skipped") {
            if (ts.skip) {
               testData.push(...sdata);
            }
         }

         if (testState === "Unspecified") {
            if (ts.unspecified) {
               testData.push(...sdata);
            }
         }

      });

      return testData;

   }

   get streamIds(): string[] {
      return Array.from(this.testStreams.keys()).sort((a, b) => a.localeCompare(b));
   }

   @observable
   updated: number = 0;

   @observable
   selectionUpdated: number = 0;

   loaded = false

   testDetails = new Map<string, GetTestDataDetailsResponse>();
   suite: GetTestSuiteResponse;
   suiteRefs: GetTestDataRefResponse[];

   refs = new Map<string, GetTestDataRefResponse>();

   metaData: GetTestMetaResponse;
   metaName: string;

   // quantized so common CL -> lands at same time in views
   changeDates = new Map<number, Date>();
   refDates = new Map<string, Date>();

   static tests = new Map<TestId, GetTestResponse>();

   private filterState?: FilterState;

   globalStatus = new Map<TestId, TestStatus>();

   selection?: TestSelection;

   metrics?: TestMetrics;

   testStreams: Map<string, TestStream> = new Map();

   static fixedTestNames = new Map<TestId, string>();
   static lowerCaseTestNames = new Map<TestId, string>();

}

type TestStream = {
   testStatus: Map<TestId, TestStatus>;
   testData: Map<TestId, TestData[]>;
}

type TestData = GetSuiteTestDataResponse & {
   refId: string;
}

class SuiteGraphRenderer {

   constructor(handler: SuiteHandler, streamId?: string, testId?: string) {
      this.streamId = streamId;
      this.testId = testId;
      this.handler = handler;
      this.margin = { top: 0, right: 8, bottom: 0, left: 16 };

      this.headerOnly = !testId;

   }

   initData() {

      const handler = this.handler;

      if (this.headerOnly) {
         this.testData = [];
      } else {
         this.testData = this.headerOnly ? [] : handler.getFilteredTestData(this.streamId!, this.testId);
      }

   }

   render(container: HTMLDivElement) {

      const handler = this.handler;

      if (!handler.testDetails.size || (this.hasRendered && !this.forceRender)) {
         return;
      }

      const { modeColors } = getHordeStyling()

      this.clear();

      this.initData();

      this.hasRendered = true;
      this.forceRender = false;

      const X = _d3.map(this.testData, (t) => t);
      const xDomain = d3.extent(handler.changeDates.values() as any, (d:any) => d.getTime() / 1000);

      let Y: TestOutcome[] = [];
      const scolors = dashboard.getStatusColors();
      const colors: Record<string, string> = {
         "Success": scolors.get(StatusColor.Success)!,
         "Failure": scolors.get(StatusColor.Failure)!,
         "Warning": scolors.get(StatusColor.Warnings)!,
         "Unspecified": scolors.get(StatusColor.Unspecified)!,
         "Skipped": scolors.get(StatusColor.Skipped)!
      };

      Y = _d3.map(this.testData, (d) => d.warningCount ? TestOutcome.Warning : d.outcome);

      const width = suiteGraphWidth - 32; // important: the 32 offset is to provide more mouse room to roll down latest CL's
      const height = this.headerOnly ? 20 : 16;

      const xRange = [this.margin.left, width - this.margin.right];

      const xScale = _d3.scaleTime(xDomain, xRange);

      const I = d3.range(X.length);

      const svgId = `suite_svg_${id_counter++}`;

      const svg = d3.select(container)
         .append("svg")
         .attr("id", svgId)
         .attr("width", width + 48)
         .attr("height", height)
         .attr("viewBox", [0, 0, width, height] as any)


      const g = svg.append("g")

      if (!this.headerOnly) {
         g.append("line")
            .attr("stroke", () => {
               return dashboard.darktheme ? "#6D6C6B" : "#4D4C4B";
            })
            .attr("stroke-width", 1)
            .attr("stroke-linecap", 4)
            .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
            .attr("x1", () => this.margin.left)
            .attr("x2", () => width - this.margin.right)
            .attr("y1", () => 8)
            .attr("y2", () => 8)
      }

      const radius = 3.5
      g.selectAll("circle")
         .data(I)
         .join("circle")
         .attr("id", i => `suite_rect${this.testData[i as any].refId}`)
         .attr("cx", i => {
            const ref = handler.refs.get(X[i as any].refId)!;
            return xScale(handler.changeDates.get(ref.buildChangeList)!.getTime() / 1000);
         })
         .attr("cy", i => 8)
         .attr("fill", i => colors[Y[i as any]] ?? "#0000FF")
         .attr("width", 4)
         .attr("r", radius);

      const xAxis = (g: SelectionType) => {

         const dateMin = new Date(xDomain[0]! * 1000);
         const dateMax = new Date(xDomain[1]! * 1000);

         let ticks: number[] = [];
         for (const date of d3.timeDays(dateMin, dateMax, 1).reverse()) {
            ticks.push(date.getTime() / 1000);
         }

         if (ticks.length > 14) {
            let nticks = [...ticks];
            // remove first and last, will be readded
            const first = nticks.shift()!;
            const last = nticks.pop()!;

            const n = Math.floor(nticks.length / 12);

            const rticks: number[] = [];
            for (let i = 0; i < nticks.length; i = i + n) {
               rticks.push(nticks[i]);
            }

            rticks.unshift(first);
            rticks.push(last);
            ticks = rticks;

         }

         g.attr("transform", `translate(0,${this.headerOnly ? 16 : 0})`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "9px")
            .call(d3.axisTop(xScale)
               .tickValues(ticks)
               .tickFormat(d => {
                  return this.headerOnly ? getHumanTime(new Date((d as number) * 1000)) : "";
               })
               .tickSizeOuter(0))
            .call(g => g.select(".domain").remove())
            .call(g => g.selectAll(".tick line").attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25).clone()
               .attr("y2", height - this.margin.bottom)
            )
      }

      svg.append("g").call(xAxis)

      const root = d3.select("#root_suite_test_list") as any as DivSelectionType

      const tooltip = this.tooltip = root
         .append("div")
         .attr("id", "tooltip")
         .style("display", "none")
         .style("background-color", modeColors.background)
         .style("border", "solid")
         .style("border-width", "1px")
         .style("border-radius", "3px")
         .style("border-color", dashboard.darktheme ? "#413F3D" : "#2D3F5F")
         .style("padding", "8px")
         .style("position", "absolute")
         .style("z-index", "1")
         .style("pointer-events", "none");

      svg.on("mousemove", (event) => handleMouseMove(event))
      svg.on("mouseleave", (event) => handleMouseLeave(event))
      svg.on("click", (event) => handleMouseClick(event))

      const closestRef = (x: number, y: number): GetTestDataRefResponse | undefined => {

         let closest = this.testData.reduce((best, data, i) => {
            const ref = handler.refs.get(data.refId)!;
            const timeStamp = handler.changeDates.get(ref.buildChangeList)!.getTime() / 1000;
            const sx = xScale(timeStamp);
            let absx = Math.abs(sx - x)

            if (absx < best.value) {
               return { index: i, value: absx };
            }
            else {
               return best;
            }
         }, { index: 0, value: Number.MAX_SAFE_INTEGER });

         if (closest) {
            return handler.refs.get(this.testData[closest.index]?.refId);
         }

         return undefined;

      }

      const handleMouseMove = (event: any) => {

         const mouseX = _d3.pointer(event)[0];
         const mouseY = _d3.pointer(event)[1];

         const closest = closestRef(mouseX, mouseY);
         if (closest) {
            svg.selectAll(`circle`).attr("r", radius);
            svg.select(`#suite_rect${closest.id}`).attr("r", radius * 2);

            const timestamp = closest.id.substring(0, 8)

            const date = getShortNiceTime(new Date(parseInt(timestamp, 16) * 1000), true, true);
            let desc = `CL ${closest.buildChangeList} <br/>`;
            //desc += `Duration ${msecToElapsed(moment.duration(closest.duration).asMilliseconds(), true, true)} <br/>`;
            desc += `${date} <br/>`;

            const timeStamp = handler.changeDates.get(closest.buildChangeList)!.getTime() / 1000;
            const tx = xScale(timeStamp);

            updateTooltip(true, tx, 0, desc);
         }
      }

      const handleMouseLeave = (event: any) => {
         svg.selectAll(`circle`).attr("r", radius);
         tooltip.style("display", "none");
      }

      const handleMouseClick = (event: any) => {

         const mouseX = _d3.pointer(event)[0];
         const mouseY = _d3.pointer(event)[1];

         const closest = closestRef(mouseX, mouseY);
         if (closest && this.testId) {
            const details = handler.testDetails.get(closest.id);
            if (details) {
               handler.setSelection({ ref: closest, details: details, testId: this.testId });
            }
         }
      }

      const updateTooltip = (show: boolean, x?: number, y?: number, html?: string) => {

         if (!tooltip) {
            return;
         }

         x = x ?? 0;
         y = y ?? 0;

         var elem = document.querySelector('#' + svgId) as any;
         var parent = document.querySelector('#root_suite_test_list') as any;

         if (!elem || !parent) {
            return;
         }

         const b = elem.getBoundingClientRect()
         const pb = parent.getBoundingClientRect()

         x += (b.left - pb.left);
         y += (b.top - pb.top);

         tooltip
            .style("display", "block")
            .html(html ?? "")
            .style("position", `absolute`)
            .style("width", `max-content`)
            .style("top", (y - 52) + "px")
            .style("left", `${x + 24}px`)
            .style("transform", "translateX(-108%)")
            .style("font-family", "Horde Open Sans Semibold")
            .style("font-size", "10px")
            .style("line-height", "16px")
            .style("shapeRendering", "crispEdges")
            .style("stroke", "none")
      }

   }

   clear() {

   }

   tooltip?: DivSelectionType;

   streamId?: string;
   testId?: string;
   testData: TestData[] = [];
   headerOnly = false;

   handler: SuiteHandler;

   private readonly clipId = "suite_details_clip_path";

   margin: { top: number, right: number, bottom: number, left: number }
   hasRendered = false;
   forceRender = false;
}

const SuiteGraph: React.FC<{ handler: SuiteHandler, streamId?: string, testId?: string }> = observer(({ handler, streamId, testId }) => {

   const { hordeClasses } = getHordeStyling();

   const graph_container_id = `automation_suite_graph_container_${id_counter++}`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const [state, setState] = useState<{ graph?: SuiteGraphRenderer }>({});

   // subscribe
   if (handler.updated) { }

   if (!state.graph) {
      setState({ ...state, graph: new SuiteGraphRenderer(handler, streamId, testId) })
      return null;
   }

   if (container) {
      try {
         state.graph?.render(container);
      } catch (err) {
         console.error(err);
      }

   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8, paddingTop: 8 }}>
         <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </Stack>;

})

const SuiteOperationsBar: React.FC<{ handler: SuiteHandler }> = observer(({ handler }) => {

   const { hordeClasses } = getHordeStyling();

   if (handler.updated) { }

   const stateItems: IDropdownOption[] = ["All", "Success", "Failed", "Skipped", "Unspecified"].map(state => {
      return {
         key: state,
         text: state
      };
   });


   return <Stack className={hordeClasses.modal} horizontal style={{ paddingBottom: 18 }}>
      <Stack className="horde-no-darktheme" grow />
      <Stack horizontal tokens={{ childrenGap: 24 }} verticalFill={true} verticalAlign={"center"}>
         <Stack>
            <Dropdown style={{ width: 180 }} options={stateItems} selectedKey={handler.getFilterState()?.testState ?? "All"}
               onChange={(event, option, index) => {

                  let cstate = handler.getFilterState() ?? {};
                  if (option) {
                     if (option.key === "All") {
                        if (cstate.testState) {
                           handler.setFilterState({ ...cstate, testState: undefined })
                        }
                     } else {
                        if (!cstate) {
                           cstate = {};
                        }
                        handler.setFilterState({ ...cstate, testState: option.key as TestState });
                     }
                  }
               }} />
         </Stack>
         <Stack>
            <TextField placeholder="Filter" style={{ width: 190 }} spellCheck={false} autoComplete="off" deferredValidationTime={1000} validateOnLoad={false} onGetErrorMessage={(newValue: any) => {
               let cstate = handler.getFilterState() ?? {};
               if (!newValue) {
                  newValue = undefined;
               }
               if (cstate.text !== newValue) {
                  handler.setFilterState({ ...cstate, text: newValue });
               }
               return undefined
            }} />
         </Stack>

      </Stack>
   </Stack>;
})

let id_counter = 0;

const SuiteTestViewModal: React.FC<{ handler: SuiteHandler }> = observer(({ handler }) => {

   const { modeColors } = getHordeStyling();

   if (handler.selectionUpdated) { }

   const selection = handler.selection;

   if (selection == null) {
      return null;
   }

   const test = SuiteHandler.tests.get(selection.testId);
   if (!test || !handler.metaData || !handler.suite) {
      return null;
   }

   return <Stack key={`suite_test_view_${id_counter++}`} style={{ height: "100%" }}>
      <Modal isOpen={true} isBlocking={false} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1180, backgroundColor: modeColors.background, hasBeenOpened: false, top: "128px", position: "absolute", height: "75vh" } }} className={suiteStyles.testModal} onDismiss={(ev) => { handler.setSelection(undefined) }}>
         <Stack style={{ height: "100%" }}>
            <Stack horizontal style={{ padding: 18 }}>
               <Stack>
                  <Text style={{ fontSize: 16, fontFamily: "Horde Open Sans Semibold" }}>Test Events</Text>
               </Stack>
               <Stack grow />
               <Stack style={{ paddingLeft: 24, paddingTop: 0 }}>
                  <IconButton
                     style={{ height: "14px" }}
                     iconProps={{ iconName: 'Cancel' }}
                     onClick={() => { handler.setSelection(undefined) }} />
               </Stack>
            </Stack>

            <Stack style={{ height: "100%", paddingLeft: 24, paddingRight: 24 }}>
               <AutomationSuiteTest suite={handler.suite} test={test} testRef={selection.ref} details={selection.details} metaData={handler.metaData} onClose={() => { }} />
            </Stack>
         </Stack>
      </Modal>
   </Stack>


});

type TestItem = {
   key: string;
   name: string;
   streamId?: string;
   testId: string;
}

const SuiteTestList: React.FC<{ handler: SuiteHandler, suite: GetTestSuiteResponse }> = observer(({ handler, suite }) => {

   const { hordeClasses, modeColors } = getHordeStyling();

   if (handler.updated) { }

   const scolors = dashboard.getStatusColors();

   const columns: IColumn[] = [
      { key: 'name_column', name: 'Name', minWidth: 308, maxWidth: 308, isResizable: false, isPadded: false, isMultiline: false },
      { key: 'graph_column', name: 'Graph', minWidth: suiteGraphWidth + 8, maxWidth: suiteGraphWidth + 8, isResizable: false, isPadded: false, isMultiline: false }
   ];

   const renderItem = (item: TestItem, index?: number, column?: IColumn) => {

      if (!column) {
         return null;
      }

      const streamId = item.streamId;
      let testStream: TestStream | undefined;
      if (streamId) {
         testStream = handler.testStreams.get(streamId);
      }


      if (column.name === "Name") {

         let status: TestStatus | undefined;
         if (testStream) {
            status = testStream.testStatus.get(item.testId) ?? ({} as TestStatus);
         } else {
            status = handler.globalStatus.get(item.testId);
         }
         let color = dashboard.darktheme ? "#E0E0E0" : undefined;
         if (status?.error) {
            color = scolors.get(StatusColor.Failure)!;
         } else if (status?.success) {
            color = scolors.get(StatusColor.Success)!;
         } else if (status?.warning) {
            color = scolors.get(StatusColor.Warnings)!;
         }

         if (!streamId) {
            return <Stack horizontal verticalAlign="center" verticalFill>
               <FontIcon style={{ color: color, paddingRight: 4 }} iconName="Square" />
               <Stack style={{ height: 18 }} verticalFill={true} verticalAlign="center"><Text style={{ fontSize: 12, fontWeight: 600 }}>{item.name}</Text></Stack>
            </Stack>
         }
         return <Stack style={{ cursor: "pointer" }} onClick={() => {

            // @todo: should be a utility function
            const data = testStream?.testData.get(item.testId);
            if (data && data.length) {
               const d = data[0];
               const ref = handler.suiteRefs.find(s => s.id === d.refId);
               if (ref) {
                  const sdetails = handler.testDetails.get(d.refId);
                  if (sdetails) {
                     handler.setSelection({ ref: ref, details: sdetails, testId: item.testId })
                  }
               }
            }
         }}>
            <Stack style={{ paddingLeft: 12 }} horizontal verticalAlign="center" verticalFill>
               <FontIcon style={{ color: color, paddingRight: 4, fontSize: 10 }} iconName="Square" />
               <Stack style={{ height: 18 }} verticalFill={true} verticalAlign="center"><Text style={{ fontSize: 10 }}>{item.name}</Text></Stack>
            </Stack>
         </Stack>
      }

      if (column.name === "Graph" && item.streamId) {

         return <Stack style={{ height: 18 }} verticalFill={true} verticalAlign="center"><SuiteGraph handler={handler} streamId={item.streamId} testId={item.testId} /></Stack>;
      }

      return null;

   };

   const testIds = new Set<TestId>();

   let testData: TestData[] = [];

   handler.streamIds.forEach(s => {
      testData.push(...handler.getFilteredTestData(s));
   });

   testData.forEach(d => testIds.add(d.testId));

   let tests: GetTestResponse[] = [];

   for (let testId of testIds) {
      const test = SuiteHandler.tests.get(testId);
      if (test) {
         tests.push(test);
      }
   }

   // test id => groupName
   const groupNames = new Map<string, string>();
   // groupname => status
   const groupStatus = new Map<string, GroupStatus>();

   tests.forEach(t => {
      const elements = t.name.split(".").map(e => e.trim()).filter(e => !!e).slice(0, -1);
      let groupName = elements.join(".");
      const replace = ["Project.Functional Tests.Tests.", "Project.Maps.AllInPIE."];
      replace.forEach(r => {
         groupName = groupName.replace(r, "");
      })

      groupNames.set(t.id, groupName);

      if (groupName) {
         const status = handler.globalStatus.get(t.id);
         let gstatus = groupStatus.get(groupName);
         if (!gstatus) {
            gstatus = { error: 0, skip: 0, unspecified: 0, success: 0, warning: 0, totalError: 0, totalSkip: 0, totalUnspecified: 0, totalSuccess: 0, totalWarning: 0 };
            groupStatus.set(groupName, gstatus);
         }

         gstatus.error += status?.error ? 1 : 0;
         gstatus.skip += status?.skip ? 1 : 0;
         gstatus.unspecified += status?.unspecified ? 1 : 0;
         gstatus.success += status?.success ? 1 : 0;
         gstatus.warning += status?.warning ? 1 : 0;

         gstatus.totalError += status?.error ?? 0;
         gstatus.totalSkip += status?.skip ?? 0;
         gstatus.totalUnspecified += status?.unspecified ?? 0;
         gstatus.totalWarning += status?.warning ?? 0;
         gstatus.totalSuccess += status?.success ?? 0;

      }
   });

   const sortedGroups = Array.from(groupStatus.keys()).sort((a, b) => {

      const statusA = groupStatus.get(a)!;
      const statusB = groupStatus.get(b)!;

      const totalA = (statusA.error ?? 0) + (statusA.skip ?? 0) + (statusA.unspecified ?? 0) + (statusA.success ?? 0)
      const failedFactorA = Math.ceil((statusA.error ?? 0) / (totalA || 1) * 20) / 20;
      const skipFactorA = Math.ceil(((statusA.skip ?? 0) + (statusA.unspecified ?? 0)) / (totalA || 1) * 20) / 20;

      const totalB = (statusB.error ?? 0) + (statusB.skip ?? 0) + (statusB.unspecified ?? 0) + (statusB.success ?? 0)
      const failedFactorB = Math.ceil((statusB.error ?? 0) / (totalB || 1) * 20) / 20;
      const skipFactorB = Math.ceil(((statusB.skip ?? 0) + (statusB.unspecified ?? 0)) / (totalB || 1) * 20) / 20;

      if (failedFactorA !== failedFactorB) {
         return failedFactorB - failedFactorA;
      }

      if (skipFactorA !== skipFactorB) {
         return skipFactorB - skipFactorA;
      }

      return a.localeCompare(b);

   });

   const testIndexes = new Map<string, number>();
   tests.forEach(t => {
      testIndexes.set(t.id, sortedGroups.indexOf(groupNames.get(t.id)!))
   });


   tests = tests.sort((a, b) => {

      const groupA = testIndexes.get(a.id)!;
      const groupB = testIndexes.get(b.id)!;

      if (groupA !== groupB) {
         return groupA - groupB;
      }

      const statusA = handler.globalStatus.get(a.id);
      const statusB = handler.globalStatus.get(b.id);


      let stA = 4;
      let stB = 4;
      if (statusA?.error) {
         stA = 3;
      } else if (statusA?.skip) {
         stA = 2;
      } else if (statusA?.unspecified) {
         stA = 1;
      } else if (statusA?.success) {
         stA = 0;
      }
      if (statusB?.error) {
         stB = 3;
      } else if (statusB?.skip) {
         stB = 2;
      } else if (statusB?.unspecified) {
         stB = 1;
      } else if (statusB?.success) {
         stB = 0;
      }

      if (stA !== stB) {
         return stB - stA;
      }

      const nameA = SuiteHandler.fixedTestNames.get(a.id)!;
      const nameB = SuiteHandler.fixedTestNames.get(b.id)!;
      return nameA.localeCompare(nameB);
   });

   const groups: SuiteGroup[] = [];
   let cgroup: SuiteGroup | undefined;
   const colors = dashboard.getStatusColors();

   const items: TestItem[] = [];

   tests.forEach((t) => {

      const groupName = groupNames.get(t.id)!;
      const status = groupStatus.get(groupName)!

      let color = colors.get(StatusColor.Success)!;
      if (status.error) {
         color = colors.get(StatusColor.Failure)!;
      } else if (status.skip || status.unspecified) {
         color = colors.get(StatusColor.Skipped)!;
      }

      if (cgroup?.name !== groupName) {

         if (cgroup) {
            cgroup.count = items.length - cgroup.startIndex;
         }

         cgroup = {
            key: `group_${groupName?.replaceAll(".", "_")}`,
            name: groupName,
            startIndex: items.length,
            count: -1,
            isCollapsed: !handler.getFilterState()?.text,
            color: color,
            status: status
         }

         groups.push(cgroup);

      }

      items.push({
         key: `automation_suite_test_${t.displayName ?? t.name}_${id_counter++}`,
         name: SuiteHandler.fixedTestNames.get(t.id)!.replace(`${groupName}.`, ""),
         testId: t.id
      });

      const streams = Array.from(new Set<string>(handler.suiteRefs.map(s => s.streamId))).sort((a, b) => a.localeCompare(b));
      streams.forEach(streamId => {
         if (handler.getFilteredTestData(streamId, t.id).length) {
            items.push({
               key: `automation_suite_test_${t.displayName ?? t.name}_${id_counter++}`,
               name: projectStore.streamById(streamId)?.fullname ?? "Unknown Stream",
               streamId: streamId,
               testId: t.id
            });
         }
      });

   });

   if (groups.length) {
      groups[groups.length - 1].count = items.length - groups[groups.length - 1].startIndex;
   }

   const onRenderGroupHeader: IDetailsGroupRenderProps['onRenderHeader'] = (props) => {
      if (props) {

         const group = props.group! as SuiteGroup;
         const status = group.status;

         const total = ((status.totalError ?? 0) + (status.totalSkip ?? 0) + (status.totalUnspecified ?? 0) + (status.totalSuccess ?? 0) + (status.totalWarning ?? 0)) || 1;

         const failedFactor = (status.totalError ?? 0) / total;
         const skipFactor = ((status.totalSkip ?? 0) + (status.totalUnspecified ?? 0)) / total;
         const warnFactor = (status.totalWarning ?? 0) / total;
         const successFactor = (status.totalSuccess ?? 0) / total;

         const stack: StatusBarStack[] = [
            {
               value: successFactor * 100,
               title: "Passed",
               color: 'transparent'
            },
            {
               value: failedFactor * 100,
               title: "Failure",
               color: colors.get(StatusColor.Failure)!,
               stripes: true
            },
            {
               value: warnFactor * 100,
               title: "Warnings",
               color: scolors.get(StatusColor.Warnings)!,
               stripes: true
            },
            {
               value: skipFactor * 100,
               title: "Skipped",
               color: scolors.get(StatusColor.Skipped)!,
               stripes: true
            }
         ]

         let groupName = group.name;
         const suitePrefix = `${suite.name}.`;
         if (groupName.startsWith(suitePrefix)) {
            groupName = groupName.replace(suitePrefix, "");
         }

         return <Stack style={{ borderTop: dashboard.darktheme ? "1px solid #575E62" : "1px solid #EDEBE9", paddingTop: 4, paddingBottom: 4, cursor: "pointer" }} onClick={() => props.onToggleCollapse!(props.group!)}>
            <Stack horizontal verticalAlign="center" style={{ paddingBottom: 4 }}>
               <Stack style={{ paddingRight: 3 }}>
                  <FontIcon style={{ fontSize: 13 }} iconName={!group.isCollapsed ? "ChevronDown" : "ChevronRight"} />
               </Stack>
               <Stack>
                  <Text style={{ fontWeight: 600, fontSize: "13px" }}>{groupName}</Text>
               </Stack>
            </Stack>

            <Stack style={{ paddingLeft: 17 }}>
               {StatusBar(stack, 360, 10, colors.get(StatusColor.Success)!, { margin: '0px !important' })}
            </Stack>

         </Stack>
      }

      return null;
   };

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = items[props.itemIndex];

         let backgroundColor: string | undefined;
         if (!item.streamId) {
            backgroundColor = `${modeColors.crumbs} !important`;
         } else {
            const testStream = handler.testStreams.get(item.streamId);
            const status = testStream?.testStatus.get(item.testId);
            if (status) {
               if (status.error) {
                  backgroundColor = scolors.get(StatusColor.Failure)!;
                  backgroundColor += (dashboard.darktheme ? "30 " : "30")
               } else if (status?.success && (!status.skip && !status.unspecified)) {
                  backgroundColor = scolors.get(StatusColor.Success)!;
                  backgroundColor += (dashboard.darktheme ? "25 " : "20")
               } else if (status?.warning && (!status.skip && !status.unspecified)) {
                  backgroundColor = scolors.get(StatusColor.Warnings)!;
                  backgroundColor += (dashboard.darktheme ? "25 " : "20")
               }
            }
         }

         return <DetailsRow styles={{ root: { backgroundColor: backgroundColor } }}{...props} />
      }
      return null;
   };



   return <Stack key={`key_automation_suite_list_${id_counter++}`} className={hordeClasses.raised} style={{ height: "100%" }}>
      <Stack style={{ paddingLeft: 352, flexBasis: "32px" }}>
         <SuiteGraph handler={handler} />
      </Stack>
      <Stack style={{ overflowY: 'scroll', overflowX: 'hidden', height: "100%" }} data-is-scrollable={true}>
         <Stack style={{ width: 1256 }}>
            <DetailsList
               styles={{ root: { overflowX: "hidden" } }}
               key={`key_automation_suite_detaillist_${id_counter++}`}
               className={suiteStyles.group}
               compact={true}
               isHeaderVisible={false}
               items={items}
               groups={groups}
               groupProps={{
                  onRenderHeader: onRenderGroupHeader,
               }}
               columns={columns}
               layoutMode={DetailsListLayoutMode.justified}
               onShouldVirtualize={() => true}
               selectionMode={SelectionMode.none}
               onRenderItemColumn={renderItem}
               onRenderRow={renderRow}
            />
         </Stack>
      </Stack>
   </Stack>


});


export const AutomationSuiteDetails: React.FC<{ suite: GetTestSuiteResponse, suiteRefs: GetTestDataRefResponse[], metaData: GetTestMetaResponse, onClose: () => void }> = observer(({ suite, suiteRefs, metaData, onClose }) => {

   const [state, setState] = useState<{ handler?: SuiteHandler }>({});
   const { hordeClasses, modeColors } = getHordeStyling();

   let handler = state.handler;

   // subscribe
   if (!handler) {
      handler = new SuiteHandler(suite, [...suiteRefs], metaData);
      setState({ handler: handler });
      return null;
   }

   // subscribe
   if (handler.updated) { }

   return <Stack>
      <SuiteTestViewModal handler={handler} />
      <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1420, backgroundColor: modeColors.background, hasBeenOpened: false, top: "24px", position: "absolute", height: "95vh" } }} className={hordeClasses.modal} onDismiss={() => { if (!handler?.selection) onClose() }}>
         <Stack className="horde-no-darktheme" style={{ height: "93vh" }}>
            <Stack style={{ height: "95%" }} >
               <Stack horizontal style={{ paddingLeft: 24, paddingTop: 12, paddingRight: 18 }}>
                  <Stack>
                     <Text style={{ fontSize: 16, fontFamily: "Horde Open Sans Semibold" }}>{handler.metaData?.projectName} - {suite.name} - {handler.metaName}</Text>
                  </Stack>
                  <Stack grow />
                  <Stack>
                     <SuiteOperationsBar handler={handler} />
                  </Stack>
                  <Stack style={{ paddingLeft: 24, paddingTop: 0 }}>
                     <IconButton
                        style={{ height: "14px" }}
                        iconProps={{ iconName: 'Cancel' }}
                        onClick={() => onClose()} />
                  </Stack>
               </Stack>
               <Stack style={{ height: "100%", paddingLeft: 24, paddingRight: 24, paddingTop: 12 }}>
                  <div id="root_suite_test_list" style={{ position: "relative", height: "100%" }}>
                     {!handler.loaded && <Stack style={{ paddingTop: 16, height: "100%" }} horizontalAlign="center" tokens={{ childrenGap: 18 }}>
                        <Stack>
                           <Text variant="mediumPlus">Loading Test Suite</Text>
                        </Stack>
                        <Stack>
                           <Spinner size={SpinnerSize.large} />
                        </Stack>
                     </Stack>}
                     {!!handler.loaded && <Stack tokens={{ childrenGap: 18 }} style={{ height: "100%" }}>
                        <SuiteTestList handler={handler} suite={suite} />
                     </Stack>}
                  </div>
               </Stack>
            </Stack>
         </Stack>
      </Modal >
   </Stack>

   //<SuiteTestViewPanel handler={handler} />
});