// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, FontIcon, IContextualMenuItem, IContextualMenuItemProps, IContextualMenuProps, Image, Label, ScrollablePane, ScrollbarVisibility, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import React, { useState } from "react";
import { Link } from "react-router-dom";
import backend from "../backend";
import { GetSuiteTestDataResponse, GetTestDataDetailsResponse, GetTestDataRefResponse, GetTestMetaResponse, GetTestResponse, GetTestSuiteResponse, TestData, TestOutcome } from "../backend/Api";
import { TestDataHandler } from "../backend/AutomationTestData";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { getShortNiceTime } from "../base/utilities/timeUtils";
import { projectStore } from "../backend/ProjectStore";

type EventType = "Error" | "Info" | "Warning";

enum ArtifactTag {
   Approved = "approved",
   Unapproved = "unapproved",
   Difference = "difference"
}

enum EventTag {
   ImageComparison = "image comparison"
};

type TestArtifact = {
   ReferencePath: string;
   Tag: ArtifactTag;
}

type TestEvent = {
   Artifacts?: TestArtifact[];
   Context: string;
   Message: string;
   Tag: EventTag;
   Type: EventType;
}

class SuiteTestHandler {

   constructor(suite: GetTestSuiteResponse, test: GetTestResponse, details: GetTestDataDetailsResponse, testRef: GetTestDataRefResponse, metaData: GetTestMetaResponse) {

      makeObservable(this);

      this.suite = suite;
      this.test = test;

      this.refs.set(testRef.id, testRef);
      this.details.set(details.id, details);
      this.metaData.set(metaData.id, metaData);

      this.initialize(metaData, testRef);
   }

   clear() {
      this.events = [];
      this.jobId = undefined;
      this.stepId = undefined;
      this.artifactV2Id = undefined;
   }

   getArtifactImagePath(referencePath: string) {
      
      return `${backend.serverUrl}/api/v2/artifacts/${this.artifactV2Id}/file?path=Engine/Programs/AutomationTool/Saved/Logs/RunUnreal/${encodeURIComponent(referencePath)}&inline=true`;
      
   }

   getSuiteTest(metaId: string) {

      const ref = Array.from(this.refs.values()).find(r => r.metaId === metaId);

      if (!ref) {
         return undefined;
      }

      const details = this.details.get(ref.id);

      if (!details) {
         return undefined;
      }

      return details.suiteTests?.find(t => t.testId === this.test.id);
   }

   async set(testRef: GetTestDataRefResponse, initialUpdate = false) {

      if (this.cref?.id === testRef.id) {
         return;
      }

      this.clear();

      this.loading = true;

      if (!initialUpdate) {
         this.setUpdated();
      }

      let details = this.details.get(testRef.id);
      if (!details) {
         console.error("Missing details");
         return;
      }

      this.cref = testRef;
      this.cmetaData = this.metaData.get(testRef.metaId)!;
      this.cdetails = details;

      // handle test data
      const requestIds = details.testDataIds.filter(id => !this.testData.get(id));
      if (requestIds) {
         const requests = requestIds.map(id => backend.getTestData(id));
         const responses = await Promise.all(requests);
         responses.forEach(r => {
            this.testData.set(r.id, r);
         });
      }
      // Automated Test Session
      // Unreal Automated Tests::UE.EditorAutomation(RunTest=Rendering) Win64 d3d12
      // Automated Test Session Result Details::e5f21037-c022-47df-b3d7-028d54e03205

      const testData = details.testDataIds.map(id => this.testData.get(id)!);

      if (!!testData.find(td => !td)) {
         console.error("Missing testdata");
         return;
      }

      const suiteTest = this.csuiteTest = details.suiteTests?.find(t => t.testId === this.test.id);
      if (!suiteTest) {
         console.error("SuiteTestHandler: Missing suite test");
         return;
      }


      // @todo: these lookups aren't great 
      const session = testData.find(t => (t.data as any)["Type"]?.trim() === "Automated Test Session");
      //const tests = testData.find(t => (t.data as any)["Type"]?.trim() === "Unreal Automated Tests");
      const results = testData.find(t => (t.key as any)?.trim().startsWith("Automated Test Session Result Details"));

      this.events = [];

      this.jobId = session?.jobId;
      this.stepId = session?.stepId;

      if (!this.jobId || !this.stepId) {
         console.error("Couldn't get job or step id ");
         return;
      }

      if (results) {

         const data = (results.data as Record<string, any>)[suiteTest.uid];
         if (data.Events) {
            for (let e of (data.Events as TestEvent[])) {
               this.events.push(e);
            }
         }

         let needArtifacts = !!this.events.find(e => e.Tag === EventTag.ImageComparison);

         if (needArtifacts) {
                        
            if (!this.artifactV2Id) {                              
               const v = await backend.getJobArtifactsV2(undefined, [`job:${this.jobId}/step:${this.stepId}`]);               
               const av2 = v?.artifacts.find(a => a.type === "step-saved")
               if (av2?.id) {
                  this.artifactV2Id = av2.id;
               }
            }
         }
      }

      this.loading = false;
      this.setUpdated();
   }

   private async initialize(srcMetaData: GetTestMetaResponse, srcRef: GetTestDataRefResponse) {

      const instance = TestDataHandler.instance;

      const suiteMetaIds = Array.from(instance.metaData.keys()).filter(meta => !!this.suite.metadata.find(m => m === meta));
      suiteMetaIds.forEach(metaId => {
         const metaData = instance.metaData.get(metaId)!;
         this.metaData.set(metaData.id, metaData);
         this.metaNames.set(metaId, `${metaData.platforms.join(" - ")} / ${metaData.configurations.join(" - ")} / ${metaData.buildTargets.join(" - ")} / ${metaData.rhi === "default" ? "Default" : metaData.rhi?.toUpperCase()} / ${metaData.variation === "default" ? "Default" : metaData.variation?.toUpperCase()}`);
      })

      let altRefs: GetTestDataRefResponse[] = [];
      const altMetaIds = suiteMetaIds.filter(id => id !== srcMetaData.id);
      if (altMetaIds.length) {
         altRefs = await backend.getTestRefs([srcRef.streamId], altMetaIds, undefined, undefined, srcRef.buildChangeList, srcRef.buildChangeList, undefined, [this.suite.id]);
         altRefs.forEach(r => {
            this.refs.set(r.id, r);
         });
      }

      // want details here, so we can put the state in the dropdown, otherwise annoying to have to cycle to each 
      if (altRefs.length) {
         const altDetails = await backend.getTestDetails(altRefs.map(v => v.id));
         altDetails.forEach(d => {
            this.details.set(d.id, d);
         });
      }

      this.initialized = true;

      await this.set(srcRef, true);
   }

   @action
   setUpdated() {
      this.updated++;
   }

   @observable
   updated: number = 0;

   suite: GetTestSuiteResponse;

   cref?: GetTestDataRefResponse;
   cdetails?: GetTestDataDetailsResponse;
   cmetaData?: GetTestMetaResponse;
   csuiteTest?: GetSuiteTestDataResponse;

   // test ref id => refs
   refs = new Map<string, GetTestDataRefResponse>();

   // test ref id => details
   details = new Map<string, GetTestDataDetailsResponse>();

   // meta data id => meta data
   metaData = new Map<string, GetTestMetaResponse>();

   testData = new Map<string, TestData>();

   metaNames = new Map<string, string>();

   test: GetTestResponse;

   initialized = false;

   loading = false;

   jobId?: string;
   stepId?: string;
   artifactV2Id?: string;
   events: TestEvent[] = [];

}

let idCounter = 0;

const EventPanel: React.FC<{ handler: SuiteTestHandler }> = observer(({ handler }) => {

   if (handler.updated) { }

   if (handler.loading) {

      return <Stack grow style={{ paddingTop: 16 }} horizontalAlign="center" tokens={{ childrenGap: 18 }}>
         <Stack>
            <Text variant="mediumPlus">Loading Events</Text>
         </Stack>
         <Stack>
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Stack>
   }

   const events = handler.events.map(e => {

      const artifactMap = new Map<string, string>();

      const order = new Map<string, number>([[ArtifactTag.Approved, 0], [ArtifactTag.Difference, 1], [ArtifactTag.Unapproved, 2]]);

      const imageArtifacts = e.Artifacts?.filter(a => {
         return (a.Tag === ArtifactTag.Approved || a.Tag === ArtifactTag.Unapproved || a.Tag === ArtifactTag.Difference) && a.ReferencePath.endsWith(".png")
      }).sort((a, b) => {
         const oa = order.get(a.Tag) ?? -1;
         const ob = order.get(b.Tag) ?? -1;
         return oa - ob;
      });      

      if (e.Tag === EventTag.ImageComparison && (handler.artifactV2Id) && imageArtifacts?.length) {         

         imageArtifacts.forEach(ta => {

            if (ta.Tag !== ArtifactTag.Approved && ta.Tag !== ArtifactTag.Unapproved && ta.Tag !== ArtifactTag.Difference) {
               return;
            }
            
            const imageLink = handler.getArtifactImagePath(ta.ReferencePath);         
            if (imageLink) {
               artifactMap.set(ta.ReferencePath, imageLink);
            }
         })
      }

      const images = imageArtifacts?.map(a => {

         const url = artifactMap.get(a.ReferencePath);

         if (!url) {
            return <Stack>Missing Image: {a.ReferencePath}</Stack>
         }

         let tag = "Unknown Tag";
         switch (a.Tag) {
            case ArtifactTag.Approved:
               tag = "Reference";
               break;
            case ArtifactTag.Unapproved:
               tag = "Produced";
               break;
            case ArtifactTag.Difference:
               tag = "Difference";
               break;

         }

         return <Stack horizontalAlign="center" style={{ paddingBottom: 12 }}>
            <Image width="320px" src={url} />
            <Text>{tag}</Text>
         </Stack>
      });

      const scolors = dashboard.getStatusColors();
      let color = scolors.get(StatusColor.Unspecified)!;

      if (e.Type === "Error") {
         color = scolors.get(StatusColor.Failure)!
      }

      if (e.Type === "Warning") {
         color = scolors.get(StatusColor.Warnings)!
      }

      return <Stack key={`key_test_suite_events_${idCounter++}`} >
         <Stack horizontal tokens={{ childrenGap: 12 }}>
            <Stack>
               <Stack style={{ width: 6, height: "100%", backgroundColor: color }} />
            </Stack>
            <Stack tokens={{ childrenGap: 12 }}>
               {!!images?.length && <Stack horizontal tokens={{ childrenGap: 48 }}>
                  {images}
               </Stack>}
               <Stack>
                  <Text style={{ lineBreak: "anywhere", whiteSpace: "pre-wrap", fontSize: 10, lineHeight: "1.65", fontFamily: "Horde Cousine Regular" }}>{e.Message}</Text>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   });

   return <Stack grow style={{ paddingTop: 8 }} >
      <div style={{ position: 'relative', width: "100%", height: "100%" }} data-is-scrollable>
         <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto} onScroll={() => { }}>
            <Stack style={{ padding: 24 }} grow tokens={{ childrenGap: 24 }}>
               {events}
            </Stack>
         </ScrollablePane>
      </div>
   </Stack>
});


const MetaChooser: React.FC<{ handler: SuiteTestHandler }> = ({ handler }) => {

   const metaIds = new Set<string>(Array.from(handler.refs.values()).map(r => r.metaId));
   const metaData: GetTestMetaResponse[] = [];
   metaIds.forEach(id => {
      metaData.push(handler.metaData.get(id)!);
   });

   const platforms = Array.from(new Set<string>(metaData.map(d => d.platforms).flat())).sort((a, b) => a.localeCompare(b));

   const options: IContextualMenuItem[] = [];

   platforms.forEach(platform => {

      const metaItems: IContextualMenuItem[] = [];

      const pmeta = metaData.filter(m => m.platforms.indexOf(platform) !== -1).sort((a, b) => handler.metaNames.get(a.id)!.localeCompare(handler.metaNames.get(b.id)!));

      pmeta.forEach(pm => {

         const suiteTest = handler.getSuiteTest(pm.id);

         metaItems.push({
            key: `meta_select_${platform}_${pm.id}}`, text: handler.metaNames.get(pm.id), onClick: (ev, item) => {

               if (!item) {
                  return;
               }

               const ref = Array.from(handler.refs.values()).find(r => r.metaId === pm.id);

               if (ref) {
                  handler.set(ref);
               }

               // don't close
               //ev?.preventDefault()
            },
            data: suiteTest
         });

      });

      const renderMeta = (props: IContextualMenuItemProps) => {

         const test = props.item.data as GetSuiteTestDataResponse | undefined;

         const scolors = dashboard.getStatusColors();

         let color = scolors.get(StatusColor.Unspecified);

         if (test?.outcome === TestOutcome.Success) {
            color = scolors.get(StatusColor.Success)!;
         }

         if (test?.outcome === TestOutcome.Failure) {
            color = scolors.get(StatusColor.Failure)!;
         }

         if (test?.outcome === TestOutcome.Skipped) {
            color = scolors.get(StatusColor.Skipped)!;
         }

         return <Stack style={{ paddingLeft: 4 }}><Stack horizontal tokens={{ childrenGap: 8 }}>
            <Stack>
               <FontIcon style={{ color: color, fontSize: 11 }} iconName="Square" />
            </Stack>
            <Text style={{ fontSize: 11 }} >{props.item.text}</Text>
         </Stack>
         </Stack>;
      };

      const subMenuProps: IContextualMenuProps = {
         contextualMenuItemAs: renderMeta,
         shouldFocusOnMount: true,
         subMenuHoverDelay: 0,
         items: metaItems,
      };

      options.push({ key: `test_platform_select_${platform}`, text: platform, subMenuProps: subMenuProps });

   });

   const menuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: options,
   };

   return <DefaultButton style={{ width: 280, textAlign: "left" }} menuProps={menuProps} text={"Alternate Platform"} />

}

const TestInfoBox: React.FC<{ handler: SuiteTestHandler }> = observer(({ handler }) => {


   const labelWidth = 64;
   const textWidth = 120;

   if (handler.updated) { }

   if (!handler.cref) {
      return null;
   }

   const infoItem = (label: string, textIn: any, link?: string) => {
      const text = textIn.toString();
      return <Stack key={`test_info_item_${idCounter++}`} horizontal verticalAlign="center" style={{ width: labelWidth + textWidth + 12, height: 18 }}>
         <Stack style={{ width: labelWidth }}>
            <Label style={{ fontSize: 11 }}>{label}:</Label>
         </Stack>
         <Stack style={{ fontSize: 11, width: textWidth }}>
            {!link &&
               <Text style={{ fontSize: 11 }}>{text}</Text>}
            {!!link &&
               <Link style={{ fontSize: 11 }} to={link} target="_blank">{text}</Link>}
         </Stack>
      </Stack>
   }

   //const errorCount = handler.events?.filter(e => e.Type === "Error").length;
   //const warningCount = handler.events?.filter(e => e.Type === "Warning").length;

   const infoItems: JSX.Element[] = [];

   const timestamp = handler.cref.id.substring(0, 8)
   const time = getShortNiceTime(new Date(parseInt(timestamp, 16) * 1000), true);
   
   const streamName = projectStore.streamById(handler.cref.streamId)?.fullname ?? "Unknown Stream";

   infoItems.push(infoItem("Stream", streamName));
   infoItems.push(infoItem("Change", handler.cref.buildChangeList));
   infoItems.push(infoItem("Date", time));

   if (handler.jobId && handler.stepId) {
      infoItems.push(infoItem("Horde", "Job Step", `/job/${handler.jobId}/?step=${handler.stepId}`));
   }


   return <Stack style={{ width: 320 }}>
      {infoItems}
   </Stack>;
});

const AutomationSuiteTestInner: React.FC<{ handler: SuiteTestHandler }> = observer(({ handler }) => {

   if (handler.updated) { }

   if (!handler.initialized) {
      return <Stack style={{ paddingTop: 16 }} horizontalAlign="center" tokens={{ childrenGap: 18 }}>
         <Stack>
            <Text variant="mediumPlus">Loading Events</Text>
         </Stack>
         <Stack>
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Stack>
   }

   const test = handler.test;
   const suiteTest = handler.csuiteTest;

   if (!suiteTest) {
      return null;
   }


   const scolors = dashboard.getStatusColors();

   let color = scolors.get(StatusColor.Unspecified);
   if (suiteTest.outcome === TestOutcome.Success) {
      color = scolors.get(StatusColor.Success)!;
      if (suiteTest.warningCount) {
         color = scolors.get(StatusColor.Warnings)!;
      }
   }
   if (suiteTest.outcome === TestOutcome.Failure) {
      color = scolors.get(StatusColor.Failure)!;
   }
   


   // @todo: centralize this, and needs to be fixed on client
   const replace = ["Project.Functional Tests.Tests.", "Project.Maps.AllInPIE."];
   function fixName(name: string) {

      for (let r of replace) {
         if (name.startsWith(r)) {
            name = name.substring(r.length)
         }
      }
      return name;
   }

   const testName = fixName(test.displayName ?? test.name);

   return <Stack style={{ height: "100%" }} >
      <Stack horizontal tokens={{ childrenGap: 4 }} verticalAlign="center" style={{ paddingBottom: 0 }}>
         <Stack className="horde-no-darktheme" style={{ paddingTop: 2, paddingRight: 4 }}>
            <FontIcon style={{ color: color }} iconName="Square" />
         </Stack>
         <Stack tokens={{ childrenGap: 4 }}>
            <Stack>
               <Text style={{ fontSize: 13, fontFamily: "Horde Open Sans Semibold" }}>{testName}</Text>
            </Stack>
            <Stack>
               <Text style={{ fontSize: 11, fontFamily: "Horde Open Sans Semibold" }}>{handler.metaNames.get(handler.cmetaData!.id)}</Text>
            </Stack>
         </Stack>
         <Stack grow />
         <Stack style={{paddingRight: 8}}>
            <MetaChooser handler={handler} />
            </Stack>
      </Stack>

      <Stack style={{paddingLeft: 22}}>
         <TestInfoBox handler={handler} />
      </Stack>
      <Stack style={{ height: "100%" }}>
         <EventPanel handler={handler} />
      </Stack>
   </Stack>


});

export const AutomationSuiteTest: React.FC<{ suite: GetTestSuiteResponse, test: GetTestResponse, details: GetTestDataDetailsResponse, testRef: GetTestDataRefResponse, metaData: GetTestMetaResponse, onClose: () => void }> = observer(({ suite, test, details, testRef, metaData, onClose }) => {

   const [state, setState] = useState<{ handler?: SuiteTestHandler }>({});

   let handler = state.handler;

   // subscribe
   if (!handler) {
      handler = new SuiteTestHandler(suite, test, details, testRef, metaData);
      setState({ handler: handler });
      return null;
   }

   // subscribe
   if (handler.updated) { }

   return <Stack style={{ height: "100%" }} >
      <AutomationSuiteTestInner handler={handler} />
   </Stack>

});
