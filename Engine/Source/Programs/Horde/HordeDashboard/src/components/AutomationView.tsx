// Copyright Epic Games, Inc. All Rights Reserved.

import { ComboBox, DefaultButton, Dropdown, FontIcon, IComboBox, IComboBoxOption, IComboBoxStyles, IContextualMenuItem, IContextualMenuProps, IDropdownOption, Label, PrimaryButton, SelectableOptionMenuItemType, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import * as d3 from "d3";
import { observer } from 'mobx-react-lite';
import moment from 'moment';
import React, { useState } from 'react';
import { useNavigate, useSearchParams } from 'react-router-dom';
import backend from '../backend';
import { GetTestDataRefResponse, GetTestMetaResponse, GetTestResponse, GetTestSuiteResponse, StreamData, TestOutcome } from '../backend/Api';
import { FilterState, TestDataHandler } from '../backend/AutomationTestData';
import dashboard, { StatusColor } from '../backend/Dashboard';
import { projectStore } from '../backend/ProjectStore';
import { useWindowSize } from '../base/utilities/hooks';
import { getHumanTime, getShortNiceTime, msecToElapsed } from '../base/utilities/timeUtils';
import { AutomationSuiteDetails } from './AutomationSuiteDetails';
import { AutomationViewSummary } from './AutomationViewSummary';
import { Breadcrumbs } from './Breadcrumbs';
import ErrorBoundary from './error/ErrorBoundary';
import { TopNav } from './TopNav';
import { getHordeStyling } from '../styles/Styles';

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type DivSelectionType = d3.Selection<HTMLDivElement, unknown, null, undefined>;

// used to regenerate combo boxes upon automation project switches
let multiComboBoxId = 0;

const StreamChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {
   // subscribe
   if (handler.updated) { }

   const automation = handler.state.automation;

   if (!automation) {
      return null;
   }

   let cstreams: Set<string> = new Set(handler.state.streams ?? []);
   const streams = handler.getAutomationStreams(automation).sort((a, b) => a.localeCompare(b)).map(a => projectStore.streamById(a)).filter(a => !!a) as StreamData[];   

   if (!streams.length) {
      return null;
   }

   if (!cstreams.size)
   {
      streams.forEach(s => handler.addStream(s.id, false));
      cstreams = new Set(handler.state.streams ?? [])
      handler.query();
   }

   const options: IComboBoxOption[] = [];
   streams.forEach(stream => {
      options.push({ key: stream.id, text: stream.fullname ?? "???" });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Streams</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="stream" optionsIn={options} initialKeysIn={handler.state.streams ?? []} updateKeys={(keys) => {

            streams.forEach(t => {
               const selected = keys.find(k => k === t.id);
               if (!selected && cstreams.has(t.id)) {
                  handler.removeStream(t.id);
               }
               else if (selected && !cstreams.has(t.id)) {
                  handler.addStream(t.id);
               }
            });
         }} />
      </Stack>
   </Stack>
});

const AutomationChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const automation = handler.state.automation;

   const options: IContextualMenuItem[] = [];

   handler.automation.forEach(a => {
      options.push({
         key: `automation_${a}`, text: a, onClick: () => {
            multiComboBoxId++;
            handler.setAutomation(a, true);
         }
      });
   });

   const menuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: options,
   };

   return <DefaultButton style={{ width: 270, textAlign: "left" }} menuProps={menuProps} text={automation ? automation : "Select"} />
});


const MultiOptionChooser: React.FC<{ id: string, optionsIn: IComboBoxOption[], initialKeysIn: string[], updateKeys: (selectedKeys: string[]) => void }> = ({ id, optionsIn, initialKeysIn, updateKeys }) => {

   const comboBoxRef = React.useRef<IComboBox>(null);

   let initialKeys = [...initialKeysIn];
   let options = [...optionsIn];

   if (options.length === initialKeys.length) {
      initialKeys.push('selectAll');
   }

   options.unshift({ key: 'selectAll', text: 'Select All', itemType: SelectableOptionMenuItemType.SelectAll });

   const comboBoxStyles: Partial<IComboBoxStyles> = { root: { width: 270 } };

   return <ComboBox componentRef={comboBoxRef} key={`multi_option_${id}_${multiComboBoxId}`} placeholder="None" defaultSelectedKey={initialKeys} multiSelect options={options} onResolveOptions={() => options} onMenuDismiss={() => {
      if (comboBoxRef?.current?.selectedOptions) {
         const selectedKeys = comboBoxRef.current.selectedOptions.map(o => o.key as string).filter(k => k !== 'selectAll');
         setTimeout(() => { multiComboBoxId++; updateKeys(selectedKeys) }, 250);
      }
   }} styles={comboBoxStyles} />
};

const TestChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const ctests: Set<string> = new Set(handler.state.tests ?? []);
   //const metaTests = handler.metaTests;
   //const metaTestIds: Set<string> = new Set(metaTests.map(t => t.id));
   const streamTests = handler.streamTests;

   if (!streamTests.length) {
      return null;
   }

   const options: IComboBoxOption[] = [];
   streamTests.forEach(t => {
      options.push({ key: t.name, text: t.name });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Tests</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="test" optionsIn={options} initialKeysIn={handler.state.tests ?? []} updateKeys={(keys) => {

            handler.streamTests.forEach(t => {
               const selected = keys.find(k => k === t.name);
               if (!selected && ctests.has(t.name)) {
                  handler.removeTest(t.name);
               }
               else if (selected && !ctests.has(t.name)) {
                  handler.addTest(t.name);
               }
            });
         }} />
      </Stack>
   </Stack>
});

const SuiteChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const csuites: Set<string> = new Set(handler.state.suites ?? []);
   //const metaSuites = handler.metaSuites;
   //const metaSuiteIds: Set<string> = new Set(metaSuites.map(s => s.id));
   const streamSuites = handler.streamSuites;

   if (!streamSuites.length) {
      return null;
   }

   const options: IComboBoxOption[] = [];
   streamSuites.forEach(t => {
      options.push({ key: t.name, text: t.name });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Suites</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="suite" optionsIn={options} initialKeysIn={handler.state.suites ?? []} updateKeys={(keys) => {

            handler.streamSuites.forEach(t => {
               const selected = keys.find(k => k === t.name);
               if (!selected && csuites.has(t.name)) {
                  handler.removeSuite(t.name);
               }
               else if (selected && !csuites.has(t.name)) {
                  handler.addSuite(t.name);
               }
            });
         }} />
      </Stack>
   </Stack>
});


const PlatformChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const cplatforms: Set<string> = new Set(handler.state.platforms ?? []);

   const options: IComboBoxOption[] = [];

   handler.platforms.forEach(p => {
      options.push({ key: p, text: p });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Platforms</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="platform" optionsIn={options} initialKeysIn={handler.state.platforms ?? []} updateKeys={(keys) => {

            handler.platforms.forEach(p => {
               const selected = keys.find(k => k === p);
               if (!selected && cplatforms.has(p)) {
                  handler.removePlatform(p);
               }
               else if (selected && !cplatforms.has(p)) {
                  handler.addPlatform(p);
               }
            });
         }} />
      </Stack>
   </Stack>
});

const ConfigChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const cconfig: Set<string> = new Set(handler.state.configurations ?? []);

   const options: IComboBoxOption[] = [];

   handler.configurations.forEach(p => {
      options.push({ key: p, text: p });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Configurations</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="config" optionsIn={options} initialKeysIn={handler.state.configurations ?? []} updateKeys={(keys) => {

            handler.configurations.forEach(p => {
               const selected = keys.find(k => k === p);
               if (!selected && cconfig.has(p)) {
                  handler.removeConfiguration(p);
               }
               else if (selected && !cconfig.has(p)) {
                  handler.addConfiguration(p);
               }
            });
         }} />
      </Stack>
   </Stack>

});

const TargetChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const ctargets: Set<string> = new Set(handler.state.targets ?? []);

   const options: IComboBoxOption[] = [];

   handler.targets.forEach(p => {
      options.push({ key: p, text: p });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Targets</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="target" optionsIn={options} initialKeysIn={handler.state.targets ?? []} updateKeys={(keys) => {

            handler.targets.forEach(p => {
               const selected = keys.find(k => k === p);
               if (!selected && ctargets.has(p)) {
                  handler.removeTarget(p);
               }
               else if (selected && !ctargets.has(p)) {
                  handler.addTarget(p);
               }
            });
         }} />
      </Stack>
   </Stack>
});

const RHIChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const crhi: Set<string> = new Set(handler.state.rhi ?? []);

   const options: IComboBoxOption[] = [];

   handler.rhi.forEach(p => {
      options.push({ key: p, text: p === "default" ? "Default" : p.toUpperCase() });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>RHI</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="rhi" optionsIn={options} initialKeysIn={handler.state.rhi ?? []} updateKeys={(keys) => {
            handler.rhi.forEach(p => {
               const selected = keys.find(k => k === p);
               if (!selected && crhi.has(p)) {
                  handler.removeRHI(p);
               }
               else if (selected && !crhi.has(p)) {
                  handler.addRHI(p);
               }
            });
         }} />
      </Stack>
   </Stack>

});

const VariationChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   const cvariation: Set<string> = new Set(handler.state.variation ?? []);

   const options: IComboBoxOption[] = [];

   handler.variation.forEach(p => {
      options.push({ key: p, text: p === "default" ? "Default" : p.toUpperCase() });
   });

   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Variation</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <MultiOptionChooser id="variation" optionsIn={options} initialKeysIn={handler.state.variation ?? []} updateKeys={(keys) => {

            handler.variation.forEach(p => {
               const selected = keys.find(k => k === p);
               if (!selected && cvariation.has(p)) {
                  handler.removeVariation(p);
               }
               else if (selected && !cvariation.has(p)) {
                  handler.addVariation(p);
               }
            });
         }} />
      </Stack>
   </Stack>

});

const AutoExpandChooser: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }



   return <Stack style={{ paddingTop: 12, paddingBottom: 4 }}>
      <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
         <Label>Results</Label>
      </Stack>
      <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
         <DefaultButton text={handler.state.autoExpand ? "Expanded" : "Collapsed"} onClick={() => {
            handler.setAutoExpand(!handler.state.autoExpand);
         }} />
      </Stack>
   </Stack>

});


const AutomationSidebarLeft: React.FC<{ handler: TestDataHandler }> = ({ handler }) => {

   const { hordeClasses } = getHordeStyling();

   return <Stack style={{ width: 300, paddingRight: 18 }}>
      <Stack className={hordeClasses.modal}>
         <Stack>
            <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
               <Label>Automation</Label>
            </Stack>
            <Stack tokens={{ childrenGap: 6 }} style={{ paddingLeft: 12 }}>
               <AutomationChooser handler={handler} />
            </Stack>
         </Stack>
         <Stack>
            <TestChooser handler={handler} />
         </Stack>
         <Stack>
            <SuiteChooser handler={handler} />
         </Stack>
         <Stack>
            <StreamChooser handler={handler} />
         </Stack>
         <Stack>
            <PlatformChooser handler={handler} />
         </Stack>
         <Stack>
            <ConfigChooser handler={handler} />
         </Stack>
         <Stack>
            <TargetChooser handler={handler} />
         </Stack>
         <Stack>
            <RHIChooser handler={handler} />
         </Stack>
         <Stack>
            <VariationChooser handler={handler} />
         </Stack>
         <Stack>
            <AutoExpandChooser handler={handler} />
         </Stack>
      </Stack>
   </Stack>
}

const AutomationOperationsBar: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   const { hordeClasses } = getHordeStyling();

   if (handler.updated) { }
   if (handler.queryLoading) { }

   const stateItems: IDropdownOption[] = ["All", "Success", "Failed", "Consecutive Failures", "Skipped"].map(state => {
      return {
         key: state,
         text: state
      };
   });

   type TimeSelection = {
      text: string;
      key: string;
      weeks: number;
   }

   const timeSelections: TimeSelection[] = [
      {
         text: "Past Week", key: "time_1_week", weeks: 1
      },
      {
         text: "Past 2 Weeks", key: "time_2_weeks", weeks: 2
      },
      {
         text: "Past Month", key: "time_4_weeks", weeks: 4
      },
      {
         text: "Past 2 Months", key: "time_8_weeks", weeks: 8
      },
      {
         text: "Past 3 Months", key: "time_12_weeks", weeks: 12
      }
   ]

   const searchDisabled = handler.queryLoading || !handler.state.automation || !handler.state.streams?.length || (!handler.state.tests?.length && !handler.state.suites?.length);

   const selectedKey = timeSelections.find(t => t.weeks === handler.state.weeks);

   return <Stack className={hordeClasses.modal} horizontal style={{ paddingBottom: 18, paddingTop: 2 }}>
      <Stack grow />
      <Stack horizontal tokens={{ childrenGap: 24 }} verticalFill={true} verticalAlign={"start"}>
         <Stack>
            <Dropdown placeholder="Filter" style={{ width: 180 }} options={stateItems} selectedKey={handler.getFilterState() ?? "All"}
               onChange={(event, option, index) => {

                  if (option) {
                     if (option.key === "All") {
                        handler.setFilterState(undefined);
                     } else {
                        handler.setFilterState(option.key as FilterState);
                     }
                  }
               }} />
         </Stack>
         <Stack>
            <Dropdown style={{ width: 180 }} options={timeSelections} selectedKey={selectedKey?.key}
               onChange={(event, option, index) => {

                  const select = option as TimeSelection;
                  if (option) {
                     handler.setQueryWeeks(select.weeks);
                  }
               }}
            />
         </Stack>
         {false && <Stack>
            <PrimaryButton disabled={searchDisabled} text="Search" onClick={() => handler.query()} />
         </Stack>}
      </Stack>
   </Stack>;
})

export const AutomationView: React.FC = observer(() => {

   const navigate = useNavigate();
   const windowSize = useWindowSize();
   const [state, setState] = useState<{ handler?: TestDataHandler, search?: string }>({});
   const [searchParams, setSearchParams] = useSearchParams();

   let handler = state.handler;

   // subscribe
   if (!handler) {
      handler = new TestDataHandler(new URLSearchParams(searchParams));
      handler.load().catch((reason) => console.error(reason)).finally(() => setState({ handler: handler, search: handler!.search.toString() }))

      return null;
   }

   // subscribe
   if (handler.updated) { }

   const csearch = handler.search.toString();

   if (state.search !== csearch) {
      // this causes an error in dev mode, might be something to fix when move to new router (createBrowserRouter)
      setSearchParams(csearch);
      setState({ ...state, search: csearch });
      return null;
   }

   let suiteRefs: GetTestDataRefResponse[] | undefined;
   let metaData: GetTestMetaResponse | undefined;
   let suite: GetTestSuiteResponse | undefined;

   if (handler.suiteRef) {
      const ref = handler.getRef(handler.suiteRef);
      if (ref?.suiteId) {
         metaData = handler.metaData.get(ref.metaId);
         suiteRefs = handler.getSuiteRefs(ref.suiteId, ref.metaId);/*.filter(r => r.streamId === ref.streamId);*/
         suite = handler.suiteMap.get(ref.suiteId);
      }
   }

   const { hordeClasses, modeColors } = getHordeStyling();


   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const automationHubDocs = "/docs/Config/AutomationHub.md";

   return (
      <Stack className={hordeClasses.horde}>
         {(!!suiteRefs?.length && !!metaData && !!suite) && <AutomationSuiteDetails suite={suite} suiteRefs={suiteRefs} metaData={metaData} onClose={() => handler?.setSuiteRef(undefined)} />}
         <TopNav />
         <Breadcrumbs items={[{ text: 'Automation' }]} />
         <ErrorBoundary>
            <Stack horizontal>
               <div key={`windowsize_automationview_${windowSize.width}_${windowSize.height}`} style={{ width: (vw / 2 - (1440 / 2)) - 12, flexShrink: 0, backgroundColor: modeColors.background }} />
               <Stack horizontalAlign="center" grow styles={{ root: { width: "100%", padding: 12, backgroundColor: modeColors.background } }}>
                  <Stack styles={{ root: { width: "100%" } }}>
                     {handler.loaded && !handler.automation?.length && <Stack horizontal style={{ width: 1440, paddingTop: 30 }} tokens={{ childrenGap: 6 }} horizontalAlign="center">
                        <Text variant="mediumPlus">No automation metadata found, for more information please see</Text>
                        <a href={automationHubDocs} style={{ fontSize: "18px", "cursor": "pointer" }} onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(automationHubDocs) }}> automation hub documentation.</a>
                     </Stack>}
                     {handler.loaded && !!handler.automation?.length && <Stack style={{ paddingTop: 8 }}>
                        <Stack horizontal >
                           <AutomationSidebarLeft handler={handler} />
                           <Stack grow style={{ overflowX: "auto", overflowY: "visible", minWidth: "1128px" }}>
                              <AutomationCenter handler={handler} />
                           </Stack>
                        </Stack>
                     </Stack>}
                  </Stack>
               </Stack>
            </Stack>
         </ErrorBoundary>
      </Stack>
   );
});

let id_counter = 0;

const AutomationCenter: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   const windowSize = useWindowSize();

   if (handler.queryLoading) { }
   if (handler.queryUpdated) { }

   const refs = handler.getFilteredRefs();

   const height = windowSize.height - 230;

   if (handler.queryLoading) {
      return <Stack horizontalAlign='center' style={{ paddingTop: 24, height: height, width: "1148px" }} tokens={{ childrenGap: 24 }}>
         <Text style={{ fontSize: 24 }}>{!handler.hasQueried ? "Loading Data" : "Refreshing Data"}</Text>
         <Spinner size={SpinnerSize.large} />
      </Stack>
   }

   if (handler.hasQueried && !refs?.length) {

   }

   const streams: Set<string> = new Set();
   const testSet: Set<string> = new Set();
   const suiteSet: Set<string> = new Set();

   refs?.forEach(r => {

      const meta = handler.metaData.get(r.metaId);

      if (!meta) {
         console.warn(`Missing meta data on test ref`);
         return;
      }

      streams.add(r.streamId);
      if (r.testId) {
         testSet.add(r.testId);
      }
      if (r.suiteId) {
         suiteSet.add(r.suiteId);
      }
   });

   const tests = Array.from(testSet).map(tid => handler.testMap.get(tid)!).filter(t => !!t).sort((ta, tb) => ta!.name.localeCompare(tb!.name));
   const suites = Array.from(suiteSet).map(sid => handler.suiteMap.get(sid)!).filter(s => !!s).sort((sa, sb) => sa!.name.localeCompare(sb!.name));

   const testViews = tests.map(t => {
      return <Stack key={`test_view_${t.id}_${id_counter++}`}>

      </Stack>
      // <AutomationTestView test={t} handler={handler} />

   });

   const suiteViews = suites.map(s => {
      return <Stack key={`suite_view_${s.id}_${id_counter++}`}>

      </Stack>

      // <AutomationSuiteView suite={s} handler={handler} />
   });

   return <Stack style={{ paddingLeft: 12, paddingRight: 24, width: "1148px" }}>
      <Stack>
         <Stack style={{ paddingRight: 24 }}>
            {false && <AutomationOperationsBar handler={handler} />}
         </Stack>
      </Stack>
      {handler.hasQueried && !refs?.length && <Stack horizontalAlign='center' style={{ paddingTop: 24, height: height }} tokens={{ childrenGap: 24 }}>
         <Text style={{ fontSize: 24 }}>No Results</Text>
      </Stack>}
      <div style={{ position: 'relative', height: height }}>

         <Stack>
            <Stack style={{ paddingLeft: 12, paddingTop: 8 }} tokens={{ childrenGap: 8 }}>
               <AutomationViewSummary handler={handler} />
            </Stack>
         </Stack>
         {!!tests.length &&
            <Stack style={{ paddingRight: 24 }}>
               <Stack style={{ paddingLeft: 12, paddingTop: 8 }} tokens={{ childrenGap: 8 }}>
                  {testViews}
               </Stack>
            </Stack>
         }
         {!!suites.length &&
            <Stack style={{ paddingRight: 24 }}>
               <Stack style={{ paddingLeft: 12, paddingTop: 8 }} tokens={{ childrenGap: 8 }}>
                  {suiteViews}
               </Stack>
            </Stack>
         }
      </div>
   </Stack>

})

class AutomationGraph {

   constructor(id: string, streamId: string, handler: TestDataHandler) {

      this.id = id;
      this.streamId = streamId;

      this.handler = handler;
      //this.refs = handler.filteredRefs.filter(r => r.streamId === streamId && (r.testId === id || r.suiteId == id));
      this.refs = handler.getFilteredRefs(streamId, id);

      this.margin = { top: 0, right: 32, bottom: 0, left: 160 };
      this.clipId = `automation_clip_path_${id}_${streamId}`;

      this.suite = !!this.refs.find(r => r.suiteId === id);
   }

   initData() {

      const refs = this.refs;
      const handler = this.handler;

      const metaSet: Set<string> = new Set();
      const streamSet: Set<string> = new Set();
      refs.forEach(r => {
         metaSet.add(r.metaId);
         streamSet.add(r.streamId);
      });

      this.metaIds = Array.from(metaSet).sort((a, b) => {
         const nameA = handler.metaNames.get(a)!;
         const nameB = handler.metaNames.get(b)!;
         return nameA.localeCompare(nameB);
      });

      const commonMeta = handler.getCommonMeta(this.metaIds);

      this.metaIds.forEach(m => {

         const metaRefs = this.refs.filter(r => r.metaId === m).sort((a, b) => a.buildChangeList - b.buildChangeList).reverse();
         if (!metaRefs.length) {
            this.metaStatus.set(m, "Unspecified");
         } else {
            const cref = metaRefs[0];
            if (this.suite) {
               if (cref.suiteErrorCount) {
                  this.metaStatus.set(m, "Failure");
               } else if (cref.suiteWarningCount) {
                  this.metaStatus.set(m, "Warning");
               } else if (cref.suiteSkipCount) {
                  this.metaStatus.set(m, "Skipped");
               } else {
                  this.metaStatus.set(m, "Unspecified");
               }

            } else {
               this.metaStatus.set(m, cref.outcome ?? "Unspecified");
            }

         }

         const meta = handler.metaData.get(m)!
         const elements: string[] = [];

         elements.push(meta.platforms.join(" - "));

         if (!commonMeta.commonConfigs) {
            elements.push(meta.configurations.join(" - "));
         }

         if (!commonMeta.commonTargets) {
            elements.push(meta.buildTargets.join(" - "));
         }

         if (!commonMeta.commonRHI) {
            elements.push(meta.rhi === "default" ? "Default" : meta.rhi.toUpperCase());
         }

         if (!commonMeta.commonVariation) {
            elements.push(meta.variation === "default" ? "Default" : meta.variation.toUpperCase());
         }

         this.metaNames.set(m, `${elements.join(" / ")}`);

      });
   }

   render(container: HTMLDivElement) {

      if (this.hasRendered && !this.forceRender) {
         return;
      }

      const { modeColors } = getHordeStyling();

      this.clear();

      this.hasRendered = true;
      this.forceRender = false;

      this.initData();

      const handler = this.handler;
      const refs = this.refs.sort((a, b) => handler.metaNames.get(a.metaId)!.localeCompare(handler.metaNames.get(b.metaId)!));
      const width = 1000

      const scolors = dashboard.getStatusColors();
      const colors: Record<string, string> = {
         "Success": scolors.get(StatusColor.Success)!,
         "Failure": scolors.get(StatusColor.Failure)!,
         "Warning": scolors.get(StatusColor.Warnings)!,
         "Unspecified": scolors.get(StatusColor.Skipped)!,
         "Skipped": scolors.get(StatusColor.Skipped)!
      };

      const X = _d3.map(refs, (r) => handler.changeDates.get(r.buildChangeList)!.getTime() / 1000);
      const Y = _d3.map(refs, (r) => this.metaIds.indexOf(r.metaId));
      let Z: (TestOutcome | "Unspecified" | "Warning" | "Success" | "Failure")[] = [];
      if (this.suite) {
         Z = _d3.map(refs, (r) => {

            if (!r.suiteErrorCount && !r.suiteWarningCount && !r.suiteSkipCount) {
               // If there are skipped, will be a shape with warning/error/success included??
               return "Success";
            }

            if (r.suiteErrorCount) {
               return "Failure";
            }
            if (r.suiteWarningCount) {
               return "Warning";
            }

            return "Unspecified";

         });
      } else {
         Z = _d3.map(refs, (r) => r.outcome ?? "Unspecified");
      }

      const xDomain = d3.extent(handler.changeDates.values() as any, (d:any) => d.getTime() / 1000);
      let yDomain: any = Y;
      yDomain = new _d3.InternSet(yDomain);

      const I = d3.range(X.length);

      const yPadding = 1;
      const height = Math.ceil((yDomain.size + yPadding) * 16) + this.margin.top + this.margin.bottom;

      const xRange = [this.margin.left, width - this.margin.right];
      let yRange = [this.margin.top, height - this.margin.bottom];

      const xScale = _d3.scaleTime(xDomain, xRange);
      const yScale = _d3.scalePoint(yDomain, yRange).round(true).padding(yPadding);

      const svg = d3.select(container)
         .append("svg")
         .attr("width", width)
         .attr("height", height + 24)
         .attr("viewBox", [0, 0, width, height + 24] as any)


      const g = svg.append("g")
         .selectAll()
         .data(_d3.group(I, i => Y[i]))
         .join("g")
         .attr("transform", ([y]) => `translate(0,${(yScale(y) as any) + 16})`);

      g.append("line")
         .attr("stroke", (y: any) => {
            // Test Vite
            const status = this.metaStatus.get(this.metaIds[y])!;
            if (status === "Failure") {
               return colors[this.metaStatus.get(this.metaIds[y])!]
            }
            return dashboard.darktheme ? "#6D6C6B" : "#4D4C4B";
         })
         .attr("stroke-width", 1)
         .attr("stroke-linecap", 4)
         .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
         .attr("x1", ([, I]: any) => {
            return this.margin.left;  //xScale(d3.min(I, i => X[i as number]) as any)
         })
         .attr("x2", ([, I]: any) => width - this.margin.right);


      const radius = 3.5
      g.selectAll("circle")
         .data(([, I]: any) => I)
         .join("circle")
         .attr("id", i => `circle${refs[i as any].id}`)
         .attr("cx", i => xScale(X[i as any]))
         .attr("fill", i => colors[Z[i as any]] ?? "#0000FF")
         .attr("r", radius);

      g.append("text")
         .attr("text-anchor", "start")
         .style("alignment-baseline", "left")
         .style("font-family", "Horde Open Sans Regular")
         .style("font-size", 10)
         .attr("dy", "0.15em") // center stream name
         .attr("x", () => 0)
         .attr("fill", ([y]) => {
            const status = this.metaStatus.get(this.metaIds[y])!;
            if (!dashboard.darktheme && status === "Failure") {
               return colors[this.metaStatus.get(this.metaIds[y])!]
            }
            return dashboard.darktheme ? "#E0E0E0" : "#2D3F5F";
         })
         .text(([y]) => this.metaNames.get(this.metaIds[y])!);

      const tooltip = this.tooltip = d3.select(container)
         .append("div")
         .attr("id", "tooltip")
         .style("display", "none")
         .style("background-color", modeColors.background)
         .style("border", "solid")
         .style("border-width", "1px")
         .style("border-radius", "3px")
         .style("border-color", dashboard.darktheme ? "#413F3D" : "#2D3F5F")
         .style("padding", "8px")
         .style("position", "absolute")
         .style("pointer-events", "none");


      const xAxis = (g: SelectionType) => {

         const dateMin = new Date(xDomain[0]! * 1000);
         const dateMax = new Date(xDomain[1]! * 1000);

         let ticks: number[] = [];
         for (const date of d3.timeDays(dateMin, dateMax, 1).reverse()) {
            ticks.push(date.getTime() / 1000);
         }

         if (ticks.length > 14) {
            let nticks = [...ticks];
            // remove first and last, will be readded 
            const first = nticks.shift()!;
            const last = nticks.pop()!;

            const n = Math.floor(nticks.length / 12);

            const rticks: number[] = [];
            for (let i = 0; i < nticks.length; i = i + n) {
               rticks.push(nticks[i]);
            }

            rticks.unshift(first);
            rticks.push(last);
            ticks = rticks;

         }

         g.attr("transform", `translate(0,16)`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "9px")
            .call(d3.axisTop(xScale)
               .tickValues(ticks)
               //.ticks(d3.timeDays(dateMin, dateMax))
               .tickFormat(d => {
                  return getHumanTime(new Date((d as number) * 1000));
               })
               .tickSizeOuter(0))
            .call(g => g.select(".domain").remove())
            .call(g => g.selectAll(".tick line").attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25).clone()
               .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
               .attr("y2", height - this.margin.bottom)
            )
      }

      // top axis
      svg.append("g").call(xAxis)


      const closestData = (x: number, y: number): GetTestDataRefResponse | undefined => {

         y -= 16;

         const metaIds = this.metaIds;
         let closest = this.refs.reduce((best, ref, i) => {

            let absy = Math.abs(yScale(metaIds.indexOf(ref.metaId) as any)! - y)
            const timeStamp = handler.changeDates.get(ref.buildChangeList)!.getTime() / 1000;
            const sx = xScale(timeStamp);
            let absx = Math.abs(sx - x)

            const length = Math.sqrt(absy * absy + absx * absx);

            if (length < best.value) {
               return { index: i, value: length };
            }
            else {
               return best;
            }
         }, { index: 0, value: Number.MAX_SAFE_INTEGER });

         if (closest) {
            return this.refs[closest.index];
         }

         return undefined;

      }

      const handleMouseMove = (event: any) => {

         const mouseX = _d3.pointer(event)[0];
         const mouseY = _d3.pointer(event)[1];

         const closest = closestData(mouseX, mouseY);
         if (closest) {
            svg.selectAll(`circle`).attr("r", radius);
            svg.select(`#circle${closest.id}`).attr("r", radius * 2);

            const timestamp = closest.id.substring(0, 8)

            const cmeta = handler.getMetaString(closest.metaId, true);
            const date = getShortNiceTime(new Date(parseInt(timestamp, 16) * 1000), true, true);
            let desc = `${cmeta} <br/>`;
            desc += `CL ${closest.buildChangeList} <br/>`;
            desc += `Duration ${msecToElapsed(moment.duration(closest.duration).asMilliseconds(), true, true)} <br/>`;
            desc += `${date} <br/>`;

            const timeStamp = handler.changeDates.get(closest.buildChangeList)!.getTime() / 1000;
            const tx = xScale(timeStamp);
            const ty = yScale(this.metaIds.indexOf(closest.metaId) as any)!

            this.updateTooltip(true, tx, ty, desc);
         }
      }

      const handleMouseLeave = (event: any) => {
         svg.selectAll(`circle`).attr("r", radius);
         tooltip.style("display", "none");
      }

      const handleMouseClick = (event: any) => {

         const mouseX = _d3.pointer(event)[0];
         const mouseY = _d3.pointer(event)[1];

         const closest = closestData(mouseX, mouseY);
         if (closest) {

            if (this.suite) {
               handler.setSuiteRef(closest.id);
            } else {
               // @todo: optimize
               backend.getTestDetails([closest.id]).then(r => {
                  backend.getTestData(r[0].testDataIds[0], "jobId,stepId").then(d => {
                     window.open(`/job/${d.jobId}?step=${d.stepId}`, '_blank')
                  });
               });

            }
         }
      }

      svg.on("mousemove", (event) => handleMouseMove(event))
      svg.on("mouseleave", (event) => handleMouseLeave(event))
      svg.on("click", (event) => handleMouseClick(event))

   }

   updateTooltip(show: boolean, x?: number, y?: number, html?: string) {
      if (!this.tooltip) {
         return;
      }

      x = x ?? 0;
      y = y ?? 0;

      this.tooltip
         .style("display", "block")
         .html(html ?? "")
         .style("position", `absolute`)
         .style("width", `max-content`)
         .style("top", (y - 78) + "px")
         .style("left", `${x}px`)
         .style("transform", "translateX(-108%)")
         .style("font-family", "Horde Open Sans Semibold")
         .style("font-size", "10px")
         .style("line-height", "16px")
         .style("shapeRendering", "crispEdges")
         .style("stroke", "none")

   }

   clear() {
      // d3.selectAll("#automation_graph_container > *").remove();
   }


   // test or suite id
   id: string;

   streamId: string;

   suite = false;

   handler: TestDataHandler;
   margin: { top: number, right: number, bottom: number, left: number }

   hasRendered = false;
   forceRender = false;

   refs: GetTestDataRefResponse[] = [];
   metaIds: string[] = [];
   // metaId => status string
   metaStatus: Map<string, string> = new Map();
   // metaId => meta string
   metaNames: Map<string, string> = new Map();

   clipId: string;

   tooltip?: DivSelectionType;
}

// --------------------------------------------------------------------------------------------------
// TestView
// --------------------------------------------------------------------------------------------------

class TestGraphRenderer extends AutomationGraph {

}

const TestGraph: React.FC<{ testId: string, streamId: string, handler: TestDataHandler }> = ({ testId, streamId, handler }) => {

   const { hordeClasses } = getHordeStyling();

   const graph_container_id = `${testId}_${streamId}_automation_graph_container`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const [state, setState] = useState<{ graph?: TestGraphRenderer }>({});

   if (!state.graph) {
      setState({ ...state, graph: new TestGraphRenderer(testId, streamId, handler) })
      return null;
   }

   if (container) {
      try {
         state.graph?.render(container);
      } catch (err) {
         console.error(err);
      }

   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8, paddingTop: 8 }}>
         <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </Stack>;

}

export const AutomationTestView: React.FC<{ test: GetTestResponse, handler: TestDataHandler }> = observer(({ test, handler }) => {

   const { hordeClasses } = getHordeStyling();

   const refs = handler.getFilteredRefs(undefined, test.id);
   const streamSet = new Set<string>();

   refs.forEach(r => {
      streamSet.add(r.streamId);
   });

   const testStreams = Array.from(streamSet).sort((a, b) => a.localeCompare(b));

   const streamMap = new Map<string, Map<string, GetTestDataRefResponse[]>>();
   refs.forEach(r => {

      let streamRefs = streamMap.get(r.streamId);
      if (!streamRefs) {
         streamRefs = new Map();
         streamMap.set(r.streamId, streamRefs);
      }

      let metaRefs = streamRefs.get(r.metaId);
      if (!metaRefs) {
         metaRefs = [];
         streamRefs.set(r.metaId, metaRefs);
      }
      metaRefs.push(r);
   });

   let anyFailed = false;
   testStreams.forEach(s => {
      const streamRefs = streamMap.get(s)!;
      for (const [, refs] of streamRefs) {
         const v = refs.sort((a, b) => a.buildChangeList - b.buildChangeList).reverse()[0];
         if (v?.outcome !== "Success") {
            anyFailed = true;
         }
      }
   })

   const testCommonMeta = handler.getCommonMeta(Array.from(new Set(refs.map(r => r.metaId))));
   const scolors = dashboard.getStatusColors();
   const streamViews = testStreams.map(s => {

      const streamName = projectStore.streamById(s)?.fullname ?? s;
      const streamRefs = streamMap.get(s)!;
      const metaIds = Array.from(new Set<string>(streamRefs.keys()));

      const commonMeta = handler.getCommonMeta(metaIds, !testCommonMeta.commonConfigs, !testCommonMeta.commonTargets, !testCommonMeta.commonRHI, !testCommonMeta.commonVariation);

      let anyPlatformFailed = false;
      for (const [, refs] of streamRefs) {
         const v = refs.sort((a, b) => a.buildChangeList - b.buildChangeList).reverse()[0];
         if (v?.outcome !== "Success") {
            anyPlatformFailed = true;
         }
      }

      let name = streamName;
      if (commonMeta.commonMetaString) {
         name += ` - ${commonMeta.commonMetaString}`
      }

      return <Stack key={`automation_test_view_${s}_${test.id}`}>
         <Stack horizontal>
            <Stack className="horde-no-darktheme" style={{ paddingTop: 2, paddingRight: 3 }}>
               <FontIcon style={{ fontSize: 11, color: anyPlatformFailed ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Success)! }} iconName="Square" />
            </Stack>
            <Stack>
               <Text style={{ fontSize: 11 }}>{name}</Text>
            </Stack>
         </Stack>
         <Stack style={{ paddingTop: 4 }}>
            <TestGraph testId={test.id} streamId={s} handler={handler} />
         </Stack>
      </Stack>
   });

   let name = test.name;
   if (testCommonMeta.commonMetaString) {
      name += ` - ${testCommonMeta.commonMetaString}`
   }

   return <Stack>
      <Stack className={hordeClasses.raised}>
         <Stack style={{ paddingBottom: 12 }}>
            <Stack horizontal>
               <Stack className="horde-no-darktheme" style={{ paddingTop: 3, paddingRight: 4 }}>
                  <FontIcon style={{ color: anyFailed ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Success)! }} iconName="Square" />
               </Stack>
               <Stack>
                  <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>{name}</Text>
               </Stack>
            </Stack>
         </Stack>
         <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
            {streamViews}
         </Stack>
      </Stack>
   </Stack>;
})

// --------------------------------------------------------------------------------------------------
// SuiteView
// --------------------------------------------------------------------------------------------------

class SuiteGraphRenderer extends AutomationGraph {
}

const SuiteGraph: React.FC<{ suiteId: string, streamId: string, handler: TestDataHandler }> = observer(({ suiteId, streamId, handler }) => {

   const { hordeClasses } = getHordeStyling();

   const graph_container_id = `${suiteId}_${streamId}_automation_suite_graph_container`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const [state, setState] = useState<{ graph?: SuiteGraphRenderer }>({});

   if (!state.graph) {
      setState({ ...state, graph: new SuiteGraphRenderer(suiteId, streamId, handler) })
      return null;
   }

   if (container) {
      try {
         state.graph?.render(container);
      } catch (err) {
         console.error(err);
      }
   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8 }}>
         <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </Stack>;

})


export const AutomationSuiteView: React.FC<{ suite: GetTestSuiteResponse, handler: TestDataHandler }> = observer(({ suite, handler }) => {

   const { hordeClasses } = getHordeStyling();
   const refs = handler.getFilteredRefs(undefined, suite.id);
   const streamSet = new Set<string>();

   refs.forEach(r => {
      streamSet.add(r.streamId);
   });

   const testStreams = Array.from(streamSet).sort((a, b) => a.localeCompare(b));

   const streamMap = new Map<string, Map<string, GetTestDataRefResponse[]>>();
   refs.forEach(r => {

      let streamRefs = streamMap.get(r.streamId);
      if (!streamRefs) {
         streamRefs = new Map();
         streamMap.set(r.streamId, streamRefs);
      }

      let metaRefs = streamRefs.get(r.metaId);
      if (!metaRefs) {
         metaRefs = [];
         streamRefs.set(r.metaId, metaRefs);
      }
      metaRefs.push(r);
   });

   let anyFailed = false;
   testStreams.forEach(s => {
      const streamRefs = streamMap.get(s)!;
      for (const [, refs] of streamRefs) {
         const v = refs.sort((a, b) => a.buildChangeList - b.buildChangeList).reverse()[0];
         if (v?.suiteErrorCount || v?.suiteSkipCount) {
            anyFailed = true;
         }
      }
   })

   const scolors = dashboard.getStatusColors();
   const suiteCommonMeta = handler.getCommonMeta(Array.from(new Set(refs.map(r => r.metaId))));

   const streamViews = testStreams.map(s => {
      const streamName = projectStore.streamById(s)?.fullname ?? s;
      const streamRefs = streamMap.get(s)!;
      const metaIds = Array.from(new Set<string>(streamRefs.keys()));


      const commonMeta = handler.getCommonMeta(metaIds, !suiteCommonMeta.commonConfigs, !suiteCommonMeta.commonTargets, !suiteCommonMeta.commonRHI);

      let anyPlatformFailed = false;
      for (const [, refs] of streamRefs) {
         const v = refs.sort((a, b) => a.buildChangeList - b.buildChangeList).reverse()[0];
         if (v?.suiteErrorCount || v?.suiteSkipCount) {
            anyPlatformFailed = true;
         }
      }

      let name = streamName;
      if (commonMeta.commonMetaString) {
         name += ` - ${commonMeta.commonMetaString}`
      }

      return <Stack key={`automation_suite_view_${s}_${suite.id}`}>
         <Stack horizontal>
            <Stack className="horde-no-darktheme" style={{ paddingTop: 2, paddingRight: 3 }}>
               <FontIcon style={{ fontSize: 11, color: anyPlatformFailed ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Success)! }} iconName="Square" />
            </Stack>
            <Stack>
               <Text style={{ fontSize: 11 }}>{name}</Text>
            </Stack>
         </Stack>
         <Stack style={{ paddingTop: 4 }}>
            <SuiteGraph suiteId={suite.id} streamId={s} handler={handler} />
         </Stack>
      </Stack>
   });

   let name = suite.name;
   if (suiteCommonMeta.commonMetaString) {
      name += ` - ${suiteCommonMeta.commonMetaString}`
   }

   return <Stack>
      <Stack className={hordeClasses.raised}>
         <Stack style={{ paddingBottom: 12 }}>
            <Stack horizontal>
               <Stack className="horde-no-darktheme" style={{ paddingTop: 3, paddingRight: 4 }}>
                  <FontIcon style={{ color: anyFailed ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Success)! }} iconName="Square" />
               </Stack>
               <Stack>
                  <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>{name}</Text>
               </Stack>
            </Stack>
         </Stack>
         <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
            {streamViews}
         </Stack>
      </Stack>
   </Stack>;

})