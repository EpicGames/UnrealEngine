// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, FontIcon, IconButton, mergeStyleSets, Modal, ScrollablePane, ScrollbarVisibility, Stack, Text } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { useState } from "react";
import backend from "../backend";
import { GetTestResponse, GetTestSuiteResponse } from "../backend/Api";
import { TestDataHandler } from "../backend/AutomationTestData";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { projectStore } from "../backend/ProjectStore";
import { getShortNiceTime } from "../base/utilities/timeUtils";
import { getHordeStyling } from "../styles/Styles";
import { StatusBar, StatusBarStack } from "./AutomationCommon";
import { AutomationFailureModal } from "./AutomationReport";
import { AutomationSuiteView, AutomationTestView } from "./AutomationView";


const TestSummaryModal: React.FC<{ test: GetTestResponse, handler: TestDataHandler, onDismiss: () => void }> = ({ test, handler, onDismiss }) => {

   const { hordeClasses } = getHordeStyling();

   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ scrollableContent: { overflow: "auto", height: "calc(100vh - 180px)" }, main: { padding: 12, width: 1120, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onDismiss()} >
      <Stack>
         <Stack horizontal verticalAlign="center">
            <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>{`${test.displayName ?? test.name} History`}</Text></Stack>
            <Stack grow />
            <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onDismiss(); }}
               />
            </Stack>
         </Stack>
         <Stack style={{ paddingRight: 12 }}>
            <div style={{ marginTop: 8, height: 'calc(100vh - 258px)', position: 'relative' }} data-is-scrollable={true}>
               <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} onScroll={() => { }}>
                  <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 12 } }}>
                     <AutomationTestView test={test} handler={handler} />
                  </Stack>
               </ScrollablePane>
            </div>
         </Stack>
      </Stack>

   </Modal>
}

const TestSummaryButton: React.FC<{ test: GetTestResponse, handler: TestDataHandler }> = ({ test, handler }) => {

   const search = new URLSearchParams(window.location.search);

   const [autoExpand, setAutoExpanded] = useState(search.get("autoexpand") === "true");

   const [showReport, setShowReport] = useState(false);
   const [expanded, setExpanded] = useState(false);
   const [streamExpanded, setStreamExpanded] = useState<Map<string, boolean>>(new Map());
   const [historyShow, setHistoryShown] = useState(false);

   const { hordeClasses, modeColors } = getHordeStyling();
   const statusColors = dashboard.getStatusColors();

   const styles = mergeStyleSets({
      metaitem: {
         selectors: {
            ':hover': {
               filter: dashboard.darktheme ? "brightness(120%)" : "brightness(90%)"
            }
         }
      }
   });

   const colorA = dashboard.darktheme ? "#181A1B" : "#e8e8e8";
   const colorB = dashboard.darktheme ? "#242729" : "#f8f8f8";

   const status = handler.getStatus(test.id);

   if (!status) {
      return <div />;
   }

   if (autoExpand) {

      const streamExpand = new Map<string, boolean>();
      Array.from(status.keys()).forEach(s => streamExpand.set(s, true));
      setStreamExpanded(streamExpand);
      setExpanded(true);
      setAutoExpanded(false);
      return null;
   }

   const streamElements: JSX.Element[] = [];

   let anyFailed = false;
   const failFactors = new Map<string, number>();

   let testFailed = 0;
   let testSkipped = 0;
   let testTotal = 0;

   for (let [streamId, testMeta] of status) {

      let streamFailed = 0;
      let streamSkipped = 0;
      let streamTotal = 0;

      for (let [, metaStatus] of testMeta) {
         if (metaStatus.error) {
            streamFailed++;
            anyFailed = true;
         }
         else if (metaStatus.skip) {
            streamSkipped++;
            anyFailed = true;
         }

         streamTotal++;
      }

      testFailed += streamFailed;
      testSkipped += streamSkipped;
      testTotal += streamTotal;

      const failedFactor = Math.ceil((streamFailed + streamSkipped) / (streamTotal || 1) * 20) / 20;
      failFactors.set(streamId, failedFactor);
   }

   let streamIds = Array.from(status.keys()).sort((a, b) => {
      const fA = failFactors.get(a)!;
      const fB = failFactors.get(b)!;
      if (fA !== fB) {
         return fB - fA;
      }
      return a.localeCompare(b);
   });

   streamIds.forEach((streamId) => {

      const streamName = projectStore.streamById(streamId)?.fullname ?? streamId;

      const failedFactor = failFactors.get(streamId)!

      const stack: StatusBarStack[] = [
         {
            value: failedFactor * 100,
            title: "Failure",
            color: statusColors.get(StatusColor.Failure)!,
            stripes: true
         },
         {
            value: (1 - failedFactor) * 100,
            title: "Passed",
            color: 'transparent'
         }
      ]

      const metaElements: JSX.Element[] = [];

      if (streamExpanded.get(streamId)) {

         const testMeta = status.get(streamId);

         if (testMeta && testMeta.size) {

            let metaIds = Array.from(testMeta?.keys());

            const commonMeta = handler.getCommonMeta(metaIds);

            metaIds = metaIds.sort((a, b) => {

               const sA = testMeta.get(a)!;
               const sB = testMeta.get(b)!;

               if (sA.refs[0].buildChangeList !== sB.refs[0].buildChangeList) {
                  return sB.refs[0].buildChangeList - sA.refs[0].buildChangeList;
               }

               const errorA = sA.error ? 1 : 0;
               const errorB = sB.error ? 1 : 0;

               if (errorA !== errorB) {
                  return errorB - errorA;
               }

               return handler.metaNames.get(a)!.localeCompare(handler.metaNames.get(b)!);
            });

            let latestCL = 0;
            for (let metaId of metaIds) {
               latestCL = Math.max(latestCL, testMeta.get(metaId)!.refs[0].buildChangeList);
            }


            for (let metaId of metaIds) {

               const metaStatus = testMeta.get(metaId)!;

               const last = metaStatus.refs[0];

               // todo move this to handler, duped in a few places
               const meta = handler.metaData.get(metaId)!
               const elements: string[] = [];

               elements.push(meta.platforms.join(" - "));

               if (!commonMeta.commonConfigs) {
                  elements.push(meta.configurations.join(" - "));
               }

               if (!commonMeta.commonTargets) {
                  elements.push(meta.buildTargets.join(" - "));
               }

               if (!commonMeta.commonRHI) {
                  elements.push(meta.rhi === "default" ? "Default" : meta.rhi.toUpperCase());
               }

               if (!commonMeta.commonVariation) {
                  elements.push(meta.variation === "default" ? "Default" : meta.variation.toUpperCase());
               }

               const metaName = `${elements.join(" / ")}`;

               let color = statusColors.get(StatusColor.Success);
               if (metaStatus.error) {
                  color = statusColors.get(StatusColor.Failure);
               } else if (metaStatus.skip) {
                  color = statusColors.get(StatusColor.Skipped);
               }

               let dateString = "";

               let date = handler.idDates.get(last.id);
               
               if (!date) {
                  date = handler.changeDates.get(last.buildChangeList);
               }               
               if (date) {
                  dateString = getShortNiceTime(date, false, false)
               }

               let fontWeight: number | undefined;
               if (last.buildChangeList === latestCL) {
                  fontWeight = 600
               }

               metaElements.push(
                  <Stack className="horde-no-darktheme">
                     <Stack className={styles.metaitem} horizontal verticalAlign="center" style={{ cursor: "pointer", backgroundColor: metaElements.length % 2 ? colorA : colorB, paddingTop: 2, paddingBottom: 2 }} onClick={(ev) => {
                        ev.stopPropagation();
                        backend.getTestDetails([last.id]).then(r => {
                           backend.getTestData(r[0].testDataIds[0], "jobId,stepId").then(d => {
                              window.open(`/job/${d.jobId}?step=${d.stepId}`, '_blank')
                           });
                        });

                     }}>
                        <Stack className="horde-no-darktheme" style={{ paddingLeft: 8, paddingTop: 1, paddingRight: 4 }}>
                           <FontIcon style={{ fontSize: 11, color: color }} iconName="Square" />
                        </Stack>
                        <Stack horizontal>
                           <Stack style={{ width: 124 }}>
                              <Text variant="xSmall" style={{ fontWeight: fontWeight }}>{metaName}</Text>
                           </Stack>
                           <Stack horizontal style={{ width: 84 }} verticalFill verticalAlign="center" tokens={{ childrenGap: 4 }}>
                              <Text variant="xSmall" style={{ fontWeight: fontWeight }}>CL {last.buildChangeList}</Text>
                              {last.buildChangeList === latestCL && <FontIcon style={{ fontSize: 11, color: color }} iconName="Star" />}
                           </Stack>
                           <Stack style={{ width: 60 }} horizontalAlign="end">
                              <Text variant="xSmall" style={{ fontWeight: fontWeight }}>{dateString}</Text>
                           </Stack>
                        </Stack>
                     </Stack>
                  </Stack>
               )
            }

         }

      }

      streamElements.push(<Stack style={{ paddingTop: streamElements.length === 0 ? 8 : 0 }} tokens={{ childrenGap: 2 }} onClick={(ev) => {
         ev.stopPropagation();
         const sexpanded = !!streamExpanded.get(streamId);
         streamExpanded.set(streamId, !sexpanded);
         setStreamExpanded(new Map(streamExpanded));
      }}>
         <Stack horizontalAlign="start" style={{ paddingRight: 1 }}>
            <Stack horizontal verticalAlign="center">
               <Stack style={{ paddingRight: 3 }}>
                  <FontIcon style={{ fontSize: 13 }} iconName={streamExpanded.get(streamId) ? "ChevronDown" : "ChevronRight"} />
               </Stack>
               <Stack>
                  <Text variant="small" style={{ fontWeight: 600 }}>{streamName}</Text>
               </Stack>
            </Stack>
         </Stack>
         {StatusBar(stack, 298, 10, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
         {!!streamExpanded.get(streamId) && <Stack style={{ border: `1px solid ${dashboard.darktheme ? "#3F4447" : "#CDCBC9"}` }}>{metaElements}</Stack>}
      </Stack>)
   });

   const testFailedFactor = Math.ceil((testFailed + testSkipped) / (testTotal || 1) * 20) / 20;

   const testStack: StatusBarStack[] = [
      {
         value: testFailedFactor * 100,
         title: "Failure",
         color: statusColors.get(StatusColor.Failure)!,
         stripes: true
      },
      {
         value: (1 - (testFailedFactor)) * 100,
         title: "Passed",
         color: 'transparent'
      }
   ]

   return <Stack className={hordeClasses.raised} style={{ cursor: "pointer", height: "fit-content", backgroundColor: dashboard.darktheme ? "#242729" : modeColors.background, padding: 12, width: 332 }} onClick={() => setExpanded(!expanded)}>
      {historyShow && <TestSummaryModal test={test} handler={handler} onDismiss={() => setHistoryShown(false)} />}
      {showReport && <AutomationFailureModal test={test} handler={handler} onDismiss={() => setShowReport(false)} />}
      <Stack horizontal verticalAlign="center" >
         <Stack className="horde-no-darktheme" style={{ paddingTop: 1, paddingRight: 4 }}>
            <FontIcon style={{ fontSize: 13 }} iconName={expanded ? "ChevronDown" : "ChevronRight"} />
         </Stack>
         <Stack>
            <Text style={{ fontWeight: 600 }}>{test.displayName ?? test.name}</Text>
         </Stack>

         <Stack grow />
         {anyFailed && <Stack style={{ paddingRight: 12 }}>
            <DefaultButton style={{ minWidth: 52, fontSize: 10, padding: 0, height: 18 }} onClick={(ev) => {
               ev.stopPropagation();
               setShowReport(true);
            }}>Failures</DefaultButton>
         </Stack>}

         <Stack horizontalAlign="end">
            <DefaultButton style={{ minWidth: 52, fontSize: 10, padding: 0, height: 18 }} onClick={(ev) => {
               ev.stopPropagation();
               setHistoryShown(true);
            }}>History</DefaultButton>
         </Stack>

      </Stack>
      <Stack>
         <Stack style={{ paddingTop: 4 }}>
            {StatusBar(testStack, 308, 10, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
         </Stack>

      </Stack>
      {!!expanded && <Stack style={{ paddingLeft: 8 }} tokens={{ childrenGap: 4 }}>
         {streamElements}
      </Stack>}
   </Stack>
}


const TestSummary: React.FC<{ handler: TestDataHandler }> = ({ handler }) => {

   const { hordeClasses } = getHordeStyling();

   let ctests = handler.getStatusTests();

   if (!ctests.length) {
      return null;
   }

   const failing: GetTestResponse[] = [];
   const passing: GetTestResponse[] = [];

   ctests.forEach(test => {

      // sort by failed
      const status = handler.getStatus(test.id);

      if (!status)
         return;

      let anyFailed = false;

      for (let [, testMeta] of status) {

         for (let [, metaStatus] of testMeta) {
            if (metaStatus.error) {
               anyFailed = true;
            }
            else if (metaStatus.skip) {
               anyFailed = true;
            }
         }
      }

      if (anyFailed) {
         failing.push(test);
      } else {
         passing.push(test);
      }
   })

   const tests = [...failing, ...passing].map(t => {

      return <TestSummaryButton test={t} handler={handler} />

   })

   return <Stack className={hordeClasses.raised}>
      <Stack style={{ paddingBottom: 12 }}>
         <Stack>
            <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>Latest Test Results</Text>
         </Stack>
      </Stack>
      <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
         <Stack wrap horizontal style={{ width: "100%" }} tokens={{ childrenGap: 12 }}>
            {tests}
         </Stack>
      </Stack>
   </Stack>
}

// Suites -------------------------------------------

const SuiteSummaryModal: React.FC<{ suite: GetTestSuiteResponse, handler: TestDataHandler, onDismiss: () => void }> = ({ suite, handler, onDismiss }) => {

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ scrollableContent: { overflow: "auto", maxHeight: "calc(100vh - 180px)" }, main: { padding: 8, width: 1134, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onDismiss()} >
      <Stack>
         <Stack horizontal verticalAlign="center">
            <Stack><Text style={{ paddingLeft: 8, fontSize: 16, fontWeight: 600 }}>{`${suite.name} History`}</Text></Stack>
            <Stack grow />
            <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onDismiss(); }}
               />
            </Stack>
         </Stack>
         <Stack>
            <div>

               <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
                  <AutomationSuiteView suite={suite} handler={handler} />
               </Stack>
            </div>
         </Stack>
      </Stack>

   </Modal>


}


const SuiteSummaryButton: React.FC<{ suite: GetTestSuiteResponse, handler: TestDataHandler }> = ({ suite, handler }) => {

   const search = new URLSearchParams(window.location.search);

   const [expanded, setExpanded] = useState(false);
   const [streamExpanded, setStreamExpanded] = useState<Map<string, boolean>>(new Map());
   const [historyShow, setHistoryShown] = useState(false);
   const statusColors = dashboard.getStatusColors();
   const { hordeClasses, modeColors } = getHordeStyling();
   const [autoExpand, setAutoExpanded] = useState(search.get("autoexpand") === "true");

   const styles = mergeStyleSets({
      metaitem: {
         selectors: {
            ':hover': {
               filter: dashboard.darktheme ? "brightness(120%)" : "brightness(90%)"
            }
         }
      }
   });

   const colorA = dashboard.darktheme ? "#181A1B" : "#e8e8e8";
   const colorB = dashboard.darktheme ? "#242729" : "#f8f8f8";

   const status = handler.getStatus(suite.id);

   if (!status) {
      return <div />;
   }

   if (autoExpand) {

      const streamExpand = new Map<string, boolean>();
      Array.from(status.keys()).forEach(s => streamExpand.set(s, true));
      setStreamExpanded(streamExpand);
      setExpanded(true);
      setAutoExpanded(false);
      return null;
   }


   const streamElements: JSX.Element[] = [];

   let anyFailed = false;
   let anyWarnings = false;
   const failFactors = new Map<string, number>();
   const warningFactors = new Map<string, number>();
   const skipFactors = new Map<string, number>();
   const successFactors = new Map<string, number>();

   const failCount = new Map<string, number>();
   const warningCount = new Map<string, number>();
   const skipCount = new Map<string, number>();
   const successCount = new Map<string, number>();


   let testFailed = 0;
   let testSkipped = 0;
   let testWarnings = 0;
   let testSuccess = 0;
   let testTotal = 0;

   for (let [streamId, testMeta] of status) {

      let streamFailed = 0;
      let streamSkipped = 0;
      let streamWarnings = 0;
      let streamSuccess = 0;
      let streamTotal = 0;

      for (let [, testStatus] of testMeta) {

         const ref = testStatus.refs[0];

         if (!ref) {
            continue;
         }

         if (!anyFailed && (testStatus.error || testStatus.skip)) {
            anyFailed = true;
         }

         if (!anyWarnings && ref.suiteWarningCount) {
            anyWarnings = true;
         }

         if (ref.suiteErrorCount) {
            streamFailed += ref.suiteErrorCount;
            streamTotal += ref.suiteErrorCount;
         }
         if (ref.suiteWarningCount) {
            streamWarnings += ref.suiteWarningCount;
            streamTotal += ref.suiteWarningCount;
         }
         if (ref.suiteSkipCount) {
            streamSkipped += ref.suiteSkipCount;
            streamTotal += ref.suiteSkipCount;
         }
         if (ref.suiteSuccessCount) {
            streamSuccess += ref.suiteSuccessCount;
            streamTotal += ref.suiteSuccessCount;
         }
      }

      testFailed += streamFailed;
      testWarnings += streamWarnings;
      testSkipped += streamSkipped;
      testSuccess += streamSuccess;


      streamTotal = streamTotal || 1;
      testTotal += streamTotal;

      const failedFactor = streamFailed / streamTotal
      const warningFactor = streamWarnings / streamTotal;
      const skipFactor = streamSkipped / streamTotal;
      const successFactor = streamSuccess / streamTotal;

      failCount.set(streamId, streamFailed);
      warningCount.set(streamId, streamWarnings);
      skipCount.set(streamId, streamSkipped);
      successCount.set(streamId, streamSuccess);

      failFactors.set(streamId, failedFactor);
      warningFactors.set(streamId, warningFactor);
      skipFactors.set(streamId, skipFactor);
      successFactors.set(streamId, successFactor);
   }

   let streamIds = Array.from(status.keys()).sort((a, b) => {
      const fA = failFactors.get(a)!;
      const fB = failFactors.get(b)!;
      if (fA !== fB) {
         return fB - fA;
      }
      return a.localeCompare(b);

   });

   streamIds.forEach((streamId) => {

      const streamName = projectStore.streamById(streamId)?.fullname ?? streamId;

      const failedFactor = failFactors.get(streamId)!
      const warningFactor = warningFactors.get(streamId)!
      const skipFactor = skipFactors.get(streamId)!
      const successFactor = successFactors.get(streamId)!

      const stack: StatusBarStack[] = [
         {
            value: successFactor * 100,
            titleValue: successCount.get(streamId) ?? 0,
            title: "Passed",
            color: 'transparent'
         },
         {
            value: failedFactor * 100,
            titleValue: failCount.get(streamId) ?? 0,
            title: "Failure",
            color: statusColors.get(StatusColor.Failure)!,
            stripes: true
         },
         {
            value: warningFactor * 100,
            titleValue: warningCount.get(streamId) ?? 0,
            title: "Warning",
            color: statusColors.get(StatusColor.Warnings)!,
            stripes: true
         },
         {
            value: skipFactor * 100,
            titleValue: skipCount.get(streamId) ?? 0,
            title: "Skipped",
            color: statusColors.get(StatusColor.Skipped)!,
            stripes: true
         }
      ]

      const metaElements: JSX.Element[] = [];

      if (streamExpanded.get(streamId)) {

         const testMeta = status.get(streamId);

         if (testMeta && testMeta.size) {

            let metaIds = Array.from(testMeta?.keys());

            const commonMeta = handler.getCommonMeta(metaIds);

            metaIds = metaIds.sort((a, b) => {

               /*
               const sA = testMeta.get(a)!;
               const sB = testMeta.get(b)!;

               const lastA = sA.refs[0];
               const lastB = sB.refs[0];

               const errorA = lastA.suiteErrorCount ? 1 : 0;
               const errorB = lastB.suiteErrorCount ? 1 : 0;

               const warnA = lastA.suiteWarningCount ? 1 : 0;
               const warnB = lastB.suiteWarningCount ? 1 : 0;

               const skipA = lastA.suiteSkipCount ? 1 : 0;
               const skipB = lastB.suiteSkipCount ? 1 : 0;

               if (errorA !== errorB) {
                  return errorB - errorA;
               }

               if (warnA !== warnB) {
                  return warnB - warnA;
               }

               if (skipA !== skipB) {
                  return skipB - skipA;
               }
               */

               return handler.metaNames.get(a)!.localeCompare(handler.metaNames.get(b)!);
            });

            for (let metaId of metaIds) {

               const metaStatus = testMeta.get(metaId)!;

               const last = metaStatus.refs[0];

               // todo move this to handler, duped in a few places
               const meta = handler.metaData.get(metaId)!
               const elements: string[] = [];

               elements.push(meta.platforms.join(" - "));

               if (!commonMeta.commonConfigs) {
                  elements.push(meta.configurations.join(" - "));
               }

               if (!commonMeta.commonTargets) {
                  elements.push(meta.buildTargets.join(" - "));
               }

               if (!commonMeta.commonRHI) {
                  elements.push(meta.rhi === "default" ? "Default" : meta.rhi.toUpperCase());
               }

               if (!commonMeta.commonVariation) {
                  elements.push(meta.variation === "default" ? "Default" : meta.variation.toUpperCase());
               }

               const metaName = `${elements.join(" / ")}`;

               let dateString = "";
               
               let date = handler.idDates.get(last.id);
               if (!date) {
                  date = handler.changeDates.get(last.buildChangeList);
               }               

               if (date) {
                  dateString = getShortNiceTime(date, false, false)
               }

               const ec = last.suiteErrorCount ?? 0;
               const wc = last.suiteWarningCount ?? 0;
               const skc = last.suiteSkipCount ?? 0;
               const suc = last.suiteSuccessCount ?? 0;
               const st = ec + wc + skc + suc;

               const metaStack: StatusBarStack[] = [
                  {
                     value: (suc / st) * 100,
                     titleValue: suc ?? 0,
                     title: "Passed",
                     color: 'transparent'
                  },
                  {
                     value: (ec / st) * 100,
                     titleValue: ec ?? 0,
                     title: "Failure",
                     color: statusColors.get(StatusColor.Failure)!,
                     stripes: true
                  },
                  {
                     value: (wc / st) * 100,
                     titleValue: wc ?? 0,
                     title: "Warning",
                     color: statusColors.get(StatusColor.Warnings)!,
                     stripes: true
                  },
                  {
                     value: (skc / st) * 100,
                     titleValue: skc ?? 0,
                     title: "Skipped",
                     color: statusColors.get(StatusColor.Skipped)!,
                     stripes: true
                  }
               ]



               metaElements.push(
                  <Stack className="horde-no-darktheme">
                     <Stack className={styles.metaitem} verticalAlign="center" style={{ cursor: "pointer", backgroundColor: metaElements.length % 2 ? colorA : colorB, padding: 8 }} onClick={(ev) => {
                        ev.stopPropagation();
                        handler.setSuiteRef(last.id);

                     }}>
                        <Stack horizontal>
                           <Stack style={{ width: 124 }}>
                              <Text variant="xSmall">{metaName}</Text>
                           </Stack>
                           <Stack style={{ width: 90 }}>
                              <Text variant="xSmall">CL {last.buildChangeList}</Text>
                           </Stack>
                           <Stack style={{ width: 60 }} horizontalAlign="end">
                              <Text variant="xSmall">{dateString}</Text>
                           </Stack>
                        </Stack>
                        <Stack style={{ paddingLeft: 0 }}>
                           {StatusBar(metaStack, 280, 8, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
                        </Stack>
                     </Stack>
                  </Stack>
               )
            }
         }
      }

      streamElements.push(<Stack style={{ paddingTop: streamElements.length === 0 ? 8 : 0 }} tokens={{ childrenGap: 2 }} onClick={(ev) => {
         ev.stopPropagation();
         const sexpanded = !!streamExpanded.get(streamId);
         streamExpanded.set(streamId, !sexpanded);
         setStreamExpanded(new Map(streamExpanded));
      }}>
         <Stack horizontalAlign="start" style={{ paddingRight: 1 }}>
            <Stack horizontal verticalAlign="center">
               <Stack style={{ paddingRight: 3 }}>
                  <FontIcon style={{ fontSize: 13 }} iconName={streamExpanded.get(streamId) ? "ChevronDown" : "ChevronRight"} />
               </Stack>
               <Stack>
                  <Text variant="small" style={{ fontWeight: 600 }}>{streamName}</Text>
               </Stack>
            </Stack>
         </Stack>
         {StatusBar(stack, 300, 10, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
         {!!streamExpanded.get(streamId) && <Stack style={{ border: `1px solid ${dashboard.darktheme ? "#3F4447" : "#CDCBC9"}` }}>{metaElements}</Stack>}
      </Stack>)
   });

   testTotal = testTotal || 1;

   const testFailedFactor = testFailed / testTotal;
   const testSkippedFactor = testSkipped / testTotal;
   const testWarningFactor = testWarnings / testTotal;
   const testSuccessFactor = testSuccess / testTotal;

   const testStack: StatusBarStack[] = [
      {
         value: testSuccessFactor * 100,
         titleValue: testSuccess,
         title: "Passed",
         color: 'transparent'
      },
      {
         value: testFailedFactor * 100,
         titleValue: testFailed,
         title: "Failed",
         color: statusColors.get(StatusColor.Failure)!,
         stripes: true
      },
      {
         value: testWarningFactor * 100,
         titleValue: testWarnings,
         title: "Warnings",
         color: statusColors.get(StatusColor.Warnings)!,
         stripes: true
      },
      {
         value: testSkippedFactor * 100,
         titleValue: testSkipped,
         title: "Skipped",
         color: statusColors.get(StatusColor.Skipped)!,
         stripes: true
      }
   ]

   return <Stack className={hordeClasses.raised} style={{ cursor: "pointer", height: "fit-content", backgroundColor: dashboard.darktheme ? "#242729" : modeColors.background, padding: 12, width: 332 }} onClick={() => setExpanded(!expanded)} >
      {historyShow && <SuiteSummaryModal suite={suite} handler={handler} onDismiss={() => setHistoryShown(false)} />}
      <Stack horizontal verticalAlign="center">
         <Stack className="horde-no-darktheme" style={{ paddingTop: 1, paddingRight: 4 }}>
            <FontIcon style={{ fontSize: 13 }} iconName={expanded ? "ChevronDown" : "ChevronRight"} />
         </Stack>
         <Stack>
            <Text style={{ fontWeight: 600 }}>{suite.name}</Text>
         </Stack>
         <Stack grow />

         <Stack horizontalAlign="end">
            <DefaultButton style={{ fontSize: 10, padding: 0, height: 18 }} onClick={(ev) => {
               ev.stopPropagation();
               setHistoryShown(true);
            }}>History</DefaultButton>
         </Stack>
      </Stack>
      <Stack style={{ paddingTop: 4 }}>
         {StatusBar(testStack, 304, 10, statusColors.get(StatusColor.Success)!, { margin: '3px !important' })}
      </Stack>
      {!!expanded && <Stack style={{ paddingLeft: 4 }} tokens={{ childrenGap: 4 }}>
         {streamElements}
      </Stack>}
   </Stack>
}

const SuiteSummary: React.FC<{ handler: TestDataHandler }> = ({ handler }) => {

   const { hordeClasses } = getHordeStyling();

   let csuites = handler.getStatusSuites();

   if (!csuites.length) {
      return null;
   }

   const failing: GetTestSuiteResponse[] = [];
   const passing: GetTestSuiteResponse[] = [];

   csuites.forEach(suite => {

      // sort by failed
      const status = handler.getStatus(suite.id);


      if (!status)
         return;

      let anyFailed = false;

      for (let [, testMeta] of status) {

         for (let [, metaStatus] of testMeta) {
            if (metaStatus.error) {
               anyFailed = true;
            }
            else if (metaStatus.skip) {
               anyFailed = true;
            }
         }
      }

      if (anyFailed) {
         failing.push(suite);
      } else {
         passing.push(suite);
      }
   })

   const suites = [...failing, ...passing].map(t => {

      return <SuiteSummaryButton suite={t} handler={handler} />

   })

   return <Stack className={hordeClasses.raised}>
      <Stack style={{ paddingBottom: 12 }}>
         <Stack>
            <Text style={{ fontFamily: "Horde Open Sans Semibold" }} variant='medium'>Latest Suite Results</Text>
         </Stack>
      </Stack>
      <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 4 }}>
         <Stack wrap horizontal style={{ width: "100%" }} tokens={{ childrenGap: 12 }}>
            {suites}
         </Stack>
      </Stack>
   </Stack>
}


export const AutomationViewSummary: React.FC<{ handler: TestDataHandler }> = observer(({ handler }) => {

   // subscribe
   if (handler.updated) { }

   return <Stack tokens={{ childrenGap: 24 }}>
      <TestSummary handler={handler} />
      <SuiteSummary handler={handler} />
   </Stack>


});