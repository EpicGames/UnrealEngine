// Copyright Epic Games, Inc. All Rights Reserved.

import { divProperties, mergeStyleSets, MessageBar, MessageBarType, Separator, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useState } from 'react';
import { Link } from 'react-router-dom';
import dashboard from '../backend/Dashboard';
import { Markdown } from "../base/components/Markdown";
import notices from '../backend/Notices';
import { getHordeTheme } from '../styles/theme';
import { getHordeStyling } from '../styles/Styles';
import { ErrorBar } from './error/ErrorBar';

export type BreadcrumbItem = {
   text: string;
   link?: string;
}

let _classes: any;

const getStyles = () => {

   const { modeColors } = getHordeStyling();

   const classes = _classes ?? mergeStyleSets({
      crumb: {
         marginLeft: '0px !important',
         marginRight: '0px !important',
         fontFamily: "Horde Open Sans Light",
         flexShrink: 1,
         color: modeColors.text,
         selectors: {
            ':active': {
               textDecoration: 'none'
            },
            ':hover': {
               textDecoration: 'none'
            }
         }
      }
   });

   _classes = classes;

   return classes;
   
}


export const Breadcrumbs: React.FC<{ items: BreadcrumbItem[], title?: string, suppressHome?: boolean, spinner?: boolean }> = observer((({ items: itemsIn, title, suppressHome, spinner }) => {

   const [, setHideAlert] = useState(false);

   const hordeTheme = getHordeTheme();
   const classes = getStyles();

   if (notices.updated) { }

   // hack fix to stop the entire stack item acting as a link
   function onLinkClick(ev: any, toLink: string | undefined) {
      if (!toLink || toLink === "") {
         ev.preventDefault();
         return false;
      }
   }

   let items: BreadcrumbItem[] = [];
   if (itemsIn?.length) {
      items = [...itemsIn];
   }

   const last = items.pop();

   if (!last) {
      return null;
   }

   if (!suppressHome) {
      if (last.text !== "Home") {
         items.unshift({ text: "Home", link: "/index" });
      }
   }

   document.title = title ?? `Horde - ${last.text}`;

   const topElements = items.map(item => {
      return <Link key={items.indexOf(item)} className={classes.crumb} to={item.link ?? ""} onClick={(ev: any) => { return onLinkClick(ev, item.link); }} style={{ cursor: item.link ? "" : "default" }}>
         <Text variant="medium" className={classes.crumb}>{item.text}</Text>
         <Text variant="medium" styles={{ root: { marginLeft: 7, marginRight: 7 } }} className={classes.crumb}>{"\u203A"}</Text>
      </Link>;
   });
   
   let bottomFontSize = 28;

   if (topElements.length) {
      bottomFontSize = 24;
   }

   const bottomElement = <Text className={classes.crumb} styles={{ root: { fontSize: bottomFontSize } }}>{last.text}</Text>;

   const alert = notices.alertText;

   return (<Stack>
      <Separator styles={{ root: { fontSize: 0, padding: 0 } }} />
      <Stack>
         <Stack styles={{ root: { userInput: 'all' } }}>
            <Stack tokens={{ childrenGap: 4 }} styles={{ root: { height: 88, padding: 0, paddingLeft: 0, paddingBottom: 8, paddingTop: 8, backgroundColor: hordeTheme.horde.breadCrumbsBackground, userSelect: 'text' } }}>
               <Stack tokens={{ childrenGap: 0 }} disableShrink={true} styles={{ root: { margin: "auto", width: "1440px"} }}>
                  <Stack horizontal tokens={{ childrenGap: 8, padding: 0 }}>{topElements}</Stack>
                  {
                     <Stack horizontal tokens={{ childrenGap: 18 }}>
                        <Stack>{bottomElement}</Stack>
                        {!!spinner && <Spinner size={SpinnerSize.large} />}
                     </Stack>
                  }
               </Stack>
            </Stack>
         </Stack>
         {!alert && !!dashboard.preview && <Stack horizontalAlign="center" disableShrink={true} style={{ position: "absolute", width: "100%", pointerEvents: "none" }} >
            <Stack style={{ paddingTop: 22 }}>
               <Text variant="large" style={{ fontFamily: "Horde Open Sans Light", opacity: 0.33 }}>Horde Preview</Text>
            </Stack>
         </Stack>}
         {!alert && !!dashboard.development && <Stack horizontalAlign="center" disableShrink={true} style={{ position: "absolute", width: "100%", pointerEvents: "none" }} >
            <Stack style={{ paddingTop: 22 }}>
               <Text variant="large" style={{ fontFamily: "Horde Open Sans Light", opacity: 0.33 }}>Horde Development</Text>
            </Stack>
         </Stack>}

         {!!alert && dashboard.alertSquelch !== alert && <Stack horizontalAlign="center" disableShrink={true} style={{ position: "absolute", width: "100%", pointerEvents:"none" }} >
            <Stack horizontal>
               <Stack grow />
               <Stack style={{pointerEvents: "auto"}}>
                  <MessageBar onDismiss={() => { dashboard.alertSquelch = alert; setHideAlert(true) }}
                     messageBarType={MessageBarType.severeWarning} isMultiline={false} >
                     <Markdown styles={{ root: { fontSize: 12, fontFamily: "Horde Open Sans Bold" } }}>{alert}</Markdown>
                  </MessageBar>
               </Stack>
               <Stack grow />
            </Stack>
         </Stack>}

         {!alert &&
            <div style={{margin: "auto", position: "absolute", width: "fit-content", left: 0, right: 0}}>
               <ErrorBar/>
            </div>
         }
      </Stack>
   </Stack>
   );
}));

/*
*/