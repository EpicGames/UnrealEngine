// Copyright Epic Games, Inc. All Rights Reserved.

import { ContextualMenu, ContextualMenuItemType, IContextualMenuItem, Point, Stack, Text } from '@fluentui/react';
import React, { MutableRefObject, useState } from 'react';
import { GetBatchResponse, GetChangeSummaryResponse, GetJobStepRefResponse, GetThinUserInfoResponse, JobStepBatchError, JobStepBatchState, JobStepOutcome, JobStepState } from '../backend/Api';
import dashboard, { StatusColor } from "../backend/Dashboard";
import { projectStore } from '../backend/ProjectStore';
import { getPreflightDisplay } from '../base/utilities/commitUtils';

export type ChangeContextMenuTarget = {

   ref?: MutableRefObject<null>;
   point?: Point;

}

// partial JobResponse type, so we don't need full job data when presenting change cl's
export type JobParameters = {
   id?: string;
   streamId: string;
   change?: number;
   preflightChange?: number;
   preflightCommitId?: string;
   preflightDescription?: string;
   abortedByUserInfo?: GetThinUserInfoResponse;
   cancellationReason?: string;
   startedByUserInfo?: GetThinUserInfoResponse;
   batches?: GetBatchResponse[];
}

export const ChangeContextMenu: React.FC<{ target: ChangeContextMenuTarget, onDismiss?: () => void, job?: JobParameters, stepRef?: GetJobStepRefResponse, commit?: GetChangeSummaryResponse, rangeCL?: number, cl?: number }> = ({ target, onDismiss, job, stepRef, commit, rangeCL, cl }) => {

   let jobChange = commit?.number ?? cl;

   if (jobChange === undefined) {
      jobChange = stepRef?.change ?? job?.change;
   }

   // Need at least a change number to show the menu
   if (jobChange === undefined) {
      return null;
   }

   let change = jobChange.toString();
   const pfMenuDisplay = !stepRef ? getPreflightDisplay(job?.preflightCommitId, job?.preflightChange) : undefined;
   if (pfMenuDisplay) {
      change = `PF ${pfMenuDisplay}`;
   }

   const copyToClipboard = (value: string | undefined) => {

      if (!value) {
         return;
      }

      const el = document.createElement('textarea');
      el.value = value;
      document.body.appendChild(el);
      el.select();
      document.execCommand('copy');
      document.body.removeChild(el);
   }


   const menuItems: IContextualMenuItem[] = [];

   if (job?.preflightDescription) {

      menuItems.push({
         key: "commit", onRender: () => {
            return <Stack style={{ padding: 18, paddingTop: 18, paddingBottom: 18, maxWidth: 800 }} tokens={{ childrenGap: 12 }}>
               <Stack horizontal tokens={{ childrenGap: 12 }}>
                  <Text variant="small" style={{ fontFamily: "Horde Open Sans Bold" }}>Description:</Text>
                  <Text variant="small" style={{ whiteSpace: "pre-wrap" }} >{job.preflightDescription}</Text>
               </Stack>
            </Stack>
         }
      });

   } else if (commit) {

      menuItems.push({
         key: "commit", onRender: () => {
            return <Stack style={{ padding: 18, paddingTop: 18, paddingBottom: 18, maxWidth: 800 }} tokens={{ childrenGap: 12 }}>
               <Stack horizontal tokens={{ childrenGap: 36 }}>
                  <Text variant="small" style={{ fontFamily: "Horde Open Sans Bold" }}>Author:</Text>
                  <Text variant="small" >{commit.authorInfo?.name}</Text>
               </Stack>
               <Stack horizontal tokens={{ childrenGap: 35 }}>
                  <Text variant="small" style={{ fontFamily: "Horde Open Sans Bold" }}>Change:</Text>
                  <Text variant="small" >{commit.number}</Text>
               </Stack>

               <Stack horizontal tokens={{ childrenGap: 12 }}>
                  <Text variant="small" style={{ fontFamily: "Horde Open Sans Bold" }}>Description:</Text>
                  <Text variant="small" style={{ whiteSpace: "pre-wrap" }}>{commit.description}</Text>
               </Stack>
            </Stack>
         }
      });

   }
   
   // Should only show cancel text if not in an incremental view where "commit" exists
   if(!commit && job?.abortedByUserInfo && job?.cancellationReason) {
      menuItems.push({
         key: "cancelled",
         onRender: () => {
            return (
               <Stack style={{ padding: 18, paddingTop: job?.preflightDescription ? 0 : 18, paddingBottom: 18, maxWidth: 800 }} horizontal tokens={{ childrenGap: 12 }}>
                  <Text variant="small" style={{ fontFamily: "Horde Open Sans Bold" }}>Cancellation Reason:</Text>
                  <Text variant="small" >{job?.cancellationReason}</Text>
               </Stack>
            )
         }
      });
   }

   if(job?.cancellationReason || commit || job?.preflightDescription){
      menuItems.push({
         key: "commit_divider",
         itemType: ContextualMenuItemType.Divider
      });
   }

   menuItems.push({
      key: 'copy_to_clipboard',
      text: 'Copy CL to Clipboard',
      onClick: () => copyToClipboard(job?.preflightCommitId ?? change.replace("PF ", ""))
   });

   if (dashboard.swarmUrl) {

      menuItems.push({ key: 'open_in_swarm', text: 'Open CL in Swarm', onClick: (ev) => { window.open(`${dashboard.swarmUrl}/changes/${change.replace("PF ", "")}`) } })

      // Stream-dependent options (history, range) only available when we have stream info
      const stream = job?.streamId ? projectStore.streamById(job.streamId) : undefined;
      if (stream) {
         const project = projectStore.byId(stream.projectId);
         const name = project?.name === "Engine" ? "UE4" : project?.name;

         const historyUrl = `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${jobChange}#commits`;
         menuItems.push({ key: 'open_in_swarm_history', text: "Open CL History in Swarm", onClick: (ev) => { window.open(historyUrl) } })

         if (rangeCL !== undefined) {
            const lowCL = Math.min(jobChange, rangeCL);
            const highCL = Math.max(jobChange, rangeCL);
            const url = `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${lowCL},@${highCL}#commits`;
            const rangeText = `Open CL Range ${lowCL} - ${highCL}`;
            menuItems.push({ key: 'open_in_swarm_range', text: rangeText, onClick: (ev) => { window.open(url) } })
         }
      }
   }

   if (job) {

      menuItems.push({
         key: 'view_job',
         text: 'Open Job Details',
         href: window.location.protocol + "//" + window.location.hostname + `/job/${job.id}`
      });


      menuItems.push({
         key: 'copy_job_to_clipboard',
         text: 'Copy Job to Clipboard',
         onClick: () => copyToClipboard(window.location.protocol + "//" + window.location.hostname + `/job/${job.id}`)
      });
   }

   return (<ContextualMenu
      styles={{ root: { paddingBottom: 12, paddingRight: 24, paddingLeft: 8, paddingTop: 12 }, list: { selectors: { '.ms-ContextualMenu-itemText': { fontSize: "10px", paddingLeft: 8 } } } }}
      items={menuItems}
      hidden={false}
      target={target.ref ?? target.point}
      onItemClick={() => { if (onDismiss) onDismiss() }}
      onDismiss={() => { if (onDismiss) onDismiss() }}
   />);

}

function getJobSummary(job: JobParameters): { text: string, color: string } {

   const colors = dashboard.getStatusColors();
   let color = colors.get(StatusColor.Skipped)!;

   const batches = job.batches;

   const cancelled = job.abortedByUserInfo?.id ? "Canceled" : "";

   const numBatches = batches?.length;

   if (cancelled || !batches || !numBatches) {
      return { text: cancelled, color: colors.get(StatusColor.Skipped)! };
   }

   let batchErrors = 0;
   let batchFinished = 0;

   let stepsCanceled = 0;
   let stepsSkipped = 0;
   let stepsWarnings = 0;
   let stepsFailures = 0;

   let stepsComplete = true;

   batches.forEach(b => {

      if (!!b.finishTime || b.state === JobStepBatchState.Complete) {
         batchFinished++;
      }
      if (b.error && b.error !== JobStepBatchError.None) {
         batchErrors++;
      }

      b.steps?.forEach(s => {

         if (s.state === JobStepState.Waiting || s.state === JobStepState.Ready || s.state === JobStepState.Running) {
            stepsComplete = false;
         }

         if (s.state === JobStepState.Aborted) {
            stepsCanceled++;
         }
         if (s.state === JobStepState.Skipped) {
            stepsSkipped++;
         }

         if (s.outcome === JobStepOutcome.Warnings) {
            stepsWarnings++;
         }

         if (s.outcome === JobStepOutcome.Failure) {
            stepsFailures++;
         }
      })
   });

   let state = (stepsComplete || batchFinished === batches.length) ? "Completed" : "Running";
   color = state === "Running" ? colors.get(StatusColor.Running)! : colors.get(StatusColor.Success)!

   if (batchErrors || stepsFailures) {
      color = colors.get(StatusColor.Failure)!
   } else if (stepsSkipped || stepsCanceled) {
      state += " with skipped steps";
      color = colors.get(StatusColor.Skipped)!
   } else if (stepsWarnings) {
      color = colors.get(StatusColor.Warnings)!
   }

   return { text: state, color: color };

}


export const ChangeButton: React.FC<{ job?: JobParameters, stepRef?: GetJobStepRefResponse, commit?: GetChangeSummaryResponse, hideAborted?: boolean, rangeCL?: number, pinned?: boolean, prefix?: string, buttonColor?: string }> = ({ job, stepRef, commit, hideAborted, rangeCL, pinned, prefix, buttonColor }) => {

   const [menuShown, setMenuShown] = useState(false);

   const spanRef = React.useRef(null);

   if (!job) {
      return null;
   }

   let showStatus = pinned || (!hideAborted && job.abortedByUserInfo?.id);

   const { text, color } = getJobSummary(job);

   let change = job?.change?.toString() ?? "Latest";
   const pfDisplay = getPreflightDisplay(job?.preflightCommitId, job?.preflightChange);
   if (pfDisplay) {
      change = `PF ${pfDisplay}`;
   }

   if (stepRef) {
      change = stepRef.change.toString();
   }

   if (prefix) {
      change = `${prefix} ${change}`;
   }

   const defaultBackgroundColor = job.startedByUserInfo ? "#0288ee" : "#035ca1";

   return (<Stack verticalAlign="center" verticalFill={true} horizontalAlign="start"> <div style={{ paddingBottom: "1px" }}>
      <Stack tokens={{ childrenGap: 4 }}>
         <span 
            ref={spanRef} 
            style={{ padding: "2px 6px 2px 6px", height: "15px", cursor: "pointer", color: "#FFFFFF", backgroundColor: buttonColor ? `${buttonColor}` : defaultBackgroundColor }} 
            className={job.startedByUserInfo ? "cl-callout-button-user" : "cl-callout-button"} 
            onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); setMenuShown(!menuShown) }}
         >
            {change}
         </span>
         {(!!showStatus) && 
            <span 
               ref={spanRef} 
               style={{ padding: "2px 6px 2px 6px", height: "16px", cursor: "pointer", userSelect: "none", fontFamily: "Horde Open Sans SemiBold", fontSize: "10px", backgroundColor: color, color: "rgb(255, 255, 255)" }} 
               className={job.startedByUserInfo ? "cl-callout-button-user" : "cl-callout-button"} 
               onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); setMenuShown(!menuShown) }}
            >
               {text}
            </span>
         }
      </Stack>
      {menuShown && <ChangeContextMenu target={{ ref: spanRef }} job={job} commit={commit} stepRef={stepRef} rangeCL={rangeCL} onDismiss={() => setMenuShown(false)} />}
   </div></Stack>);
};

