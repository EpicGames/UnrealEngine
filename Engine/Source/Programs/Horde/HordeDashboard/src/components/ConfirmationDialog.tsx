// Copyright Epic Games, Inc. All Rights Reserved.

import { Dialog, PrimaryButton, DialogFooter, DefaultButton, DialogType, TextField, Checkbox, ICheckbox } from '@fluentui/react';
import React, { useState } from 'react';

export const ConfirmationDialog: React.FC<{ isOpen: boolean; title: string; cancelText: string; confirmText: string; subText?: string; isTextBoxSpawned?: boolean; textBoxLabel?: string; isTextRequired?: boolean, checkBoxText?: string; onConfirm: (...args: any[]) => any; onCancel: (...args: any[]) => any }> = (({ isOpen, title, cancelText, confirmText, subText, isTextBoxSpawned, textBoxLabel, isTextRequired, checkBoxText, onConfirm, onCancel }) => {
   const [textFieldText, setTextFieldText] = useState<string | undefined>(undefined);
   const checkboxRef = React.useRef<ICheckbox>(null);

   return (
      <Dialog
         hidden={!isOpen}
         onDismiss={onCancel}
         dialogContentProps={{
            type: DialogType.normal,
            title: title,
            subText: subText
         }}
         modalProps={{
            isBlocking: true,
            styles: { main: { maxWidth: 450, minHeight: 160 } },
         }}
      >
         {!!checkBoxText && <Checkbox componentRef={checkboxRef} label={checkBoxText} defaultChecked={false} />}
         {isTextBoxSpawned && <TextField label={textBoxLabel} value={textFieldText} onChange={(event, value) => setTextFieldText(value)}></TextField>}
         <DialogFooter>
            <PrimaryButton disabled={isTextBoxSpawned && isTextRequired && !textFieldText?.length} onClick={() => { onConfirm(textFieldText,checkboxRef?.current?.checked ); setTextFieldText(undefined); }} text={confirmText} />
            <DefaultButton onClick={() => { onCancel(textFieldText); setTextFieldText(undefined); }} text={cancelText} />
         </DialogFooter>
      </Dialog>
   );
});