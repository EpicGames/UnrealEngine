// Copyright Epic Games, Inc. All Rights Reserved.

import { ColorPicker, ComboBox, DefaultButton, DialogFooter, IColorPickerStyles, IComboBoxOption, Label, MessageBar, MessageBarType, Modal, Pivot, PivotItem, PrimaryButton, Spinner, SpinnerSize, Stack, Text, TextField, Toggle } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useState } from 'react';
import backend from '../backend';
import { AuthMethod, DashboardPreference, DownloadFormat, UpdateCurrentAccountRequest } from '../backend/Api';
import dashboard, { StatusColor, WebBrowser } from '../backend/Dashboard';
import { useWindowSize } from '../base/utilities/hooks';
import { getHordeStyling } from '../styles/Styles';
import { Breadcrumbs } from './Breadcrumbs';
import { TopNav } from './TopNav';
import { UserAccountPanel } from './accounts/UserAccountPanel';

const colorBlind1 = new Map<StatusColor, string>([
   [StatusColor.Success, "#37A862"],
   [StatusColor.Warnings, "#EECC17"],
   [StatusColor.Failure, "#FD827D"],
   [StatusColor.Running, "#7A6FF1"],
   [StatusColor.User1, "#8e4747"],
   [StatusColor.User2, "#785EF0"],
   [StatusColor.User3, "#FE6100"]

]);

const colorBlind2 = new Map<StatusColor, string>([
   [StatusColor.Success, "#208FA3"],
   [StatusColor.Warnings, "#FFB000"],
   [StatusColor.Failure, "#E8384F"],
   [StatusColor.Running, "#AA71FF"],
   [StatusColor.User1, "#8e4747"],
   [StatusColor.User2, "#785EF0"],
   [StatusColor.User3, "#FE6100"],            
]);

const statusText = new Map<StatusColor, string>([
   [StatusColor.Success, "Success"],
   [StatusColor.Warnings, "Warnings"],
   [StatusColor.Failure, "Failure"],
   [StatusColor.Waiting, "Waiting"],
   [StatusColor.Ready, "Ready"],
   [StatusColor.Skipped, "Skipped"],
   [StatusColor.Aborted, "Canceled"],
   [StatusColor.Running, "Running"],
   [StatusColor.User1, "User1"],
   [StatusColor.User2, "User2"],
   [StatusColor.User3, "User3"]
]);

const orderedStatus = [
   StatusColor.Success,
   StatusColor.Warnings,
   StatusColor.Failure,
   StatusColor.Running,
   StatusColor.User1,
   StatusColor.User2,
   StatusColor.User3
];

const statusPrefs = new Map<StatusColor, DashboardPreference>([
   [StatusColor.Success, DashboardPreference.ColorSuccess],
   [StatusColor.Warnings, DashboardPreference.ColorWarning],
   [StatusColor.Failure, DashboardPreference.ColorError],
   [StatusColor.Running, DashboardPreference.ColorRunning],
   [StatusColor.User1, DashboardPreference.ColorUser1],
   [StatusColor.User2, DashboardPreference.ColorUser2],
   [StatusColor.User3, DashboardPreference.ColorUser3]
]);


const ColorPreferenceDialog: React.FC<{ shown: boolean, statusIn: StatusColor, onClose: () => void }> = ({ shown, statusIn, onClose }) => {

   let [state, setState] = useState<{ status: StatusColor, colors: Map<StatusColor, string> }>({ status: statusIn, colors: new Map<StatusColor, string>() });

   const { hordeClasses } = getHordeStyling();

   if (!shown) {
      return null;
   }

   const defaultStatusColors = dashboard.getDefaultStatusColors();

   let anyMissing = false;

   statusPrefs.forEach((value, key) => {

      if (!state.colors.has(key)) {

         anyMissing = true;
      }

   });

   if (anyMissing) {

      statusPrefs.forEach((value, key) => {

         if (state.colors.has(key)) {
            return;
         }

         const pref = dashboard.getPreference(value);
         const color = pref ? pref : defaultStatusColors.get(key)!;

         state.colors.set(key, color);

      })

      setState({ status: state.status, colors: state.colors });
      return null;
   }

   const status = state.status;
   let color = state.colors.get(status)!;

   const setColor = (c: string) => {

      state.colors.set(status, c);
      setState({ status: state.status, colors: state.colors });
   }

   const setStatus = (status: StatusColor) => {

      setState({ status: status, colors: state.colors });
   }

   // tol: https://personal.sron.nl/~pault/
   const tolColors = [
      defaultStatusColors.get(status)!,
      "#332288",
      "#117733",
      "#44AA99",
      "#88CCEE",
      "#DDCC77",
      "#CC6677",
      "#AA4499",
      "#882255",
      "#648FFF",
      "#785EF0",
      "#DC267F",
      "#FE6100",
      "#FFB000"
   ];

   const suggestions = tolColors.map((c, index) => {
      return <DefaultButton key={`color_${index}`} style={{ backgroundColor: c, width: 80 }} text={statusText.get(status)} onClick={() => { setColor(c) }} />
   })

   const colorPicker = () => {
      const colorPickerStyles: Partial<IColorPickerStyles> = {
         panel: { padding: 12 },
         root: {
            maxWidth: 352,
            minWidth: 352,
         },
         colorRectangle: { height: 268 },
      };


      return <Stack className="horde-no-darktheme">
         <ColorPicker
            color={color}
            onChange={(ev, color) => { setColor(`#${color.hex}`) }}
            alphaType="none"
            showPreview={true}
            styles={colorPickerStyles}
         />
      </Stack >;

   }

   const BadgeColors: React.FC = () => {

      const buttons = orderedStatus.map(status => {

         return <div className="horde-no-darktheme">
            <Stack className={hordeClasses.badgeNoIcon} key={`badge_${status}`}> <DefaultButton style={{ backgroundColor: state.colors.get(status) ?? defaultStatusColors.get(status) }} text={statusText.get(status)} onClick={() => { setStatus(status) }} />
            </Stack>
         </div>
      });

      return <Stack horizontal tokens={{ childrenGap: 12 }}>{buttons}</Stack>;

   }


   return <Modal isOpen={shown} isBlocking={true} styles={{ main: { padding: 8, width: 570, } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
      <Stack style={{ paddingLeft: 12, paddingRight: 12 }}>
         <Stack style={{ padding: 12 }}>
            <Stack style={{ paddingBottom: 24 }}>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{`Set Status Colors`}</Text>
            </Stack>

            <Stack tokens={{ childrenGap: 12 }}>
               <Stack horizontalAlign="center">
                  <BadgeColors />
               </Stack>
               <Stack>
                  <Stack style={{ paddingBottom: 12 }}>
                     <Text styles={{ root: { fontSize: "15px", fontFamily: "Horde Open Sans SemiBold" } }}>{`Profiles`}</Text>
                  </Stack>
                  <Stack style={{ paddingLeft: 12 }} horizontalAlign="center">
                     <Stack horizontal tokens={{ childrenGap: 12 }}>
                        <DefaultButton style={{ height: 24 }} text="Default" onClick={() => {
                           setState({ status: state.status, colors: new Map(defaultStatusColors) });
                        }} />
                        <DefaultButton style={{ height: 24 }} text="Color Blind 1" onClick={() => {
                           setState({ status: state.status, colors: new Map(colorBlind1) });
                        }} />
                        <DefaultButton style={{ height: 24 }} text="Color Blind 2" onClick={() => {
                           setState({ status: state.status, colors: new Map(colorBlind2) });
                        }} />
                     </Stack>
                  </Stack>

               </Stack>
               <Stack>
                  <Stack style={{ paddingTop: 8 }}>
                     <Text styles={{ root: { fontSize: "15px", fontFamily: "Horde Open Sans SemiBold" } }}>{`Custom Color`}</Text>
                  </Stack>
                  <Stack horizontal tokens={{ childrenGap: 48 }}>
                     {colorPicker()}
                     <Stack className="horde-no-darktheme" style={{ paddingTop: 8 }}>
                        <Stack style={{ paddingTop: 2 }} className={hordeClasses.badgeNoIcon} tokens={{ childrenGap: 6 }}>
                           {suggestions}
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
         <Stack style={{ paddingBottom: 18, marginRight: 12 }} >
            <DialogFooter>
               <PrimaryButton text="Set" onClick={() => {
                  state.colors.forEach((value, key) => {
                     const pref = statusPrefs.get(key)!;
                     dashboard.setStatusColor(pref, value);
                  })
                  onClose();
               }} />
               <DefaultButton text="Cancel" onClick={() => onClose()} />
            </DialogFooter>

         </Stack>
      </Stack>
   </Modal>

}

const ChangePasswordModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const [submitting, setSubmitting] = useState(false);
   const { hordeClasses } = getHordeStyling();
   const [error, setError] = useState("");
   const [secrets, setSecrets] = useState<{ currentPassword?: string, newPassword?: string, confirmPassword?: string }>({});


   const onValidate = () => {


      if (!secrets.newPassword) {
         return "Please enter a password";
      }

      if (secrets.newPassword !== secrets.confirmPassword) {
         return "New and confirm passwords don't match";
      }

      return undefined;
   }

   const onSave = async () => {

      const nerror = onValidate();

      if (nerror) {
         setError(nerror);
         return;
      }

      setError("");

      try {

         setSubmitting(true);
         const uaccount: UpdateCurrentAccountRequest = { newPassword: secrets.newPassword, oldPassword: secrets.currentPassword ?? "" };
         await backend.updateCurrentAccount(uaccount);
         setSubmitting(false);
         onClose();

      } catch (reason) {
         console.error(reason);
         setError("Error updating password, please make sure the current password matches");
         setSubmitting(false);
      }

   }

   if (submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="mediumPlus">Please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   }

   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 512, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 16 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>Change Password</Text>
         </Stack>
         {!!error && <Stack>
            <MessageBar key={`validation_error`} messageBarType={MessageBarType.error} isMultiline={false}>{error}</MessageBar>
         </Stack>}

         <Stack style={{ padding: 8 }}>
            <TextField label={"Current Password"} autoComplete="off" defaultValue={secrets.currentPassword} spellCheck={false} type="password" canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, currentPassword: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label={"New Password"} autoComplete="new-password" spellCheck={false} type="password" defaultValue={secrets.newPassword} canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, newPassword: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label={"Confirm Password"} autoComplete="new-password" spellCheck={false} type="password" defaultValue={secrets.confirmPassword} canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, confirmPassword: value ?? "" }) }} />
         </Stack>

         <Stack horizontal style={{ paddingTop: 64 }}>
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 28 }}>
               <PrimaryButton onClick={() => onSave()} text="Save" />
               <DefaultButton onClick={() => onClose()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>
}


const GeneralPanel: React.FC = observer(() => {

   const [pivotKey, setPivotKey] = useState("Settings");
   let [colorState, setColorState] = useState<{ status?: StatusColor }>({});
   let [changePassword, setChangePassword] = useState(false);
   const { hordeClasses, modeColors } = getHordeStyling();

   const defaultStatusColors = dashboard.getDefaultStatusColors();

   // subscribe
   if (dashboard.updated) { }

   const BadgeColors: React.FC = () => {

      const buttons = orderedStatus.map(status => {

         const dashboardPref = statusPrefs.get(status)!;
         const pref = dashboard.getPreference(dashboardPref);
         const color = pref ? pref : defaultStatusColors.get(status)!;

         return <Stack className={hordeClasses.badgeNoIcon} key={`badge_${status}`}> <DefaultButton style={{ backgroundColor: color }} text={statusText.get(status)} onClick={() => { setColorState({ status: status }) }} />
         </Stack>
      });

      return <Stack horizontal tokens={{ childrenGap: 12 }}>{buttons}</Stack>;

   }

   let name = dashboard.username;

   const downloadOptions: IComboBoxOption[] = [
      { key: `key_download_option_zip`, text: "Zip", data: DownloadFormat.Zip },
      { key: `key_download_option_ugs`, text: "Unreal Game Sync", data: DownloadFormat.UGS },
      { key: `key_download_option_toolbox`, text: "Unreal Toolbox", data: DownloadFormat.UTOOLBOX }
   ];

   let defaultDownloadKey = "key_download_option_zip";
   if (dashboard.downloadFormat === DownloadFormat.UGS) {
      defaultDownloadKey = "key_download_option_ugs";
   } else if (dashboard.downloadFormat === DownloadFormat.UTOOLBOX) {
      defaultDownloadKey = "key_download_option_toolbox";
   }

   const pivotItems = ["Settings", "Claims", "Entitlements"].map(tab => {
      return <PivotItem headerText={tab} itemKey={tab} key={tab} style={{ color: modeColors.text }} />;
   });

   return (<Stack>
      {colorState.status !== undefined && <ColorPreferenceDialog shown={true} statusIn={colorState.status} onClose={() => setColorState({})} />}
      {changePassword && <ChangePasswordModal onClose={() => setChangePassword(false)} />}
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12, width: "100%" } }} >
         <Stack tokens={{ childrenGap: 12 }} style={{ height: 'calc(100vh - 200px)' }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{name}</Text>
            <Stack style={{ paddingBottom: 12 }}>
               <Pivot className={hordeClasses.pivot}
                  overflowBehavior='menu'
                  selectedKey={pivotKey}
                  linkSize="normal"
                  linkFormat="links"
                  onLinkClick={(item) => {
                     setPivotKey(item?.props.itemKey ?? "Settings")
                  }}>
                  {pivotItems}
               </Pivot>
            </Stack>
            {pivotKey === "Claims" && <UserAccountPanel mode={pivotKey} />}
            {pivotKey === "Entitlements" && <UserAccountPanel mode={pivotKey} />}
            {pivotKey === "Settings" && <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 12 }}>
               <Stack style={{ paddingLeft: 8 }}>
                  <Toggle label="Dark Mode" inlineLabel={true} defaultChecked={dashboard.darktheme} onChange={(ev, checked) => {
                     dashboard.setDarkTheme(checked ? true : false);
                     setColorState({ ...colorState });
                  }} />
                  <Toggle label="Display UTC Times" inlineLabel={true} defaultChecked={dashboard.displayUTC} onChange={(ev, checked) => {
                     dashboard.setDisplayUTC(checked ? true : false);
                  }} />
                  <Toggle label="24 Hour Clock" inlineLabel={true} defaultChecked={dashboard.display24HourClock} onChange={(ev, checked) => {
                     dashboard.setDisplay24HourClock(checked ? true : false);
                  }} />
                  <Toggle label="Show All Preflights" inlineLabel={true} defaultChecked={dashboard.showPreflights} onChange={(ev, checked) => {
                     dashboard.setShowPreflights(checked ? true : false);
                  }} />
                  <Toggle label="Always Tag Preflight Changelists" inlineLabel={true} defaultChecked={dashboard.alwaysTagPreflightCL} onChange={(ev, checked) => {
                     dashboard.alwaysTagPreflightCL = checked ? true : false;
                  }} />
                  <Toggle label="Prefer Compact Views" inlineLabel={true} defaultChecked={dashboard.compactViews} onChange={(ev, checked) => {
                     dashboard.setCompactViews(checked ? true : false);
                  }} />
                  <Toggle label="Left Align Log View" inlineLabel={true} defaultChecked={dashboard.leftAlignLog} onChange={(ev, checked) => {
                     dashboard.setLeftAlignLog(checked ? true : false);
                  }} />
               </Stack>
               <Stack style={{ paddingLeft: 8 }} tokens={{ childrenGap: 12 }}>
                  <ComboBox style={{ width: 218 }} label="Download Format" defaultSelectedKey={defaultDownloadKey} options={downloadOptions} autoComplete="off" onChange={(ev, option, index, value) => {
                     if (option) {
                        dashboard.setDownloadFormat(option?.data);
                     }                     
                   }} />
               </Stack>
               <Stack style={{ paddingLeft: 8 }} tokens={{ childrenGap: 12 }}>
                  <Label>Status Colors</Label>
                  <BadgeColors />
               </Stack>
               {dashboard.authMethod === AuthMethod.Horde && <Stack style={{ paddingTop: 24, paddingLeft: 8 }}>
                  <PrimaryButton style={{ width: 180 }} text="Change Password" onClick={() => setChangePassword(true)} />
               </Stack>}


               {false && <Stack style={{ paddingBottom: 4, paddingTop: 12 }}>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Experimental</Text>
               </Stack>}

               {false && <Stack style={{ paddingLeft: 12 }}>

                  <Toggle label="General Features" inlineLabel={true} defaultChecked={dashboard.experimentalFeatures} onChange={(ev, checked) => {
                     dashboard.experimentalFeatures = checked ? true : false;
                  }} />
                  {dashboard.browser !== WebBrowser.Chromium && <Toggle label="Data Caching (Page Reload Required)" inlineLabel={true} defaultChecked={dashboard.localCache} onChange={(ev, checked) => {
                     dashboard.setLocalCache(checked ? true : false);
                  }} />}
               </Stack>}


            </Stack>}
         </Stack>
      </Stack>
   </Stack>
   );
});

export const DashboardView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Preferences' }]} />
      <Stack horizontal>
         <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
            <Stack style={{ maxWidth: 1440, paddingTop: 16, marginLeft: 4 }}>
               <Stack horizontal className={hordeClasses.raised}>
                  <Stack style={{ width: "100%" }} tokens={{ childrenGap: 18 }}>
                     {dashboard.available && <GeneralPanel />}
                     {!dashboard.available && <Text>User settings unavailable, try logging out and back in</Text>}
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};