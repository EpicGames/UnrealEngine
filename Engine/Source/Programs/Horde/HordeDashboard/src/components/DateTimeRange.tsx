
// Copyright Epic Games, Inc. All Rights Reserved.
import { ComboBox, DatePicker, DefaultButton, defaultDatePickerStrings, IComboBoxOption, Modal, PrimaryButton, Stack, Text } from '@fluentui/react';
import moment from 'moment';
import React, { useState } from 'react';
import dashboard from '../backend/Dashboard';
import { displayTimeZone } from '../base/utilities/timeUtils';
import { getHordeStyling } from '../styles/Styles';

function getDefaultHour() {
   return dashboard.displayUTC ? moment.utc().hour() : moment().hour()
}

function formatHour(hour: number): string {

   const tz = moment().tz(displayTimeZone()).format("z");

   let suffix = " AM";

   if (!dashboard.display24HourClock) {

      if (hour >= 12) {
         suffix = " PM";
      }

      if (hour > 12) {
         hour -= 12;
      }
   }

   let hourStr = hour.toString();

   if (hour < 10) {
      hourStr = `0` + hour;
   }

   return dashboard.display24HourClock ? `${hourStr}:00 ${tz}` : `${hourStr}:00${suffix} ${tz}`

}



// Time Picker (the contributed Fluent one doesn't work very well)
const MyTimePicker: React.FC<{ defaultHour: number, onChange: (hour: number) => void }> = ({ defaultHour, onChange }) => {

   const [text, setText] = useState(formatHour(defaultHour))

   const options: IComboBoxOption[] = [];


   for (let i = 0; i < 24; i++) {

      const key = `time_picker_${i}`;

      const hour = formatHour(i);

      options.push({
         key: key,
         text: hour,
         data: i
      })
   }

   return <ComboBox options={options} text={text} allowFreeform={false} autoComplete="off" useComboBoxAsMenuWidth onChange={(ev, option, index, value) => {

      if (option) {
         setText(option.text);
         onChange(option.data as number);
      } else if (value) {

      }

   }} />
}

function getDefaultDate() {
   let date = moment.utc().toDate();
   date.setHours(0, 0, 0, 0);
   return date;
}

export const DateTimeRange: React.FC<{ onChange: (minDate: Date, maxDate: Date) => void, onDismiss: () => void }> = ({ onChange, onDismiss }) => {

   const [startDate, setStartDate] = useState(getDefaultDate());
   const [endDate, setEndDate] = useState(getDefaultDate());
   const [startHour, setStartHour] = useState(getDefaultHour());
   const [endHour, setEndHour] = useState(getDefaultHour() + 1);

   const { hordeClasses } = getHordeStyling();

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 24, width: 448, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
      <Stack tokens={{ childrenGap: 12 }} >
         <Stack grow verticalAlign="center">
            <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Select Range</Text>
         </Stack>

         <Stack tokens={{ childrenGap: 24 }} style={{ padding: 8 }}>
            <Stack horizontal tokens={{ childrenGap: 48 }}>
               <Stack tokens={{ childrenGap: 12 }}>
                  <Text variant="medium">Start Date</Text>
                  <DatePicker
                     strings={defaultDatePickerStrings}
                     onSelectDate={(date) => {
                        if (date) {
                           date.setHours(0, 0, 0, 0);
                           setStartDate(date)
                        }
                     }}
                     value={startDate}
                     showGoToToday
                     textField={{ style: { width: 166 } }}
                  />
               </Stack>
               <Stack tokens={{ childrenGap: 12 }}>
                  <Text variant="medium">End Date</Text>
                  <DatePicker
                     strings={defaultDatePickerStrings}
                     onSelectDate={(date) => {
                        if (date) {
                           date.setHours(0, 0, 0, 0);
                           setEndDate(date)
                        }
                     }}

                     value={endDate}
                     showGoToToday
                     textField={{ style: { width: 166 } }}
                  />
               </Stack>

            </Stack>
            <Stack horizontal tokens={{ childrenGap: 48 }}>
               <Stack tokens={{ childrenGap: 12 }}>
                  <Stack>
                     <Text variant="medium">Start Time</Text>
                  </Stack>
                  <MyTimePicker defaultHour={startHour} onChange={(hour) => { setStartHour(hour) }} />
               </Stack>

               <Stack tokens={{ childrenGap: 12 }}>
                  <Stack>
                     <Text variant="medium">End Time</Text>
                  </Stack>
                  <MyTimePicker defaultHour={endHour} onChange={(hour) => { setEndHour(hour) }} />
               </Stack>
            </Stack>
         </Stack>
         <Stack horizontal>
            <Stack grow />
            <Stack horizontal style={{ paddingTop: 24 }} tokens={{ childrenGap: 16 }}>
               <Stack style={{ paddingBottom: 8, paddingRight: 0 }}>
                  <PrimaryButton text="Ok" onClick={() => {


                     startDate.setHours(0, 0, 0, 0);
                     endDate.setHours(0, 0, 0, 0);

                     let offset = 0;
                     
                     if (dashboard.displayUTC) {
                        offset = -(new Date().getTimezoneOffset()) * 60 * 1000;
                     }                     

                     const start = moment.utc(startDate.getTime() + startHour * 3600000 + offset).toDate();
                                          
                     const end = moment.utc(endDate.getTime() + endHour * 3600000 + offset).toDate()                     

                     onChange(start, end)
                  }} />
               </Stack>
               <Stack style={{ paddingBottom: 8, paddingRight: 8 }}>
                  <DefaultButton text="Cancel" onClick={() => { onDismiss() }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>

}