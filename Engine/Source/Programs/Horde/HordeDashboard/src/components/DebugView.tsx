// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { useNavigate, useParams } from 'react-router-dom';
import backend from '../backend';

export const DebugView: React.FC = () => {

    const navigate = useNavigate();
    const params = useParams<{ leaseId?: string }>();

    if (params.leaseId) {

        const getLease = async () => {
            try {
                const lease = await backend.getLease(params.leaseId!);

                if (lease) {

                  navigate(`/log/${lease.logId}`, {replace: true});

                } else {
                    console.error("Unable to get lease ", params.leaseId);
                }

            } catch (error) {
                console.error(error);
            }
        }

        getLease();
    }

    return null;

}