// Copyright Epic Games, Inc. All Rights Reserved.
import { DefaultButton, Dropdown, IconButton, IDropdownOption, MessageBar, MessageBarType, Modal, PrimaryButton, Stack, Text, TextField } from '@fluentui/react';
import React, { useState } from 'react';
import backend from '../backend';
import { GetJobResponse, JobState, Priority } from '../backend/Api';
import { getHordeStyling } from '../styles/Styles';

export const EditJobModal: React.FC<{ jobData: GetJobResponse | undefined; show: boolean; onClose: () => void }> = ({ jobData, show, onClose }) => {

   const [error, setError] = useState("");

   const { hordeClasses } = getHordeStyling();
   
   if (!jobData) {
      return null;
   }

	const close = () => {
		setError("");
		onClose();
   };
   

	let priority = jobData.priority!;
   let name = jobData.name!;
   
   const jobComplete = jobData?.state === JobState.Complete;

	const onEdit = async () => {

		name = name.trim();

		if (!name) {
			setError("Please specify a new job name");
			return;
		}

		if (name === jobData.name && priority === jobData.priority) {
			console.log("Nothing changed");
			close();
			return;
		}		

		if (name.length > 64) {
			setError("Please specify a new job name with 64 characters or less");
			return;
		}
		
		// left in case we want to constrain characters in some way
		//const result = name.match(/[A-Za-z0-9 ,_&()]*/);
		/*		
		if (!result || result[0] !== result.input) {
			setError("Please use only alphanumeric, space, and ',_&()' characters in job name");
			return;
		}
		*/

		setError("");

		await backend.updateJob(jobData.id, { priority: priority, name: name }).then((response) => {
			console.log("Job edited", response);
			jobData.name = name;
			jobData.priority = priority;			
		}).catch((reason) => {
			// @todo: error ui
			console.error(reason);
		}).finally(() => {
			close();
		});

	};

	const height = error ? 300 : 260;

	const options: IDropdownOption[] = [];

	for (const p in Priority) {
		options.push({
			text: p,
			key: p,
			isSelected: jobData?.priority === p
		});
	}

	return <Modal isOpen={show} className={hordeClasses.modal }styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { if (show) close() }}>
		<Stack horizontal styles={{ root: { padding: 8 } }}>
			<Stack.Item grow={2}>
				<Text variant="mediumPlus">Job Settings</Text>
			</Stack.Item>
			<Stack.Item grow={0}>
				<IconButton
					iconProps={{ iconName: 'Cancel' }}
					ariaLabel="Close popup modal"
					onClick={() => { setError(""); close(); }}
				/>
			</Stack.Item>
		</Stack>

		{error && <MessageBar
			messageBarType={MessageBarType.error}
			isMultiline={false}> {error} </MessageBar>}

		<Stack tokens={{ childrenGap: 8 }} styles={{ root: { paddingLeft: 8, width: 500 } }}>
			<TextField label="Job Name" defaultValue={jobData?.name} onChange={(ev, value) => name = value ?? ""} />
         <Dropdown disabled={ jobComplete} label="Priority" options={options} onChange={(ev, option) => priority = (option?.key as Priority) ?? priority} />
		</Stack>


		<Stack horizontal styles={{ root: { padding: 8, paddingTop:16 } }}>
			<Stack grow/>
			<Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
				<PrimaryButton text="Save" disabled={false} onClick={() => { onEdit(); }} />
				<DefaultButton text="Cancel" disabled={false} onClick={() => { setError(""); close(); }} />
			</Stack>
		</Stack>
	</Modal>;

};