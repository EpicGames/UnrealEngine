// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, CommandBar, CommandBarButton, DefaultButton, Dialog, DialogFooter, DialogType, Dropdown, DropdownMenuItemType, getFocusStyle, ICommandBarItemProps, Icon, IconButton, IContextualMenuItem, IContextualMenuItemProps, IContextualMenuProps, IDropdownOption, IGroup, Label, List, mergeStyleSets, MessageBar, MessageBarType, Modal, Persona, PersonaSize, Pivot, PivotItem, PrimaryButton, Separator, Spinner, SpinnerSize, Stack, TagPicker, Text, TextField, TooltipHost } from "@fluentui/react";
import { action, makeObservable, observable } from 'mobx';
import { observer } from "mobx-react-lite";
import moment from "moment";
import React, { useEffect, useRef, useState } from "react";
import { Link, useLocation, useNavigate } from "react-router-dom";
import backend from "../backend";
import { CommitId, CreateExternalIssueResponse, CreateJobRequest, EventSeverity, GetChangeSummaryResponse, GetExternalIssueProjectResponse, GetExternalIssueResponse, GetIssueResponse, GetIssueSpanResponse, GetIssueStepResponse, GetIssueStreamResponse, GetLogEventResponse, GetTemplateRefResponse, GetUserResponse, IssueData, IssueSeverity, UpdateIssueRequest } from "../backend/Api";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { projectStore } from "../backend/ProjectStore";
import templateCache from '../backend/TemplateCache';
import { Markdown } from '../base/components/Markdown';
import { useWindowSize } from "../base/utilities/hooks";
import { displayTimeZone, getHumanTime, getShortNiceTime } from "../base/utilities/timeUtils";
import { getHordeStyling } from "../styles/Styles";
import { getHordeTheme } from "../styles/theme";
import { errorDialogStore } from "./error/ErrorStore";
import { useQuery } from "horde/base/utilities/hooks";
import { renderLine } from "./logs/LogRender";
import { UserSelect } from "./UserSelect";
import { CommitIdField } from "./commits/commitField";
import { hasPluginPermission } from 'hordePlugins';

const smallThreshhold = 1100;

let _customClasses: any;
const getCustomClasses = () => {

   const theme = getHordeTheme();
   const { modeColors } = getHordeStyling();

   const background = dashboard.darktheme ? modeColors.background : "#FAF9F9";

   const customClasses = _customClasses ?? mergeStyleSets({
      actionBar: {
         backgroundColor: background,
         fontSize: "12px !important",
         ':hover': {
            filter: dashboard.darktheme ? undefined : "brightness(95%)",
         },
         selectors: {
            '.ms-Button': {
               minWidth: 64,
               height: 32
            },
            '.ms-Icon': {
               fontSize: "12px !important"
            },
            '.ms-Button:hover': {
               backgroundColor: dashboard.darktheme ? theme.palette.neutralLight : undefined
            },
            '.ms-CommandBar': {
               backgroundColor: background
            },
            '.ms-Button--commandBar': {
               backgroundColor: background
            },
         }
      },
      container: {
         selectors: {
            '.ms-List-cell:nth-child(even)': {
               background: "rgb(230, 229, 229)",
            },
            '.ms-List-cell:nth-child(odd)': {
               background: "#FFFFFF",
            }
         }
      }

   });

   _customClasses = customClasses;
   return customClasses;
}


type ChangeItem = {
   change: number;
   commit?: GetChangeSummaryResponse;
   success?: boolean;
   stepResponse?: GetIssueStepResponse;
   stepName?: string;
   templateId?: string;
}

type ChangeGroup = IGroup & ChangeItem;

class IssueDetails {

   constructor() {
      makeObservable(this);
   }

   async set(issueId: number) {

      if (this.issue?.id === issueId) {
         return;
      }

      try {
         await this.refresh(issueId);
      } catch (reason) {
         this.issueError = reason as string;
         this.updated();
      }

   }

   clear() {

      this.issue = undefined;
      this.jiraIssue = undefined;
      this.issueStreams = undefined;
      this.logEvents.clear();
      this.commits.clear();
      this.shownGroupEvents = undefined;
      this.valid = false;
      this.outputClicked = undefined;
      this.suspects = undefined;
      this.minTime = undefined;
      this.maxTime = undefined;
      this.issueError = undefined;
   }

   getSuspectSwarmRange(): string | undefined {


      const streams = this.issueStreams?.filter(a => !!a.minChange).sort((a, b) => a.minChange! - b.minChange!);
      if (!streams?.length) {
         return;
      }

      const issueStream = streams[0];

      if (!issueStream.minChange) {
         return;
      }

      let CL: number | undefined;
      issueStream.nodes.filter(n => n.lastSuccess?.change === issueStream.minChange).forEach(n => {
         n.steps.forEach(s => {
            if (!CL || s.change < CL) {
               CL = s.change;
            }
         });
      });

      if (!CL) {
         return undefined;
      }

      const stream = projectStore.streamById(issueStream.streamId);
      const project = projectStore.byId(stream?.projectId)!;
      const name = project?.name === "Engine" ? "UE4" : project?.name;

      if (!stream || !project || !name) {
         return undefined;
      }


      return `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${issueStream.minChange},@${CL}#commits`;

   }


   getDefaultNode(): { streamId: string, node: GetIssueSpanResponse } | undefined {

      if (!this.issue) {
         return undefined;
      }

      let bestFailure: { streamId: string, node: GetIssueSpanResponse } | undefined;
      let bestSuccess: { streamId: string, node: GetIssueSpanResponse } | undefined;


      this.issueStreams?.forEach(stream => {

         stream.nodes.forEach(n => {

            // a success
            if (n.nextSuccess) {

               if (!bestSuccess) {
                  bestSuccess = {
                     streamId: stream.streamId,
                     node: n
                  };

               } else {

                  if (bestSuccess.node.nextSuccess!.change < n.nextSuccess.change) {
                     bestSuccess = {
                        streamId: stream.streamId,
                        node: n
                     };
                  }
               }

            } else {

               if (!bestFailure) {
                  bestFailure = {
                     streamId: stream.streamId,
                     node: n
                  };

               } else {

                  let curChange = 0;
                  bestFailure.node.steps.forEach(s => {
                     if (s.change > curChange) {
                        curChange = s.change;
                     }
                  });

                  let newChange = 0;
                  n.steps.forEach(s => {
                     if (s.change > newChange) {
                        curChange = newChange;
                     }
                  });

                  if (newChange > curChange) {
                     bestFailure = {
                        streamId: stream.streamId,
                        node: n
                     };
                  }
               }

            }

         })

      })

      if (bestFailure) {
         return bestFailure;
      }

      return bestSuccess;

   }

   getStepName(streamId: string, step: GetIssueStepResponse): string | undefined {

      if (!this.issue) {
         return undefined;
      }

      const stream = this.issueStreams?.find(s => s.streamId === streamId);

      if (!stream) {
         return undefined;
      }

      for (let i = 0; i < stream.nodes.length; i++) {
         const node = stream.nodes[i];

         if (step.logId === node.lastSuccess?.logId) {
            return node.name;
         }

         if (step.logId === node.nextSuccess?.logId) {
            return node.name;
         }

         if (node.steps.find(s => s.logId === step.logId)) {
            return node.name;
         }

      }

      return undefined;

   }

   getTemplateName(templateId: string): string {

      if (!this.issue) {
         return templateId;
      }

      let name = templateId;

      this.issue.affectedStreams.forEach(stream => {

         if (name !== templateId) {
            return;
         }
         name = stream.affectedTemplates.find(tinfo => tinfo.templateId === templateId)?.templateName ?? templateId;
      })

      return name;
   }

   resolved(streamId?: string, templateId?: string): boolean {

      if (!this.issue) {
         return false;
      }

      if (streamId) {


         if (!templateId) {

            return this.issue.resolvedStreams.indexOf(streamId) !== -1;
         }

         const stream = this.issue.affectedStreams.find(s => s.streamId === streamId)!

         return !!stream.resolvedTemplateIds?.find(t => t === templateId);

      }

      return false;
   }

   getLogEvents(jobId: string, logId: string) {

      const result = this.logEvents.get(jobId + logId);

      if (result === undefined) {
         this.queryLogEvents(jobId, logId);
         return [];
      }

      return result;
   }

   async queryLogEvents(jobId: string, logId: string) {

      if (!this.issue) {
         return;
      }

      const id = jobId + logId;

      const existing = this.logEvents.get(id);

      if (existing) {
         return existing;
      }

      this.logEvents.set(id, []);

      const values = await backend.getIssueEvents(this.issue.id, jobId, [logId], 100);

      this.logEvents.set(id, values);

      this.updated();

      return values;
   }


   @observable
   update = 0

   @action
   private updated() {
      this.update++;
   }

   async refresh(issueId: number) {

      let value: IssueData | undefined;

      try {
         value = await backend.getIssue(issueId);
      } catch {
         throw new Error(`Unable to get issue ${issueId}`);
      }

      this.issue = value;

      if (this.issue.primarySuspectsInfo?.length) {

         let squery = [...this.issue.primarySuspectsInfo];
         this.suspects = [];

         while (squery.length) {

            const results = await backend.getUsers({
               ids: squery.slice(0, 256).map(s => s.id),
               includeClaims: true,
               includeAvatar: true
            });

            if (results?.length) {
               this.suspects.push(...results);
            }

            squery = squery.slice(256)
         }
      }

      this.issueStreams = await backend.getIssueStreams(issueId);

      this.jiraIssue = undefined;

      // cache external issue projects for stream
      try {
         for (let i = 0; i < this.issueStreams.length; i++) {
            const issueStream = this.issueStreams[i];
            let externalProjects = IssueDetails.externalStreamProjects.get(issueStream.streamId);
            if (!externalProjects) {
               externalProjects = await backend.getExternalIssueProjects(issueStream.streamId);
               IssueDetails.externalStreamProjects.set(issueStream.streamId, externalProjects);
            }
         }



         if (this.issue.externalIssueKey) {

            if (this.issueStreams?.length) {
               const jissue = await backend.getExternalIssues(this.issueStreams[0].streamId, [this.issue.externalIssueKey]);
               if (jissue && jissue.length) {
                  this.jiraIssue = jissue[0];
               }
            }
         }
      } catch (reason) {
         console.error("Unable to get external issues", reason);
      }


      let minTime: number | undefined;
      let maxTime: number | undefined;

      this.fakeCLTimes.clear();

      this.issueStreams?.forEach(stream => {

         stream.nodes.forEach(n => {

            const steps = [...n.steps];

            if (n.lastSuccess) {
               steps.unshift(n.lastSuccess);
            }

            if (n.nextSuccess) {
               steps.push(n.nextSuccess);
            }

            steps?.forEach(step => {

               const curTime = this.fakeCLTimes.get(step.change);
               const stepTime = new Date(step.stepTime);
               if (!curTime || curTime > stepTime) {
                  this.fakeCLTimes.set(step.change, stepTime);
               }
            })
         });
      });


      this.issueStreams?.forEach(stream => {

         stream.nodes.forEach(n => {

            const steps = [...n.steps];

            if (n.lastSuccess) {
               steps.unshift(n.lastSuccess);
            }

            if (n.nextSuccess) {
               steps.push(n.nextSuccess);
            }

            steps.forEach(step => {

               const clTime = this.fakeCLTimes.get(step.change)!.toISOString();

               const time = new Date(clTime).getTime();
               if (minTime === undefined || time < minTime) {
                  minTime = time;
               }
               if (maxTime === undefined || time > maxTime) {
                  maxTime = time;
               }

            });
         });
      });

      this.minTime = new Date(minTime ?? 0);
      this.maxTime = new Date(maxTime ?? 0);


      this.valid = true;

      this.updated();

   }

   getExternalProjectId(name?: string): string | undefined {

      if (!name) {
         return undefined;
      }

      let id: string | undefined;
      IssueDetails.externalStreamProjects.forEach((projects) => {
         const project = projects.find(p => p.name === name);
         if (project) {
            id = project.id;
         }
      });

      return id;

   }

   getExternalProjectComponentId(projectName?: string, componentName?: string): string | undefined {

      if (!projectName || !componentName) {
         return undefined;
      }

      let id: string | undefined;
      IssueDetails.externalStreamProjects.forEach((projects) => {
         const project = projects.find(p => p.name === projectName);
         if (project) {
            for (let cid in project.components) {
               const cname = project.components[cid];
               if (cname === componentName) {
                  id = cid;
               }
            }
         }
      });

      return id;

   }


   getExternalProjectIssueTypeId(projectName?: string, issueTypeName?: string): string | undefined {

      if (!projectName || !issueTypeName) {
         return undefined;
      }

      let id: string | undefined;
      IssueDetails.externalStreamProjects.forEach((projects) => {
         const project = projects.find(p => p.name === projectName);
         if (project) {
            for (let issueId in project.issueTypes) {
               const name = project.issueTypes[issueId];
               if (name === issueTypeName) {
                  id = issueId;
               }
            }
         }
      });

      return id;

   }


   getExternalIssueProjects(): GetExternalIssueProjectResponse[] {
      const result: GetExternalIssueProjectResponse[] = [];

      this.issueStreams?.forEach(stream => {
         const projects = IssueDetails.externalStreamProjects.get(stream.streamId);
         projects?.forEach(project => {
            if (!result.find(p => p.projectKey === project.projectKey)) {
               result.push(project);
            }
         })
      });

      return result.sort((a, b) => a.name.localeCompare(b.name));
   }

   valid: boolean = false;

   issue?: GetIssueResponse;
   jiraIssue?: GetExternalIssueResponse;

   commits: Map<string, GetChangeSummaryResponse[]> = new Map();

   logEvents: Map<string, GetLogEventResponse[]> = new Map();

   issueStreams?: GetIssueStreamResponse[];

   suspects?: GetUserResponse[];

   shownGroupEvents?: ChangeGroup;
   outputClicked?: boolean;

   minTime?: Date;
   maxTime?: Date;

   fakeCLTimes = new Map<number, Date>();

   issueError?: string;

   static externalStreamProjects: Map<string, GetExternalIssueProjectResponse[]> = new Map();

}

const details = new IssueDetails();

type StepBox = {
   minX: number;
   maxX: number;
   minY: number;
   maxY: number;
   streamId: string;
   node: GetIssueSpanResponse;
   step: GetIssueStepResponse;
}

type JobStep = {
   node: GetIssueSpanResponse;
   step: GetIssueStepResponse;
}

let stepBoxes: StepBox[] = [];

const StreamCanvas: React.FC = () => {

   const [state, setState] = useState<{ zoom: number, hoverStep?: StepBox, selectedStep?: StepBox, height?: number }>({ zoom: 1 });
   const [canvas, setCanvas] = useState<HTMLCanvasElement | null>(null);

   const scrollRef = useRef<HTMLDivElement>(null);

   useEffect(() => {
      // this prevents mouse scroll on bus stop div overflow as we use that for zoom
      const listener = (ev: WheelEvent) => { ev.preventDefault(); return false }
      const current = scrollRef.current;
      if (current) {
         current.addEventListener('wheel', listener, { passive: false });
      }
      return () => {
         stepBoxes = [];

         if (current) {
            current.removeEventListener('wheel', listener);
         }
      };

   }, []);


   const colors = dashboard.getStatusColors();
   const { hordeClasses } = getHordeStyling();

   if (!details.issueStreams) {
      return null;
   }

   if (!details.minTime || !details.maxTime) {
      return null;
   }

   const color0 = dashboard.darktheme ? "#FFFFFF" : "#323130";
   const color1 = dashboard.darktheme ? "#555555" : "#C0C0C0";

   // stream text blue
   const color2 = dashboard.darktheme ? "#4DB1F0" : "#0872C4";

   let minTime = details.minTime.getTime() / 1000;
   let maxTime = details.maxTime.getTime() / 1000;

   const streamJobs = new Map<string, Map<string, JobStep[]>>();
   let templates: GetTemplateRefResponse[] = [];

   details.issueStreams.forEach((stream) => {

      let jobSteps = streamJobs.get(stream.streamId);
      if (!jobSteps) {
         jobSteps = new Map();
         streamJobs.set(stream.streamId, jobSteps);
      }

      stream.nodes.forEach(node => {

         const template = projectStore.streamById(stream.streamId)?.templates?.find(t => t.id === node.templateId);

         if (!template) {
            return;
         }

         if (!templates.find(t => t.id === template.id)) {
            templates.push(template);
         }

         const steps = [...node.steps];

         // note that last and next success have unspecified severity
         if (node.lastSuccess) {
            steps.unshift(node.lastSuccess);
         }

         if (node.nextSuccess) {
            steps.push(node.nextSuccess);
         }

         steps.forEach(step => {
            let key = template.name + node.name;
            let steps = jobSteps!.get(key);
            if (!steps) {
               steps = [];
               jobSteps!.set(key, steps);
            }

            steps.push({ step: step, node: node });

         })
      })
   });

   templates = templates.sort((a, b) => a.name.localeCompare(b.name));

   function drawRoundedRect(context: CanvasRenderingContext2D, minX: number, minY: number, maxX: number, maxY: number, radius: number, color: string) {

      context.fillStyle = color;

      if (maxX > minX && maxY > minY) {
         radius = Math.min(radius, Math.min(maxY - minY, maxX - minX));

         context.beginPath();
         context.moveTo(minX + radius, minY);
         context.arc(maxX - radius, minY + radius, radius, -Math.PI * 0.5, 0.0);
         context.arc(maxX - radius, maxY - radius, radius, 0.0, +Math.PI * 0.5);
         context.arc(minX + radius, maxY - radius, radius, +Math.PI * 0.5, +Math.PI * 1.0);
         context.arc(minX + radius, minY + radius, radius, -Math.PI * 1.0, -Math.PI * 0.5);
         context.closePath();
         context.fill();
      }
   }

   let y = 0;

   if (canvas) {

      const context = canvas.getContext('2d')!;

      type CLDraw = {
         step: GetIssueStepResponse;
         minX: number;
         maxX: number;
         minY: number;
         maxY: number;
         change: number;
      }

      let drawCL: CLDraw | undefined;

      stepBoxes = [];

      context.clearRect(0, 0, canvas.width, canvas.height);

      let timeSpan = (maxTime - minTime) / state.zoom;

      if (timeSpan < 1.0) {
         timeSpan = 1.0;
      }

      minTime = maxTime - timeSpan;

      y = 8;

      let first = true;

      const dividers: number[] = [];

      streamJobs.forEach((jobs, streamId) => {

         const stream = projectStore.streamById(streamId);

         if (!stream) {
            return;
         }

         // time headers
         if (first) {

            const maxColumns = 5;
            const totalDays = Math.ceil(moment.duration(moment(maxTime * 1000).diff(minTime * 1000)).asDays());

            const spanWidth = 980;
            const columnWidth = spanWidth / maxColumns;

            for (let i = 0; i < maxColumns; i++) {

               let x = 0;
               let time = "";

               // if one day, just center the max time
               if (totalDays === 1) {
                  x = 2 * columnWidth + (columnWidth / 2);
                  time = getHumanTime(new Date((maxTime * 1000)));
               } else {

                  x = i * columnWidth + (columnWidth / 2);
                  time = getHumanTime(new Date((minTime * 1000) + (timeSpan * (x / spanWidth)) * 1000));

                  // see how many columns have this date
                  let ni = i + 1;
                  let c = 0;
                  while (ni < maxColumns) {
                     let nx = ni * columnWidth + (columnWidth / 2);
                     let ntime = getHumanTime(new Date((minTime * 1000) + (timeSpan * (nx / spanWidth)) * 1000));
                     if (ntime !== time) {
                        break;
                     }
                     c++;
                     ni++;
                  }

                  if (c) {
                     const width = (ni - i) * columnWidth;
                     x = i * columnWidth + width / 2;
                     time = getHumanTime(new Date((minTime * 1000) + (timeSpan * (x / spanWidth)) * 1000));
                     i = ni - 1;
                  }

               }


               const text = time;
               context.font = "11px Horde Open Sans Regular";
               context.fillStyle = color0;
               context.textAlign = 'left';
               context.textBaseline = 'middle';
               const metrics = context.measureText(text);
               context.fillText(text, x - metrics.width / 2 + 300, Math.floor(y + 8));

               if (totalDays === 1) {
                  break;
               }

               dividers.push(x - metrics.width / 2 + 300 + metrics.width / 2);

            }

         }

         if (first) {
            y += 32;
         } else {
            y += 8;
         }


         context.font = "12px Horde Open Sans SemiBold";
         context.fillStyle = color2;
         context.textAlign = 'left';
         context.textBaseline = 'middle';
         context.fillText(stream.fullname!, 0, y);

         y += 24;

         templates.forEach(template => {

            let drawTemplateText = true;
            const sorted = Array.from(jobs.keys()).sort((a, b) => a.localeCompare(b));

            let gotone = false;
            sorted.forEach((key) => {

               let steps = jobs.get(key)!;
               const node = steps[0].node;

               if (node.templateId !== template.id) {
                  return;
               }

               gotone = true;

               if (drawTemplateText) {
                  context.font = "11px Horde Open Sans SemiBold";
                  context.fillStyle = color0;
                  context.textAlign = 'left';
                  context.textBaseline = 'middle';
                  context.fillText(template.name, 0, y);
                  drawTemplateText = false;
                  y += 20;
               }
               steps = steps.sort((a, b) => {
                  const timeA = details.fakeCLTimes.get(a.step.change)!;
                  const timeB = details.fakeCLTimes.get(b.step.change)!;
                  return timeA.getTime() - timeB.getTime();
               });

               context.font = "11px Horde Open Sans Regular";
               context.fillStyle = color0;
               context.textAlign = 'left';
               context.textBaseline = 'middle';
               context.fillText(node.name, 8, y + 1);

               let lastStep: GetIssueStepResponse | undefined;
               let lastX: number | undefined;
               let lastColor: string | undefined;

               steps.forEach((jobStep, index) => {

                  const step = jobStep.step;

                  const width = 980;
                  const startX = 280;

                  let color = color1;
                  if (step.severity === IssueSeverity.Warning) {
                     color = colors.get(StatusColor.Warnings)!;
                  }
                  if (step.severity === IssueSeverity.Error) {
                     color = colors.get(StatusColor.Failure)!;
                  }

                  if (details.issue?.quarantinedByUserInfo && index && step.severity === IssueSeverity.Unspecified) {
                     color = colors.get(StatusColor.Success)!;
                  }

                  let baseTime = details.fakeCLTimes.get(step.change)!.getTime() / 1000;

                  if (baseTime < minTime) {

                     lastX = startX;
                     lastStep = step;
                     lastColor = color;

                     return;
                  }

                  let chosenOne = 0;

                  if (state.hoverStep?.step.logId === step.logId) {
                     chosenOne = 1;

                  }

                  if (state.selectedStep?.step.logId === step.logId) {
                     chosenOne = 2;
                  }

                  baseTime -= minTime;

                  let x = startX + width * (baseTime / timeSpan);

                  if (lastStep) {

                     context.beginPath();
                     context.strokeStyle = lastColor!;
                     context.moveTo(lastX!, y);
                     context.lineTo(x, y);
                     context.stroke();

                  }

                  let size = 4;
                  if (chosenOne === 1) {
                     size = 6;
                  }
                  if (chosenOne === 2) {
                     size = 8;
                  }


                  const minX = x - size;
                  const maxX = x + size;
                  const minY = y - size;
                  const maxY = y + size;

                  if (index !== steps.length - 1) {
                     drawRoundedRect(context, minX, minY, maxX, maxY, size, color);
                     if (chosenOne === 2 && !state.hoverStep) {
                        drawCL = {
                           change: step.change,
                           step: step,
                           minX: minX,
                           maxX: maxX,
                           minY: minY,
                           maxY: maxY
                        };
                     }
                  } else {
                     if (step.severity === IssueSeverity.Unspecified) {
                        if (index > 0 && steps[index - 1].step.change < step.change) {
                           color = colors.get(StatusColor.Success)!;
                        }
                     }
                     context.fillStyle = color;
                     if (chosenOne !== 2) {
                        context.fillRect(x - 5, y - 5, 10, 10);
                     } else {

                        if (!state.hoverStep) {
                           drawCL = {
                              change: step.change,
                              step: step,
                              minX: minX,
                              maxX: maxX,
                              minY: minY,
                              maxY: maxY
                           };
                        }

                        context.fillRect(x - 7, y - 7, 14, 14);

                     }
                  }

                  stepBoxes.push({ minX: minX, maxX: maxX, minY: minY, maxY: maxY, step: step, streamId: streamId, node: jobStep.node });

                  if (state.hoverStep?.step.logId === step.logId) {

                     drawCL = {
                        change: step.change,
                        step: step,
                        minX: minX,
                        maxX: maxX,
                        minY: minY,
                        maxY: maxY
                     };
                  }

                  lastX = x;
                  lastStep = step;
                  lastColor = color;
               })

               y += 20;

            })

            if (gotone) {
               y += 6;
            }

            first = false;

         });

         dividers.forEach(x => {
            // divider
            context.beginPath();
            context.strokeStyle = color1;
            context.moveTo(x, 32);
            context.lineTo(x, y + 16);
            context.stroke();
         })

         if (drawCL) {

            const date = getShortNiceTime(drawCL.step.stepTime);

            const change = `${date} - CL ${drawCL.step.change.toString()}`;

            context.font = `11px Horde Open Sans Regular`;
            context.textAlign = 'center';
            context.textBaseline = 'top';
            const metrics = context.measureText(change);

            context.fillStyle = "#F3F2F1";
            const minx = Math.min(drawCL.minX, 1200);
            context.fillRect(minx - metrics.width / 2 - 3, drawCL.minY - 28, metrics.width + 21, 25);



            context.fillStyle = "#2D3F5F";
            context.fillText(change, minx + 6, drawCL.minY - 21);

         }
      });
   }

   const onChartScroll = (e: any) => {
      e.stopPropagation();
      const direction = Math.sign(e.deltaY);
      if (direction < 0 && state.zoom < 10) {
         setState({ ...state, zoom: state.zoom + 1 });
      } else if (direction > 0 && state.zoom > 1) {
         setState({ ...state, zoom: state.zoom - 1 });
      }
   };

   const getStepBox = (ev: React.MouseEvent<HTMLCanvasElement, MouseEvent>): StepBox | undefined => {

      if (!canvas) {
         return;
      }

      let rect = canvas.getBoundingClientRect();
      let scaleX = 1.0;  //canvas.width / rect.width;
      let scaleY = 1.0; //canvas.height / rect.height;

      let xPos = (ev.clientX - rect.left) * scaleX;
      let yPos = (ev.clientY - rect.top) * scaleY;

      let step: StepBox | undefined;
      let best = Number.MAX_VALUE;
      for (let i = 0; i < stepBoxes.length; i++) {
         const box = stepBoxes[i];

         const centerX = (box.maxX - box.minX) / 2 + box.minX;
         const centerY = (box.maxY - box.minY) / 2 + box.minY;

         const dist = Math.sqrt((xPos - centerX) ** 2 + (yPos - centerY) ** 2);
         if (dist < best) {
            best = dist;
            step = box;
         }
      }

      return step;
   };

   const onCanvasMouseMove = (ev: React.MouseEvent<HTMLCanvasElement, MouseEvent>) => {

      setState({ ...state, hoverStep: getStepBox(ev) });

   };

   const onCanvasMouseLeave = (ev: React.MouseEvent<HTMLCanvasElement, MouseEvent>) => {

      setState({ ...state, hoverStep: undefined });

   };


   const onCanvasClick = (ev: React.MouseEvent<HTMLCanvasElement, MouseEvent>) => {

      setState({ ...state, selectedStep: getStepBox(ev) });

   };

   if (y && state.height !== y) {
      setState({ ...state, height: y })
   }

   // todo default to most recent
   const currentStep = state.selectedStep;

   if (!currentStep) {
      let defaultStep = stepBoxes.sort((a, b) => {
         return new Date(b.step.stepTime).getTime() - new Date(a.step.stepTime).getTime();
      }).find(s => s.step.severity !== IssueSeverity.Unspecified);

      if (defaultStep) {
         setState({ ...state, selectedStep: defaultStep });
      }
   }

   let height = 50;
   if (state.height) {
      height = state.height;
   }

   // Blurry text render fix
   const canvasBaseWidth = 1280;
   const ratio = Math.ceil(window.devicePixelRatio);
   const canvasWidth = canvasBaseWidth * ratio;
   const canvasHeight = height * ratio;
   const canvasStyleWidth = `${canvasBaseWidth}px`;
   const canvasStyleHeight = `${height}px`;

   if (canvas) {
      canvas.getContext('2d')?.setTransform(ratio, 0, 0, ratio, 0, 0);
   }

   return <Stack style={{ height: "100%" }} tokens={{ childrenGap: 18 }}>
      <Stack className={hordeClasses.raised} style={{ maxHeight: window.innerHeight < smallThreshhold ? 320 : 400, flexShrink: 1 }}>
         <Stack className="horde-no-darktheme" style={{ position: "relative", height: "100%" }}>
            <div ref={scrollRef} style={{ overflowX: "hidden", overflowY: "auto" }}>
               <Stack>
                  <canvas ref={newRef => setCanvas(newRef)} width={canvasWidth} height={canvasHeight} onWheel={onChartScroll} style={{ width: canvasStyleWidth, height: canvasStyleHeight, overflowX: "visible" }} onClick={(ev) => onCanvasClick(ev)} onMouseLeave={(ev => onCanvasMouseLeave(ev))} onMouseMove={(ev) => onCanvasMouseMove(ev)}></canvas>
               </Stack>
            </div>
         </Stack>
      </Stack>
      <Stack style={{ flexShrink: 0, flexGrow: 1 }}>
         <Stack className={hordeClasses.raised} style={{ height: window.innerHeight < smallThreshhold ? "calc(100% - 18px)" : "100%" }}>
            {!!currentStep && <StepPanel streamId={currentStep.streamId} hstep={currentStep.step} />}
         </Stack>
      </Stack>
   </Stack>
}


type SummaryItem = {
   title: string;
   text: string;
   link?: string;
   strike?: boolean;
}

const IssueHeader: React.FC<{ items?: SummaryItem[] }> = ({ items }) => {

   const issue = details.issue!;

   if (!items?.length && !issue.description) {
      return null;
   }
   const renderSummaryItem = (title: string, text: string | undefined, link?: string, strike?: boolean) => {

      if (!text) {
         return <Stack />;
      }

      return <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }}>
         <Stack >
            <Label style={{ padding: 0, minWidth: 110 }}>{`${title}:`}</Label>
         </Stack>
         <Stack>
            {!!link && <a href={link} target="_blank" rel="noreferrer" style={{ fontSize: "12px", textDecoration: strike ? "line-through" : undefined }}>
               {text}
            </a>
            }
            {!link && <Text style={{ textDecoration: strike ? "line-through" : undefined }} variant="small">{text}</Text>}
         </Stack>

      </Stack>
   }

   let summaryItems: JSX.Element[] | undefined;
   if (items?.length) {
      summaryItems = items.map(i => renderSummaryItem(i.title, i.text, i.link, i.strike));
   }

   return <Stack horizontal tokens={{ childrenGap: 48 }} >
      {!!summaryItems && <Stack style={{ width: 540, padding: 0 }} tokens={{ childrenGap: 6 }}>
         {summaryItems}
      </Stack>}
      {!!issue.description && <Stack>
         <Label style={{ padding: 0 }}>{`Description:`}</Label>
         <Markdown styles={{ root: { maxHeight: 240, maxWidth: 1000, overflow: "auto", th: { fontSize: 12 } } }}>{issue.description}</Markdown>
      </Stack>}
   </Stack>

}

const IssueSummaryPanel: React.FC = () => {

   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;

   if (!issue) {
      return null;
   }

   const items: SummaryItem[] = [];

   if (issue.duplicateIssueId !== undefined) {
      items.push({
         title: `Duplicate of Issue`,
         text: issue.duplicateIssueId === 0 ? "Unspecified" : issue.duplicateIssueId.toString(),
         link: issue.duplicateIssueId === 0 ? undefined : `/index?issue=${issue.duplicateIssueId}`
      })
   }


   if (issue.workflowThreadUrl) {
      items.push({
         title: "Workflow",
         text: "Slack Thread",
         link: issue.workflowThreadUrl
      })
   }

   if (!!issue.quarantineTimeUtc && !!issue.quarantinedByUserInfo) {
      items.push({
         title: "Quarantined By",
         text: `${issue.quarantinedByUserInfo.name} on ${getShortNiceTime(issue.quarantineTimeUtc)}`
      })
   }

   if (!!issue.forceClosedByUserInfo) {
      items.push({
         title: "Force Closed By",
         text: `${issue.forceClosedByUserInfo.name}`
      })
   }

   if (!!issue.resolvedAt) {
      items.push({
         title: "Resolved By",
         text: `${issue.resolvedByInfo?.name ?? "Horde"} on ${getShortNiceTime(issue.resolvedAt, true, true)}`
      })

      if (issue.fixChange) {
         items.push({
            title: "Fixed CL",
            text: `${issue.fixChange}`,
            link: dashboard.swarmUrl ? `${dashboard.swarmUrl}/changes/${issue.fixChange}` : undefined
         })
      }
   }

   if (!issue.resolvedAt && issue.acknowledgedAt && issue.ownerInfo) {
      items.push({
         title: "Acknowledged By",
         text: `${issue.ownerInfo.name} on ${getShortNiceTime(issue.acknowledgedAt, true, true)}`
      })
   }

   const jiraIssue = details.jiraIssue;

   let jiraAssignee = jiraIssue?.assigneeDisplayName ?? "";
   if (jiraIssue && !jiraAssignee) {
      jiraAssignee = "Unassigned";
   }
   const jiraStatus = jiraIssue?.resolutionName ?? jiraIssue?.statusName ?? "";

   if (!!jiraIssue) {
      items.push({
         title: "Jira",
         text: jiraIssue.key,
         link: jiraIssue.link!,
         strike: !!jiraIssue.resolutionName
      })

      if (jiraAssignee) {
         items.push({
            title: "Jira Assignee",
            text: jiraAssignee,
            link: jiraIssue.link!,
            strike: !!jiraIssue.resolutionName
         })
      }

      if (jiraStatus) {
         items.push({
            title: "Jira Status",
            text: jiraStatus,
            link: jiraIssue.link!,
            strike: !!jiraIssue.resolutionName
         })
      }
   }

   if (issue.fingerprintDescription) {
      const components = issue.fingerprintDescription.split(" / ");
      if (components.length) {
         items.push({
            title: "Fingerprint",
            text: components[0].replace("Type:", "").trim()
         })
      }
   }


   if (issue.rootCommitId) {
      items.push({
         title: "Root Change",
         text: `${issue.rootCommitId}`,
         link: dashboard.swarmUrl ? `${dashboard.swarmUrl}/changes/${issue.rootCommitId}` : undefined
      })
   }
     if (issue.rootCauseOwnerInfo?.name ) {
      items.push({
         title: "Root Cause Owner",
         text: issue.rootCauseOwnerInfo.name
      })
   }
   if (issue.rootCauseCategory) {
      items.push({
         title: "Root Category",
         text: issue.rootCauseCategory
      })
   }
   if (issue.rootCauseSummary) {
      items.push({
         title: "Root Summary",
         text: issue.rootCauseSummary
      })
   }

   if (!items.length && !issue.description) {
      return null;
   }

   return <Stack style={{ flexBasis: "70px", flexShrink: 0 }}>
      <Stack className={hordeClasses.raised} style={{ maxHeight: "100%" }}>
         <IssueHeader items={items.length ? items : undefined} />
      </Stack>
   </Stack>
}

let _errorStyles: any;

const getErrorStyles = () => {

   const theme = getHordeTheme();

   const errorStyles = _errorStyles ?? mergeStyleSets({
      gutter: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: "#EC4C47",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      gutterWarning: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: "rgb(247, 209, 84)",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      itemCell: [
         getFocusStyle(theme, { inset: -1 }),
         {
            selectors: {
               '&:hover': { background: theme.palette.neutralLight }
            }
         }
      ],
   });

   _errorStyles = errorStyles;

   return errorStyles;
}



// fixes issue with the lne items in stack doubling up with same key 
let lineKey = 0;

export const ErrorPane: React.FC<{ events?: GetLogEventResponse[]; onClose?: () => void }> = ({ events }) => {

   const navigate = useNavigate();
   useWindowSize();
   const { modeColors } = getHordeStyling();
   const errorStyles = getErrorStyles();

   const listEvents = events?.sort((a, b) => {
      if (a.severity === EventSeverity.Warning && b.severity === EventSeverity.Error) {
         return 1;
      }
      if (b.severity === EventSeverity.Warning && a.severity === EventSeverity.Error) {
         return -1;
      }
      return a.lineIndex - b.lineIndex;

   })

   const onRenderCell = (item?: GetLogEventResponse, index?: number, isScrolling?: boolean): JSX.Element => {

      if (!item) {
         return <div>???</div>;
      }

      const url = `/log/${item.logId}?lineIndex=${item.lineIndex + 1}`;

      const lines = item.lines.filter(line => line.message?.trim().length).map(line => <Stack key={`errorpane_line_${item.lineIndex}_${lineKey++}`} styles={{ root: { paddingLeft: 8, paddingRight: 8, lineBreak: "normal", whiteSpace: "pre-wrap", lineHeight: 18, fontSize: 10, fontFamily: "Horde Cousine Regular, monospace, monospace" } }}> <Link style={{ color: modeColors.text }} to={url}>{renderLine(navigate, line, undefined, {})}</Link></Stack>);

      return (<Stack className={errorStyles.itemCell} style={{ padding: 8 }}><Stack className={item.severity === EventSeverity.Warning ? errorStyles.gutterWarning : errorStyles.gutter} styles={{ root: { padding: 0, margin: 0 } }}>
         <Stack styles={{ root: { paddingLeft: 14 } }}>
            {!!lines.length && lines}
            {!lines.length && <Text>Missing Log Data</Text>}
         </Stack>
      </Stack>
      </Stack>);
   };

   return <div style={{ padding: 4, paddingLeft: 12, paddingRight: 0, paddingTop: 18, paddingBottom: 18, height: "100%" }}>
      <Stack style={{ overflowY: 'scroll', height: "90%" }} data-is-scrollable={true} >
         {<List
            style={{ maxHeight: 0, minHeight: 0 }} // IMMPORTANT HACK: The Fluent List component computes a height, which is based on the height of all pages, and pushes flex out from parent container.  This "fixes" that behavior
            items={listEvents}
            data-is-focusable={false}
            onRenderCell={onRenderCell} />}
      </Stack>
   </div>
};

// specific job step
const StepPanel: React.FC<{ streamId: string, hstep: GetIssueStepResponse }> = observer(({ streamId, hstep }) => {

   if (details.update) { }

   const { hordeClasses } = getHordeStyling();
   const theme = getHordeTheme();

   const statusColors = dashboard.getStatusColors();
   const RED = statusColors.get(StatusColor.Failure);
   const YELLOW = statusColors.get(StatusColor.Warnings);

   let description = "";

   const stream = projectStore.streamById(streamId);

   if (stream) {

      const jobName = hstep.jobName;

      if (jobName) {
         description = `${jobName}`;
      }

      const stepName = details.getStepName(streamId, hstep);
      if (stepName) {
         description = stepName;
      }

      const warning = hstep.severity === IssueSeverity.Warning;
      const success = hstep.severity === IssueSeverity.Unspecified;

      let backgroundColor = theme.horde.dividerColor;

      let color = RED;

      if (warning) {
         color = YELLOW
      }

      if (success) {
         backgroundColor = theme.horde.dividerColor;
         color = backgroundColor;
      }

      let logEvents: GetLogEventResponse[] | undefined;

      if (!success) {
         logEvents = details.getLogEvents(hstep.jobId, hstep.logId!);
      }

      const displayTime = moment(hstep.stepTime).tz(displayTimeZone());

      const format = dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z";

      let displayTimeStr = displayTime.format(format);

      let changeUrl = `${dashboard.swarmUrl}/changes/${hstep.change}`;
      const stream = projectStore.streamById(streamId)!;
      const project = projectStore.byId(stream?.projectId)!;
      const name = project?.name === "Engine" ? "UE4" : project.name;
      if (name && project) {

         const issueStream = details.issueStreams?.find(s => s.streamId === streamId);
         if (issueStream) {
            const span = issueStream.nodes.find(n => n.name === stepName);
            let prevCL = span?.steps.find(step => step.change < hstep.change)?.change;
            if (!prevCL) {
               prevCL = issueStream.nodes.find(n => n.name === stepName && !!n.lastSuccess)?.lastSuccess?.change;
            }
            if (!prevCL) {
               prevCL = issueStream.minChange;
            }
            if (prevCL) {
               changeUrl = `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${prevCL},@${hstep.change}#commits`;
            }
         } else {
            changeUrl = `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${hstep.change}#commits`;
         }
      }

      const textSize = "small";


      let viewLogURL = `/log/${hstep.logId}`;
      if (logEvents?.length) {
         viewLogURL += `?lineIndex=${logEvents[0].lineIndex + 1}`;
      }

      return <div style={{ paddingTop: 8, height: "100%" }}>
         <Stack style={{ flexBasis: "52px", flexShrink: 0 }}>
            <Stack horizontal verticalAlign="center" style={{ backgroundColor: backgroundColor, width: "100%", paddingLeft: 8, padding: 12 }}>
               <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 12 }} style={{ width: "100%" }} >
                  <Stack className="horde-no-darktheme" style={{ paddingTop: 2 }}>
                     <Icon style={{ color: color }} iconName="Square" />
                  </Stack>
                  <Stack horizontal verticalAlign="center" style={{ width: "100%" }}>
                     <Stack horizontal tokens={{ childrenGap: 24 }}>
                        <Stack verticalFill={true} disableShrink={true}>
                           <Link to={`/job/${hstep.jobId}?step=${hstep.stepId}`} target="_blank">
                              <Stack verticalFill={true} disableShrink={true}>
                                 <Text style={{ fontWeight: 600 }} variant="medium">{description}</Text>
                              </Stack>
                           </Link>
                        </Stack>
                     </Stack>
                     <Stack grow />
                     <Stack horizontal tokens={{ childrenGap: 24 }}>
                        <Stack>
                           <Text>{displayTimeStr}</Text>
                        </Stack>
                        <Stack style={{ height: "100%" }}>
                           <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
                              <Icon styles={{ root: { margin: '0px', padding: '0px', paddingTop: "2px", userSelect: "none" } }} iconName="Commit" className={hordeClasses.iconBlue} />
                              {!!dashboard.swarmUrl && <a href={changeUrl} target="blank">
                                 <Text variant={textSize} styles={{ root: { margin: '0px', padding: '0px' } }} >{`CL ${hstep.change}`}</Text>
                              </a>}
                              {!dashboard.swarmUrl && <Text variant={textSize} styles={{ root: { margin: '0px', padding: '0px' } }} >{`CL ${hstep.change}`}</Text>}
                           </Stack>
                        </Stack>
                        <Stack style={{ paddingRight: 18 }}>
                           <Link className={"view-log-link"} to={viewLogURL} target="_blank">
                              <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
                                 <Icon styles={{ root: { margin: '0px', padding: '0px', paddingTop: "2px" } }} iconName="AlignLeft" className={hordeClasses.iconBlue} />
                                 <Text variant={textSize} styles={{ root: { margin: '0px', padding: '0px' } }} className="view-log-link">View Log</Text>
                              </Stack>
                           </Link>
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
         <Stack style={{ flexGrow: 0, flexShrink: 0, height: "100%" }}>
            {<ErrorPane events={logEvents} onClose={() => { }} />}
         </Stack>
      </div>
   }

   // 

   return null;
});


const IssuePanelNew: React.FC = () => {

   useEffect(() => {

      return () => {
         details.clear();
      };


   }, []);
   const issue = details.issue!;

   if (!issue) {
      return null;
   }


   return <Stack style={{ paddingLeft: 12 + 8, paddingRight: 12 + 8, paddingTop: 8, height: "100%" }} tokens={{ childrenGap: 12 }}>
      <IssueSummaryPanel />
      <Stack style={{ flexGrow: 1, flexShrink: 0 }}>
         <StreamCanvas />
      </Stack>
   </Stack>

}

const IssueCommandBar: React.FC = () => {

   const query = useQuery();

   const [assignToOtherShown, setAssignToOtherShown] = useState<{ shown?: boolean, defaultUser?: GetUserResponse }>({});

   const [markFixedShown, setMarkFixedShown] = useState<{shown?:boolean, autoFix?:boolean}>({shown: !!query.has("modify"), autoFix: !!query.has("markfix")});
   const [declineShown, setDeclineShown] = useState(false);
   const [ackShown, setAckShown] = useState(false);
   const [externalIssueLinkShown, setExternalIssueLinkShown] = useState(false);
   const [externalIssueCreateShown, setExternalIssueCreateShown] = useState(false);
   const [quarantineShown, setQuarantineShown] = useState(false);
   const [forceCloseShown, setForceCloseShown] = useState(false);
   const [testFixShown, setTestFixShown] = useState(false);

   const issue = details.issue!;

   if (!issue) {
      return null;
   }

   let assignedToText = "Unassigned";

   if (issue.ownerInfo?.name) {
      assignedToText = `Assigned To ${issue.ownerInfo.name}`;
      if (!issue.acknowledgedAt) {
         assignedToText += " (Pending)"
      }
   }

   type CommandItemData = {
      user?: GetUserResponse;
   };

   const renderCommandItem = (props: IContextualMenuItemProps) => {

      const data = props.item.data as (CommandItemData | undefined);

      if (data?.user?.image24) {
         return <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0 }}>
            <Persona styles={{ root: { paddingLeft: 2, maxWidth: 34, selectors: { ".ms-Persona-initials": { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } } } }} imageShouldFadeIn={false} imageUrl={data?.user?.image24} size={PersonaSize.size24} />
            <Text>{props.item.text}</Text>
         </Stack>
      }

      return <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0 }}>
         <Persona styles={{ root: { paddingLeft: 2, maxWidth: 34, selectors: { ".ms-Persona-initials": { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } } } }} imageShouldFadeIn={false} size={PersonaSize.size24} />
         <Text>{props.item.text}</Text>
      </Stack>

   }

   const assignItems: IContextualMenuItem[] = [];

   const commandItems: ICommandBarItemProps[] = [
      {
         key: 'assign_items',
         text: assignedToText,
         iconProps: { iconName: 'Person' },
         subMenuProps: {
            contextualMenuItemAs: renderCommandItem,
            items: assignItems
         }
      }
   ];

   assignItems.push({
      key: 'issue_suspects_me',
      text: dashboard.username,
      data: { user: dashboard.user },
      style: { fontSize: 12 },
      onClick: () => { setAssignToOtherShown({ shown: true, defaultUser: dashboard.user }); },
   })

   const suspects = details.suspects?.filter(u => u.name.toLowerCase() !== "buildmachine" && u.name.toLowerCase() !== "svc-p4-hordeproxy-p" && u.name.toLowerCase() !== "robomerge" && u.id !== dashboard.user?.id).slice(0, 5);

   // suspects
   if (suspects?.length && suspects?.length <= 5) {

      suspects.sort((a, b) => a.name.localeCompare(b.name)).forEach(suspect => {

         assignItems.push({
            key: `issue_suspect_${suspect.id}`,
            text: `${suspect.name}`,
            style: { fontSize: 11 },
            data: { user: suspect },
            onClick: () => { setAssignToOtherShown({ shown: true, defaultUser: suspect }); },
         })

      })
   }

   assignItems.push({
      key: 'show_assign_to_other',
      text: 'Assign To Other',
      onClick: () => { setAssignToOtherShown({ shown: true }); },
      style: { fontSize: 12 }
   });

   // build up status list
   const statusList: IContextualMenuItem[] = [{
      key: 'mark_fixed',
      iconProps: { iconName: issue.resolvedAt ? "IssueNew" : "IssueClosed" },
      text: issue.resolvedAt ? 'Clear Fixed' : "Mark Fixed",
      onClick: () => { setMarkFixedShown({shown: true, autoFix: true}); }
   }];

   const statusItems: ICommandBarItemProps[] = [
      {
         key: 'status_items',
         text: issue.resolvedAt ? "Resolved" : "Open",
         iconProps: { iconName: issue.resolvedAt ? "Endorsed" : "Pulse" },
         subMenuProps: {
            items: statusList
         }
      }
   ];

   const externalList: IContextualMenuItem[] = [];

   if (dashboard.externalIssueService) {

      if (issue.externalIssueKey) {
         externalList.push({
            key: 'external_issue_view',
            text: `View Issue`,
            iconProps: { iconName: "Share" },
            href: `${dashboard.externalIssueService.url}/browse/${issue.externalIssueKey}`,
            target: "_blank",
            style: { color: "unset" }
         });
      }

      externalList.push({
         key: 'external_issue_link',
         iconProps: { iconName: "Link" },
         text: `Link Issue`,
         onClick: () => { setExternalIssueLinkShown(true); }
      })

      if (details.getExternalIssueProjects().length) {
         externalList.push({
            key: 'external_issue_create',
            text: `Create New Issue`,
            iconProps: { iconName: "Add" },
            onClick: () => { setExternalIssueCreateShown(true); }
         });
      }

   };

   const externalItems: ICommandBarItemProps[] = [
      {
         key: 'external_items',
         text: issue.externalIssueKey ? issue.externalIssueKey : dashboard.externalIssueService?.name ?? "???",
         iconProps: { iconName: "Tag" },
         subMenuProps: {
            items: externalList
         }
      }
   ];

   const moreList: IContextualMenuItem[] = [
      {
         key: 'more_modify_issue',
         iconProps: { iconName: "Edit" },
         text: `Modify Issue`,
         style: { color: "unset" },
         onClick: () => { setMarkFixedShown({shown: true}); }
      },
      {
         key: 'more_history_link',
         iconProps: { iconName: "SearchTemplate" },
         text: `History`,
         href: `/audit/issue/${issue.id}`,
         target: "_blank",
         style: { color: "unset" }
      }, {
         key: 'more_quarantine_link',
         iconProps: { iconName: "Flash" },
         text: `Quarantine`,
         onClick: () => { setQuarantineShown(true); }
      }, {
         key: 'more_forceclose_link',
         iconProps: { iconName: "Delete" },
         text: `Force Close`,
         onClick: () => { setForceCloseShown(true); }
      }, {
         key: 'more_test_fix',
         iconProps: { iconName: "Wrench" },
         text: `Test Fix`,
         onClick: () => { setTestFixShown(true); }
      }];

   // Add AI Analysis link if user has BuildHealthViewAI permission (insert after Modify Issue)
   if (hasPluginPermission('buildhealth', 'BuildHealthViewAI')) {
      moreList.splice(1, 0, {
         key: 'more_ai_analysis',
         iconProps: { iconName: "PredictiveAnalysis" },
         text: `AI Analysis`,
         href: `/buildhealth/ai?issue=${issue.id}`,
         target: "_blank",
         style: { color: "unset" }
      });
   }

   const moreItems: ICommandBarItemProps[] = [
      {
         key: 'more_items',
         text: "More",
         iconProps: { iconName: "More" },
         subMenuProps: {
            items: moreList
         }
      }
   ];


   const canAck = !details.issue?.acknowledgedAt && details.issue?.ownerInfo?.id === dashboard.userId;

   const canDecline = !!details.suspects?.find(suspect => suspect.id === dashboard.userId);
   if (canDecline) {
      commandItems[0].subMenuProps?.items.push({
         key: 'decline_issue',
         text: 'Decline Issue',
         onClick: () => { setDeclineShown(true); },
         style: { fontSize: 12 }
      })
   }

   const suspectRange = details.getSuspectSwarmRange();

   const customClasses = getCustomClasses();


   return <Stack >
      {assignToOtherShown.shown && <AssignToOtherModal defaultUser={assignToOtherShown.defaultUser} onClose={() => { setAssignToOtherShown({}) }} />}
      {markFixedShown.shown && <MarkFixedModal autoFix={markFixedShown.autoFix} onClose={() => { setMarkFixedShown({}) }} />}
      {ackShown && <AckIssueModal onClose={() => { setAckShown(false) }} />}
      {declineShown && <DeclineIssueModal onClose={() => { setDeclineShown(false) }} />}
      {externalIssueLinkShown && <LinkExternalIssueModal onClose={() => { setExternalIssueLinkShown(false) }} />}
      {externalIssueCreateShown && <CreateExternalIssueModal onClose={() => { setExternalIssueCreateShown(false) }} />}
      {quarantineShown && <IssueQuarantineModal onClose={() => { setQuarantineShown(false) }} />}
      {forceCloseShown && <IssueForceCloseModal onClose={() => { setForceCloseShown(false) }} />}
      {testFixShown && <TestFixModal onClose={() => { setTestFixShown(false) }} />}
      <Stack horizontal>
         <Stack>
            <Stack styles={{ root: { paddingBottom: 0, paddingRight: 32 } }}>
               <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
                  <Stack horizontal style={{ paddingLeft: 12 }}>
                     <Stack horizontal verticalAlign="center" style={{ paddingRight: 18 }}>
                        {!!suspectRange && <a href={suspectRange} target="blank"> <Stack>
                           <CommandBarButton className={customClasses.actionBar} styles={{ root: { padding: "10px 8px 10px 8px" } }} iconProps={{ iconName: "Locate" }} text="Suspects" />
                        </Stack>
                        </a>}

                     </Stack>
                     <Stack horizontal verticalAlign="center" verticalFill={true} tokens={{ childrenGap: 18 }}>
                        <Stack>
                           <CommandBar styles={{ root: { height: 32, padding: 0, backgroundColor: "unset" } }}
                              className={customClasses.actionBar}
                              items={commandItems}
                              onReduceData={() => undefined} />
                        </Stack>
                        {canAck && <Stack className="horde-no-darktheme">
                           <PrimaryButton style={{ animationName: "red-pulse", animationDuration: "2s", animationIterationCount: "infinite", backgroundColor: "#FF0000", border: "1px solid #FF0000", padding: 0, paddingLeft: 4, paddingRight: 4, height: 22, fontWeight: "unset", fontSize: 12 }} text="Acknowledge" onClick={() => { setAckShown(true) }}></PrimaryButton>
                        </Stack>}
                        <Stack>
                           <CommandBar styles={{ root: { height: 32, padding: 0, backgroundColor: "unset" } }}
                              className={customClasses.actionBar}
                              items={statusItems}
                              onReduceData={() => undefined} />
                        </Stack>
                        {!!externalList.length && <Stack>
                           <CommandBar styles={{ root: { height: 32, padding: 0, backgroundColor: "unset" } }}
                              className={customClasses.actionBar}
                              items={externalItems}
                              onReduceData={() => undefined} />
                        </Stack>}
                        <Stack>
                           <CommandBar styles={{ root: { height: 32, padding: 0, backgroundColor: "unset" } }}
                              className={customClasses.actionBar}
                              items={moreItems}
                              onReduceData={() => undefined} />
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
}

export const IssueModalV2: React.FC<{ popHistoryOnClose: boolean, issueId?: string | null, streamId?: string, onCloseExternal?: () => void }> = observer(({ popHistoryOnClose, issueId, onCloseExternal, streamId }) => {

   const navigate = useNavigate();
   const query = useQuery();
   const location = useLocation();
   const [editShown, setEditShown] = useState(false);
   useWindowSize();

   const { hordeClasses, modeColors } = getHordeStyling();

   if (details.update) { }

   if (!issueId) {
      return null;
   }

   // clean this up, from old version of model, calling sites still need it
   let queryId: number | undefined;
   if (typeof (query.get("issue")) === 'string') {
      queryId = parseInt(query.get("issue")! as string);
      if (isNaN(queryId)) {
         queryId = undefined;
      }
   }

   const onClose = () => {
      if (queryId) {
         if (popHistoryOnClose) {
            navigate(-1);
         } else {
            let search = new URLSearchParams(location.search);
            search = new URLSearchParams(Array.from(search.entries()).filter(e => e[0] !== 'issue'));
            location.search = search.toString();
            navigate(location, { replace: true });
         }
      }
   }

   // subscribe
   if (details.update) { }

   if (details.issueError) {
      return <Dialog hidden={false} onDismiss={() => { details.clear(); if (onCloseExternal) { onCloseExternal() } else { onClose() } }} dialogContentProps={{
         type: DialogType.normal,
         title: 'Error Loading Issue',
         subText: `Unable to load issue ${issueId}`
      }}
         modalProps={{ styles: { main: { width: "640px !important", minWidth: "640px !important", maxWidth: "640px !important" } } }}>
         <DialogFooter>
            <PrimaryButton onClick={() => { details.clear(); if (onCloseExternal) { onCloseExternal() } else { onClose() } }} text="Ok" />
         </DialogFooter>
      </Dialog>
   }

   details.set(parseInt(issueId));

   let title = `Issue ${issueId}`;

   if (details.issue?.summary) {
      title += `: ${details.issue?.summary}`;
   }

   if (!details.valid) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, backgroundColor: modeColors.background, height: 180, hasBeenOpened: false, top: "24px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { if (onCloseExternal) { onCloseExternal() } else { onClose() } }}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8, paddingBottom: 24 } }}>
            <Stack grow verticalAlign="center" horizontalAlign="center" style={{ paddingTop: 12 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{`Loading Issue ${issueId}`}</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>
   }

   return <Modal isOpen={true} topOffsetFixed={true} isBlocking={true} styles={{ main: { padding: 8, width: 1420, backgroundColor: dashboard.darktheme ? `${modeColors.background} !important` : modeColors.background, hasBeenOpened: false, top: "24px", position: "absolute", height: "95vh" } }} className={hordeClasses.modal} onDismiss={() => { if (onCloseExternal) { onCloseExternal() } else { onClose() } }}>
      {editShown && <EditIssueModal onClose={() => { setEditShown(false) }} />}
      <Stack style={{ height: "93vh" }}>
         <Stack style={{ height: "100%" }}>
            <Stack style={{ flexBasis: "70px", flexShrink: 0 }}>
               <Stack horizontal styles={{ root: { padding: 8 } }} style={{ padding: 20, paddingBottom: 8 }}>
                  <Stack horizontal style={{ width: 1024 }} tokens={{ childrenGap: 24 }} verticalAlign="center" verticalFill={true}>
                     <Stack >
                        <Text styles={{ root: { fontWeight: "unset", maxWidth: 720, wordBreak: "break-word", fontFamily: "Horde Open Sans SemiBold", fontSize: "14px", textDecoration: details.issue?.resolvedAt ? "line-through" : undefined } }}>{title}</Text>
                     </Stack>

                     <Stack onClick={() => { setEditShown(true) }} style={{ cursor: "pointer" }}>
                        <Icon styles={{ root: { margin: '0px', padding: '0px', paddingTop: "0px" } }} iconName="Edit" className={hordeClasses.iconBlue} />
                     </Stack>
                     <Stack grow />
                  </Stack>
                  <Stack>
                     <IssueCommandBar />
                  </Stack>
                  <Stack grow />
                  <Stack horizontalAlign="end">
                     <IconButton
                        iconProps={{ iconName: 'Cancel' }}
                        ariaLabel="Close popup modal"
                        onClick={() => { if (onCloseExternal) { onCloseExternal() } else { onClose() } }}
                     />
                  </Stack>

               </Stack>
            </Stack>
            <Stack style={{ flexGrow: 1, flexShrink: 0 }}>
               <IssuePanelNew />
            </Stack>
         </Stack>
      </Stack>
   </Modal>
});


const AssignToOtherModal: React.FC<{ defaultUser?: GetUserResponse, onClose: () => void }> = ({ defaultUser, onClose }) => {

   const [state, setState] = useState<{ error?: string, submitting?: boolean, userId?: string, reopen?: boolean }>({});

   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;
   const issueId = issue.id;

   let headerText = `Assign Issue ${issueId}`;

   let showReopenOption = issue.resolvedAt ? true: false;

   const close = () => {
      onClose();
   };

   const onAssign = async () => {

      state.submitting = true;
      setState({ ...state });

      const userId = state.userId;

      if (userId && userId === issue.ownerInfo?.id) {
         state.submitting = false;
         state.error = `Already assigned to this user`;
         setState({ ...state });
         return;
      }

      const resolved = state.reopen ? false : undefined;
      const fixChange = state.reopen ? 0 : undefined;

      const request: UpdateIssueRequest = {
         nominatedById: (!userId || dashboard.userId === userId) ? null : dashboard.userId,
         ownerId: userId ? userId : "",
         acknowledged: dashboard.userId === userId ? true : false,
         resolved: resolved,
         fixChange: fixChange
      };

      try {

         await backend.updateIssue(issueId, request);
         state.submitting = false;
         state.error = "";
         setState({ ...state });
         details.refresh(issueId);
         close();

      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   let height = state.error ? 265 : 225;
   if (showReopenOption) {
      height += 35;
   }

   return <Modal isOpen={true} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { close() }}>

      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>
      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack grow>
            <Stack tokens={{ childrenGap: 4 }}>
               <Label>Assign To:</Label>
               <UserSelect handleSelection={(id => {
                  setState({ ...state, userId: id })
               })} userIdHints={issue.primarySuspectsInfo.map(s => s.id)} defaultUser={defaultUser} noResultsFoundText="No users found, default will be assigned" />
               <Text variant="small" style={{ paddingTop: 4 }}>Leave blank to assign to default owner</Text>
            </Stack>
         </Stack>
         
         {showReopenOption && <Stack styles={{ root: { paddingTop: 20} }}>
            <Checkbox label='Reopen issue?'
               defaultChecked={false}
               onChange={(ev, newValue) => {
                  setState({ ...state, reopen: !!newValue});
               }} />
         </Stack>}

         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack grow />
            <PrimaryButton text="Assign" disabled={state.submitting ?? false} onClick={() => { onAssign(); }} />
            <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};

const DeclineIssueModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const [state, setState] = useState<{ error?: string, submitting?: boolean }>({});

   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;
   const issueId = issue.id;

   let headerText = `Decline Issue`;

   const close = () => {
      onClose();
   };

   const onDecline = async () => {

      state.submitting = true;
      setState({ ...state });

      try {

         const request: UpdateIssueRequest = {
            declined: true
         }

         if (issue.ownerInfo?.id === dashboard.userId) {
            request.ownerId = "";
            request.acknowledged = false;
            request.resolved = false;
            request.fixChange = 0;
         }

         await backend.updateIssue(issueId, request);

         state.submitting = false;
         state.error = "";
         setState({ ...state });
         details.refresh(issueId);
         close();

      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   const height = state.error ? 180 : 140;

   return <Modal isOpen={true} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { close() }}>

      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>
      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack grow />
            <PrimaryButton text="Decline" disabled={state.submitting ?? false} onClick={() => { onDecline(); }} />
            <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};

const AckIssueModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const [state, setState] = useState<{ error?: string, submitting?: boolean }>({});

   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;
   const issueId = issue.id;

   let headerText = `Acknowledge Issue`;

   const close = () => {
      onClose();
   };

   const onAck = async () => {

      state.submitting = true;
      setState({ ...state });

      try {

         const request: UpdateIssueRequest = {
            acknowledged: true
         }

         await backend.updateIssue(issueId, request);

         state.submitting = false;
         state.error = "";
         setState({ ...state });
         details.refresh(issueId);
         close();

      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   const height = state.error ? 180 : 140;

   return <Modal isOpen={true} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { close() }}>

      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>
      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack grow />
            <PrimaryButton text="Acknowledge" disabled={state.submitting ?? false} onClick={() => { onAck(); }} />
            <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};

// @todo: Make these server configurable 
const rcaCategoryOptions: IDropdownOption[] = [
   { key: 'rca_asset_content', text: 'Asset/Content' },
   { key: 'rca_code', text: 'Code' },
   { key: 'rca_ui', text: 'UI/Localization' },
   { key: 'rca_deployment', text: 'Build/Deployment/Version' },
   { key: 'rca_data', text: 'Configuration/Data' },
   { key: 'rca_gameplay', text: 'Gameplay/Engine' },
   { key: 'rca_backend', text: 'Backend/Server/Network' },
   { key: 'rca_monitoring', text: 'Monitoring/No Code Change' },
   { key: 'rca_use', text: 'Unintended Side Effect' },
   { key: 'rca_buildagent', text: 'Build Agent' },
   { key: 'rca_changebackout', text: 'Change Backout' },
   { key: 'rca_other', text: 'Other' },
   { key: 'rca_clear', text: 'Clear Selection' }
];

const MarkFixedModal: React.FC<{ onClose: () => void, autoFix?:boolean } > = ({ onClose, autoFix }) => {

   const issue = details.issue!;

   const [state, setState] = useState<{ error?: string, fixCL?: number, rootCommitId?: CommitId, rootCauseOwnerId?: string, rootCauseSummary?: string, rootCauseCategory?: string, submitting?: boolean, dupeCheck?: boolean, dupeId?: string }>({
      fixCL: issue.fixChange ?? (autoFix ? 0 : undefined),
      rootCommitId: issue.rootCommitId,
      dupeId: issue.duplicateIssueId ? (issue.duplicateIssueId.toString() !== "0" ? issue.duplicateIssueId.toString() : undefined) : undefined,
      dupeCheck: issue.duplicateIssueId === 0 || !!issue.duplicateIssueId,
      rootCauseOwnerId: issue.rootCauseOwnerInfo?.id,
      rootCauseCategory: issue.rootCauseCategory,
      rootCauseSummary: issue.rootCauseSummary
   });

   const navigate = useNavigate();

   const { hordeClasses } = getHordeStyling();


   const issueId = issue.id;

   const headerText = `Modify Issue ${issueId}`;

   const close = () => {

      const url = new URL(window.location.href);
      if (url.searchParams.has("modify")) {
         url.searchParams.delete("modify");
         url.searchParams.delete("markfix");
         navigate(url.pathname + url.search, { replace: true });
      }
      onClose();
   };

   const onUpdate = async (clearFixed?: boolean) => {

      state.submitting = true;
      setState({ ...state });

      let fixCL: number | null | undefined;
      if (clearFixed) {
         fixCL = -1;
      } else {
         fixCL = state.fixCL;
      }

      // Root Cause Analysis
      let rootCommitId = state.rootCommitId?.length ? state.rootCommitId : null;
      const rootCauseOwnerId = state.rootCauseOwnerId?.length ? state.rootCauseOwnerId : issue.rootCauseOwnerInfo?.id?.length ? "" : null;
      const rootCauseCategory = state.rootCauseCategory?.length ? state.rootCauseCategory : "";
      const rootCauseSummary = state?.rootCauseSummary?.trim().length ? state.rootCauseSummary.trim() : "";

      let dupeId = state.dupeCheck ? 0 : -1;
      if (dupeId === 0 && state.dupeId) {
         dupeId = parseInt(state.dupeId)
         if (dupeId === issueId) {
            dupeId = -1;
         }
      }

      let summary: string | undefined;

      if (dupeId >= 0) {
         if (!issue.summary.startsWith("(Dupe) ")) {
            summary = "(Dupe) " + issue.summary;
         }
      } else if (issue.summary.startsWith("(Dupe) ")) {
         summary = issue.summary.replace("(Dupe) ", "");
      }
      
      const request: UpdateIssueRequest = {
         fixChange: fixCL,
         summary: summary,
         rootCommitId: rootCommitId ?? "",
         rootCauseOwnerId: rootCauseOwnerId,
         rootCauseCategory: rootCauseCategory,
         rootCauseSummary: rootCauseSummary,
         //ownerId: !issue.ownerInfo ? dashboard.userId : issue.ownerInfo.id,
         //acknowledged: !issue.ownerInfo || issue.ownerInfo?.id === dashboard.userId ? true : undefined,
         resolved: clearFixed ? false : (fixCL != undefined && fixCL >= 0),
         duplicateIssueId: dupeId
      }

      try {
         await backend.updateIssue(issueId, request);

         state.submitting = false;
         state.error = "";
         setState({ ...state });
         details.refresh(issueId);
         close();

      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   let height = state.error ? 300 : 260;

   height += 384;

   const selectedCatKey = rcaCategoryOptions.find(c => c.text === state.rootCauseCategory)?.key ?? null;

   const disableRoot = !!state.dupeCheck;

   // Unique set of users that are most likely to be the RCA owner
   const rcaOwnerCandidates = [... new Set([ dashboard.userId, issue.rootCauseOwnerInfo?.id ?? null, issue.ownerInfo?.id ?? null, issue.resolvedByInfo?.id ?? null])];


   return <Modal className={hordeClasses.modal} isOpen={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}

      <Stack style={{ paddingLeft: 8, paddingRight: 8 }}>
         <Stack styles={{ root: { paddingLeft: 8, width: 500, paddingTop: 12 } }}>
            <CommitIdField label="Fix Change" placeholder="Change which fixed this issue. Enter 0 to mark as fixed, without a change" defaultCommitId={state.fixCL?.toString()} onChange={(commitId => { 
               setState({ ...state, fixCL: commitId ? parseInt(commitId) : undefined }) })} />
         </Stack>

         <Stack styles={{ root: { paddingLeft: 8, width: 500, paddingTop: 12 } }}>
            <Stack style={{ paddingLeft: 8, width: 500, paddingBottom: 12 }}>
               <Separator />
            </Stack>
            <Stack tokens={{ childrenGap: 16 }}>
               <Stack>
                  <Label>Root Cause Owner</Label>
                  <UserSelect defaultId={state.rootCauseOwnerId} handleSelection={(id => {
                     setState({ ...state, rootCauseOwnerId: id })
                  })} userIdHints={rcaOwnerCandidates.filter(c => c !== null)} noResultsFoundText="No users found" />
               </Stack>
               <Stack>
                  <CommitIdField disabled={disableRoot} label="Root Cause Change" defaultCommitId={state.rootCommitId} placeholder="Change which caused this issue, please only enter if certain on root cause" onChange={(commitId => { setState({ ...state, rootCommitId: commitId }) })} />
               </Stack>
               <Stack>
                  <Dropdown
                     disabled={disableRoot}
                     selectedKey={selectedCatKey}
                     placeholder="Select an option"
                     label="Root Cause Category (required)"
                     options={rcaCategoryOptions}
                     onChange={(event, option) => {
                        if (option?.key === "rca_clear") {
                           option = undefined;
                        }
                        setState({ ...state, rootCauseCategory: option?.text as string | undefined })
                     }}
                  />
               </Stack>
               <Stack>
                  <TextField disabled={disableRoot} spellCheck={false} autoComplete="off" label="Root Cause Summary" placeholder="Please enter a brief summary describing the root cause of this issue and any suggestions to prevent it." value={state.rootCauseSummary} style={{ minHeight: 120, maxHeight: 120 }} multiline resizable={false} onChange={(ev, newValue) => {
                     ev.preventDefault();
                     setState({ ...state, rootCauseSummary: newValue ? newValue : "" })
                  }} />
               </Stack>

               <Stack horizontal verticalFill verticalAlign="center" tokens={{ childrenGap: 24 }}>
                  <Checkbox label="Duplicate Issue" checked={state.dupeCheck} onChange={(_, checked) => setState({ ...state, dupeCheck: checked })} />
                  <TextField style={{width: 352}} disabled={!state.dupeCheck} spellCheck={false} autoComplete="off" placeholder="Optional Issue ID of the source issue this issue duplicates" value={state.dupeId} onChange={(ev, newValue) => {
                     ev.preventDefault();

                     newValue = newValue?.trim();

                     // support putting in a link
                     if (newValue?.startsWith("http")) {
                        const lastIndex = newValue.lastIndexOf("/");
                        const issueId = parseInt( newValue.slice(lastIndex + 1));
                        if (typeof(issueId) === "number" && !isNaN(issueId)) {
                           newValue = issueId.toString();
                        }
                     }
                     setState({ ...state, dupeId: newValue ? newValue?.trim() : "" })
                  }} />
               </Stack>

            </Stack>
         </Stack>

         <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack horizontal>
               <Stack>
                  {!!issue.resolvedAt && <DefaultButton text="Clear Fixed" disabled={state.submitting} onClick={() => { onUpdate(true); }} />}
               </Stack>
               <Stack grow />
               <Stack>
                  <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                     <PrimaryButton text="Update" disabled={state.submitting ?? false} onClick={() => { onUpdate(); }} />
                     <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { close(); }} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;
}

const LinkExternalIssueModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const issue = details.issue!;
   const issueId = issue.id;

   const [state, setState] = useState<{ error?: string, issueKey?: string, submitting?: boolean }>({ issueKey: issue.externalIssueKey });
   const { hordeClasses } = getHordeStyling();

   const externalIssueProvider = dashboard.externalIssueService?.name;

   const headerText = `${externalIssueProvider}: Link Issue`;

   const close = () => {
      onClose();
   };

   const onSetExternalIssueLink = async (unlink?: boolean) => {

      state.submitting = true;
      setState({ ...state });

      try {

         let issueKey = state.issueKey;

         if (state.issueKey && state.issueKey.toLowerCase().startsWith("http")) {
            issueKey = "";
            try {
               const url = new URL(state.issueKey);
               if (url.pathname) {
                  const path = url.pathname.split("/");
                  if (path.length > 1) {
                     issueKey = path[path.length - 1];
                  }
               }
            } catch {
               issueKey = "";
            }
         }


         // catch invalid issue key, tracking https://jira.it.epicgames.com/browse/UE-153911 for server side validation
         if (issueKey) {
            const issueKeyRegex = /^[A-Z]{2,}-\d+$/;
            if (!issueKey.match(issueKeyRegex)) {
               state.submitting = false;
               state.error = `Please provide a valid ${externalIssueProvider} issue key, for example: UE-12345 or ${dashboard.externalIssueService?.url}/browse/UE-12345`;
               setState({ ...state });
               return;
            }
         }


         if (!unlink && !issueKey) {
            state.submitting = false;
            state.error = `Please provide a valid ${externalIssueProvider} issue key, for example: UE-12345 or ${dashboard.externalIssueService?.url}/browse/UE-12345`;
            setState({ ...state });

         } else {

            await backend.updateIssue(issueId, {
               externalIssueKey: unlink ? "" : issueKey
            });

            state.submitting = false;
            state.error = "";
            setState({ ...state });
            details.refresh(issueId);
            close();

         }
      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   const height = state.error ? 250 : 210;

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}


      <Stack styles={{ root: { paddingLeft: 8, width: 500, paddingTop: 12 } }}>
         <TextField label={`${externalIssueProvider} Issue Key`} placeholder={`Example: UE-12345 or ${dashboard.externalIssueService?.url}/browse/UE-12345`} autoComplete="off"
            value={state.issueKey ?? ""}
            onChange={(ev, newValue) => {
               ev.preventDefault();
               setState({ ...state, issueKey: newValue?.trim() ?? "" })
            }} />
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack>
               {!!issue.externalIssueKey && <DefaultButton text="Unlink Issue" disabled={state.submitting} onClick={() => { onSetExternalIssueLink(true); }} />}
            </Stack>
            <Stack grow />
            <Stack>
               <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <PrimaryButton text="Link Issue" disabled={state.submitting ?? false} onClick={() => { onSetExternalIssueLink(); }} />
                  <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { close(); }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;
}

const CreateExternalIssueModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const [state, setState] = useState<{ error?: string, newDesc?: string, newSummary?: string, submitting?: boolean, externalIssue?: CreateExternalIssueResponse, projectName?: string, componentName?: string, issueType?: string }>({ newSummary: details.issue?.summary, newDesc: details.issue!.description });
   const { hordeClasses, modeColors } = getHordeStyling();

   const externalIssueProvider = dashboard.externalIssueService?.name;
   const issue = details.issue!;
   const issueId = issue.id;

   const projects = details.getExternalIssueProjects();

   if (!state.projectName || !state.issueType) {
      let projectName = state.projectName;
      if (!projectName) {
         projectName = projects[0].name;
      }

      const project = projects.find(p => p.name === projectName)!;

      let componentName: string | undefined;
      for (let id in project.components) {
         if (project.components[id] === state.componentName) {
            componentName = state.componentName;
            break;
         }
      }

      let issueType = state.issueType;
      if (!issueType) {
         let issueTypes: string[] = [];
         for (let id in project.issueTypes) {
            issueTypes.push(project.issueTypes[id]);
         }
         issueType = issueTypes.sort((a, b) => a.localeCompare(b))[0];
      }

      setState({ ...state, projectName: projectName, componentName: componentName, issueType: issueType });
      return null;
   }


   const headerText = `${externalIssueProvider}: Create and Link Issue`;

   const close = () => {
      onClose();
   };

   const onEditSummary = async () => {

      let error = "";

      const summary = state.newSummary?.trim();
      if (!summary) {
         error = "Please provide an issue summary";
      }

      const issueTypeId = details.getExternalProjectIssueTypeId(state.projectName, state.issueType);
      if (!issueTypeId) {
         error = "Unable to get issue type id";
      }

      const componentId = details.getExternalProjectComponentId(state.projectName, state.componentName);
      if (!componentId) {
         error = state.componentName ? "Unable to get component id for issue" : "Please select a component for the issue";
      }

      const projectId = details.getExternalProjectId(state.projectName);
      if (!projectId) {
         error = "Unable to get project id for issue";
      }

      // find a stream the user has permission on
      let streamId: string | undefined;
      if (details.issueStreams) {
         const stream = details.issueStreams.find(istream => {
            if (projectStore.streamById(istream.streamId)) {
               return true;
            }
            return false;
         });

         if (stream) {
            streamId = stream.streamId;
         }
      }

      if (!streamId) {
         error = "Unable to get stream id for issue";
      }


      if (error) {
         state.error = error;
         setState({ ...state });
         return;
      }

      state.submitting = true;
      setState({ ...state });

      try {

         const result = await backend.createExternalIssue({
            issueId: issueId,
            summary: summary!,
            projectId: projectId!,
            streamId: streamId!,
            componentId: componentId!,
            issueTypeId: issueTypeId!,
            description: state.newDesc,
            hordeIssueLink: window.location.href
         });

         state.submitting = false;
         state.error = "";

         let shouldClose = false;
         if (!result.key) {
            state.error = `Unable to create ${externalIssueProvider} issue, no key returned`;
            setState({ ...state });
         } else {
            if (result.link) {
               state.externalIssue = result;
               setState({ ...state });
            } else {
               shouldClose = true;
            }
         }

         details.refresh(issueId);
         if (shouldClose) {
            onClose();
         }


      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   if (state.submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, backgroundColor: modeColors.background, height: 180, hasBeenOpened: false, top: "24px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8, paddingBottom: 24 } }}>
            <Stack grow verticalAlign="center" horizontalAlign="center" style={{ paddingTop: 12 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{`Creating ${externalIssueProvider} Issue, One Moment Please...`}</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>
   }

   if (state.externalIssue) {

      const markdown = `${externalIssueProvider} issue [${state.externalIssue.link}](${state.externalIssue.link}) was created successfully.  \n\nPlease visit the new issue link to customize any necessary fields.`

      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, backgroundColor: modeColors.background, hasBeenOpened: false, top: "24px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal styles={{ root: { paddingLeft: 12 } }}>
               <Stack grow style={{ paddingTop: 12, paddingLeft: 8 }}>
                  <Label style={{ fontSize: 18, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }}>{`${externalIssueProvider} Issue ${state.externalIssue.key} Created`}</Label>
               </Stack>
               <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     ariaLabel="Close popup modal"
                     onClick={() => { close(); }}
                  />
               </Stack>
            </Stack>

            <Stack tokens={{ childrenGap: 32 }} style={{ paddingLeft: 8 }}>

               <Stack horizontal>
                  <Stack style={{ minWidth: 600 }}>
                     <Markdown styles={{ root: { paddingLeft: 24, paddingRight: 24, overflow: "auto", th: { fontSize: 12 } } }}>{markdown}</Markdown>
                  </Stack>
               </Stack>

            </Stack>

            <Stack styles={{ root: { padding: 12 } }}>
               <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
                  <Stack grow />
                  <PrimaryButton text="Ok" disabled={false} onClick={() => { close(); }} />
               </Stack>
            </Stack>
         </Stack>

      </Modal>
   }

   const projectMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: projects.map(p => {
         return {
            key: `external_project_id_${p.id}`,
            text: p.name,
            onClick: () => setState({ ...state, projectName: p.name, componentName: undefined, issueType: undefined })
         }
      })
   };

   const cproject = projects.find(p => p.name === state.projectName)!;

   type ComponentPickerItem = {
      key: string;
      name: string;
   }

   let components: string[] = [];
   for (let id in cproject.components) {
      components.push(cproject.components[id]);
   }

   components = components.sort((a, b) => a.localeCompare(b));

   const componentPickerItems: ComponentPickerItem[] = components.map(c => {
      return {
         key: `component_picker_item_${c}`,
         name: c
      }
   })


   let issueTypeItems: IContextualMenuItem[] = [];
   for (let id in cproject.issueTypes) {
      const iname = cproject.issueTypes[id];
      issueTypeItems.push({
         key: `external_issuetype_id_${id}`,
         text: iname,
         onClick: () => setState({ ...state, issueType: iname })
      });
   }

   issueTypeItems = issueTypeItems.sort((a, b) => a.text!.localeCompare(b.text!));

   const issueTypeMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: issueTypeItems
   };


   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, isVisible: true, hasBeenOpened: true, top: "24px", position: "absolute" } }} onDismiss={() => { close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack style={{ padding: 8 }} tokens={{ childrenGap: 18 }}>

         {!!state.error && <MessageBar
            messageBarType={MessageBarType.error}
            isMultiline={false}> {state.error} </MessageBar>}
         <Stack >
            <TextField label={`${externalIssueProvider} Issue Summary`}
               value={state.newSummary ?? ""}
               onChange={(ev, newValue) => {
                  ev.preventDefault();
                  setState({ ...state, newSummary: newValue ? newValue : "" })
               }} />
         </Stack>
         <Stack horizontal tokens={{ childrenGap: 24 }}>
            <Stack>
               <Label>Project</Label>
               <DefaultButton style={{ textAlign: "left" }} menuProps={projectMenuProps} text={state.projectName} />
            </Stack>
            <Stack>
               <Label>Issue Type</Label>
               <DefaultButton style={{ textAlign: "left" }} menuProps={issueTypeMenuProps} text={state.issueType} />
            </Stack>
            <Stack grow>
               <Stack>
                  <Label>Component</Label>
                  <TagPicker
                     itemLimit={1}
                     onResolveSuggestions={(filter, selected) => {
                        return componentPickerItems.filter(i => {
                           return !selected?.find(s => i.key === s.key) && i.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1;
                        });
                     }}

                     onEmptyResolveSuggestions={(selected) => {
                        return componentPickerItems.filter(i => {
                           return !selected?.find(s => i.key === s.key);
                        });
                     }}

                     selectedItems={state.componentName ? [{ key: `component_picker_item_${state.componentName!}`, name: state.componentName! }] : []}

                     onChange={(items?) => {

                        if (items?.length) {
                           setState({ ...state, componentName: items[0].name });
                        } else {
                           setState({ ...state, componentName: undefined });
                        }

                     }}
                  />
               </Stack>
            </Stack>
         </Stack>
         <Stack>

            <Stack>
               <Label>Description</Label>
               <TextField placeholder="Please enter optional description text." value={state.newDesc} style={{ minHeight: 240 }} multiline resizable={false} onChange={(ev, newValue) => {
                  ev.preventDefault();
                  setState({ ...state, newDesc: newValue ? newValue : "" })
               }} />
            </Stack>

         </Stack>

      </Stack>

      <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack grow />
         <PrimaryButton text="Create Issue" disabled={state.submitting ?? false} onClick={() => { onEditSummary(); }} />
         <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
      </Stack>
   </Modal>;
}

const EditIssueModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const [state, setState] = useState<{ error?: string, newDesc?: string, newSummary?: string, submitting?: boolean }>({ newSummary: details.issue?.summary, newDesc: details.issue!.description });
   const [showPreview, setShowPreview] = useState(false);

   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;
   const issueId = issue.id;

   const headerText = `Edit Issue`;

   const close = () => {
      onClose();
   };

   const onEditSummary = async () => {

      state.submitting = true;
      setState({ ...state });

      try {

         const summary = state.newSummary?.trim();

         if (!summary) {
            state.submitting = false;
            state.error = "Please provide an issue summary";
            setState({ ...state });

         } else {

            await backend.updateIssue(issueId, {
               summary: summary,
               description: state.newDesc ? state.newDesc : ""
            });

            state.submitting = false;
            state.error = "";
            setState({ ...state });
            details.refresh(issueId);
            close();

         }
      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   const pivotItems: JSX.Element[] = [];

   pivotItems.push(<PivotItem headerText={"Markdown"} itemKey={`preview_pivot_false`} key={`_preview_pivot_false`} headerButtonProps={{ onClick: () => { setShowPreview(false) } }} />);
   pivotItems.push(<PivotItem headerText={"Preview"} itemKey={`preview_pivot_true`} key={`_preview_pivot_true`} headerButtonProps={{ onClick: () => { setShowPreview(true) } }} />);

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 640, isVisible: true, hasBeenOpened: true, top: "80px", position: "absolute" } }} onDismiss={() => { close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack style={{ padding: 8 }} tokens={{ childrenGap: 18 }}>

         {!!state.error && <MessageBar
            messageBarType={MessageBarType.error}
            isMultiline={false}> {state.error} </MessageBar>}


         <Stack >
            <TextField label="Summary"
               value={state.newSummary ?? ""}

               onChange={(ev, newValue) => {
                  ev.preventDefault();

                  setState({ ...state, newSummary: newValue ? newValue : "" })
               }} />
         </Stack>

         <Stack>
            <Stack style={{ paddingBottom: 8 }}>
               <Pivot className={hordeClasses.pivot}
                  selectedKey={showPreview ? "preview_pivot_true" : "preview_pivot_false"}
                  linkSize="normal"
                  linkFormat="links"
                  onLinkClick={(item => {
                     item!.props.itemKey === "preview_pivot_true" ? setShowPreview(true) : setShowPreview(false)
                  })}>
                  {pivotItems}
               </Pivot>
            </Stack>


            {!showPreview && <Stack /*className={multilineStyle.multiline}*/>
               <TextField placeholder="Please enter description text." value={state.newDesc} style={{ minHeight: 320, maxHeight: 320 }} multiline resizable={false} onChange={(ev, newValue) => {
                  ev.preventDefault();
                  setState({ ...state, newDesc: newValue ? newValue : "" })
               }} />
            </Stack>}

            {showPreview && <Stack style={{ paddingLeft: 12, paddingTop: 4, paddingBottom: 4, backgroundColor: dashboard.darktheme ? "#2B2D2E" : undefined }}>
               <Markdown styles={{ root: { minHeight: 320, maxHeight: 320, overflow: "auto", th: { fontSize: 12 } } }}>{state.newDesc ?? ""}</Markdown>
            </Stack>}

         </Stack>

      </Stack>

      <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack grow />
         <PrimaryButton text="Save" disabled={state.submitting ?? false} onClick={() => { onEditSummary(); }} />
         <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
      </Stack>
   </Modal>;

}

const IssueQuarantineModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const issue = details.issue!;
   const issueId = issue.id;

   const [state, setState] = useState<{ submitting?: boolean, quarantine?: boolean, error?: string }>({ quarantine: !!issue.quarantinedByUserInfo });

   const { hordeClasses } = getHordeStyling();

   const onSave = async () => {

      setState({ ...state, submitting: true });

      try {

         await backend.updateIssue(issueId, {
            quarantinedById: state.quarantine ? dashboard.userId : ""
         });

         details.refresh(issueId).finally(() => {
            setState({ ...state, submitting: false, error: "" });
            onClose();
         })

      } catch (reason) {

         setState({ ...state, submitting: false, error: reason as string });
      }

   };


   const height = state.error ? 200 : 180;

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { onClose() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">Quarantine Issue</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { paddingLeft: 8, width: 540, paddingTop: 12 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            {!!state.error && <MessageBar
               messageBarType={MessageBarType.error}
               isMultiline={false}> {state.error} </MessageBar>}

            <Stack>
               <Checkbox label={!issue.quarantinedByUserInfo ? "Quarantine Issue" : `Quarantined by ${issue.quarantinedByUserInfo.name} on ${getShortNiceTime(issue.quarantineTimeUtc)}`}
                  defaultChecked={state.quarantine}
                  onChange={(ev, newValue) => {
                     setState({ ...state, quarantine: !!newValue });
                  }} />
            </Stack>
         </Stack>
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack>
            </Stack>
            <Stack grow />
            <Stack>
               <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <PrimaryButton text="Save" disabled={state.submitting ?? false} onClick={() => { onSave() }} />
                  <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { onClose(); }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;

};


const IssueForceCloseModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const issue = details.issue!;
   const issueId = issue.id;

   const [state, setState] = useState<{ submitting?: boolean, forceClosed?: boolean, error?: string }>({ forceClosed: !!issue.forceClosedByUserInfo });

   const { hordeClasses } = getHordeStyling();

   const onSave = async () => {

      setState({ ...state, submitting: true });

      try {

         await backend.updateIssue(issueId, {
            forceClosedById: state.forceClosed ? dashboard.userId : ""
         });

         details.refresh(issueId).finally(() => {
            setState({ ...state, submitting: false, error: "" });
            onClose();
         })

      } catch (reason) {

         setState({ ...state, submitting: false, error: reason as string });
      }

   };


   const height = state.error ? 260 : 240;

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { onClose() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">Force Close Issue</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { paddingLeft: 18, paddingRight: 18, width: 540, paddingTop: 12 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            {!!state.error && <MessageBar
               messageBarType={MessageBarType.error}
               isMultiline={false}> {state.error} </MessageBar>}
            <Stack>
               <Text variant="medium">Warning: Force closing issues without verification is not regular workflow.  It is intended for special cases, such as steps being removed from a job.  Please proceed with caution.</Text>
            </Stack>
            <Stack>
               <Checkbox disabled={!!issue.forceClosedByUserInfo} label={!issue.forceClosedByUserInfo ? "Force Close Issue" : `Force Closed by ${issue.forceClosedByUserInfo.name}`}
                  defaultChecked={state.forceClosed}
                  onChange={(ev, newValue) => {
                     setState({ ...state, forceClosed: !!newValue });
                  }} />
            </Stack>
         </Stack>
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack>
            </Stack>
            <Stack grow />
            <Stack>
               {!!issue.forceClosedByUserInfo && <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <DefaultButton text="Ok" disabled={state.submitting} onClick={() => { onClose(); }} />
               </Stack>}
               {!issue.forceClosedByUserInfo && <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <PrimaryButton text="Save" disabled={state.submitting ?? false} onClick={() => { onSave() }} />
                  <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { onClose(); }} />
               </Stack>}
            </Stack>
         </Stack>
      </Stack>
   </Modal>;

};

const TestFixModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const navigate = useNavigate();
   const [state, setState] = useState<{ loadingTemplates?: boolean, templates?: Map<string, GetTemplateRefResponse[]>, error?: string, submitting?: boolean, shelvedCL?: string, baseCL?: string, streamId?: string, templateId?: string, target?: string, updateIssues?: boolean }>({ updateIssues: true });
   const { hordeClasses } = getHordeStyling();

   const issue = details.issue!;

   const testStreams: TestStream[] = [];

   issue.affectedStreams.forEach(stream => {
      stream.templateIds?.forEach(id => {
         let testStream = testStreams.find(s => s.streamId === stream.streamId);
         if (!testStream) {
            testStream = { streamId: stream.streamId, templateIds: [], resolvedIds: [...stream.resolvedTemplateIds] };
            testStreams.push(testStream);
         }
         if (!testStream.templateIds.find(tid => tid === id)) {
            testStream.templateIds.push(id);
         }
      });
   });

   if (!state.templates) {

      if (!state.loadingTemplates) {

         const getTemplates = async () => {

            const templateMap: Map<string, GetTemplateRefResponse[]> = new Map();

            for (let i = 0; i < testStreams.length; i++) {

               const testStream = testStreams[i];
               const stream = projectStore.streamById(testStream.streamId);

               if (stream) {
                  const templates = await templateCache.getStreamTemplates(stream);
                  templateMap.set(testStream.streamId, templates);
               }
            }

            setState({ ...state, templates: templateMap, loadingTemplates: undefined })
         }

         getTemplates();
         setState({ ...state, loadingTemplates: true })

      }

      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Loading Templates</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>

   }

   if (state.submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Creating Job</Text>
            </Stack>
            <Stack horizontalAlign="center">
               <Text variant="mediumPlus">The job is being created and will be available soon.</Text>
            </Stack>
            <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
               <Spinner size={SpinnerSize.large} />
            </Stack>

         </Stack>
      </Modal>
   }

   const headerText = `Test Fix`;

   const onTestFix = async (unlink?: boolean) => {

      let errorReason: any = undefined;

      setState({ ...state, submitting: true });

      let change: number | undefined;

      if (state.shelvedCL) {
         change = state.baseCL ? parseInt(state.baseCL) : undefined;
      }

      const preflightChange = state.shelvedCL ? parseInt(state.shelvedCL) : undefined;

      let error = "";
      if (change !== undefined && isNaN(change)) {
         error = "Invalid Base CL";
      }
      if (preflightChange !== undefined && isNaN(preflightChange)) {
         error = "Invalid Shelved CL";
      }

      if (!state.streamId) {
         error = "Stream id not set";
      }

      if (!state.templateId) {
         error = "Template id not set";
      }

      if (!state.target) {
         error = "Target not set";
      }

      const template = state.templates?.get(state.streamId!)?.find(t => t.id === state.templateId);
      if (!template) {
         error = `Unable to find stream ${state.streamId} template ${state.templateId}`;
         console.log(state.templates?.get(state.streamId!));
      }

      if (template?.allowPreflights === false && preflightChange) {
         error = `Template ${template.name} does not allow preflights`;
      }

      if (error) {
         setState({ ...state, error: error });
         return;
      }

      try {

         let args = template?.arguments?.filter(a => !a.toLowerCase().trim().startsWith("-target=")) ?? [];
         args.unshift(`-Target=${state.target}`);

         setState({ ...state, submitting: true, error: undefined });

         const data: CreateJobRequest = {
            arguments: args,
            streamId: state.streamId!,
            templateId: state.templateId!,
            preflightChange: preflightChange ? preflightChange : undefined,
            change: change ? change : undefined,
            updateIssues: !preflightChange ? true : undefined
         };

         const submit = async () => {

            let redirected = false;

            await backend.createJob(data).then(async (data) => {

               console.log(`Job created: ${JSON.stringify(data)}`);
               console.log("Updating notifications")
               try {
                  await backend.updateNotification({ slack: true }, "job", data.id);
               } catch (reason) {
                  console.log(`Error on updating notifications: ${reason}`);
               }

               redirected = true;
               navigate(`/job/${data.id}`);

            }).catch(reason => {
               // "Not Found" is generally a permissions error
               errorReason = reason ? reason : "Unknown";
               console.log(`Error on job creation: ${errorReason}`);

            }).finally(() => {

               if (!redirected) {

                  setState({ ...state, submitting: false });

                  if (errorReason) {

                     errorDialogStore.set({

                        reason: `${errorReason}`,
                        title: `Error Creating Job`,
                        message: `There was an issue creating the job.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`

                     }, true);

                  }
               }
            });
         }

         await submit();

      } catch (reason) {

         state.submitting = false;
         state.error = reason as string;
         setState({ ...state });
      }

   };

   type TestStream = {
      streamId: string,
      templateIds: string[],
      resolvedIds: string[]
   }

   const streamItems: IContextualMenuItem[] = [];;
   testStreams.forEach(testStream => {

      const stream = projectStore.streamById(testStream.streamId);

      if (!stream) {
         return;
      }

      const issueStream = details!.issueStreams?.find(s => s.streamId === testStream.streamId)!;
      const templates = testStream.templateIds.map(id => stream.templates.find(t => t.id === id)).filter(t => !!t) as GetTemplateRefResponse[];

      const subItems: IContextualMenuItem[] = [];

      templates.sort((a, b) => a.name < b.name ? -1 : 1).forEach(t => {

         issueStream.nodes.sort((a, b) => a.name.localeCompare(b.name)).forEach(node => {

            if (node.templateId !== t.id) {
               return;
            }

            let name = t.name + " / " + node.name;

            if (testStream.resolvedIds.find(id => id === t.id)) {
               name += " (Resolved)";
            }

            subItems.push({ key: `template_choice_${t.id}`, data: t.id, text: name, onClick: () => { setState({ ...state, streamId: stream.id, templateId: t.id, target: node.name }) } });

         });

      });

      streamItems.push({ key: `test_stream_${testStream.streamId}`, data: testStream.streamId, text: stream.fullname, subMenuProps: { items: subItems } });

   });

   if ((!state.streamId || !state.templateId || !state.target) && (!state.error && testStreams.length)) {

      let streamId = state.streamId;
      let templateId = state.templateId;
      let target = state.target;

      let stream = testStreams.find(ts => ts.templateIds?.length !== ts.resolvedIds?.length);
      if (!stream) {
         stream = testStreams[0];
      }

      streamId = stream.streamId;
      templateId = stream.templateIds.find(tid => !!stream!.resolvedIds?.length || !stream!.resolvedIds.find(rid => rid === tid));
      if (!templateId && stream.templateIds.length) {
         templateId = stream.templateIds[0];
      }

      if (!streamId) {
         setState({ ...state, error: "Can't get a default stream" });
         return null;
      }

      if (!templateId) {
         setState({ ...state, error: "Can't get a default stream" });
         return null;
      }

      target = details.issueStreams?.find(s => s.streamId === streamId)?.nodes?.find(n => n.templateId === templateId)?.name;

      if (!target) {
         setState({ ...state, error: "Can't get a default target" });
         return null;
      }

      setState({ ...state, streamId: streamId, templateId: templateId, target: target });
      return null;
   }

   const streamMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: streamItems
   };

   let templateName = "None Available";

   if (state.streamId && state.templateId) {

      const stream = projectStore.streamById(state.streamId);
      if (stream) {

         const template = stream.templates.find(t => t.id === state.templateId);
         if (template) {
            const resolved = !!testStreams.find(s => s.streamId === stream.id)?.resolvedIds.find(id => id === template.id);
            templateName = `${stream.fullname} - ${template.name} / ${state.target}`;
            if (resolved) {
               templateName += " (Resolved)";
            }
         }
      }
   }

   const height = state.error ? 340 : 310;
   let baseCL: string | undefined = "";
   if (state.shelvedCL) {
      baseCL = state.baseCL;
   }

   /*
               <Stack>
                  <Checkbox
                     label={`Update Issues${!!state.shelvedCL ? " (Disabled)" : ""}`}
                     disabled={!!state.shelvedCL}
                     checked={!!state.updateIssues}
                     onChange={(ev, checked) => {
                        setState({ ...state, updateIssues: checked });
                     }} />
               </Stack>
                     */

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 660, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { onClose() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(); }}
            />
         </Stack.Item>
      </Stack>

      {!!state.error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}> {state.error} </MessageBar>}

      <Stack style={{ paddingLeft: 12, paddingTop: 12 }}>
         <Stack tokens={{ childrenGap: 24 }}>
            <Stack>
               <Label>Target</Label>
               <DefaultButton style={{ width: 610, textAlign: "left" }} menuProps={streamMenuProps} text={templateName} />
            </Stack>

            <Stack horizontal tokens={{ childrenGap: 32 }} verticalFill={true} verticalAlign="center">
               <Stack>
                  <TextField style={{ width: 130 }} label={`Base CL`} autoComplete="off"
                     value={baseCL ?? ""}
                     disabled={!state.shelvedCL}
                     placeholder="Latest Change"
                     onChange={(ev, newValue) => {
                        ev.preventDefault();
                        setState({ ...state, baseCL: newValue?.trim() ?? "" })
                     }} />
               </Stack>
               <Stack>
                  <TextField style={{ width: 180 }} label={`Shelved CL (Preflights will not update issue)`} autoComplete="off"
                     value={state.shelvedCL ?? ""}
                     placeholder="Optional"
                     onChange={(ev, newValue) => {
                        ev.preventDefault();
                        const v = newValue?.trim() ?? "";
                        setState({ ...state, shelvedCL: newValue?.trim() ?? "", updateIssues: v ? false : state.updateIssues })
                     }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 48, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack grow />
            <Stack>
               <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 16 }}>
                  <PrimaryButton text="Start Job" disabled={state.submitting || !state.templateId} onClick={() => { onTestFix(); }} />
                  <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { onClose(); }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;
}