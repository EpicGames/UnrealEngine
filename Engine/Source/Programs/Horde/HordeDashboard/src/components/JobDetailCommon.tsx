// Copyright Epic Games, Inc. All Rights Reserved.

import { BatchData, GetJobStepRefResponse, JobStepBatchError, JobStepBatchState, JobStepOutcome, JobStepState, StepData } from '../backend/Api';

type StepItem = {
   step?: StepData;
   batch?: BatchData;
   agentId?: string;
   agentRow?: boolean;
   agentType?: string;
   agentPool?: string;
};

export const getStepStatusMessage = (step: StepData) => {

   let message = "";

   if (step.abortRequested && step.state !== JobStepState.Aborted) {
      return "Cancel Requested";
   }

   switch (step.state) {
      case JobStepState.Waiting:
         message = "Waiting";
         break;
      case JobStepState.Skipped:
         message = "Skipped";
         break;
      case JobStepState.Aborted:
         message = "Canceled";
         break;
      case JobStepState.Ready:
         message = "Ready";
         break;
   }

   if (!message) {
      switch (step.outcome) {
         case JobStepOutcome.Failure:
            message = "Errors";
            break;
         case JobStepOutcome.Success:
            message = "Success";
            break;
         case JobStepOutcome.Warnings:
            message = "Warnings";
      }
   }

   return message;

};

export const getStepRefStatusMessage = (step: GetJobStepRefResponse) => {

   let message = "";

   if (!step.finishTime && (step.outcome !== JobStepOutcome.Failure && step.outcome !== JobStepOutcome.Warnings)) {
      message = "Running";
   }


   if (!message) {

      switch (step.outcome) {
         case JobStepOutcome.Failure:
            message = "Errors";
            break;
         case JobStepOutcome.Success:
            message = "Success";
            break;
         case JobStepOutcome.Warnings:
            message = "Warnings";
      }
   }


   return message;
};


export const getBatchText = (item: StepItem): string | undefined => {

   const batch = item.batch;

   let statusText: string | undefined;

   if (batch) {

      if (batch.error === JobStepBatchError.UnknownAgentType) {
         statusText = `Unknown agent type ${item.agentType}`;
      }

      if (batch.error === JobStepBatchError.UnknownPool) {
         statusText = `Unknown pool ${item.agentPool}`;
      }

      if (batch.error === JobStepBatchError.NoAgentsInPool) {
         statusText = `No agents in pool ${item.agentPool}`;
      }


      if (batch.error === JobStepBatchError.NoAgentsInPool) {
         statusText = `Unable to find agents in pool ${item.agentPool}`;
      }

      if (batch.error === JobStepBatchError.UnknownWorkspace) {
         statusText = `Unknown workspace`;
      }

      if (batch.error === JobStepBatchError.LostConnection) {
         statusText = `${item.agentId} : Lost Connection`;
      }

      if (batch.error === JobStepBatchError.ExecutionError) {
         statusText = `${item.agentId} :  Error while executing lease`;
      }

      if (batch.error === JobStepBatchError.Incomplete) {
         statusText = `${item.agentId} : Lease incomplete`;
      }

      if (batch.error === JobStepBatchError.NoLongerNeeded) {
         statusText = `${item.agentId} : No longer needed`;
      }

      if (batch.error === JobStepBatchError.SyncingFailed) {
         statusText = `${item.agentId} : Syncing failed`;
      }

      if (batch.error === JobStepBatchError.Cancelled) {
         if (item.agentId) {
            statusText = `${item.agentId} - Cancelled`;
         } else {
            statusText = `Cancelled`;
         }

      }

      if (batch.error === JobStepBatchError.NoAgentsOnline) {
         if (item.agentPool) {
            statusText = `No agents online : ${item.agentPool}`;
         } else {
            statusText = `No agents online : Unknown pool`;
         }
      }


      if (!statusText && batch.error !== undefined && batch.error !== JobStepBatchError.None) {
         statusText = `${item.agentId} : ${batch.error}`;
      }

      if (!statusText) {

         if (batch.state === JobStepBatchState.Starting) {
            statusText = item.agentId ? item.agentId! : "Batch is in starting state";
         }

         if (batch.state === JobStepBatchState.Stopping) {
            statusText = item.agentId ? item.agentId! : "Batch is in stopping state";
         }

         if (batch.state === JobStepBatchState.Waiting) {
            statusText = "Waiting for dependencies";
         }

         if (batch.state === JobStepBatchState.Ready && !batch.agentId) {
            statusText = `Waiting for agent : ${item.agentPool}`;
         }

      }
   }

   return statusText;

};


