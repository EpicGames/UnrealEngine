// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { Checkbox, ContextualMenuItemType, DefaultButton, Dropdown, IContextualMenuItem, IContextualMenuProps, Label, Stack, Text, TextField } from "@fluentui/react";
import React, { useEffect, useState, useSyncExternalStore } from "react";
import { useBackend } from "../backend";
import { GetJobsTabResponse, TabType, GetTemplateRefResponse, StreamData } from "../backend/Api";
import { FilterStatus, JobHandler } from "../backend/JobHandler";
import templateCache from '../backend/TemplateCache';
import { getHordeStyling } from "../styles/Styles";
import { observer } from "mobx-react-lite";
import dashboard from "horde/backend/Dashboard";

// todo, store filter for streamId so same when return
// todo, put on a timeout so can capture a number of changes before update

const statusItems = ["Running", "Complete", "Succeeded", "Failed", "Waiting", "Cancelled"].map(status => {
   return {
      key: status,
      text: status,
      status: status
   };
});

class JobFilter {

   constructor() {
      makeObservable(this);
   }

   @action
   setUpdated() {
      this.updated++;
   }

   set(templates?: GetTemplateRefResponse[], status?: FilterStatus[]) {

      let templatesDirty = true;

      if (!templates?.length && !this.templates?.length) {
         templatesDirty = false;
      }

      if (templates && this.templates) {

         if (templates.length === this.templates.length && templates.every((val, index) => val === this.templates![index])) {
            templatesDirty = false;
         }

      }

      if (templatesDirty) {
         this.templates = templates;
      }

      let statusDirty = true;

      if (!status?.length && !this.status?.length) {
         statusDirty = false;
      }

      if (status && this.status) {

         if (status.length === this.status.length && status.every((val, index) => val === this.status![index])) {
            statusDirty = false;
         }

      }

      if (statusDirty) {
         this.status = status;
      }

      // check any dirty
      if (templatesDirty || statusDirty) {
         this.setUpdated();
      }
   }

   setMinCL(value: number) {
      if(value !== this.minCL){
         this.setUpdated();
      }

      this.minCL = value;
   }

   setMaxCL(value: number) {
      if(value !== this.maxCL){
         this.setUpdated();
      }

      this.maxCL = value;
   }

   @observable
   updated: number = 0;

   templates?: GetTemplateRefResponse[];
   status?: FilterStatus[];
   minCL: number;
   maxCL: number;
}

export const jobFilter = new JobFilter();

const TemplateSelector: React.FC<{ stream: StreamData, templates: GetTemplateRefResponse[] }> = observer(({ stream, templates }) => {
   const [selectedTemplates, setSelectedTemplates] = useState<Set<GetTemplateRefResponse>>(new Set());

   useEffect(() => {
      jobFilter.set(Array.from(selectedTemplates), jobFilter.status);
   }, [selectedTemplates]);

   const templateOptions: IContextualMenuItem[] = [];
   const grouped = new Map<string, GetTemplateRefResponse[]>();

   templates.forEach(t => {

      stream.tabs.forEach(tab => {
         if (tab.type !== TabType.Jobs) {
            return;
         }

         const jtab = tab as GetJobsTabResponse;
         if (!jtab.templates?.find(template => template === t.id)) {
            return;
         }

         if (!grouped.has(jtab.title)) {
            grouped.set(jtab.title, []);
         }

         grouped.get(jtab.title)!.push(t);

      })
   })

   Array.from(grouped.keys()).sort((a, b) => a < b ? -1 : 1).forEach(cat => {

      const catTemplates = grouped.get(cat);
      if (!catTemplates?.length) {
         return;
      }

      const subItems: IContextualMenuItem[] = catTemplates.sort((a, b) => a.name < b.name ? -1 : 1).map(t => {

         return {
            itemType: ContextualMenuItemType.Normal,
            key: t.id,
            text: t.name,
            data: t,
            onRender: () => {
               return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 12px", cursor: "pointer" }} onClick={(ev) => {
                  ev?.preventDefault();
                  ev?.stopPropagation();

                  const newTemplates = new Set(selectedTemplates);
                  newTemplates.has(t) ? newTemplates.delete(t) : newTemplates.add(t);

                  setSelectedTemplates(newTemplates);
               }} >
                  <Checkbox checked={selectedTemplates.has(t)} />
                  <Text>{t.name}</Text></Stack>
            }
         };
      })

      const isSelected = (ct: GetTemplateRefResponse) => selectedTemplates.has(ct);
      let allCatTemplatesSelected: boolean = catTemplates.every(isSelected);
      let someCatTemplatesSelected: boolean = catTemplates.some(isSelected) && !allCatTemplatesSelected;

      if (catTemplates?.length > 1) {
         subItems.unshift({
            key: `select_all_${cat}`,
            onRender: () => {
               return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 12px", cursor: "pointer" }} onClick={(ev) => { 
                  ev?.stopPropagation();
                  ev?.preventDefault();
      
                  allCatTemplatesSelected = !allCatTemplatesSelected;
                  
                  const newTemplates = new Set(selectedTemplates);

                  if (allCatTemplatesSelected) {
                     catTemplates.forEach( ct => 
                        newTemplates.add(ct)
                     )
                  }
                  else {
                     catTemplates.forEach( ct => 
                        newTemplates.delete(ct)
                     )
                  }

                  setSelectedTemplates(newTemplates);        
                  return false;
               }} >
               <Checkbox indeterminate={someCatTemplatesSelected} checked={allCatTemplatesSelected} />
               <Text style={{ color: dashboard.darktheme ? "#ABABAB" : "#575757" }}>Select All</Text></Stack>
            }
         });
      }

      templateOptions.push({ 
         key: `${cat}_category`,
         text: cat,
         iconProps: { iconName: allCatTemplatesSelected ? 'Tick' : someCatTemplatesSelected ? 'Form' : '', style: { fontSize: '14px' } },
         subMenuProps: { items: subItems }
      });

   })

   let allTemplatesSelected: boolean = selectedTemplates.size === templates.length;
   let someTemplatesSelected: boolean = selectedTemplates.size > 0 && !allTemplatesSelected;

   templateOptions.unshift({
      key: `select_all_templates`,
      onRender: () => {
         return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 7px", cursor: "pointer" }} onClick={(ev) => { 
            ev?.stopPropagation();
            ev?.preventDefault();

            allTemplatesSelected = !allTemplatesSelected;
            
            setSelectedTemplates(allTemplatesSelected ? new Set(templates) : new Set());
            return false;
         }} >
         <Checkbox indeterminate={someTemplatesSelected} checked={allTemplatesSelected} />
         <Text style={{ padding: "6px 2px"}}>Select All</Text></Stack>
      }
   });

   const templateMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: templateOptions,
   };

   let templateText = "None Selected";
   if (selectedTemplates.size) {
      templateText = Array.from(selectedTemplates).map(t => t.name).join(", ");
      if (templateText.length > 84) {
         templateText = templateText.slice(0, 84);
         templateText += "...";
      }
   }

   return <Stack>
      <Label>{selectedTemplates.size > 1 ? "Templates" : "Template"}</Label>
      <DefaultButton style={{ width: 352, textAlign: "left" }} menuProps={templateMenuProps} text={templateText} />
   </Stack>

})

const StatusSelector: React.FC = () => {
   const [selectedStatuses, setSelectedStatuses] = useState<Set<string>>(new Set());

   useEffect(() => {
      jobFilter.set(jobFilter.templates, Array.from(selectedStatuses) as FilterStatus[]);
   }, [selectedStatuses])

   let statusOptions: IContextualMenuItem[] = [];

   let allStatusesSelected: boolean = selectedStatuses.size === statusItems.length;
   let someStatusesSelected: boolean = selectedStatuses.size > 0 && !allStatusesSelected;

   const statuses = statusItems.map(s => s.text);

   statusOptions.push({
      itemType: ContextualMenuItemType.Normal,
      key: `select_all_statuses`,
      onRender: () => {
         return (
            <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "8px 12px 8px 12px", cursor: "pointer" }} onClick={(ev) => {
               ev?.preventDefault();
               ev?.stopPropagation();

               allStatusesSelected = !allStatusesSelected;

               setSelectedStatuses(allStatusesSelected ? new Set(statuses) : new Set());
            }} >
               <Checkbox indeterminate={someStatusesSelected} checked={allStatusesSelected} />
               <Text>Select All</Text>
            </Stack>
         )
      }
      
   });

   statuses.forEach((status) => {
      statusOptions.push({ 
         itemType: ContextualMenuItemType.Normal,
         key: `${status}_key`,
         onRender: () => {
            return (
               <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "8px 12px 8px 12px", cursor: "pointer" }} onClick={(ev) => {
                  ev?.preventDefault();
                  ev?.stopPropagation();

                  const newStatuses = new Set(selectedStatuses);

                  if (newStatuses.has(status)) {
                     newStatuses.delete(status);
                  } else {
                     newStatuses.add(status);
                  }

                  setSelectedStatuses(newStatuses);
               }} >
                  <Checkbox checked={selectedStatuses.has(status)} />
                  <Text>{status}</Text>
               </Stack>
            ) 
         }
      });
   })

   const statusMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      items: statusOptions
   };

   let statusText = "None Selected";
   if (selectedStatuses.size) {
      statusText = Array.from(selectedStatuses).join(", ");
      if (statusText.length > 84) {
         statusText = statusText.slice(0, 84);
         statusText += "...";
      }
   }

   return (
      <Stack>
         <Label>{selectedStatuses.size > 1 ? "Statuses" : "Status"}</Label>
         <DefaultButton style={{ width: 352, textAlign: "left" }} menuProps={statusMenuProps} text={statusText} />
      </Stack>
   )
}

// Job filter bar for "all" jobs view
export const JobFilterBar: React.FC<{ streamId: string }> = observer(({ streamId }) => {

   const { projectStore } = useBackend();

   const [templates, setTemplates] = useState<GetTemplateRefResponse[]>();

   const stream = projectStore.streamById(streamId);
   
   const { hordeClasses } = getHordeStyling();

   const [changelistBounds, setChangelistBounds] = useState<{min?: string, max?: string}>({min: undefined, max: undefined})

   useEffect(() => {
      // Cancellation flag for managing stale async calls (prevent updating the UI with old data)
      let cancelled = false;
      if(stream) {
         templateCache.getStreamTemplates(stream).then(data => {
            if(cancelled) return;

            setTemplates(data);
            jobFilter.set(data, jobFilter.status); // Init to all templates
         })
      }
      return () => {
         cancelled = true;
         jobFilter.set(undefined, undefined);
      };

   }, [stream]);

   useEffect(() => {
      jobFilter.setMinCL(Number(changelistBounds.min));
      jobFilter.setMaxCL(Number(changelistBounds.max));
   }, [changelistBounds])

   if (!stream || !templates) {
      console.error("unable to get stream or templates");
      return <div>unable to get stream or templates</div>;
   }

   return (
      <Stack horizontal tokens={{ childrenGap: 24 }} className={hordeClasses.modal}>
         <TemplateSelector 
            stream={stream}
            templates={templates} 
         />
         <StatusSelector />
         <TextField value={changelistBounds.min ?? ""} style={{ width: 168 }} label="Min Changelist" onChange={(ev, newValue) => {
            newValue = newValue?.trim();
            if (!newValue) {
               newValue = undefined;
            }
            setChangelistBounds({...changelistBounds, min: newValue});
         }} />
         <TextField value={changelistBounds.max ?? ""} style={{ width: 168 }} label="Max Changelist" onChange={(ev, newValue) => {
            newValue = newValue?.trim();
            if (!newValue) {
               newValue = undefined;
            }
            setChangelistBounds({ ...changelistBounds, max: newValue })
         }} />
      </Stack>
   );
});