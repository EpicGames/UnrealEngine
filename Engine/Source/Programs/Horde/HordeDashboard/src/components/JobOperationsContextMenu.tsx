// Copyright Epic Games, Inc. All Rights Reserved.

import { ContextualMenu, IContextualMenuItem } from '@fluentui/react';
import React, { useState } from 'react';
import { GetJobResponse, JobState } from '../backend/Api';
import dashboard from '../backend/Dashboard';
import { projectStore } from '../backend/ProjectStore';
import { getPreflightDisplay } from '../base/utilities/commitUtils';
import { AbortJobModal } from './jobDetailsV2/AbortJobModal';
// import { EditJobModal } from './EditJobModal';

export const JobOperationsContextMenu: React.FC<{ job: GetJobResponse, children?: any }> = ({ job, children }) => {

   const [state, setState] = useState<{ pos?: { x: number, y: number } }>({});


   const [abortShown, setAbortShown] = useState(false);
   // const [editShown, setEditShown] = useState(false);

   const copyToClipboard = (value: string | undefined) => {

      if (!value) {
         return;
      }

      const el = document.createElement('textarea');
      el.value = value;
      document.body.appendChild(el);
      el.select();
      document.execCommand('copy');
      document.body.removeChild(el);
   }

   const pinned = dashboard.jobPinned(job.id);

   let change = job.change!.toString();
   const pfDisplay = getPreflightDisplay(job.preflightCommitId, job.preflightChange);
   if (pfDisplay) {
      change = pfDisplay;
   }

   let menuItems: IContextualMenuItem[] = [
      {
         key: 'open_in_new_window',
         text: 'Open in New Tab',
         onClick: () => window.open(`/job/${job.id}`),
      },
      {
         key: 'pin_job',
         text: pinned ? 'Unpin from Home' : "Pin to Home",
         onClick: () => { pinned ? dashboard.unpinJob(job.id) : dashboard.pinJob(job.id) },
      },
      {
         key: 'copy_to_clipboard',
         text: 'Copy Job to Clipboard',
         onClick: () => copyToClipboard(window.location.protocol + "//" + window.location.hostname + `/job/${job.id}`)
      },
      {
         key: 'copy_cl_to_clipboard',
         text: 'Copy CL to Clipboard',
         onClick: () => copyToClipboard(change),
      }
   ];

   if (dashboard.swarmUrl) {
      menuItems.push(...[
         {
            key: 'open_in_swarm', text: 'Open CL in Swarm', onClick: () => {
               window.open(`${dashboard.swarmUrl}/changes/${change}`);
            }
         },
         {
            key: 'open_in_swarm_history', text: 'Open CL History in Swarm', onClick: () => {

               const stream = projectStore.streamById(job.streamId)!;
               const project = projectStore.byId(stream!.projectId)!;
               const name = project.name === "Engine" ? "UE4" : project.name;
               const rangeUrl = `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${change}#commits`;

               window.open(rangeUrl)
            }
         }])
   }

   const abortDisabled = job.state === JobState.Complete;

   /*
   const editDisabled = job.state === JobState.Complete;
   {!editDisabled && <EditJobModal jobData={job} show={editShown} onClose={() => { setEditShown(false); }} />}
   if (!editDisabled) {
      menuItems.push({
         key: 'edit_job',
         text: 'Edit',
         onClick: () => setEditShown(true),
      });
   }
   */

   if (!abortDisabled) {
      menuItems.push({
         key: 'abort_job',
         text: 'Cancel Job...',
         onClick: () => setAbortShown(true),
      });
   }


   return (<div onContextMenu={(ev) => {
      ev.preventDefault();

      // chrome nulls out the event properties so capture them
      const pos = { x: ev.clientX, y: ev.clientY };

      setState({ pos: pos });

      return false;
   }}>
      {!abortDisabled && <AbortJobModal jobDataIn={job} show={abortShown} onClose={() => { setAbortShown(false); }} />}

      <ContextualMenu
         styles={{ list: { selectors: { '.ms-ContextualMenu-itemText': { fontSize: "10px" } } } }}
         items={menuItems}
         hidden={(!state.pos)}
         target={state.pos!}
         onItemClick={(ev) => { setState({}); ev?.preventDefault(); }}
         onDismiss={(ev) => { setState({}); ev?.preventDefault(); }} />
      {children}
   </div>)
};