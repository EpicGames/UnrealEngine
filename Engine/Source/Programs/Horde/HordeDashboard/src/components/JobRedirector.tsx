// Copyright Epic Games, Inc. All Rights Reserved.

import React, { useState } from 'react';
import { useNavigate } from 'react-router-dom';
import backend from '../backend';
import { useQuery } from "horde/base/utilities/hooks";
import {errorDialogStore} from "./error/ErrorStore";


function setError(message: string) {

   console.error(message);

   message = `${message}\n\nReferring URL: ${encodeURI(window.location.href)}`

   errorDialogStore.set({

      title: `Error handling Preflight`,
      message: message

   }, true);
}

// redirect from external source, where horde stream id, etc are not known by that application
export const JobRedirector: React.FC = () => {

   const [state, setState] = useState({ backendQueried: false })

   const navigate = useNavigate();
   const query = useQuery();

   const streamId = !query.get("streamId") ? "" : query.get("streamId")!;
   const templateId = query.getAll("templateId");
   const nodeName = !query.get("nodeName") ? "" : query.get("nodeName")!;

   const complete = query.get("complete") !== null;

   if (!streamId || !templateId.length || !nodeName) {
      setError("Please specify streamId, templateId, and nodeName");
      return null;
   }


   if (complete && templateId.length > 1) {
      setError("Please specify a single template id when using complete queries");
      return null;
   }


   if (!state.backendQueried) {

      if (!complete) {

         backend.getJobs({ streamId: streamId, template: templateId, count: 10 * templateId.length, includePreflight: false, filter:"id,batches,change"}).then(results => {

            results = results.sort((a, b) => (b.change ?? 0) - (a.change ?? 0))

            const job = results.find(job => {
               const batch = job.batches?.find(b => b.steps?.find(s => s.name === nodeName));
               const step = batch?.steps?.find(s => s.name === nodeName);
               if (!step) {
                  return false;
               }

               return true;
            })

            if (job) {
               const batch = job.batches?.find(b => b.steps?.find(s => s.name === nodeName))!;
               const step = batch?.steps?.find(s => s.name === nodeName)!;

               navigate(`/job/${job.id}?step=${step.id}`, { replace: true });

            } else {
               throw `Could not find job using step states`
            }

         }).catch(reason => {
            setError(`Error getting last job for ${streamId} / ${templateId} / ${nodeName}: Please check parameters\n${reason}`);
         })
      }
      else {
         backend.getJobStepHistory(streamId, nodeName, 1, templateId[0]).then(results => {
            if (!results?.length) {
               setError(`No history results for ${streamId} / ${templateId} / ${nodeName}`);
               return;
            }

            const result = results[0];

            navigate(`/job/${result.jobId}?step=${result.stepId}`, { replace: true });

         }).catch(reason => {
            setError(`Error getting last job for ${streamId} / ${templateId} / ${nodeName}: Please check parameters\n${reason}`);
         });
      }

      return null;
   }

   setState({ backendQueried: true })

   return null;

}
