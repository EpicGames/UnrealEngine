// Copyright Epic Games, Inc. All Rights Reserved.

import { Callout, Checkbox, ContextualMenuItemType, DatePicker, DefaultButton, DetailsList, DetailsRow, DirectionalHint, IColumn, IconButton, IContextualMenuItem, IContextualMenuItemProps, IContextualMenuProps, IDetailsGroupRenderProps, IDetailsListProps, IGroup, Label, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useEffect, useMemo, useRef, useState } from "react";
import { Link, useLocation } from "react-router-dom";
import backend from "../backend";
import { GetJobResponse, GetJobsTabResponse, GetJobStepRefResponse, JobData, JobQuery, TabType, GetTemplateRefResponse, GetJobsTabLabelColumnResponse, GetLabelStateResponse, LabelOutcome, LabelState, GetJobsTabColumnResponse, JobStepOutcome } from "../backend/Api";
import dashboard from "../backend/Dashboard";
import { projectStore } from "../backend/ProjectStore";
import templateCache from '../backend/TemplateCache';
import { displayTimeZone } from "../base/utilities/timeUtils";
import { ChangeButton } from "./ChangeButton";
import { StepRefStatusIcon } from "./StatusIcon";
import { UserSelect } from "./UserSelect";
import { getHordeTheme } from "../styles/theme";
import { getHordeStyling } from "../styles/Styles";
import { StreamChooser } from "./projects/StreamChooser";
import { getLabelColor } from "horde/styles/colors";
import LZString from "lz-string"
import { errorBarStore } from "./error/ErrorStore";
import { useQuery } from "horde/base/utilities/hooks";

type SearchState = {
   templates: Set<string>, 
   statuses: Set<string>, 
   name?: string, 
   userId?: string,
   global?: boolean,
   streamIdOverride?: string,
   preflights: boolean,
   preflightOnly?: boolean,
   minDate?: Date,
   maxDate?: Date,
   minCL?: string,
   maxCL?: string,
   containsStep?: string,
   containsParameter?: string 
}

const jobToStep = new Map<string, GetJobStepRefResponse>();
let jobStepName = "";
let streamChooserId = 0;

export const JobSearchSimpleModal: React.FC<{ streamIdIn: string, changeListIn?: string, jobTab?: GetJobsTabResponse | undefined, autoloadIn?: boolean, onClose: () => void }> = observer(({ streamIdIn, jobTab, changeListIn, onClose, autoloadIn }) => {

   type JobItem = {
      job: GetJobResponse
   };

   const statuses = [ "Running", "Complete", "Succeeded", "Failed", "Waiting", "Cancelled"]
   const allStatuses = new Set<string>(statuses);

   const [searchState, setSearchState] = useState<SearchState>({ preflights: false, minCL: changeListIn, maxCL: changeListIn, templates: new Set(jobTab?.templates ?? []), statuses: allStatuses });
   const [searchResults, setSearchResults] = useState<{items: JobItem[], groups?: IGroup[], querying: boolean}>({items: [], querying: false});
   const query = useQuery();
   const [templateData, setTemplateData] = useState<{ streamId: string; templates: GetTemplateRefResponse[] } | undefined>(undefined);
   const [templateFilter, setTemplateFilter] = useState("");
   const [templateSearchOpen, setTemplateSearchOpen] = useState(false);
   const templateSearchRef = useRef<HTMLDivElement>(null);
   const location = useLocation();
   const [isFirstRender, setIsFirstRender] = useState(true);

   const { hordeClasses, detailClasses } = getHordeStyling();

   const streamId = searchState.streamIdOverride ?? streamIdIn;

   const stream = projectStore.streamById(streamId);

   // Make available before queryJobs, recomputing the values when needed/able
   const ctemplates = useMemo(
      () => (templateData?.templates ?? []).filter(t => searchState.templates.has(t.id)),
      [templateData, searchState.templates]
   );

   const cstatuses = useMemo(
      () => statuses.filter(s => searchState.statuses.has(s)),
      [searchState.statuses]
   );

   // Update the search params in the url for sharing and bookmarking queires
   const updateSearch = (state: SearchState) => {
      const tab = query.get("tab");
      query.clear();
      if(tab) query.set("tab", tab);

      // JSON.stringify doesn't support Sets
      const stringyState = {
         ...state,
         templates: Array.from(state.templates),
         statuses: Array.from(state.statuses)
      };
      // Compress current state to prevent 414
      query.setCompressed("search", JSON.stringify(stringyState));
   }

   const queryJobs = async (state: SearchState) => {
      updateSearch(state);
      setSearchResults(prev => ({ ...prev, querying: true }));

      const c_templates = (templateData?.templates ?? []).filter(t => state.templates.has(t.id));
      const c_statuses = statuses.filter(s => state.statuses.has(s));

      let jobs: JobData[] = [];

      try {

         let name = state.name?.trim();
         if (!name) {
            name = undefined;
         } else {
            name = "$" + name;
         }

         let minCL: number | undefined = parseInt(state.minCL ?? "");
         if (isNaN(minCL)) {
            minCL = undefined;
         }
         let maxCL: number | undefined = parseInt(state.maxCL ?? "");
         if (isNaN(maxCL)) {
            maxCL = undefined;
         }

         if (!minCL && maxCL) {
            minCL = 0;
         }

         if (minCL && !maxCL) {
            maxCL = 999999999;
         }

         let minCreateTime = state.minDate;
         let maxCreateTime = state.maxDate;
         
         if (maxCreateTime) {

            const maxTime = moment(maxCreateTime);
            maxTime.add(23, "hours");
            maxTime.add(59, "minutes");
            maxTime.add(59, "seconds");

            maxCreateTime = maxTime.toDate();

         }

         const maxCount = 2048;

         let searchStreamId: string | undefined = streamId;

         if (state.streamIdOverride) {
            searchStreamId = state.streamIdOverride
         } else if (state.global && !state.containsStep) {
            searchStreamId = undefined;
         }

         const stepName = state.containsStep?.trim();
         const stepJobIds: string[] = []

         jobToStep.clear();
         jobStepName = stepName ?? "";
         if (stepName) {

            let stepTemplates: string[] = [];

            if (c_templates?.length) {
               stepTemplates.push(...c_templates.map(t => t.id));
            } else {
               stepTemplates = templates.filter(t => !!t.id).map(t => t.id);
            }

            const numTemplates = stepTemplates.length;
            while (stepTemplates.length) {

               const batch = stepTemplates.slice(0, 5);
               stepTemplates = stepTemplates.slice(5);

               // @todo: get endpoint to handle multiple templates
               const results = await Promise.all(batch.map(template => backend.getJobStepHistory(searchStreamId!, stepName, numTemplates === 1 ? 256 : 64, template)));
               results.forEach(r => {
                  r.forEach(j => {
                     stepJobIds.push(j.jobId)
                     jobToStep.set(j.jobId, j);
                  });
               });

               if (stepJobIds.length >= 256) {
                  break;
               }
            }
         }

         const jobIds = stepJobIds.slice(0, 256);

         if (stepName && !jobIds.length) {
            jobs = [];
         } else {
            
            const query: JobQuery = {
               id: jobIds.length ? jobIds : undefined,
               streamId: searchStreamId?.toLowerCase(),
               name: name,
               filter: "id,labels,defaultLabel,name,change,createTime,streamId,preflightChange,preflightCommitId,startedByUserInfo,arguments,abortedByUserInfo,state,batches,cancellationReason",
               minChange: minCL,
               maxChange: maxCL,
               t: (!state.global && c_templates.length && !state.containsStep) ? c_templates.map(t => t.id) : undefined,
               minCreateTime: minCreateTime?.toUTCString(),
               maxCreateTime: maxCreateTime?.toUTCString(),
               includePreflight: state.preflights || !!state.preflightOnly,
               preflightOnly: !!state.preflightOnly,
               startedByUserId: state.userId,
               count: maxCount
            }

            jobs = await backend.getJobs(query);

            if (c_statuses.length && c_statuses.length != allStatuses.size) {
               jobs = jobs.filter(j => {
                  const jobSucceeded = j.batches?.every(b => b.steps.every(s => s.outcome === JobStepOutcome.Success || s.outcome === JobStepOutcome.Warnings));

                  return (
                     c_statuses.includes('Cancelled') && j.abortedByUserInfo?.id // Cancelled
                     || c_statuses.includes(j.state) && !j.abortedByUserInfo?.id // Running, Complete, Waiting
                     || c_statuses.includes("Failed") && !jobSucceeded // Failed
                     || c_statuses.includes("Succeeded") && jobSucceeded // Succeeded
                  )
               })
            }

            const p = state.containsParameter?.trim().toLowerCase();
            if (p) {
               jobs = jobs.filter(j => {
                  return !!j.arguments.find(a => {
                     return a.toLowerCase().indexOf(p) !== -1
                  })
               });
            }
         }

      } catch (reason) {
         console.error(reason);
      } finally {

         const streamIds = new Set<string>();
         const idToFullname = new Map<string, string>();

         jobs.forEach(j => {
            const stream = projectStore.streamById(j.streamId);
            if (!stream || !stream.fullname) {
               streamIds.add(j.streamId);
               idToFullname.set(j.streamId, j.streamId);
               return;
            };
            streamIds.add(j.streamId);
            idToFullname.set(j.streamId, stream.fullname);

         });

         const sorted = Array.from(streamIds).sort((a, b) => {
            const aname = idToFullname.get(a)!;
            const bname = idToFullname.get(b)!;

            if (aname < bname) return -1;
            if (aname > bname) return 1;
            return 0;
         })

         // emit groups
         const groups: IGroup[] = [];
         const items: JobItem[] = [];
         sorted.forEach(id => {
            const streamJobs = jobs.filter(j => j.streamId === id);
            groups.push({ key: `jobs_${id}`, name: idToFullname.get(id)!, startIndex: items.length, count: streamJobs.length, isCollapsed: false });
            items.push(...streamJobs.map(j => { return { job: j } }));
         })

         setSearchResults({items, groups, querying: false });
         setSearchState(state);
      }

   }

   // Sets state from search params and autoloads jobs
   useEffect(() => {
      // StreamView's param detection + the 'get' check below allow linking to an empty search modal "https://...?tab=...&search="
      if(!query.has("search") || !query.get("search") || !isFirstRender || !templateData) return;

      setIsFirstRender(false);
      
      const searchValue = query.get("search")!;
      try {
         // Decompress URL data
         const stringyState = JSON.parse(LZString.decompressFromEncodedURIComponent(searchValue));
         
         const state: SearchState = {
            ...stringyState,
            minDate: stringyState.minDate ? new Date(stringyState.minDate) : undefined,
            maxDate: stringyState.maxDate ? new Date(stringyState.maxDate) : undefined,
            templates: new Set(stringyState.templates),
            statuses: new Set(stringyState.statuses)
         };
         
         if(state.global){
            streamChooserId++;
            state.templates = new Set();
            state.streamIdOverride = undefined;
         }

         queryJobs(state);
      } catch(err) {
         const truncateMsg = searchValue.length > 50;
         
         // Truncate param string for display friendliness. Doesn't offer complete information, but better than nothing.
         const searchValueMsg = truncateMsg ? searchValue.slice(0,50) + "..." : searchValue;
         errorBarStore.set({title: "Job Search Error", message: `Could not load search from URL search param \'${searchValueMsg}\'`}, true)
         
         // Best to offer complete information somewhere
         if(truncateMsg){
            console.error("Bad search param value:\n\n" + searchValue + "\n\n");
         }

         query.clear(["tab"]);
      }
   }, [templateData, location.search])

   // Enable retrigger of url state loading when navigating between the result of a clear action and a search action
   useEffect(() => {
      setIsFirstRender(true);
   }, [location.search])

   if (!stream) {
      console.error("unable to get stream");
      return <div>unable to get stream</div>;
   }

   const hordeTheme = getHordeTheme();

   const StatusModal: React.FC<{ text: string }> = ({ text }) => {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{text}</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>

         </Stack>
      </Modal>
   }

   // templates
   if (!templateData || streamId !== templateData.streamId) {
      templateCache.getStreamTemplates(stream).then(data => setTemplateData({ streamId: streamId!, templates: data }));
      return <StatusModal text={"Loading Templates"} />
   } else if (templateData.streamId !== streamId) {
      setTemplateData(undefined);
      return null;
   }

   let templates = templateData.templates;

   let templateOptions: IContextualMenuItem[] = [];


   const sorted = new Map<string, GetTemplateRefResponse[]>();

   // filter and sort templates into categories
   templates.forEach(t => {

      stream.tabs.forEach(tab => {
         if (tab.type !== TabType.Jobs) {
            return;
         }

         const jtab = tab as GetJobsTabResponse;
         if (!jtab.templates?.find(template => template === t.id)) {
            return;
         }

         if (!sorted.has(jtab.title)) {
            sorted.set(jtab.title, []);
         }

         sorted.get(jtab.title)!.push(t);

      })
   })

   const allTemplates = new Set<string>();

   Array.from(sorted.keys()).sort((a, b) => a < b ? -1 : 1).forEach(cat => {

      const catTemplates = sorted.get(cat);
      if (!catTemplates?.length) {
         return;
      }

      const subItems: IContextualMenuItem[] = catTemplates.sort((a, b) => a.name < b.name ? -1 : 1).map(t => {

         allTemplates.add(t.id);
         return {
            itemType: ContextualMenuItemType.Normal,
            key: t.id,
            text: t.name,
            data: t,
            onRender: () => {
               return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 12px", cursor: "pointer" }} onClick={(ev) => {
                  ev?.preventDefault();
                  ev?.stopPropagation();

                  setSearchState(prev => {
                     const newTemplates = new Set(prev.templates);

                     if (newTemplates.has(t.id)) {
                        newTemplates.delete(t.id);
                     } else {
                        newTemplates.add(t.id);
                     }

                     return {
                        ...prev,
                        templates: newTemplates
                     };   
                  });
               }} >
                  <Checkbox checked={searchState.templates.has(t.id)} />
                  <Text>{t.name}</Text></Stack>
            }
         };
      })

      const isSelected = (ct: GetTemplateRefResponse) => searchState.templates.has(ct.id);
      let allCatTemplatesSelected: boolean = catTemplates.every(isSelected);
      let someCatTemplatesSelected: boolean = catTemplates.some(isSelected) && !allCatTemplatesSelected;

      if (catTemplates?.length > 1) {
         subItems.unshift({
            key: `select_all_${cat}`,
            onRender: () => {
               return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 12px", cursor: "pointer" }} onClick={(ev) => { 
                  ev?.stopPropagation();
                  ev?.preventDefault();
      
                  allCatTemplatesSelected = !allCatTemplatesSelected;
                  
                  const newTemplates = new Set(searchState.templates);

                  if (allCatTemplatesSelected) {
                     catTemplates.forEach( ct => 
                        newTemplates.add(ct.id)
                     )
                  }
                  else {
                     catTemplates.forEach( ct => 
                        newTemplates.delete(ct.id)
                     )
                  }

                  setSearchState(prev => ({...prev, templates: newTemplates}));                  
                  return false;
               }} >
               <Checkbox indeterminate={someCatTemplatesSelected} checked={allCatTemplatesSelected} />
               <Text style={{ color: dashboard.darktheme ? "#ABABAB" : "#575757" }}>Select All</Text></Stack>
            }
         });
      }

      templateOptions.push({ 
         key: `${cat}_category`,
         text: cat,
         iconProps: { iconName: allCatTemplatesSelected ? 'Tick' : someCatTemplatesSelected ? 'Form' : '', style: { fontSize: '14px' } },
         subMenuProps: { items: subItems }
      });

   })

   let allTemplatesSelected: boolean =  searchState.templates.size === allTemplates.size;
   let someTemplatesSelected: boolean =  searchState.templates.size > 0 && !allTemplatesSelected;

   templateOptions.unshift({
      key: `select_all_templates`,
      onRender: () => {
         return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "4px 12px 4px 7px", cursor: "pointer" }} onClick={(ev) => {
            ev?.stopPropagation();
            ev?.preventDefault();

            allTemplatesSelected = !allTemplatesSelected;

            if (allTemplatesSelected) {
               searchState.templates = allTemplates;
               setSearchState({ ...searchState, containsParameter: undefined })
            }
            else {
               searchState.templates = new Set();
               setSearchState({ ...searchState, containsParameter: undefined })
            }
            return false;
         }} >
         <Checkbox indeterminate={someTemplatesSelected} checked={allTemplatesSelected} />
         <Text style={{ padding: "6px 2px"}}>Select All</Text></Stack>
      }
   });


   const templateMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: templateOptions
   };

   // Statuses
   let statusOptions: IContextualMenuItem[] = [];

   let allStatusesSelected: boolean = statuses.every(s => searchState.statuses.has(s));
   let someStatusesSelected: boolean = statuses.some(s => searchState.statuses.has(s)) && !allStatusesSelected;

   statusOptions.push({
      itemType: ContextualMenuItemType.Normal,
      key: `select_all_statuses`,
      onRender: () => {
         return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "8px 12px 8px 12px", cursor: "pointer" }} onClick={(ev) => {
            ev?.preventDefault();
            ev?.stopPropagation();

            allStatusesSelected = !allStatusesSelected;

            if (allStatusesSelected) {
               searchState.statuses = allStatuses;
               setSearchState({ ...searchState })
            } else {
               searchState.statuses = new Set();
               setSearchState({ ...searchState })
            }
         }} >

            <Checkbox indeterminate={someStatusesSelected} checked={allStatusesSelected} />
            <Text>Select All</Text></Stack>
      }
      
   });

   statuses.forEach((status) => {
      statusOptions.push({ 
         itemType: ContextualMenuItemType.Normal,
         key: `${status}_key`,
         onRender: () => {
            return <Stack horizontal verticalFill verticalAlign="center" style={{ padding: "8px 12px 8px 12px", cursor: "pointer" }} onClick={(ev) => {
               ev?.preventDefault();
               ev?.stopPropagation();

               setSearchState(prev => {
                  const newStatuses = new Set(prev.statuses);

                  if (newStatuses.has(status)) {
                     newStatuses.delete(status);
                  } else {
                     newStatuses.add(status);
                  }

                  return {
                     ...prev,
                     statuses: newStatuses
                  };   
               });
            }} >

               <Checkbox checked={searchState.statuses.has(status)} />
               <Text>{status}</Text></Stack>
         }
      });
   })

   const statusMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      items: statusOptions
   };

   // results
   const labelColumnWidth = 396;
   const columns = [
      { key: 'column1', name: 'Change', minWidth: 60, maxWidth: 60, isResizable: false },
      { key: 'column2', name: 'Name', minWidth: 140, maxWidth: 140, isResizable: false },
      { key: 'column3', name: 'Labels', minWidth: labelColumnWidth, maxWidth: labelColumnWidth, isResizable: false },
      //{ key: 'column4', name: 'Step', minWidth: 200, maxWidth: 200, isResizable: false },
      { key: 'column5', name: 'StartedBy', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column6', name: 'Created', minWidth: 100, maxWidth: 100, isResizable: false },
   ];

   const onRenderGroupHeader: IDetailsGroupRenderProps['onRenderHeader'] = (props) => {
      if (props) {

         const group = props.group!;

         return (
            <div className="view-log-link">
               <div className={detailClasses.headerAndFooter} style={{ marginRight: 0, padding: 2, width: 900 }} onClick={() => {

                  //.handler.collapsedIssueGroups.set(group.key, !handler.collapsedIssueGroups.get(group.key));
                  props.onToggleCollapse!(props.group!);
               }}>
                  <div style={{
                     fontSize: "13px",
                     padding: '4px 8px',
                     userSelect: 'none',
                     color: dashboard.darktheme ? "#FFFFFFFF" : "#404040",
                     width: "100%",
                     fontFamily: "Horde Open Sans SemiBold"
                  }}>{`${group.name} (${group.count})`}</div>
               </div>
            </div>
         );
      }

      return null;
   };

   const JobLabel: React.FC<{ item: JobItem; label: GetLabelStateResponse }> = ({ item, label }) => {

      const defaultLabel = item.job.defaultLabel;

      const aggregates = item.job.labels;

      // note details may not be loaded here, as only initialized on callout for optimization (details.getLabelIndex(label.Name, label.Category);)
      const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
      let labelIdx = -1;
      if (jlabel) {
         labelIdx = aggregates?.indexOf(jlabel)!;
      }

      let state: LabelState | undefined;
      let outcome: LabelOutcome | undefined;
      if (label === defaultLabel) {
         state = defaultLabel.state;
         outcome = defaultLabel.outcome;
      } else {
         state = item.job.labels![labelIdx]?.state;
         outcome = item.job.labels![labelIdx]?.outcome;
      }

      const color = getLabelColor(state, outcome);

      let url = `/job/${item.job.id}`;

      if (labelIdx >= 0) {
         url = `/job/${item.job.id}?label=${labelIdx}`;
      }

      const target = `label_search_${item.job.id}_${label.dashboardName}_${label.dashboardCategory}`.replace(/[^A-Za-z0-9]/g, "");

      return <Stack>
         <div id={target} className="horde-no-darktheme">
            <Link to={url}><Stack className={hordeClasses.badgeNoIcon}><DefaultButton key={label.dashboardName} style={{ backgroundColor: color.primaryColor }} text={label.dashboardName}
               onMouseOver={(ev) => ev.stopPropagation()}
               onMouseMove={(ev) => ev.stopPropagation()}>
               {!!color.secondaryColor && <div style={{
                  borderLeft: "10px solid transparent",
                  borderRight: `10px solid ${color.secondaryColor}`,
                  borderBottom: "10px solid transparent",
                  height: 0,
                  width: 0,
                  position: "absolute",
                  right: 0,
                  top: 0,
                  zIndex: 1
               }} />}

            </DefaultButton></Stack></Link>
         </div>
      </Stack>;

   };

   const renderLabels = (item: JobItem): JSX.Element | null => {

      let labels = (item.job.labels ?? []).filter(l => !!l.dashboardName?.length && !!l.state && l.state !== LabelState.Unspecified);

      if (item.job.defaultLabel) {
         item.job.defaultLabel.dashboardName = "Other";
         item.job.defaultLabel.dashboardCategory = "zzt";
         labels.push(item.job.defaultLabel);
      }

      if (!labels.length) {
         return null;
      }

      labels = labels.sort((a, b) => {

         if (a.dashboardCategory === b.dashboardCategory) {
            return a.dashboardName!.localeCompare(b.dashboardName!);
         }

         return (a.dashboardCategory ?? "").localeCompare(b.dashboardCategory ?? "");
      })


      const defaultLabel = item.job.defaultLabel;

      if (!labels.length && !defaultLabel) {
         return <div />;
      }

      let key = 0;

      const buttons = labels.map(label => {
         return <JobLabel key={`label_${item.job.id}_${label.dashboardCategory ?? ""}_${label.dashboardName ?? ""}_${key++}`} item={item} label={label} />;
      });

      return (<Stack horizontalAlign="center">
         <Stack wrap horizontal horizontalAlign="center" tokens={{ childrenGap: 4 }} styles={{ root: { paddingTop: 2, width: labelColumnWidth } }}>
            {buttons}
         </Stack>
      </Stack>
      );

   };


   const renderItem = (item: JobItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      if (column.name === "Name") {
         return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ whiteSpace: "break-spaces" }} ><Text variant="tiny">{item.job.name}</Text></Stack>;
      }

      if (column.name === "Change") {

         return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ paddingTop: 4 }} ><ChangeButton job={item.job} hideAborted={false} /></Stack>;
      }

      if (column.name === "Labels") {
         return <Stack horizontal verticalFill={true} verticalAlign="center" >{renderLabels(item)}</Stack>;
      }

      if (column.name === "StartedBy") {
         let startedBy = item.job.startedByUserInfo?.name;
         if (!startedBy) {
            startedBy = "Scheduler";
         }
         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"center"}><Text variant="tiny">{startedBy}</Text></Stack>;
      }

      if (column.name === "Step") {
         const ref = jobToStep.get(item.job.id);
         if (!ref || !jobStepName) {
            return null;
         }
         return <Link to={`/job/${ref.jobId}?step=${ref.stepId}`}><Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }} >{<StepRefStatusIcon stepRef={ref} />}<Stack style={{ whiteSpace: "break-spaces" }}><Text variant="tiny">{jobStepName}</Text></Stack></Stack></Link>;
      }


      if (column.name === "Created") {

         if (item.job.createTime) {

            const displayTime = moment(item.job.createTime).tz(displayTimeZone());
            const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

            let displayTimeStr = displayTime.format('MMM Do') + ` at ${displayTime.format(format)}`;


            return <Stack verticalAlign="center" horizontalAlign="end" tokens={{ childrenGap: 0, padding: 0 }} style={{ height: "100%", paddingRight: 18 }}>
               <Stack ><Text variant="tiny">{displayTimeStr}</Text></Stack>
            </Stack>;

         } else {
            return "???";
         }
      }

      return <Stack />;
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as JobItem;

         const url = `/job/${item.job.id}`;

         const commonSelectors = { ".ms-DetailsRow-cell": { "overflow": "visible", padding: 0 } };

         props.styles = { ...props.styles, root: { paddingTop: 8, paddingBottom: 8, selectors: { ...commonSelectors as any } } };

         return <Link to={url} onClick={(ev) => { if (!ev.ctrlKey) onClose() }}><div className="job-item"><DetailsRow {...props} /> </div></Link>;

      }
      return null;
   };

   let title = `Find Jobs in ${stream.fullname}`;
   if (searchState.global) {
      title = `Find Jobs in All Streams`;
   }
   if (searchState.streamIdOverride) {
      let nstream = projectStore.streamById(searchState.streamIdOverride)
      title = `Find Jobs in ${nstream?.fullname ?? searchState.streamIdOverride}`;
   }

   let templateText = "None Selected";
   if (searchState.templates.size) {
      templateText = ctemplates.map(t => t.name).join(", ");
      if (templateText.length > 84) {
         templateText = templateText.slice(0, 84);
         templateText += "...";
      }
   }

   let statusText = "None Selected";
   if (searchState.statuses.size) {
      statusText = cstatuses.map(s => s).join(", ");
      if (statusText.length > 84) {
         statusText = statusText.slice(0, 84);
         statusText += "...";
      }
   }

   // Can auto load normally (e.g., clicking "related jobs") if not already a search link
   if(autoloadIn && isFirstRender && !query.has("search")){
      queryJobs(searchState);
      setIsFirstRender(false);
      return null;
   }
   
   return (
   <Modal 
      isOpen={true} 
      topOffsetFixed={true} 
      styles={{ main: { padding: 8, width: 1400, height: '820px', backgroundColor: hordeTheme.horde.contentBackground, hasBeenOpened: false, top: "80px", position: "absolute" } }} 
      className={hordeClasses.modal} 
      onDismiss={() => { query.clear(["tab"]); onClose(); }}
   >
      {searchResults.querying && <StatusModal text={"Finding Jobs"} />}
      <Stack styles={{ root: { paddingTop: 8, paddingLeft: 24, paddingRight: 12, paddingBottom: 8 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal styles={{ root: { padding: 0 } }}>
               <Stack style={{ paddingLeft: 0, paddingTop: 4 }} grow>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{title}</Text>
               </Stack>
               <Stack grow horizontalAlign="end">
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     onClick={() => { query.clear(["tab"]); onClose(); }}
                  />
               </Stack>
            </Stack>

            <Stack horizontal tokens={{ childrenGap: 48 }} >

               <Stack>
                  <Stack tokens={{ childrenGap: 12 }}>

                     {!searchState.global && <Stack key={`stream_chooser_${streamChooserId}`}>
                        <Label>Stream</Label>
                        <StreamChooser defaultStreamId={streamId} width={352} allowAll={false} onChange={(streamId) => {
                           const templates = searchState.templates;
                           const global = searchState.global;
                           setSearchState({ ...searchState, streamIdOverride: streamId, global: streamId ? false : global, templates: streamId ? new Set() : templates })

                        }} />
                     </Stack>}

                     <Stack>
                        <Label>{searchState.templates.size > 1 ? "Templates" : "Template"}</Label>
                        <div ref={templateSearchRef}>
                           <Stack horizontal tokens={{ childrenGap: 4 }}>
                              {!templateSearchOpen && <DefaultButton disabled={searchState.global} style={{ width: 320, textAlign: "left" }} menuProps={templateMenuProps} text={templateText} />}
                              {templateSearchOpen && <TextField
                                 autoFocus
                                 disabled={searchState.global}
                                 value={templateFilter}
                                 spellCheck={false}
                                 autoComplete="off"
                                 style={{ width: 320 }}
                                 placeholder="Filter templates..."
                                 onChange={(ev, newValue) => setTemplateFilter(newValue ?? "")}
                                 onKeyDown={(ev) => { if (ev.key === "Escape") { setTemplateSearchOpen(false); setTemplateFilter(""); } }}
                              />}
                              <IconButton
                                 iconProps={{ iconName: templateSearchOpen ? "Cancel" : "Filter" }}
                                 title={templateSearchOpen ? "Close filter" : "Filter templates"}
                                 disabled={searchState.global}
                                 styles={{ root: { height: 32, width: 32 } }}
                                 onClick={() => {
                                    if (templateSearchOpen) {
                                       setTemplateSearchOpen(false);
                                       setTemplateFilter("");
                                    } else {
                                       setTemplateSearchOpen(true);
                                    }
                                 }}
                              />
                           </Stack>
                        </div>
                        {templateSearchOpen && templateFilter && templateSearchRef.current && (() => {
                           const filter = templateFilter.toLowerCase();
                           const filteredCategories: { category: string; templates: GetTemplateRefResponse[] }[] = [];
                           Array.from(sorted.keys()).sort((a, b) => a < b ? -1 : 1).forEach(cat => {
                              const catTemplates = sorted.get(cat);
                              if (!catTemplates?.length) return;
                              const matched = catTemplates.filter(t => t.name.toLowerCase().includes(filter) || t.id.toLowerCase().includes(filter));
                              if (matched.length > 0) {
                                 filteredCategories.push({ category: cat, templates: matched.sort((a, b) => a.name < b.name ? -1 : 1) });
                              }
                           });
                           const totalMatches = filteredCategories.reduce((sum, c) => sum + c.templates.length, 0);
                           return <Callout
                              target={templateSearchRef.current}
                              directionalHint={DirectionalHint.bottomLeftEdge}
                              isBeakVisible={false}
                              onDismiss={() => { setTemplateSearchOpen(false); setTemplateFilter(""); }}
                              styles={{ root: { width: 352 } }}
                           >
                              <div style={{ maxHeight: 320, overflowY: "auto", padding: "4px 0" }}>
                                 {totalMatches === 0 && <Stack style={{ padding: "8px 12px" }}><Text variant="small" styles={{ root: { color: "#888" } }}>No matching templates</Text></Stack>}
                                 {filteredCategories.map(({ category, templates: catTemplates }) => (
                                    <Stack key={category}>
                                       <Stack style={{ padding: "6px 12px 2px 12px" }}>
                                          <Text variant="smallPlus" styles={{ root: { fontWeight: 600, color: dashboard.darktheme ? "#ABABAB" : "#575757" } }}>{category}</Text>
                                       </Stack>
                                       {catTemplates.map(t => (
                                          <Stack key={t.id} horizontal verticalAlign="center" style={{ padding: "4px 12px 4px 20px", cursor: "pointer" }}
                                             onClick={(ev) => {
                                                ev.preventDefault();
                                                ev.stopPropagation();
                                                setSearchState(prev => {
                                                   const newTemplates = new Set(prev.templates);
                                                   if (newTemplates.has(t.id)) {
                                                      newTemplates.delete(t.id);
                                                   } else {
                                                      newTemplates.add(t.id);
                                                   }
                                                   return { ...prev, templates: newTemplates };
                                                });
                                             }}>
                                             <Checkbox checked={searchState.templates.has(t.id)} />
                                             <Text variant="small">{t.name}</Text>
                                          </Stack>
                                       ))}
                                    </Stack>
                                 ))}
                              </div>
                              {totalMatches > 0 && <Stack horizontal style={{ padding: "4px 12px", borderTop: "1px solid #eee" }} tokens={{ childrenGap: 12 }}>
                                 <Text variant="tiny" styles={{ root: { color: "#888", lineHeight: "24px" } }}>{totalMatches} match{totalMatches !== 1 ? "es" : ""}</Text>
                                 <DefaultButton text="Select All Matches" styles={{ root: { height: 24, minWidth: 0, fontSize: 11, padding: "0 8px" } }} onClick={() => {
                                    setSearchState(prev => {
                                       const newTemplates = new Set(prev.templates);
                                       filteredCategories.forEach(({ templates: ct }) => ct.forEach(t => newTemplates.add(t.id)));
                                       return { ...prev, templates: newTemplates };
                                    });
                                 }} />
                                 <DefaultButton text="Deselect All" styles={{ root: { height: 24, minWidth: 0, fontSize: 11, padding: "0 8px" } }} onClick={() => {
                                    setSearchState(prev => {
                                       const newTemplates = new Set(prev.templates);
                                       filteredCategories.forEach(({ templates: ct }) => ct.forEach(t => newTemplates.delete(t.id)));
                                       return { ...prev, templates: newTemplates };
                                    });
                                 }} />
                              </Stack>}
                           </Callout>;
                        })()}
                     </Stack>

                     <Stack>
                        <Label>{searchState.statuses.size > 1 ? "Statuses" : "Status"}</Label>
                        <DefaultButton style={{ width: 352, textAlign: "left" }} menuProps={statusMenuProps} text={statusText} />
                     </Stack>

                     {!searchState.global && <Stack horizontal tokens={{ childrenGap: 12 }} >
                        <TextField value={searchState.minCL ?? ""} style={{ width: 168 }} label="Min Changelist" onChange={(ev, newValue) => {
                           newValue = newValue?.trim();
                           if (!newValue) {
                              newValue = undefined;
                           }
                           setSearchState({ ...searchState, minCL: newValue })
                        }} />
                        <TextField value={searchState.maxCL ?? ""} style={{ width: 168 }} label="Max Changelist" onChange={(ev, newValue) => {
                           newValue = newValue?.trim();
                           if (!newValue) {
                              newValue = undefined;
                           }
                           setSearchState({ ...searchState, maxCL: newValue })
                        }} />
                     </Stack>}


                     <Stack horizontal tokens={{ childrenGap: 12 }} >
                        <DatePicker value={searchState.minDate} key="setMinDate" label="Min Date" onSelectDate={(date) => { setSearchState({ ...searchState, minDate: date ? date : undefined }) }} style={{ width: 168 }} />
                        <DatePicker value={searchState.maxDate} label="Max Date" onSelectDate={(date) => { setSearchState({ ...searchState, maxDate: date ? date : undefined }) }} style={{ width: 168 }} />
                     </Stack>

                     <Stack tokens={{ childrenGap: 4 }}>
                        <Label>Started By</Label>
                        <UserSelect defaultId={searchState.userId} handleSelection={(id => {
                           setSearchState({ ...searchState, userId: id })
                        })} userIdHints={[]} noResultsFoundText="No users found" />
                     </Stack>

                     {!searchState.global && <Stack tokens={{ childrenGap: 12 }}>
                        <Stack>
                           <TextField disabled={!ctemplates.length} value={searchState.containsParameter ?? ""} spellCheck={false} autoComplete="off" style={{ width: 350 }} label="Contains Parameter" placeholder={!ctemplates.length ? "Please select a template to search parameters" : ""} onChange={(event, newValue) => {
                              setSearchState({ ...searchState, containsParameter: newValue?.trim() ? newValue : undefined })
                           }} ></TextField>
                        </Stack>

                        <Stack>
                           <TextField value={searchState.containsStep ?? ""} spellCheck={false} autoComplete="off" style={{ width: 350 }} label="Contains Step" description="Note: Exact match is required" onChange={(event, newValue) => {
                              setSearchState({ ...searchState, containsStep: newValue?.trim() ? newValue : undefined })
                           }} ></TextField>
                        </Stack>
                     </Stack>}

                     <Stack>
                        <TextField value={searchState.name ?? ""} spellCheck={false} autoComplete="off" style={{ width: 350 }} label="Job Name" onChange={(event, newValue) => {
                           setSearchState({ ...searchState, name: newValue })

                        }} ></TextField>
                     </Stack>

                     <Stack horizontal tokens={{ childrenGap: 18 }}>
                        <Stack verticalFill={true} verticalAlign="center">
                           <Checkbox checked={searchState.preflights} label="Include Preflights" onChange={(ev, checked) => {
                              setSearchState({ ...searchState, preflights: checked ? checked : false })
                           }} />
                        </Stack>

                        <Stack verticalFill={true} verticalAlign="center">
                           <Checkbox checked={searchState.preflightOnly ?? false} label="Only Preflights" onChange={(ev, checked) => {
                              setSearchState({ ...searchState, preflightOnly: checked ? checked : undefined })
                           }} />
                        </Stack>
                     </Stack>


                     <Stack verticalFill={true} verticalAlign="center">
                        <Checkbox checked={searchState.global} label="Global Search" onChange={(ev, checked) => {
                           const templates = searchState.templates;
                           const streamIdOverride = searchState.streamIdOverride;
                           streamChooserId++;
                           setSearchState({ ...searchState, global: checked ? checked : false, templates: checked ? new Set() : templates, streamIdOverride: checked ? undefined : streamIdOverride })
                        }} />
                     </Stack>

                     <Stack horizontal style={{ paddingTop: 20 }}>
                        <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                           <PrimaryButton 
                              disabled={searchResults.querying} 
                              text="Search" 
                              onClick={() => {
                                 setIsFirstRender(false); // Important, prevents double query
                                 queryJobs(searchState);
                              }} 
                           />
                        </Stack>
                        <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                           <DefaultButton disabled={searchResults.querying} text="Clear" onClick={() => { 
                                 setSearchState({ preflights: false, userId: undefined, templates: new Set(), statuses: new Set()}) 
                                 setSearchResults({items: [], groups: [], querying: false});
                                 query.set("search", "");
                              }} />
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>

               <Stack>
                  <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingBottom: 4 } }}>
                     <Stack>
                        <Label>Jobs</Label>
                     </Stack>

                     {!searchResults.items.length && <Stack>
                        <Text>No Results</Text>
                     </Stack>}

                     {!!searchResults.items.length && <Stack>
                        <div style={{ overflowY: 'auto', overflowX: 'hidden', height: "700px" }} data-is-scrollable={true} >
                           <Stack tokens={{ childrenGap: 12 }} style={{ paddingRight: 12 }}>
                              <DetailsList
                                 compact={true}
                                 isHeaderVisible={false}
                                 indentWidth={0}
                                 items={searchResults.items}
                                 groups={searchResults.groups}
                                 columns={columns}
                                 setKey="set"
                                 selectionMode={SelectionMode.none}
                                 onRenderItemColumn={renderItem}
                                 onRenderRow={renderRow}
                                 groupProps={{
                                    onRenderHeader: onRenderGroupHeader,
                                 }}

                              />
                           </Stack>
                        </div>
                     </Stack>}
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>);
});


