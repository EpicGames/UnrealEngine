// Copyright Epic Games, Inc. All Rights Reserved.

import { ConstrainMode, DetailsList, DetailsListLayoutMode, DetailsRow, FontIcon, IColumn, IDetailsListProps, mergeStyleSets, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useState } from 'react';
import { Link, useParams } from 'react-router-dom';
import { useBackend } from '../backend';
import { GetJobsTabLabelColumnResponse, GetJobsTabParameterColumnResponse, GetLabelStateResponse, JobData, JobsTabColumnType, JobsTabData, LabelOutcome, LabelState, StreamData } from '../backend/Api';
import { LabelBadge } from './labels/LabelBadge';
import { CommitCache } from '../backend/CommitCache';
import dashboard from '../backend/Dashboard';
import { JobHandler } from '../backend/JobHandler';
import { filterJob, JobFilterSimple } from '../base/utilities/filter';
import { displayTimeZone } from '../base/utilities/timeUtils';
import { getLabelColor } from '../styles/colors';
import { ChangeButton } from './ChangeButton';
import { useQuery } from "horde/base/utilities/hooks";
import { JobOperationsContextMenu } from './JobOperationsContextMenu';
import { getHordeStyling } from '../styles/Styles';


type JobItem = {
   key: string;
   job: JobData;
   stream: StreamData;
   startedby: string;
   jobTab: JobsTabData;
};

let jobItems: JobItem[] = [];


type JobGroup = {
   count: number;
   key: string;
   name: string;
   headerText: string;
   startIndex: number;
   level?: number;
   isCollapsed?: boolean;
   loadJobs?: boolean;
}

const buildColumns = (jobTab: JobsTabData): IColumn[] => {

   const fixedWidths: Record<string, number | undefined> = {
      "Change": 46,
      "Build": 200
   };

   const minWidths: Record<string, number | undefined> = {};

   let cnames = ["Change"];
   if (jobTab.showNames) {
      cnames.push("Build");
   }

   // Always combine label columns into one "Labels" column
   cnames.push("Labels");

   // Keep Parameter columns as separate
   if (jobTab.columns) {
      const paramColumns = jobTab.columns.filter(c => c.type === JobsTabColumnType.Parameter);
      for (const pc of paramColumns) {
         minWidths[pc.heading] = 120;
         fixedWidths[pc.heading] = 120;
         cnames.push(pc.heading);
      }
   }

   const classNames = mergeStyleSets({
      header: {
         selectors: {
            ".ms-DetailsHeader-cellTitle": {
               padding: 0,
               paddingLeft: 4
            }
         }
      }
   });

   return cnames.map(c => {

      const isLabels = c === "Labels";
      const column = {
         key: c,
         name: "",
         headerClassName: classNames.header,
         fieldName: c.replace(" ", "").toLowerCase(),
         minWidth: isLabels ? 200 : (minWidths[c] ?? fixedWidths[c]),
         maxWidth: isLabels ? 1070 : fixedWidths[c],
         isPadded: false,
         isResizable: false,
         isCollapsible: false,
         isMultiline: true,
         flexGrow: isLabels ? 1 : 0
      } as IColumn;

      column.styles = (props: any): any => {
         props.cellStyleProps = { ...props.cellStyleProps };
         props.cellStyleProps.cellLeftPadding = 4;
         props.cellStyleProps.cellRightPadding = isLabels ? 16 : 0;
      };


      return column;
   });

};

let prevGroups: JobGroup[] = [];

const sortJobGroups = (items: JobItem[]): [JobItem[], JobGroup[]] => {

   // sort into time headers (fabric groups are indexed into one array so this is a bit complicated)
   // we also have to track collapsed state between renders
   const now = moment.utc().tz(displayTimeZone());
   const nowTimeStr = now.format('dddd, MMMM Do');
   const dateMap: Map<string, JobItem[]> = new Map();

   items.forEach((item) => {

      const time = moment(item.job!.createTime).tz(displayTimeZone());
      let timeStr = time.format('dddd, MMMM Do');
      if (timeStr === nowTimeStr) {
         timeStr += " (Today)";
      } else if (time.calendar().toLowerCase().indexOf("yesterday") !== -1) {
         timeStr += " (Yesterday)";
      } else {
         timeStr = time.format('dddd, MMMM Do');
      }
      if (!dateMap.has(timeStr)) {
         dateMap.set(timeStr, [item]);
      } else {
         dateMap.get(timeStr)?.push(item);
      }
   });

   // sort the header times
   let keys = Array.from(dateMap.keys());

   keys = keys.sort((a, b) => {
      const timeA = moment.utc(dateMap.get(a)![0].job.createTime);
      const timeB = moment.utc(dateMap.get(b)![0].job.createTime);
      return timeA < timeB ? 1 : -1;
   });

   // new create groups and sort group items
   const groups: JobGroup[] = [];
   let nitems: JobItem[] = [];

   keys.forEach(key => {

      let jobs = dateMap.get(key)!;
      jobs = jobs.sort((a, b) => {

         const changeA = a.job.change!;
         const changeB = b.job.change!;

         if (changeA === changeB) {
            const timeA = moment.utc(a.job.createTime);
            const timeB = moment.utc(b.job.createTime);

            return timeA < timeB ? 1 : -1;
         }

         return changeA < changeB ? 1 : -1;
      });

      const collapsed = prevGroups.find(group => group.key === key)?.isCollapsed;

      groups.push({
         startIndex: nitems.length,
         count: jobs.length,
         key: key,
         name: key,
         headerText: key,
         isCollapsed: collapsed ?? false
      });

      nitems = nitems.concat(jobs);
   });

   prevGroups = groups;

   return [nitems, groups];
};

const commitCache = new CommitCache();
const jobHandler = new JobHandler(false, undefined, commitCache);

const JobList: React.FC<{ tab: string; filter: JobFilterSimple; aliasTemplates?: string[] }> = observer(({ tab, filter, aliasTemplates }) => {

   const { streamId } = useParams<{ streamId: string }>();
   const query = useQuery();
   const { projectStore } = useBackend();
   const stream = projectStore.streamById(streamId);
   let [update, setUpdate] = useState(0);

   useEffect(() => {

      return () => {
         jobHandler.clear();
      };

   }, []);

   const { hordeClasses, detailClasses, modeColors } = getHordeStyling();

   const filterTemplate = query.get("template") ?? undefined;


   // @todo: this pattern is becoming annoying, have to reference once in render for observable to see
   if (jobHandler.updated) { }
   if (commitCache.updated) { }

   if (!streamId || !stream || !projectStore.streamById(streamId)) {
      console.error("bad stream id setting up JobList");
      return <div />;
   }

   const explicit: Set<string> = new Set();
   stream.tabs.forEach(t => {
      (t as JobsTabData).templates?.forEach(name => {
         explicit.add(name);
      });
   });

   const jobTab = stream.tabs.find(t => t.title === tab)! as JobsTabData;

   if (!jobTab) {
      console.error("Unable to get stream tab in JobList");
      return <div />;
   }

   const aliasTemplateSet = aliasTemplates ? new Set(aliasTemplates) : undefined;

   const templateNames: Set<string> = new Set();
   jobTab.templates?.forEach(name => {
      if (filterTemplate?.length && filterTemplate !== name) {
         return;
      }
      // Filter by alias templates if present
      if (aliasTemplateSet && !aliasTemplateSet.has(name)) {
         return;
      }
      templateNames.add(name);
   });

   if (!templateNames.size && filterTemplate?.length) {
      // Only add filterTemplate if it's in the alias templates (or no alias filter)
      if (!aliasTemplateSet || aliasTemplateSet.has(filterTemplate)) {
         templateNames.add(filterTemplate);
      }
   }

   let jobs = jobHandler.jobs.filter(j => filterJob(j, filter.filterKeyword));

   jobItems = jobs.map(j => {

      let startedBy = j.startedByUserInfo?.name ?? "Scheduler";
      if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
         startedBy = "TestAccount";
      }

      return {
         key: j.id,
         time: "",
         job: j,
         startedby: startedBy,
         stream: stream,
         jobTab: jobTab
      } as JobItem;

   });

   if (!templateNames.size) {
      jobItems = jobItems.filter(j => !explicit.has(j.job.name));
   }

   const [renderItems, groups] = sortJobGroups(jobItems.filter(item => item.job));

   if (jobHandler.jobs.length >= jobHandler.count || jobHandler.bumpCount) {
      const loadjobs = "loadjobs";
      groups.push({
         startIndex: renderItems.length,
         loadJobs: true,
         count: 1,
         key: `key_${loadjobs}_${update++}`,
         name: loadjobs,
         headerText: loadjobs
      })
   }

   // Fluent detail list group vertical estimates are problematic, so handle group row rendering ourselves
   const renderGroup = (jobGroup: JobGroup) => {


      if (jobGroup.loadJobs) {

         if (jobHandler.bumpCount) {
            return <Stack className={detailClasses.headerAndFooter} style={{ marginLeft: 24, marginRight: 24 }} tokens={{ childrenGap: 12 }} horizontal horizontalAlign="center">
               <Stack className={detailClasses.headerTitle} style={{ padding: 0 }}>{`Loading Jobs`}</Stack>
               <Stack className={detailClasses.headerTitle} style={{ padding: 0 }} verticalAlign="center"><Spinner size={SpinnerSize.medium} /></Stack>
            </Stack>

         }

         return <Stack className={detailClasses.headerAndFooter} style={{ cursor: "pointer", marginLeft: 24, marginRight: 24 }} tokens={{ childrenGap: 8 }} onClick={() => { jobHandler.addJobs(); setUpdate(update++) }} horizontal horizontalAlign="center">
            <Stack className={detailClasses.headerTitle} style={{ padding: 0 }}>{`Showing ${jobs.length} of ${jobHandler.jobs.length} Jobs.`}</Stack>
            <Stack className={detailClasses.headerTitle} style={{ padding: 0, color: "rgb(0, 120, 212)" }} >Show more...</Stack>
         </Stack>
      }

      return (
         <div className={detailClasses.headerAndFooter} style={{ marginLeft: 24, marginRight: 24 }} onClick={() => { }}>
            <div className={detailClasses.headerTitle}>{`${jobGroup.headerText}`}</div>
         </div>
      );

   };

   const filterLabels = (job: JobData, category: string | undefined) => {

      const labels = job.labels;

      if (!labels || !labels.length) {
         return [];
      }

      const view = labels.filter(a => {

         const idx = labels.indexOf(a);
         if (!job.labels) {
            return false;
         }

         if (idx === -1) {
            return false;
         }

         if (!job.labels[idx]) {
            console.error("Unexpected label index when filtering job view");
            return false;
         }

         return job.labels[idx].state !== LabelState.Unspecified;
      });

      const unassigned: GetLabelStateResponse[] = [];

      view.forEach(label => {
         if (!label.dashboardCategory || !jobTab?.columns?.find(c => { return ((c as GetJobsTabLabelColumnResponse).category === label.dashboardCategory) })) {
            unassigned.push(label);
         }
      });

      return view.filter(label => (label.dashboardName && ((label.dashboardCategory === category) || ((category === "Other" || !category) && unassigned.indexOf(label) !== -1)))).sort((a, b) => a.dashboardName! < b.dashboardName! ? -1 : 1);

   };

   const JobLabel: React.FC<{ item: JobItem; column: GetJobsTabLabelColumnResponse; label: GetLabelStateResponse }> = ({ item, column, label }) => {

      const defaultLabel = item.job.defaultLabel;

      if (label === defaultLabel && column.category !== "Other" && column.heading !== "Other") {
         return <div />;
      }

      const aggregates = item.job.labels;

      // note details may not be loaded here, as only initialized on callout for optimization (details.getLabelIndex(label.Name, label.Category);)
      const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
      let labelIdx = -1;
      if (jlabel) {
         labelIdx = aggregates?.indexOf(jlabel)!;
      }

      let state: LabelState | undefined;
      let outcome: LabelOutcome | undefined;
      if (label === defaultLabel) {
         state = defaultLabel.state;
         outcome = defaultLabel.outcome;
      } else {
         state = item.job.labels![labelIdx]?.state;
         outcome = item.job.labels![labelIdx]?.outcome;
      }

      return <LabelBadge
         label={{ ...label, state: state!, outcome: outcome! }}
         jobId={item.job.id}
         labelIndex={labelIdx}
         streamId={streamId!}
         templateId={item.job.templateId!}
         templateName={item.job.name}
         change={item.job.change}
      />;

   };

   function onRenderItemColumnInner(item: JobItem, _index?: number, column?: IColumn) {

      const fieldContent = item[column!.fieldName as keyof JobItem] as string;

      const jobColumn = item.jobTab?.columns?.find(c => c.heading === column!.key);

      const style: any = {
         margin: 0,
      };

      if (jobColumn?.type === JobsTabColumnType.Parameter) {
         const paramColumn = jobColumn as GetJobsTabParameterColumnResponse;
         let arg = item.job.arguments?.find(a => a.toLowerCase().startsWith(paramColumn.parameter?.toLowerCase() ?? "____"))
         arg = arg?.split("=")[1];
         return <Stack verticalAlign="center" verticalFill={true}><Text>{arg}</Text></Stack>;
      }

      // Combined Labels column: render all categories grouped together
      if (column!.key === "Labels") {
         const labelColumns = (item.jobTab?.columns?.filter(c => c.type === JobsTabColumnType.Labels) ?? []) as GetJobsTabLabelColumnResponse[];

         // Build ordered category list from tab columns, plus "Other" for uncategorized
         const categories: { name: string; category: string | undefined }[] = [];
         for (const lc of labelColumns) {
            categories.push({ name: lc.heading, category: lc.category });
         }

         // If no label columns defined, fall back to a single "Other" category
         if (!categories.length) {
            categories.push({ name: "Other", category: "Other" });
         }

         let rowIdx = 0;
         return <div className={`horde-no-darktheme ${hordeClasses.labelCategoryRow}`} style={{
            display: "grid",
            gridTemplateColumns: "auto 1fr",
            columnGap: 8,
            width: "100%"
         }}>
               {categories.map(cat => {
                  const isOtherCategory = cat.category === "Other" || cat.name === "Other" || !cat.category;
                  const labels = filterLabels(item.job!, cat.category);

                  // Add default label for "Other" category
                  const defaultLabel = item.job.defaultLabel;
                  if (defaultLabel && isOtherCategory) {
                     defaultLabel.dashboardName = "Other";
                     if (!labels.find(l => l === defaultLabel)) {
                        labels.push(defaultLabel);
                     }
                  }

                  if (!labels.length) return null;

                  if (!labels.length) return null;

                  // Sort labels: failures first, then warnings, then by name
                  const outcomePriority: Record<string, number> = {
                     [LabelOutcome.Failure]: 0,
                     [LabelOutcome.Warnings]: 1,
                     [LabelOutcome.Success]: 2,
                     [LabelOutcome.Unspecified]: 3
                  };
                  labels.sort((a, b) => {
                     const aggregates = item.job.labels;
                     const getOutcome = (label: GetLabelStateResponse): LabelOutcome => {
                        if (label === item.job.defaultLabel) return label.outcome ?? LabelOutcome.Unspecified;
                        const jl = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
                        if (!jl) return LabelOutcome.Unspecified;
                        const idx = aggregates?.indexOf(jl)!;
                        return item.job.labels![idx]?.outcome ?? LabelOutcome.Unspecified;
                     };
                     const pa = outcomePriority[getOutcome(a)] ?? 3;
                     const pb = outcomePriority[getOutcome(b)] ?? 3;
                     if (pa !== pb) return pa - pb;
                     return (a.dashboardName ?? "") < (b.dashboardName ?? "") ? -1 : 1;
                  });

                  // Compute worst-case state/outcome across this category's labels
                  let catState: LabelState = LabelState.Complete;
                  let catOutcome: LabelOutcome = LabelOutcome.Success;
                  for (const label of labels) {
                     let ls: LabelState | undefined;
                     let lo: LabelOutcome | undefined;
                     if (label === item.job.defaultLabel) {
                        ls = label.state;
                        lo = label.outcome;
                     } else {
                        const aggregates = item.job.labels;
                        const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
                        if (!jlabel) { continue; }
                        const idx = aggregates?.indexOf(jlabel)!;
                        ls = item.job.labels![idx]?.state;
                        lo = item.job.labels![idx]?.outcome;
                     }
                     if (ls === LabelState.Running) catState = LabelState.Running;
                     if (lo === LabelOutcome.Failure) catOutcome = LabelOutcome.Failure;
                     else if (lo === LabelOutcome.Warnings && catOutcome !== LabelOutcome.Failure) catOutcome = LabelOutcome.Warnings;
                  }
                  const catColor = getLabelColor(catState, catOutcome);
                  const isEven = rowIdx++ % 2 === 0;
                  const rowBg = isEven ? (dashboard.darktheme ? "rgba(255,255,255,0.04)" : "rgba(0,0,0,0.03)") : "transparent";

                  const labelColumn = labelColumns.find(lc => lc.category === cat.category) ?? labelColumns[0];

                  return <React.Fragment key={cat.name}>
                     {/* Category name cell */}
                     <div style={{
                        display: "flex", alignItems: "center", gap: 6,
                        padding: "3px 4px", backgroundColor: rowBg,
                        whiteSpace: "nowrap"
                     }}>
                        <FontIcon iconName="CircleFill" style={{
                           fontSize: 8,
                           color: catColor.primaryColor,
                           flexShrink: 0
                        }} />
                        <Text variant="small" style={{
                           color: modeColors.text,
                           fontFamily: "Horde Open Sans SemiBold",
                           fontSize: 11
                        }}>{cat.name}</Text>
                     </div>
                     {/* Badges cell */}
                     <div style={{
                        display: "flex", flexWrap: "wrap", alignItems: "center",
                        gap: "2px 3px", padding: "3px 0", backgroundColor: rowBg
                     }}>
                        {labels.map((label, idx) => (
                           <JobLabel key={`${item.job.id}_${cat.name}_${label.dashboardName}_${label.dashboardCategory}_${idx}`}
                              item={item}
                              column={labelColumn ?? { heading: "Other", category: "Other" } as GetJobsTabLabelColumnResponse}
                              label={label} />
                        ))}
                     </div>
                  </React.Fragment>;
               })}
         </div>;
      }

      if (column!.key === "Build") {
         const displayTime = moment(item.job!.createTime).tz(displayTimeZone());
         const format = dashboard.display24HourClock ? "HH:mm z" : "LT z";
         const displayTimeStr = displayTime.format(format);
         const startedBy = item.startedby;

         return <Stack verticalAlign="center" verticalFill={true} tokens={{ childrenGap: 1 }}>
            <Text variant="small" style={{ whiteSpace: "normal" }}>{item.job.name}</Text>
            <Text variant="xSmall" style={{ color: modeColors.text, opacity: 0.7 }}>{startedBy} &middot; {displayTimeStr}</Text>
         </Stack>;
      }

      const commit = streamId ? commitCache.getCommit(streamId, item.job.change!) : undefined;

      switch (column!.key) {

         case 'Change':
            return <ChangeButton job={item.job} commit={commit} />;

         default:
            return <Stack style={style} verticalAlign="center" verticalFill={true} horizontalAlign={"center"}><Text>{fieldContent}</Text></Stack>;
      }
   }

   function onRenderItemColumn(item: JobItem, index?: number, column?: IColumn) {

      if (column?.key === "Change") {
         return onRenderItemColumnInner(item, index, column);
      }

      return <Link to={`/job/${item.job.id}`}>{onRenderItemColumnInner(item, index, column)}</Link>;

   }

   const onRenderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         let group = groups.find(g => g.startIndex === props.itemIndex);

         const item = renderItems[props.itemIndex];

         const row = <JobOperationsContextMenu job={item.job}>
            <Stack style={{ marginLeft: 24, marginRight: 24 }}>
               <DetailsRow styles={{ root: { paddingTop: 8, paddingBottom: 8, width: "100%" }, cell: { selectors: { "a, a:visited, a:active, a:hover": { color: modeColors.text } } } }} {...props} />
            </Stack>
         </JobOperationsContextMenu>;

         if ((props.itemIndex === renderItems.length - 1) && groups.find(g => !!g.loadJobs)) {
            return <React.Fragment>
               {row}
               {renderGroup(groups.find(g => !!g.loadJobs)!)}
            </React.Fragment>

         }

         if (group) {
            return <React.Fragment>
               {renderGroup(group)}
               {row}
            </React.Fragment>
         }


         return row;

      }
      return null;
   };

   const forcePreflights = dashboard.showPreflights || !!jobTab?.showPreflights || tab?.toLowerCase() === "swarm" || tab?.toLowerCase() === "presubmit" || tab?.toLowerCase() === "services";
   let preflightStartedByUserId: string | undefined;

   if (!forcePreflights && !filter.showOthersPreflights) {
      preflightStartedByUserId = dashboard.userId;
   }

   jobHandler.filter(stream, templateNames.size ? Array.from(templateNames.values()) : undefined, preflightStartedByUserId);


   let nojobs = !jobHandler.initial && !jobs.length;

   const width = 1440;

   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = () => {
      return null as any;
   };


   return (
      <Stack tokens={{ childrenGap: 0 }} className={detailClasses.detailsRow}>
         <div className={detailClasses.container} style={{ width: "100%", height: 'calc(100vh - 240px)', position: 'relative', marginTop: 0 }}>
            {<ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} style={{ overflow: "visible" }}>
               {renderItems.length > 0 &&
                  <Stack style={{ width: width, maxWidth: width, overflow: "hidden", marginLeft: 4, background: modeColors.content, boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)" }}>
                     <DetailsList
                        styles={{ root: { paddingBottom: 32 }, headerWrapper: { overflow: "hidden" } }}
                        indentWidth={0}
                        compact={false}
                        selectionMode={SelectionMode.none}
                        items={renderItems}
                        columns={buildColumns(jobTab)}
                        layoutMode={DetailsListLayoutMode.fixedColumns}
                        constrainMode={ConstrainMode.unconstrained}
                        onRenderRow={onRenderRow}
                        onRenderDetailsHeader={onRenderDetailsHeader}
                        onRenderItemColumn={onRenderItemColumn}
                        onShouldVirtualize={() => { return true; }}
                     />
                  </Stack>
               }

               {nojobs && <Stack style={{ width: width }}>
                  <Stack horizontalAlign="center" styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                     <Text variant="mediumPlus">No jobs found</Text>
                  </Stack>
               </Stack>
               }

               {!nojobs && !renderItems.length && <Stack styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                  <Stack style={{ width: width }}><Spinner size={SpinnerSize.large} /></Stack>
               </Stack>
               }
            </ScrollablePane>
            }
         </div>
      </Stack>
   );

});

export const JobView: React.FC<{ tab: string; filter: JobFilterSimple; aliasTemplates?: string[] }> = ({ tab, filter, aliasTemplates }) => {

   return (<JobList tab={tab} filter={filter} aliasTemplates={aliasTemplates} />);

};
