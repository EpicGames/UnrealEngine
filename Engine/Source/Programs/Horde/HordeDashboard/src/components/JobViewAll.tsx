// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, DetailsRow, FocusZone, FocusZoneDirection, FontIcon, IColumn, IDetailsListProps, mergeStyleSets, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import moment from "moment-timezone";
import React, { useEffect } from "react";
import { Link, useParams } from "react-router-dom";
import { useBackend } from "../backend";
import { GetLabelStateResponse, GetStepResponse, JobData, JobState, JobStepOutcome, LabelOutcome, LabelState, StepData, StreamData } from "../backend/Api";
import { CommitCache } from "../backend/CommitCache";
import dashboard from "../backend/Dashboard";
import { JobHandler } from "../backend/JobHandler";
import { filterJob, JobFilterSimple } from "../base/utilities/filter";
import { displayTimeZone } from '../base/utilities/timeUtils';
import { getJobStateColor, getLabelColor } from "../styles/colors";
import { getHordeStyling } from "../styles/Styles";
import { ChangeButton } from "./ChangeButton";
import { LabelHoverCard } from './labels/LabelHoverCard';
import { jobFilter, JobFilterBar } from "./JobFilterBar";
import { JobOperationsContextMenu } from "./JobOperationsContextMenu";
import { StepStatusIcon } from "./StatusIcon";


type JobItem = {
   key: string;
   job: JobData;
   stream: StreamData;
   startedby: string;
};

const customStyles = mergeStyleSets({
   detailsRow: {
      selectors: {
         '.ms-DetailsRow': {
            borderBottom: '0px',
            width: "100%"
         },
         '.ms-DetailsRow-cell': {
            position: 'relative',
            textAlign: "center",
            padding: 0,
            overflow: "visible",
            whiteSpace: "nowrap"
         }

      }
   },
   header: {
      selectors: {
         ".ms-DetailsHeader-cellTitle": {
            padding: 0,
            paddingLeft: 4
         }
      }
   }

});


const commitCache = new CommitCache();
const jobHandler = new JobHandler(true, 300, commitCache);

const JobFilterPanel: React.FC = () => {

   const { streamId } = useParams<{ streamId: string }>();

   return <Stack>
      <JobFilterBar streamId={streamId!} />
   </Stack>
}

const JobViewAllInner: React.FC<{ filter: JobFilterSimple }> = observer(({ filter }) => {

   const { streamId } = useParams<{ streamId: string }>();
   const { projectStore } = useBackend();

   useEffect(() => {

      return () => {
         jobHandler.clear();
      };

   }, []);

   const { hordeClasses, detailClasses, modeColors } = getHordeStyling();


   // subscribe
   if (jobHandler.updated) { }
   if (jobFilter.updated) { }
   if (commitCache.updated) { }

   const stream = projectStore.streamById(streamId);

   if (!streamId || !stream || !projectStore.streamById(streamId)) {
      console.error("bad stream id setting up JobList");
      return <div />;
   }

   let jobs = jobHandler.getFilteredJobs(jobFilter.status, jobFilter.minCL, jobFilter.maxCL);
   jobs = jobs.filter(j => filterJob(j, filter.filterKeyword));

   let jobItems: JobItem[] = jobs.map(j => {

      let startedBy = j.startedByUserInfo?.name ?? "Scheduler";
      if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
         startedBy = "TestAccount";
      }

      return {
         key: j.id,
         job: j,
         startedby: startedBy,
         stream: stream
      } as JobItem;

   });

   let columns: IColumn[] = [
      { key: 'jobview_column1', name: 'Status', minWidth: 16, maxWidth: 16, onRenderHeader: () => null },
      { key: 'jobview_column2', name: 'Change', minWidth: 64, maxWidth: 64 },
      { key: 'jobview_column3', name: 'Job', minWidth: 220, maxWidth: 220 },
      { key: 'jobview_column4', name: 'Labels', minWidth: 300, maxWidth: 300 },
      { key: 'jobview_column5', name: 'Steps', minWidth: 324, maxWidth: 324 },
      { key: 'jobview_column6', name: 'StartedBy', minWidth: 290, maxWidth: 290 },
      { key: 'jobview_column7', name: 'Time', minWidth: 50, maxWidth: 50 },

   ];

   columns = columns.map(c => {

      c.isResizable = true;
      c.isPadded = false;
      c.isMultiline = true;
      c.isCollapsible = false;
      c.headerClassName = customStyles.header;

      c.styles = (props: any): any => {
         props.cellStyleProps = { ...props.cellStyleProps };
         props.cellStyleProps.cellLeftPadding = 4;
         props.cellStyleProps.cellRightPadding = 0;
      };

      return c;

   })

   const renderSteps = (job: JobData) => {

      type StepItem = {
         step: StepData;
         name: string;
      }

      const jobId = job.id;

      if (!job.batches) {
         return null;
      }

      let steps: GetStepResponse[] = job.batches.map(b => b.steps).flat().filter(step => !!step.startTime && (step.outcome === JobStepOutcome.Warnings || step.outcome === JobStepOutcome.Failure));

      if (!steps || !steps.length) {
         return null;
      }

      const onRenderCell = (stepItem: StepItem): JSX.Element => {

         const step = stepItem.step;

         const stepUrl = `/job/${jobId}?step=${step.id}`;

         return <Stack tokens={{ childrenGap: 12 }} key={`step_${step.id}_job_${job.id}_${stepItem.name}`}>
            <Stack horizontal>
               <Link to={stepUrl} onClick={(ev) => { ev.stopPropagation(); }}><div style={{ cursor: "pointer" }}>
                  <Stack horizontal>
                     <StepStatusIcon step={step} style={{ fontSize: 10 }} />
                     <Text styles={{ root: { fontSize: 10, paddingRight: 4, userSelect: "none" } }}>{`${stepItem.name}`}</Text>
                  </Stack>
               </div></Link>
            </Stack>
         </Stack>;
      };

      const andMore = (errors: number, warnings: number): JSX.Element => {

         return <Stack tokens={{ childrenGap: 12 }} key={`job_${job.id}_andmore`}>
            <Stack horizontal>
               <Stack horizontal>
                  <Link to={`/job/${jobId}`} onClick={(ev) => { ev.stopPropagation(); }}><div style={{ cursor: "pointer" }}>
                     <Text styles={{ root: { fontSize: 10, paddingRight: 4, paddingLeft: 19, userSelect: "none" } }}>{`( +${errors + warnings} more )`}</Text>
                  </div></Link>
               </Stack>
            </Stack>
         </Stack>;
      };


      let items = steps.map(step => {


         return {
            step: step,
            name: step.name
         }

      })

      items = items.filter(item => !!item);

      items = items.sort((a, b) => {

         const stepA = a!.step;
         const stepB = b!.step;

         const outA = stepA.outcome;
         const outB = stepB.outcome;

         if (outA === JobStepOutcome.Failure && outB === JobStepOutcome.Warnings) {
            return -1;
         }

         if (outA === JobStepOutcome.Warnings && outB === JobStepOutcome.Failure) {
            return 1;
         }

         return a!.name < b!.name ? -1 : 1;

      });

      let numErrors = 0;
      let numWarnings = 0;

      items.forEach(i => {
         if (i?.step.outcome === JobStepOutcome.Warnings) {
            numWarnings++;
         } else {
            numErrors++;
         }
      });

      items = items.slice(0, 10);

      const render = items.map(s => onRenderCell(s!));

      let shownErrors = 0;
      let shownWarnings = 0;

      items.forEach(i => {
         if (i?.step.outcome === JobStepOutcome.Warnings) {
            shownWarnings++;
         } else {
            shownErrors++;
         }
      });

      if (shownErrors !== numErrors || numWarnings !== shownWarnings) {
         render.push(andMore(numErrors - shownErrors, numWarnings - shownWarnings));
      }

      return render;

   }

   const renderDetailStatus = (job: JobData): JSX.Element => {

      if (job.state === JobState.Complete) {
         return <div />;
      }

      return <Stack className="horde-no-darktheme" ><FontIcon iconName="FullCircle" style={{ color: getJobStateColor(job.state) }} /></Stack>;
   };


   const renderItemInner = (item: JobItem, index?: number, column?: IColumn) => {

      const job = item.job;

      const style: any = {
         margin: 0,
         position: "absolute",
         top: "50%",
         left: "50%",
         msTransform: "translate(-50%, -50%)",
         transform: "translate(-50%, -50%)"
      };

      if (column!.name === "Space") {
         return <div />
      }

      if (column!.name === "Labels") {
         return <Stack verticalAlign="center" verticalFill={true}>{renderLabels(item)}</Stack>;

      }

      if (column!.name === "Change") {
         const commit = streamId ? commitCache.getCommit(streamId, job.change!) : undefined;
         return <ChangeButton job={job} commit={commit} />;

      }

      if (column!.name === "StartedBy") {
         let startedBy = job.startedByUserInfo?.name ?? "Scheduler";
         if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
            startedBy = "TestAccount";
         }

         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign="start">{startedBy}</Stack>

      }
      if (column!.name === "Steps") {
         return <Stack verticalAlign="center" verticalFill={true} style={{ paddingLeft: 64 }}>{renderSteps(job)}</Stack>;
      }


      if (column!.name === "Job") {

         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign="center" style={{ whiteSpace: "normal" }}>{item.job.name}</Stack>

      }

      if (column!.name === "Time") {

         const displayTime = moment(item.job!.createTime).tz(displayTimeZone());

         const format = dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z";

         let displayTimeStr = displayTime.format(format);

         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"}>{displayTimeStr}</Stack>;

      }

      if (column!.name === "Status") {
         return <div style={style}>{renderDetailStatus(job)}</div>;
      }

      return <Stack style={style} horizontalAlign="center">{item.job.state}</Stack>
   }

   const renderItem = (item: JobItem, index?: number, column?: IColumn) => {

      if (column?.name === "Change") {
         return renderItemInner(item, index, column);
      } else {
         return <Link to={`/job/${(item as JobItem).job.id}`}>{renderItemInner(item, index, column)}</Link>;
      }
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = jobItems[props.itemIndex];

         return <JobOperationsContextMenu job={item.job}>
            <DetailsRow styles={{ root: { paddingTop: 8, paddingBottom: 8, width: "100%" }, cell: { selectors: { "a, a:visited, a:active, a:hover": { color: modeColors.text } } } }} {...props} />
         </JobOperationsContextMenu>
      }
      return null;
   };

   const JobLabel: React.FC<{ item: JobItem; label: GetLabelStateResponse }> = ({ item, label }) => {

      const defaultLabel = item.job.defaultLabel;
      const aggregates = item.job.labels;

      // note details may not be loaded here, as only initialized on callout for optimization (details.getLabelIndex(label.Name, label.Category);)
      const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
      let labelIdx = -1;
      if (jlabel) {
         labelIdx = aggregates?.indexOf(jlabel)!;
      }

      let state: LabelState | undefined;
      let outcome: LabelOutcome | undefined;
      if (label === defaultLabel) {
         state = defaultLabel.state;
         outcome = defaultLabel.outcome;
      } else {
         state = item.job.labels![labelIdx]?.state;
         outcome = item.job.labels![labelIdx]?.outcome;
      }

      const color = getLabelColor(state, outcome);


      let url = `/job/${item.job.id}`;

      if (labelIdx >= 0) {
         url = `/job/${item.job.id}?label=${labelIdx}`;
      }

      const target = `label_${item.job.id}_${label.dashboardName}_${label.dashboardCategory}`.replace(/ /g, "");

      const badge = <Link to={url}><Stack className={hordeClasses.badgeNoIcon}>
                  <DefaultButton key={label.dashboardName} style={{ backgroundColor: color.primaryColor, fontSize: 9 }} text={label.dashboardName}>
                     {!!color.secondaryColor && <div style={{
                        borderLeft: "10px solid transparent",
                        borderRight: `10px solid ${color.secondaryColor}`,
                        borderBottom: "10px solid transparent",
                        height: 0,
                        width: 0,
                        position: "absolute",
                        right: 0,
                        top: 0,
                        zIndex: 1
                     }} />}
                  </DefaultButton>
               </Stack>
               </Link>;

      return <Stack>
         <div id={target}>
            {labelIdx >= 0 ? <LabelHoverCard
               jobId={item.job.id}
               labelIndex={labelIdx}
               streamId={streamId!}
               templateId={item.job.templateId!}
               templateName={item.job.name}
               dashboardName={label.dashboardName}
               dashboardCategory={label.dashboardCategory}
               state={state}
               outcome={outcome}
               change={item.job.change}
            >
               {badge}
            </LabelHoverCard> : badge}
         </div>
      </Stack>;
   };

   const renderLabels = (item: JobItem): JSX.Element => {

      const job = item.job!;
      const defaultLabel = job.defaultLabel;
      const labels = job.labels ?? [];

      if (!labels.length && !defaultLabel) {
         return <div />;
      }

      const view = labels.filter(a => {
         const idx = labels.indexOf(a)!;
         if (!job.labels) {
            return false;
         }
         return job.labels[idx].state !== LabelState.Unspecified;
      });

      const catergories = new Set<string>();
      view.forEach(label => { if (label.dashboardCategory?.length) catergories.add(label.dashboardCategory) });

      const sorted = Array.from(catergories).sort((a, b) => { return a < b ? -1 : 1 });

      if (!catergories.has("Other") && defaultLabel) {

         defaultLabel.dashboardCategory = "Other";
         defaultLabel.dashboardName = "Other";
         view.push(defaultLabel);         
         sorted.push("Other");
      }

      let key = 0;

      const labelStacks = sorted.map(cat => {

         const buttons = view.filter(label => label.dashboardCategory === cat).map(label => {
            return <JobLabel key={`label_${item.job.id}_${label.dashboardName}_${key++}`} item={item} label={label} />;
         });

         if (!buttons.length) {
            return <div />;
         }

         return <Stack tokens={{ childrenGap: 8 }}>
            <Stack horizontal tokens={{ childrenGap: 8 }}>
               <Stack horizontalAlign="end" style={{ minWidth: 100, maxWidth: 100 }}>
                  <Text variant="small" style={{}}>{cat}:</Text>
               </Stack>
               <Stack className="horde-no-darktheme" wrap horizontal horizontalAlign="start" tokens={{ childrenGap: 4 }}>
                  {buttons}
               </Stack>
            </Stack>
         </Stack>
      })


      return (
         <Stack tokens={{ childrenGap: 4 }} styles={{ root: { paddingTop: 2 } }}>
            {labelStacks}
         </Stack>

      );

   };

   if (jobFilter.templates && jobFilter.templates.length) {

      let preflightStartedByUserId: string | undefined;
      if (!filter.showOthersPreflights) {
         preflightStartedByUserId = dashboard.userId;
      }

      if (jobHandler.filter(stream, jobFilter.templates?.map(t => t.id), preflightStartedByUserId)) {

      }
   }

   let nojobs = !jobHandler.initial && !jobs.length;

   return (
      <Stack>
         <Stack tokens={{ childrenGap: 0 }} style={{}}>
            <Stack horizontalAlign="center" style={{ width: 1440, marginLeft: 4, boxShadow: "0 3px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)" }}>
               <Stack style={{ paddingTop: 18, paddingBottom: 24 }}>
                  <JobFilterPanel />
               </Stack>
            </Stack>
         </Stack>
         <Stack tokens={{ childrenGap: 0 }} className={customStyles.detailsRow}>
            <FocusZone direction={FocusZoneDirection.vertical}>
               <div className={detailClasses.container} style={{ height: 'calc(100vh - 352px)', position: 'relative', marginTop: 0 }} data-is-scrollable={true}>
                  {<ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} style={{ overflow: "visible" }}>
                     <Stack style={{ width: 1440, marginLeft: 4, background: modeColors.content, boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)" }}>
                        <Stack>
                           <DetailsList
                              styles={{ root: { paddingLeft: 8, paddingRight: 8 } }}
                              compact={true}
                              indentWidth={0}
                              items={jobItems}
                              columns={columns}
                              setKey="set"
                              selectionMode={SelectionMode.none}
                              layoutMode={DetailsListLayoutMode.fixedColumns}
                              onRenderItemColumn={renderItem}
                              onRenderRow={renderRow}
                              onShouldVirtualize={() => true}
                           />
                        </Stack>
                     </Stack>

                     {nojobs && <Stack style={{ width: 1364 }}>
                        <Stack horizontalAlign="center" styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                           <Text variant="mediumPlus">No jobs found</Text>
                        </Stack>
                     </Stack>
                     }

                     {!nojobs && !jobItems.length && <Stack styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                        <Stack style={{ width: 1440 }}><Spinner size={SpinnerSize.large} /></Stack>
                     </Stack>
                     }

                  </ScrollablePane>
                  }
               </div>
            </FocusZone>
         </Stack>
      </Stack>
   );

});


export const JobViewAll: React.FC<{ filter: JobFilterSimple }> = ({ filter }) => {

   return (<JobViewAllInner filter={filter} />);

};
