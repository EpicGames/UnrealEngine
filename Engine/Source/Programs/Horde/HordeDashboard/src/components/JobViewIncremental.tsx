// Copyright Epic Games, Inc. All Rights Reserved.

import { CollapseAllVisibility, ConstrainMode, DefaultButton, DetailsHeader, DetailsList, DetailsListLayoutMode, DetailsRow, FocusZone, FocusZoneDirection, FontIcon, IColumn, IDetailsListProps, mergeStyleSets, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Sticky, StickyPositionType, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useState } from 'react';
import { Link, useParams } from 'react-router-dom';
import { useBackend } from '../backend';
import { GetChangeSummaryResponse, GetJobsTabLabelColumnResponse, GetJobsTabParameterColumnResponse, GetLabelStateResponse, JobData, JobsTabColumnType, JobsTabData, JobState, LabelOutcome, LabelState, StreamData } from '../backend/Api';
import { CommitCache } from '../backend/CommitCache';
import dashboard from '../backend/Dashboard';
import { JobHandler } from '../backend/JobHandler';
import { getPreflightDisplay } from '../base/utilities/commitUtils';
import { filterJob, JobFilterSimple } from '../base/utilities/filter';
import { displayTimeZone } from '../base/utilities/timeUtils';
import { getJobStateColor, getLabelColor } from '../styles/colors';
import { ChangeContextMenu, ChangeContextMenuTarget } from './ChangeButton';
import { JobOperationsContextMenu } from './JobOperationsContextMenu';
import { LabelHoverCard } from './labels/LabelHoverCard';
import { getHordeStyling } from '../styles/Styles';


const lineHeight = 25;

type JobItem = {
   key: string;
   job: JobData;
   stream: StreamData;
   startedby: string;
   jobTab: JobsTabData;
};

let jobItems: JobItem[] = [];


type JobGroup = {
   count: number;
   key: string;
   name: string;
   headerText: string;
   startIndex: number;
   level?: number;
   isCollapsed?: boolean;
   loadJobs?: boolean;
}

const commitCache = new CommitCache();
const jobHandler = new JobHandler(false, undefined, commitCache);

const buildColumns = (jobTab: JobsTabData, streamId: string): IColumn[] => {

   const startedByName = jobTab.showNames ? "Started By" : "Change";

   const fixedWidths: Record<string, number | undefined> = {
      "Status": 16, // note this doesn't have header text, and need to match the status dot width of 20 after sizing
      "Time": 60,      
   };

   fixedWidths[startedByName] = jobTab.showNames ? 180 : 32;

   const minWidths: Record<string, number | undefined> = {};

   let cnames = ["Status", startedByName];

   if (jobTab.columns) {

      let total = 0;
      jobTab.columns.forEach(c => total += c.relativeWidth ?? 1);
      const w = (jobTab.showNames ? 940 : 940 + 144) / total;

      jobTab.columns.forEach(c => { minWidths[c.heading] = w * (c.relativeWidth ?? 1); cnames.push(c.heading); });
   } else {
      cnames.push("Labels");
   }

   cnames = cnames.concat(["Time"]);

   const classNames = mergeStyleSets({
      header: {
         selectors: {
            ".ms-DetailsHeader-cellTitle": {
               padding: 0,
               paddingLeft: 4
            }
         }
      }
   });

   return cnames.map(c => {

      const column = { key: c, name: c === "Status" ? "" : c, headerClassName: classNames.header, fieldName: c.replace(" ", "").toLowerCase(), minWidth: minWidths[c] ?? fixedWidths[c], maxWidth: fixedWidths[c], isPadded: false, isResizable: false, isCollapsible: false, isMultiline: false } as IColumn;

      column.styles = (props: any): any => {
         props.cellStyleProps = { ...props.cellStyleProps };
         props.cellStyleProps.cellLeftPadding = 4;
         props.cellStyleProps.cellRightPadding = 0;
      };


      return column;
   });

};

let prevGroups: JobGroup[] = [];

const sortJobGroups = (items: JobItem[]): [JobItem[], JobGroup[]] => {

   // sort into time headers (fabric groups are indexed into one array so this is a bit complicated)
   // we also have to track collapsed state between renders
   const now = moment.utc().tz(displayTimeZone());
   const nowTimeStr = now.format('dddd, MMMM Do');
   const dateMap: Map<string, JobItem[]> = new Map();

   items.forEach((item) => {

      const time = moment(item.job!.createTime).tz(displayTimeZone());
      let timeStr = time.format('dddd, MMMM Do');
      if (timeStr === nowTimeStr) {
         timeStr += " (Today)";
      } else if (time.calendar().toLowerCase().indexOf("yesterday") !== -1) {
         timeStr += " (Yesterday)";
      } else {
         timeStr = time.format('dddd, MMMM Do');
      }
      if (!dateMap.has(timeStr)) {
         dateMap.set(timeStr, [item]);
      } else {
         dateMap.get(timeStr)?.push(item);
      }
   });

   // sort the header times
   let keys = Array.from(dateMap.keys());

   keys = keys.sort((a, b) => {
      const timeA = moment.utc(dateMap.get(a)![0].job.createTime);
      const timeB = moment.utc(dateMap.get(b)![0].job.createTime);
      return timeA < timeB ? 1 : -1;
   });

   // new create groups and sort group items
   const groups: JobGroup[] = [];
   let nitems: JobItem[] = [];

   keys.forEach(key => {

      let jobs = dateMap.get(key)!;
      jobs = jobs.sort((a, b) => {

         const changeA = a.job.change!;
         const changeB = b.job.change!;

         if (changeA === changeB) {
            const timeA = moment.utc(a.job.createTime);
            const timeB = moment.utc(b.job.createTime);

            return timeA < timeB ? 1 : -1;
         }

         return changeA < changeB ? 1 : -1;
      });

      const collapsed = prevGroups.find(group => group.key === key)?.isCollapsed;

      groups.push({
         startIndex: nitems.length,
         count: jobs.length,
         key: key,
         name: key,
         headerText: key,
         isCollapsed: collapsed ?? false
      });

      nitems = nitems.concat(jobs);
   });

   prevGroups = groups;

   return [nitems, groups];
};

const JobList: React.FC<{ tab: string; filter: JobFilterSimple; aliasTemplates?: string[] }> = observer(({ tab, filter, aliasTemplates }) => {

   const { streamId } = useParams<{ streamId: string }>();
   const { projectStore } = useBackend();
   const stream = projectStore.streamById(streamId);
   let [update, setUpdate] = useState(0);

   // depending how this is optimized in production, might need to change around so rolling over cl doesn't re-render
   const [commitState, setCommitState] = useState<{ target?: ChangeContextMenuTarget, job?: JobData, commit?: GetChangeSummaryResponse, rangeCL?: number }>({});

   useEffect(() => {

      return () => {
         jobHandler.clear();
      };

   }, []);

   const { hordeClasses, detailClasses, modeColors } = getHordeStyling();

   // @todo: this pattern is becoming annoying, have to reference once in render for observable to see
   if (jobHandler.updated) { }
   if (commitCache.updated) { }

   if (!streamId || !stream || !projectStore.streamById(streamId)) {
      console.error("bad stream id setting up JobList");
      return <div />;
   }

   const explicit: Set<string> = new Set();
   stream.tabs.forEach(t => {
      (t as JobsTabData).templates?.forEach(name => {
         explicit.add(name);
      });
   });

   const jobTab = stream.tabs.find(t => t.title === tab)! as JobsTabData;

   if (!jobTab) {
      console.error("Unable to get stream tab in JobList");
      return <div />;
   }

   const startedByName = jobTab.showNames ? "Started By" : "Change";

   const aliasTemplateSet = aliasTemplates ? new Set(aliasTemplates) : undefined;

   const templateNames: Set<string> = new Set();
   jobTab.templates?.forEach(name => {
      // Filter by alias templates if present
      if (aliasTemplateSet && !aliasTemplateSet.has(name)) {
         return;
      }
      templateNames.add(name);
   });

   let jobs = jobHandler.jobs;

   jobs = jobHandler.jobs.filter((j) => {

      const additionKeywords: string[] = [];
      const commit = commitCache.getCommit(streamId, j.change!);
      if (commit) {
         additionKeywords.push(commit.authorInfo?.name);
      }

      return filterJob(j, filter.filterKeyword, additionKeywords)
   });

   jobItems = jobs.map(j => {

      let startedBy = j.startedByUserInfo?.name ?? "Scheduler";
      if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
         startedBy = "TestAccount";
      }

      return {
         key: j.id,
         time: "",
         job: j,
         startedby: startedBy,
         stream: stream,
         jobTab: jobTab
      } as JobItem;

   });

   if (!templateNames.size) {
      jobItems = jobItems.filter(j => !explicit.has(j.job.name));
   }

   const [renderItems, groups] = sortJobGroups(jobItems.filter(item => item.job));

   if (jobHandler.jobs.length >= jobHandler.count || jobHandler.bumpCount) {
      const loadjobs = "loadjobs";
      groups.push({
         startIndex: renderItems.length,
         loadJobs: true,
         count: 1,
         key: `key_${loadjobs}_${update++}`,
         name: loadjobs,
         headerText: loadjobs
      })
   }

   // Fluent detail list group vertical estimates are problematic, so handle group row rendering ourselves
   const renderGroup = (jobGroup: JobGroup) => {


      if (jobGroup.loadJobs) {

         if (jobHandler.bumpCount) {
            return <Stack className={detailClasses.headerAndFooter} style={{ marginLeft: 12, marginRight: 24 }} tokens={{ childrenGap: 24 }} horizontal horizontalAlign="center">
               <Stack className={detailClasses.headerTitle} style={{ padding: 0 }}>{`Loading Jobs`}</Stack>
               <Stack className={detailClasses.headerTitle} style={{ padding: 0 }} verticalAlign="center"><Spinner size={SpinnerSize.medium} /></Stack>
            </Stack>

         }

         return <Stack className={detailClasses.headerAndFooter} style={{ cursor: "pointer", marginLeft: 24, marginRight: 24 }} tokens={{ childrenGap: 8 }} onClick={() => { jobHandler.addJobs(); setUpdate(update++) }} horizontal horizontalAlign="center">
            <Stack className={detailClasses.headerTitle} style={{ padding: 0 }}>{`Showing ${jobs.length} of ${jobHandler.jobs.length} Jobs.`}</Stack>
            <Stack className={detailClasses.headerTitle} style={{ padding: 0 }} >Show more...</Stack>
         </Stack>
      }

      return (
         <div className={detailClasses.headerAndFooter} style={{ marginLeft: 24, marginRight: 24 }} onClick={() => { }}>
            <div className={detailClasses.headerTitle}>{`${jobGroup.headerText}`}</div>
         </div>
      );

   };

   const width = 1440;

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {

      const classNames = mergeStyleSets({
         sticky: {
            width: width,
            marginLeft: 4,

         }
      });

      if (props) {
         props.selectionMode = SelectionMode.none;
         props.collapseAllVisibility = CollapseAllVisibility.hidden;
         return <Sticky stickyClassName={classNames.sticky} stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
            <DetailsHeader className={detailClasses.detailsHeader}  {...props} styles={{ root: { paddingLeft: 12, backgroundColor: dashboard.darktheme ? `${modeColors.content} !important` : undefined } }} />
         </Sticky>

      }
      return null;
   };

   const renderDetailStatus = (job: JobData): JSX.Element => {

      if (job.state === JobState.Complete) {
         return <div />;
      }

      return <Stack className="horde-no-darktheme" ><FontIcon iconName="FullCircle" style={{ color: getJobStateColor(job.state) }} /></Stack>;
   };

   const filterLabels = (job: JobData, category: string | undefined) => {

      const labels = job.labels;

      if (!labels || !labels.length) {
         return [];
      }

      const view = labels.filter(a => {
         const idx = labels.indexOf(a)!;
         if (!job.labels) {
            return false;
         }
         return job.labels[idx].state !== LabelState.Unspecified;
      });

      const unassigned: GetLabelStateResponse[] = [];

      view.forEach(label => {
         if (!label.dashboardCategory || !jobTab?.columns?.find(c => { return ((c as GetJobsTabLabelColumnResponse).category === label.dashboardCategory) })) {
            unassigned.push(label);
         }
      });

      return view.filter(label => (label.dashboardName && ((label.dashboardCategory === category) || ((category === "Other" || !category) && unassigned.indexOf(label) !== -1)))).sort((a, b) => a.dashboardName! < b.dashboardName! ? -1 : 1);
   };


   const JobLabel: React.FC<{ item: JobItem; column: GetJobsTabLabelColumnResponse; label: GetLabelStateResponse }> = ({ item, column, label }) => {

      const defaultLabel = item.job.defaultLabel;

      if (label === defaultLabel && column.category !== "Other" && column.heading !== "Other") {
         return <div />;
      }

      const aggregates = item.job.labels;

      // note details may not be loaded here, as only initialized on callout for optimization (details.getLabelIndex(label.Name, label.Category);)
      const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
      let labelIdx = -1;
      if (jlabel) {
         labelIdx = aggregates?.indexOf(jlabel)!;
      }

      let state: LabelState | undefined;
      let outcome: LabelOutcome | undefined;
      if (label === defaultLabel) {
         state = defaultLabel.state;
         outcome = defaultLabel.outcome;
      } else {
         state = item.job.labels![labelIdx]?.state;
         outcome = item.job.labels![labelIdx]?.outcome;
      }

      const color = getLabelColor(state, outcome);

      let url = `/job/${item.job.id}`;

      if (labelIdx >= 0) {
         url = `/job/${item.job.id}?label=${labelIdx}`;
      }

      const target = `label_${item.job.id}_${label.dashboardName}_${label.dashboardCategory}`.replace(/[^A-Za-z0-9]/g, "");

      const badge = <Link to={url}><Stack className={hordeClasses.badgeNoIcon}><DefaultButton key={label.dashboardName} style={{ backgroundColor: color.primaryColor }} text={label.dashboardName}
                  onMouseOver={(ev) => ev.stopPropagation() }
                  onMouseMove={(ev) => ev.stopPropagation()}>

                  {!!color.secondaryColor && <div style={{
                     borderLeft: "10px solid transparent",
                     borderRight: `10px solid ${color.secondaryColor}`,
                     borderBottom: "10px solid transparent",
                     height: 0,
                     width: 0,
                     position: "absolute",
                     right: 0,
                     top: 0,
                     zIndex: 1
                  }} />}

               </DefaultButton></Stack></Link>;

      return <Stack>
         <div id={target} className="horde-no-darktheme">
            {labelIdx >= 0 ? <LabelHoverCard
               jobId={item.job.id}
               labelIndex={labelIdx}
               streamId={streamId!}
               templateId={item.job.templateId!}
               templateName={item.job.name}
               dashboardName={label.dashboardName}
               dashboardCategory={label.dashboardCategory}
               state={state}
               outcome={outcome}
               change={item.job.change}
            >
               {badge}
            </LabelHoverCard> : badge}
         </div>
      </Stack>;

   };

   const renderLabels = (item: JobItem, labels: GetLabelStateResponse[], jobColumn: GetJobsTabLabelColumnResponse, column?: IColumn): JSX.Element => {

      const defaultLabel = item.job.defaultLabel;

      if (!labels.length && !defaultLabel) {
         return <div />;
      }

      if (defaultLabel && (!jobColumn.category || jobColumn.category === "Other")) {
         defaultLabel.dashboardCategory = "";
         defaultLabel.dashboardName = "Other";
         labels.push(defaultLabel);
      }

      let key = 0;

      const buttons = labels.map(label => {
         return <JobLabel key={`label_${item.job.id}_${jobColumn.category}_${label.dashboardName}_${key++}`} item={item} column={jobColumn} label={label} />;
      });

      return (
         <Stack horizontalAlign="center" >
            <Stack horizontal horizontalAlign="center" tokens={{ childrenGap: 4 }} styles={{ root: { paddingTop: 2, width: column?.maxWidth } }}>
               {buttons}
            </Stack>
         </Stack>
      );

   };

   function onRenderItemColumnInner(item: JobItem, index?: number, column?: IColumn) {

      const fieldContent = item[column!.fieldName as keyof JobItem] as string;

      const jobColumn = item.jobTab?.columns?.find(c => c.heading === column!.key);

      const style: any = {
         margin: 0,
      };


      if (jobColumn?.type === JobsTabColumnType.Labels) {
         const labelColumn = jobColumn as GetJobsTabLabelColumnResponse;
         const labels = filterLabels(item.job!, labelColumn?.category);
         return <Stack verticalAlign="center" verticalFill={true}>{renderLabels(item, labels, labelColumn, column)}</Stack>;
      }

      if (jobColumn?.type === JobsTabColumnType.Parameter) {
         const paramColumn = jobColumn as GetJobsTabParameterColumnResponse;
         let arg = item.job.arguments?.find(a => a.toLowerCase().startsWith(paramColumn.parameter?.toLowerCase() ?? "____"))
         arg = arg?.split("=")[1];
         return <Stack verticalAlign="center" verticalFill={true}><Text>{arg}</Text></Stack>;
      }

      if (column!.key === "Time") {

         const displayTime = moment(item.job!.createTime).tz(displayTimeZone());
         const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

         let displayTimeStr = displayTime.format(format);


         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"}>< Text variant="small">{displayTimeStr}</Text></Stack>;

      }

      const commit = commitCache.getCommit(streamId!, item.job!.change!);

      if (column!.key === startedByName) {

         let authorName = item.job.startedByUserInfo?.name;

         if (!authorName) {
            authorName = commit?.authorInfo?.name;
         }

         if (!authorName) {
            authorName = "Scheduler";
         }

         let change = item.job?.change?.toString() ?? "Latest";
         const pfDisplay = getPreflightDisplay(item.job?.preflightCommitId, item.job?.preflightChange);
         if (pfDisplay) {
            change = `PF ${pfDisplay}`;
         }


         return <Stack style={{ margin: 0, cursor: "pointer" }} onClick={(ev) => {
            ev.stopPropagation();
            ev.preventDefault();

            let rangeCL: number | undefined;
            if (item.job) {

               const cjob = jobs.find((j => j.id === item.job!.id));
               if (cjob) {
                  const index = jobs.indexOf(cjob);
                  if (index < jobs.length - 1) {
                     rangeCL = jobs[index + 1].change;
                     if (rangeCL) {
                        rangeCL += 1;
                     }
                  }
               }
            }

            setCommitState({ job: item.job, rangeCL: rangeCL, commit: commit, target: { point: { x: ev.clientX, y: ev.clientY } } })
         }} >
            <Stack verticalAlign="center" verticalFill={true} horizontal horizontalAlign={"start"} tokens={{ childrenGap: 18 }}>
               <Stack verticalAlign="center" verticalFill={true} horizontalAlign="start"> <div style={{ paddingBottom: "1px" }}>
                  <span style={{ padding: "2px 6px 2px 6px", height: "18px", cursor: "pointer", color: "#FFFFFF", backgroundColor: item.job.startedByUserInfo ? "#0288ee" : "#035ca1" }} className={item.job.startedByUserInfo ? "cl-callout-button-user" : "cl-callout-button"} >{change}</span>
               </div></Stack>
               {!!jobTab.showNames &&  <Text variant="small">{authorName}</Text>}
            </Stack>
         </Stack>;
      }

      switch (column!.key) {

         case 'Build':
            return <Stack verticalAlign="center" verticalFill={true}><Text variant="small" style={{ whiteSpace: "normal" }} >{item.job.name}</Text></Stack>;

         case 'Status':
            return <Stack style={style} verticalAlign="center" verticalFill={true} horizontalAlign={"center"}>{renderDetailStatus(item.job)}</Stack>;

         default:
            return <Stack style={style} verticalAlign="center" verticalFill={true} horizontalAlign={"center"}><Text>{fieldContent}</Text></Stack>;
      }
   }

   function onRenderItemColumn(item: JobItem, index?: number, column?: IColumn) {

      if (column?.key === startedByName) {
         return onRenderItemColumnInner(item, index, column);
      }

      return <Link to={`/job/${item.job.id}`}>{onRenderItemColumnInner(item, index, column)}</Link>;

   }

   const onRenderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         let group = groups.find(g => g.startIndex === props.itemIndex);

         const item = renderItems[props.itemIndex];

         const row = <JobOperationsContextMenu job={item.job}>
            <Stack style={{ marginLeft: 24, marginRight: 24 }}>
               <DetailsRow styles={{ root: { minHeight: lineHeight, width: "100%" }, cell: { lineHeight: lineHeight, height: lineHeight, minHeight: lineHeight } }} {...props} />
            </Stack>
         </JobOperationsContextMenu>;

         if ((props.itemIndex === renderItems.length - 1) && groups.find(g => !!g.loadJobs)) {
            return <React.Fragment>
               {row}
               {renderGroup(groups.find(g => !!g.loadJobs)!)}
            </React.Fragment>

         }

         if (group) {
            return <React.Fragment>
               {renderGroup(group)}
               {row}
            </React.Fragment>
         }


         return row;

      }
      return null;
   };

   const forcePreflights = tab?.toLowerCase() === "swarm" || tab?.toLowerCase() === "presubmit";
   let preflightStartedByUserId: string | undefined;

   if (!forcePreflights && !filter.showOthersPreflights) {
      preflightStartedByUserId = dashboard.userId;
   }

   jobHandler.filter(stream, templateNames.size ? Array.from(templateNames.values()) : undefined, preflightStartedByUserId);


   let nojobs = !jobHandler.initial && !jobs.length;

   return (
      <Stack tokens={{ childrenGap: 0 }} className={detailClasses.detailsRow}>
         <FocusZone direction={FocusZoneDirection.vertical} >
            {commitState.target && commitState.job && <ChangeContextMenu target={commitState.target} job={commitState.job} commit={commitState.commit} rangeCL={commitState.rangeCL} onDismiss={() => setCommitState({})} />}
            <div className={detailClasses.container} style={{ width: "100%", height: 'calc(100vh - 240px)', position: 'relative', marginTop: 0 }} data-is-scrollable={true}>

               {<ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} style={{ overflow: "visible" }}>
                  {renderItems.length > 0 &&
                     <Stack style={{ width: width, marginLeft: 4, background: modeColors.content, boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)",  }}>
                        <DetailsList
                           styles={{
                              headerWrapper: { overflow: "hidden" }, root: {
                                 paddingBottom: 32,
                                 selectors: {
                                    '.ms-List-cell': {
                                       minHeight: lineHeight
                                    }
                                 }
                              }
                           }}
                           indentWidth={0}
                           compact={true}
                           selectionMode={SelectionMode.none}
                           items={renderItems}
                           columns={buildColumns(jobTab, streamId)}
                           layoutMode={DetailsListLayoutMode.fixedColumns}
                           constrainMode={ConstrainMode.unconstrained}
                           onRenderRow={onRenderRow}
                           onRenderDetailsHeader={onRenderDetailsHeader}
                           onRenderItemColumn={onRenderItemColumn}
                           onShouldVirtualize={() => { return true; }}
                        />
                     </Stack>
                  }

                  {nojobs && <Stack style={{ width: 1364 }}>
                     <Stack horizontalAlign="center" styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                        <Text variant="mediumPlus">No jobs found</Text>
                     </Stack>
                  </Stack>
                  }

                  {!nojobs && !renderItems.length && <Stack styles={{ root: { paddingTop: 20, paddingBottom: 20 } }}>
                     <Stack style={{ width: width }}><Spinner size={SpinnerSize.large} /></Stack>
                  </Stack>
                  }
               </ScrollablePane>
               }
            </div>
         </FocusZone>
      </Stack>
   );

});

export const JobViewIncremental: React.FC<{ tab: string; filter: JobFilterSimple; aliasTemplates?: string[] }> = ({ tab, filter, aliasTemplates }) => {

   return (<JobList tab={tab} filter={filter} aliasTemplates={aliasTemplates} />);

};
