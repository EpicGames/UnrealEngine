// Copyright Epic Games, Inc. All Rights Reserved.
import { Checkbox, ComboBox, ContextualMenuItemType, DefaultButton, DirectionalHint, Dropdown, DropdownMenuItemType, IComboBoxOption, Icon, IconButton, IContextualMenuItem, IContextualMenuProps, IDropdownOption, Label, MessageBar, MessageBarType, Modal, Pivot, PivotItem, PrimaryButton, ScrollablePane, ScrollbarVisibility, Spinner, SpinnerSize, Stack, TagPicker, Text, TextField, TooltipHost, ValidationState } from '@fluentui/react';
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import moment from 'moment';
import React, { useState } from 'react';
import backend, { useBackend } from '../backend';
import { BoolParameterData, CreateJobRequest, GetJobsTabResponse, GroupParameterData, JobsTabData, ListParameterData, ListParameterItemData, ListParameterStyle, ParameterData, ParameterType, Priority, TabType, GetTemplateRefResponse, TextParameterData, ChangeQueryConfig } from '../backend/Api';
import templateCache from '../backend/TemplateCache';
import { errorDialogStore } from "./error/ErrorStore";
import { useQuery } from "horde/base/utilities/hooks";
import { JobDetailsV2 } from './jobDetailsV2/JobDetailsViewCommon';
import dashboard from '../backend/Dashboard';
import { getHordeStyling } from '../styles/Styles';
import { Markdown } from '../base/components/Markdown';
import { NewBuildV2 } from './build/NewBuildV2';

let toolTipId = 0;

type ValidationError = {
   paramString: string;
   error: string;
   value: string;
   context?: string;
   param?: TextParameterData;
};

class BuildParameters {

   static paramKey = 0;

   constructor(template: GetTemplateRefResponse, jobDetails?: JobDetailsV2, existing?: BuildParameters) {

      makeObservable(this);

      BuildParameters.paramKey++;

      this.template = template;
      this.jobDetails = jobDetails;

      if (jobDetails) {
         this.preflight = jobDetails.jobData?.preflightChange?.toString();
         this.change = jobDetails.jobData?.change?.toString();
         this.targets = jobDetails.targets;
         this.advJobName = jobDetails.jobData?.name;
      } else {

         if (existing) {
            this.copySettings(existing);
         }

         const targets = new Set<string>();
         template.arguments.forEach(a => {
            if (a.toLowerCase().startsWith("-target=")) {
               targets.add(a.slice(8).trim());
            }
         })

         this.targets = Array.from(targets);

      }
      const detailSet = new Set(jobDetails?.jobData?.arguments);

      // setup default values and enumerate non-group parameters
      const parameters: ParameterData[] = [];
      this.template.parameters.forEach(p => {
         this.initParam(p, detailSet, jobDetails);
         this.enumerateParameters(p, parameters);
      });

      // setup advanced options 
      // the new build dialog has gone through so many iterations at this point
      // this and the enumeration above could be combined 
      if (jobDetails && !existing) {

         const job = jobDetails.jobData!;

         // capture what args are part of a multi-argument
         const multiArgs = new Set<string>();
         parameters.forEach((p) => {
            if (p.type === ParameterType.Bool) {
               const param = p as BoolParameterData;

               param.argumentsIfDisabled?.forEach(a => {
                  multiArgs.add(a.toLowerCase().trim());
               })
               param.argumentsIfEnabled?.forEach(a => {
                  multiArgs.add(a.toLowerCase().trim());
               })
            }

            if (p.type === ParameterType.List) {
               const param = p as ListParameterData;
               param.items.forEach(item => {
                  item.argumentsIfDisabled?.forEach(a => {
                     multiArgs.add(a.toLowerCase().trim());
                  })
                  item.argumentsIfEnabled?.forEach(a => {
                     multiArgs.add(a.toLowerCase().trim());
                  })
               });
            }
         });

         const args = job.arguments.filter(arg => {

            arg = arg.toLowerCase().trim();

            if (multiArgs.has(arg)) {
               return false;
            }

            if (this.template.arguments?.find(targ => {
               return targ.toLowerCase().trim() === arg;
            })) {
               return false;
            }

            let found = false;

            parameters.forEach((p) => {

               if (found) {
                  return;
               }

               if (p.type === ParameterType.Bool) {
                  const param = p as BoolParameterData;

                  if (param.argumentIfEnabled?.toLowerCase().trim() === arg) {
                     found = true;
                  }

                  if (param.argumentIfDisabled?.toLowerCase().trim() === arg) {
                     found = true;
                  }

               }
               else if (p.type === ParameterType.Text) {

                  const param = p as TextParameterData;

                  if (arg.startsWith(param.argument?.toLowerCase().trim())) {
                     found = true;
                  }
               }
               else if (p.type === ParameterType.List) {
                  const param = p as ListParameterData;

                  param.items.forEach(item => {

                     if (found) {
                        return;
                     }
                     if (item.argumentIfEnabled?.toLowerCase().trim() === arg) {
                        found = true;
                     }

                     if (item.argumentIfDisabled?.toLowerCase().trim() === arg) {
                        found = true;
                     }

                  });
               }
            })

            return !found;
         });

         const options = args.filter(arg => !arg.toLowerCase().trim().startsWith("-target="));

         if (options.length) {
            this.advAdditionalArgs = options.join(" ");
         }

         this.advJobPriority = jobDetails.jobData!.priority;
         this.advUpdateIssues = (jobDetails.jobData!.updateIssues && !template.updateIssues) ? true : undefined;

      }

      this.initialTargets = [...this.targets];

   }

   addTarget(target: string, updateChanged = true) {
      target = target.trim();
      const unique = new Set<string>(this.targets);
      unique.add(target);
      this.targets = Array.from(unique);

      const parameters: ParameterData[] = [];
      this.template.parameters.forEach(p => {
         this.enumerateParameters(p, parameters);
      });

      parameters.forEach(p => {

         if (p.parameterKey && p.type === ParameterType.Text) {
            const text = p as TextParameterData;
            if (text.argument.toLowerCase().startsWith("-target=")) {
               if (!this.values[p.parameterKey]) {
                  this.values[p.parameterKey] = target;
               }
            }
         }

         if (p.parameterKey && p.type === ParameterType.Bool) {

            const data = p as BoolParameterData;

            let enabledTargets: string[] = [];
            let disabledTargets: string[] = [];

            if (data.argumentIfEnabled?.toLowerCase().startsWith("-target=")) {
               enabledTargets.push(data.argumentIfEnabled.slice(8));
            }

            data.argumentsIfEnabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  enabledTargets.push(a.slice(8));
               }
            });

            if (data.argumentIfDisabled?.toLowerCase().startsWith("-target=")) {
               disabledTargets.push(data.argumentIfDisabled.slice(8));
            }

            data.argumentsIfDisabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  disabledTargets.push(a.slice(8));
               }
            });

            if (enabledTargets.length) {
               this.values[p.parameterKey] = !enabledTargets.find(t => !unique.has(t));
            }

            if (disabledTargets.length && disabledTargets.find(t => unique.has(t))) {
               this.values[p.parameterKey] = false;
            }
         }

      })

      if (updateChanged) {
         this.setChanged();
      }

   }

   removeTarget(target: string, updateChanged = true) {

      target = target.trim();
      const unique = new Set<string>(this.targets.filter(t => t !== target));
      this.targets = Array.from(unique);

      const parameters: ParameterData[] = [];
      this.template.parameters.forEach(p => {
         this.enumerateParameters(p, parameters);
      });

      parameters.forEach(p => {

         if (p.parameterKey && p.type === ParameterType.Text) {
            const text = p as TextParameterData;
            if (text.argument.toLowerCase().startsWith("-target=")) {
               if (this.values[p.parameterKey] === target) {
                  this.values[p.parameterKey] = this.targets.length ? this.targets[0] : "";
               }
            }
         }

         if (p.parameterKey && p.type === ParameterType.Bool) {

            const data = p as BoolParameterData;

            let enabledTargets: string[] = [];
            let disabledTargets: string[] = [];

            if (data.argumentIfEnabled?.toLowerCase().startsWith("-target=")) {
               enabledTargets.push(data.argumentIfEnabled.slice(8));
            }

            data.argumentsIfEnabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  enabledTargets.push(a.slice(8));
               }
            });

            if (data.argumentIfDisabled?.toLowerCase().startsWith("-target=")) {
               disabledTargets.push(data.argumentIfDisabled.slice(8));
            }

            data.argumentsIfDisabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  disabledTargets.push(a.slice(8));
               }
            });

            if (enabledTargets.length) {
               this.values[p.parameterKey] = !enabledTargets.find(t => !unique.has(t));
            }

            if (disabledTargets.length && disabledTargets.find(t => unique.has(t))) {
               this.values[p.parameterKey] = false;
            }

         }

      })

      if (updateChanged) {
         this.setChanged();
      }

   }


   setTargets(targets: string[], updateChanged = true) {
      const newTargets = Array.from(new Set(targets));
      const oldTargets = this.targets.filter(t => newTargets.indexOf(t) === -1);
      oldTargets.forEach(t => this.removeTarget(t, false));
      if (updateChanged) {
         this.setChanged();
      }
   }

   setInitialTargets() {
      this.setTargets([], false);
      this.initialTargets.forEach(t => this.addTarget(t, false));
      this.setChanged();
   }

   enumerateParameters(param: ParameterData, parameters: ParameterData[]) {

      if (param.type === ParameterType.Group) {

         (param as GroupParameterData).children?.forEach(c => {
            this.enumerateParameters(c, parameters);
         })

         return;
      }

      parameters.push(param);

   }

   // copies base and advanced settings
   copySettings(src: BuildParameters) {

      this.change = src.change;
      this.changeOption = src.changeOption;
      this.preflight = src.preflight;
      this.autoSubmit = src.autoSubmit;

      this.advJobName = src.advJobName;
      this.advJobPriority = src.advJobPriority;
      this.advUpdateIssues = src.advUpdateIssues;
      this.advAdditionalArgs = src.advAdditionalArgs;
      this.advForceAgent = src.advForceAgent;
   }

   initParam(p: ParameterData, detailSet: Set<string>, jobDetails?: JobDetailsV2) {

      if (p.type === ParameterType.Bool) {
         const param = p as BoolParameterData;

         let value = this.jobDetails ? false : param.default;

         if (param.argumentsIfEnabled) {

            if (!param.argumentsIfEnabled.find(a => !detailSet.has(a))) {
               value = true;
            }

         } else if (param.argumentIfEnabled) {

            if (detailSet.has(param.argumentIfEnabled!)) {
               value = true;
            }
         }

         if (param.argumentsIfDisabled) {

            if (param.argumentsIfDisabled.find(a => detailSet.has(a))) {
               value = false;
            }

         } else if (param.argumentIfDisabled) {
            if (detailSet.has(param.argumentIfDisabled!)) {
               value = false;
            }
         }

         if (value) {
            this.values[this.paramKey(p)] = true;
         }
      } else if (p.type === ParameterType.List) {
         const param = p as ListParameterData;
         param.items.forEach(item => {

            let value = this.jobDetails ? false : item.default;

            if (item.argumentsIfEnabled) {

               if (!item.argumentsIfEnabled.find(a => !detailSet.has(a))) {
                  value = true;
               }

            } else if (item.argumentIfEnabled) {
               if (detailSet.has(item.argumentIfEnabled)) {
                  value = true;
               }
            }

            if (item.argumentsIfDisabled) {

               if (item.argumentsIfDisabled.find(a => detailSet.has(a))) {
                  value = false;
               }

            } else if (item.argumentIfDisabled) {
               if (detailSet.has(item.argumentIfDisabled)) {
                  value = false;
               }
            }

            if (value) {
               this.values[this.paramKey(p, item)] = true;
            }
         });
      } else if (p.type === ParameterType.Text) {
         const param = p as TextParameterData;
         if (jobDetails) {
            this.values[this.paramKey(p)] = "";
            // filter out template args
            const args = jobDetails.jobData!.arguments.filter(a => this.template!.arguments.indexOf(a) === -1);
            const a = args.find(a => a.startsWith(param.argument));
            if (a) {

               let value = a;
               let index = a.indexOf("=");
               if (index !== -1) {
                  value = a.slice(index + 1)?.trim();
               }

               this.values[this.paramKey(p)] = value;
            }

         } else {
            const value = param.default ?? "";
            this.values[this.paramKey(p)] = value;
            if (value && param.argument.toLowerCase().startsWith("-target=")) {
               this.addTarget(value);
            }
         }

      } else if (p.type === ParameterType.Group) {
         const param = p as GroupParameterData;
         this.pushGroup(param);
         param.children?.forEach(p => {
            this.initParam(p, detailSet, jobDetails);
         });
         this.popGroup();
      }

   }

   generateArgument(p: ParameterData, args: string[]): void {
      if (p.type === ParameterType.Bool) {
         const param = p as BoolParameterData;
         if (this.values[this.paramKey(p)]) {
            if (param.argumentIfEnabled) {
               args.push(param.argumentIfEnabled);
            }
            if (param.argumentsIfEnabled) {
               args.push(...param.argumentsIfEnabled);
            }

         } else {
            if (param.argumentIfDisabled) {
               args.push(param.argumentIfDisabled);
            }
            if (param.argumentsIfDisabled) {
               args.push(...param.argumentsIfDisabled);
            }
         }
      } else if (p.type === ParameterType.List) {
         const param = p as ListParameterData;
         param.items.forEach(item => {

            if (this.values[this.paramKey(p, item)]) {
               if (item.argumentIfEnabled) {
                  args.push(item.argumentIfEnabled);
               }
               if (item.argumentsIfEnabled) {
                  args.push(...item.argumentsIfEnabled);
               }
            } else {
               if (item.argumentIfDisabled) {
                  args.push(item.argumentIfDisabled);
               }
               if (item.argumentsIfDisabled) {
                  args.push(...item.argumentsIfDisabled);
               }
            }
         });
      } else if (p.type === ParameterType.Text) {

         const param = p as TextParameterData;

         let value = (this.values[this.paramKey(p)] as string)?.trim();

         // default already populated in text entry
         // value = value ? value : param.default;

         if (!value) {
            return;
         }

         // special case Target parameter with a ; in the value
         if (param.argument.toLowerCase().startsWith("-target=") && value.indexOf(";") !== -1) {

            const targets = value.split(";").map(t => t.trim());
            targets.forEach(target => {
               target = target.replace(/["']/g, "");
               if (target) {
                  args.push(`${param.argument}${target}`);
               }
            })

            return;
         }

         args.push(`${param.argument}${value}`);


      } else if (p.type === ParameterType.Group) {
         const param = p as GroupParameterData;
         this.pushGroup(param);
         param.children?.forEach(p => this.generateArgument(p, args));
         this.popGroup();
      }
   }

   generateArguments(): string[] {

      let args: string[] = [];

      // gather parameters
      this.template.parameters.forEach(p => {

         this.generateArgument(p, args);
      });

      const filter = new Set<string>();

      args = args.filter(a => {
         if (filter.has(a)) {
            return false;
         }
         filter.add(a);
         return true;
      });

      return args;

   }

   pushGroup(group: GroupParameterData) {

      if (this.currentGroup) {
         console.error("Misaligned group push");
      }

      this.currentGroup = group;
   }

   popGroup() {
      this.currentGroup = undefined;
   }

   paramKey(param: ParameterData, listItem?: ListParameterItemData): string {

      const paramGroupKey = this.currentGroup ? `groupdata_${this.currentGroup.label}` : "";

      if (param.type === ParameterType.Bool) {
         const p = param as BoolParameterData;
         p.parameterKey = `${p.type}_${p.label}_${BuildParameters.paramKey}_${paramGroupKey}`;
         return p.parameterKey;
      }

      if (param.type === ParameterType.List) {
         const p = param as ListParameterData;
         if (listItem) {
            return `${p.type}_${p.label}_${listItem.text}_${listItem.group ? listItem.group : "__nogroup"}_${paramGroupKey}`;
         }
         return `${p.type}_${p.label}_${BuildParameters.paramKey}_${paramGroupKey}`;
      }

      if (param.type === ParameterType.Text) {
         const p = param as TextParameterData;
         p.parameterKey = `${p.type}_${p.label}_${BuildParameters.paramKey}_${paramGroupKey}`;
         return p.parameterKey;
      }

      if (param.type === ParameterType.Group) {
         const p = param as GroupParameterData;
         return `${p.type}_${p.label}_${BuildParameters.paramKey}_${paramGroupKey}`;
      }

      return "";
   }

   get advancedModified(): boolean {

      // @todo: detect advanced targets modified, https://jira.it.epicgames.com/browse/UE-136665      
      return !!((this.advJobPriority && this.advJobPriority !== Priority.Normal) || this.advAdditionalArgs || this.advForceAgent || this.advUpdateIssues)
   }

   validateParameter(p: ParameterData, errors: ValidationError[]): void {

      if (p.type === ParameterType.Text) {

         const param = p as TextParameterData;

         if (!param.validation) {
            return;
         }

         let value = (this.values[this.paramKey(p)] as string) ?? param.default;

         let regex: RegExp | undefined;

         try {
            regex = new RegExp(param.validation, 'gm');
         } catch (reason: any) {

            errors.push({
               param: param,
               paramString: param.label,
               value: value,
               error: `Invalid validation regex: ${param.validation}`,
               context: reason.toString()
            });

            return;
         }

         const match = value.match(regex);

         if (!match || !match.length || match[0]?.trim() !== value.trim()) {
            errors.push({
               param: param,
               paramString: param.label,
               value: value,
               error: `Did not match regex: ${param.validation}`,
               context: param.validationError ?? param.hint
            });
            return;
         }

      } else if (p.type === ParameterType.Group) {
         const param = p as GroupParameterData;
         this.pushGroup(param);
         param.children?.forEach(p => this.validateParameter(p, errors));
         this.popGroup();
      }

   }

   validate(errors: ValidationError[]): boolean {

      if (this.preflight && !this.template.allowPreflights) {
         errors.push({ error: `Template "${this.template.name}" does not allow preflights`, paramString: "Shelved Change", value: this.preflight })
      }

      this.template.parameters.forEach(p => {

         this.validateParameter(p, errors);
      });

      return !!errors.length;

   }


   @observable
   changed = 0;

   @action
   setChanged() {
      this.changed++;
   }

   template: GetTemplateRefResponse;

   jobDetails?: JobDetailsV2;

   values: { [key: string]: string | boolean } = {};


   // base settings

   change?: string;
   changeOption?: string;
   preflight?: string;
   autoSubmit?: boolean;

   // advanced options
   advJobName?: string;
   advJobPriority?: Priority;
   advUpdateIssues?: boolean;
   advAdditionalArgs?: string;
   advForceAgent?: string;

   targets: string[] = [];
   // for reset
   initialTargets: string[] = [];

   private currentGroup?: GroupParameterData;

}

export const NewBuild: React.FC<{ streamId: string; show: boolean; onClose: (newJobId: string | undefined) => void, jobKey?: string; jobDetails?: JobDetailsV2, readOnly?: boolean }> = observer(({ streamId, jobKey, show, onClose, jobDetails, readOnly }) => {

   const { projectStore } = useBackend();
   const query = useQuery();
   const [build, setBuild] = useState<{ template?: GetTemplateRefResponse; buildParams?: BuildParameters }>({});
   const [submitting, setSubmitting] = useState(false);
   const [templateData, setTemplateData] = useState<{ streamId: string; templates: GetTemplateRefResponse[] } | undefined>(undefined);
   const [mode, setMode] = useState<"Basic" | "Advanced">("Basic");
   const [showAllTemplates, setShowAllTemplates] = useState<boolean>(false);
   const [validationErrors, setValidationErrors] = useState<ValidationError[]>([]);
   const [useNewBuildV2, setNewBuildV2] = useState<boolean>(false);

   const targetPicker = React.useRef(null)
   const { hordeClasses, modeColors } = getHordeStyling();

   const stream = projectStore.streamById(streamId);

   if (useNewBuildV2) {
      return <NewBuildV2 show={true} streamId={streamId} onClose={onClose} jobKey={jobKey} jobDetails={jobDetails} readOnly={readOnly} />
   }

   if (!show) {
      return null;
   }

   if (!stream) {
      console.error("unable to get stream");
      return <div>unable to get stream</div>;
   }

   if (!showAllTemplates && (jobKey === "all" || jobKey === "summary")) {
      setShowAllTemplates(true);
      return null;
   }

   let templates: GetTemplateRefResponse[] = [];

   // @todo: better way of checking whether we are in a preflight submit
   const isPreflightSubmit = !!query.get("shelvedchange");
   const isFromP4V = !!query.get("p4v");
   const queryTemplateId = !query.get("templateId") ? "" : query.get("templateId")!;

   let defaultShelvedChange: string | undefined = query.get("shelvedchange") ? query.get("shelvedchange")! : undefined;

   if (!defaultShelvedChange) {
      defaultShelvedChange = jobDetails?.jobData?.preflightChange?.toString();
   }

   const allowtemplatechange = query.get("allowtemplatechange") === "true";

   if (!templateData || streamId !== templateData.streamId) {
      templateCache.getStreamTemplates(stream).then(data => setTemplateData({ streamId: streamId!, templates: data }));
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Loading Templates</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>

         </Stack>
      </Modal>
   } else if (templateData.streamId !== streamId) {
      setTemplateData(undefined);
      return null;
   }


   if (!templateData.templates.length) {
      return null;
   }


   if (!jobDetails) {

      if (!showAllTemplates) {

         const tab = stream.tabs.find(tab => tab.title === jobKey) as JobsTabData | undefined;

         if (!tab) {
            const msg = `unable to find tab for jobKey: ${jobKey}`;
            console.error(msg);
            return <div>{msg}</div>;
         }

         if (tab.type !== TabType.Jobs) {
            const msg = `Tab is not of Jobs type: ${tab.title}`;
            console.error(msg);
            return <div>{msg}</div>;
         }

         templates = [];

         tab.templates?.forEach(templateName => {
            const t = templateData.templates.find(t => t.id === templateName);
            if (!t) {
               console.error(`Could not find template ${templateName}`);
               return;
            }
            templates.push(t);
         });

      } else {
         templates = templateData.templates.filter(template => {
            return true;
         });
      }

   } else {

      if (allowtemplatechange) {

         templates = templateData.templates.filter(template => {
            return true;
         });


         if (jobDetails?.template) {
            if (!templates.find(t => t.name === jobDetails.template!.name)) {
               templates.push(jobDetails.template);
            }
         }
      } else {
         templates = [jobDetails.template!];
      }

   }

   const template = build.template;
   const buildParams = build.buildParams;

   let defaultPreflightQuery: ChangeQueryConfig[] | undefined;

   if (template?.defaultChange) {
      defaultPreflightQuery = template.defaultChange;
   } else {
      const defaultChange = stream?.defaultPreflight?.change;

      if (defaultChange) {
         if (!defaultChange.name && defaultChange.templateId) {
            const defaultTemplate = templateData.templates.find(t => t.id === defaultChange.templateId);
            if (defaultTemplate) {
               defaultChange.name = "Latest Success - " + defaultTemplate.name;
            }
         }

         if (defaultChange.name) {
            defaultPreflightQuery = [defaultChange];
         }
      }
   }

   const defaultStreamPreflightTemplate = stream.templates.find(t => t.id === stream.defaultPreflight?.templateId);

   if (!defaultPreflightQuery && stream.defaultPreflight?.templateId) {

      if (defaultStreamPreflightTemplate) {
         defaultPreflightQuery = defaultStreamPreflightTemplate.defaultChange;
      }

      if (!defaultPreflightQuery) {
         console.error(`Unable to find default stream preflight template ${stream.defaultPreflight?.templateId} in stream templates`);
      }
   }

   if (buildParams?.preflight) {
      templates = templates.filter(t => t.allowPreflights);
   }

   if (!template || templates.indexOf(template) === -1) {

      let t: GetTemplateRefResponse | undefined;

      if (jobDetails?.template) {
         t = templates.find(t => t.name === jobDetails.template!.name);
      }

      // handle preflight redirect case
      if (!t && defaultShelvedChange && !jobDetails) {

         if (queryTemplateId) {
            let errorReason = "";
            t = templateData.templates?.find(t => t.id === queryTemplateId);
            if (!t) {
               errorReason = `Unable to find queryTemplateId ${queryTemplateId} in stream ${streamId}`;
               console.error(errorReason);

            } else if (!t.allowPreflights) {
               errorReason = `Template does not allow preflights: queryTemplateId ${queryTemplateId} in stream ${streamId}`;
               console.error(errorReason);
               t = undefined;
            }

            if (errorReason) {
               errorDialogStore.set({
                  reason: `${errorReason}`,
                  title: `Preflight Template Error`,
                  message: `There was an issue with the specified preflight template.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`
               }, true);
            }
         }

         if (!t) {
            t = defaultStreamPreflightTemplate;
            if (!t) {
               console.error(`Stream default preflight template cannot be found for stream ${stream.fullname} : stream defaultPreflightTemplate ${stream.defaultPreflight?.templateId}, will use first template in list`);
            }
         }
      }

      if (!t) {
         const pref = dashboard.getLastJobTemplateSettings(stream.id, templates.map(t => t.id));
         if (pref) {
            t = templates.find(t => stream.id === pref.streamId && t.id === pref.templateId)
         }
      }
      // default to sane template when all are shown
      if (!t && (showAllTemplates) && stream.tabs.length > 0) {
         const stab = stream.tabs[0] as JobsTabData;
         if (stab.templates && stab.templates.length > 0) {
            t = templates.find(t => t.id === stab.templates![0]);
         }
      }

      if (!t) {
         t = templates.length > 0 ? templates[0] : undefined;
      }

      if (!t) {
         const msg = `Unable to get default template for jobKey: ${jobKey}, check that tab has a valid template list`;
         console.error(msg);
      } else {
         console.log(`BuildParams 1: ${jobDetails?.template?.id}, ${t.id}`);
         const p = new BuildParameters(t, (jobDetails?.template?.id !== t.id) ? undefined : jobDetails);

         if (query.get("autosubmit") === "true") {
            p.autoSubmit = true;
         }

         if (!!defaultShelvedChange) {
            p.preflight = defaultShelvedChange;
         }

         // when coming from P4V, always clear the base CL
         if (isFromP4V) {
            p.change = undefined;
            p.changeOption = undefined;
         }

         setBuild({ template: t, buildParams: p });
         setValidationErrors([]);
      }

      return null;
   }

   if (!buildParams) {
      return null;
   }

   // for observer
   if (buildParams!.changed) { }

   const onTemplateChange = (templateId: string) => {
      const update = templates.find((t) => templateId === t.id);
      if (update && template !== update) {
         const newParams = new BuildParameters(update, undefined, buildParams);
         newParams.advJobName = undefined;
         setBuild({ template: update, buildParams: newParams });
      }
   };

   let templateOptions: IContextualMenuItem[] = [];

   if (!showAllTemplates) {
      templateOptions = templates.map(t => {
         return { key: t.id, text: t.name, onClick: () => onTemplateChange(t.id) };
      }).sort((a, b) => a.text < b.text ? -1 : 1);
   } else {

      const sorted = new Map<string, GetTemplateRefResponse[]>();

      templates.forEach(t => {

         stream.tabs.forEach(tab => {
            if (tab.type !== TabType.Jobs) {
               return;
            }

            const jtab = tab as GetJobsTabResponse;
            if (!jtab.templates?.find(template => template === t.id)) {
               return;
            }

            if (!sorted.has(jtab.title)) {
               sorted.set(jtab.title, []);
            }

            sorted.get(jtab.title)!.push(t);

         })
      })

      Array.from(sorted.keys()).sort((a, b) => a < b ? -1 : 1).forEach(cat => {

         const templates = sorted.get(cat);
         if (!templates?.length) {
            return;
         }

         const subItems = templates.sort((a, b) => a.name < b.name ? -1 : 1).map(t => {
            return { key: t.id, text: t.name, onClick: () => onTemplateChange(t.id) };
         })

         templateOptions.push({ key: `${cat}_category`, text: cat, subMenuProps: { items: subItems } });

      })
   }

   if (jobKey !== "all" && jobKey !== "summary") {

      templateOptions.push({ key: `show_all_templates_divider`, itemType: ContextualMenuItemType.Divider });

      if (!showAllTemplates) {
         templateOptions.push({ key: `show_all_templates`, text: "Show All Templates", onClick: (ev) => { ev?.stopPropagation(); ev?.preventDefault(); setShowAllTemplates(true) } });
      } else {
         templateOptions.push({ key: `show_all_templates`, text: "Filter Templates", onClick: (ev) => { ev?.stopPropagation(); ev?.preventDefault(); setShowAllTemplates(false) } });
      }

   }

   const templateMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: templateOptions,
   };

   const renderBoolParam = (param: BoolParameterData) => {
      const key = buildParams.paramKey(param);
      return <Checkbox key={key}
         label={param.label}
         defaultChecked={!!buildParams?.values[key]}
         disabled={readOnly}
         onChange={(ev, value) => {

            const enabled = !!value;
            let enabledTargets: string[] = [];
            let disabledTargets: string[] = [];

            if (param.argumentIfEnabled?.toLowerCase().startsWith("-target=")) {
               enabledTargets.push(param.argumentIfEnabled.slice(8));
            }

            param.argumentsIfEnabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  enabledTargets.push(a.slice(8));
               }
            })

            if (param.argumentIfDisabled?.toLowerCase().startsWith("-target=")) {
               disabledTargets.push(param.argumentIfDisabled.slice(8));
            }

            param.argumentsIfDisabled?.forEach(a => {
               if (a.toLowerCase().startsWith("-target=")) {
                  disabledTargets.push(a.slice(8));
               }
            })

            let curTargets = enabled ? disabledTargets : enabledTargets;
            let newTargets = enabled ? enabledTargets : disabledTargets;

            curTargets.forEach(t => {
               buildParams.removeTarget(t, false);
            })

            newTargets.forEach(t => {
               buildParams.addTarget(t, false);
            })

            buildParams!.values[key] = enabled;
            buildParams!.setChanged();
         }}
      />;
   };

   const renderTextParam = (param: TextParameterData) => {
      const key = buildParams.paramKey(param);
      return <TextField key={key}
         placeholder={jobDetails ? "" : param.hint}
         label={param.label}
         spellCheck={false}
         defaultValue={(buildParams!.values[buildParams.paramKey(param)] as string) ?? ""}
         disabled={readOnly}
         onChange={(ev, value) => {

            const target = param.argument.toLowerCase().startsWith("-target=");

            if (target) {
               const curTarget = buildParams!.values[key] as string;
               if (curTarget) {
                  buildParams.removeTarget(curTarget);
               }
            }

            if (value === undefined) {
               buildParams!.values[key] = "";
            } else {
               buildParams!.values[key] = value;
            }

            if (value && target) {
               buildParams.addTarget(value);
            } else {
               buildParams.setChanged();
            }

         }}
      />;
   };

   const renderBasicListParam = (param: ListParameterData) => {

      const key = buildParams.paramKey(param);

      const options: IDropdownOption[] = [];

      param.items.forEach((item, index) => {
         options.push({
            key: key,
            text: item.text,
            selected: !!buildParams!.values[buildParams.paramKey(param, param.items[index!])]
         });
      });

      return <Dropdown key={key}
         label={param.label}
         options={options}
         disabled={readOnly}
         placeholder={jobDetails ? "" : "Select option"}
         onChange={(ev, option, index) => {
            param.items.forEach(item => {
               buildParams!.values[buildParams.paramKey(param, item)] = false;
            });
            buildParams!.values[buildParams.paramKey(param, param.items[index!])] = true;
            buildParams!.setChanged();
         }} />;

   };

   const renderMultiListParam = (param: ListParameterData) => {

      const gset: Set<string> = new Set();

      param.items.forEach(item => {
         if (!item.group) {
            item.group = "__nogroup";
         }
         if (item.group) {
            gset.add(item.group);
         }
      });

      const groups = Array.from(gset).sort((a, b) => a.localeCompare(b));

      const options: IDropdownOption[] = [];

      const selectedKeys: string[] = [];

      groups.forEach(group => {

         if (group !== "__nogroup") {
            options.push({
               key: `group_${group}`,
               text: group,
               itemType: DropdownMenuItemType.Header
            });
         }

         const dupes = new Map<string, number>();

         param.items.forEach(item => {
            const v = dupes.get(item.text);
            dupes.set(item.text, !v ? 1 : v + 1);
         });

         param.items.forEach(item => {
            if (item.group === group) {
               const key = buildParams.paramKey(param, item);
               const selected = buildParams!.values[buildParams.paramKey(param, item)] ? true : false;
               if (selected) {
                  selectedKeys.push(key);
               }
               options.push({
                  key: key,
                  text: (item.group !== "__nogroup" && dupes.get(item.text)! > 1) ? `${item.text} - ${item.group}` : item.text,
                  selected: selected
               });
            }
         });
      });

      const key = buildParams.paramKey(param);

      return <Dropdown
         key={key}
         disabled={readOnly}
         placeholder={jobDetails ? "" : "Select options"}
         styles={{
            callout: {
               selectors: {
                  ".ms-Callout-main": {
                     padding: "4px 4px 12px 12px",
                     overflow: "hidden"
                  }
               }
            },
            dropdownItemHeader: { fontSize: 12, color: modeColors.text },
            dropdownOptionText: { fontSize: 12 },
            dropdownItem: {
               minHeight: 18, lineHeight: 18, selectors: {
                  '.ms-Checkbox-checkbox': {
                     width: 14,
                     height: 14,
                     fontSize: 11
                  }
               }
            },
            dropdownItemSelected: {
               minHeight: 18, lineHeight: 18, backgroundColor: "inherit",
               selectors: {
                  '.ms-Checkbox-checkbox': {
                     width: 14,
                     height: 14,
                     fontSize: 11
                  }
               }
            }
         }}
         label={param.label}
         defaultSelectedKeys={selectedKeys}
         onChange={(event, option, index) => {

            if (!option) {
               return;
            }

            buildParams!.values[option.key] = option.selected ? true : false;
            buildParams!.setChanged();
         }}
         multiSelect
         options={options}
      />;
   };

   const renderTagPickerParam = (param: ListParameterData) => {

      const key = buildParams.paramKey(param);

      type PickerItem = {
         itemData: ListParameterItemData;
         key: string;
         name: string;
      }

      const allItems: PickerItem[] = param.items.map(item => {
         return {
            itemData: item,
            key: item.text,
            name: item.text
         };
      });

      const selectedItems = allItems.filter(item => {
         return buildParams!.values[buildParams.paramKey(param, item.itemData)];
      });

      // tag picker
      return <Stack key={key}>
         <Label> {param.label}</Label>
         <TagPicker
            disabled={readOnly}
            onResolveSuggestions={(filter, selected) => {
               return allItems.filter(i => {
                  return !selected?.find(s => i.key === s.key) && i.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1;
               });
            }}

            onEmptyResolveSuggestions={(selected) => {
               return allItems.filter(i => {
                  return !selected?.find(s => i.key === s.key);
               });
            }}

            selectedItems={selectedItems}

            onChange={(items?) => {

               allItems.forEach(item => {
                  if (!items) {
                     return false;
                  }
                  buildParams!.values[buildParams.paramKey(param, item.itemData)] = items.find(i => i.key === item.key) ? true : false;
               });

               buildParams!.setChanged();
            }}
         />
      </Stack>;

   };

   const renderListParam = (param: ListParameterData) => {

      if (param.style === ListParameterStyle.TagPicker) {
         return renderTagPickerParam(param);
      } else if (param.style === ListParameterStyle.MultiList) {
         return renderMultiListParam(param);
      } else if (param.style === ListParameterStyle.List) {
         return renderBasicListParam(param);
      }

      return <Text>Unknown List Style</Text>;
   };

   const renderGroupParam = (param: GroupParameterData) => {

      buildParams.pushGroup(param);

      const result = <Stack style={{ paddingBottom: 4 }}>
         <Label >{param.label}</Label>
         <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 6 }}>
            {param.children.map((p) => {
               if ((p as any).toolTip) {
                  return <TooltipHost content={(p as any).toolTip} id={`unique_tooltip_${toolTipId++}`} directionalHint={DirectionalHint.leftCenter}>
                     {renderParameter(p)}
                  </TooltipHost>
               } else {
                  return renderParameter(p);
               }

            })}
         </Stack>
      </Stack>;

      buildParams.popGroup();

      return result;
   };


   const renderParameter = (param: ParameterData) => {
      switch (param.type) {
         case ParameterType.Bool:
            return renderBoolParam(param as BoolParameterData);
         case ParameterType.Text:
            return renderTextParam(param as TextParameterData);
         case ParameterType.List:
            return renderListParam(param as ListParameterData);
         case ParameterType.Group:
            return renderGroupParam(param as GroupParameterData);
         default:
            return <Text>Unknown Parameter Type</Text>;
      }
   };

   const close = () => {
      setBuild({});
      setValidationErrors([]);
      setMode("Basic");
      setShowAllTemplates(false);
      onClose(undefined);
   };

   const onSubmit = () => {

      let errorReason: any = undefined;

      // validate
      const errors: ValidationError[] = [];

      let change: number | undefined;
      let preflight: number | undefined;
      if (buildParams!.change) {
         change = parseInt(buildParams!.change);
         if (isNaN(change)) {
            errors.push({
               paramString: "Change",
               value: buildParams!.change,
               error: "Invalid changelist"
            });
         }
      }

      let changeQueries: ChangeQueryConfig[] | undefined;

      if (typeof change !== 'number') {
         const changeOption = buildParams.changeOption ?? "Latest Change";
         if (changeOption !== "Latest Change" && defaultPreflightQuery) {
            changeQueries = defaultPreflightQuery;
         }
         else {
            const changeQuery = defaultPreflightQuery?.find(p => p.name === buildParams.changeOption);
            if (changeQuery) {
               changeQueries = [{ ...changeQuery, condition: undefined }];
            }
         }
      }

      if (buildParams!.preflight) {
         preflight = parseInt(buildParams!.preflight);
         if (isNaN(preflight)) {
            errors.push({
               paramString: "Shelved Change",
               value: buildParams!.preflight,
               error: "Invalid shelved changelist"
            });
         }
      }

      buildParams!.validate(errors);

      const args = buildParams!.generateArguments();

      let additionalArgs: string[] = [];

      if (buildParams!.advAdditionalArgs) {
         const argRegex = /"(\\"|[^"])*?"|[^ ]+/g;
         buildParams.advAdditionalArgs.trim().match(argRegex)?.forEach(arg => additionalArgs.push(arg.replace(/"/g, "")));
      }

      let templateArgs = [...template!.arguments].filter(arg => { return !arg.endsWith("=") && !arg.toLowerCase().startsWith("-target=") });

      let allArgs = [...templateArgs, ...args, ...additionalArgs].map(arg => arg.trim());

      const targets: Set<string> = new Set();

      allArgs.forEach(a => {
         if (a.toLowerCase().startsWith("-target=")) {
            targets.add(a.slice(8).trim());
         }
      });

      // remove all argument targets
      allArgs = allArgs.filter(a => !a.toLowerCase().startsWith("-target="));

      buildParams.targets.forEach(t => targets.add(t));

      if (targets.size === 0) {
         errors.push({ error: `No targets specified`, paramString: "Targets", value: "" });
      }

      if (errors.length) {
         setValidationErrors(errors);
         return;
      }

      Array.from(targets).sort((a, b) => { return a.localeCompare(b) }).map(a => `-Target=${a}`).reverse().forEach(target => allArgs.unshift(target));

      allArgs = allArgs.map(a => a.trim());

      const templateId = template?.id ? template.id : jobDetails?.jobData?.templateId;
      const updateIssues = template?.updateIssues ? undefined : buildParams!.advUpdateIssues;

      const data: CreateJobRequest = {
         streamId: streamId,
         templateId: templateId!,
         name: buildParams!.advJobName,
         priority: buildParams!.advJobPriority,
         updateIssues: updateIssues,
         changeQueries: changeQueries,
         arguments: allArgs
      };

      if (!data.templateId) {
         console.error("No template id on submit");
         return;
      }

      if (typeof (change) === 'number') {
         data.change = change;
      }

      if (typeof (preflight) === 'number') {

         data.preflightChange = preflight;

         data.updateIssues = false;

         if (buildParams.autoSubmit) {
            data.autoSubmit = true;
         }

      }


      const submit = async () => {

         if (!!import.meta.env.VITE_HORDE_DEBUG_NEW_JOB) {

            console.log("Debug Job Submit");
            console.log(JSON.stringify(data));

         } else {


            setSubmitting(true);

            console.log("Submitting job");
            console.log(JSON.stringify(data));

            let redirected = false;

            await backend.createJob(data).then(async (data) => {
               console.log(`Job created: ${JSON.stringify(data)}`);
               console.log("Updating notifications")
               try {
                  await backend.updateNotification({ slack: true }, "job", data.id);
                  const user = await backend.getCurrentUser();
                  if (user.jobTemplateSettings) {
                     dashboard.jobTemplateSettings = user.jobTemplateSettings;
                  }

               } catch (reason) {
                  console.log(`Error on updating notifications: ${reason}`);
               }

               redirected = true;
               onClose(data.id);
            }).catch(reason => {
               // "Not Found" is generally a permissions error
               errorReason = reason ? reason : "Unknown";
               console.log(`Error on job creation: ${errorReason}`);

            }).finally(() => {

               if (!redirected) {
                  setSubmitting(false);
                  setBuild({});
                  setValidationErrors([]);
                  onClose(undefined);

                  if (errorReason) {

                     if (errorReason?.trim().endsWith("does not exist")) {
                        errorReason += ".  Perforce edge server replication for the change may be in progress."
                     }

                     errorDialogStore.set({

                        reason: `${errorReason}`,
                        title: `Error Creating Job`,
                        message: `There was an issue creating the job.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`

                     }, true);

                  }
               }
            });
         }
      };

      submit();

   };

   const advancedPivotRenderer = (link: any, defaultRenderer: any): JSX.Element => {
      return (
         <Stack horizontal>
            {defaultRenderer(link)}
            {buildParams!.advancedModified && <Icon iconName="Issue" style={{ color: '#EDC74A', paddingLeft: 8 }} />}
         </Stack>
      );
   }

   const pivotItems = ["Basic", "Advanced"].map(tab => {
      return <PivotItem headerText={tab} itemKey={tab} key={tab} onRenderItemLink={tab === "Advanced" ? advancedPivotRenderer : undefined} />;
   });

   const priorityOptions: IDropdownOption[] = [];

   for (const p in Priority) {
      priorityOptions.push({
         text: p,
         key: p,
         isSelected: buildParams!.advJobPriority ? p === buildParams!.advJobPriority : Priority.Normal === p
      });
   }

   if (submitting) {

      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Creating Job</Text>
            </Stack>
            <Stack horizontalAlign="center">
               <Text variant="mediumPlus">The job is being created and will be available soon.</Text>
            </Stack>
            <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>

   }

   // change setup

   const changeOptions = ["Latest Change"];

   if (defaultPreflightQuery) {
      changeOptions.push(...defaultPreflightQuery.filter(n => !!n.name).map(n => n.name!));
   }

   const changeItems: IComboBoxOption[] = changeOptions.map(name => { return { key: `key_change_option_${name}`, text: name } });

   let changeText: string | undefined;

   if (buildParams.change) {
      changeText = buildParams.change;
   }
   if (buildParams.changeOption) {
      changeText = buildParams.changeOption;
   }

   const parameterGap = 6;

   let estimatedHeight = parameterGap * template.parameters.length;

   const estimateHeight = (param: ParameterData) => {

      switch (param.type) {

         case ParameterType.List:
            estimatedHeight += 54;
            break;
         case ParameterType.Bool:
            estimatedHeight += 20;
            break;
         case ParameterType.Text:
            estimatedHeight += 54;
            break;
         case ParameterType.Group:
            estimatedHeight += 20;
            const group = (param as GroupParameterData);
            if (group.children?.length) {
               estimatedHeight += group.children.length * parameterGap;
               group.children.forEach(c => {
                  estimateHeight(c);
               })
            }
            break;
      }

   }

   template.parameters.forEach(p => {
      estimateHeight(p);
   });

   if (!!buildParams.preflight) {
      estimatedHeight += 32;
   }

   // target options for picker
   type TargetPickerItem = {
      key: string;
      name: string;
   }

   const targetItems: TargetPickerItem[] = buildParams.targets.sort((a, b) => a.localeCompare(b)).map(t => {
      return {
         key: t,
         name: t
      }
   });

   return <Stack>
      <ValidationErrorModal show={!!validationErrors.length} errors={validationErrors} onClose={() => setValidationErrors([])} />
      <Modal key="new_build_key" isOpen={show} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { if (show && !submitting) close() }}>
         <Stack>
            <Stack horizontal styles={{ root: { padding: 8 } }}>
               <Stack grow verticalAlign="center">
                  <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{template?.name ? template.name : jobDetails?.jobData?.name}</Text>
               </Stack>

               <Stack grow />
               <Stack>
                  {<Pivot className={hordeClasses.pivot}
                     selectedKey={mode}
                     linkSize="normal"
                     linkFormat="links"
                     onLinkClick={(item) => {
                        setMode(item!.props!.itemKey as "Basic" | "Advanced")
                     }}>
                     {pivotItems}
                  </Pivot>
                  }
               </Stack>
            </Stack>

            <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8, paddingTop: 4 } }}>
               <Stack tokens={{ childrenGap: 8 }}>
                  <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingBottom: 4 }}>
                     <Stack horizontal verticalAlign="center" verticalFill={true} tokens={{ childrenGap: 18 }}>
                        <Stack>
                           <Label>Template</Label>
                           <DefaultButton style={{ width: 360, textAlign: "left" }} disabled={readOnly} menuProps={templateMenuProps} text={template.name} />
                        </Stack>
                     </Stack>
                     <Stack grow />
                     <Stack>
                        <ComboBox style={{ width: 240 }} label="Change" text={changeText} options={changeItems} disabled={readOnly} allowFreeform autoComplete="off" placeholder="Latest Change" defaultValue={jobDetails?.jobData?.change?.toString()} onChange={(ev, option, index, value) => {
                           ev.preventDefault();

                           if (option) {
                              buildParams.change = undefined;
                              buildParams.changeOption = option.text;
                           } else {
                              buildParams.changeOption = undefined;
                              if (!value) {
                                 buildParams.change = undefined;
                              } else {
                                 buildParams.change = value;
                              }
                           }
                           buildParams!.setChanged();
                        }}
                        />
                     </Stack>

                     <Stack>
                        <TextField
                           label="Shelved Change"
                           placeholder={!template!.allowPreflights ? "Disabled by template" : "None"}
                           title={!template!.allowPreflights ? "Preflights are disabled for this template" : undefined}
                           autoComplete="off" defaultValue={defaultShelvedChange} disabled={!template!.allowPreflights || readOnly || isPreflightSubmit} onChange={(ev, newValue) => {
                              ev.preventDefault();

                              buildParams!.preflight = newValue;

                              buildParams!.setChanged();
                           }} />

                     </Stack>
                  </Stack>
                  {!!template?.description && <Stack>
                     <Stack style={{ width: 767 }}>
                        <Stack style={{ marginTop: "-8px", paddingTop: "4px", paddingBottom: "20px" }}>
                           <Markdown styles={{ root: { maxHeight: 240, overflow: "auto", th: { fontSize: 12 } } }}>{template.description}</Markdown>
                        </Stack>
                     </Stack>
                  </Stack>}

                  <Stack>

                     {mode === "Basic" && <Stack style={{
                        height: estimatedHeight,
                        position: 'relative',
                        width: 767,
                        maxHeight: 'calc(100vh - 360px)'
                     }}><ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
                           <Stack tokens={{ childrenGap: parameterGap }}>
                              {template.parameters.map((p) => {

                                 if ((p as any).toolTip) {
                                    return <TooltipHost content={(p as any).toolTip} id={`unique_tooltip_${toolTipId++}`} directionalHint={DirectionalHint.leftCenter}>
                                       {renderParameter(p)}
                                    </TooltipHost>
                                 } else {
                                    return renderParameter(p);
                                 }

                              })}
                              {!!buildParams.preflight && <Stack style={{ paddingTop: 8 }}><Checkbox label="Automatically submit preflight on success"
                                 checked={buildParams.autoSubmit}
                                 onChange={(ev, checked) => {
                                    buildParams.autoSubmit = checked ? true : false;
                                    buildParams!.setChanged();
                                 }}
                              /></Stack>}
                           </Stack>
                        </ScrollablePane>
                     </Stack>}

                     {mode === "Advanced" && <Stack style={{ paddingBottom: 12, width: 767, }} tokens={{ childrenGap: 12 }}>
                        {!!stream.configRevision && <Stack>
                           <TextField label="Template Path" readOnly={true} value={stream.configPath ?? ""} />
                        </Stack>
                        }
                        <Stack>
                           <Dropdown disabled={readOnly} key={"key_adv_priority"} defaultValue={buildParams.advJobPriority} label="Priority" options={priorityOptions} onChange={(ev, option) => buildParams.advJobPriority = option?.key as Priority} />
                        </Stack>
                        <Stack>
                           <Checkbox disabled={readOnly || buildParams.template?.updateIssues} key="key_adv_update_issues"
                              label="Update Build Health Issues"
                              defaultChecked={buildParams.template?.updateIssues ? true : buildParams.advUpdateIssues}
                              onChange={(ev, checked) => {
                                 buildParams.advUpdateIssues = checked ? true : undefined;
                              }} />
                        </Stack>

                     </Stack>}

                     {mode === "Advanced" && <Stack grow tokens={{ childrenGap: 12 }}>

                        <TextField key={"key_adv_job_name"} spellCheck={false} defaultValue={buildParams.advJobName} label="Job Name" onChange={(ev, newValue) => buildParams.advJobName = newValue} />
                        <Stack horizontal tokens={{ childrenGap: 12 }}>
                           <Stack grow >
                              <Stack>
                                 <Label>Targets</Label>
                                 <TagPicker

                                    componentRef={targetPicker}

                                    disabled={readOnly}

                                    onBlur={(ev) => {
                                       if (ev.target.value && ev.target.value.trim()) {
                                          buildParams.addTarget(ev.target.value.trim())
                                       }

                                       // This is using undocumented behavior to clear the input when you lose focus
                                       // It could very well break
                                       if (targetPicker.current) {
                                          try {
                                             (targetPicker.current as any).input.current._updateValue("");
                                          } catch (reason) {
                                             console.error("There was an error adding target to list and clearing the input in process\n" + reason);
                                          }

                                       }

                                    }}

                                    onResolveSuggestions={(filter, selected) => {
                                       return [];
                                    }}

                                    onEmptyResolveSuggestions={(selected) => {
                                       return [];
                                    }}

                                    onRemoveSuggestion={(item) => { }}

                                    createGenericItem={(input: string, ValidationState: ValidationState) => {
                                       return {
                                          item: { name: input, key: input },
                                          selected: true
                                       };
                                    }}

                                    onChange={(items) => { if (items) buildParams.setTargets(items.map(i => i.name)) }}

                                    onValidateInput={(input?: string) => input ? ValidationState.valid : ValidationState.invalid}

                                    selectedItems={targetItems}

                                    onItemSelected={(item) => {
                                       if (!item || !item.name?.trim()) {
                                          return null;
                                       }
                                       buildParams.addTarget(item.name.trim());
                                       return null;
                                    }}
                                 />
                              </Stack>
                           </Stack>
                           <Stack style={{ paddingTop: 22 }}>
                              <DefaultButton text="Reset" disabled={readOnly} onClick={() => { buildParams.setInitialTargets() }} />
                           </Stack>
                        </Stack>
                        <Stack>
                           <TextField key={"key_adv_add_args"} multiline resizable={false} disabled={readOnly} spellCheck={false} defaultValue={buildParams.advAdditionalArgs} label="Additional Arguments" onChange={(ev, newValue) => buildParams.advAdditionalArgs = newValue} />
                        </Stack>

                     </Stack>}


                     {mode === "Advanced" && jobDetails && readOnly && <Stack grow tokens={{ childrenGap: 8 }} style={{ height: 180, paddingTop: 12 }}>
                        <TextField style={{ height: 160 }} key={"key_adv_job_arguments"} defaultValue={jobDetails.jobData?.arguments.map(arg => {
                           if (arg.indexOf("=") !== -1) {
                              const components = arg.split("=");
                              if (components[1].indexOf(" ") === -1) {
                                 return arg;
                              }
                              return `${components[0]}="${components[1]}"`;
                           } else {
                              return arg;
                           }
                        }).join(" ")} disabled={true} label="Job Arguments" multiline resizable={false} />
                     </Stack>}
                  </Stack>
               </Stack>
               <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
                  <Stack>
                     <DefaultButton text="Use New Dialog" style={{ width: 160 }} onClick={() => setNewBuildV2(true)} />
                  </Stack>

                  <Stack grow />
                  <PrimaryButton text="Start Job" disabled={submitting || !template || readOnly} onClick={() => { onSubmit(); }} />
                  <DefaultButton text="Cancel" disabled={submitting} onClick={() => { close(); }} />
               </Stack>
            </Stack>
         </Stack>
      </Modal >
   </Stack >;

});


const ValidationErrorModal: React.FC<{ errors: ValidationError[], show: boolean; onClose: () => void }> = ({ errors, show, onClose }) => {

   const { hordeClasses } = getHordeStyling();

   const close = () => {
      onClose();
   };

   const messages = errors.map((e, idx) => {
      return <MessageBar key={`validation_error_${idx}`} messageBarType={MessageBarType.error} isMultiline={true}>
         <Stack>
            <Stack grow tokens={{ childrenGap: 12 }}>
               <Stack tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Parameter:</Text>
                  <Text nowrap >{e.paramString}</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Value:</Text>
                  <Text nowrap >{e.value ? e.value : "no value"}</Text>
               </Stack>
               <Stack grow tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Error:</Text>
                  <Text>{e.error}</Text>
               </Stack>
            </Stack>
            {
               e.context && <Stack style={{ paddingTop: 18, paddingBottom: 18 }}>
                  <Stack grow tokens={{ childrenGap: 12 }}>
                     <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold" }}>Context:</Text>
                     <Text>{e.context}</Text>
                  </Stack>
               </Stack>
            }

         </Stack>
      </MessageBar>
   })

   return <Modal isOpen={show} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 700 } }} onDismiss={() => { if (show) close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack style={{ paddingLeft: 8, paddingTop: 4 }} grow>
            <Text variant="mediumPlus">Validation Errors</Text>
         </Stack>
         <Stack grow horizontalAlign="end">
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack>
      </Stack>

      <Stack style={{ paddingLeft: 20, paddingTop: 8, paddingBottom: 8 }}>
         <Text style={{ fontSize: 15 }}>Please correct the following errors and try again.</Text>
      </Stack>

      <Stack tokens={{ childrenGap: 8 }} styles={{ root: { paddingLeft: 20, paddingTop: 18, paddingBottom: 24, width: 660 } }}>
         {messages}
      </Stack>


      <Stack horizontal styles={{ root: { padding: 8, paddingTop: 8 } }}>
         <Stack grow />
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <PrimaryButton text="Ok" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};