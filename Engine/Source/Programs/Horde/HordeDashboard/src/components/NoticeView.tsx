// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, Dialog, DialogFooter, DialogType, DirectionalHint, IColumn, Icon, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField, TooltipHost } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import moment from "moment-timezone";
import React, { useState } from "react";
import backend from "../backend";
import { GetNoticeResponse } from "../backend/Api";
import dashboard from "../backend/Dashboard";
import notices from "../backend/Notices";
import { useWindowSize } from "../base/utilities/hooks";
import { displayTimeZone } from "../base/utilities/timeUtils";
import { Breadcrumbs } from "./Breadcrumbs";
import { TopNav } from "./TopNav";
import { getHordeStyling } from "../styles/Styles";


const NoticePanel: React.FC = observer(() => {

   const [state, setState] = useState<{ showEditor?: boolean, editNotice?: GetNoticeResponse }>({});

   // subscribe
   if (notices.updated) { };

   const columns = [
      { key: 'column_message', name: 'Message', minWidth: 740, maxWidth: 740, isResizable: false },
      { key: 'column_starttime', name: 'Start Time', minWidth: 160, maxWidth: 160, isResizable: false },
      { key: 'column_finishtime', name: 'Finish Time', minWidth: 160, maxWidth: 160, isResizable: false },
      { key: 'column_createdby', name: 'Created By', minWidth: 200, maxWidth: 200, isResizable: false },
   ];

   let allNotices = [...notices.allNotices];

   const userNotice = allNotices.find(n => n.active && n.createdByUser);

   const renderItem = (item: GetNoticeResponse, index?: number, column?: IColumn) => {

      if (column?.name === "Message") {

         if (item.createdByUser && !!dashboard.user?.dashboardFeatures?.showNoticeEditor) {
            return <div onClick={() => { setState({ showEditor: true, editNotice: item }) }} style={{ "cursor": "pointer" }}>
               <Stack horizontal tokens={{ childrenGap: 24 }}>
                  <Icon style={{ paddingTop: 2 }} iconName="Edit" />
                  <Text>{item.message}</Text>
               </Stack>
            </div>
         }

         return <Stack horizontal tokens={{ childrenGap: 24 }}>
            <Icon style={{ paddingTop: 2 }} iconName="Build" />
            <Text>{item.scheduledDowntime ? "Horde Scheduled Downtime" : item.message ?? "???"}</Text>
         </Stack>
      }
      function getDisplayTime(date?: Date) {
         return date ? moment(date).tz(displayTimeZone()).format(dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z") : "";
      }

      if (column?.name === "Start Time") {
         if (item.startTime) {
            return <Text>{getDisplayTime(item.startTime as Date)}</Text>
         }
         
         return <Stack style={{width: 25}}>
               <TooltipHost
                  content={'User notices displayed immediately'}
                  directionalHint={DirectionalHint.bottomCenter}>
                  <Text>{'N/A'}</Text>
               </TooltipHost>
         </Stack>
      }

      if (column?.name === "Finish Time") {
         if (item.finishTime) {
            return <Text>{getDisplayTime(item.finishTime as Date)}</Text>
         }

         return <Stack style={{width: 25}}>
            <TooltipHost
               content={'User notices displayed until issue is resolved'}
               directionalHint={DirectionalHint.bottomCenter}>
               <Text>{'N/A'}</Text>
            </TooltipHost>
         </Stack>
         
      }

      if (column?.name === "Created By") {
         return <Text>{item.createdByUser?.name ?? "Horde Configuration"}</Text>
      }

      return null;
   };

   const { hordeClasses } = getHordeStyling();

   return <Stack>
      {!!state.showEditor && <NoticeEditor noticeIn={state.editNotice} onClose={() => {
         setState({})
      }} />}
      <Stack style={{ paddingBottom: 12 }}>
         <Stack verticalAlign="center">
            <Stack horizontalAlign="end">
               { !!dashboard.user?.dashboardFeatures?.showNoticeEditor &&<PrimaryButton styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }} onClick={() => setState({ showEditor: true, editNotice: userNotice })}>Set Notice</PrimaryButton> }
            </Stack>
            {!allNotices.length && <Stack horizontalAlign="center">
               <Text variant="mediumPlus">No Service Notices Found</Text>
            </Stack>}
         </Stack>
      </Stack>

      {!!allNotices.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
            <Stack>
            <DetailsList
                  items={allNotices}
                  columns={columns}
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  compact={true}
                  onRenderItemColumn={renderItem}
               />
            </Stack>
         </Stack>
      </Stack>}
   </Stack>
});


export const NoticeView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Notices' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <NoticePanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

const NoticeEditor: React.FC<{ noticeIn?: GetNoticeResponse, onClose: () => void }> = ({ noticeIn, onClose }) => {

   const [submitting, setSubmitting] = useState(false);
   const [confirmDelete, setConfirmDelete] = useState(false);
   const { hordeClasses } = getHordeStyling();

   let notice = noticeIn ? { ...noticeIn } : { id: undefined, message: undefined };

   const onSave = async () => {

      try {

         setSubmitting(true);
         if (notice.id) {
            await backend.updateNotice({ id: notice.id, message: notice.message });
         } else {
            await backend.createNotice({ message: notice.message! })
         }
         setSubmitting(false);
         onClose();
         notices.poll();

      } catch (reason) {
         console.error(reason);
         setSubmitting(false);
      }

   }

   const onDelete = async () => {

      try {
         setSubmitting(true);
         await backend.deleteNotice(notice.id!);
         setSubmitting(false);
         onClose();
         notices.poll();
      } catch (reason) {
         console.error(reason);
         setSubmitting(false);
      }
   }

   if (submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="mediumPlus">Please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   }

   if (confirmDelete) {
      return <Dialog
         hidden={false}
         onDismiss={() => setConfirmDelete(false)}
         minWidth={400}
         dialogContentProps={{
            type: DialogType.normal,
            title: `Confirm Deletion`,
         }}
         modalProps={{ isBlocking: true, topOffsetFixed: true, styles: { main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } } }} >
         <DialogFooter>
            <PrimaryButton onClick={() => { setConfirmDelete(false); onDelete() }} text="Delete" />
            <DefaultButton onClick={() => setConfirmDelete(false)} text="Cancel" />
         </DialogFooter>
      </Dialog>
   }


   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, hasBeenOpened: false, top: "120px", position: "absolute" } }} >

      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 16 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>{noticeIn ? "Edit Notice" : "New Notice"}</Text>
         </Stack>
         <Stack tokens={{ childrenGap: 8 }} style={{ padding: 8 }}>
            <TextField label="Message" autoComplete="off" placeholder="Please enter the notice message" defaultValue={notice.message ? notice.message : ""} onChange={(ev, value) => { notice.message = value ?? "" }} />
         </Stack>

         <Stack horizontal style={{ paddingTop: 64 }}>
            {!!noticeIn && <PrimaryButton style={{ backgroundColor: "red", border: 0 }} onClick={() => setConfirmDelete(true)} text="Delete Notice" />}
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 28 }}>
               <PrimaryButton onClick={() => onSave()} text="Save" />
               <DefaultButton onClick={() => onClose()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>

}