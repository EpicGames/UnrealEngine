// Copyright Epic Games, Inc. All Rights Reserved.

import { CommandBar, ContextualMenuItemType, ICommandBarItemProps, IContextualMenuProps } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect } from "react";
import backend from "../backend";
import { CreateSubscriptionRequest, GetNotificationResponse, GetSubscriptionResponse, JobState, JobStepState, LabelCompleteEventRecord, LabelState, StepCompleteEventRecord, SubscriptonNotificationType, UpdateNotificationsRequest } from "../backend/Api";
import { action, makeObservable, observable } from "mobx";
import { JobDetailsV2 } from './jobDetailsV2/JobDetailsViewCommon';
import { useQuery } from "horde/base/utilities/hooks";
import { getHordeStyling } from "../styles/Styles";


type NotificationType = "Job" | "Step" | "Label";
type NotificationOutcome = "Warnings" | "Success" | "Failure";

class SubscriptionHandler  {

   constructor() {
      makeObservable(this);
   }

   @observable
   private updated = 0

   @action
   updateReady() {
      this.updated++;
   }

   subscribe() { 
      if (this.updated) {}
   }   

   async initialize(details: JobDetailsV2, stepId?: string, labelIdx?: number) {

      if (this.initialized || this.initializing) {
         return;
      }

      this.initializing = true;

      this.details = details;      

      this.njob = await backend.getNotification("job", this.details?.jobId!);
      this.subs = await backend.getSubscriptions();

      let type:NotificationType = "Job";
      if (stepId) {
         type = "Step";
      }
      if (typeof(labelIdx) === `number`) {
         type = "Label";
      }

      this.updateMulti(type);

      this.initialized = true;
      this.initializing = false;
      this.updateReady();
   }

   async updateNotification(request: UpdateNotificationsRequest, labelIdx?: string, stepId?: string) {

      const jobId = this.details?.jobId;

      if (!jobId) {
         return;
      }

      let type = "Job";
      if (labelIdx) {
         type = "Label";
      }
      if (stepId) {
         type = "Step";
      }

      let batchId: string | undefined;

      if (stepId) {
         batchId = this.details?.batchByStepId(this.stepId)?.id;
         if (!batchId) {
            console.error(`Unable to get batch for job ${jobId} step ${stepId}`);
            return;
         }
      }

      const success = await backend.updateNotification(request, type.toLowerCase(), jobId, labelIdx, batchId, stepId);

      if (!success) {
         console.error(`Unable to update notification for job ${jobId} type: ${type} step: ${stepId} label: ${labelIdx}`);
      }

      const notification = await backend.getNotification(type.toLowerCase(), jobId, labelIdx, batchId, stepId);
      if (type === "Job") {
         this.njob = notification;
      }
      if (type === "Step") {
         this.nstep = notification;
      }
      if (type === "Label") {
         this.nlabel = notification;
      }

      this.updateReady();
   }

   async createSubscription() {
      await backend.createSubscription([]);
   }

   async addMultiSubscription(outcome: NotificationOutcome | "Any") {

      const details = this.details;
      if (!details) {
         return;
      }
      const templateId = details.stream?.templates.find(c => c.name === details.template?.name)?.id;
      if (!templateId) {
         return;
      }

      let stepName: string | undefined;
      if (this.stepId) {
         stepName = details.getStepName(this.stepId);
      }

      const label = details.labelByIndex(this.labelIdx)?.stateResponse

      const labelName = label?.dashboardName;
      const categoryName = label?.dashboardCategory;

      const streamId = details.stream!.id;

      let type:NotificationType = "Job";
      if (this.stepId) {
         type = "Step";
      }
      if (typeof(this.labelIdx) === `number`) {
         type = "Label";
      }

      const outcomes = this.outcomes;

      const cfailure = outcomes.get('Failure');
      const csuccess = outcomes.get('Success');
      const cwarnings = outcomes.get('Warnings');

      const nfailure: CreateSubscriptionRequest = { event: { type: type, streamId: streamId, templateId: templateId, outcome: 'failure' }, notificationType: SubscriptonNotificationType.Slack };
      const nsuccess: CreateSubscriptionRequest = { event: { type: type, streamId: streamId, templateId: templateId, outcome: 'success' }, notificationType: SubscriptonNotificationType.Slack };
      const nwarnings:CreateSubscriptionRequest = { event: { type: type, streamId: streamId, templateId: templateId, outcome: 'warnings' }, notificationType: SubscriptonNotificationType.Slack };

      let add: CreateSubscriptionRequest[] = [];
      let remove: string[] = [];

      if (outcome === 'Any') {
         // if set to any, subscribe to all three
         if (!cfailure) {
            add.push(nfailure);
         }
         if (!csuccess) {
            add.push(nsuccess);
         }
         if (!cwarnings) {
            add.push(nwarnings);
         }
      }
      else if (outcome === 'Warnings') {
         // subscribe to errors and warnings if flagged as 'warnings'
         if (!cfailure) {
            add.push(nfailure);
         }
         if (!cwarnings) {
            add.push(nwarnings);
         }
      }
      else if (outcome === 'Failure') {

         if (cwarnings) {
            remove.push(cwarnings);
         }

         if (!cfailure) {
            add.push(nfailure);
         }
      }
      else if (outcome === 'Success') {
         if (!csuccess) {
            add.push(nsuccess);
         }
      }


      if (type === "Step" && stepName) {
         add.forEach(a => {
            (a.event as StepCompleteEventRecord).stepName = stepName!;
         });
      }
      if (type === "Label" && categoryName && labelName) {

         add.forEach(a => {
            (a.event as LabelCompleteEventRecord).labelName = labelName!;
            (a.event as LabelCompleteEventRecord).categoryName = categoryName!;
         });
      }

      if (add.length) {
         await backend.createSubscription(add);

      }

      const deletes = remove.map(id => backend.deleteSubscription(id));

      await Promise.all(deletes);

      if (add.length || remove.length) {
         this.subs = await backend.getSubscriptions();
         this.updateMulti(type);
         this.updateReady();
      }
   }

   async removeMultiSubscription(outcome: NotificationOutcome | "Any") {

      const outcomes = this.outcomes;

      const cfailure = outcomes.get('Failure');
      const csuccess = outcomes.get('Success');
      const cwarnings = outcomes.get('Warnings');

      const remove: string[] = [];

      if (outcome === 'Any') {
         // if set to any, subscribe to all three
         if (cfailure) {
            remove.push(cfailure);
         }
         if (csuccess) {
            remove.push(csuccess);
         }
         if (cwarnings) {
            remove.push(cwarnings);
         }
      }
      else if (outcome === 'Warnings') {

         if (cfailure) {
            remove.push(cfailure);
         }
         if (cwarnings) {
            remove.push(cwarnings);
         }
      }
      else if (outcome === 'Failure') {

         if (cwarnings) {
            remove.push(cwarnings);
         }

         if (cfailure) {
            remove.push(cfailure);
         }
      }
      else if (outcome === 'Success') {
         if (csuccess) {
            remove.push(csuccess);
         }
      }

      const deletes = remove.map(id => backend.deleteSubscription(id));

      await Promise.all(deletes);

      let type:NotificationType = "Job";
      if (this.stepId) {
         type = "Step";
      }
      if (typeof(this.labelIdx) === `number`) {
         type = "Label";
      }

      if (remove.length) {
         this.subs = await backend.getSubscriptions();
         this.updateMulti(type);
         this.updateReady();
      }

   }

   updateMulti(type: NotificationType) {

      const details = this.details;
      if (!details) {
         return;
      }

      this.outcomes = new Map();

      const templateId = details.stream?.templates.find(c => c.name === details.template?.name)?.id;

      if (!templateId) {
         return null;
      }

      let subs = this.subs?.filter(s => {
         return (s.event.type === type) && (s.event.streamId === details.stream?.id) && (s.event.templateId === templateId);
      });

      let label = details.labelByIndex(this.labelIdx)?.stateResponse;

      if (label) {
         subs = subs?.filter((s) => (s.event as LabelCompleteEventRecord).labelName === label!.dashboardName && (s.event as LabelCompleteEventRecord).categoryName === label!.dashboardCategory);
      }

      if (this.stepId) {
         const stepName = this.details?.getStepName(this.stepId);
         subs = subs?.filter((s) => (s.event as StepCompleteEventRecord).stepName === stepName);
      }


      if (!subs?.length) {
         return;
      }

      subs.forEach(s => this.outcomes.set(s.event.outcome as NotificationOutcome, s.id));

   }


   async set(stepId?: string, labelIdx?: number) {

      //console.trace("filter Update")

      const details = this.details;

      if (!details) {
         return;
      }

      if ((this.stepId === stepId) && (this.labelIdx === labelIdx)) {
         return;
      }
      
      let batchId: string | undefined;
      let type: string | undefined;

      if (stepId) {
         type = "Step";
         batchId = details.batchByStepId(stepId)?.id;
         if (!batchId) {
            console.error(`Error getting batch for job ${details.jobId} step ${stepId}`);
         }
      }
      if (typeof (labelIdx) === `number`) {
         const label = details.labelByIndex(labelIdx);
         if (label) {
            type = "Label";         
         } else {
            console.error(`Error getting label for job ${details.jobId} label index ${labelIdx}`);
         }
      }

      if (!type) {
         // console.log("no type")
         this.nstep = this.nlabel = undefined;
         this.labelIdx = this.stepId = undefined;
         this.updateReady();
         this.updateMulti("Job");
         return;
      }


      // console.log("No Match", type, this.stepId, ":", filter.step?.id, " - ", this.labelIdx, ":", labelIdx);

      this.stepId = stepId;
      this.labelIdx = labelIdx;

      const notification = await backend.getNotification(type.toLowerCase(), details.jobId!, labelIdx?.toString(), batchId, stepId);

      this.nlabel = this.nstep = undefined;

      if (stepId) {
         this.nstep = notification;
      }

      if (typeof (labelIdx) === "number") {
         this.nlabel = notification;
      }

      this.updateMulti(type as NotificationType);
      this.updateReady();
   }

   clear() {
      this.initialized = false;
      this.initializing = undefined;
      this.details = undefined;
      this.ready = false;
      this.labelIdx = undefined;
      this.stepId = undefined;
      this.subs = undefined;

      this.njob = undefined;
      this.nstep = undefined;
      this.nlabel = undefined;
      this.outcomes = new Map();
   }

   detailsUpdated() {

   }

   njob?: GetNotificationResponse;
   nstep?: GetNotificationResponse;
   nlabel?: GetNotificationResponse;

   labelIdx?: number
   stepId?: string;

   subs?: GetSubscriptionResponse[];

   outcomes: Map<NotificationOutcome, string> = new Map();

   initialized?: boolean;
   initializing?: boolean;
   details?: JobDetailsV2;

   ready = false;
}

const subHandler = new SubscriptionHandler();

export const NotificationDropdown: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const query = useQuery();
   const { hordeClasses } = getHordeStyling();

   const stepId = query.get("step") ? query.get("step")! : undefined;
   let labelIdx = query.get("label") ? parseInt(query.get("label")!) : undefined;

   if ((typeof (labelIdx) === `number`) && labelIdx < 0) {
      console.error(`NotificationDropdown: Invalid label idx from query string ${labelIdx}`);
      labelIdx = undefined;
   }

   useEffect(() => {
      return () => {
         subHandler.clear();      
      };
   }, []);

   subHandler.subscribe();

   if (!jobDetails.jobData) {
      jobDetails.subscribe();
      return null;
   }

   if (!subHandler.initialized) {
      subHandler.initialize(jobDetails, stepId, labelIdx);
      return null;
   }

   subHandler.set(stepId, labelIdx);

   // console.log("render", "Job:", subHandler.njob?.slack, "Label:", subHandler.nlabel?.slack, "Step:", subHandler.nstep?.slack);

   const step = jobDetails.stepById(stepId);
   const label = jobDetails.labelByIndex(labelIdx);

   let type = "Job";
   let subscribed = subHandler.njob?.slack ?? false;

   let complete = jobDetails.jobData?.state === JobState.Complete;
   if (step) {
      type = "Step";
      subscribed = subHandler.nstep?.slack ?? false;
      complete = !!step.finishTime || (step.state === JobStepState.Aborted) || (step.state === JobStepState.Skipped) || (step.state === JobStepState.Completed);
   }
   if (label) {
      type = "Label";
      const aggregate = jobDetails.findLabel(label.stateResponse.dashboardName ?? "", label.stateResponse.dashboardCategory)!;
      complete = aggregate?.stateResponse?.state === LabelState.Complete;
      subscribed = subHandler.nlabel?.slack ?? false;
   }

   let iconStatuses: { [type: string]: { 'icon': string, className: string } } = {
      'Any': { icon: 'Message', className: '' },
      'Success': { icon: 'Message', className: '' },
      'Warnings': { icon: 'Message', className: '' },
      'Failure': { icon: 'Message', className: '' },
   }

   for (let key in iconStatuses) {
      const outcome = subHandler.outcomes.get(key as NotificationOutcome);
      if (outcome !== undefined) {
         iconStatuses[key].icon = 'MessageFill';
         iconStatuses[key].className = hordeClasses.iconBlue;
      }
   }

   if (subHandler.outcomes.get("Warnings") && subHandler.outcomes.get("Failure") && subHandler.outcomes.get("Success")) {
      iconStatuses['Any'].icon = "MessageFill";
      iconStatuses['Any'].className = hordeClasses.iconBlue;
   }

   // remove failure highlight if both errors and warnings
   if (subHandler.outcomes.get("Warnings") && subHandler.outcomes.get("Failure")) {
      iconStatuses['Failure'].icon = "Message";
      iconStatuses['Failure'].className = '';
   }


   const menuProps: IContextualMenuProps = {
      items: [
         {
            key: 'notifyThis',
            text: `This ${type}`,
            style: {
               fontSize: '12px'
            },
            disabled: complete,
            iconProps: {
               iconName: subscribed ? 'MessageFill' : 'Message',
               className: !complete ? hordeClasses.iconBlue : hordeClasses.iconDisabled,
               style: {
                  position: 'relative',
                  top: '1px',
                  fontSize: '13px'
               }
            },
            onClick: () => { subHandler.updateNotification({ slack: !subscribed }, labelIdx?.toString(), step?.id) }
         },
         {
            key: 'notifyAll',
            subMenuProps: {
               styles: { root: { overflow: 'hidden' } },
               items: [
                  {
                     key: 'Any',
                     text: 'Any Outcome',
                     style: {
                        fontSize: '12px'
                     },
                     iconProps: {
                        iconName: iconStatuses['Any'].icon,
                        className: iconStatuses['Any'].className,
                        style: {
                           position: 'relative',
                           top: '1px',
                           fontSize: '13px'
                        }
                     },
                     onClick: () => iconStatuses['Any'].icon === 'Message' ? subHandler.addMultiSubscription('Any') : subHandler.removeMultiSubscription('Any')
                  },
                  {
                     key: 'divider',
                     text: '-',
                     style: {
                        fontSize: '12px'
                     },
                     itemType: ContextualMenuItemType.Divider
                  },
                  {
                     key: 'Success',
                     text: 'Success',
                     style: {
                        fontSize: '12px'
                     },
                     iconProps: {
                        iconName: iconStatuses['Success'].icon,
                        className: iconStatuses['Success'].className,
                        style: {
                           position: 'relative',
                           top: '1px',
                           fontSize: '13px'
                        }
                     },
                     onClick: () => { iconStatuses['Success'].icon === 'Message' ? subHandler.addMultiSubscription('Success') : subHandler.removeMultiSubscription('Success') }
                  },
                  {
                     key: 'Warnings',
                     text: 'Errors and Warnings',
                     style: {
                        fontSize: '12px'
                     },
                     iconProps: {
                        iconName: iconStatuses['Warnings'].icon,
                        className: iconStatuses['Warnings'].className,
                        style: {
                           position: 'relative',
                           top: '1px',
                           fontSize: '13px'
                        }
                     },
                     onClick: () => { iconStatuses['Warnings'].icon === 'Message' ? subHandler.addMultiSubscription('Warnings') : subHandler.removeMultiSubscription('Warnings') }
                  },
                  {
                     key: 'Failure',
                     text: 'Only Errors',
                     style: {
                        fontSize: '12px'
                     },
                     iconProps: {
                        iconName: iconStatuses['Failure'].icon,
                        className: iconStatuses['Failure'].className,
                        style: {
                           position: 'relative',
                           top: '1px',
                           fontSize: '13px'
                        }
                     },
                     onClick: () => { iconStatuses['Failure'].icon === 'Message' ? subHandler.addMultiSubscription('Failure') : subHandler.removeMultiSubscription('Failure') }
                  },
               ]
            },
            text: `All ${type}s of This Type`,
            style: {
               fontSize: '12px'
            }
         }
      ]
   };

   const notificationItems: ICommandBarItemProps[] = [
      {
         key: 'notification_items',
         text: "Notifications",
         iconProps: { iconName: "Message" },
         subMenuProps: menuProps
      }
   ];

   return <CommandBar items={notificationItems} onReduceData={() => undefined} />
});
