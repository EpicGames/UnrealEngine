// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, IColumn, SelectionMode, Stack, Text, TooltipHost } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect } from "react";
import backend from "../backend";
import { GetPerforceServerStatusResponse } from "../backend/Api";
import { PollBase } from "../backend/PollBase";
import { useWindowSize } from "../base/utilities/hooks";
import { Breadcrumbs } from "./Breadcrumbs";
import { TopNav } from "./TopNav";
import { getHordeStyling } from "../styles/Styles";

class PerforceServerHandler extends PollBase {

   constructor(pollTime = 5000) {

      super(pollTime);

   }

   clear() {
      this.loaded = false;
      super.stop();
   }

   async poll(): Promise<void> {

      try {

         this.status = await backend.getPerforceServerStatus();
         this.loaded = true;
         this.setUpdated();

      } catch (err) {

      }

   }
   loaded = false;
   status: GetPerforceServerStatusResponse[] = [];
}

const handler = new PerforceServerHandler();

const ServerPanel: React.FC = observer(() => {

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { hordeClasses, modeColors } = getHordeStyling();

   // subscribe
   if (handler.updated) { };

   const columns = [
      { key: 'column_cluster', name: 'Cluster', fieldName: 'cluster', minWidth: 140, maxWidth: 140, isResizable: false },
      { key: 'column_numleases', name: 'Leases', fieldName: 'numLeases', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'column_status', name: 'Status', fieldName: 'status', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_server', name: 'Server', fieldName: 'serverAndPort', minWidth: 330, maxWidth: 330, isResizable: false },
      { key: 'column_detail', name: 'Detail', fieldName: 'detail', minWidth: 200, maxWidth: 200, isResizable: false, isMultiline: true },
   ];

   let status = [...handler.status];

   if (!handler.loaded) {
      return null;
   }

   if (!status.length) {
      return <Stack horizontalAlign="center">
         <Text variant="mediumPlus">No Perforce Servers Found</Text>
      </Stack>
   }

   status = status.sort((a, b) => {

      if (a.cluster === b.cluster) {
         return b.numLeases - a.numLeases;
      }

      if (a.cluster === "Default") return -1;
      if (b.cluster === "Default") return 1;

      return a.cluster.toLowerCase() < b.cluster.toLowerCase() ? -1 : 1;
   });

   const renderItem = (item: any, index?: number, column?: IColumn) => {
      if (!column?.fieldName) {
         return null;
      }

      return <TooltipHost content={column?.fieldName === "detail" ? item[column?.fieldName] : ''} styles={{ root: { display: 'inline-block' } }}>
               <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
                  <Text style={{ color: modeColors.text }}>{item[column?.fieldName]}</Text>
               </Stack>
         </TooltipHost>
         
   };

   return <Stack>
      {!status.length && <Stack style={{ paddingBottom: 12 }}>
         <Stack verticalAlign="center">
            {!status.length && handler.loaded && <Stack horizontalAlign="center">
               <Text variant="mediumPlus">No Perforce Servers Found</Text>
            </Stack>}
         </Stack>
      </Stack>
      }

      {!!status.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
            <Stack>
               <DetailsList
                  items={status}
                  columns={columns}
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  compact={true}
                  onRenderItemColumn={renderItem}
               />
            </Stack>
         </Stack>
      </Stack>
      }
   </Stack>
});


export const PerforceServerView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Perforce Servers' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <ServerPanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

