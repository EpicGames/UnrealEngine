// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, ComboBox, DefaultButton, DetailsList, DetailsListLayoutMode, Dialog, DialogFooter, DialogType, IColumn, IComboBoxOption, IconButton, MessageBar, MessageBarType, Modal, Position, PrimaryButton, ScrollablePane, ScrollbarVisibility, SelectionMode, Slider, SpinButton, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import moment from "moment";
import React, { useEffect, useState } from "react";
import { Link, useSearchParams } from "react-router-dom";
import backend from "../backend";
import { GetAgentLeaseResponse, GetAgentResponse, GetBatchResponse, GetJobTimingResponse, GetPoolResponse, GetStepResponse, JobData, JobQuery, JobState, JobStepBatchError, JobStepBatchState, JobStepState, LeaseState, PoolSizeStrategy, UpdatePoolRequest } from "../backend/Api";
import dashboard from "../backend/Dashboard";
import { PollBase } from "../backend/PollBase";
import { projectStore } from "../backend/ProjectStore";
import { getElapsedString, getShortNiceTime, getStepElapsed, getStepStartTime } from "../base/utilities/timeUtils";
import { getHordeStyling, linearInterpolate } from "../styles/Styles";
import { AgentPanel } from "./agents/AgentView";
import { HistoryModal } from "./agents/HistoryModal";
import { LeaseStatusIcon, StepStatusIcon } from "./StatusIcon";


type PendingBatch = {
   job: JobData;
   batch: GetBatchResponse;
}

// UI visible text for values that are undefined
const UNSET_VALUE: string = "-";

const percent = (value: number) => {
   return value.toLocaleString(undefined, { style: 'percent', minimumFractionDigits: 0 })
}


class PoolHandler extends PollBase {

   constructor(pollTime = 15000) {
      super(pollTime);
   }

   clear() {
      this.poolId = undefined;
      this.pool = undefined;
      this.agents = undefined;
      this.agentJobs = new Map();
      this.activeSteps = new Map();
      this.pendingSteps = new Map();
      this.pendingBatches = [];
      this.jobData = new Map();
      this.activeConforms = new Map<string, GetAgentLeaseResponse>();
      super.stop();
   }

   set(poolId?: string) {

      if (poolId === this.poolId) {
         return;
      }

      if (poolId) {
         this.clear();
         this.poolId = poolId;
         this.start();
      } else {
         this.poolId = undefined;
      }
   }

   async poll(): Promise<void> {

      if (!this.poolId) {
         return;
      }

      try {

         const pool = this.pool = await backend.getPool(this.poolId);

         // @todo: optimize using modifiedAfter
         const agents = this.agents = await backend.getAgents({ poolId: this.poolId });

         this.agentJobs = new Map();
         this.activeSteps = new Map();
         this.pendingSteps = new Map();
         this.pendingBatches = [];
         this.jobData = new Map();

         const streamJobIds = new Set<string>();

         const wstreams = new Set<string>();

         projectStore.projects.map(p => p.streams ?? []).flat().filter(s => {
            for (let v in s.agentTypes) {
               if (s.agentTypes[v].pool === pool.id) {
                  return true;
               }
            }
            return false;

         }).forEach(s => wstreams.add(s.id));


         // build up stream job list
         let poolStreams = Array.from(wstreams);

         const minCreateTime = new Date(Math.round((new Date().getTime() / 1000) - (48 * 3600)) * 1000).toISOString();

         this.activeConforms = new Map<string, GetAgentLeaseResponse>();

         agents.forEach(a => {
            const lease = a.leases?.find(lease => lease.name === "Updating workspaces" && (lease.state === LeaseState.Pending || lease.state === LeaseState.Active))
            if (lease) {
               // make sure populated
               lease.agentId = a.id;
               this.activeConforms.set(a.id, lease);
            }
         })

         while (poolStreams.length) {

            const batch = poolStreams.slice(0, 5);

            const requests = batch.map(streamId => {

               const query: JobQuery = {
                  streamId: streamId,
                  filter: "id,state",
                  minCreateTime: minCreateTime,
                  count: 250
               }
               return backend.getJobs(query);

            }).filter(r => !!r);

            await Promise.all(requests).then((responses) => {

               responses.forEach(r => {

                  r!.forEach(j => {
                     if (j.state !== JobState.Complete) {
                        streamJobIds.add(j.id);
                     }
                  });
               });

            }).catch((errors) => {
               console.log(errors);
               // eslint-disable-next-line
            }).finally(() => {
               poolStreams = poolStreams.slice(5);
            });
         }

         if (streamJobIds.size) {

            const streamJobs = await backend.getJobs({ id: Array.from(streamJobIds), filter: "id,name,batches,change,createTime,streamId,preflightChange,preflightCommitId" });

            streamJobs.forEach(job => {

               job.batches?.forEach(b => {

                  if (b.agentId) {
                     return;
                  }

                  if (!b.steps.length || (b.steps[0].state === JobStepState.Aborted || b.steps[0].state === JobStepState.Skipped)) {
                     return;
                  }


                  const agentType = b?.agentType;

                  const stream = projectStore.streamById(job.streamId);

                  if (!stream) {
                     return;
                  }

                  const agentPool = stream.agentTypes[agentType!]?.pool;

                  if (!agentPool || agentPool !== this.pool?.id) {
                     return;
                  }

                  this.pendingBatches.push({ job: job, batch: b });

               })
            })
         }

         const jobIds = agents.map(agent => {

            const lease = agent.leases?.find(lease => {
               return lease.details && lease.details["jobId"];
            });

            return {
               agent: agent,
               jobId: lease?.details!["jobId"]
            }

         }).filter(jobInfo => !!jobInfo.jobId).map(jobInfo => {
            this.agentJobs.set(jobInfo.agent.id, jobInfo.jobId!);
            return jobInfo.jobId!
         });


         const activeJobs: JobData[] = [];

         if (jobIds.length) {

            const query: JobQuery = {
               id: jobIds,
               filter: "id,name,batches,change,createTime,streamId,preflightChange,preflightCommitId",
            }

            const jobs = await backend.getJobs(query);

            jobs.forEach(job => {

               this.jobData.set(job.id, job);

               job.batches?.forEach(batch => {

                  if (!this.agents!.find(a => a.id === batch.agentId) || !batch.startTime || batch.finishTime) {
                     return;
                  }

                  if (!batch) {
                     return;
                  }

                  const active = batch.steps?.find(step => !!step.startTime && !step.finishTime);
                  const pending = !!batch?.startTime && batch.steps?.find(step => !step.startTime && step.state !== JobStepState.Skipped && step.state !== JobStepState.Aborted);

                  if (active) {
                     if (!activeJobs.find(j => j.id === job.id)) {
                        activeJobs.push(job);
                     }
                     this.activeSteps.set(batch.agentId!, active);
                  }
                  if (pending) {
                     this.pendingSteps.set(batch.agentId!, pending);
                  }

               });

            });
         }

         this.setUpdated();

      } catch (err) {

      }

   }

   poolId?: string;
   // agentId => jobId
   agentJobs: Map<string, string> = new Map();

   activeConforms = new Map<string, GetAgentLeaseResponse>();

   // agentId => step
   activeSteps: Map<string, GetStepResponse> = new Map();

   // agentId => step
   pendingSteps: Map<string, GetStepResponse> = new Map();

   // list of batches which don't have an agent
   pendingBatches: PendingBatch[] = [];

   jobData: Map<string, JobData> = new Map();
   jobTiming: Map<string, GetJobTimingResponse> = new Map();
   agents?: GetAgentResponse[];
   pool?: GetPoolResponse;
}

const handler = new PoolHandler();

enum StepState {
   Active = "Active",
   Pending = "Pending",
   Previous = "Previous"
}

const PoolAgentPanel: React.FC<{ poolId: string }> = ({ poolId }) => {

   return <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12 } }} >
      <Stack tokens={{ childrenGap: 12 }}>
         <Stack>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Agents</Text>
         </Stack>
         <AgentPanel poolId={poolId} agents={handler.agents} />
      </Stack>
   </Stack>
}

const StepPanel: React.FC<{ stepState: StepState }> = ({ stepState }) => {

   const [lastSelectedAgent, setLastSelectedAgent] = useState<string | undefined>(undefined);

   const { modeColors } = getHordeStyling();

   const pool = handler.pool;
   if (!pool || !handler.agents || !handler.agents.length) {
      return null;
   }

   let agents = [...handler.agents];

   // active steps
   if (stepState === StepState.Active) {

      agents = agents.filter(a => !!handler.activeSteps.get(a.id));

      agents = agents.sort((a, b) => {
         const stepA = handler.activeSteps.get(a.id)!;
         const stepB = handler.activeSteps.get(b.id)!;
         return new Date(stepA.startTime!) < new Date(stepB.startTime!) ? -1 : 1;

      });

   }

   // pending steps
   if (stepState === StepState.Pending) {

      agents = agents.filter(a => !!handler.pendingSteps.get(a.id));

      agents = agents.sort((a, b) => {
         const jobA = handler.jobData.get(handler.agentJobs.get(a.id)!)!;
         const jobB = handler.jobData.get(handler.agentJobs.get(b.id)!)!;
         const stepA = handler.pendingSteps.get(a.id)!;
         const stepB = handler.pendingSteps.get(b.id)!;
         const batchA = jobA?.batches?.find(b => b.steps.find(s => s.id === stepA?.id))!;
         const batchB = jobB?.batches?.find(b => b.steps.find(s => s.id === stepB?.id))!;

         if (!batchA?.startTime || !batchB?.startTime) {
            console.error("Bad batch start time in pool view");
            return 0;
         }
         return new Date(batchA.startTime!) < new Date(batchB.startTime!) ? -1 : 1;

      });

   }


   type AgentItem = {
      agent: GetAgentResponse;
   }

   let columns: IColumn[] = [];

   columns = [
      { key: 'column1', name: 'Job', minWidth: 790, maxWidth: 790 },
      { key: 'column2', name: 'Agent', minWidth: 100, maxWidth: 100 },
      { key: 'column3', name: 'Time Active', minWidth: 100, maxWidth: 100 },
      { key: 'column4', name: 'Start Time', minWidth: 100, maxWidth: 100 },
      { key: 'column5', name: 'View Log', minWidth: 80, maxWidth: 80 }
   ];

   const agentItems: AgentItem[] = agents.map(a => {
      return {
         agent: a
      }
   }) ?? [];

   const onRenderItemColumn = (item: AgentItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;
      const agent = item.agent;
      const job = handler.jobData.get(handler.agentJobs.get(agent.id)!);
      let step: GetStepResponse | undefined;
      if (stepState === StepState.Active) {
         step = handler.activeSteps.get(agent.id);
      }
      if (stepState === StepState.Pending) {
         step = handler.pendingSteps.get(agent.id);
      }

      if (!job || !step) {
         return null;
      }

      const batch = job.batches!.find(b => b.steps.find(s => s.id === step!.id));
      if (!batch) {
         return null;
      }
      if (column.name === "Time Active") {
         if (stepState === StepState.Pending) {
            return null;
         }
         return <Stack horizontalAlign={"end"}><Text>{getStepElapsed(step)}</Text></Stack>;
      }

      if (column.name === "Start Time") {
         const started = getStepStartTime(step);
         if (!started.display || !started.server) {
            return null;
         } else {
            return <Stack horizontalAlign={"end"}><Text style={{ fontSize: "13px" }}>{started.display}</Text></Stack>;
         }
      }


      if (column.name === "View Log") {
         if (stepState === StepState.Pending) {
            return null;
         }
         return <Stack style={{ paddingRight: 8 }}>
            <Link to={`/log/${step.logId}`} target="_blank">
               <Stack horizontal horizontalAlign={"end"} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}>
                  <Text styles={{ root: { margin: '0px', padding: '0px', paddingRight: '8px' } }} className={"view-log-link"}>View Log</Text>
               </Stack>
            </Link>
         </Stack>
      }


      if (column.name === "Job") {
         let jobName = job.name;
         if (jobName.indexOf("- Kicked By") !== -1) {
            jobName = jobName.split("- Kicked By")[0];
         }
         let stepName = `${jobName} - ${step.name}`;
         const stepUrl = `/job/${job.id}?step=${step.id}`;

         const stream = projectStore.streamById(job.streamId);
         if (stream) {
            stepName = `${stream.fullname ?? stream.id} - ${stepName}`
         }

         return <Stack>
            <Link target="_blank" to={stepUrl}>
               <Stack horizontal>
                  <StepStatusIcon step={step} />
                  <Text>{stepName}</Text>
               </Stack>
            </Link>
         </Stack>;

      }

      if (column.name === "Agent") {
         return <div style={{ cursor: "pointer" }} onClick={() => { setLastSelectedAgent(agent.id) }}>
            <Text>{agent.name}</Text>
         </div>

      }

      return null;
   }

   const count = Math.min(agentItems.length, stepState === StepState.Active ? 8 : 6);
   let height = count * 32;

   return (<Stack>
      <HistoryModal agentId={lastSelectedAgent} onDismiss={() => setLastSelectedAgent(undefined)} />
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12 } }} >
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{stepState} Steps ({agentItems.length})</Text>
            </Stack>
            <Stack style={{ position: "relative", height: height, width: 1280 }}>
               <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always}>
                  <DetailsList
                     styles={{
                        root: {
                           selectors: {
                              '.ms-List-cell': {
                                 color: modeColors.text,
                                 minHeight: 22,
                                 fontSize: 12
                              }
                           }
                        }
                     }}
                     compact={true}
                     items={agentItems}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />
               </ScrollablePane>
            </Stack>
         </Stack>
      </Stack>
   </Stack>)
};


const BatchPanel: React.FC = () => {

   const { modeColors } = getHordeStyling();

   const pool = handler.pool;
   if (!pool || !handler.agents || !handler.agents.length) {
      return null;
   }

   let batches = handler.pendingBatches.sort((a, b) => {

      if (a.batch.state === JobStepBatchState.Ready && b.batch.state === JobStepBatchState.Waiting) {
         return -1;
      }

      if (b.batch.state === JobStepBatchState.Ready && a.batch.state === JobStepBatchState.Waiting) {
         return 1;
      }

      return new Date(a.job.createTime) < new Date(b.job.createTime) ? -1 : 1;

   });

   type BatchItem = {
      pending: PendingBatch;
   }

   let columns: IColumn[] = [];

   columns = [
      { key: 'column1', name: 'Job', minWidth: 700, maxWidth: 700 },
      { key: 'column2', name: 'Job Created', minWidth: 200, maxWidth: 200 },
      { key: 'column3', name: 'Status', minWidth: 200, maxWidth: 200 },
      { key: 'column4', name: 'WaitTime', minWidth: 100, maxWidth: 100 },
   ];

   const batchItems: BatchItem[] = batches.map(b => {
      return {
         pending: b
      }
   }) ?? [];

   const onRenderItemColumn = (item: BatchItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;
      const job = item.pending.job;
      const batch = item.pending.batch;

      const stream = projectStore.streamById(job.streamId);


      if (column.name === "Status") {
         let statusText = "";

         if (batch.state === JobStepBatchState.Waiting) {
            statusText = "Waiting for dependencies";
         }

         if (batch.state === JobStepBatchState.Ready) {
            statusText = `Waiting for agent`;
         }

         return <Stack horizontalAlign="end"><Text>{statusText}</Text></Stack>
      }

      if (column.name === "WaitTime") {

         let statusText = "";

         if (batch.state === JobStepBatchState.Ready) {            
            if (batch.readyTime) {
               statusText += `${getElapsedString(moment(batch.readyTime), moment(Date.now()), false)}`;
            }
         }

         if (!statusText) {
            return null;
         }

         return <Stack style={{ paddingRight: 32 }} horizontalAlign="end"><Text>{statusText}</Text></Stack>
      }


      if (column.name === "Job Created") {

         return <Stack ><Text style={{ fontSize: "13px", paddingRight: 12 }}>{getShortNiceTime(job.createTime, true, true)}</Text></Stack>;
      }


      if (column.name === "Job") {
         let jobName = job.name;
         if (jobName.indexOf("- Kicked By") !== -1) {
            jobName = jobName.split("- Kicked By")[0];
         }
         let step: GetStepResponse | undefined;
         if (batch.steps.length) {
            step = batch.steps[0];
            jobName += ` - ${step.name}`;
            if (stream) {
               jobName = `${stream.fullname ?? stream.id} - ` + jobName;
            }
         }

         const jobUrl = `/job/${job.id}`;

         return <Stack horizontal>
            {!!step && <StepStatusIcon step={step} />}
            <Link target="_blank" to={jobUrl}>
               <Text>{jobName}</Text>
            </Link>
         </Stack>;

      }

      return null;
   }

   const count = Math.min(batchItems.length, 8);
   let height = 60 + count * 32;

   return (<Stack>
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12 } }} >
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Pending Batches ({batchItems.length})</Text>
            </Stack>
            {!!batchItems.length && <Stack style={{ position: "relative", height: height, width: 1280 }}>
               <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always}>
                  <DetailsList
                     styles={{
                        root: {
                           selectors: {
                              '.ms-List-cell': {
                                 color: modeColors.text,
                                 minHeight: 22,
                                 fontSize: 12
                              }
                           }
                        }
                     }}
                     compact={true}
                     items={batchItems}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />
               </ScrollablePane>
            </Stack>}
         </Stack>
      </Stack>
   </Stack>)
};

const ConformPanel: React.FC = () => {

   const [lastSelectedAgent, setLastSelectedAgent] = useState<string | undefined>(undefined);
   const { modeColors } = getHordeStyling();

   const pool = handler.pool;
   if (!pool) {
      return null;
   }

   let conforms = Array.from(handler.activeConforms.values());

   conforms = conforms.sort((a, b) => {

      if (a.state === LeaseState.Active && b.state === LeaseState.Pending) {
         return -1;
      }

      if (b.state === LeaseState.Active && a.state === LeaseState.Pending) {
         return 1;
      }

      return new Date(a.startTime) < new Date(b.startTime) ? -1 : 1;

   });

   type ConformItem = {
      lease: GetAgentLeaseResponse;
   }

   const items = conforms.map(c => { return { lease: c } });

   let columns: IColumn[] = [];

   columns = [
      { key: 'column1', name: 'AgentId', minWidth: 100, maxWidth: 100 },
      { key: 'column2', name: 'Type', minWidth: 600, maxWidth: 600 },
      { key: 'column3', name: 'Duration', minWidth: 100, maxWidth: 100 },
      { key: 'column4', name: 'Started', minWidth: 100, maxWidth: 100 },
      { key: 'column5', name: 'ViewLog', minWidth: 80, maxWidth: 80 }
   ];

   const onRenderItemColumn = (item: ConformItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;
      const lease = item.lease;

      if (column.name === "AgentId") {
         return <div style={{ cursor: "pointer" }} onClick={() => { setLastSelectedAgent(lease.agentId) }}><Stack horizontal verticalAlign="center">
            <LeaseStatusIcon lease={lease} />
            <Text style={{ fontSize: "13px" }}>{lease.agentId}</Text>
         </Stack></div>;
      }

      if (column.name === "Type") {
         return <Stack ><Text style={{ fontSize: "13px" }}>{lease.name ?? "Unknown lease type"}</Text></Stack>;
      }

      if (column.name === "Duration") {


         return <Stack horizontalAlign="end"><Text style={{ fontSize: "13px" }}>{getElapsedString(moment(lease.startTime), moment.utc(), true)}</Text></Stack>;
      }

      if (column.name === "Started") {

         return <Stack horizontalAlign="end"><Text style={{ fontSize: "13px" }}>{getShortNiceTime(lease.startTime, false, true, false)}</Text></Stack>;
      }

      if (column.name === "ViewLog") {
         if (!lease.logId) {
            return null;
         }
         return <Stack horizontalAlign="end" style={{ paddingRight: 32 }}>
            <Link to={`/log/${lease.logId}`} target="_blank">
               <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}>
                  <Text styles={{ root: { margin: '0px', padding: '0px', paddingRight: '8px' } }} className={"view-log-link"}>View Log</Text>
               </Stack>
            </Link>
         </Stack>
      }


      return null;
   }

   const count = Math.min(items.length, 8);
   let height = 60 + count * 32;

   return (<Stack>
      <HistoryModal agentId={lastSelectedAgent} onDismiss={() => setLastSelectedAgent(undefined)} />
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12 } }} >
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Active Conforms ({items.length})</Text>
            </Stack>
            {!!items.length && <Stack style={{ position: "relative", height: height, width: 1280 }}>
               <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always}>
                  <DetailsList
                     styles={{
                        root: {
                           selectors: {
                              '.ms-List-cell': {
                                 color: modeColors.text,
                                 minHeight: 22,
                                 fontSize: 12
                              }
                           }
                        }
                     }}
                     compact={true}
                     items={items}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />
               </ScrollablePane>
            </Stack>}
         </Stack>
      </Stack>
   </Stack>)
};


const AutoScalerPanel: React.FC = () => {

   const [state, setState] = useState<{ error?: string, submitting?: boolean, confirmed?: boolean, modified?: boolean, name?: string, color?: string, autoscale?: boolean, minAgents?: number, reserveAgents?: number, strategy?: PoolSizeStrategy, conformInterval?: number }>({});
   const { hordeClasses } = getHordeStyling();

   const pool = handler.pool;
   if (!pool) {
      return null;
   }

   const stratItems: IComboBoxOption[] = [
      { key: PoolSizeStrategy.LeaseUtilization, text: 'Lease Utilization', data: PoolSizeStrategy.LeaseUtilization },
      { key: PoolSizeStrategy.JobQueue, text: 'Job Queue', data: PoolSizeStrategy.JobQueue }
   ];

   if (state.autoscale === undefined) {

      let color = "1";
      if (pool.properties && pool.properties["Color"]) {
         color = pool.properties["Color"];
      }

      setState({ name: pool.name, color: color, autoscale: pool.enableAutoscaling ?? false, minAgents: pool.minAgents, reserveAgents: pool.numReserveAgents, strategy: pool.sizeStrategy ?? PoolSizeStrategy.LeaseUtilization, conformInterval: pool.conformInterval ?? 24 });
      return null;
   }

   let submitDialog: JSX.Element | undefined;

   if (state.submitting) {

      const poolName = state.name?.trim();

      if (!poolName) {
         setState({ ...state, submitting: false, confirmed: false, error: "Please specify a pool name" });
         return null;
      }
      if (!state.color) {
         setState({ ...state, submitting: false, confirmed: false, error: "Invalid color setting, this is a bug" });
         return null;
      }

      if (!state.strategy) {
         setState({ ...state, submitting: false, confirmed: false, error: "Invalid strategy setting, this is a bug" });
         return null;
      }

      if (!state.confirmed) {

         const onSubmit = async () => {

            try {

               const update: UpdatePoolRequest = {
                  name: poolName,
                  enableAutoscaling: state.autoscale ?? false,
                  minAgents: state.minAgents,
                  numReserveAgents: state.reserveAgents,
                  sizeStrategy: state.strategy!,
                  conformInterval: typeof (state.conformInterval) === "number" ? state.conformInterval : 24,
                  properties: { Color: state.color! }
               }

               await backend.updatePool(pool.id, update);

               handler.pool = await backend.getPool(pool.id);

               setState({ ...state, submitting: false, confirmed: undefined, modified: false, error: undefined });

            } catch (reason) {
               setState({ ...state, submitting: false, confirmed: false, error: (reason as any).toSrtring() })
            }
         }

         const close = () => { setState({ ...state, submitting: undefined }) };

         submitDialog = <Dialog
            onDismiss={() => close()}
            isOpen={true}
            minWidth={600}
            dialogContentProps={{
               type: DialogType.largeHeader,
               title: `Confirm Settings for ${pool.name}`,
               isMultiline: true
            }}
            modalProps={{
               isBlocking: true,
               //styles: { main: { minHeight: 600 } },
            }}>
            <Stack tokens={{ childrenGap: 12 }}>
               <Text>{`Pool Name: ${poolName}`}</Text>
               <Text>{`Conform Interval: ${state.conformInterval ?? "24"}`}</Text>
               <Text>{`Autoscale: ${state.autoscale ? "On" : "Off"}`}</Text>
               <Text>{`Minimum Agents: ${state.minAgents ?? UNSET_VALUE}`}</Text>
               <Text>{`Reserve Agents: ${state.reserveAgents ?? UNSET_VALUE}`}</Text>
               <Text>{`Pool Strategy: ${state.strategy}`}</Text>

            </Stack>
            <DialogFooter>
               <PrimaryButton onClick={() => { onSubmit(); setState({ ...state, confirmed: true }) }} text={state.error ? "Try Again" : "Confirm"} />
               <DefaultButton onClick={() => { close() }} text="Cancel" />
            </DialogFooter>
         </Dialog>
      }
   }

   const disabled = !dashboard.user?.dashboardFeatures?.showPoolEditor || state.submitting || pool.configManaged === true;

   let selectedColor = state.color ?? "0";
   let defaultColor = "0";

   if (pool.properties && pool.properties["Color"]) {
      defaultColor = pool.properties["Color"];
   }

   return <Stack>
      {!!submitDialog && submitDialog}
      <Stack style={{ minWidth: 320 }} tokens={{ childrenGap: 12 }}>
         <Stack>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Settings</Text>
            {!!state.error && <MessageBar
               messageBarType={MessageBarType.error}
               isMultiline={false}> {`Error submitting changes: ${state.error}`} </MessageBar>}
         </Stack>
         <Stack horizontal tokens={{ childrenGap: 32 }}>
            <Stack tokens={{ childrenGap: 8 }}>
               <Stack>
                  <TextField label="Pool Name" value={state.name} style={{ width: 240 }} onChange={(ev, newValue) => {
                     ev.preventDefault();
                     setState({ ...state, name: newValue ? newValue : "", modified: true })
                  }} />
               </Stack>
               <Stack horizontal style={{ width: 240 }}>
                  <Stack grow>
                     <Slider
                        styles={{ root: { paddingTop: 15 } }}
                        label="Color"
                        onChange={(value: number) => setState({ ...state, modified: true, color: value.toString() })}
                        defaultValue={parseInt(defaultColor)}
                        max={599}
                        showValue={false}
                     />
                  </Stack>
                  <Stack>
                     <div className={hordeClasses.colorPreview} style={{ backgroundColor: linearInterpolate(selectedColor) }} />
                  </Stack>
               </Stack>

               {!!dashboard.user?.dashboardFeatures?.showPoolEditor && <Stack style={{ paddingTop: 24 }}>
                  <PrimaryButton disabled={disabled || !state.modified} text="Save Changes" onClick={() => { setState({ ...state, submitting: true, confirmed: false, error: undefined }) }} />
               </Stack>}
            </Stack>
            <Stack style={{ paddingLeft: 12 }} tokens={{ childrenGap: 8 }}>
               <Checkbox styles={{ root: { paddingTop: 12 } }} disabled={disabled} label="Autoscaling" checked={state.autoscale} onChange={(ev, checked) => {
                  setState({ ...state, modified: true, autoscale: checked })
               }} />
               <Stack horizontal tokens={{ childrenGap: 18 }}>
                  <ComboBox styles={{ root: { width: 180 } }} disabled={disabled} label="Strategy" selectedKey={state.strategy} options={stratItems} onChange={(ev, option, index, value) => {
                     setState({ ...state, modified: true, strategy: option!.data })
                  }} />
                  <SpinButton styles={{ root: { width: 180 } }} label={`Conform Interval${state.conformInterval === 0 ? " (Disabled)" : " (Hours)"}`} min={0} max={24 * 7} value={state.conformInterval?.toString() ?? "24"} labelPosition={Position.top} onChange={(ev, value) => {
                     setState({ ...state, modified: true, conformInterval: parseInt(value ?? "24") })
                  }} />
               </Stack>
               <Stack horizontal tokens={{ childrenGap: 18 }}>
                  <SpinButton styles={{ root: { width: 128 } }} disabled={disabled} label="Minimum Agents" value={state.minAgents?.toString() ?? UNSET_VALUE} labelPosition={Position.top} onChange={(ev, value) => {
                     setState({ ...state, modified: true, minAgents: value === UNSET_VALUE ? undefined : parseInt(value ?? "0") })
                  }} />
                  <SpinButton styles={{ root: { width: 128 } }} disabled={disabled} label="Reserve Agents" value={state.reserveAgents?.toString() ?? UNSET_VALUE} labelPosition={Position.top} onChange={(ev, value) => {
                     setState({ ...state, modified: true, reserveAgents: value === UNSET_VALUE ? undefined : parseInt(value ?? "0") })
                  }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>

}

const SettingsModal: React.FC<{ onClose: () => void }> = observer(({ onClose }) => {

   const { hordeClasses, modeColors } = getHordeStyling();

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, backgroundColor: modeColors.background, hasBeenOpened: false, top: "24px", position: "absolute", height: "320px" } }} className={hordeClasses.modal} onDismiss={() => onClose()}>
      <Stack style={{ height: "93vh" }}>
         <Stack style={{ height: "100%" }}>
            <Stack style={{ flexBasis: "70px", flexShrink: 0 }}>
               <Stack horizontal styles={{ root: { padding: 8 } }} style={{ padding: 20, paddingBottom: 8 }}>
                  <Stack horizontal style={{ width: 1024 }} tokens={{ childrenGap: 24 }} verticalAlign="center" verticalFill={true}>
                     <AutoScalerPanel />
                     <Stack grow />
                  </Stack>
                  <Stack grow />
                  <Stack horizontalAlign="end">
                     <IconButton
                        iconProps={{ iconName: 'Cancel' }}
                        ariaLabel="Close popup modal"
                        onClick={() => { onClose() }}
                     />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>

});


const StreamPanel: React.FC = observer(() => {

   // subscrive
   if (handler.updated) { }

   const agents = handler.agents;

   const pool = handler.pool;
   if (!pool || !agents?.length) {
      return null;
   }

   type StreamItem = {
      streamId: string;
      streamName: string;
      preflights: number;
      agents: number;
   }

   type StreamMetrics = {
      id: string,
      name: string;
      agents: number;
      preflights: number;
   }

   const active = agents.filter(a => !!a.leases?.length).length;
   const ready = agents.filter(a => !a.leases?.length && a.online && a.enabled && !a.pendingConform && !a.pendingFullConform).length;

   const streamMetrics = new Map<string, StreamMetrics>();

   handler.jobData.forEach((j, a) => {

      const streamId = j.streamId;
      const streamName = projectStore.streamById(streamId)?.fullname ?? "Unknown";

      if (!streamMetrics.has(streamId)) {
         streamMetrics.set(streamId, { id: streamId, name: streamName, agents: 0, preflights: 0 });
      }

      let agents = 0;

      j.batches?.forEach(b => {
         if (!b.leaseId || b.finishTime || b.state === JobStepBatchState.Complete || b.error !== JobStepBatchError.None) {
            return;
         }
         const found = handler.agents?.find((a) => a.id === b.agentId);
         if (found) {
            agents++;
         }
      })

      if (agents) {

         let m = streamMetrics.get(streamId)!;

         if (j.preflightChange) {
            m.preflights += agents;
         } else {
            m.agents += agents;
         }
      }


   })

   const streamItems: StreamItem[] = Array.from(streamMetrics.values()).sort((a, b) => (b.agents + b.preflights) - (a.agents + a.preflights)).map(m => {
      return {
         streamId: m.id,
         streamName: m.name,
         preflights: m.preflights,
         agents: m.agents
      }
   });


   const streamColumns = [
      { key: 'column1', name: 'Stream', minWidth: 240, maxWidth: 240 },
      { key: 'column2', name: 'Jobs', minWidth: 200, maxWidth: 200 },
   ];

   const onRenderStreamColumn = (item: StreamItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;

      // simple cases
      switch (column.name) {
         case 'Stream':
            return <Text >{`${item.streamName}`}</Text>
         case 'Jobs':

            let text = "";
            if (ready + active) {
               text = `${percent((item.preflights + item.agents) / (ready + active))} (`
            }

            if (item.agents) {
               text += `${item.agents}`;
            }

            if (item.preflights) {
               if (item.agents) {
                  text += ` + ${item.preflights} Preflights`;
               } else {
                  text += `${item.preflights} Preflights`;
               }
            }

            text += ")"


            return <Text >{text}</Text>
         default:
            break;
      }

      return null;
   }

   return <Stack style={{ minWidth: 460 }}>
      <Stack style={{ paddingBottom: 12 }}>
         <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Active Agents</Text>
      </Stack>
      <DetailsList
         compact={true}
         items={streamItems}
         columns={streamColumns}
         setKey="set"
         layoutMode={DetailsListLayoutMode.justified}
         isHeaderVisible={false}
         selectionMode={SelectionMode.none}
         onRenderItemColumn={onRenderStreamColumn}
      />
   </Stack>

});

const PoolPanel: React.FC = () => {

   const [state, setState] = useState<{ showSettings?: boolean, showDeleteShadow?: boolean, deletingShadow?: boolean, deleteError?: string }>({})

   const pool = handler.pool;
   if (!pool) {
      return null;
   }

   const onDeleteShadow = async () => {
      setState({ ...state, deletingShadow: true, deleteError: undefined });
      try {
         await backend.deletePool(pool.id);
         handler.pool = await backend.getPool(pool.id);
         setState({ showDeleteShadow: false, deletingShadow: false });
      } catch (reason) {
         setState({ ...state, deletingShadow: false, deleteError: `${reason}` });
      }
   };


   type WorkspaceItem = {
      identifier: string;
      stream: string;
      incremental: boolean;
   }

   const columns = [
      { key: 'column1', name: 'Identifier', minWidth: 100, maxWidth: 100 },
      { key: 'column2', name: 'Stream', minWidth: 100, maxWidth: 200 },
      //{ key: 'column3', name: 'Incremental', minWidth: 100, maxWidth: 200 },
   ];

   const workspaceItems: WorkspaceItem[] = pool.workspaces.map(w => {
      return {
         identifier: w.identifier,
         stream: w.stream,
         incremental: w.bIncremental
      }
   });

   const onRenderItemColumn = (item: WorkspaceItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;

      // simple cases
      switch (column.name) {
         case 'Identifier':
            return <Text>{item.identifier}</Text>
         case 'Stream':
            return <Text>{item.stream}</Text>
         case 'Incremental':
            return <Text >{item.incremental ? "True" : "False"}</Text>
         default:
            break;
      }

      return null;
   }

   type SummaryItem = {
      name: string;
      value: string;
   }

   const summaryColumns = [
      { key: 'column1', name: 'Name', minWidth: 100, maxWidth: 100 },
      { key: 'column2', name: 'Value', minWidth: 100, maxWidth: 100 },
   ];

   const onRenderSummaryItemColumn = (item: SummaryItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;

      if (column.name === "Name") {
         return item.name;
      }

      if (column.name === "Value") {
         if (item.name === "Agents") {
            return <Stack horizontal tokens={{ childrenGap: 4 }}>
               <Stack>{item.value}</Stack>
               <Stack horizontal tokens={{ childrenGap: 4 }}>
                  <Stack> - </Stack>
                  <Stack><Link to={`/reports/utilization?pools=${pool.id}`}>Utilization</Link></Stack>
               </Stack>
            </Stack>
         }
         return item.value;
      }

      return null;
   }

   const summaryItems: SummaryItem[] = [];
   const agents = handler.agents;
   if (agents?.length) {
      const total = agents.length;
      const active = agents.filter(a => !!a.leases?.length).length;
      const offline = agents.filter(a => !a.online).length;
      const disabled = agents.filter(a => !a.enabled).length;
      const ready = agents.filter(a => !a.leases?.length && a.online && a.enabled && !a.pendingConform && !a.pendingFullConform).length;

      let interval = "";

      if (pool.conformInterval === 0) {
         interval = "0 - Disabled";
      } else if (pool.conformInterval !== undefined) {
         interval = `${pool.conformInterval}h`
      } else {
         interval = "24h - Default";
      }

      summaryItems.push({ name: "Agents", value: `${total} ` });
      summaryItems.push({ name: "Active", value: `${percent(active / total)} (${active})` });
      summaryItems.push({ name: "Ready", value: `${percent(ready / total)} (${ready})` });
      summaryItems.push({ name: "Offline", value: `${percent(offline / total)} (${offline})` });
      summaryItems.push({ name: "Disabled", value: `${percent(disabled / total)} (${disabled})` });
      summaryItems.push({ name: "Autoscaling", value: pool.enableAutoscaling ? "On" : "Off" });
      if (pool.enableAutoscaling) {
         summaryItems.push({ name: "Min/Reserve", value: `${pool.minAgents?.toString() ?? "???"} / ${pool.numReserveAgents?.toString() ?? "???"}` });
         summaryItems.push({ name: "Strategy", value: pool.sizeStrategy ?? PoolSizeStrategy.LeaseUtilization });
      }
      summaryItems.push({ name: "Conform Interval", value: interval });

   }

   let color = pool.colorValue;

   return (<Stack>
      {!!state.showSettings && <SettingsModal onClose={() => { setState({ ...state, showSettings: false }) }} />}
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12, width: "100%" } }} >
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal tokens={{ childrenGap: 48 }}>
               <Stack style={{ minWidth: 224 }}>
                  <Stack horizontal style={{ paddingBottom: 18 }} tokens={{ childrenGap: 8 }}>
                     <PrimaryButton text={pool.name} href={`/agents?agent=${encodeURI(pool.id)}&exact=true`} target="_blank" style={{ color: "#FFFFFF", backgroundColor: color, border: "unset", flexShrink: 1 }} />
                     {!!dashboard.user?.dashboardFeatures?.showPoolEditor && pool.configManaged !== true && <IconButton iconProps={{ iconName: "Edit" }} onClick={() => { setState({ ...state, showSettings: true }) }} style={{ color: "#FFFFFF", backgroundColor: color, border: "unset", flexShrink: 1 }} />}
                  </Stack>
                  {pool.configManaged === true && pool.hasDatabaseEntry === true && (
                     <Stack tokens={{ childrenGap: 8 }} styles={{ root: { marginBottom: 8 } }}>
                        <MessageBar messageBarType={MessageBarType.warning} isMultiline={false}>
                           This pool is managed via config files but also has a database entry (shadow pool). The database entry can be deleted to complete migration.
                        </MessageBar>
                        <DefaultButton text="Delete Database Entry" iconProps={{ iconName: "Delete" }} onClick={() => setState({ ...state, showDeleteShadow: true })} />
                        {!!state.showDeleteShadow && <Dialog
                           hidden={false}
                           onDismiss={() => setState({ ...state, showDeleteShadow: false, deleteError: undefined })}
                           minWidth={480}
                           dialogContentProps={{
                              type: DialogType.largeHeader,
                              title: `Delete Shadow Pool Entry`,
                              subText: `This will delete the database entry for pool "${pool.name}". The pool will continue to exist via config files.`
                           }}
                           modalProps={{ isBlocking: true }}>
                           {!!state.deleteError && <MessageBar messageBarType={MessageBarType.error}>{state.deleteError}</MessageBar>}
                           <DialogFooter>
                              <PrimaryButton onClick={onDeleteShadow} text={state.deletingShadow ? "Deleting..." : "Delete"} disabled={state.deletingShadow} />
                              <DefaultButton onClick={() => setState({ ...state, showDeleteShadow: false, deleteError: undefined })} text="Cancel" />
                           </DialogFooter>
                        </Dialog>}
                     </Stack>
                  )}
                  {pool.configManaged === true && pool.hasDatabaseEntry !== true && (
                     <MessageBar messageBarType={MessageBarType.info} isMultiline={false} styles={{ root: { marginBottom: 8 } }}>
                        This pool is managed via config files. To modify settings, edit pools.global.json in Perforce.
                     </MessageBar>
                  )}
                  <Stack>
                     <DetailsList
                        compact={true}
                        items={summaryItems}
                        columns={summaryColumns}
                        setKey="set"
                        layoutMode={DetailsListLayoutMode.justified}
                        isHeaderVisible={false}
                        selectionMode={SelectionMode.none}
                        onRenderItemColumn={onRenderSummaryItemColumn}
                     />
                  </Stack>
               </Stack>
               <Stack style={{ minWidth: 320 }}>
                  <Stack style={{ paddingBottom: 12 }}>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Workspaces</Text>
                  </Stack>
                  <DetailsList
                     compact={true}
                     items={workspaceItems}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />
               </Stack>
               <StreamPanel />
            </Stack>
         </Stack>
      </Stack>
   </Stack>)
};


export const PoolView: React.FC = observer(() => {

   const [searchParams] = useSearchParams();

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { hordeClasses } = getHordeStyling();

   // subscribe
   if (handler.updated) { };


   const poolId = searchParams.get("pool") ?? "";

   if (poolId) {
      handler.set(poolId);
   }

   const pool = handler.pool;

   return <Stack className={hordeClasses.raised} >
      {!!pool && <Stack style={{ width: "100%", height: "100%" }}>
         <div style={{ marginTop: 8, width: "100%", height: 'fit-content', paddingBottom: 24 }}>
            <Stack tokens={{ childrenGap: 18 }}>
               <PoolPanel />
               <PoolAgentPanel poolId={poolId} />
               <StepPanel stepState={StepState.Active} />
               <ConformPanel />
               <StepPanel stepState={StepState.Pending} />
               <BatchPanel />
            </Stack>
         </div>
      </Stack>}
      {!pool && !!poolId && <Stack horizontalAlign="center">
         <Spinner size={SpinnerSize.large} />
      </Stack>}
   </Stack>
});

