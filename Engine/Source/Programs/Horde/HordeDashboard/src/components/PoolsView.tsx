// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsHeader, DetailsList, FontIcon, IColumn, IDetailsHeaderStyles, IDetailsListProps, ITag, Icon, Pivot, PivotItem, SelectionMode, Spinner, SpinnerSize, Stack, Sticky, StickyPositionType, TagPicker, Text, TooltipHost, mergeStyleSets, mergeStyles } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import { useNavigate, useSearchParams } from "react-router-dom";
import { Sparklines, SparklinesLine } from "react-sparklines";
import backend from "../backend";
import { GetDashboardPoolCategoryResponse, GetPoolSummaryResponse } from "../backend/Api";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { PollBase } from "../backend/PollBase";
import { useWindowSize } from "../base/utilities/hooks";
import { getHordeStyling } from "../styles/Styles";
import { BreadcrumbItem, Breadcrumbs } from "./Breadcrumbs";
import { HistoryModal } from "./agents/HistoryModal";
import { PoolView } from "./PoolView";
import { TopNav } from "./TopNav";

class PoolsHandler extends PollBase {

   constructor(pollTime = 30000) {

      super(pollTime);

   }

   clear() {
      this.category = undefined;
      this.selectedAgentId = undefined;
      this.loaded = false;
      this.pools = [];
      this.poolLookup.clear();
      super.stop();
   }

   setSelectedAgentId(agentId?: string) {
      this.selectedAgentId = agentId;
      this.setUpdated();
   }

   setPoolCategory(category?: GetDashboardPoolCategoryResponse) {

      if (this.category === category) {
         return;
      }

      this.category = category;

      // toggle to cancel and immediate query
      this.stop();
      this.start();

      this.loaded = false;
      this.setUpdated();

   }

   setFilter(filter: string) {
      this.filter = filter;
      this.setUpdated();
   }

   async poll(): Promise<void> {

      try {

         this.pools = await backend.getPoolsV2({ condition: this.category?.condition, stats: true, numUtilizationSamples: 32, numAgents: 5 });
         this.pools.forEach(p => {
            this.poolLookup.set(p.id, p);
         })


         this.loaded = true;
         this.setUpdated();

      } catch (err) {

      }

   }

   filter = "";

   category?: GetDashboardPoolCategoryResponse;

   loaded = false;

   selectedAgentId?: string;

   pools: GetPoolSummaryResponse[] = [];
   poolLookup = new Map<string, GetPoolSummaryResponse>();
}

const handler = new PoolsHandler();

type StatusBarStack = {
   value: number,
   title?: string,
   color?: string,
   stripes?: boolean
}

const stripeStyles = mergeStyleSets({

   stripes: {
      backgroundImage: 'repeating-linear-gradient(-45deg, rgba(255, 255, 255, .2) 25%, transparent 25%, transparent 50%, rgba(255, 255, 255, .2) 50%, rgba(255, 255, 255, .2) 75%, transparent 75%, transparent)',
   }

});

const StatusBar = (stack: StatusBarStack[], width: number, height: number, basecolor?: string, style?: any): JSX.Element => {

   stack = stack.filter(s => s.value > 0);

   const mainTitle = stack.map((item) => {
      return item.title;
   }).join(' ');

   return (
      <div className={mergeStyles({ backgroundColor: basecolor, width: width, height: height, verticalAlign: 'middle', display: "flex" }, style)} title={mainTitle}>
         {stack.map((item) => <span key={item.title!}
            className={item.stripes ? stripeStyles.stripes : undefined}
            style={{
               width: `${Math.ceil(item.value)}%`, height: '100%',
               backgroundColor: item.color,
               display: 'block',
               cursor: 'inherit',
               backgroundSize: `${height * 2}px ${height * 2}px`
            }} />)}
      </div>
   );
}


const PoolList: React.FC = observer(() => {

   const [sortState, setSortState] = useState<{ sortBy?: string, sortDescend?: boolean }>({ sortBy: "Agents", sortDescend: true });

   const navigate = useNavigate();

   const { hordeClasses } = getHordeStyling();

   handler.subscribe();

   const statusColors = dashboard.getStatusColors();

   const columns: IColumn[] = [{
      key: 'column1',
      name: 'Name',
      isSorted: sortState.sortBy === "Name",
      isSortedDescending: sortState.sortDescend,
      minWidth: 208,
      maxWidth: 208,
      onRender: (pool: GetPoolSummaryResponse) => {
         const textColor = "white";
         const color = pool.colorValue;
         return <Stack horizontal verticalAlign="center" verticalFill tokens={{ childrenGap: 4 }}>
            <DefaultButton
               text={pool.name}
               primary
               onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(`?pool=${pool.id}`); handler.setUpdated() }}
               styles={{
                  root: {
                     height: '26px',
                     width: "max-content",
                     font: '8pt Horde Open Sans SemiBold !important',
                     flexShrink: '0 !important',
                     paddingLeft: 6,
                     paddingRight: 6,
                     border: '0px', backgroundColor: color, color: textColor
                  },
                  rootHovered: { border: '0px', backgroundColor: color, color: textColor, },
                  rootPressed: { border: '0px', backgroundColor: color, color: textColor, }
               }} />
            {pool.configManaged === true && pool.hasDatabaseEntry === true && (
               <TooltipHost content="Shadow pool - has database entry that can be cleaned up">
                  <Icon iconName="Warning" styles={{ root: { fontSize: 12, color: statusColors.get(StatusColor.Warnings) }}} />
               </TooltipHost>
            )}
            {pool.configManaged === true && pool.hasDatabaseEntry !== true && (
               <TooltipHost content="Managed via config files (read-only)">
                  <Icon iconName="Lock" styles={{ root: { fontSize: 12, color: statusColors.get(StatusColor.Skipped) }}} />
               </TooltipHost>
            )}
         </Stack>
      }
   },
   {
      key: 'column2',
      name: 'Agents',
      minWidth: 64,
      maxWidth: 64,
      isSorted: sortState.sortBy === "Agents",
      isSortedDescending: sortState.sortDescend,
      onRender: (pool: GetPoolSummaryResponse) => {

         return <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
            <Text>{pool.stats?.numAgents ?? ""}</Text>
         </Stack>;
      }
   },
   {
      key: 'column3',
      name: 'Status',
      minWidth: 192,
      maxWidth: 192,
      isSorted: sortState.sortBy === "Offline",
      isSortedDescending: sortState.sortDescend,
      onRender: (pool: GetPoolSummaryResponse) => {

         if (!pool?.stats?.numAgents) {
            return null;
         }

         const numBusy = pool.stats.numAgents - (pool.stats.numIdle + pool.stats.numDisabled + pool.stats.numOffline);

         const busyFactor = (numBusy) / pool.stats.numAgents;
         const idleFactor = (pool.stats.numIdle) / pool.stats.numAgents;
         const offlineFactor = pool.stats.numOffline / pool.stats.numAgents;
         const disabledFactor = pool.stats.numDisabled / pool.stats.numAgents;

         const stack: StatusBarStack[] = [
            {
               value: busyFactor * 100,
               title: `Busy: ${(numBusy)}`,
               color: statusColors.get(StatusColor.Running)!,
               stripes: true
            },
            {
               value: idleFactor * 100,
               title: `Idle: ${pool.stats.numIdle}`,
               color: statusColors.get(StatusColor.Success)!,
            },
            {
               value: disabledFactor * 100,
               title: `Disabled: ${pool.stats.numDisabled}`,
               color: statusColors.get(StatusColor.Warnings)!,
            },
            {
               value: offlineFactor * 100,
               title: `Offline: ${pool.stats.numOffline}`,
               color: statusColors.get(StatusColor.Skipped)!,
            }
         ]


         return <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
            {StatusBar(stack, 180, 10, "transparent", { margin: '3px !important' })}
         </Stack>;
      }
   },
   {
      key: 'column4',
      name: '',
      minWidth: 600,
      maxWidth: 600,
      isSorted: sortState.sortBy === "Disabled",
      isSortedDescending: sortState.sortDescend,
      onRender: (pool: GetPoolSummaryResponse) => {

         if (!pool.agents?.length) {
            return null;
         }

         const statusColors = dashboard.getStatusColors();

         let textCount = 0;
         let agents = pool.agents.filter(a => {

            textCount += a.agentId.length;
            if (textCount > 64) {
               return false;
            }

            return true;

         });

         const agentStacks = agents.map((a, index) => {

            let color = "#000000";
            if (a.disabled) {
               color = statusColors.get(StatusColor.Skipped)!;
            }
            else if (a.offline) {
               color = statusColors.get(StatusColor.Warnings)!;
            } else {
               color = a.idle ? statusColors.get(StatusColor.Success)! : statusColors.get(StatusColor.Running)!;
            }

            let text = a.agentId;

            if (index !== agents!.length - 1) {
               text += ","
            }

            return <Stack key={`agent_stack_${a.agentId}`} horizontal style={{ cursor: "pointer", }} onClick={() => { handler.setSelectedAgentId(a.agentId) }} verticalAlign="center" verticalFill>
               <Stack horizontal style={{}} verticalAlign="center" tokens={{ childrenGap: 2 }} verticalFill>
                  <FontIcon style={{ color: color, paddingTop: 1, fontSize: 13 }} iconName="Square" />
                  <Text variant="small">{text}</Text>
               </Stack>
               {(pool.stats!.numAgents > pool.agents!.length) && (index === agents.length - 1) && <Stack verticalFill verticalAlign="center" key={`agent_stack_more_${pool.id}`} onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); navigate(`?pool=${pool.id}`); handler.setUpdated() }}>
                  <Text>, ...</Text>
               </Stack>}
            </Stack>
         })

         return <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="center" verticalFill>
            {agentStacks}
         </Stack>

      }
   },
   {
      key: 'column5',
      name: 'Utilization',
      minWidth: 160,
      isSorted: sortState.sortBy === "Disabled",
      isSortedDescending: sortState.sortDescend,
      onRender: (pool: GetPoolSummaryResponse) => {

         if (!pool.utilization?.length) {
            return null;
         }

         return <Stack verticalAlign="center" verticalFill style={{ paddingRight: 12 }}>
            <Sparklines width={160} height={24} data={pool.utilization}>
               <SparklinesLine color={dashboard.darktheme ? "lightblue" : "blue"} />
            </Sparklines>
         </Stack>;
      }
   },];

   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      const customStyles: Partial<IDetailsHeaderStyles> = {root: {paddingTop: 0}};
      if (props) {
         return (
            <Sticky stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
               <DetailsHeader {...props} styles={customStyles} onColumnClick={(ev: React.MouseEvent<HTMLElement>, column: IColumn) => {
                  if (column.name === "Agents") {
                     setSortState({ sortBy: "Agents", sortDescend: sortState.sortBy === "Agents" && !sortState.sortDescend })
                  } else if (column.name === "Name") {
                     setSortState({ sortBy: "Name", sortDescend: sortState.sortBy === "Name" && !sortState.sortDescend })
                  }
               }} />
            </Sticky>
         );
      }
      return null;
   };


   const filter = handler.filter.toLowerCase();
   const filtered = handler.pools.filter(p => {

      // filter out anonymous pools from display
      if (!p.name) {
         return false;
      }

      if (!filter) {
         return true;
      }
      return p.name.toLowerCase().indexOf(filter) !== -1
   })

   let items = filtered;

   if (sortState.sortBy) {

      items = items.sort((a, b) => {

         if (sortState.sortBy === "Name" || a.stats?.numAgents === b.stats?.numAgents) {
            return a.name.localeCompare(b.name);
         }

         return (a.stats?.numAgents ?? 0) - (b.stats?.numAgents ?? 0);

      })

      if (sortState.sortDescend) {
         items = items.reverse();
      }

   }

   /*
   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      const customStyles: Partial<IDetailsRowStyles> = {};
      if (props) {
         if (props.itemIndex % 2 === 0) {
            // Every other row renders with a different background color
            customStyles.root = { backgroundColor: "#F3F2F1" };
         }

         return <DetailsRow {...props} styles={customStyles} />;
      }
      return null;
   };
   */

   return <Stack className={hordeClasses.raised} >
      <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
         <DetailsList
            compact
            selectionMode={SelectionMode.none}
            items={items}
            columns={columns}
            isHeaderVisible={true}
            onRenderDetailsHeader={onRenderDetailsHeader}
         />
      </Stack>
   </Stack>
})

const PoolPicker: React.FC = observer(() => {

   const [searchParams, setSearchParams] = useSearchParams();

   const poolId = searchParams.get("pool") ?? "";

   handler.subscribe();

   const poolTags: ITag[] = handler.pools
      .filter(pool => pool.name !== "")
      .sort((a, b) => a.name.localeCompare(b.name))
      .map(p => {
         return { key: p.id, name: p.name };
      });

   let selectedItems: ITag[] = [];
   if (poolId) {
      const d = poolTags.find(t => t.key === poolId);
      if (d) {
         selectedItems = [d];
      }
   }

   const listContainsTagList = (tag: ITag, tagList?: ITag[]) => {
      if (!tagList || !tagList.length || tagList.length === 0) {
         return false;
      }
      return tagList.some(compareTag => compareTag.key === tag.key);
   };

   const filterSuggestedTags = (filterText: string, tagList?: ITag[]): ITag[] => {
      handler.setFilter(filterText);
      return filterText
         ? poolTags.filter(
            tag => tag.name.toLowerCase().indexOf(filterText.toLowerCase()) !== -1 && !listContainsTagList(tag, tagList),
         )
         : poolTags;
   };

   const getTextFromItem = (item: ITag) => item.name;

   return <Stack horizontal style={{ paddingTop: 8, paddingBottom: 12 }}>
      <Stack grow />
      <Stack style={{ width: 320 }}>
         <TagPicker inputProps={{ placeholder: "Filter" }}
            selectedItems={selectedItems}
            onResolveSuggestions={filterSuggestedTags}
            getTextFromItem={getTextFromItem}
            onEmptyResolveSuggestions={(selected) => {
               return poolTags;
            }}

            onChange={(items) => {
               if (!items?.length) {
                  setSearchParams("");
                  handler.setFilter("");
               }
            }}

            onDismiss={() => false}

            onItemSelected={(item) => {

               if (!item?.key) {
                  return null;
               }

               handler.setFilter("");
               setSearchParams(`?pool=${item.key}`, { replace: true });

               return item;

            }}

            itemLimit={1} />
      </Stack>
   </Stack>
})

const PoolPivot: React.FC = () => {

   const { hordeClasses, modeColors } = getHordeStyling();

   const categories = dashboard.poolCategories;

   const pivotItems = categories.map(tab => {
      return <PivotItem headerText={tab.name} itemKey={tab.name} key={tab.name} style={{ color: modeColors.text }} />;
   });

   pivotItems.unshift(<PivotItem headerText="All" itemKey="all" key={"all"} style={{ color: modeColors.text }} />);

   return <Stack grow style={{paddingBottom: 12}}>
      <Pivot className={hordeClasses.pivot}
         overflowBehavior='menu'
         selectedKey={handler.category?.name ?? "all"}
         linkSize="normal"
         linkFormat="links"
         onLinkClick={(item) => {
            if (item) {

               if (item.props.itemKey === "all") {
                  handler.setPoolCategory(undefined);
                  return;
               }

               const cat = categories.find(c => c.name === item.props.itemKey);

               handler.setPoolCategory(cat);
            }
         }}>
         {pivotItems}
      </Pivot>
   </Stack>

}


export const PoolsView: React.FC = observer(() => {

   const [searchParams] = useSearchParams();

   const windowSize = useWindowSize();

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   handler.subscribe();

   const poolId = searchParams.get("pool") ?? "";

   const { hordeClasses, modeColors } = getHordeStyling();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;
   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   let crumbs: BreadcrumbItem[] = [{
      text: "Pools",
      link: poolId ? "/pools" : undefined
   }];

   if (poolId) {
      const pool = handler.pools.find(p => p.id === poolId);
      if (pool) {
         crumbs.push({
            text: pool.name
         })
      }
   }

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={crumbs} />
      {!!handler.selectedAgentId && <HistoryModal agentId={handler.selectedAgentId} onDismiss={() => handler.setSelectedAgentId(undefined)} />}
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               {<Stack style={{ paddingTop: "12px", paddingBottom: "4px" }}>
                  <Stack style={{ width: 1440, marginLeft: centerAlign }}>
                     <Stack horizontal style={{ width: "100%" }}>
                        {!poolId && <PoolPivot />}
                        <Stack grow />
                        <PoolPicker />
                     </Stack>
                  </Stack>
               </Stack>}
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        {!poolId && !handler.loaded && <Stack>
                           <Spinner size={SpinnerSize.large} />
                        </Stack>}
                        {!poolId && handler.loaded && <Stack>
                           <PoolList />
                        </Stack>}
                        {!!poolId && <Stack>
                           <PoolView />
                        </Stack>}
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
});