// Copyright Epic Games, Inc. All Rights Reserved.

import React, { useState } from 'react';
import { useNavigate } from 'react-router-dom';
import backend, { useBackend } from '../backend';
import { useQuery } from "horde/base/utilities/hooks";
import {errorDialogStore} from './error/ErrorStore';
import { GetJobsTabResponse, JobsTabData, TabType } from '../backend/Api';

function setError(message: string) {

   console.error(message);

   message = `${message}\n\nReferring URL: ${encodeURI(window.location.href)}`

   errorDialogStore.set({

      title: `Error handling Preflight`,
      message: message

   }, true);

}

// redirect from external source, where horde stream id, etc are not known by that application
export const PreflightRedirector: React.FC = () => {

   const [state, setState] = useState({ preflightQueried: false })
   const navigate = useNavigate();
   const query = useQuery();
   const { projectStore } = useBackend();

   const streamName = !query.get("stream") ? "" : query.get("stream")!;
   const change = !query.get("change") ? "" : query.get("change")!;

   // whether to autosubmit
   const autosubmit = !query.get("submit") ? "" : query.get("submit")!;

   // whether a template is specified
   const templateId = !query.get("templateId") ? "" : query.get("templateId")!;

   const parameters: string[] = [];

   query.raw.searchParams.forEach((value, key) => {
      if (key.startsWith("id")) {
         parameters.push(`${key}=${value}`);
      }
   })

   const version = "2";

   if (!change) {
      setError("No preflight change specified");
      return null;
   }

   const cl = parseInt(change);
   const isNumericChange = !isNaN(cl);

   if (!isNumericChange && !/^[a-f0-9]+$/i.test(change)) {
      setError(`Bad change in preflight ${change}`);
      return null;
   }

   if (!streamName) {
      setError("No stream in query");
      return null;
   }

   let stream = projectStore.streamByFullname(streamName);
   let resolvedAlias: string | undefined;

   if (!stream) {

      stream = projectStore.streamById(streamName?.replace("//", "").replaceAll("/", "-").toLowerCase());

      if (!stream) {
         stream = projectStore.streamByFullname(streamName + "-VS");
      }

      // Check if the stream name matches a stream alias
      if (!stream) {
         stream = projectStore.streamByAliasName(streamName);
         if (stream) {
            resolvedAlias = streamName;
         }
      }

   }

   if (!stream) {
      setError(`Unable to resolve stream with name ${streamName}, please verify that this isn't a virtual stream and that you have access.`);
      return null;
   }

   const project = stream?.project;

   if (!stream || !project) {
      setError("Bad stream or project id");
      return null;
   }

   let tab = "summary";
   stream.tabs.find(t => {
      if (t.type !== TabType.Jobs) {
         return false;
      }

      if (!!(t as GetJobsTabResponse).templates?.find(t => t === templateId)) {
         tab = t.title;
      }
   })

   if (!state.preflightQueried) {

      console.log(`Redirecting preflight: ${window.location.href}`);

      const buildRedirectUrl = (basePath: string) => {
         let url = `${basePath}?newbuild=true&shelvedchange=${encodeURIComponent(change)}&p4v=true`;

         if (autosubmit === "true") {
            url += "&autosubmit=true";
         }

         if (templateId) {
            url += `&templateId=${templateId}`;
         }

         if (parameters.length) {
            url += ("&" + parameters.join("&"));
         }

         if (version === "2") {
            url += "&newbuildversion=2"
         }

         if (resolvedAlias) {
            url += `&alias=${encodeURIComponent(resolvedAlias)}`;
         }

         return url;
      };

      // For numeric CLs, check if there's an existing job to redirect to
      const jobLookup = isNumericChange
         ? backend.getJobs({ filter: "id,streamId", count: 1, preflightChange: cl })
         : Promise.resolve([]);

      jobLookup.then(result => {

         if (result && result.length === 1) {

            if (stream.id === result[0].streamId) {
               navigate(buildRedirectUrl(`/job/${result[0].id}`) + "&allowtemplatechange=true", { replace: true });
               return;
            }
         }

         navigate(buildRedirectUrl(`/stream/${stream!.id}`) + `&tab=${tab}`, { replace: true });

      }).catch(reason => {
         console.error(`Error getting job for preflight: `, reason);
         navigate("/", { replace: true });
      })

      setState({ preflightQueried: true })

      return null;

   }

   return null;
}
