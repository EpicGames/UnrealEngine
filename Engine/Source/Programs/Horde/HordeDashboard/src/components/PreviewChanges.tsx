// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, IColumn, IconButton, Modal, PrimaryButton, SelectionMode, Stack, Text } from "@fluentui/react";
import { Link as ReactRouterLink } from 'react-router-dom';
import { getHordeStyling } from "../styles/Styles";


type ChangeItem = {
   //
   id: string;
   // A summary of the change
   summary: string;
   link?: string;
   slack?: string;
}

const changeItems: ChangeItem[] = [];

const columns = [
   { key: 'column1', name: 'ID', minWidth: 100, maxWidth: 100 },
   { key: 'column2', name: 'Summary', minWidth: 540, maxWidth: 540 },
   { key: 'column3', name: 'Preview', minWidth: 100, maxWidth: 100 },
   { key: 'column4', name: 'Discussion', minWidth: 100, maxWidth: 200 },

];

const onRenderItemColumn = (item: ChangeItem, index?: number, columnIn?: IColumn) => {

   const column = columnIn!;

   // simple cases
   switch (column.name) {
      case 'ID':
         return <Stack verticalFill verticalAlign="center"><Text styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{`PREVIEW-${item.id}`}</Text></Stack>
      case 'Summary':
         return <Text style={{ whiteSpace: "pre-wrap" }}>{item.summary}</Text>
      case 'Preview':
         return !item.link ? null : <Stack verticalFill verticalAlign="center"><ReactRouterLink to={item.link}><Text>Example Link</Text></ReactRouterLink></Stack>
      case 'Discussion':
         return !item.slack ? null : <Stack verticalFill verticalAlign="center"><a href={item.slack} target="_blank" rel="noreferrer"><Text >Thread</Text></a></Stack>
   }

   return null;
}


export const PreviewChangesModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const { hordeClasses } = getHordeStyling();

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1000, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
      <Stack style={{ paddingRight: 18 }}>
         <Stack style={{ paddingLeft: 18, paddingTop: 8 }}>
            <Stack horizontal>
               <Stack verticalAlign="center">
                  <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Preview Changes</Text>
               </Stack>
               <Stack grow />
               <Stack>
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     ariaLabel="Close popup modal"
                     onClick={() => { onClose() }}
                  />
               </Stack>
            </Stack>
         </Stack>
         <Stack style={{ paddingLeft: 24, paddingBottom: 24 }}>
            <DetailsList
               items={changeItems}
               columns={columns}
               layoutMode={DetailsListLayoutMode.justified}
               isHeaderVisible={true}
               selectionMode={SelectionMode.none}
               onRenderItemColumn={onRenderItemColumn}
            />
         </Stack>
         <Stack horizontal style={{ paddingBottom: 24 }}>
            <Stack grow />
            <Stack>
               <PrimaryButton text="Ok" onClick={() => onClose()} />
            </Stack>
         </Stack>
      </Stack>
   </Modal>
}