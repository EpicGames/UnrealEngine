// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { observer } from 'mobx-react-lite';
import { Link, useParams } from 'react-router-dom';
import { Stack, SearchBox, Text, FocusZoneDirection, ScrollbarVisibility, FocusZone, ScrollablePane } from '@fluentui/react';
import { Breadcrumbs, BreadcrumbItem } from './Breadcrumbs';
import { TopNav } from './TopNav';
import { action, makeObservable, observable } from 'mobx';
import backend, { useBackend } from '../backend';
import { ProjectData, StreamData } from '../backend/Api';
import dashboard from '../backend/Dashboard';
import { useWindowSize } from '../base/utilities/hooks';
import { getStreamLink } from '../base/utilities/streamUtils';
import { getHordeTheme } from '../styles/theme';
import { getHordeStyling } from '../styles/Styles';

type FilteredCategory = {
   key: string;
   name: string;
   streams: StreamData[];
};

class LocalState {
   constructor() {
      makeObservable(this);
   }
   filterString = "";
   activeProject: ProjectData | undefined = undefined;
   @observable.shallow filteredCategories: FilteredCategory[] = [];

   async setActiveProject(project?: ProjectData) {
      if (this.activeProject?.id !== project?.id) {
         backend.getProject(project!).then(projectWithMetadata => {
            this.activeProject = projectWithMetadata;
            this._filterStreams();
         });
      }
   }

   setFilterString(newFilter: string) {
      if (this.filterString.toLowerCase() !== newFilter.toLowerCase()) {
         this.filterString = newFilter;
         this._filterStreams();
      }
   }

   @action
   private _filterStreams() {
      if (this.activeProject?.streams) {
         this.filteredCategories = [];
         if (this.activeProject.categories) {
            this.activeProject.categories?.forEach(category => {
               let streams: StreamData[] = [];
               category.streams!.forEach(stream => {
                  let foundStream = this.activeProject!.streams!.find(s => s.id === stream && s.name.toLowerCase().indexOf(this.filterString.toLowerCase()) !== -1);
                  if (foundStream) {
                     streams.push(foundStream);
                  }
               });
               if (streams.length !== 0) {
                  this.filteredCategories.push({ key: category.name!, name: category.name!, streams: streams });
               }
               streams.sort((a, b) => a.name < b.name ? -1 : 1)
            });
         }
      }

   }
}


const localState = new LocalState();

export const ProjectHome: React.FC = observer(() => {
   const { projectId } = useParams<{ projectId: string }>();
   const { projectStore } = useBackend();

   const { hordeClasses, detailClasses } = getHordeStyling();
   const hordeTheme = getHordeTheme();

   const activeProject = projectStore.byId(projectId);

   if (!activeProject || activeProject.id !== projectId) {
      projectStore.setActive(projectId);
   }
   localState.setActiveProject(activeProject);

   const crumbs: BreadcrumbItem[] = [];
   if (activeProject) {
      crumbs.push({ text: activeProject.name });
   }

   // Check if there are any aliases to display
   const hasAliases = activeProject?.streams?.some(stream =>
      stream.aliases?.some(alias =>
         !localState.filterString || alias.name.toLowerCase().indexOf(localState.filterString.toLowerCase()) !== -1
      )
   );

   let categoryCount = localState.filteredCategories.length + (hasAliases ? 1 : 0);
   let width = categoryCount > 0 ? (100 / categoryCount) : 100;

   let categoryStackItems: any[] = [];

   for (let idx = 0; idx < localState.filteredCategories.length; idx++) {
      let category = localState.filteredCategories[idx];

      let links: any[] = [];
      category.streams.forEach(stream => {
         links.push(<Link style={{ marginLeft: 10, fontSize: 15 }} to={`/stream/${stream.id}`}>{stream.name}</Link>);
      })

      categoryStackItems.push(
         <Stack key={'stack_' + idx} styles={{ root: { width: `${width}%` } }} tokens={{ childrenGap: 5 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{category.name}</Text>
            {links}
         </Stack>
      );
   }

   // Collect all aliases from all streams
   const aliasLinks: any[] = [];
   activeProject?.streams?.forEach(stream => {
      stream.aliases?.forEach(alias => {
         if (!localState.filterString || alias.name.toLowerCase().indexOf(localState.filterString.toLowerCase()) !== -1) {
            aliasLinks.push(
               <Link key={`${stream.id}_${alias.name}`} style={{ marginLeft: 10, fontSize: 15 }} to={getStreamLink(stream.id, alias.name)}>
                  {alias.name}
               </Link>
            );
         }
      });
   });

   if (aliasLinks.length > 0) {
      categoryStackItems.push(
         <Stack key={'stack_aliases'} styles={{ root: { width: `${width}%` } }} tokens={{ childrenGap: 5 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Aliases</Text>
            {aliasLinks}
         </Stack>
      );
   }

   let logoSrc = "";
   if (activeProject) {
      logoSrc = `/api/v1/projects/${activeProject.id}/logo${dashboard.darktheme ? "?darkTheme=true" : ""}`;
   }

   const windowSize = useWindowSize();

   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <Breadcrumbs items={crumbs} />
         <Stack horizontal style={{ backgroundColor: hordeTheme.horde.neutralBackground }}>
            <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0 }} />
            <Stack tokens={{ childrenGap: 0 }} styles={{ root: { width: "100%" } }}>
               <Stack style={{ maxWidth: 1440, paddingTop: 6, marginLeft: 4 }}>
                  <FocusZone direction={FocusZoneDirection.vertical} style={{ padding: 0 }}>
                     <div className={detailClasses.container} style={{ width: "100%", height: 'calc(100vh - 208px)', position: 'relative' }} data-is-scrollable={true}>
                        <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto} onScroll={() => { }}>
                           <Stack tokens={{ childrenGap: 18 }} style={{ padding: 0 }}>
                              <Stack.Item align={'center'} className={hordeClasses.projectLogoCard}>
                                 { !!logoSrc && <img src={logoSrc} alt="Project logo" height={300} /> }
                              </Stack.Item>
                              <Stack tokens={{ childrenGap: 20 }} className={hordeClasses.raised}>
                                 <SearchBox
                                    placeholder="Search"
                                    onChange={(event?: React.ChangeEvent<HTMLInputElement> | undefined, newValue?: string | undefined) => { localState.setFilterString(newValue ?? ""); }}
                                    onClear={() => { localState.setFilterString(""); }}
                                 />
                                 <Stack>
                                    <Stack horizontal>
                                       {categoryStackItems}
                                    </Stack>
                                 </Stack>
                              </Stack>
                           </Stack>
                        </ScrollablePane>
                     </div>
                  </FocusZone>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   );
});