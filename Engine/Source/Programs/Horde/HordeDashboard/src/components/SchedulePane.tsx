// Copyright Epic Games, Inc. All Rights Reserved.
import { IIconProps, mergeStyleSets, SearchBox, Stack } from '@fluentui/react';
import { DetailsList, DetailsListLayoutMode, DetailsRow, IColumn, IDetailsListProps, SelectionMode } from '@fluentui/react/lib/DetailsList';
import moment from 'moment-timezone';
import React, { useState } from 'react';
import { Link } from 'react-router-dom';
import backend from '../backend';
import { GetJobResponse, GetTemplateRefResponse, JobState } from '../backend/Api';
import dashboard from '../backend/Dashboard';
import { displayTimeZone } from "../base/utilities/timeUtils";

const classes = mergeStyleSets({
   detailsRow: {
      selectors: {
         '.ms-DetailsRow-cell': {
            overflow: "hidden",
            whiteSpace: "nowrap",
            padding: 8
         }
      }
   }
});

type ScheduleItem = {
   template: GetTemplateRefResponse
};

const scheduledJobs = new Map<string, GetJobResponse | null>();

export const SchedulePane: React.FC<{ streamId: string, templates: GetTemplateRefResponse[] }> = ({ streamId, templates }) => {

   let [updated, setUpdated] = useState(new Date().getTime());
   const [searchText, setSearchText] = useState("");

   const ids = new Set<string>();

   if (new Date().getTime() - updated > 10000) {

      scheduledJobs.forEach((job, key) => {
         if (job?.state !== JobState.Complete) {
            ids.add(key);
         }
      });

   }

   templates.forEach(t => {
      if (t.schedule && t.schedule.enabled && t.schedule.maxActive === 1 && t.schedule.activeJobs.length) {
         const jid = t.schedule.activeJobs[0];
         if (!scheduledJobs.has(jid)) {
            scheduledJobs.set(jid, null);
            ids.add(jid);
         }
      }
   })

   if (ids.size) {
      backend.getJobsByIds(Array.from(ids.values()), { filter: "id,createTime,state" }).then(response => {
         response.forEach(j => {
            scheduledJobs.set(j.id, j);
         });

         setUpdated(new Date().getTime());
      });
   }

   const items = templates.filter(t => !!t.schedule?.enabled).map(t => {
      return { template: t };
   }).filter(item => {
      if (!searchText) return true;

      const lowerSearch = searchText.toLowerCase();
      const template = item.template;
      const schedule = template.schedule!;

      if (template.name.toLowerCase().includes(lowerSearch)) return true;

      const gateTemplateId = schedule.gate?.templateId;
      if (gateTemplateId) {
         const gateTemplate = templates.find(t => t.id === gateTemplateId);
         if (gateTemplate?.name.toLowerCase().includes(lowerSearch)) return true;
      }

      const target = schedule.gate?.target;
      if (target?.toLowerCase().includes(lowerSearch)) return true;

      return false;
   });

   const columns = [
      { key: 'schedule_column1', name: 'job_name', minWidth: 320, maxWidth: 320, isResizable: false },
      { key: 'schedule_column3', name: 'time_0', minWidth: 140, maxWidth: 140, isResizable: false },
      { key: 'schedule_column4', name: 'time_1', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column5', name: 'time_2', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column6', name: 'time_3', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column7', name: 'time_4', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column8', name: 'time_5', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column9', name: 'time_6', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'schedule_column10', name: 'audit', minWidth: 56, maxWidth: 56, isResizable: false },
      { key: 'schedule_column11', name: 'space', minWidth: 2, isResizable: false }

   ];

   const renderItem = (item: ScheduleItem, index?: number, column?: IColumn) => {

      const name = column!.name;
      const template = item.template;
      const schedule = template.schedule!;

      if (name === "job_name") {

         let name = item.template.name;
         let target = schedule.gate?.target;
         let templateId = schedule.gate?.templateId;
         let gateTemplate = templates.find(t => t.id === templateId);


         return <Stack horizontalAlign="start" verticalFill verticalAlign="center" tokens={{ childrenGap: 4 }} style={{ paddingLeft: 12 }}>
            <Stack style={{ fontWeight: 600 }}>{name}</Stack>
            {!!gateTemplate && <Stack tokens={{ childrenGap: 4 }}>
               <Stack horizontal tokens={{ childrenGap: 8 }}>
                  <Stack style={{ fontWeight: 600, fontSize: 10 }}>Gate:</Stack>
                  <Stack style={{ fontSize: 10, paddingLeft: 8 }}>{gateTemplate?.name}</Stack>
               </Stack>
               <Stack horizontal tokens={{ childrenGap: 8 }}>
                  <Stack style={{ fontWeight: 600, fontSize: 10 }}>Target:</Stack>
                  <Stack style={{ fontSize: 10 }}>{target}</Stack>
               </Stack>
            </Stack>
            }
         </Stack>;
      }

      if (name === "audit") {
         return <Link to={`/audit/template/${encodeURIComponent(streamId)}/${encodeURIComponent(template.id)}`}>
            <Stack horizontalAlign="end" verticalFill verticalAlign="center" tokens={{ childrenGap: 4 }}>
               <Stack tokens={{ childrenGap: 4 }}>
                  <Stack >View Log</Stack>
               </Stack>
            </Stack>
         </Link>
      }

      if (name.startsWith("time_")) {

         const index = parseInt(name.slice(5));

         const triggerTimes = schedule.nextTriggerTimesUTC.slice(0, 5);

         let times = triggerTimes.map(t => moment(t));
         times.unshift(moment(schedule.lastTriggerTime));

         times = times.sort((a, b) => {
            return a.unix() - b.unix();
         })

         if (index >= times.length) {
            return null;
         }

         const displayTime = times[index].tz(displayTimeZone());
         const now = moment.utc().tz(displayTimeZone());
         let timeStr = dashboard.display24HourClock ? displayTime.format('H:mm z') : displayTime.format('h:mma z');
         if (!(now.isSame(displayTime, 'day'))) {
            timeStr += displayTime.format(' (ddd)');
         }

         if (index === 0) {

            if (schedule?.activeJobs.length) {
               const job = scheduledJobs.get(schedule.activeJobs[0]);
               if (job) {
                  const time = moment(job.createTime).tz(displayTimeZone());
                  const timeStr = dashboard.display24HourClock ? time.format("MMM Do H:mm z") : time.format("MMM Do h:mma z");

                  let state = "Complete";

                  if (job.state === JobState.Waiting || job.state === JobState.Running) {
                     state = "Running";
                  }

                  return <Link to={`/job/${job.id}`}>
                     <Stack horizontalAlign="center" tokens={{ childrenGap: 4 }}>
                        <Stack >{timeStr}</Stack>
                        <Stack >{state}</Stack>
                     </Stack>
                  </Link>;
               }
            }
         }

         return <Stack horizontalAlign="center" verticalFill verticalAlign="center">{timeStr}</Stack>;
      }


      return null;

   };

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const nprops = { ...props, styles: { root: { backgroundColor: "unset", padding: 8 } } };

         let background: string | undefined;
         if (props.itemIndex % 2 === 0) {
            background = dashboard.darktheme ? "#1D2021" : "#FAF9F9";
         }

         return <DetailsRow {...nprops} styles={{ root: { background: background } }} />

      }
      return null;
   };

   
   const filterIcon: IIconProps = { iconName: 'Filter' };

   return <Stack>
      <Stack horizontalAlign="end">
         <SearchBox
            iconProps={filterIcon}
            placeholder="Filter schedules..."
            value={searchText}
            onChange={(_, newValue) => setSearchText(newValue || "")}
            onClear={() => setSearchText("")}
            styles={{ root: { marginBottom: 8, width: 400 } }}
         />
      </Stack>
      <div className={classes.detailsRow} style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "820px" }} data-is-scrollable={true}>

         <DetailsList
            styles={{ root: { overflow: "unset" } }}
            compact={true}
            isHeaderVisible={false}
            indentWidth={0}
            items={items}
            columns={columns}
            setKey="set"
            selectionMode={SelectionMode.none}
            layoutMode={DetailsListLayoutMode.justified}
            onRenderItemColumn={renderItem}
            onRenderRow={renderRow}
         />
      </div>
   </Stack>


}
