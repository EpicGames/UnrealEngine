

// Copyright Epic Games, Inc. All Rights Reserved.

import { FontIcon, Stack } from '@fluentui/react';
import React, { CSSProperties } from 'react';
import { DeviceStatus, FindIssueResponse, GetAgentLeaseResponse, GetBatchResponse, GetIssueResponse, GetJobStepRefResponse, GetLabelStateResponse, GetStepResponse, IssueSeverity, JobStepBatchError, JobStepOutcome, JobStepState, LabelOutcome, LabelState, LeaseOutcome, LeaseState } from '../backend/Api';
import dashboard, { StatusColor } from "../backend/Dashboard";
import { getStepStatusColor } from '../styles/colors';

const StatusIcon: React.FC<{ iconName: string, style?: CSSProperties }> = ({ iconName, style }) => {

   return <Stack className="horde-no-darktheme"><FontIcon style={{ ...style }} iconName={iconName} /></Stack>
}

const getLabelIcon = (state: LabelState | undefined, outcome: LabelOutcome | undefined): { icon: string, color: string } => {

   const colors = dashboard.getStatusColors();

   let color: string = "#000000";
   const icon = "Square";

   if (state === LabelState.Running) {
      color = colors.get(StatusColor.Running)!;
   }


   if (outcome === LabelOutcome.Warnings) {
      //icon = "Warning";
      color = colors.get(StatusColor.Warnings)!;
   }

   if (outcome === LabelOutcome.Failure) {
      //icon = "FAWindowCloseSolid";
      color = colors.get(StatusColor.Failure)!;
   }

   if (state === LabelState.Complete) {
      if (outcome === LabelOutcome.Success) {
         //icon = "TickCircle";
         color = colors.get(StatusColor.Success)!;
      }
   }

   if (state === LabelState.Running) {
      //icon = "FullCircle";
   }

   return {
      icon: icon,
      color: color
   }

}


export const LabelStatusIcon: React.FC<{ label: GetLabelStateResponse, style?: CSSProperties }> = ({ label, style }) => {

   style = style ?? {};
   style.fontSize = style.fontSize ?? 13;
   style.paddingTop = style.paddingTop ?? 3;
   style.paddingRight = style.paddingRight ?? 8;

   const icon = getLabelIcon(label.state, label.outcome);

   return <StatusIcon iconName={icon.icon} style={{ ...style, color: icon.color }} />;
}

export const IssueStatusIcon: React.FC<{ issue: GetIssueResponse, style?: CSSProperties }> = ({ issue, style }) => {

   const colors = dashboard.getStatusColors();

   style = style ?? {};
   style.fontSize = style.fontSize ?? 13;
   style.paddingTop = style.paddingTop ?? 4;
   style.paddingRight = style.paddingRight ?? 8;

   let icon = "Square";//issue.severity === IssueSeverity.Warning ? "Warning" : "FAWindowCloseSolid";
   style.color = issue.severity === IssueSeverity.Warning ? colors.get(StatusColor.Warnings) : colors.get(StatusColor.Failure);

   /*
   if (streamId) {
      const affectedStream = issue.affectedStreams.find(s => s.streamId === streamId);
      if (affectedStream) {

         let severity = IssueSeverity.Unspecified;
         affectedStream.affectedTemplates.forEach(t => {
            if (t.severity === IssueSeverity.Error) {
               severity = IssueSeverity.Error;
            } else if (severity !== IssueSeverity.Error && t.severity === IssueSeverity.Warning) {
               severity = IssueSeverity.Warning;
            }
         });

         if (severity !== IssueSeverity.Unspecified) {
            style.color = severity === IssueSeverity.Warning ? colors.get(StatusColor.Warnings) : colors.get(StatusColor.Failure);
         }
      }
   }
   */

   return <StatusIcon iconName={icon} style={style} />
}

export const IssueStatusIconV2: React.FC<{ issue: FindIssueResponse, style?: CSSProperties }> = ({ issue, style }) => {

   const colors = dashboard.getStatusColors();

   style = style ?? {};
   style.fontSize = style.fontSize ?? 13;
   style.paddingTop = style.paddingTop ?? 4;
   style.paddingRight = style.paddingRight ?? 8;

   let icon = "Square";//issue.severity === IssueSeverity.Warning ? "Warning" : "FAWindowCloseSolid";
   style.color = issue.severity === IssueSeverity.Warning ? colors.get(StatusColor.Warnings) : colors.get(StatusColor.Failure);

   return <StatusIcon iconName={icon} style={style} />
}


const StepStateStatusIcon: React.FC<{ state: JobStepState, outcome: JobStepOutcome, style?: CSSProperties }> = ({ state, outcome, style }) => {

   // defaults
   style = style ?? {};
   style.fontSize = style.fontSize ?? 13;
   style.paddingTop = style.paddingTop ?? 3;
   style.paddingRight = style.paddingRight ?? 8;

   const icon = "Square";
   style.color = getStepStatusColor(state, outcome);

   return <StatusIcon iconName={icon} style={style} />
}



export const StepRefStatusIcon: React.FC<{ stepRef: GetJobStepRefResponse, style?: CSSProperties }> = ({ stepRef, style }) => {

   return <StepStateStatusIcon state={stepRef.finishTime ? JobStepState.Completed : JobStepState.Running} outcome={stepRef.outcome!} style={style} />;
}

export const StepStatusIcon: React.FC<{ step: GetStepResponse, style?: CSSProperties }> = ({ step, style }) => {

   return <StepStateStatusIcon state={step.state} outcome={step.outcome} style={style} />;
}

export const DeviceStatusIcon: React.FC<{ status: DeviceStatus, style?: CSSProperties }> = ({status, style}) => {

   const colors = dashboard.getStatusColors();

   style = style ?? {};
   style.fontSize = style.fontSize ?? 13;
   style.paddingTop = style.paddingTop ?? 3;
   style.paddingRight = style.paddingRight ?? 8;

   const icon = 'Square';

   if (status === DeviceStatus.Normal) {
      style.color = colors.get(StatusColor.Success);
   } else {
      style.color = colors.get(StatusColor.Failure);
   }

   return <StatusIcon iconName={icon} style={style} />
}

export const LeaseStatusIcon: React.FC<{ lease: GetAgentLeaseResponse, style?: CSSProperties }> = ({ lease, style }) => {

   const colors = dashboard.getStatusColors();

   const outcome = lease.outcome;

   // defaults
   style = style ?? {};
   style.fontSize = style.fontSize ?? 12;
   style.paddingTop = style.paddingTop ?? 0;
   style.paddingRight = style.paddingRight ?? 8;

   const icon = "Square";
   let color: string | undefined;

   if (lease.executing || (lease.name === "Updating workspaces" && (lease.state === LeaseState.Active || lease.state === LeaseState.Pending))) {
      color = colors.get(StatusColor.Running);
   }

   if (!color && !lease.finishTime) {
      color = colors.get(StatusColor.Waiting);
   }

   if (!color && outcome) {

      if (outcome === LeaseOutcome.Cancelled) {
         color = colors.get(StatusColor.Aborted);
      }
      if (outcome === LeaseOutcome.Unspecified) {
         color = colors.get(StatusColor.Unspecified);
      }
      if (outcome === LeaseOutcome.Success) {
         color = colors.get(StatusColor.Success);
      }
      if (outcome === LeaseOutcome.Failed) {
         color = colors.get(StatusColor.Failure);
      }

   }

   style.color = color;

   return <StatusIcon iconName={icon} style={style} />

}

export const BatchStatusIcon: React.FC<{ batch: GetBatchResponse, style?: CSSProperties }> = ({ batch, style }) => {

   const colors = dashboard.getStatusColors();

   // defaults
   style = style ?? {};
   style.fontSize = style.fontSize ?? 12;
   style.paddingTop = style.paddingTop ?? 0;
   style.paddingRight = style.paddingRight ?? 8;

   const icon = "Square";
   let color: string | undefined;

   if (batch.error !== JobStepBatchError.None) {
      color = colors.get(StatusColor.Failure);

      if (batch.error === JobStepBatchError.Cancelled) {
         color = colors.get(StatusColor.Skipped);
      }

   }

   if (!color && !batch.finishTime) {
      color = colors.get(StatusColor.Running);
   }

   if (!color) {
      color = colors.get(StatusColor.Success);
   }

   style.color = color;

   return <StatusIcon iconName={icon} style={style} />

}
