// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, IconButton, MessageBar, MessageBarType, Modal, PrimaryButton, Stack, Text } from '@fluentui/react';
import React, { useState } from 'react';
import backend from '../backend';
import { GetTemplateRefResponse } from '../backend/Api';
import dashboard from '../backend/Dashboard';
import { projectStore } from '../backend/ProjectStore';
import { getShortNiceTime } from '../base/utilities/timeUtils';
import { getHordeStyling } from '../styles/Styles';

export const PauseStepModal: React.FC<{ streamId: string, templateName: string, stepName: string, onClose: () => void }> = ({ streamId, stepName, templateName, onClose }) => {

   const [state, setState] = useState<{ ref?: GetTemplateRefResponse, submitting?: boolean, pause?: boolean, error?: string }>({});

   const { hordeClasses } = getHordeStyling();

   const stream = projectStore.streamById(streamId);
   if (!stream) {
      console.error(`Unable to find stream ${streamId}`);
      onClose();
      return null;
   }

   if (!state.ref) {

      const project = projectStore.projectByStreamId(streamId);
      const stream = project?.streams?.find(s => s.id === streamId);
      const template = stream?.templates?.find(t => t.name === templateName);

      if (!template) {
         console.error(`Unable to find stream ${streamId} template ref for name ${templateName}`);
         onClose();
         return null;
      }

      const stepState = template.stepStates?.find(s => s.name === stepName);

      setState({ ...state, ref: template, pause: !!stepState?.pausedByUserInfo });

      return null;
   }

   const ref = state.ref;

   const onSave = async () => {

      setState({ ...state, submitting: true });

      try {

         await backend.updateTemplateRef(streamId, ref.id, {
            stepStates: [{
               name: stepName,
               pausedByUserId: state.pause ? dashboard.userId : undefined               
            }]
         });

         setState({ ...state, submitting: false, error: "" });

         projectStore.update().finally(() => onClose());                     

      } catch (reason) {

         setState({ ...state, submitting: false, error: reason as string });
      }

   };

   const stepState = ref.stepStates?.find(s => s.name === stepName);

   const height = state.error ? 230 : 210;

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { onClose() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">Edit Step State</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { paddingLeft: 8, width: 540, paddingTop: 12 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            {!!state.error && <MessageBar
               messageBarType={MessageBarType.error}
               isMultiline={false}> {state.error} </MessageBar>}

            <Stack>
               <Checkbox label={!stepState?.pausedByUserInfo ? "Pause Step" : `Paused by ${stepState?.pausedByUserInfo?.name} on ${getShortNiceTime(stepState?.pauseTimeUtc)}`}
                  defaultChecked={!!state.pause}
                  onChange={(ev, newValue) => {
                     setState({ ...state, pause: !!newValue });
                  }} />               
            </Stack>
         </Stack>
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack>
            </Stack>
            <Stack grow />
            <Stack>
               <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <PrimaryButton text="Save" disabled={state.submitting ?? false} onClick={() => { onSave() }} />
                  <DefaultButton text="Cancel" disabled={state.submitting} onClick={() => { onClose(); }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;
  
};
