// Copyright Epic Games, Inc. All Rights Reserved.

import { Callout, DefaultButton, DetailsList, DetailsListLayoutMode, DetailsRow, Dropdown, IColumn, IconButton, IDetailsGroupRenderProps, IDetailsListProps, IDropdownOption, mergeStyleSets, Pivot, PivotItem, ScrollablePane, ScrollbarVisibility, SelectionMode, Slider, Spinner, SpinnerSize, Stack, Text, Toggle, TooltipHost } from '@fluentui/react';
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useMemo, useState } from 'react';
import { Link, useLocation, useNavigate, useParams } from 'react-router-dom';
import backend, { useBackend } from '../backend';
import { FindIssueResponse, GetExternalIssueResponse, JobsTabData, TabType } from '../backend/Api';
import dashboard from '../backend/Dashboard';
import { ProjectStore } from '../backend/ProjectStore';
import { getElapsedString, getShortNiceTime } from '../base/utilities/timeUtils';
import { IssueModalV2 } from './IssueViewV2';
import { useQuery } from "horde/base/utilities/hooks";
import { SchedulePane } from './SchedulePane';
import { IssueStatusIconV2 } from './StatusIcon';
import { getHordeStyling } from '../styles/Styles';


class SummaryHandler {

   constructor() {
      makeObservable(this);
   }

   set(streamId: string) {

      if (this.streamId === streamId) {
         return;
      }

      this.streamId = streamId;
      this.poll();

   }

   clear() {

      for (let i = 0; i < this.cancelID; i++) {
         this.canceled.add(i);
      }

      clearTimeout(this.timeoutID);

      this.updating = false;
      this.streamId = undefined;
      this.issues = [];
      this.unpromoted = [];

      this.collapsedIssueGroups = new Map();

      this.initialLoad = true;
   }

   private async poll() {

      try {

         const streamId = this.streamId;

         if (!streamId || this.updating) {
            return;
         }

         clearTimeout(this.timeoutID);
         if (this.updateMS) {
            this.timeoutID = setTimeout(() => { this.poll(); }, this.updateMS);
         }

         this.updating = true;
         const cancelID = this.cancelID++;

         const values = await Promise.all([backend.getIssuesV2({ streamId: streamId, count: 512, resolved: false })]);

         // early out if has been canceled 
         if (this.canceled.has(cancelID)) {
            return;
         }

         let jiraKeys: string[] = [];

         const jiraIssues: Set<number> = new Set();
         values[0].forEach(issue => {
            if (issue.externalIssueKey) {
               jiraIssues.add(issue.id)
               jiraKeys.push(issue.externalIssueKey);
            }
         });

         const now = moment(Date.now());
         jiraKeys = jiraKeys.filter(j => {
            const issue = this.jiraIssues.get(j);
            if (!issue) {
               return true;
            }
            if (moment.duration(now.diff(issue.cacheTime)).asMinutes() <= 2) {
               return false;
            }
            return true;
         });

         if (jiraKeys.length) {
            const jiras = await backend.getExternalIssues(streamId, jiraKeys);
            jiras.forEach(j => {
               this.jiraIssues.set(j.key, { issue: j, cacheTime: moment(Date.now()) });
            })
         }

         // early out if has been canceled 
         if (this.canceled.has(cancelID)) {
            return;
         }

         this.setIssues(values, jiraIssues);

         this.initialLoad = false;
         this.updated();

      } catch (reason) {
         console.error(reason);
      } finally {
         this.updating = false;
      }

   }

   exportIssues(projectStore: ProjectStore, items: HealthItem[], groups: IssueGroup[], promoted: boolean) {

      const stream = projectStore.streamById(this.streamId);

      if (!stream || !stream.project) {
         return;
      }

      const timeStr = moment().format('YYYY-MM-DD');

      const name = `++${stream.project.name}+${stream.name}-BuildHealth-${promoted ? "Promoted" : "Current"}-${timeStr}.html`;

      // build issue table
      let html = `<!DOCTYPE html>
        <html>
        <head>
        <style>
		body {
			max-width: 1480px;
			margin: auto;
		}
        table {
          font-family: arial, sans-serif;
          font-size: 12px;
          border-collapse: collapse;
          width: 100%;
        }
        
        td, th {
          border: 1px solid #dddddd;
          text-align: center;
          padding: 8px;
        }
        
        tr:nth-child(odd) {
          background-color: #dedddc;
        }
        </style>
        </head>        
        <body>`;

      html += `<h2>Build Health - ${stream.fullname} - ${promoted ? "Promoted" : "Current"} - ${timeStr}</h2>`;

      html += `<table>`;

      html += `<tr>
        <td>Category</td>
        <td>Issue</td>
        <td>Summary</td>
        <td>Status</td>
        <td>Workflow</td>
        <td>Open Since</td>
        <td>Promoted</td>
        <td>Jira</td>
      </tr>`

      groups.forEach(g => {


         for (let i = g.startIndex; i < g.startIndex + g.count; i++) {

            const item = items[i];

            const issue = item.issue;

            let status = "Unassigned";
            if (issue.owner?.name) {
               status = issue.owner?.name;
            }

            if (issue.resolvedAt) {
               status = "Resolved";
               if (issue.fixChange) {
                  status += ` in CL ${issue.fixChange}`;
               }
            }

            let summary = issue.summary;
            if (summary.length > 100) {
               summary = summary.slice(0, 100) + "...";
            }

            const openSince = `${getShortNiceTime(issue.createdAt)} (${getElapsedString(moment(issue.createdAt), moment.utc(), false).trim()})`;

            const href = `${window.location.protocol}//${window.location.hostname}${window.location.pathname}?tab=summary&issue=${issue.id}`;

            let jiraTag = "";

            if (issue.externalIssueKey) {
               const url = `${dashboard.externalIssueService?.url}/browse/${issue.externalIssueKey}`;
               jiraTag = `<a href="${url}" target="_blank">${issue.externalIssueKey}</a>`;
            }

            let workflowTag = "";
            if (issue.workflowThreadUrl) {
               workflowTag = `<a href="${issue.workflowThreadUrl}" target="_blank">Slack Thread</a>`;
            }

            html += `<tr>
                <td>${g.name}</td>
                <td><a href="${href}" target="_blank">${issue.id}</a></td>
                <td style="text-align:left"><a href="${href}" target="_blank">${summary}</a></td>
                <td>${status}</td>
                <td>${workflowTag}</td>

                <td>${openSince}</td>
                <td>${issue.promoted ? "Yes" : "No"}</td>
                <td>${jiraTag}</td>
              </tr>`;

         }

      });

      html += `</table>`;

      // close html
      html += `</body>
        </html>`;

      const element = document.createElement('a');
      element.setAttribute('href', 'data:text/plain;charset=utf-8,' + encodeURIComponent(html));
      element.setAttribute('download', name);

      element.style.display = 'none';
      document.body.appendChild(element);

      element.click();

      document.body.removeChild(element);
   }

   @observable
   update = 0


   @action
   private updated() {
      this.update++;
   }

   initialLoad = true;

   @observable
   issues: FindIssueResponse[] = [];
   @observable
   unpromoted: FindIssueResponse[] = [];

   @action
   private setIssues(values: [FindIssueResponse[]], jiraIssues: Set<number>) {
      this.issues = values[0].filter(p => p.promoted || jiraIssues.has(p.id));
      this.unpromoted = values[0].filter(p => !p.promoted && !jiraIssues.has(p.id));
   }

   collapsedIssueGroups: Map<string, boolean> = new Map();

   streamId?: string;

   private updating = false;

   private updateMS = 30000;
   private timeoutID?: any;
   private canceled = new Set<number>();
   private cancelID = 0;

   jiraIssues: Map<string, { issue: GetExternalIssueResponse, cacheTime: moment.Moment }> = new Map();

}


const handler = new SummaryHandler();

type HealthItem = {
   issue: FindIssueResponse;
};

type IssueGroup = {
   count: number;
   key: string;
   name: string;
   headerText: string;
   startIndex: number;
   level?: number;
   isCollapsed?: boolean;
}

const sortIssueGroups = (itemsIn: HealthItem[], projectStore: ProjectStore): [HealthItem[], IssueGroup[]] => {

   const stream = projectStore.streamById(handler.streamId)!;

   const foundIssues = new Set<number>();

   const tabIssues = new Map<string, HealthItem[]>();

   const groups: IssueGroup[] = [];

   let curIndex = 0;

   stream?.tabs?.forEach(tab => {

      if (tab.type !== TabType.Jobs) {
         return;
      }

      const jobTab = tab as JobsTabData;

      itemsIn.forEach(item => {

         const issue = item.issue;

         const templates = new Set(issue.spans.map(s => s.templateId));

         if (!(jobTab).templates?.find((tid) => templates.has(tid))) {
            return;
         }


         // only show on first tab
         if (foundIssues.has(issue.id)) {
            return;
         }

         foundIssues.add(issue.id);

         if (!tabIssues.has(tab.title)) {

            const key = `issue_group_${tab.title}`;

            groups.push({
               key: key,
               name: tab.title,
               count: 1,
               headerText: tab.title,
               startIndex: curIndex,
               isCollapsed: handler.collapsedIssueGroups.get(key)
            });

            tabIssues.set(tab.title, []);
         }

         tabIssues.get(tab.title)!.push({
            issue: issue
         })

         curIndex++;
      })

   })

   const name = "Uncategorized";
   const uncategorized = itemsIn.filter(item => !foundIssues.has(item.issue.id));
   if (uncategorized.length) {
      const key = `issue_group_${name}`;
      groups.push({
         key: key,
         name: name,
         count: 1,
         headerText: name,
         startIndex: curIndex,
         isCollapsed: handler.collapsedIssueGroups.get(key)
      });

      tabIssues.set(name, []);

      uncategorized.forEach(item => {

         tabIssues.get(name)!.push({
            issue: item.issue
         })
         curIndex++;
      })
   }

   const allIssues: HealthItem[] = [];
   tabIssues.forEach((items, name) => {

      const group = groups.find(g => g.name === name)!
      group.count = items.length;

      allIssues.push(...items.sort((a, b) => b.issue.id - a.issue.id));

   });

   return [allIssues, groups];

}

const HealthPanelIssues: React.FC<{ desktopAlerts?: boolean }> = observer(({ desktopAlerts }) => {

   type FilterState = {
      visible: boolean,
      unassigned?: boolean,
      timeLower?: number,
      timeUpper?: number,
      timeUnit?: IDropdownOption
   }

   const query = useQuery();
   const location = useLocation();
   const { projectStore } = useBackend();
   const [issueHistory, setIssueHistory] = useState(false);
   const [currentPivot, setCurrentPivot] = useState(query.get("workflow") === null ? "" : query.get("workflow")!);
   const { hordeClasses, detailClasses } = getHordeStyling();
   const navigate = useNavigate();
   const [filterState, setFilterState] = useState<FilterState>({ visible: false, unassigned: false, timeLower: 0, timeUpper: 0 });

   // subscribe
   if (handler.update) { }

   const allIssues = handler.issues.concat(handler.unpromoted);

   let issues: FindIssueResponse[] = [];

   if (currentPivot === "$promoted") {
      issues = handler.issues;
   }

   if (currentPivot === "$current") {
      issues = handler.unpromoted;
   }

   const stream = projectStore.streamById(handler.streamId);
   if (stream) {
      const config = stream.workflows?.find(w => w.id === currentPivot);
      if (config) {
         issues = allIssues.filter(issue => {
            return issue.openWorkflows.indexOf(currentPivot) !== -1;
         });
      }
   }

   // apply user-specified filters
   if (filterState.unassigned) { 
      issues = issues?.filter(issue => {
         return !!!issue.owner?.name;
      });
   }

   if (filterState.timeLower !== undefined && filterState.timeUpper !== undefined && filterState.timeUpper !== 0) {
      // zero time gap
      if (filterState.timeLower === filterState.timeUpper ) {
         issues = [];
      }
      else {
         let conversionFactor: number;
         switch (filterState.timeUnit?.key) {
            case 'days':
               conversionFactor = 24 * 60 * 60 * 1000;
               break;
            case 'minutes': 
               conversionFactor = 60 * 1000;
               break;
            default:
               conversionFactor = 60 * 60 * 1000;
         }

         let minTime = (Date.now() - (filterState.timeUpper * conversionFactor));
         let maxTime = (Date.now() - (filterState.timeLower * conversionFactor));
         
         issues = issues?.filter(issue => {
            let issueTime = new Date(issue.createdAt).getTime();
            return (issueTime >= minTime && issueTime <= maxTime);
         });
      }
   }


   const [items, groups] = sortIssueGroups(issues.map(i => { return { issue: i } }), projectStore);

   // 508
   const columns: IColumn[] = [
      { key: 'health_column1', name: 'Summary', minWidth: 370, maxWidth: 370, isResizable: false, isPadded: false }, // see summary render for px setting needed by ellipsis
      { key: 'health_column2', name: 'Quarantine', minWidth: 80, maxWidth: 80, isResizable: false, isPadded: false },
      { key: 'health_column3', name: 'Jira', minWidth: 80, maxWidth: 80, isResizable: false, isPadded: false },
      { key: 'health_column4', name: 'JiraPriority', minWidth: 64, maxWidth: 64, isResizable: false, isPadded: false },
      { key: 'health_column5', name: 'JiraAssignee', minWidth: 200, maxWidth: 200, isResizable: false, isPadded: false },
      { key: 'health_column6', name: 'JiraStatus', minWidth: 64, maxWidth: 64, isResizable: false, isPadded: false },
      { key: 'health_column7', name: 'Status', minWidth: 160, maxWidth: 160, isResizable: false, isPadded: false },
      { key: 'health_column8', name: 'Opened', minWidth: 140, maxWidth: 140, isResizable: false, isPadded: false },

   ];

   columns.forEach(c => {
      c.isPadded = false;
   })

   const renderItem = (item: HealthItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      const issue = item.issue;

      const textSize = "small";

      let ack = issue.acknowledgedAt ? "" : "Pending";
      let status = "Unassigned";
      if (issue.owner?.name) {
         status = issue.owner?.name;
      }
      if (issue.resolvedAt) {
         status = "Resolved";
         if (issue.fixChange) {
            status += ` in CL ${issue.fixChange}`;
         }
      }

      if (status === "Unassigned" || issue.resolvedAt) {
         ack = "";
      }

      if (column.name === "Summary") {

         let summary = issue.summary;

         return <Stack horizontal style={{ paddingTop: 6, height: "100%" }}>{<IssueStatusIconV2 issue={issue} />}<Text style={{ width: "370px", display: "block", textAlign: "left", overflow: "hidden", "textOverflow": "ellipsis" }} variant={textSize}>{`${issue.id} - ${summary}`}</Text></Stack>;
      }

      if (column.name === "Quarantine") {

         if (!issue.quarantinedBy && !issue.workflowThreadUrl) {
            return null;
         }

         if (issue.workflowThreadUrl) {
            return <a onClick={(e) => e.stopPropagation()} href={issue.workflowThreadUrl} target="_blank" rel="noreferrer"><Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="start" disableShrink={true}  >Slack Thread</Stack></a>
         }

         return <Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="start" disableShrink={true} ><Text variant={textSize}>Quarantined</Text></Stack>;
      }


      if (column.name === "Status") {

         if (ack) {
            status += ` (${ack})`;
         }

         return <Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="end" disableShrink={true} ><Text variant={textSize}>{status}</Text></Stack>;
      }

      if (issue.externalIssueKey) {

         const jissue = handler.jiraIssues.get(issue.externalIssueKey)?.issue;

         const textDecoration = jissue?.resolutionName ? "line-through" : undefined;

         if (column.name === "JiraPriority") {

            if (!jissue || !jissue.priorityName) {
               return null;
            }

            let priority = jissue.priorityName;
            if (priority.indexOf("- ") !== -1) {
               priority = priority.slice(priority.indexOf("- ") + 2);
            }

            return <a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="start" disableShrink={true}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{priority}</Text></Stack></a>;

         }


         if (column.name === "JiraAssignee") {

            if (!jissue) {
               return null;
            }

            return <a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="center" disableShrink={true}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{jissue.assigneeDisplayName ?? "Unassigned (Jira)"}</Text></Stack></a>
         }

         if (column.name === "JiraStatus") {

            const desc = jissue?.resolutionName ?? jissue?.statusName;

            if (!jissue) {
               return null;
            }

            return <a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="start" disableShrink={true}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{desc}</Text></Stack></a>;
         }

         if (column.name === "Jira") {

            if (!jissue) {
               return null;
            }

            return <a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Stack style={{ paddingTop: 6, height: "100%" }} horizontalAlign="start" disableShrink={true}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{jissue.key}</Text></Stack></a>;
         }
      }


      if (column.name === "Ack") {
         return <Text>{ack}</Text>;
      }


      if (column.name === "Opened") {
         const openSince = `${getShortNiceTime(issue.createdAt)} (${getElapsedString(moment(issue.createdAt), moment.utc(), false).trim()})`;
         return <Stack style={{ paddingRight: 8, paddingTop: 6, height: "100%" }} horizontalAlign="end"><Text variant={textSize}>{openSince}</Text></Stack>;
      }

      return null;
   }

   const classes = mergeStyleSets({
      detailsRow: {
         selectors: {
            '.ms-DetailsRow-cell': {
               padding: 0,
               overflow: "hidden",
               whiteSpace: "nowrap"
            }
         }
      }
   });

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as HealthItem;
         const newQuery = query.getCopy();
         newQuery.set("issue", item.issue.id.toString());
         const href = `${location.pathname}?${newQuery.toString()}`;

         return <Link to={href} className="job-item" onClick={() => setIssueHistory(true)}>
            <DetailsRow className={classes.detailsRow} {...props} />
         </Link>;

      }
      return null;
   };


   const onRenderGroupHeader: IDetailsGroupRenderProps['onRenderHeader'] = (props) => {
      if (props) {

         const group = props.group! as IssueGroup;

         return (
            <div className={detailClasses.headerAndFooter} style={{ marginRight: 8, padding: 2 }} onClick={() => {

               handler.collapsedIssueGroups.set(group.key, !handler.collapsedIssueGroups.get(group.key));
               props.onToggleCollapse!(props.group!);
            }}>
               <div style={{
                  fontSize: "13px",
                  padding: '4px 8px',
                  userSelect: 'none',
                  fontFamily: "Horde Open Sans SemiBold"
               }}>{`${group.headerText}`}</div>
            </div>
         );
      }

      return null;
   };


   groups.forEach(g => g.headerText += ` (${g.count})`);

   const pivotItems: JSX.Element[] = [];
   const pivotKeys: string[] = [];

   let issueText = "Promoted";
   let triageText = "Current";
   if (handler.issues.length) {
      issueText += ` (${handler.issues.length})`
   }
   if (handler.unpromoted.length) {
      triageText += ` (${handler.unpromoted.length})`
   }

   stream?.workflows?.forEach(config => {

      const workflowIssues = allIssues.filter(issue => issue.openWorkflows.indexOf(config.id) !== -1);
      if (!workflowIssues.length) {
         return;
      }
      
      pivotItems.push(<PivotItem headerText={`${config.summaryTab ?? config.id} (${workflowIssues.length})`} itemKey={`issue_pivot_item_key_${config.id}`} key={`issue_pivot_key_${config.id}`} headerButtonProps={{ onClick: () => { setCurrentPivot(config.id) } }} />);
      pivotKeys.push(config.id)
   });

   if (!handler.initialLoad) {
      pivotItems.push(<PivotItem headerText={issueText} itemKey={`issue_pivot_item_key_$promoted`} key={`issue_pivot_key_$promoted`} headerButtonProps={{ onClick: () => { setCurrentPivot("$promoted") } }} />);
      pivotKeys.push(`$promoted`)
      pivotItems.push(<PivotItem headerText={triageText} itemKey={`issue_pivot_item_key_$current`} key={`issue_pivot_key_$current`} headerButtonProps={{ onClick: () => { setCurrentPivot("$current") } }} />);   
      pivotKeys.push(`$current`)
   }

   if (!currentPivot && !handler.initialLoad) {

      setCurrentPivot(pivotKeys[0])   
      return null;      
   }

   function toggleUnassigned(_event: React.MouseEvent<HTMLElement>, checked?: boolean) {
      setFilterState(prev => ({ ...prev, unassigned: checked ?? false }));
   }

   function toggleFiltersVisisble(_event: React.MouseEvent<HTMLElement>) {
      setFilterState(prev => ({ ...prev, visible: !filterState.visible }));
   }

   const updateTimeRange = (_event: unknown, range: [number, number]) => {
      setFilterState(prev => ({ ...prev, timeLower: range[0], timeUpper: range[1] }));
   };

   const timeOptions: IDropdownOption[] = useMemo( () => [
      { key: 'minutes', text: 'minutes' },
      { key: 'hours', text: 'hours' },
      { key: 'days', text: 'days' }
   ], [filterState.timeUnit]);

  const updateTimeUnit = (_event: React.FormEvent<HTMLDivElement>, item: IDropdownOption): void => {
      setFilterState(prev => ({ ...prev, timeUnit: item }));
   };

  function clearFilters(_event: React.MouseEvent<HTMLElement>) {
      setFilterState({ visible: false });
   }

   return <Stack style={{ width: 1324, marginLeft: 4 }}>
      <IssueModalV2 issueId={query.get("issue")} streamId={handler.streamId!} popHistoryOnClose={issueHistory} />
      <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 4, paddingBottom: 4 } }}>
         <Stack horizontal verticalAlign='center'>
            <Stack horizontalAlign={"start"} >
               <Pivot className={hordeClasses.pivot}
                  style={{width: 1180}}
                  overflowBehavior='menu'
                  selectedKey={`issue_pivot_item_key_${currentPivot}`}
                  linkSize="normal"
                  linkFormat="links"
                  onLinkClick={(item => {
                     if (!item || !item.props.itemKey) {
                        return;
                     }

                     const search = new URLSearchParams(window.location.search);
                     const workflowId = item.props.itemKey.replace("issue_pivot_item_key_", "");
                     search.set("workflow", workflowId)
                     const url = `${window.location.pathname}?` + search.toString();
                     navigate(url, { replace: true })
         
                     setCurrentPivot(workflowId);
                  })}>
                  {pivotItems}
               </Pivot>
            </Stack>
            <Stack grow />
            <Stack horizontal style={{ paddingLeft: 10, paddingRight: 12}}>
               {!handler.initialLoad && (<TooltipHost content='Filters'>
                     <IconButton id='filtersButton' iconProps={{iconName: 'FilterList'}} onClick={toggleFiltersVisisble} />
                  </TooltipHost>
               )}
               {filterState.visible && (<Callout gapSpace={0} target={'#filtersButton'} onDismiss={toggleFiltersVisisble} setInitialFocus style={{ margin: 14 }} >
                  <Stack tokens={{ childrenGap: 28 }}>
                     <Text style={{ fontSize: 18, fontWeight: 'bold' }}>Filters</Text>
                     <Toggle label='Unassigned only' inlineLabel onChange={toggleUnassigned} checked={filterState.unassigned}/>
                     <Stack tokens={{ childrenGap: 5 }}>
                     <Text style={{ fontWeight: 'bold' }}>Show issues from past</Text>
                        <Stack horizontal verticalAlign='center' tokens={{ childrenGap: 4 }}>
                           <Text style={{ fontSize: 14, minWidth: 20 }}>{filterState.timeLower ?? 0}</Text>
                           <Slider ranged min={0} max={72} value={filterState.timeUpper} lowerValue={filterState.timeLower} showValue={false} onChange={updateTimeRange} styles={{ root : { width: 300 }, slideBox: { paddingLeft: 2 } }} />
                           <Text style={{ fontSize: 14, minWidth: 20 }}>{filterState.timeUpper && filterState.timeUpper !== 0 ? filterState.timeUpper : ' ' }</Text>
                           <Dropdown options={timeOptions} defaultSelectedKey={'hours'} selectedKey={filterState.timeUnit ? filterState.timeUnit.key : undefined} onChange={updateTimeUnit} styles={{ root : { width: 75, paddingBottom: 3 } }}/>
                        </Stack>
                     </Stack>
                     <Stack horizontal horizontalAlign='end'>
                        <DefaultButton text='Clear' onClick={clearFilters} style={{ fontSize: 13, height: 28, minWidth: 0, width: 62, padding: 0, marginTop: 10 }}/>
                     </Stack>
                  </Stack>
               </Callout>
               )}
            </Stack>
            <Stack >
               {(currentPivot === "$promoted" || currentPivot === "$current") && !handler.initialLoad && <DefaultButton text={`Export ${currentPivot === "$promoted" ? "Promoted" : "Current"}`} style={{ fontSize: 10, padding: 12, height: "24px", minWidth: "48px" }} onClick={() => handler.exportIssues(projectStore, items, groups, currentPivot === "$promoted")} />}
            </Stack>
         </Stack>

         {!issues.length && !handler.initialLoad && <Stack style={{ paddingBottom: 12, paddingTop: 12 }} horizontal tokens={{ childrenGap: 12 }}><Text variant="medium">No issues found</Text></Stack>}
         {!issues.length && handler.initialLoad && <Stack style={{ paddingBottom: 12, paddingTop: 12 }} horizontal tokens={{ childrenGap: 12 }}><Text variant="medium">Loading issues</Text><Spinner size={SpinnerSize.medium} /></Stack>}
         {!!issues.length && <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "640px" }} data-is-scrollable={true}>
            <Stack style={{ paddingTop: 6 }}>
               <DetailsList
                  compact={true}
                  isHeaderVisible={false}
                  indentWidth={0}
                  items={items}
                  groups={groups}
                  columns={columns}
                  setKey="set"
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  onRenderItemColumn={renderItem}
                  onRenderRow={renderRow}
                  groupProps={{
                     onRenderHeader: onRenderGroupHeader,
                  }}

               />
            </Stack>
         </div>}
      </Stack>
   </Stack>

});

const HealthPanel: React.FC<{ desktopAlerts?: boolean }> = observer(({ desktopAlerts }) => {

   const { hordeClasses } = getHordeStyling();

   if (handler.update) { }

   return <Stack style={{ width: 1384, marginLeft: 4 }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{desktopAlerts ? "Open Issues" : "Build Health"}</Text>
            </Stack>
            <Stack>
               <HealthPanelIssues desktopAlerts={desktopAlerts} />
            </Stack>
         </Stack>
      </Stack>
   </Stack>
});


const SchedulePanel: React.FC = observer(() => {

   const { projectStore } = useBackend();
   const { hordeClasses } = getHordeStyling();

   // subscribe
   if (handler.update) { }

   const stream = projectStore.streamById(handler.streamId!)!;

   let templates = stream.templates.filter(t => !!t.schedule).sort((a, b) => a.name < b.name ? -1 : 1);

   if (!templates.length) {
      return null;
   }

   return (<Stack style={{ width: 1384, marginLeft: 4 }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Schedule</Text>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <SchedulePane streamId={stream.id} templates={templates} />
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
})


const StreamSummaryInner: React.FC = observer(() => {

   const { streamId } = useParams<{ streamId: string }>();

   if (!streamId) {
      return null;
   }

   // subscribe
   if (handler.update) { }

   return <Stack tokens={{ childrenGap: 0 }} styles={{ root: { width: "100%", backgroundColor: "#fffffff", margin: 0, paddingTop: 8 } }}>
      <Stack style={{ padding: 0 }}>
         <div style={{ marginTop: 8, height: 'calc(100vh - 258px)', position: 'relative' }} data-is-scrollable={true}>
            <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} onScroll={() => { }}>
               <Stack tokens={{ childrenGap: 18 }} style={{ padding: 0 }}>
                  <HealthPanel desktopAlerts={false} />
                  <SchedulePanel />
               </Stack>
            </ScrollablePane>
         </div>
      </Stack>
   </Stack>
});

export const StreamSummary: React.FC = () => {

   const { streamId } = useParams<{ streamId: string }>();

   useEffect(() => {

      return () => {
         handler.clear();
      };

   }, []);


   if (!streamId) {
      return null;
   }

   handler.set(streamId!);

   return <StreamSummaryInner />;

};
