// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, FontIcon, HoverCard, HoverCardType, IContextualMenuProps, Pivot, PivotItem, PrimaryButton, Spinner, SpinnerSize, Stack, Text, TextField, mergeStyleSets } from '@fluentui/react';
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import React, { useEffect, useState } from 'react';
import { Link, Navigate, useLocation, useNavigate, useParams, useSearchParams } from 'react-router-dom';
import backend, { useBackend } from '../backend';
import { GetJobsTabResponse, GetLabelStateResponse, GetStreamTabResponse, GetTemplateRefResponse, JobData, JobsTabData, LabelOutcome, LabelState, ProjectData, TabStyle } from '../backend/Api';
import dashboard, { StatusColor } from '../backend/Dashboard';
import { projectStore } from '../backend/ProjectStore';
import { JobFilterSimple } from '../base/utilities/filter';
import { useWindowSize } from '../base/utilities/hooks';
import { getHordeStyling } from '../styles/Styles';
import { getHordeTheme } from '../styles/theme';
import { BreadcrumbItem, Breadcrumbs } from './Breadcrumbs';
import { useQuery } from "horde/base/utilities/hooks";
import { getAliasDisplayName, getStreamDisplayName } from "../base/utilities/streamUtils";
import { JobSearchSimpleModal } from './JobSearchSimple';
import { JobView } from './JobView';
import { JobViewAll } from './JobViewAll';
import { JobViewIncremental } from './JobViewIncremental';
import { NewBuild } from './NewBuild';
import { StreamSummary } from './StreamSummary';
import { TopNav } from './TopNav';
import { NewBuildV2 } from './build/NewBuildV2';

type StreamIncemental = {
   streamId: string;
   template: GetTemplateRefResponse;
   jobs: JobData[];
   labelState: LabelState;
   labelOutcome: LabelOutcome;
}

type JobLabelStatus = { index: number, outcome: LabelOutcome };

class IncrementalState {
   constructor() {
      makeObservable(this);
   }

   set(project?: ProjectData) {

      if (!project) {
         this.streamOutcome = new Map();
         this.project = undefined;
         return;
      }

      if (this.project?.id === project.id && this.lastPoll) {
         if (((Date.now() - this.lastPoll.getTime()) / 1000) < 120) {
            return;
         }
      }

      this.project = project;
      this.query();
   }

   @action
   setUpdated() {
      this.updated++;
   }

   @observable
   updated: number = 0;

   async query() {

      if (!this.project?.streams) {
         return;
      }

      // streams with incremental tabs
      // stream id => template id map
      const templateMap = new Map<string, GetTemplateRefResponse>();
      this.project.streams.forEach(s => {
         const tab = s.tabs.find(t => t.title === "Incremental") as GetJobsTabResponse;
         if (!tab) {
            return;
         }

         if (tab.templates?.length === 1) {
            const template = s.templates.find(t => t.id === tab.templates![0]);
            if (template)
               templateMap.set(s.id, template);
         }
      })

      let incrementals: StreamIncemental[] = [];

      templateMap.forEach((template, streamId) => {
         incrementals.push({
            streamId: streamId,
            template: template,
            jobs: [],
            labelState: LabelState.Unspecified,
            labelOutcome: LabelOutcome.Success
         })
      })

      if (!incrementals.length) {
         return;
      }

      let rincrementals = [...incrementals];

      this.querying = true;
      this.setUpdated()

      this.lastPoll = new Date();

      const jobStatus = new Map<string, JobLabelStatus[]>();

      while (rincrementals.length) {

         const batch = rincrementals.slice(0, 5);

         await Promise.all(batch.map(b => {
            return backend.getStreamJobs(b.streamId, { template: [b.template.id], count: 5, filter: "id,labels,createTime,streamId,defaultLabel,preflightChange,preflightCommitId" })
            // eslint-disable-next-line no-loop-func
         })).then((r) => {
            for (let i = 0; i < r.length; i++) {
               let jobs = r[i];
               // filter out jobs > 3 days
               jobs = jobs.filter(j => !j.preflightChange && (Date.now() - new Date(j.createTime).getTime()) < (1000 * 60 * 60 * 24 * 3));

               // eslint-disable-next-line no-loop-func
               jobs.forEach(j => {

                  let labels: GetLabelStateResponse[] = [];

                  if (j.defaultLabel) {
                     labels.push(j.defaultLabel)
                  }
                  if (j.labels) {
                     labels.push(...j.labels)
                  }

                  labels = labels.filter(label => label.state !== LabelState.Unspecified);

                  const labelStatus: JobLabelStatus[] = [];

                  labels.forEach((label, index) => {

                     if (label.outcome === LabelOutcome.Failure || label.outcome === LabelOutcome.Warnings) {
                        labelStatus.push({ index: index, outcome: label.outcome });
                     } else if (label.state === LabelState.Complete && label.outcome === LabelOutcome.Success) {
                        labelStatus.push({ index: index, outcome: label.outcome });
                     }
                  });

                  if (!labelStatus.length) {
                     return;
                  }

                  jobStatus.set(j.id, labelStatus);

                  const incremental = incrementals.find(i => i.streamId === j.streamId)
                  if (incremental) {
                     incremental.jobs.push(j);
                  }
               })
            }

         }).catch((errors) => {
            console.error(errors);
            // eslint-disable-next-line
         }).finally(() => {

            rincrementals = rincrementals.slice(5);
         });

      }

      incrementals = incrementals.filter(i => i.jobs.length > 0);

      const streamOutcome = new Map<string, LabelOutcome>();

      // figure out stream status
      incrementals.forEach(i => {
         if (!i.jobs.length) {
            return;
         }

         let jobs = i.jobs.sort((a, b) => new Date(b.createTime).getTime() - new Date(a.createTime).getTime());

         const labelStatus = new Map<number, LabelOutcome>();

         for (let x = 0; x < jobs.length; x++) {

            const j = jobs[x];
            const status = jobStatus.get(j.id);

            if (!status?.length) {
               continue;
            }

            for (let y = 0; y < status.length; y++) {
               const label = status[y];

               if (labelStatus.get(label.index)) {
                  continue;
               }

               labelStatus.set(label.index, label.outcome);
            }
         }

         const error = Array.from(labelStatus.values()).find(outcome => outcome === LabelOutcome.Failure)
         const warning = Array.from(labelStatus.values()).find(outcome => outcome === LabelOutcome.Warnings)

         if (!!error) {
            streamOutcome.set(i.streamId, LabelOutcome.Failure);
         } else if (!!warning) {
            streamOutcome.set(i.streamId, LabelOutcome.Warnings);
         }

      });

      Array.from(streamOutcome.keys()).forEach(k => { if (streamOutcome.get(k) === LabelOutcome.Success) streamOutcome.delete(k) });

      this.streamOutcome = streamOutcome;
      this.querying = false;
      this.setUpdated();
   }

   streamOutcome = new Map<string, LabelOutcome>();

   querying = false;

   project?: ProjectData;

   lastPoll?: Date;

}

const incrementalState = new IncrementalState();

const IncrementalPanel: React.FC<{ project: ProjectData }> = observer(({ project }) => {

   incrementalState.set(project)

   // subscribe
   if (incrementalState.updated) { }

   if (incrementalState.querying) {
      return <Stack key={`Incrementalpanel_spinner_${incrementalState.updated}`} style={{ padding: 32 }} tokens={{ childrenGap: 24 }} >
         <Stack>
            <Text style={{ fontWeight: 600 }} variant="medium">Querying Stream Labels</Text>
         </Stack>
         <Stack>
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Stack>
   }

   const items = Array.from(incrementalState.streamOutcome.keys()).map(streamId => {

      const name = projectStore.streamById(streamId)?.name ?? "Unknown Stream";
      const outcome = incrementalState.streamOutcome.get(streamId)!;

      const scolors = dashboard.getStatusColors();
      let color = scolors.get(StatusColor.Success)!;
      if (outcome === LabelOutcome.Warnings) {
         color = scolors.get(StatusColor.Warnings)!;
      }
      if (outcome === LabelOutcome.Failure) {
         color = scolors.get(StatusColor.Failure)!;
      }

      return <Link to={`/stream/${streamId}`}><Stack horizontal verticalFill verticalAlign='center' tokens={{ childrenGap: 8 }}><Stack>
         <FontIcon style={{ color: color, paddingTop: 2 }} iconName="Square" />
      </Stack>
         <Stack>
            <Text>{name}</Text>
         </Stack>
      </Stack>
      </Link>
   })

   if (!items.length) {
      return <Stack key={`Incrementalpanel_${incrementalState.updated}`} style={{ padding: 32 }} tokens={{ childrenGap: 12 }}>
         <Text style={{ fontWeight: 600 }} variant='medium'>No Label Issues</Text>
      </Stack>
   }


   return <Stack key={`Incrementalpanel_${incrementalState.updated}`} style={{ padding: 32 }} tokens={{ childrenGap: 18 }}>
      <Stack>
         <Text style={{ fontWeight: 600 }} variant='medium'>Label Issues</Text>
      </Stack>
      <Stack tokens={{ childrenGap: 12 }} >
         {items}
      </Stack>
   </Stack>

})

const StreamViewInner: React.FC = observer(() => {

   const windowSize = useWindowSize();

   const { streamId } = useParams<{ streamId: string }>();
   const navigate = useNavigate();
   const query = useQuery();

   const [showOthersPreflights, setShowOthersPreflights] = useState<boolean | undefined>(dashboard.showPreflights);

   const [shown, setShown] = useState(query.get("newbuild") ? true : false);
   const [searchParams] = useSearchParams();
   const location = useLocation();
   const searchInURL: boolean = searchParams.has("search");
   const [findJobsShown, setFindJobsShown] = useState(searchInURL);

   const { projectStore } = useBackend();

   const { hordeClasses, modeColors } = getHordeStyling();

   const hordeTheme = getHordeTheme();

   const stream = projectStore.streamById(streamId);
   const project = stream?.project;

   // Respond to search param even when navigating between search queries within the same route
   // One consequence being pressing the browser's back button can sometimes close the search modal depending on search state
   useEffect(() => {
      setFindJobsShown(searchParams.has("search"));
   }, [location.search])

   //let newBuildVersion: string = query.get("newbuildversion") ? query.get("newbuildversion") : "2";
   const newBuildVersion: string = "2"

   if (!stream || !project) {
      console.error("Bad stream or project id in StreamView");
      return <Navigate to="/" replace={true} />
   }

   let filter = query.get("filter") ? query.get("filter")! : undefined;
   const aliasParam = query.get("alias");

   // Get the active alias and filter tabs/templates if present
   const activeAlias = aliasParam ? stream.aliases?.find(a => a.name === aliasParam) : undefined;
   const aliasTemplateSet = activeAlias ? new Set(activeAlias.templates) : undefined;

   // Filter tabs to only those with templates matching the alias
   const filteredTabs = aliasTemplateSet
      ? stream.tabs.filter(tab => {
         const jobsTab = tab as GetJobsTabResponse;
         return jobsTab.templates?.some(t => aliasTemplateSet.has(t));
      })
      : stream.tabs;

   let queryTab = query.get("tab") ?? undefined;
   const currentTab = filteredTabs.find(t => t.title === queryTab) as JobsTabData | undefined;

   // When alias is active, don't allow summary/all tabs
   const allowSummaryAll = !activeAlias;
   const validTab = currentTab || (allowSummaryAll && (queryTab?.toLowerCase() === "summary" || queryTab?.toLowerCase() === "all"));

   if (!queryTab || !validTab) {
      const aliasQueryParam = aliasParam ? `&alias=${encodeURIComponent(aliasParam)}` : "";

      if (!!filteredTabs.find(t => t.title.toLowerCase() === "incremental")) {
         return <Navigate to={`/stream/${streamId}?tab=Incremental${aliasQueryParam}`} replace={true} />
      } else if (filteredTabs.length) {
         return <Navigate to={`/stream/${streamId}?tab=${filteredTabs[0].title}${aliasQueryParam}`} replace={true} />
      } else if (allowSummaryAll) {
         return <Navigate to={`/stream/${streamId}?tab=all${aliasQueryParam}`} replace={true} />
      } else {
         // Alias with no matching tabs - show error or redirect to stream without alias
         return <Navigate to={`/stream/${streamId}`} replace={true} />
      }
   }

   const isSummary = queryTab === 'summary';
   const isAllView = queryTab === 'all';

   let crumbText = getStreamDisplayName(stream, aliasParam);

   if (!isSummary && !isAllView) {
      const templateFilter = query.get("template") ?? undefined;
      if (templateFilter) {
         const template = stream.templates?.find(t => t.id === templateFilter);
         if (template) {
            crumbText += ` - ${template.name}`;
         }
      }
   }

   const crumbItems: BreadcrumbItem[] = [
      {
         text: stream.project!.name,
         link: `/project/${stream.project!.id}`
      },
      {
         text: crumbText
      }
   ];

   const crumbTitle = `Horde: ${stream.fullname ?? `//${stream.project?.name}/${stream.name}`}`;

   const onRenderPlainCard = (tab: GetStreamTabResponse) => {
      return <Stack>
         <IncrementalPanel project={project} />
      </Stack>
   }

   const aliasQueryParam = aliasParam ? `&alias=${encodeURIComponent(aliasParam)}` : "";

   const pivotItems = filteredTabs.map(tab => {
      return <PivotItem headerText={tab.title} itemKey={tab.title} key={tab.title} onRenderItemLink={() => {
         if ((tab as GetJobsTabResponse).title === "Incremental") {
            return <HoverCard cardOpenDelay={250} type={HoverCardType.plain} plainCardProps={{ onRenderPlainCard: onRenderPlainCard, renderData: tab }}>
               <Link to={`/stream/${streamId}?tab=${encodeURIComponent(tab.title)}${aliasQueryParam}`} style={{ color: modeColors.text }}>{tab.title}</Link>
            </HoverCard>
         }
         return <Link to={`/stream/${streamId}?tab=${encodeURIComponent(tab.title)}${aliasQueryParam}`} style={{ color: modeColors.text }}>{tab.title}</Link>
      }
      } />;
   });

   // Show alias indicator when viewing via alias, otherwise show All and Summary tabs
   if (activeAlias) {
      const aliasDisplayName = getAliasDisplayName(activeAlias.name);
      pivotItems.unshift(
         <PivotItem headerText={aliasDisplayName} itemKey="alias_indicator" key="pivot_item_alias" onRenderItemLink={() =>
            <HoverCard cardOpenDelay={0} type={HoverCardType.plain} plainCardProps={{
               onRenderPlainCard: () => (
                  <Stack style={{ padding: 8 }}>
                     <Link to={`/stream/${streamId}?tab=summary`} style={{ color: modeColors.text }}>View Full Stream</Link>
                  </Stack>
               )
            }}>
               <Stack horizontal verticalAlign="center" style={{ cursor: 'pointer' }}>
                  <Text style={{ color: modeColors.text }}>{aliasDisplayName}</Text>
                  <FontIcon iconName="ChevronDown" style={{ fontSize: 10, marginLeft: 4, color: modeColors.text }} />
               </Stack>
            </HoverCard>
         } />
      )
   } else {
      pivotItems.unshift(<PivotItem headerText="All" itemKey="all" key="pivot_item_all" onRenderItemLink={() => <Link to={`/stream/${streamId}?tab=all${aliasQueryParam}`} style={{ color: modeColors.text }}>All</Link>} />)
      pivotItems.unshift(<PivotItem headerText="Summary" itemKey="summary" key="pivot_item_summary" onRenderItemLink={() => <Link to={`/stream/${streamId}?tab=summary${aliasQueryParam}`} style={{ color: modeColors.text }}>Summary</Link>} />)
   }

   let findJobsItems: IContextualMenuProps = { items: [] };

   if (!isSummary) {

      findJobsItems = {
         items: [
            {
               key: 'no_results',
               onRender: (item, dismissMenu) => {
                  return <Stack style={{ padding: 8, paddingTop: 16, paddingBottom: 12 }}>
                     <TextField
                        deferredValidationTime={750}
                        validateOnLoad={false}
                        defaultValue={filter}
                        spellCheck={false}
                        placeholder="Filter Jobs"

                        onGetErrorMessage={(newValue) => {

                           const search = new URLSearchParams(window.location.search);
                           if (newValue?.trim().length) {
                              search.set("filter", newValue);
                           } else {
                              search.delete("filter");
                           }

                           const url = `${window.location.pathname}?` + search.toString();
                           navigate(url, { replace: true });

                           return undefined;
                        }}

                     />
                  </Stack>
               }
            },
            {
               key: 'show_other_preflights',
               text: 'Show preflights for all users',
               iconProps: { iconName: showOthersPreflights ? 'Tick' : "" },
               onClick: () => {
                  setShowOthersPreflights(!showOthersPreflights);
               }
            }
         ]
      };
   }


   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const newBuildTab = queryTab?.toLowerCase() === "summary" ? "all" : queryTab;
   let isIncrementalTab = currentTab?.style === TabStyle.Compact;

   const windowWidth = windowSize.width;

   const simpleFilter: JobFilterSimple = {
      showOthersPreflights: !!showOthersPreflights,
      filterKeyword: filter
   }

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <Breadcrumbs items={crumbItems} title={crumbTitle} />
         {shown && newBuildVersion == "2" && <NewBuildV2 streamId={streamId!} jobKey={newBuildTab!} show={true} aliasTemplates={activeAlias?.templates} onClose={(newJobId) => {
            setShown(false);
            if (newJobId) {
               navigate(`/job/${newJobId}`);
            } else {
               if (query.get("newbuild")) {
                  navigate(`/stream/${streamId}?tab=${queryTab}`, { replace: true });
               }
            }
         }} />}
         {shown && newBuildVersion == "1" && <NewBuild streamId={streamId!} jobKey={newBuildTab!} show={true} onClose={(newJobId) => {
            setShown(false);
            if (newJobId) {
               navigate(`/job/${newJobId}`);
            } else {
               if (query.get("newbuild")) {
                  navigate(`/stream/${streamId}?tab=${queryTab}`, { replace: true });
               }
            }
         }} />}
         {findJobsShown && <JobSearchSimpleModal onClose={() => { setFindJobsShown(false) }} streamIdIn={stream.id} jobTab={currentTab} autoloadIn={searchInURL} />}
         <Stack horizontal style={{ backgroundColor: hordeTheme.horde.neutralBackground }}>
            <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: (vw / 2 - (1440 / 2)) - 12, flexShrink: 0, backgroundColor: modeColors.background }} />
            <Stack tokens={{ childrenGap: 0 }} styles={{ root: { width: "100%" } }}>
               <Stack style={{ width: 1440, paddingTop: 12, marginLeft: 4 }}>
                  <Stack style={{ maxWidth: windowWidth - 12 }} horizontal verticalAlign='center' verticalFill={true}>
                     <Stack style={{ paddingLeft: 4, paddingTop: 2, width: 1180 }}>
                        <Pivot className={hordeClasses.pivot}
                           overflowBehavior='menu'
                           selectedKey={queryTab}
                           linkSize="normal"
                           linkFormat="links"
                           onLinkClick={(item) => {
                              if (item!.props.itemKey !== queryTab) {
                                 // this still needs to be here, even though specific button href in item
                                 navigate(`/stream/${streamId}?tab=${encodeURIComponent(item!.props.itemKey!)}`);
                              }
                           }}>
                           {pivotItems}
                        </Pivot>
                     </Stack>
                     <Stack grow />
                     <Stack horizontal verticalAlign="center" horizontalAlign={"end"} tokens={{ childrenGap: 8 }}>
                        <Stack horizontal tokens={{ childrenGap: 18 }}>
                           <DefaultButton
                              styles={{ root: { fontFamily: "Horde Open Sans SemiBold !important" } }}
                              text="Search"
                              split={!isSummary}
                              menuProps={!isSummary ? findJobsItems : undefined}
                              onClick={() => setFindJobsShown(true)}
                           />
                           <PrimaryButton
                              styles={{ root: { fontFamily: "Horde Open Sans SemiBold !important" } }}
                              onClick={() => { setShown(true); }}>
                              New&nbsp;Job
                           </PrimaryButton>
                        </Stack>
                     </Stack>
                  </Stack>

               </Stack>
               <Stack style={{ width: "100%", paddingTop: 12, marginLeft: 4 }} >
                  <Stack >
                     {!isSummary && !isAllView && !isIncrementalTab && queryTab && <JobView tab={queryTab} filter={simpleFilter} aliasTemplates={activeAlias?.templates} />}
                     {isSummary && <StreamSummary />}
                     {isAllView && <JobViewAll filter={simpleFilter} />}
                     {isIncrementalTab && <JobViewIncremental tab={queryTab} filter={simpleFilter} aliasTemplates={activeAlias?.templates} />}
                  </Stack>
               </Stack>

            </Stack>
         </Stack>
      </Stack>
   );
});

export const StreamView: React.FC = () => {

   return <StreamViewInner />;

};


