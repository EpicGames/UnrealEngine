// Copyright Epic Games, Inc. All Rights Reserved.
import { IconButton, Modal, PrimaryButton, Stack, Text, Label } from '@fluentui/react';
import React from 'react';
import dashboard from '../backend/Dashboard';
import { getHordeStyling } from '../styles/Styles';


export const SupportModal: React.FC<{ show: boolean, onClose: () => void }> = ({ show, onClose }) => {

    const { hordeClasses } = getHordeStyling();

    const close = () => {
        onClose();
    };

    const helpEmail = dashboard.helpEmail;
    const helpSlack = dashboard.helpSlack;

    return (
        <Modal className={hordeClasses.modal} isOpen={show} styles={{ main: { padding: 16, width: 540, display: "relative" } }} onDismiss={() => { if (show) close() }}>
            <IconButton
                style={{position: "absolute", right: 10, top: 10}}
                iconProps={{ iconName: 'Cancel' }}
                ariaLabel="Close popup modal"
                onClick={() => { close(); }}
            />
            
            <Text block style={{ width: "100%", fontSize: 18, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold", marginBottom: 16 }}>Horde Support</Text>

            <Text block style={{ fontSize: 15, marginBottom: 12 }}>For support, feedback, and suggestions:</Text>
            
            <Stack tokens={{childrenGap: 8}}>
                {helpEmail && <Text block style={{ fontSize: 15, marginLeft: 8 }}>
                    Email:
                    <Text style={{ marginLeft: 12, fontSize: 15, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }}>{helpEmail}</Text>
                </Text>}
                {helpSlack && <Text block style={{ fontSize: 15, marginLeft: 8 }}>
                    Slack:
                    <Text style={{ marginLeft: 12, fontSize: 15, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }}>{helpSlack}</Text>
                </Text>}
            </Stack>

            <PrimaryButton style={{ position: "absolute", right: 16, bottom: 16 }} text="Ok" disabled={false} onClick={() => { close(); }} />
        </Modal>
    );

};
