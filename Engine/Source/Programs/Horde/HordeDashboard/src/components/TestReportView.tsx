// Copyright Epic Games, Inc. All Rights Reserved.

import React, { useEffect, useRef, useState } from 'react';
import { TopNav } from './TopNav';
import { Breadcrumbs, BreadcrumbItem } from './Breadcrumbs';
import { Stack, Spinner, SpinnerSize, Text } from '@fluentui/react';
import { PluginMount } from  '../legacyPlugins/pluginTypes';
import hordePlugins from  '../legacyPlugins';
import { projectStore } from '../backend/ProjectStore';
import { useParams } from 'react-router-dom';
import { TestDataCollection, TestDataWrapper } from '../backend/TestDataHandler'
import { useQuery } from "horde/base/utilities/hooks";
import { getStreamDisplayName, getStreamLink } from "../base/utilities/streamUtils";
import { getHordeStyling } from '../styles/Styles';

const getFirstComponentFromPlugin = (mount: PluginMount, id: string): React.FC<any> | undefined => {
    let component : React.FC<any> | undefined;
    const components = hordePlugins.getComponents(mount, id);
    if (components.length > 0) {
        component = components[0].component;
        if (components.length > 1) {
            console.warn("There is several registered test data type under '"+ id +"'. Verify the Horde plugins.");
        }
    }
    return component;
}

export const getStreamData = (streamId: string) => {
    return projectStore.streamById(streamId);
}

export const getProjectName = (streamId: string): string => {
    const stream = getStreamData(streamId);
    let projectName = stream?.project?.name;
    if (projectName === "Engine") {
        projectName = "UE4";
    }

    return projectName ?? "Unknown Project";
}

const getTestdataBreadcrumbProps = async (testdata: TestDataWrapper) => {

    const jobName = await testdata.getJobName();
    const stepName = await testdata.getJobStepName();
    const changeName = await testdata.getChangeName();

    const stream = getStreamData(testdata.streamId);
    const aliasParam = new URLSearchParams(window.location.search).get("alias");

    const crumbItems: BreadcrumbItem[] = [
        {
            text: getProjectName(testdata.streamId),
            link: `/project/${stream?.project?.id}`
        },
        {
            text: getStreamDisplayName(stream, aliasParam),
            link: stream ? getStreamLink(stream.id, aliasParam) : undefined
        },
        {
            text: `${changeName}: ${jobName}`,
            link: testdata.getJobLink(),
        },
        {
            text: stepName ?? "",
            link: testdata.getJobStepLink(),
        },
        {
            text: testdata.key 
        }
    ];

    return {
        items: crumbItems,
        title: `Horde - ${changeName}: ${jobName} - ${testdata.key}`
    }
}

export const testDataHandler = new TestDataCollection();

export const ComponentMount: React.FC<any> = (props) => {
    const { component } = props;
    return component?component(props):<div/>;
}

export const TestReportView: React.FC = () => {
    
    const { testdataId } = useParams<{ testdataId: string }>();
    const component = useRef<React.FC<any> | undefined>(undefined);
    const [loading, setLoading] = useState(false);
    const [testdata, setTestdata] = useState<TestDataWrapper | undefined>(undefined);
    const [crumbProps, setCrumbProps] = useState<{items: BreadcrumbItem[], title: string}>({items: [], title:"Test Report"});

    const query = useQuery();

    useEffect(() => {
        if (testdataId) {
            const getTestdata = async () => {
                setLoading(true);
                try {
                    const testdata = await testDataHandler.setFromId(testdataId);
                    setTestdata(testdata);
                    if (testdata) {
                        const type = testdata.type || "Undefined";
                        component.current = getFirstComponentFromPlugin(PluginMount.TestReportPanel, type);
                        if (!component.current) {
                            console.error("Unknown test data type '"+ type +"'! Register testdata type through Horde plugins.");
                        }
                        const props = await getTestdataBreadcrumbProps(testdata);
                        setCrumbProps(props);
                    }
                } catch {
                    console.error("Bad test data id!");
                    // Make that a better error message
                } finally {
                    setLoading(false);
                }
            }
            getTestdata();
        } else {
            console.error("Bad test data id!");
            testDataHandler.desactivate();
        }
        return function cleanup() {testDataHandler.desactivate()}
    }, [testdataId]);
   
    const { hordeClasses } = getHordeStyling();

    return (
        <Stack className={hordeClasses.horde} style={{height: '100vh'}}>
            <TopNav />
            <Breadcrumbs items={crumbProps.items.concat()} title={crumbProps.title}/>
            <Stack styles={{ root: { overflow: 'auto'}}} grow>
                {!loading && testdata && testDataHandler.cursor && <ComponentMount component={component.current} data={testdata.data} query={query}/>}
                {loading && <Stack.Item align="center" styles={{ root: { padding: 10 }}}><Text variant="mediumPlus">Loading TestData</Text><Spinner styles={{ root: { padding: 10 }}} size={SpinnerSize.large}></Spinner></Stack.Item>}
            </Stack>
        </Stack>
    );
}
