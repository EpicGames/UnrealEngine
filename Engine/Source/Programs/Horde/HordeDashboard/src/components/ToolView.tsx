// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, DirectionalHint, IColumn, IContextualMenuProps, IDetailsGroupDividerProps, IDetailsGroupRenderProps, IDetailsListProps, IGroup, IconButton, Pivot, PivotItem, PrimaryButton, SelectionMode, Stack, Text } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import backend from "../backend";
import { GetToolSummaryResponse } from "../backend/Api";
import { PollBase } from "../backend/PollBase";
import { useWindowSize } from "../base/utilities/hooks";
import { getHordeStyling } from "../styles/Styles";
import { Breadcrumbs } from "./Breadcrumbs";
import { TopNav } from "./TopNav";
import { ToolManagementModal } from "./tools/ToolManagement";

const defaultCategory = "General";

enum Platform {
   Windows = "win-x64",
   MacOS = "osx-x64",
   Linux = "linux-x64",
   Unknown = "Unknown"
}

class ToolHandler extends PollBase {

   constructor(pollTime = 30000) {
      super(pollTime);

      const userAgent = window.navigator.userAgent;

      if (userAgent.indexOf("Win") != -1) this.clientPlatform = Platform.Windows;
      else if (userAgent.indexOf("Mac") != -1) this.clientPlatform = Platform.MacOS;
      else if (userAgent.indexOf("X11") != -1) this.clientPlatform = Platform.Linux;
      else if (userAgent.indexOf("Linux") != -1) this.clientPlatform = Platform.Linux;
      else {
         // guess windows
         this.clientPlatform = Platform.Windows;
      }

   }

   clear() {
      this.categories = new Map();
      this.loaded = false;
      super.stop();
   }

   async poll(): Promise<void> {

      if (this.loaded) {
         return;
      }

      try {

         this.tools = await backend.getTools();
         this.tools = this.tools.filter(t => t.showInDashboard);
         this.categories = new Map();

         this.tools.forEach(t => {

            // categories
            const cat = t.category ?? defaultCategory;
            if (!this.categories.has(cat)) {
               this.categories.set(cat, []);
            }
            this.categories.get(cat)!.push(t);

         })


         // sort by prefered platform
         this.tools = this.tools.sort((a, b) => {

            if (a.group && !b.group) {
               return -1;
            }

            if (!a.group && b.group) {
               return 1;
            }

            if (a.group && b.group && (a.group !== b.group)) {
               return a.group.localeCompare(b.group);
            }


            if (a.platforms?.length && !b.platforms?.length) {
               return -1;
            }

            if (!a.platforms?.length && b.platforms?.length) {
               return 1;
            }

            if (a.platforms?.length && b.platforms?.length) {

               if ((a.platforms.indexOf(this.clientPlatform) !== -1) && (b.platforms.indexOf(this.clientPlatform) === -1)) {
                  return -1;
               }
               if ((a.platforms.indexOf(this.clientPlatform) === -1) && (b.platforms.indexOf(this.clientPlatform) !== -1)) {
                  return 1;
               }

               if ((a.platforms.indexOf("any")) !== -1 && (b.platforms.indexOf("any") === -1)) {
                  return -1;
               }

               if ((a.platforms.indexOf("any")) === -1 && (b.platforms.indexOf("any") !== -1)) {
                  return 1;
               }
            }

            return a.name.localeCompare(b.name);
         })


         this.loaded = true;
         this.setUpdated();

      } catch (err) {

      }

   }

   clientPlatform: Platform;

   categories: Map<string, GetToolSummaryResponse[]> = new Map();

   loaded = false;
   tools: GetToolSummaryResponse[] = [];
}

const handler = new ToolHandler();

const ToolPanel: React.FC<{ selectedKey: string }> = observer(({ selectedKey }) => {

   const [manageTool, setManageTool] = useState("");

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { hordeClasses, modeColors } = getHordeStyling();

   // subscribe
   if (handler.updated) { };

   const columns: IColumn[] = [
      { key: 'column_name', name: 'Name', minWidth: 240, maxWidth: 240, isResizable: false },
      { key: 'column_desc', name: 'Description', fieldName: 'description', minWidth: 580, maxWidth: 580, isResizable: false, isMultiline: true },
      { key: 'column_version', name: 'Version', fieldName: 'version', minWidth: 280, maxWidth: 280, isResizable: false, headerClassName: hordeClasses.detailsHeader },
      { key: 'column_download', name: '', minWidth: 16, maxWidth: 16, isResizable: false }
   ];

   let tools = [...handler.tools];

   tools = tools.filter(t => (t.category ?? defaultCategory) === selectedKey)

   const groups: IGroup[] = [];

   let cgroup: string | undefined;

   const toolHide = new Set<string>();

   let ungrouped = 0;

   // emit groups
   for (let i = 0; i < tools.length; i++) {
      const tool = tools[i];

      if (tool.group != cgroup) {

         if (cgroup) {
            groups[groups.length - 1].count = i - groups[groups.length - 1].startIndex;
         }

         cgroup = tool.group ?? `Ungrouped ${ungrouped++}`;

         if (cgroup) {
            groups.push({ startIndex: i, name: tool.name, key: `group_key_${cgroup}`, count: 0, isCollapsed: true, data: tool });
            toolHide.add(tool.id);
         }
      }
   }

   if (cgroup) {
      groups[groups.length - 1].count = tools.length - groups[groups.length - 1].startIndex;
   }

   const renderTool = (tool: GetToolSummaryResponse, groupProps?: IDetailsGroupDividerProps) => {

      let pad = 0;
      let width = 330;
      if (groupProps && groupProps.group!.count < 2) {
         pad = 36;
         width += pad
      }

      if (!groupProps) {
         pad = 36 + 12;
         width = 366;
      }

      const downloadProps: IContextualMenuProps = {
         items: [
            {
               key: 'managetool',
               text: 'Manage',
               onClick: () => {
                  setManageTool(tool.id);
               }
            },
         ],
         directionalHint: DirectionalHint.bottomRightEdge
      };


      return <Stack horizontal verticalAlign="center" verticalFill={true} style={{ padding: 12 }}>
         {!!groupProps && groupProps.group!.count > 1 && <IconButton style={{ color: modeColors.text, fontSize: 18, marginRight: 4 }} iconProps={{ iconName: groupProps.group?.isCollapsed ? 'ChevronRight' : 'ChevronDown' }} onClick={(event: any) => {
            event?.stopPropagation();
            groupProps.onToggleCollapse!(groupProps.group!);
         }}
         />}

         <Stack style={{ width: width, paddingLeft: pad }}>
            <Text style={{ fontFamily: (groupProps || !tool.group) ? "Horde Open Sans SemiBold" : undefined, color: modeColors.text }}>{tool.name}</Text>
         </Stack>
         <Stack style={{ width: 580 }}>
            <Text style={{ color: modeColors.text }}>{tool.description ?? ""}</Text>
         </Stack>
         <Stack style={{ width: 220 }}>
            <Text style={{ color: modeColors.text }}>{tool.version}</Text>
         </Stack>
         <Stack >
            <PrimaryButton split={!tool.bundled} menuProps={!tool.bundled ? downloadProps : undefined} style={{ width: 130, color: "#FFFFFF" }} text="Download" onClick={() => window.location.assign(`/api/v1/tools/${tool.id}?action=download`)} />
         </Stack>
      </Stack>
   }

   const onRenderGroupHeader: IDetailsGroupRenderProps['onRenderHeader'] = (props: IDetailsGroupDividerProps) => {
      if (props) {

         const group = props.group!;

         return renderTool(group.data as GetToolSummaryResponse, props)

      }

      return null;
   };

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as GetToolSummaryResponse;
         if (toolHide.has(item.id)) {
            return null;
         }

         return renderTool(item)

      }
      return null;
   };



   return <Stack>
      {!!manageTool && <ToolManagementModal toolId={manageTool} toolName={handler.tools.find(t => t.id === manageTool)?.name} onClose={() => setManageTool("")} />}
      {!tools.length && handler.loaded && <Stack style={{ paddingBottom: 12 }}>
         <Stack verticalAlign="center">
            <Stack horizontalAlign="center">
               <Text variant="mediumPlus">No Tools Found</Text>
            </Stack>
         </Stack>
      </Stack>}

      {!!tools.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{
            root: {
               paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%", selectors: {
                  ".ms-DetailsHeader": { "paddingTop": "0px" }, '.ms-List-cell': {
                     minHeight: "0px !important"
                  }
               }
            }
         }} >
            <DetailsList
               isHeaderVisible={false}
               styles={{
                  headerWrapper: {
                     paddingTop: 0
                  }
               }}
               items={tools}
               groups={groups.length ? groups : undefined}
               groupProps={{
                  onRenderHeader: onRenderGroupHeader,
               }}
               columns={columns}
               selectionMode={SelectionMode.none}
               layoutMode={DetailsListLayoutMode.justified}
               compact={true}
               onRenderRow={renderRow}
            />
         </Stack>
      </Stack>}
   </Stack>
});

const ToolViewInner: React.FC = observer(() => {

   const [selectedKey, setSelectedKey] = useState<string>(defaultCategory);

   const windowSize = useWindowSize();

   // subscribe
   if (handler.updated) { };

   const { hordeClasses, modeColors } = getHordeStyling();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;
   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   const categories = Array.from(handler.categories.keys()).sort((a, b) => a.localeCompare(b));

   const pivotItems = categories.map(cat => {
      if (cat === defaultCategory) {
         return undefined;
      }
      return <PivotItem headerText={cat} itemKey={cat} key={cat} style={{ color: modeColors.text }} />;
   }).filter(p => !!p);

   if (handler.categories.get(defaultCategory)?.length) {
      pivotItems.unshift(<PivotItem headerText={defaultCategory} itemKey={defaultCategory} key={defaultCategory} style={{ color: modeColors.text }} />);
   } else {
      if (categories.length) {
         if (!selectedKey || selectedKey === defaultCategory) {
            setSelectedKey(categories[0]);
         }
      }
   }

   return <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
      <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
         <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
            <div style={{ overflowX: "auto", overflowY: "visible" }}>
               <Stack horizontal style={{ paddingTop: 12, paddingBottom: 48 }}>
                  <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                  <Stack style={{ width: 1440 }}>
                     <Stack style={{ paddingBottom: 12 }}>
                        <Pivot className={hordeClasses.pivot}
                           overflowBehavior='menu'
                           selectedKey={selectedKey}
                           linkSize="normal"
                           linkFormat="links"
                           onLinkClick={(item) => {
                              if (item?.props.itemKey) {

                                 setSelectedKey(item.props.itemKey);
                              }
                           }}>
                           {pivotItems}
                        </Pivot>

                     </Stack>

                     <ToolPanel selectedKey={selectedKey} />
                  </Stack>
               </Stack>
            </div>
         </Stack>
      </Stack>
   </Stack>

})

export const ToolView: React.FC = () => {


   const { hordeClasses } = getHordeStyling();


   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Tools' }, { text: 'Downloads' }]} />
      <ToolViewInner />
   </Stack>
};

