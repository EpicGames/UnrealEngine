// Copyright Epic Games, Inc. All Rights Reserved.

import { CommandBar, CommandBarButton, ContextualMenu, ContextualMenuItem, ContextualMenuItemType, DefaultButton, Dialog, DialogFooter, DialogType, IButton, IButtonProps, ICommandBarItemProps, IContextualMenuItem, IContextualMenuItemProps, IContextualMenuItemStyles, IContextualMenuStyles, Persona, PersonaSize, PrimaryButton, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useRef, useState } from 'react';
import { Link, useNavigate } from 'react-router-dom';
import backend, { useBackend } from '../backend';
import { AuthMethod, ProjectData } from "../backend/Api";
import dashboard from '../backend/Dashboard';
import { ProjectStore } from '../backend/ProjectStore';
import { PreviewChangesModal } from './PreviewChanges';
import { VersionModal } from './VersionModal';
import { getHordeTheme } from '../styles/theme';
import { getHordeStyling } from '../styles/Styles';
import { ArtifactQueryState, FindArtifactsModal } from './artifacts/ArtifactsSearch';
import { getHordePlugins, getFilteredMounts, MountType } from 'hordePlugins';
import { SupportModal } from './SupportModal';
import { getAliasDisplayName, getStreamLink } from '../base/utilities/streamUtils';

const getStyles = () => {

   const theme = getHordeTheme();

   // Styles for both command bar and overflow/menu items
   const itemStyles: Partial<IContextualMenuItemStyles> = {
      label: { fontSize: 12 },
      root: {
         paddingLeft: 6,
         selectors: {
            'a:link,a:visited': {
               color: theme.semanticColors.bodyText
            }
         }
      }
   };
   // For passing the styles through to the context menus
   const menuStyles: Partial<IContextualMenuStyles> = {
      root: {
         fontSize: 12, selectors: {
            'a:link,a:visited': {
               color: theme.semanticColors.bodyText
            }
         }
      },
      header: { fontFamily: "Horde Open Sans Bold" },
      subComponentStyles: { menuItem: itemStyles, callout: {} }
   };

   return [itemStyles, menuStyles];
}


type ButtonData = {
   project: ProjectData;
}

// Top level project button component
const ProjectButton: React.FunctionComponent<IButtonProps> = (props) => {

   const hordeTheme = getHordeTheme();
   const [itemStyles] = getStyles();

   const navigate = useNavigate();
   const { project } = props.data as ButtonData;

   const buttonRef = React.createRef<IButton>();
   return (
      <Stack disableShrink={true}
         onMouseEnter={() => {
            buttonRef.current?.openMenu(true);
         }}
         onMouseLeave={() => {
            buttonRef.current?.dismissMenu();
         }} >
         <CommandBarButton
            componentRef={buttonRef}
            menuProps={{ subMenuHoverDelay: 0, items: [] }}
            href={`/project/${project?.id}`}

            onMenuClick={(ev) => {

               if (ev?.metaKey || ev?.ctrlKey) {
                  window.open(`/project/${project?.id}`);
               } else {
                  navigate(`/project/${project?.id}`);
               }

            }}
            {...props}
            styles={{
               ...props.styles,
               ...itemStyles,
               root: {
                  backgroundColor: hordeTheme.horde.topNavBackground,
                  height: 30,
                  paddingLeft: 12,
                  paddingRight: 12,
                  margin: 0,
                  fontFamily: "Horde Raleway Regular",
                  fontSize: 12
               },
               label: {
                  color: hordeTheme.semanticColors.bodyText
               },
               menuIcon: {
                  display: "none"
               }
            }}
         />
      </Stack>
   );
};

type IProjectContextualMenuItem = IContextualMenuItem & {
   link?: string;
}

// individual menu items
const ProjectMenuItem: React.FunctionComponent<IContextualMenuItemProps> = props => {

   const { modeColors } = getHordeStyling();

   const item = props.item as IProjectContextualMenuItem;

   if (!item.link) {
      return <ContextualMenuItem {...props} />;
   }

   // Due to ContextualMenu implementation quirks, passing styles here doesn't work
   return <Link style={{ color: modeColors.text }} to={item.link} onClick={(ev: any) => { ev.stopPropagation(); return true; }}> <ContextualMenuItem onClick={(ev: any) => { ev.preventDefault(); return true; }} {...props} /></Link>;
};

const generateProjectMenu = (store: ProjectStore) => {

   const { modeColors } = getHordeStyling();

   const projects = store.projects;

   if (!projects || !projects.length) {
      return [];
   }

   const [, menuStyles] = getStyles();

   const cbProps: ICommandBarItemProps[] = [];

   projects.sort((a: ProjectData, b: ProjectData) => {
      return a.order - b.order;
   }).forEach(p => {

      // setup streams as sub items
      const subItems: IProjectContextualMenuItem[] = [];
      let useSubMenu = false;
      let numStreams = 0;

      if (p.categories) {

         const cats = p.categories.filter(c => c.showOnNavMenu);
         const track = new Set<string>();

         // count up streams
         cats.forEach(c => {

            if (track.has(c.name)) {
               return;
            }

            const streams = p.streams?.filter(s => c.streams.indexOf(s.id) !== -1);

            if (!streams || !streams.length) {
               return;
            }

            numStreams += streams.length;

            track.add(c.name);
         });

         const useSubMenu = numStreams >= 10;
         track.clear();

         cats.forEach(c => {

            if (track.has(c.name)) {
               return;
            }

            const streams = p.streams?.filter(s => c.streams.indexOf(s.id) !== -1);

            if (!streams || !streams.length) {
               return;
            }


            track.add(c.name);

            // style: { fontFamily: "Horde Open Sans SemiBold" },

            const catItem: IContextualMenuItem = {
               itemType: useSubMenu ? ContextualMenuItemType.Normal : ContextualMenuItemType.Section,
               text: useSubMenu ? c.name : undefined,
               key: `stream_category_${c.name}`,
               sectionProps: useSubMenu ? undefined : {
                  title: c.name,
                  items: [],
                  bottomDivider: true
               },
               subMenuProps: useSubMenu ? {
                  items: [],
                  contextualMenuItemAs: useSubMenu ? ProjectMenuItem : undefined
               } : undefined
            }

            const items = useSubMenu ? catItem.subMenuProps!.items : catItem.sectionProps!.items;

            streams.forEach(stream => {
               items.push(
                  {
                     style: { color: modeColors.text, fontSize: 12 },
                     key: stream.id, text: stream.name, data: { project: p, stream: stream }, link: `/stream/${stream.id}`
                  }
               );
            })

            subItems.push(catItem);

         })

         if (subItems.length) {
            subItems.push(
               {
                  key: `show_all_${p.id}`, text: "Show All", data: { project: p, stream: undefined }, link: `/project/${p.id}`
               }
            );
         }
      }

      if (!subItems.length) {

         p.streams?.forEach(stream => {
            subItems.push(
               {
                  key: stream.id, text: stream.name, data: { project: p, stream: stream }, link: `/stream/${stream.id}`
               }
            );
         });
      }

      // Collect all aliases from all streams
      const allAliases: IProjectContextualMenuItem[] = [];
      p.streams?.forEach(stream => {
         stream.aliases?.forEach(alias => {
            allAliases.push({
               style: { color: modeColors.text, fontSize: 12 },
               key: `${stream.id}_alias_${alias.name}`,
               text: getAliasDisplayName(alias.name),
               data: { project: p, stream: stream, alias: alias },
               link: getStreamLink(stream.id, alias.name)
            });
         });
      });

      if (allAliases.length > 0) {
         subItems.push({
            key: `aliases_${p.id}`,
            text: "Aliases",
            subMenuProps: {
               contextualMenuItemAs: ProjectMenuItem,
               items: allAliases
            }
         });
      }

      if (!subItems.length) {
         return;
      }

      const style = { ...menuStyles } as Partial<IContextualMenuStyles>;

      // project button
      const cbItem: ICommandBarItemProps = {
         key: p.id,
         text: p.name.toUpperCase(),
         data: {
            project: p

         },

         subMenuProps: {
            contextualMenuItemAs: useSubMenu ? undefined : ProjectMenuItem,
            subMenuHoverDelay: 0,
            styles: style,
            items: subItems
         }
      };

      cbProps.push(cbItem);
   });

   return cbProps;
};


const MenuButton: React.FC<{ props: IButtonProps, link?: string }> = ({ props, link }) => {

   const navigate = useNavigate();

   const buttonRef = React.createRef<IButton>();

   const hordeTheme = getHordeTheme();

   const [itemStyles] = getStyles();

   return (
      <Stack disableShrink={true}
         onMouseEnter={() => {
            buttonRef.current?.openMenu();
         }}
         onMouseLeave={() => {
            buttonRef.current?.dismissMenu();
         }} >
         <CommandBarButton
            componentRef={buttonRef}
            menuProps={{ subMenuHoverDelay: 0, items: [] }}

            onMouseUp={(ev => {
               if (!link) {
                  return;
               }
               if (ev?.button === 1) {
                  window.open(link);
               }
            })}

            onMenuClick={(ev) => {

               if (!link) {
                  return;
               }
               if (ev?.metaKey || ev?.ctrlKey) {
                  window.open(link);
               } else {
                  navigate(link);
               }
            }}
            {...props}
            styles={{
               ...props.styles,
               ...itemStyles,
               root: {
                  backgroundColor: hordeTheme.horde.topNavBackground,
                  color: hordeTheme.semanticColors.bodyText,
                  height: 30,
                  paddingLeft: 12,
                  paddingRight: 12,
                  margin: 0,
                  fontFamily: "Horde Raleway Regular",
                  fontSize: 12
               },
               menuIcon: {
                  display: "none"
               }
            }}
         />
      </Stack>
   );
};

const ToolsButton: React.FunctionComponent<IButtonProps> = (props) => {
   return <MenuButton props={props} link="/tools"/>
}


const HelpButton: React.FunctionComponent<IButtonProps> = (props) => {
   return <MenuButton props={props} link="/docs" />
}

const ServerButton: React.FunctionComponent<IButtonProps> = (props) => {
   return <MenuButton props={props} link="/serverstatus" />
}


// this module global is needed to enforce showing dialog ony once, in case component is recreated
let globalHasShown = true; // true, disabled
const RequestLogout: React.FC = observer(() => {

   const [hasShown, setHasShown] = useState(false);

   // subscribe
   if (dashboard.updated) { }

   const logoutURL = dashboard.authMethod === AuthMethod.Horde ? "/index" : "/account";

   if (!dashboard.requestLogout || hasShown || globalHasShown) {
      return null
   }

   const dialogContentProps = {
      type: DialogType.normal,
      title: 'New Features',
      subText: 'New features require logging out of Okta',
   };

   const modalProps = {
      isBlocking: true
   };

   const logout = async () => {
      try {
         console.log('Logging out of server');
         await backend.serverLogout(logoutURL);

      } catch (reason) {
         console.error(reason);
      }
   }

   return <Dialog
      hidden={false}
      onDismiss={() => { globalHasShown = true; setHasShown(true) }}
      modalProps={modalProps}
      dialogContentProps={dialogContentProps}

   >
      <Stack style={{ paddingTop: 8 }}>
         <DialogFooter>
            <PrimaryButton onClick={() => logout()} text="Logout" />
            <DefaultButton onClick={() => { globalHasShown = true; setHasShown(true); }} text="Cancel" />
         </DialogFooter>
      </Stack>
   </Dialog>

});



export const TopNav: React.FC<{ suppressServer?: boolean }> = observer(({ suppressServer }) => {

   const [showMenu, setShowMenu] = useState(false);
   const [showVersion, setShowVersion] = useState(false);
   const [showSupport, setShowSupport] = useState(false);
   const [showPreviewChanges, setShowPreviewChanges] = useState(false);
   const navigate = useNavigate();
   const { projectStore } = useBackend();

   const divRef = useRef(null);

   const query = new URLSearchParams(window.location.search);
   const showArtifacts = query.has("showArtifacts");

   // subscribe
   if (dashboard.updated) { }

   const logoutURL = dashboard.authMethod === AuthMethod.Horde ? "/index" : "/account";

   const hordeTheme = getHordeTheme();

   const generateToolsMenu = () => {

      const [, menuStyles] = getStyles();

      const cbProps: ICommandBarItemProps[] = [];

      const features = dashboard.user?.dashboardFeatures;

      let serviceItems: IContextualMenuItem[] = [];

      // get the plugins and their filtered mounts (based on user ACLs)
      const plugins = getHordePlugins();

      // Add plugin topnav mount points (filtered by user ACLs)
      plugins.forEach(p => {
         const mounts = getFilteredMounts(p).filter(m => m.type === MountType.TopNav);
         mounts.forEach(m => {
            serviceItems.push({
               key: `admin_plugin_${m.text}`,
               text: m.text,
               link: m.route
            });
         });
      })

      if (features?.showTests !== false) {
         serviceItems.push({
            key: "admin_tests",
            text: "Automation Hub (legacy)",
            link: `/automation`
         });
      }

      serviceItems.push({
         key: "job_artifacts",
         text: "Artifacts",
         onClick(ev, item) {

            const search = new URLSearchParams(window.location.search);
            search.append("showArtifacts", "true")

            const url = `${window.location.pathname}?` + search.toString();
            navigate(url, { replace: true })

         },
      });

      serviceItems.push({
         key: "build_health_dashboard",
         text: "Build Health",
         link: "/buildhealth"
      });

      serviceItems = serviceItems.sort((a, b) => {
         return (a.text ?? "").localeCompare(b.text ?? "")
      })

      // Always put downloads at the button for easy navigation
      serviceItems.push({
         key: "software_tools",
         text: "Downloads",
         link: `/tools`
      });

      const style = { ...menuStyles } as Partial<IContextualMenuStyles>;

      const cbItem: ICommandBarItemProps = {
         key: "tools_button",
         text: "TOOLS",
         subMenuProps: {
            contextualMenuItemAs: ProjectMenuItem,
            styles: style,
            items: serviceItems
         }
      };

      cbProps.push(cbItem);

      return cbProps;

   }

   const generateServerMenu = () => {

      const [, menuStyles] = getStyles();

      const cbProps: ICommandBarItemProps[] = [];

      const subItems: IContextualMenuItem[] = [];

      // Resources      
      const resourceItems: IContextualMenuItem[] = [];
      const monitoringItems: IContextualMenuItem[] = [];

      const features = dashboard.user?.dashboardFeatures;

      if (features?.showAgents === true) {

         resourceItems.push({
            key: "admin_agents",
            text: "Agents",
            link: `/agents`
         });

         if (features?.showAgentRegistration === true) {

            resourceItems.push({
               key: "admin_agents_registration",
               text: "Agent Enrollment",
               link: `/agents/registration`
            });

         }

      }

      if (features?.showDeviceManager === true) {
         resourceItems.push({
            key: "admin_devices",
            text: "Devices",
            link: `/devices`
         });
      }

      if (features?.showAgents === true) {
         resourceItems.push({
            key: "admin_pools",
            text: "Pools",
            link: `/pools`
         });
      }

      if (dashboard.hordeAdmin) {
         resourceItems.push({
            key: "admin_storage",
            text: "Storage",
            link: `/storage`
         });
      }


      if (features?.showAccounts) {
         resourceItems.push({
            key: "admin_user_accounts",
            text: "User Accounts",
            link: `/accounts`
         });
      }

      resourceItems.push({
         key: "admin_service_accounts",
         text: "Service Accounts",
         link: `/accounts/service`
      });

      if (resourceItems.length) {
         subItems.push({
            itemType: ContextualMenuItemType.Section,
            key: `admin_resources`,
            sectionProps: {
               title: "Resources",
               items: resourceItems,
               bottomDivider: true
            }
         });
      }

      // Monitoring

      if (features?.showAgents !== false) {
         monitoringItems.push({
            key: "admin_utilization",
            text: "Agent Utilization",
            link: `/reports/utilization`
         });
      }

      monitoringItems.push({
         key: "admin_notices",
         text: "Notices",
         link: `/notices`
      });

      if (features?.showPerforceServers !== false) {
         monitoringItems.push({
            key: "admin_perforce_servers",
            text: "Perforce Servers",
            link: `/perforce/servers`
         });
      }

      monitoringItems.push({
         key: "admin_serverstatus",
         text: "Status",
         link: `/serverstatus`
      });

      if (monitoringItems.length) {
         subItems.push({
            itemType: ContextualMenuItemType.Section,
            key: `admin_monitoring`,
            sectionProps: {
               title: "Monitoring",
               items: monitoringItems,
               bottomDivider: true
            }
         });
      }

      const style = { ...menuStyles } as Partial<IContextualMenuStyles>;

      if (dashboard.preview) {
         const previewItem: ICommandBarItemProps = {
            key: "preview_button",
            text: "CHANGES",
            onClick: () => { setShowPreviewChanges(true) }
         };
         cbProps.push(previewItem);
      }

      const cbItem: ICommandBarItemProps = {
         key: "admin_button",
         text: "SERVER",
         subMenuProps: {
            contextualMenuItemAs: ProjectMenuItem,
            styles: style,
            items: subItems
         }
      };

      cbProps.push(cbItem);

      return cbProps;
   }

   const generateHelpMenu = () => {

      const [, menuStyles] = getStyles();

      const cbProps: ICommandBarItemProps[] = [];

      const hordeItems: IContextualMenuItem[] = [];

      hordeItems.push({
         key: "server_docs",
         text: "Documentation",
         link: `/docs`
      });

      hordeItems.push({
         key: "server_api",
         text: "API Browser",
         href: `/swagger/index.html`
      });

      hordeItems.push({
         key: "server_docs_releasenotes",
         text: "Release Notes",
         link: `/docs/ReleaseNotes.md`
      });

      hordeItems.push({
         key: "server_versions",
         text: "Version",
         onClick: () => { setShowVersion(true) }
      });

      if(dashboard.helpEmail || dashboard.helpSlack){
         hordeItems.push({
            key: "support_contacts",
            text: "Support",
            onClick: () => { setShowSupport(true) }
         })
      }

      const style = { ...menuStyles } as Partial<IContextualMenuStyles>;

      const cbItem: ICommandBarItemProps = {
         key: "help_button",
         text: "HELP",
         subMenuProps: {
            contextualMenuItemAs: ProjectMenuItem,
            styles: style,
            items: hordeItems
         }
      };

      cbProps.push(cbItem);

      return cbProps;

   }

   function getInitials(name: string) {
      const nameArray = name.indexOf(".") === -1 ? name.split(" ") : name.split(".");
      if (nameArray.length === 1) {
         return name.toUpperCase().slice(0, 2);
      }
      const firstInitial = nameArray[0].charAt(0).toUpperCase();
      const lastInitial = nameArray[nameArray.length - 1].charAt(0).toUpperCase();
      return firstInitial + lastInitial;
   }

   let initials = "??";
   try {
      initials = getInitials(dashboard.username ?? "? ?")
   } catch (reason) {
      console.log(reason);
   }

   const menuItems: IContextualMenuItem[] = [
      {
         itemType: ContextualMenuItemType.Section,
         key: `user_section`,
         sectionProps: {
            items: [
               {
                  key: 'settingsItem',
                  text: 'Preferences',
                  href: `/dashboard`,
                  onClick: (ev: any) => {
                     ev.preventDefault();
                     navigate("/dashboard");
                  }
               }],
            bottomDivider: true
         }
      },
      {
         key: 'logoutItem',
         text: 'Sign Out',
         onClick: () => {

            const logout = async () => {
               console.log('Logging out of server');
               await backend.serverLogout(logoutURL);
            }
            logout();
         }
      }
   ];

   let showServer = true;//dashboard.hordeAdmin || dashboard.internalEmployee;

   // @todo: better no auth server config detection: https://jira.it.epicgames.com/browse/UE-119421
   if (dashboard.username === "Anonymous") {
      showServer = true;
   }

   if (suppressServer) {
      showServer = false;
   }

   let logoSrc = "/images/horde.svg";

   if (dashboard.development) {
      logoSrc = dashboard.darktheme ? "/images/horde_white.svg" : "/images/horde_black.svg";
   }

   return (
      <div style={{ backgroundColor: hordeTheme.horde.topNavBackground }}>
         {showArtifacts && <FindArtifactsModal onClose={() => {
            query.delete("showArtifacts");
            ArtifactQueryState.clearSearch(query);
            const url = `${window.location.pathname}?` + query.toString();
            navigate(url, { replace: true })

         }} />}
         {showVersion && <VersionModal show={true} onClose={() => { setShowVersion(false) }} />}
         {showSupport && <SupportModal show={true} onClose={() => {setShowSupport(false) }}/>}
         {showPreviewChanges && <PreviewChangesModal onClose={() => { setShowPreviewChanges(false) }} />}
         <Stack tokens={{ maxWidth: 1440, childrenGap: 0 }} disableShrink={true} styles={{ root: { backgroundColor: hordeTheme.horde.topNavBackground, margin: "auto", width: "100%" } }}>
            <Stack horizontal verticalAlign='center' styles={{ root: { height: "60px" } }} >

               <Link to="/index"><Stack horizontal styles={{ root: { paddingLeft: 0, cursor: 'pointer' } }}>
                  <Stack styles={{ root: { paddingTop: 2, paddingRight: 6 } }}>
                     <img style={{ width: 32 }} src={logoSrc} alt="" />
                  </Stack>
                  <Stack styles={{ root: { paddingTop: 11 } }}>
                     <Text styles={{ root: { fontFamily: "Horde Raleway Bold" } }}>HORDE{dashboard.preview ? " PREVIEW" : ""}</Text>
                  </Stack>
               </Stack>
               </Link>

               <RequestLogout />

               <Stack grow>
                  <CommandBar styles={{ root: { paddingTop: 17, backgroundColor: hordeTheme.horde.topNavBackground } }}
                     buttonAs={ProjectButton}
                     onReduceData={() => undefined}
                     items={generateProjectMenu(projectStore)}
                  />
               </Stack>

               <Stack grow />
               <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0 }}>
                  <Stack>
                     <CommandBar styles={{ root: { paddingTop: 17, paddingLeft: 0, paddingRight: 12, backgroundColor: hordeTheme.horde.topNavBackground } }}
                        buttonAs={ToolsButton}
                        onReduceData={() => undefined}
                        items={generateToolsMenu()}
                     />
                  </Stack>

                  <Stack>
                     {showServer && <CommandBar styles={{ root: { paddingTop: 17, paddingLeft: 0, paddingRight: 12, backgroundColor: hordeTheme.horde.topNavBackground } }}
                        buttonAs={ServerButton}
                        onReduceData={() => undefined}
                        items={generateServerMenu()}
                     />}
                  </Stack>

                  <Stack>
                     {showServer && <CommandBar styles={{ root: { paddingTop: 17, paddingLeft: 0, paddingRight: 32, backgroundColor: hordeTheme.horde.topNavBackground } }}
                        buttonAs={HelpButton}
                        onReduceData={() => undefined}
                        items={generateHelpMenu()}
                     />}
                  </Stack>

                  <Stack style={{ width: "32px" }} onMouseEnter={() => setShowMenu(true)}
                     onMouseLeave={() => { setShowMenu(false) }} >
                     <div ref={divRef}>
                        <Link to={"/dashboard"}>
                           <Persona styles={{ root: { selectors: { ".ms-Persona-initials": { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold", cursor: "pointer" } } } }} 
                              imageShouldFadeIn={false} imageInitials={initials} imageUrl={dashboard.userImage32} size={PersonaSize.size32} />
                        </Link>
                     </div>
                     <ContextualMenu
                        items={menuItems}
                        hidden={!showMenu}
                        target={divRef}
                        isBeakVisible={true}
                        gapSpace={0}
                        beakWidth={0}
                        onItemClick={() => { setShowMenu(false); }}
                        onDismiss={() => { setShowMenu(false); }}
                     />
                  </Stack>
               </Stack>


            </Stack>
         </Stack></div>);
});
