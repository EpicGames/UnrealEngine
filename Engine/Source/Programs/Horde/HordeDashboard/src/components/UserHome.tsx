// Copyright Epic Games, Inc. All Rights Reserved.

import { CollapseAllVisibility, DefaultButton, DetailsHeader, DetailsList, DetailsListLayoutMode, DetailsRow, FocusZone, FocusZoneDirection, FontIcon, IColumn, IconButton, IDetailsListProps, mergeStyleSets, PrimaryButton, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import moment from "moment-timezone";
import { default as React, useEffect, useState } from 'react';
import { Link, Navigate, useLocation, useNavigate } from "react-router-dom";
import backend, { useBackend } from "../backend";
import { GetBisectTaskResponse, GetIssueResponse, GetLabelStateResponse, GetStepResponse, JobData, JobQuery, JobState, JobStepOutcome, LabelOutcome, LabelState, ProjectData, StepData, StreamData } from "../backend/Api";
import dashboard, { StatusColor } from "../backend/Dashboard";
import { PollBase } from '../backend/PollBase';
import { useWindowSize } from '../base/utilities/hooks';
import { displayTimeZone, getElapsedString, getShortNiceTime } from '../base/utilities/timeUtils';
import { getLabelColor } from "../styles/colors";
import { getHordeStyling } from '../styles/Styles';
import { getHordeTheme } from '../styles/theme';
import { BisectionList } from './bisection/BisectionList';
import { Breadcrumbs } from './Breadcrumbs';
import { ChangeButton } from "./ChangeButton";
import { IssueModalV2 } from './IssueViewV2';
import { useQuery } from "horde/base/utilities/hooks";
import { JobOperationsContextMenu } from "./JobOperationsContextMenu";
import { PreflightConfigModal } from './preflights/PreflightConfigCheck';
import { IssueStatusIcon, StepStatusIcon } from "./StatusIcon";
import { TopNav } from './TopNav';

type JobItem = {
   key: string;
   job: JobData;
   startedby: string;
   stream?: StreamData;
};

const customStyles = mergeStyleSets({
   detailsRow: {
      selectors: {
         '.ms-DetailsRow': {
            borderBottom: '0px',
            width: "100%"
         },
         '.ms-DetailsRow-cell': {
            position: 'relative',
            textAlign: "center",
            padding: 0,
            overflow: "visible",
            whiteSpace: "nowrap"
         }
      }
   },
   header: {
      selectors: {
         ".ms-DetailsHeader-cellTitle": {
            padding: 0,
            paddingLeft: 4
         }
      }
   }

});

const homeWidth = 1400;

/// Projects

const ProjectsPanel: React.FC = observer(() => {

   const { projectStore } = useBackend();

   const theme = getHordeTheme();

   let projects = projectStore.projects.sort((a: ProjectData, b: ProjectData) => {
      return a.order - b.order;
   })

   projects = projects.slice(0, 3);

   /*
   if (!!dashboard.pinnedJobsIds.length) {
       projects = projects.slice(0, 3);
   }
   */
   
   return (
      <Stack style={{ width: homeWidth, marginLeft: 4 }}>
         <Stack tokens={{}} horizontal verticalFill wrap horizontalAlign='center' >
            {
               projects.map((project) => {
                  const logoSrc = `/api/v1/projects/${project.id}/logo${dashboard.darktheme ? "?darkTheme=true" : ""}`
                  return (
                     <Stack key={project.id} style={{
                        width: 560 * .65,
                        height: 280 * .65,
                        margin: '10px 20px',
                        backgroundColor: theme.horde.contentBackground,
                        boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)"
                     }}
                     >
                        <Link onClick={() => { projectStore.setActive(project.id); }} to={`/project/${project.id}`}>
                           <img src={logoSrc} alt={project.name + " logo button"} width={560 * .65} height={280 * .65} />
                        </Link>
                     </Stack>
                  );
               })
            }
         </Stack>
      </Stack>
   );
});


/// Health ========================================================================================

class HealthHandler {

   constructor() {
      makeObservable(this);
   }

   startPolling() {

      if (this.polling) {
         return;
      }

      this.polling = true;

      this.poll();

   }

   clear() {

      for (let i = 0; i < this.cancelID; i++) {
         this.canceled.add(i);
      }

      clearTimeout(this.timeoutID);

      this.updating = false;
      this.issues = [];
      this.polling = false;
      this.issues = undefined;

   }

   private async poll() {

      try {

         if (this.updating) {
            return;
         }

         clearTimeout(this.timeoutID);
         if (this.updateMS) {
            this.timeoutID = setTimeout(() => { this.poll(); }, this.updateMS);
         }

         this.updating = true;
         const cancelID = this.cancelID++;

         const values = await Promise.all([backend.getIssues({ ownerId: dashboard.userId, count: 1024, resolved: false })]);
         this.issues = values[0];

         if (this.canceled.has(cancelID)) {
            return;
         }

         this.updated();

      } catch (reason) {
         console.error(reason);
      } finally {
         this.updating = false;
      }

   }

   @observable
   update = 0


   @action
   private updated() {
      this.update++;
   }

   issues?: GetIssueResponse[];

   private updating = false;

   private updateMS = 10000;
   private timeoutID?: any;
   private canceled = new Set<number>();
   private cancelID = 0;
   private polling: boolean = false;

}


const handler = new HealthHandler();

type HealthItem = {
   issue: GetIssueResponse;
};

const HealthPanel: React.FC = observer(() => {

   const query = useQuery();
   const location = useLocation();
   const [issueHistory, setIssueHistory] = useState(false);
   const { hordeClasses } = getHordeStyling();

   // subscribe
   if (handler.update) { }

   let issues = handler.issues?.filter(issue => issue.ownerInfo?.id === dashboard.userId && !issue.resolvedAt).sort((a, b) => a.id - b.id);

   const items = issues?.map(i => { return { issue: i } });

   const columns: IColumn[] = [
      { key: 'health_column1', name: 'Summary', minWidth: 600, maxWidth: 600, isResizable: false },
      { key: 'health_column2', name: 'Workflow', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'health_column3', name: 'Status', minWidth: 320, maxWidth: 320, isResizable: false },
      { key: 'health_column4', name: 'Opened', minWidth: 100, maxWidth: 100, isResizable: false },
   ];

   const renderItem = (item: HealthItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      const issue = item.issue;

      let ack = issue.acknowledgedAt ? "Acknowledged" : "Unacknowledged";
      let status = "Unassigned";
      if (issue.ownerInfo) {
         status = issue.ownerInfo.name;
      }
      if (issue.resolvedAt) {
         status = "Resolved";
         if (issue.fixChange) {
            status += ` in CL ${issue.fixChange}`;
         }
      }

      if (status === "Unassigned" || status === "Resolved") {
         ack = "";
      }

      if (column.name === "Summary") {

         let summary = issue.summary;
         if (summary.length > 100) {
            summary = summary.slice(0, 100) + "...";
         }

         return <Stack style={{ padding: 8 }} horizontal disableShrink={true}>{<IssueStatusIcon issue={issue} />}<Text>{`Issue ${issue.id} - ${summary}`}</Text></Stack>;
      }

      if (column.name === "Status") {

         if (ack) {
            status += ` (${ack})`;
         }

         return <Stack style={{ padding: 8 }} horizontalAlign="end" disableShrink={true}><Text>{status}</Text></Stack>;
      }

      if (column.name === "Workflow") {
         if (!issue.workflowThreadUrl) {
            return null;
         }

         return <a onClick={(e) => e.stopPropagation()} style={{ fontSize: "13px" }} href={issue.workflowThreadUrl} target="_blank" rel="noreferrer"><Stack style={{ padding: 8 }} horizontalAlign="start" disableShrink={true}>Slack Thread</Stack></a>;
      }

      if (column.name === "Opened") {
         const openSince = `${getShortNiceTime(issue.createdAt)} (${getElapsedString(moment(issue.createdAt), moment.utc(), false).trim()})`;
         return <Stack style={{ padding: 8 }} horizontalAlign="end"><Text>{openSince}</Text></Stack>;
      }

      return <Stack />;
   }

   const classes = mergeStyleSets({
      detailsRow: {
         selectors: {
            '.ms-DetailsRow-cell': {
               overflow: "hidden",
               whiteSpace: "nowrap"
            }
         }
      }
   });

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as HealthItem;
         const newQuery = query.getCopy();
         newQuery.set("issue", item.issue.id.toString());
         const href = `${location.pathname}?${newQuery.toString()}`;

         return <Link to={href} className="job-item" onClick={() => { setIssueHistory(true) }}>
            <DetailsRow className={classes.detailsRow} {...props} />
         </Link>;

      }
      return null;
   };


   return (<Stack style={{ width: homeWidth, marginLeft: 4 }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <IssueModalV2 issueId={query.get("issue")} popHistoryOnClose={issueHistory} />
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Issues</Text>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "380px" }} data-is-scrollable={true}>
                  {!items && <Stack style={{ paddingBottom: 12 }} horizontal tokens={{ childrenGap: 12 }}><Text variant="medium">Loading issues</Text><Spinner size={SpinnerSize.small} /></Stack>}
                  {!!items && !items.length && <Stack style={{ paddingBottom: 12 }}><Text variant="medium">No issues found</Text></Stack>}
                  {!!items && !!items.length && <DetailsList
                     compact={true}
                     isHeaderVisible={false}
                     indentWidth={0}
                     items={items}
                     columns={columns}
                     setKey="set"
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.justified}
                     onRenderItemColumn={renderItem}
                     onRenderRow={renderRow}
                  />}
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});


/// Jobs ==========================================================================================

const jobsRefreshTime = 10000;
class UserJobsHandler {

   constructor() {
      makeObservable(this);
   }

   set(jobIds: string[]): boolean {

      if (jobIds.length === this.jobIds.length) {
         if (jobIds.every((v, i) => this.jobIds[i] === v)) {
            return false;
         }
      }

      this.clear();

      this.jobIds = jobIds;

      this.update();

      return true;
   }

   clear() {
      clearTimeout(this.timeoutId);
      this.timeoutId = undefined;
      this.modifiedAfter = undefined;
      this.jobs = [];
      this.jobIds = [];
      this.initial = true;

      // cancel any pending        
      for (let i = 0; i < this.cancelId; i++) {
         this.canceled.add(i);
      }

   }

   @action
   setUpdated() {
      this.updated++;
   }

   async update() {

      clearTimeout(this.timeoutId);
      this.timeoutId = setTimeout(() => { this.update(); }, jobsRefreshTime);

      if (!this.jobIds.length) {
         return;
      }

      if (this.updating) {
         return;
      }

      this.updating = true;

      // cancel any pending        
      for (let i = 0; i < this.cancelId; i++) {
         this.canceled.add(i);
      }

      const maxJobs = 100;

      try {
         let filter = "id,streamId,name,change,preflightChange,preflightCommitId,templateId,templateHash,graphHash,startedByUserInfo,createTime,state,arguments,updateTime,labels,defaultLabel,batches,autoSubmit,autoSubmitChange,preflightDescription,abortedByUserInfo,cancellationReason";

         const query: JobQuery = {
            filter: filter,
            count: maxJobs,
         };

         const cancelId = this.cancelId++;
         const queryTime = moment.utc().toISOString();

         const jobIds = this.jobIds.slice(-maxJobs);

         const mjobs = await backend.getJobsByIds(jobIds, query);

         // check for canceled after modified test
         if (this.canceled.has(cancelId)) {
            return;
         }

         const jobs: JobData[] = [];

         mjobs.forEach(j1 => {
            jobs.push(j1);
         })

         jobs.push(...this.jobs.filter(j1 => !jobs.find(j2 => j1.id === j2.id)));

         this.jobs = jobs.sort((a, b) => {
            const timeA = new Date(a.createTime).getTime();
            const timeB = new Date(b.createTime).getTime();
            if (timeA === timeB) return 0;
            return timeA < timeB ? 1 : -1;
         });


         // check for canceled during graph request
         if (this.canceled.has(cancelId)) {
            return;
         }

         if (mjobs.length || this.initial) {
            this.initial = false;
            this.setUpdated();
         }

         this.modifiedAfter = queryTime;


      } catch (reason) {
         console.log(reason);
      } finally {
         this.updating = false;
      }

   }

   initial = true;

   jobs: JobData[] = [];

   jobIds: string[] = [];

   @observable
   updated: number = 0;

   private modifiedAfter?: string;

   private timeoutId: any;

   private canceled = new Set<number>();
   private cancelId = 0;

   updating = false;

}

const JobsPanel: React.FC<{ includeOtherPreflights: boolean }> = observer(({ includeOtherPreflights }) => {

   const [jobHandler] = useState(new UserJobsHandler())

   const { projectStore } = useBackend();

   useEffect(() => {

      dashboard.startPolling();

      return () => {
         dashboard.stopPolling();
         jobHandler.clear();
      };

   }, [jobHandler]);

   const { hordeClasses, detailClasses, modeColors } = getHordeStyling();


   // subscribe
   if (jobHandler.updated) { }
   if (dashboard.updated) { }

   jobHandler.set(dashboard.pinnedJobsIds);

   let jobs: JobData[] = jobHandler.jobs;

   let jobItems: JobItem[] = jobs.map(j => {

      let startedBy = j.startedByUserInfo?.name ?? "Scheduler";
      if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
         startedBy = "TestAccount";
      }

      // note: stream can be undefined here if it has been removed, we still want to present pinned jobs to users (as no other way to unpin them currently)
      const stream = projectStore.streamById(j.streamId);

      return {
         key: j.id,
         job: j,
         startedby: startedBy,
         stream: stream
      } as JobItem;

   });


   let columns: IColumn[] = [
      { key: 'jobview_column1', name: 'Status', minWidth: 16, maxWidth: 16 },
      { key: 'jobview_column2', name: 'Change', minWidth: 114, maxWidth: 114 },
      { key: 'jobview_column3', name: 'Submit', minWidth: 130, maxWidth: 130 },
      { key: 'jobview_column4', name: 'Labels', minWidth: 320, maxWidth: 320 },
      { key: 'jobview_column5', name: 'Steps', minWidth: 260, maxWidth: 260 },
      { key: 'jobview_column6', name: 'StartedBy', minWidth: 140, maxWidth: 140 },
      { key: 'jobview_column7', name: 'Time', minWidth: 130, maxWidth: 130 },
      { key: 'jobview_column8', name: 'Dismiss', minWidth: 32, maxWidth: 32 },

   ];

   columns = columns.map(c => {

      c.isResizable = true;
      c.isPadded = false;
      c.isMultiline = true;
      c.isCollapsible = false;
      c.headerClassName = customStyles.header;

      c.styles = (props: any): any => {
         props.cellStyleProps = { ...props.cellStyleProps };
         props.cellStyleProps.cellLeftPadding = 4;
         props.cellStyleProps.cellRightPadding = 0;
      };

      return c;

   })

   const renderSteps = (job: JobData) => {

      type StepItem = {
         step: StepData;
         name: string;
      }

      const jobId = job.id;

      if (!job.batches) {
         return null;
      }

      let steps: GetStepResponse[] = job.batches.map(b => b.steps).flat().filter(step => !!step.startTime && (step.outcome === JobStepOutcome.Warnings || step.outcome === JobStepOutcome.Failure));

      if (!steps || !steps.length) {
         return null;
      }

      const onRenderCell = (stepItem: StepItem): JSX.Element => {

         const step = stepItem.step;

         const stepUrl = `/job/${jobId}?step=${step.id}`;

         return <Stack tokens={{ childrenGap: 12 }} key={`step_${step.id}_job_${job.id}_${stepItem.name}`}>
            <Stack horizontal>
               <Link to={stepUrl} onClick={(ev) => { ev.stopPropagation(); }}><div style={{ cursor: "pointer" }}>
                  <Stack horizontal>
                     <StepStatusIcon step={step} style={{ fontSize: 10 }} />
                     <Text styles={{ root: { fontSize: 10, paddingRight: 4, userSelect: "none" } }}>{`${stepItem.name}`}</Text>
                  </Stack>
               </div></Link>
            </Stack>
         </Stack>;
      };

      const andMore = (errors: number, warnings: number): JSX.Element => {

         return <Stack tokens={{ childrenGap: 12 }} key={`job_${job.id}_andmore`}>
            <Stack horizontal>
               <Stack horizontal>
                  <Link to={`/job/${jobId}`} onClick={(ev) => { ev.stopPropagation(); }}><div style={{ cursor: "pointer" }}>
                     <Text styles={{ root: { fontSize: 10, paddingRight: 4, paddingLeft: 19, userSelect: "none" } }}>{`( +${errors + warnings} more )`}</Text>
                  </div></Link>
               </Stack>
            </Stack>
         </Stack>;
      };

      let items = steps.map(step => {
         return {
            step: step,
            name: step.name
         }
      })

      items = items.filter(item => !!item);

      items = items.sort((a, b) => {

         const stepA = a!.step;
         const stepB = b!.step;

         const outA = stepA.outcome;
         const outB = stepB.outcome;

         if (outA === JobStepOutcome.Failure && outB === JobStepOutcome.Warnings) {
            return -1;
         }

         if (outA === JobStepOutcome.Warnings && outB === JobStepOutcome.Failure) {
            return 1;
         }

         return a!.name < b!.name ? -1 : 1;

      });

      let numErrors = 0;
      let numWarnings = 0;

      items.forEach(i => {
         if (i?.step.outcome === JobStepOutcome.Warnings) {
            numWarnings++;
         } else {
            numErrors++;
         }
      });

      items = items.slice(0, 10);

      const render = items.map(s => onRenderCell(s!));

      let shownErrors = 0;
      let shownWarnings = 0;

      items.forEach(i => {
         if (i?.step.outcome === JobStepOutcome.Warnings) {
            shownWarnings++;
         } else {
            shownErrors++;
         }
      });

      if (shownErrors !== numErrors || numWarnings !== shownWarnings) {
         render.push(andMore(numErrors - shownErrors, numWarnings - shownWarnings));
      }

      return render;

   }

   const renderItemInner = (item: JobItem, index?: number, column?: IColumn) => {

      const job = item.job;

      const style: any = {
         margin: 0,
         position: "absolute",
         top: "50%",
         left: "50%",
         msTransform: "translate(-50%, -50%)",
         transform: "translate(-50%, -50%)"
      };

      if (column!.name === "Space") {
         return <div />
      }

      if (column!.name === "Labels") {
         return <Stack verticalAlign="center" horizontalAlign="start" verticalFill={true}>{renderLabels(item)}</Stack>;

      }

      if (column!.name === "StartedBy") {
         let startedBy = job.startedByUserInfo?.name ?? "Scheduler";
         if (startedBy.toLowerCase().indexOf("testaccount") !== -1) {
            startedBy = "TestAccount";
         }

         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign="center"><Text variant="small">{startedBy}</Text></Stack>

      }
      if (column!.name === "Steps") {
         return <Stack verticalAlign="center" verticalFill={true} style={{ paddingLeft: 32 }}>{renderSteps(job)}</Stack>;
      }

      if (column!.name === "Change") {

         return <Stack verticalAlign="center" verticalFill={true} tokens={{ childrenGap: 4 }}>
            <Stack horizontalAlign="start" style={{ whiteSpace: "normal" }}><Text style={{ fontSize: 13, fontFamily: "Horde Open Sans SemiBold", whiteSpace: "pre" }}>{item.job.name}</Text></Stack>
            {!!item.stream && <Stack horizontalAlign="start" style={{ whiteSpace: "normal" }}><Text style={{ fontSize: 11, fontFamily: "Horde Open Sans SemiBold", whiteSpace: "pre" }}>{`//${item.stream.project!.name}/${item.stream.name}`}</Text></Stack>}
            <Stack><ChangeButton job={job} pinned={true} /></Stack>
         </Stack>

      }

      if (column!.name === "Submit") {

         if (!job.autoSubmit) {
            return null;
         }

         let message = "";
         let url = "";

         const colors = dashboard.getStatusColors();
         let color = colors.get(StatusColor.Ready);

         if (job.state !== JobState.Complete) {
            message = "Submit Pending";
            color = colors.get(StatusColor.Running);
         } else {

            if (!job.autoSubmitChange) {
               message = "Submit Failed";
               color = colors.get(StatusColor.Failure);
            } else {
               color = colors.get(StatusColor.Success);
               message = `Submitted\nCL ${job.autoSubmitChange}`;
               if (dashboard.swarmUrl) {
                  url = `${dashboard.swarmUrl}/change/${job.autoSubmitChange}`;
               }
            }
         }

         if (!message) {
            return null;
         }

         const style = { fontSize: 13, color: color, paddingTop: 2 };

         const icon = <Stack className="horde-no-darktheme"><FontIcon style={style} iconName="Square" /></Stack>;

         return <Stack horizontalAlign="start" verticalAlign="center" verticalFill={true} tokens={{ childrenGap: 12 }}>
            <Stack horizontal verticalAlign='center' verticalFill={true} tokens={{ childrenGap: 6 }}>
               {!url && <Stack horizontal verticalAlign='center' verticalFill={true} tokens={{ childrenGap: 8 }}>{icon}<Stack><Text style={{ fontSize: 11 }}>{message}</Text></Stack></Stack>}
               {!!url && <Stack horizontal verticalAlign='center' verticalFill={true} tokens={{ childrenGap: 8 }}>{icon}<Stack style={{ textAlign: "left" }}><a href={url} target="_blank" rel="noreferrer" onClick={ev => ev?.stopPropagation()}><Text variant="tiny" style={{ whiteSpace: "pre" }}>{message}</Text></a></Stack></Stack>}
            </Stack>
         </Stack>
      }

      if (column!.name === "Time") {

         const displayTime = moment(item.job!.createTime).tz(displayTimeZone());

         const format = dashboard.display24HourClock ? "MMM Do, HH:mm z" : "MMM Do, h:mma z";

         let displayTimeStr = displayTime.format(format);

         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"}><Text variant="small">{displayTimeStr}</Text></Stack>;

      }

      if (column!.name === "Dismiss") {
         return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"} onClick={(ev) => {
            ev.preventDefault();
            ev.stopPropagation();
            dashboard.unpinJob(job.id);
         }}>
            <IconButton iconProps={{ iconName: 'Pin' }} />
         </Stack>

      }

      if (column!.name === "Status") {
         return <div />;
      }

      return <Stack style={style} horizontalAlign="center"><Text variant="small">{item.job.state}</Text></Stack>
   }

   const renderItem = (item: JobItem, index?: number, column?: IColumn) => {

      if (column?.name === "Change") {
         return renderItemInner(item, index, column);
      } else {
         return <Link to={`/job/${(item as JobItem).job.id}`}>{renderItemInner(item, index, column)}</Link>;
      }
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = jobItems[props.itemIndex];

         let background: string | undefined;
         if (props.itemIndex % 2 === 0) {
            background = dashboard.darktheme ? "#1D2021" : "#FAF9F9";
         }

         return <JobOperationsContextMenu job={item.job}>
            <DetailsRow styles={{ root: { paddingTop: 8, paddingBottom: 8, backgroundColor: background }, cell: { selectors: { "a, a:visited, a:activem, a:hover": { color: modeColors.text } } } }} {...props} />
         </JobOperationsContextMenu>
      }
      return null;
   };

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      if (props) {
         props.selectionMode = SelectionMode.none;
         props.collapseAllVisibility = CollapseAllVisibility.hidden;
         return <DetailsHeader className={detailClasses.detailsHeader}  {...props} styles={{ root: {} }} />

      }
      return null;
   };


   const JobLabel: React.FC<{ item: JobItem; label: GetLabelStateResponse }> = ({ item, label }) => {

      const defaultLabel = item.job.defaultLabel;
      const aggregates = item.job.labels;

      // note details may not be loaded here, as only initialized on callout for optimization (details.getLabelIndex(label.Name, label.Category);)
      const jlabel = aggregates?.find((l, idx) => l.dashboardCategory === label.dashboardCategory && l.dashboardName === label.dashboardName && item.job.labels![idx]?.state !== LabelState.Unspecified);
      let labelIdx = -1;
      if (jlabel) {
         labelIdx = aggregates?.indexOf(jlabel)!;
      }

      let state: LabelState | undefined;
      let outcome: LabelOutcome | undefined;
      if (label === defaultLabel) {
         state = defaultLabel.state;
         outcome = defaultLabel.outcome;
      } else {
         state = item.job.labels![labelIdx]?.state;
         outcome = item.job.labels![labelIdx]?.outcome;
      }

      const color = getLabelColor(state, outcome);

      let url = `/job/${item.job.id}`;

      if (labelIdx >= 0) {
         url = `/job/${item.job.id}?label=${labelIdx}`;
      }

      const target = `label_${item.job.id}_${label.dashboardName}_${label.dashboardCategory}`.replace(/ /g, "");

      return <Stack>
         <div id={target} className="horde-no-darktheme">
            <Link to={url}><Stack className={hordeClasses.badgeNoIcon}><DefaultButton key={label.dashboardName} style={{ backgroundColor: color.primaryColor, fontSize: 9 }} text={label.dashboardName}>
               {!!color.secondaryColor && <div style={{
                  borderLeft: "10px solid transparent",
                  borderRight: `10px solid ${color.secondaryColor}`,
                  borderBottom: "10px solid transparent",
                  height: 0,
                  width: 0,
                  position: "absolute",
                  right: 0,
                  top: 0,
                  zIndex: 1
               }} />}
            </DefaultButton>
            </Stack>
            </Link>
         </div>
      </Stack>;
   };

   const renderLabels = (item: JobItem): JSX.Element => {

      const job = item.job!;
      const defaultLabel = job.defaultLabel;

      const labels = job.labels ?? [];

      if (!labels.length && !defaultLabel) {
         return <div />;
      }

      const view = labels.filter(a => {
         const idx = labels.indexOf(a)!;
         if (idx === -1 || !job.labels?.length) {
            return false;
         }
         return job.labels[idx].state !== LabelState.Unspecified;
      });

      const catergories = new Set<string>();
      view.forEach(label => { if (label.dashboardCategory?.length) { catergories.add(label.dashboardCategory) } });

      const sorted = Array.from(catergories).sort((a, b) => { return a < b ? -1 : 1 });

      if (!catergories.has("Other") && defaultLabel) {

         defaultLabel.dashboardCategory = defaultLabel.dashboardName = "Other";
         view.push(defaultLabel);
         sorted.push("Other");
      }

      let key = 0;

      const labelStacks = sorted.map(cat => {

         const buttons = view.filter(label => label.dashboardCategory === cat).map(label => {
            return <JobLabel key={`label_${item.job.id}_${label.dashboardName}_${key++}`} item={item} label={label} />;
         });

         if (!buttons.length) {
            return <div />;
         }

         return <Stack tokens={{ childrenGap: 8 }}>
            <Stack horizontal tokens={{ childrenGap: 8 }}>
               <Stack horizontalAlign="end" style={{ minWidth: 80, maxWidth: 80 }}>
                  <Text variant="small" style={{}}>{cat}:</Text>
               </Stack>
               <Stack wrap horizontal horizontalAlign="start" tokens={{ childrenGap: 4 }}>
                  {buttons}
               </Stack>
            </Stack>
         </Stack>
      })


      return (
         <Stack tokens={{ childrenGap: 4 }} styles={{ root: { paddingTop: 2 } }}>
            {labelStacks}
         </Stack>

      );

   };

   return (
      <Stack style={{ width: homeWidth, marginLeft: 4 }} className={customStyles.detailsRow}>
         <Stack className={hordeClasses.raised}>
            <Stack tokens={{ childrenGap: 12 }}>
               <Stack horizontal>
                  <Stack>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Pinned Jobs</Text>
                  </Stack>
                  <Stack grow />
                  <Stack>

                     {!!jobItems.length && <PrimaryButton text="Clear All" style={{ fontFamily: "Horde Open Sans SemiBold", fontSize: 10, padding: 8, height: "24px", minWidth: "48px" }} onClick={() => { dashboard.clearPinnedJobs() }} />}
                  </Stack>
               </Stack>
               <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
                  {!!dashboard.pinnedJobsIds.length && !jobItems.length && <Stack><Spinner size={SpinnerSize.large} /></Stack>}
                  {!!jobItems.length && <DetailsList
                     styles={{ root: { paddingLeft: 8, paddingRight: 8, marginBottom: 18 } }}
                     compact={true}
                     isHeaderVisible={false}
                     indentWidth={0}
                     items={jobItems}
                     columns={columns}
                     setKey="set"
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.fixedColumns}
                     onRenderDetailsHeader={onRenderDetailsHeader}
                     onRenderItemColumn={renderItem}
                     onRenderRow={renderRow}
                     onShouldVirtualize={() => true}
                  />}
                  {!dashboard.pinnedJobsIds.length && <Stack style={{ paddingBottom: 12 }}><Text variant="medium">No jobs are pinned</Text></Stack>}
               </Stack>
            </Stack>
         </Stack>
      </Stack>

   );

});

// Bisections

class BisectionHandler extends PollBase {

   constructor(pollTime = 5000) {

      super(pollTime);

   }

   clear() {
      super.stop();
   }

   async poll(): Promise<void> {

      try {

         if (dashboard.pinnedBisectTaskIds?.length) {
            this.bisections = await backend.getBisections({ id: dashboard.pinnedBisectTaskIds });
         } else {
            this.bisections = [];
         }

         this.setUpdated();

      } catch (err) {

      }

   }

   bisections: GetBisectTaskResponse[] = [];
}

const bisectionHandler = new BisectionHandler();

const BisectionPanel: React.FC = observer(() => {

   // Bisection is currently disabled via feature flag
   return null;

   const { hordeClasses } = getHordeStyling();

   useEffect(() => {

      bisectionHandler.start();

      return () => {
         bisectionHandler.clear();
      };

   }, []);

   // subscribe
   dashboard.subscribe();

   if (bisectionHandler.updated) { };

   if (bisectionHandler.bisections.length === 0) {
      return null;
   }

   return (<Stack style={{ width: homeWidth, marginLeft: 4 }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Bisections</Text>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 12 } }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "384px" }} data-is-scrollable={true}>
                  <BisectionList bisections={bisectionHandler.bisections} />
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);

});

const UserHomeViewInner: React.FC = () => {

   const search = new URLSearchParams(window.location.search);
   const change = !search.get("preflightconfig") ? "" : search.get("preflightconfig")!;

   const navigate = useNavigate();


   useEffect(() => {

      return () => {
         handler.clear();
      };

   }, []);

   const { detailClasses, modeColors } = getHordeStyling();

   handler.startPolling();

   return <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, margin: 0, paddingTop: 8 } }}>
      {(!!change || !!search.has("preflightconfig")) && <PreflightConfigModal onClose={() => { navigate("/index", { replace: true }) }} />}
      <Stack style={{ padding: 0 }} className={detailClasses.detailsRow}>
         <FocusZone direction={FocusZoneDirection.vertical} style={{ padding: 0 }}>
            <div className={detailClasses.container} style={{ width: "100%", height: 'calc(100vh - 192px)', position: 'relative' }} data-is-scrollable={true}>
               <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto} onScroll={() => { }}>
                  <Stack tokens={{ childrenGap: 18 }} style={{ padding: 0 }}>
                     <ProjectsPanel />
                     <BisectionPanel />
                     <HealthPanel />
                     <JobsPanel includeOtherPreflights={false} />
                  </Stack>
               </ScrollablePane>
            </div>
         </FocusZone>
      </Stack>
   </Stack>
};

export const UserHomeView: React.FC = () => {

   const windowSize = useWindowSize();

   if (dashboard.user?.dashboardFeatures?.showLandingPage) {
      return <Navigate to="/docs/Landing.md" replace={true} />
   }

   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <Breadcrumbs items={[{ text: 'Home' }]} />
         <Stack horizontal>
            <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
            <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
               <Stack style={{ maxWidth: 1440, paddingTop: 6, marginLeft: 4 }}>
                  <UserHomeViewInner />
               </Stack>
            </Stack>
         </Stack>
      </Stack>);

}
