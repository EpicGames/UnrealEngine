// Copyright Epic Games, Inc. All Rights Reserved.

import { CompactPeoplePicker, IBasePickerSuggestionsProps, IInputProps, IPersonaProps, Label, Stack } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import { useWindowSize } from "../base/utilities/hooks";
import { Breadcrumbs } from "./Breadcrumbs";
import { TopNav } from "./TopNav";
import userCache from "../backend/UserCache";
import backend from "../backend";
import { GetUserResponse } from "../backend/Api";
import { getHordeStyling } from "../styles/Styles";

const loadedUsers: Map<string, GetUserResponse> = new Map();

export const UserSelect: React.FC<{ handleSelection: (userId: string | undefined) => void, userIdHints?: string[], noResultsFoundText?: string, defaultUser?: GetUserResponse, autoFocus?: boolean, defaultId?: string }> = observer(({ handleSelection, userIdHints, noResultsFoundText, defaultUser, autoFocus, defaultId }) => {
   const [selectedItems, setSelectedItems] = useState<IPersonaProps[]>([]);

   const loadUser = (user: GetUserResponse) => {
      if(!user) return;

      loadedUsers.set(user.id, user);
      
      const persona = {
         text: user.name,
         imageUrl: user.image24,
         optionalText: user.id,
      } as IPersonaProps;

      setSelectedItems([persona]);
      handleSelection(user.id);
   }

   const loadUserFromId = async (userId: string) => {
      if(!userId) return;

      const users = await backend.getUsers({ids: [userId], includeAvatar: true});
      if(users.length) loadUser(users[0]);
   }

   useEffect(() => {
      if(defaultUser) {
         loadUser(defaultUser);
      } else if(defaultId) {
         loadUserFromId(defaultId);
      } else {
         setSelectedItems([])
         handleSelection(undefined);
      }
   }, [defaultUser, defaultId]);

   const suggestionProps: IBasePickerSuggestionsProps = {
      suggestionsHeaderText: '',
      mostRecentlyUsedHeaderText: '',
      noResultsFoundText: noResultsFoundText ? noResultsFoundText : 'No results found',
      loadingText: 'Loading',
      showRemoveButtons: false
   };

   const inputProps: IInputProps = { 
      autoFocus: autoFocus
   };

   // subscribe
   if (userCache.updated) { }

   const onFilterChanged = async (filter: string) => {

      if (!filter.length) return [];

      let users = await userCache.getUsers(filter);

      users = users.filter(u => u.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1)

      return users.map(u => {
         return {
            text: u.name,
            imageUrl: u.image24,
            optionalText: u.id // hack, no way to add custom data to persona props
         } as IPersonaProps
      })
   }

   const onEmptyResolveSuggestions = async () => {

      if (!userIdHints?.length) {
         return [];
      }

      const newIds = userIdHints.filter(id => !loadedUsers.get(id));

      if (newIds.length) {

         const users = await backend.getUsers({
            ids: newIds,
            includeAvatar: true
         });

         users.forEach(u => {
            loadedUsers.set(u.id, u);
         });

      }

      let defaultUsers = userIdHints.map(id => loadedUsers.get(id)!).sort((a, b) => {

         const aname = a.name.replaceAll(".", "");
         const bname = b.name.replaceAll(".", "");

         if (aname < bname) {
            return -1;
         }
         if (aname > bname) {
            return 1;
         }
         return 0;
      });

      defaultUsers = defaultUsers.filter(u => u.name.toLowerCase() !== "buildmachine" && u.name.toLowerCase() !== "svc-p4-hordeproxy-p");

      return defaultUsers.map(u => {
         return {
            text: u.name,
            imageUrl: u.image24,
            optionalText: u.id // hack, no way to add custom data to persona props
         } as IPersonaProps
      })
   }

   return <Stack style={{ width: 320 }}>
      <CompactPeoplePicker
         itemLimit={1}
         // eslint-disable-next-line react/jsx-no-bind
         onResolveSuggestions={onFilterChanged}
         // eslint-disable-next-line react/jsx-no-bind
         onEmptyResolveSuggestions={onEmptyResolveSuggestions}
         pickerSuggestionsProps={suggestionProps}
         className={'ms-PeoplePicker'}
         selectedItems={selectedItems}
         inputProps={inputProps}
         onChange={(items) => {
            setSelectedItems(items ?? []);
            if (items && items?.length) {
               handleSelection(items[0].optionalText!);
            } else {
               handleSelection(undefined);
            }
         }}

         onInputChange={(input) => {
            return input.replaceAll(".", " ");
         }}

         resolveDelay={1000}
      />
   </Stack>
});


export const UserSelectTestView: React.FC = () => {

   const { hordeClasses } = getHordeStyling();

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Tests' }]} />
      <Stack horizontal>
         <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - 720, flexShrink: 0, backgroundColor: 'rgb(250, 249, 249)' }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: 'rgb(250, 249, 249)', width: "100%" } }}>
            <Stack style={{ maxWidth: 1420, paddingTop: 6, marginLeft: 4, height: 'calc(100vh - 8px)' }}>
               <Stack horizontal className={hordeClasses.raised}>
                  <Stack style={{ width: "100%", height: 'calc(100vh - 228px)' }} tokens={{ childrenGap: 18 }}>
                     <Label>Test User Selection Input</Label>
                     <UserSelect handleSelection={(id) => console.log(id)} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};


