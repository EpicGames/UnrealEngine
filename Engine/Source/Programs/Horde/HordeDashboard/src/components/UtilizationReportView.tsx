// Copyright Epic Games, Inc. All Rights Reserved.

import { DatePicker, Dropdown, DropdownMenuItemType, IDatePickerStrings, IDropdownOption, IDropdownStyles, ISliderStyles, mergeStyleSets, Slider, Stack, Callout, DirectionalHint } from '@fluentui/react';
import { IVerticalStackedChartProps, IVSChartDataPoint, VerticalStackedBarChart } from '@fluentui/react-charting';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useState } from 'react';
import { useNavigate } from 'react-router-dom';
import backend from '../backend';
import { GetUtilizationTelemetryResponse } from '../backend/Api';
import { useWindowSize } from '../base/utilities/hooks';
import { Breadcrumbs } from './Breadcrumbs';
import { useQuery } from "horde/base/utilities/hooks";
import { TopNav } from './TopNav';
import { getHordeStyling } from '../styles/Styles';

// state object for the report
type ChartOptionsState = {
   updating: boolean;
   rawData: GetUtilizationTelemetryResponse[] | null;
   chartData: IVerticalStackedChartProps[] | null;
   selectedDate: Date;
   selectedPoolKeys: string[];
   selectedRange: number;
   selectedFormat: number;
   poolDropdownOptions: IDropdownOption[];
   poolDropdownKeys: { [category: string]: string[] }
   yMaxValue: number;
   noData: boolean;
}

// how far to fudge y-axis values in the svg text for the x axis
const yValuesCustomText = [
   '.4em',
   '1.6em'
];

type IntervalOptions = {
   bucketInterval: number;
   hourStrings: { [key: number]: string };
   barWidth: number;
   rangeInterval: { [key: number]: number }
}

type UpdateStateOptions = {
   selectedDate?: Date | null | undefined;
   selectedPools?: (IDropdownOption | undefined)[] | null | undefined;
   selectedRange?: number | null | undefined;
   selectedFormat?: number | null | undefined;
}

type PinnedCalloutState = {
   legend: string;
   data: number;
   color: string;
   yAxisCalloutData: string;
   x: number,
   y: number
}

const intervalToOptions: { [key: number]: IntervalOptions } = {
   1: {
      bucketInterval: 1,
      rangeInterval: { 1: 1, 2: 1, 3: 2, 4: 3, 5: 3, 6: 3 },
      hourStrings: {
         0: "12am", 1: "1am", 2: "2am", 3: "3am", 4: "4am", 5: "5am",
         6: "6am", 7: "7am", 8: "8am", 9: "9am", 10: "10am", 11: "11am",
         12: "12pm", 13: "1pm", 14: "2pm", 15: "3pm", 16: "4pm", 17: "5pm",
         18: "6pm", 19: "7pm", 20: "8pm", 21: "9pm", 22: "10pm", 23: "11pm"
      },
      barWidth: 32
   },
   2: {
      bucketInterval: 2,
      rangeInterval: { 1: 2, 2: 2, 3: 2, 4: 3, 5: 3, 6: 4 },
      hourStrings: {
         0: "12am-2am", 1: "2am-4am", 2: "4am-6am", 3: "6am-8am",
         4: "8am-10am", 5: "10am-12pm", 6: "12pm-2pm", 7: "2pm-4pm",
         8: "4pm-6pm", 9: "6pm-8pm", 10: "8pm-10pm", 11: "10pm-12am",
      },
      barWidth: 64
   },
   3: {
      bucketInterval: 4,
      rangeInterval: { 1: 1, 2: 1, 3: 1, 4: 2, 5: 2, 6: 2 },
      hourStrings: { 0: "12am-4am", 1: "4am-8am", 2: "8am-12pm", 3: "12pm-4pm", 4: "4pm-8pm", 5: "8pm-12am", },
      barWidth: 128
   },
   4: {
      bucketInterval: 6,
      rangeInterval: { 1: 1, 2: 1, 3: 1, 4: 1, 5: 1, 6: 2 },
      hourStrings: { 0: "12am-6am", 1: "6am-12pm", 2: "12pm-6pm", 3: "6pm-12am", },
      barWidth: 192
   },
   5: {
      bucketInterval: 12,
      rangeInterval: { 1: 3, 2: 3, 3: 3, 4: 1, 5: 1, 6: 1 },
      hourStrings: { 0: "12am-12pm", 1: "12pm-12am", },
      barWidth: 384
   },
   6: {
      bucketInterval: 24,
      rangeInterval: { 1: 4, 2: 6, 3: 3, 4: 2, 5: 1, 6: 1 },
      hourStrings: { 0: "12am-12am" },
      barWidth: 768
   }
};

// strings for the calendar widget
const DayPickerStrings: IDatePickerStrings = {
   months: [
      'January',
      'February',
      'March',
      'April',
      'May',
      'June',
      'July',
      'August',
      'September',
      'October',
      'November',
      'December',
   ],
   shortMonths: ['Jan', 'Feb', 'Mar', 'Apr', 'May', 'Jun', 'Jul', 'Aug', 'Sep', 'Oct', 'Nov', 'Dec'],
   days: ['Sunday', 'Monday', 'Tuesday', 'Wednesday', 'Thursday', 'Friday', 'Saturday'],
   shortDays: ['S', 'M', 'T', 'W', 'T', 'F', 'S'],
   goToToday: 'Go to today'
};

const colorSteps = [
   "#FF00F2",
   "#5AC95A",
   "#FF6600",
   "#DF8BE5",
   "#00BCF2",
];

let colorIndex = 0;
const poolColors = new Map<string, string>();
function getPoolColor(pool: string): string {

   let color = poolColors.get(pool.toLowerCase());
   if (color) {
      return color;
   }

   color = colorSteps[colorIndex]
   poolColors.set(pool.toLowerCase(), color);

   colorIndex++;
   colorIndex %= colorSteps.length;

   return color;
}

// custom styles
const controlClass = mergeStyleSets({
   control: {
      margin: '0 0 15px 0',
      maxWidth: '300px',
   },
});

let currentSliderValue: number = 1;

export const UtilizationReportView: React.FC = observer(() => {
   const windowSize = useWindowSize();
   const query = useQuery();
   const navigate = useNavigate();

   const [optionsState, setOptionsState] = useState<ChartOptionsState>({
      chartData: null,
      rawData: null,
      updating: true,
      selectedDate: new Date(),
      selectedPoolKeys: [],
      selectedRange: 1,
      selectedFormat: 1,
      poolDropdownOptions: [],
      poolDropdownKeys: {},
      yMaxValue: 0,
      noData: false
   });

   const [pinnedCallout, setPinnedCallout] = useState<PinnedCalloutState | null>(null);

   const dropdownStyles: Partial<IDropdownStyles> = { dropdown: { width: 300 } };
   const intervalStyle: Partial<ISliderStyles> = { root: { width: 200, marginRight: -40 }, titleLabel: { position: 'relative', top: 5 }, container: { position: 'relative', top: 13 } };
   const rangeStyle: Partial<ISliderStyles> = { root: { width: 200, marginRight: -15 }, titleLabel: { position: 'relative', top: 5 }, container: { position: 'relative', top: 13 } };

   // get current date's data
   useEffect(() => {
      currentSliderValue = 1;
      let now = new Date();

      // date query param
      let queryDate = query.get('date');
      if (queryDate) {
         try {
            let parts = queryDate.split('-');
            if (parts.length === 3) {
               let year = parseInt(parts[0]);
               let month = parseInt(parts[1]);
               let day = parseInt(parts[2]);
               if (year && month && day) {
                  let parsedDate = new Date(year, month - 1, day);
                  if (parsedDate < now) {
                     now = parsedDate;
                  }
               }

            }
         }
         // don't do anything if we fail, just fallback to today
         finally {

         }
      }
      now.setHours(0, 0, 0, 0);

      // range query param
      let rangeValue = 1;
      let queryRange = query.get('range');
      if (queryRange) {
         try {
            let rangeNumber = parseInt(queryRange);
            if (rangeNumber && rangeNumber >= 1 && rangeNumber <= 7) {
               rangeValue = rangeNumber;
            }
         }
         finally {

         }
      }

      // interval query param
      let intervalOption = 1;
      let queryInterval = query.get('interval');
      if (queryInterval) {
         try {
            let intervalNumber = parseInt(queryInterval);
            if (intervalNumber && intervalNumber >= 1 && intervalNumber <= 24) {
               Object.keys(intervalToOptions).forEach((key: any) => {
                  if (intervalToOptions[key].bucketInterval === intervalNumber) {
                     intervalOption = parseInt(key)
                  }
               })
            }
         }
         finally {

         }
      }

      let poolOptions: IDropdownOption[] = [];
      let queryPools = query.get('pools');
      if (queryPools) {
         try {
            queryPools.split(';').forEach(pool => {
               let poolText = pool;
               if (poolText.startsWith('g-')) {
                  poolText = poolText.substring(2);
               }
               poolOptions.push({ key: pool, text: poolText, selected: true })
            });
         }
         finally {

         }
      }

      beginUpdateState({ selectedDate: now, selectedRange: rangeValue, selectedFormat: intervalOption, selectedPools: poolOptions });
      let root = document.querySelector('body');

      if (root) {
         const observer = new MutationObserver(addCustomText);
         observer.observe(root, { attributes: true, childList: true, subtree: true });
      }
      // complaining about beginUpdateState dependency
      // eslint-disable-next-line       
   }, []);

   const { hordeClasses, modeColors } = getHordeStyling();

   // hacky observer to mutate the svg's text on the x axis
   const addCustomText = function (mutationsList: any, observer: any) {
      // check and see if we already redrew this render
      let redrawnElementsExist = document.querySelector('.redrawnText');
      if (!redrawnElementsExist) {
         // get all of the <text> tags
         let textElements = document.querySelectorAll('text');
         if (textElements.length !== 0) {
            textElements.forEach(textElement => {
               // get Y offsets; the y AXIS doesn't have Y offsets
               let yPos = textElement.getAttribute('y')!;
               if (yPos) {
                  let yIdx = 0;
                  let textDatas = textElement.innerHTML.split('\n');
                  textElement.innerHTML = "";
                  // go through each text object and insert tspans in place of the text, split by newlines
                  textDatas.forEach(textData => {
                     // if the text is preceeded with a dash, don't render it
                     if (textData.charAt(0) !== '-') {
                        let newTSpan = document.createElementNS("http://www.w3.org/2000/svg", "tspan");
                        newTSpan.setAttribute('y', yPos);
                        newTSpan.setAttribute('dy', yValuesCustomText[yIdx]);
                        newTSpan.setAttribute('x', '0');
                        newTSpan.classList.add('redrawnText');
                        newTSpan.textContent = textData;
                        textElement.appendChild(newTSpan);
                     }
                     yIdx++;
                     if (yIdx === yValuesCustomText.length) {
                        yIdx = 0;
                     }
                  });

               }
            });
         }
      }
   };

   const getTimeFormat = (newState: ChartOptionsState, dayIdx: moment.Moment, numBuckets: number, bucketIdx: number): string => {
      // format the time based on the number of buckets
      let timeFormat = `${intervalToOptions[newState.selectedFormat].hourStrings[bucketIdx]}`;

      // further format based on the number of days we're displaying and the interval
      // add in the minus signs to tell the observer we don't want to render in certain cases
      if (newState.selectedRange !== 1) {
         if (newState.selectedFormat !== 5) {
            if (newState.selectedFormat === 6 || bucketIdx % intervalToOptions[newState.selectedFormat].rangeInterval[newState.selectedRange] !== 0) {
               timeFormat = `-${timeFormat}`;
            }
         }

         let dayFormat = `${dayIdx.local().format('LL')}`;
         if (newState.selectedFormat !== 5 && newState.selectedFormat !== 6) {
            if (bucketIdx !== numBuckets / 2) {
               dayFormat = `-${dayFormat}`;
            }
         }
         timeFormat = `${dayFormat}\n${timeFormat}`;
      }

      return timeFormat;
   };

   // updates the state without getting new data
   const updateStateFilter = (optionsState: ChartOptionsState, data: GetUtilizationTelemetryResponse[]): void => {

      const unix: Map<string, number> = new Map();
      data.forEach(d => unix.set(d.startTime as any as string, moment.utc(d.startTime).unix()))

      data = [...data].sort((a, b) => {
         return unix.get(a.startTime as any as string)! - unix.get(b.startTime as any as string)!;
      });

      let newState = Object.assign({}, optionsState);

      // reset relevant data
      newState.rawData = data;
      newState.chartData = [];
      newState.updating = false;
      newState.yMaxValue = 0;

      setPinnedCallout(null);

      let dayIdx = moment();

      // get start and end date
      let startDate = moment(optionsState.selectedDate).subtract(optionsState.selectedRange - 1, "days");
      let endDate = moment(optionsState.selectedDate).add(1, "days").subtract(1, "seconds");

      let poolList: Set<string> = new Set<string>();
      data.forEach((hour, hourIdx) => {
         // keep track of latest day
         dayIdx = moment.utc(hour.startTime);

         // if this day isnt in the calendar-range, throw it out
         if (dayIdx.isBefore(startDate) || dayIdx.isAfter(endDate)) {
            return;
         }

         hour.pools.forEach(pool => {
            if (pool.numAgents !== 0) {
               poolList.add(pool.poolId);
            }
         });
      });

      // populate dropdowns
      newState.poolDropdownOptions = [];
      newState.poolDropdownKeys = {};
      let poolListArray = Array.from(poolList);
      let categoriesAndHeaders: { [category: string]: string[] } = {};
      poolListArray.sort();

      poolListArray.forEach(pool => {
         let split = pool.split('-');
         if (split.length > 1) {
            let categoryKey = `g-${split[0]}`;
            if (!categoriesAndHeaders[categoryKey]) {
               categoriesAndHeaders[categoryKey] = [];
            }
            categoriesAndHeaders[categoryKey].push(pool);
         }
      });

      newState.poolDropdownOptions.push({ key: 'categoriesHeader', text: 'Bulk Select', itemType: DropdownMenuItemType.Header });

      let sortedCategories = Object.keys(categoriesAndHeaders).sort();
      // add in the categories for bulk selection first
      sortedCategories.forEach(category => {
         newState.poolDropdownOptions.push({ key: category, text: category.split('-')[1] });
      });
      newState.poolDropdownOptions.push({ key: 'bulkDivider', text: '-', itemType: DropdownMenuItemType.Divider });
      // then loop through again and add all the categories as headers and the items underneath
      sortedCategories.forEach((category, idx) => {
         newState.poolDropdownOptions.push({ key: `categoryHeader-${category}`, text: category.split('-')[1], itemType: DropdownMenuItemType.Header });
         categoriesAndHeaders[category].sort().forEach(pool => {
            newState.poolDropdownOptions.push({ key: pool, text: pool });
         });
         newState.poolDropdownOptions.push({ key: `bulkDivider-${idx}`, text: '-', itemType: DropdownMenuItemType.Divider });
      });

      // remove the last divider
      newState.poolDropdownOptions.pop();
      newState.poolDropdownKeys = categoriesAndHeaders;

      // filter out any junk from the query params
      newState.selectedPoolKeys = newState.selectedPoolKeys.filter(key => { return poolListArray.includes(key) || sortedCategories.includes(key) });

      // turn off the bulk select if any of the categories in the filter aren't selected
      let categoryRemove: string[] = [];
      newState.selectedPoolKeys.filter(key => key.startsWith('g-')).forEach(key => {
         if (!categoriesAndHeaders[key].every(pool => newState.selectedPoolKeys.includes(pool))) {
            categoryRemove.push(key);
         }
      });
      newState.selectedPoolKeys = newState.selectedPoolKeys.filter(toRemove => { return !categoryRemove.includes(toRemove); })

      // inversely, add any categories we everything selected for that might not be there
      let categoryAdd: string[] = [];
      sortedCategories.forEach(category => {
         if (categoriesAndHeaders[category].every(pool => newState.selectedPoolKeys.includes(pool)) && !newState.selectedPoolKeys.includes(category)) {
            categoryAdd.push(category);
         }
      });
      newState.selectedPoolKeys = [...newState.selectedPoolKeys, ...categoryAdd];


      let bucketInterval = intervalToOptions[newState.selectedFormat].bucketInterval;
      let numBuckets = 24 / bucketInterval;

      // loop through each hour and make the baseline for time the max number of machines we ever have available
      newState.yMaxValue = 0;
      let isAllPoolKeys = !newState.selectedPoolKeys.length;

      let bucketIdx = 0;
      let bucketIntervalIdx = 0;
      let chartPoints: IVSChartDataPoint[] = [];
      let streamTotalTime: { [streamId: string]: number } = {};

      let poolStreamTime: { [poolId: string]: number } = {};
      let poolOtherTime = 0;
      let poolAdminTime = 0;
      let poolHibernationTime = 0;
      data.forEach((hour, hourIdx) => {
         let selectedPoolsYValues: { [pool: string]: number } = {};
         // keep track of latest day
         dayIdx = moment.utc(hour.startTime);

         // if this day isnt in the calendar-range, throw it out
         if (dayIdx.isBefore(startDate) || dayIdx.isAfter(endDate)) {
            return;
         }

         hour.pools.forEach(pool => {
            if (isAllPoolKeys || newState.selectedPoolKeys.includes(pool.poolId)) {
               let poolAgents = pool.numAgents * bucketInterval;
               //newState.yMaxValue = Math.max(newState.yMaxValue, poolAgents);
               selectedPoolsYValues[pool.poolId] = poolAgents;
               poolAdminTime += pool.adminTime;
               if (pool.hibernatingTime) {
                  poolHibernationTime += pool.hibernatingTime;
               }

               if (!poolStreamTime[pool.poolId]) {
                  poolStreamTime[pool.poolId] = 0;
               }
               pool.streams.forEach(stream => {
                  if (!streamTotalTime[stream.streamId]) {
                     streamTotalTime[stream.streamId] = 0;
                  }
                  poolStreamTime[pool.poolId] += stream.time;
                  streamTotalTime[stream.streamId] += stream.time;
               });
            }
         });
         let yValues = Object.values(selectedPoolsYValues);
         let totalTime = 0;
         if (yValues.length) {
            totalTime = Object.values(selectedPoolsYValues).reduce((curr, prev) => { return prev + curr });
         }
         newState.yMaxValue = Math.max(newState.yMaxValue, totalTime);

         bucketIntervalIdx++;
         if (bucketIntervalIdx === bucketInterval || hourIdx === data.length - 1) {
            let totalTakenTime = 0;
            if (isAllPoolKeys) {
               for (let poolKey in poolStreamTime) {
                  if (poolStreamTime[poolKey] !== 0) {
                     chartPoints.push({
                        legend: poolKey,
                        data: poolStreamTime[poolKey],
                        color: getPoolColor(poolKey),
                        yAxisCalloutData: `${(poolStreamTime[poolKey] / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })}`
                     });
                     totalTakenTime += poolStreamTime[poolKey];
                  }
               }
            }
            else {
               for (let streamKey in streamTotalTime) {
                  if (streamTotalTime[streamKey] !== 0) {
                     const streamTime = streamTotalTime[streamKey];
                     chartPoints.push({
                        legend: streamKey,
                        data: streamTime,
                        color: getPoolColor(streamKey),
                        yAxisCalloutData: `${(streamTime / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })} (${streamTime.toFixed(1)} agent hours)`
                     });
                     totalTakenTime += streamTotalTime[streamKey];
                  }
               }
            }
            if (!isAllPoolKeys && poolOtherTime !== 0) {
               chartPoints.push({
                  legend: 'Other Time',
                  data: poolOtherTime,
                  color: '#eaf3fa',
                  yAxisCalloutData: `${(poolOtherTime / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })}`
               });
               totalTakenTime += poolOtherTime;
            }
            chartPoints.push({
               legend: 'Admin Time',
               data: poolAdminTime,
               color: '#696969',
               yAxisCalloutData: `${(poolAdminTime / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })}`
            });
            totalTakenTime += poolAdminTime;
            totalTakenTime += poolHibernationTime;

            let totalFreeTime = totalTime - totalTakenTime;

            if (totalFreeTime !== 0) {
               chartPoints.push({
                  legend: 'Idle Time',
                  data: totalFreeTime,
                  color: '#696969bf',
                  yAxisCalloutData: `${(totalFreeTime / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })}`
               });
            }

            if (poolHibernationTime !== 0) {
               chartPoints.push({
                  legend: 'Hibernation Time',
                  data: poolHibernationTime,
                  color: '#69696980',
                  yAxisCalloutData: `${(poolHibernationTime / totalTime).toLocaleString(undefined, { style: 'percent', maximumFractionDigits: 1 })}`
               });
            }

            newState.chartData!.push({
               chartData: chartPoints,
               xAxisPoint: getTimeFormat(newState, dayIdx, numBuckets, bucketIdx)
            });

            chartPoints = [];
            streamTotalTime = {};
            poolStreamTime = {};
            poolOtherTime = 0;
            poolAdminTime = 0;
            poolHibernationTime = 0;

            bucketIntervalIdx = 0;
            bucketIdx++;
            if (bucketIdx === numBuckets) {
               bucketIdx = 0;
            }
         }

      });

      function fillBlankBars() {
         while (bucketIdx < numBuckets) {
            newState.chartData!.push({
               chartData: [{ legend: '', data: 0 }],
               xAxisPoint: getTimeFormat(newState, dayIdx, numBuckets, bucketIdx)
            });
            bucketIdx++;
         }
      }

      // fill in any missing bars, whether thats remaining for a day or if we have no data at all
      newState.noData = false;
      if (bucketIdx !== 0 || !newState.chartData!.length) {
         if (!newState.chartData!.length) {
            newState.noData = true;
         }
         fillBlankBars();
      }
      // render
      setOptionsState(newState);

      let dateParamString = `${newState.selectedDate.getFullYear()}-${newState.selectedDate.getMonth() + 1}-${newState.selectedDate.getDate()}`;
      navigate(`/reports/utilization?date=${dateParamString}&range=${newState.selectedRange}&interval=${newState.selectedFormat}&pools=${newState.selectedPoolKeys.join(';')}`);
   };

   const beginUpdateState = (options: UpdateStateOptions): void => {

      function arePoolKeysEqual(initial: string[], updated: string[]) {
         if (initial.length === updated.length) {
            return initial.every(element => {
               if (updated.includes(element)) {
                  return true;
               }
               return false;
            })
         }
         return false;
      }
      // ensure we're operating on something
      let definedOptions = Object.keys(options);
      if (!!definedOptions.length) {
         let newOptionsState = Object.assign({}, optionsState);
         newOptionsState.updating = true;

         definedOptions.forEach(option => {
            // operate on selectedpools as an array
            if (option === 'selectedPools' && options.selectedPools) {
               options.selectedPools.forEach(selectedPool => {
                  let item = selectedPool;
                  if (item) {
                     let itemKey = item.key as string;
                     if ((itemKey).startsWith('g-')) {
                        if (item.selected) {
                           newOptionsState.selectedPoolKeys = [...newOptionsState.selectedPoolKeys, itemKey];
                           if (newOptionsState.poolDropdownKeys && newOptionsState.poolDropdownKeys[itemKey]) {
                              let toAdd = newOptionsState.poolDropdownKeys[itemKey].filter(newItem => { return !newOptionsState.selectedPoolKeys.includes(newItem) });
                              newOptionsState.selectedPoolKeys = [...newOptionsState.selectedPoolKeys, ...toAdd];
                           }
                        }
                        else {
                           // remove each of the pools in the category
                           newOptionsState.poolDropdownKeys[itemKey].forEach(pool => {
                              let foundIdx = newOptionsState.selectedPoolKeys.indexOf(pool);
                              if (foundIdx !== -1) {
                                 newOptionsState.selectedPoolKeys.splice(foundIdx, 1);
                              }
                           });
                           // remove the category itself
                           newOptionsState.selectedPoolKeys = newOptionsState.selectedPoolKeys.filter(key => key !== itemKey);
                        }
                     }
                     else {
                        newOptionsState.selectedPoolKeys = item.selected ? [...newOptionsState.selectedPoolKeys, itemKey] : newOptionsState.selectedPoolKeys.filter(key => key !== itemKey);
                     }
                  }

               });
            }
            else {
               newOptionsState.selectedDate = options.selectedDate ?? newOptionsState.selectedDate;
               newOptionsState.selectedFormat = options.selectedFormat ?? newOptionsState.selectedFormat;
               newOptionsState.selectedRange = options.selectedRange ?? newOptionsState.selectedRange;
            }
         });

         // check to make sure that the options arent all identical
         if (optionsState.selectedDate === newOptionsState.selectedDate &&
            optionsState.selectedFormat === newOptionsState.selectedFormat &&
            optionsState.selectedRange === newOptionsState.selectedRange &&
            arePoolKeysEqual(optionsState.selectedPoolKeys, newOptionsState.selectedPoolKeys)) {
            // break out, we didnt change anything
            return;
         }

         if (options.selectedDate || options.selectedRange || !optionsState.rawData) {
            if (options.selectedRange) {
               currentSliderValue = options.selectedRange;
            }
            setOptionsState(newOptionsState);
            updateStateQuery(newOptionsState);
         }
         else {
            updateStateFilter(newOptionsState, optionsState.rawData);
         }
      }
   };


   const updateStateQuery = (newState: ChartOptionsState): void => {
      backend.getUtilizationData(moment(newState.selectedDate).format('YYYY-MM-DDT00:00:00'), newState.selectedRange, moment().utcOffset() / 60).then(data => {
         if (currentSliderValue === newState.selectedRange) {
            updateStateFilter(newState, data);
         }
      });
   };

   const formatSliderFormat = (value: number) => {
      return `${intervalToOptions[value].bucketInterval}h`;
   };

   const formatRangeFormat = (value: number) => {
      return `${value} day${value !== 1 ? 's' : ''}`;

   }

   function yAxisFormatter(hours?: any) {
      return (hours / intervalToOptions[optionsState.selectedFormat].bucketInterval).toFixed(0);
   }

   let yMaxValue = Math.ceil(optionsState.yMaxValue / 20) * 20;
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <Breadcrumbs items={[{ text: 'Utilization Report' }]} />
         <Stack horizontal>
            <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
            <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
               <Stack style={{ width: 1440, paddingTop: 6, marginLeft: 4, height: 'calc(100vh - 8px)' }}>
                  <Stack styles={{ root: { paddingTop: 18, paddingLeft: 12, paddingRight: 12, width: "100%" } }} >
                     <Stack className={hordeClasses.raised}>
                        <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "calc(100vh - 250px)" }} data-is-scrollable={true}>
                           <Stack>
                              <Stack horizontal tokens={{ childrenGap: 32 }} style={{ paddingLeft: 12, paddingRight: 12 }}>
                                 <Stack>
                                    <DatePicker
                                       value={optionsState.selectedDate}
                                       label="End Date"
                                       className={controlClass.control}
                                       strings={DayPickerStrings}
                                       firstWeekOfYear={1}
                                       showMonthPickerAsOverlay={true}
                                       placeholder="Select a date..."
                                       onSelectDate={(date) => beginUpdateState({ selectedDate: date })}
                                       maxDate={moment().toDate()}
                                       style={{ width: 200 }}

                                    />
                                 </Stack>
                                 <Stack>
                                    <Slider
                                       label="Range"
                                       min={1}
                                       max={7}
                                       value={optionsState.selectedRange}
                                       showValue={true}
                                       onChange={(value) => beginUpdateState({ selectedRange: value })}
                                       styles={rangeStyle}
                                       valueFormat={formatRangeFormat}
                                    />
                                 </Stack>
                                 <Stack>
                                    <Slider
                                       label="Interval"
                                       min={1}
                                       max={6}
                                       value={optionsState.selectedFormat}
                                       showValue={true}
                                       onChange={(value) => beginUpdateState({ selectedFormat: value })}
                                       //onChanged={(event, value) => beginUpdateState('selectedFormat', value)}
                                       valueFormat={formatSliderFormat}
                                       styles={intervalStyle}
                                    />
                                 </Stack>
                                 <Stack grow />
                                 <Stack>
                                    <Dropdown
                                       label="Pool"
                                       selectedKeys={optionsState.selectedPoolKeys}
                                       multiSelect={true}
                                       onChange={(event, item) => beginUpdateState({ selectedPools: [item] })}
                                       placeholder="All pools"
                                       options={optionsState.poolDropdownOptions}
                                       styles={dropdownStyles}
                                    />
                                 </Stack>
                              </Stack>
                              <Stack className="horde-no-darktheme" style={{ paddingLeft: 12 }}>
                                 {!optionsState.updating ? <div>
                                    {optionsState.noData && <div>No data for this date range.</div>}
                                    <VerticalStackedBarChart
                                       styles={{ root: { height: '900px' } }}
                                       data={optionsState.chartData!}
                                       barCornerRadius={5}
                                       yMaxValue={yMaxValue}
                                       yAxisTickCount={20}
                                       barWidth={intervalToOptions[optionsState.selectedFormat].barWidth / optionsState.selectedRange}
                                       hideLegend={true}
                                       yAxisTickFormat={yAxisFormatter}
                                       onBarClick={(event, data: any) => {
                                          if (pinnedCallout?.legend === data.legend) {
                                             setPinnedCallout(null);
                                          } else {
                                             setPinnedCallout({
                                                legend: data.legend,
                                                data: data.data,
                                                color: data.color,
                                                yAxisCalloutData: data.yAxisCalloutData,
                                                x: event.clientX,
                                                y: event.clientY
                                             });
                                          }
                                       }}
                                    />
                                    {pinnedCallout && (
                                       <Callout
                                          target={{ x: pinnedCallout.x, y: pinnedCallout.y }}
                                          onDismiss={() => setPinnedCallout(null)}
                                          directionalHint={DirectionalHint.bottomLeftEdge}
                                          isBeakVisible={true}
                                          gapSpace={10}
                                          setInitialFocus
                                       >
                                          <div style={{ padding: 12, minWidth: 150 }}>
                                             <div style={{ fontWeight: 600, color: pinnedCallout.color, marginBottom: 4 }}>
                                                {pinnedCallout.legend}
                                             </div>
                                             <div>{pinnedCallout.yAxisCalloutData}</div>
                                          </div>
                                       </Callout>
                                    )}
                                 </div>
                                    :
                                    <div>Fetching Data...</div>
                                 }
                              </Stack>
                           </Stack>
                        </div>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack >
      </Stack >);
});


