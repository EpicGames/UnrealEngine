// Copyright Epic Games, Inc. All Rights Reserved.
import { DetailsList, DetailsListLayoutMode, IColumn, IconButton, Label, Modal, PrimaryButton, SelectionMode, Stack, Text } from '@fluentui/react';
import React, { useState } from 'react';
import backend from '../backend';
import { GetServerInfoResponse } from '../backend/Api';
import { getHordeStyling } from '../styles/Styles';

export const VersionModal: React.FC<{ show: boolean, onClose: () => void }> = ({ show, onClose }) => {

   const [version, setVersion] = useState<{ serverInfo?: GetServerInfoResponse, querying?: boolean }>({});
   const { hordeClasses } = getHordeStyling();

   type GeneralItem = {
      name: string;
      value?: string;
      userAcls?: string[];
   }

   const close = () => {
      onClose();
   };

   if (!version.serverInfo && !version.querying) {

      setVersion({ querying: true });

      (async () => {

         const serverInfo = await backend.getServerInfo();

         setVersion({ serverInfo: serverInfo })
      })()

   }

   const versionItems: GeneralItem[] = [];

   const columns = [
      { key: 'column1', name: 'Name', fieldName: 'name', minWidth: 120, maxWidth: 120 },
      { key: 'column2', name: 'Value', fieldName: 'value', minWidth: 100, maxWidth: 200 },
      { key: 'column3', name: 'User ACLs', fieldName: 'userAcls', minWidth: 200, maxWidth: 400 },
   ];

   let dashboardVersion: string | undefined;

   try {
      // note: this must be exactly `import.meta.env.VITE_VERSION_INFO`
      // as webpack does a simple find and replace to the value in the .env
      // so import?.meta?.env?.VITE_VERSION_INFO for example is invalid
      dashboardVersion = import.meta.env.VITE_VERSION_INFO;
   } catch (reason) {
      console.log("Process env error:", reason);
   }

   if (dashboardVersion) {
      versionItems.push({ name: "Dashboard", value: dashboardVersion });
   }

   versionItems.push({ name: "Server", value: version.serverInfo?.serverVersion ?? "Querying" });

   if (version.serverInfo?.agentVersion) {
      versionItems.push({ name: "Agent", value: version.serverInfo?.agentVersion });
   }

   if (version.serverInfo?.plugins?.length) {

      versionItems.push({ name: "Plugins:", value: "" });

      version.serverInfo?.plugins.forEach(p => {
         versionItems.push({ name: p.name, value: p.version ?? "", userAcls: p.userAcls });
      })
   
   }


   const onRenderItemColumn = (item: GeneralItem, _index?: number, columnIn?: IColumn) => {

      const column = columnIn!;

      switch (column.name) {
         case 'Name':
            return <Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{item.name}</Text>
         case 'Value':
            if (item.value) {
               return <Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{item.value}</Text>
            }
            break;
         case 'User ACLs':
            if (item.userAcls === undefined) {
               return null; // Plugin doesn't use ACLs
            }
            if (item.userAcls.length === 0) {
               return <Text variant="small" styles={{ root: { fontFamily: "Horde Open Sans Regular", color: "#888" } }}>None</Text>
            }
            return <Text variant="small" styles={{ root: { fontFamily: "Horde Open Sans Regular" } }}>{item.userAcls.join(", ")}</Text>
      }

      return null;
   }


   return <Modal className={hordeClasses.modal} isOpen={show} styles={{ main: { padding: 4, width: 800 } }} onDismiss={() => { if (show) close() }}>
      <Stack tokens={{ childrenGap: 12 }}>
         <Stack horizontal styles={{ root: { paddingLeft: 12 } }}>
            <Stack grow style={{ paddingTop: 12, paddingLeft: 8 }}>
               <Label style={{ fontSize: 18, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }}>Horde Versions</Label>
            </Stack>
            <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { close(); }}
               />
            </Stack>
         </Stack>

         <Stack tokens={{ childrenGap: 32 }} style={{ paddingLeft: 8 }}>

            <Stack horizontal>
               <Stack style={{ minWidth: 760 }}>
                  <DetailsList
                     items={versionItems}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />
               </Stack>
            </Stack>

         </Stack>

         <Stack styles={{ root: { padding: 12 } }}>
            <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
               <Stack grow />
               <PrimaryButton text="Ok" disabled={false} onClick={() => { close(); }} />
            </Stack>
         </Stack>
      </Stack>
   </Modal>;

};
