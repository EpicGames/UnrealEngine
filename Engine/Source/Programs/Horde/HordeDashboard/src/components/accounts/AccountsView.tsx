// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, DetailsList, DetailsListLayoutMode, Dialog, DialogFooter, DialogType, IColumn, ITag, Icon, Label, MessageBar, MessageBarType, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, TagPicker, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import backend from "../../backend";
import { AccountClaimMessage, CreateAccountRequest, GetAccountResponse, UpdateAccountRequest } from "../../backend/Api";
import { PollBase } from "../../backend/PollBase";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import { errorDialogStore } from "../error/ErrorStore";
import moment from "moment";

class AccountHandler extends PollBase {

   constructor(pollTime = 60000) {

      super(pollTime);

   }

   clear() {
      this.loaded = false;
      this.accounts = [];
      this.accountGroups = [];
      super.stop();
   }

   async poll(): Promise<void> {

      try {

         this.accounts = (await backend.getAccounts()).sort((a, b) => a.name.localeCompare(b.name));
         this.accountGroups = await backend.getAccountGroups();
         this.loaded = true;
         this.setUpdated();

      } catch (err) {

      }

   }

   loaded = false;

   accounts: GetAccountResponse[] = [];
   accountGroups: AccountClaimMessage[] = [];
}

const handler = new AccountHandler();

const AccountPanel: React.FC = observer(() => {

   const [state, setState] = useState<{ showEditor?: boolean, editAccount?: GetAccountResponse }>({});

   // subscribe
   if (handler.updated) { };

   const columns: IColumn[] = [
      { key: 'column_name', name: 'Full Name', minWidth: 240, maxWidth: 240, isResizable: false },
      { key: 'column_status', name: 'Status', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'column_login', name: 'Username', minWidth: 160, maxWidth: 160, isResizable: false },
      { key: 'column_email', name: 'Email', minWidth: 240, maxWidth: 240, isResizable: false },
      { key: 'column_description', name: 'Description', minWidth: 440, maxWidth: 440, isResizable: false },
      { key: 'column_edit', name: 'Edit', minWidth: 48, maxWidth: 48, isResizable: false, onRenderHeader: () => null },
   ];

   let accounts = [...handler.accounts];

   const renderItem = (item: GetAccountResponse, index?: number, column?: IColumn) => {
      if (!column) {
         return null;
      }

      if (column.name === "Status") {

         return <Stack>
            <Text>{item.enabled ? "Enabled" : "Disabled"}</Text>
         </Stack>
      }

      if (column.name === "Full Name") {

         return <Stack><Text>{item.name}</Text></Stack>
      }

      if (column.name === "Username") {

         return <Stack><Text>{item.login}</Text></Stack>
      }

      if (column.name === "Email") {

         return <Stack><Text>{item.email ?? ""}</Text></Stack>
      }

      if (column.name === "Description") {

         return <Stack><Text>{item.description ?? ""}</Text></Stack>
      }

      if (column.name === "Edit") {

         return <Stack horizontalAlign="end" onClick={() => { setState({ showEditor: true, editAccount: item }) }} style={{ "cursor": "pointer" }}>
            <Icon style={{ paddingTop: 4, paddingRight: 8 }} iconName="Edit" />
         </Stack>
      }

      return null;
   };

   const { hordeClasses } = getHordeStyling();

   return <Stack>
      {!!state.showEditor && <AccountEditor accountIn={state.editAccount} onClose={() => {
         setState({})
      }} />}
      <Stack style={{ paddingBottom: 12 }}>
         <Stack verticalAlign="center">
            <Stack horizontalAlign="end">
               <PrimaryButton styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }} onClick={() => setState({ showEditor: true, editAccount: undefined })}>Create Account</PrimaryButton>
            </Stack>
            {!accounts.length && handler.loaded && <Stack horizontalAlign="center">
               <Text variant="mediumPlus">No User Accounts Found</Text>
            </Stack>}
         </Stack>
      </Stack>

      {!!accounts.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
            <Stack>
               <DetailsList
                  items={accounts}
                  columns={columns}
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  compact={true}
                  onRenderItemColumn={renderItem}
               />
            </Stack>
         </Stack>
      </Stack>}
   </Stack>
});


export const AccountsView: React.FC = () => {

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'User Accounts' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <AccountPanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

const AccountEditor: React.FC<{ accountIn?: GetAccountResponse, onClose: () => void }> = ({ accountIn, onClose }) => {

   const [submitting, setSubmitting] = useState(false);
   const [confirmDelete, setConfirmDelete] = useState(false);
   const { hordeClasses } = getHordeStyling();
   const [error, setError] = useState("");
   const [account, setAccount] = useState<GetAccountResponse>(accountIn ? { ...accountIn } : { id: "", name: "", login: "", claims: [], description: undefined, email: undefined, enabled: true });
   const [secrets, setSecrets] = useState<{ password?: string }>({});

   type ClaimTag = ITag & {
      claimType: string;
   }

   const claimTags: ClaimTag[] = handler.accountGroups.sort((a, b) => a.value.localeCompare(b.value)).map(c => {
      return { key: c.value, name: c.value, claimType: c.type }
   });


   const onValidate = () => {

      if (!account.name) {
         return "Please enter a name";
      }

      if (!account.login) {
         return "Please enter a login";
      }

      return undefined;
   }

   const onSave = async () => {

      const nerror = onValidate();

      if (nerror) {
         setError(nerror);
         return;
      }

      setError("");

      try {

         setSubmitting(true);

         const password = secrets.password ? secrets.password : undefined;
         if (accountIn?.id) {
            const uaccount: UpdateAccountRequest = { ...account, password: password };
            await backend.updateAccount(accountIn.id, uaccount);
         } else {
            const naccount: CreateAccountRequest = { ...account, password: password };
            await backend.createAccount(naccount)
         }
         setSubmitting(false);
         onClose();
         handler.poll();

      } catch (reason) {
         console.error(reason);
         errorDialogStore.set({
            reason: `${reason}`,
            title: `User Modification Error`,
            message: `There was an issue modifying the user.\n\nReason: ${reason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`
         }, true);

         setSubmitting(false);
      }

   }

   const onDelete = async () => {

      try {
         setSubmitting(true);
         await backend.deleteAccount(account.id!);
         setSubmitting(false);
         onClose();
         handler.poll();
      } catch (reason) {
         console.error(reason);
         setSubmitting(false);
      }
   }

   if (submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="mediumPlus">Please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   }

   if (confirmDelete) {
      return <Dialog
         hidden={false}
         onDismiss={() => setConfirmDelete(false)}
         minWidth={400}
         dialogContentProps={{
            type: DialogType.normal,
            title: `Delete Account`,
            subText: `Confirm deletion of account ${account.name}`
         }}
         modalProps={{ isBlocking: true, topOffsetFixed: true, styles: { main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } } }} >
         <DialogFooter>
            <PrimaryButton onClick={() => { setConfirmDelete(false); onDelete() }} text="Delete" />
            <DefaultButton onClick={() => setConfirmDelete(false)} text="Cancel" />
         </DialogFooter>
      </Dialog>
   }


   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 600, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 16 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>{accountIn ? "Edit Account" : "New Account"}</Text>
         </Stack>
         {!!error && <Stack>
            <MessageBar key={`validation_error`} messageBarType={MessageBarType.error} isMultiline={false}>{error}</MessageBar>
         </Stack>}

         <Stack style={{ padding: 8 }}>
            <TextField label="Username" autoComplete="off" spellCheck={false} placeholder="" required defaultValue={account.login} onChange={(ev, value) => { setAccount({ ...account, login: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label="Full Name" autoComplete="off" spellCheck={false} placeholder="" required defaultValue={account.name} onChange={(ev, value) => { setAccount({ ...account, name: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label="Email" autoComplete="off" spellCheck={false} placeholder="" defaultValue={account.email} onChange={(ev, value) => { setAccount({ ...account, email: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label="Description" autoComplete="off" spellCheck={false} placeholder="" defaultValue={account.description} onChange={(ev, value) => { setAccount({ ...account, description: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <Label >Groups</Label>
            <TagPicker inputProps={{ placeholder: account.claims.length === 0 ? "Select claims" : undefined }}
               selectedItems={claimTags.filter(c => !!account.claims.find(ac => c.name === ac.value))}
               onResolveSuggestions={(filter) => claimTags.filter(c => !account.claims.find(ac => c.name === ac.value)).filter(c => !filter || c.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1)}
               getTextFromItem={(item) => item.name}
               onEmptyResolveSuggestions={() => claimTags.filter(c => !account.claims.find(ac => c.name === ac.value))}
               onChange={(items) => {
                  setAccount({
                     ...account, claims: items?.map(c => {
                        const ct = c as ClaimTag;
                        return {
                           value: c.name,
                           type: ct.claimType
                        }
                     }) ?? []
                  })
               }}
            />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <TextField label={account.id ? "Change Password" : "Password"} autoComplete="new-password" spellCheck={false} placeholder="User Password" type="password" canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, password: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <Checkbox label="Enabled" checked={account.enabled} onChange={(ev, value) => { setAccount({ ...account, enabled: value ? true : false }) }} />
         </Stack>

         <Stack horizontal style={{ paddingTop: 64 }}>
            {!!accountIn && <PrimaryButton style={{ backgroundColor: "red", border: 0 }} onClick={() => setConfirmDelete(true)} text="Delete Account" />}
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 28 }}>
               <PrimaryButton onClick={() => onSave()} text="Save" />
               <DefaultButton onClick={() => onClose()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>

}