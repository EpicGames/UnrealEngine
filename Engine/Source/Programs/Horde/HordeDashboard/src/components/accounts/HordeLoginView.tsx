// Copyright Epic Games, Inc. All Rights Reserved.

import { ThemeProvider, Stack, Image, TextField, PrimaryButton, MessageBar, MessageBarType } from "@fluentui/react";
import dashboard from "../../backend/Dashboard";
import { darkTheme } from "../../styles/darkTheme";
import { lightTheme } from "../../styles/lightTheme";
import { useState } from "react";


export const HordeLoginView: React.FC = () => {

   const [state, setState] = useState<{ error?: string, username?: string, password?: string, submitting?: boolean }>({});
   const search = new URLSearchParams(window.location.search);

   const redirect = search.get("redirect") ? atob(search.get("redirect")!) : undefined;

   const error = state.error;

   document.body.setAttribute('style', `background: ${dashboard.darktheme ? "#0F0F0F" : "#FFFFFFF"}`)   

   const onLogin = async () => {
      const username = state.username?.trim();
      const password = state.password?.trim();
      if (!username) {
         setState({ ...state, error: "Please provide a username", submitting: false })
         return;
      }

      try {

         const request: RequestInit = {
            method: "POST",
            mode: "cors",
            cache: "no-cache",
            credentials: "same-origin",
            headers: { "Content-Type": "application/json" },
            redirect: "follow",
            body: JSON.stringify({
               username: username,
               password: password,
               returnUrl: redirect
            })
         };

         const response = await fetch("/account/login/dashboard", request);

         if (!response.ok || response.status !== 200 || !response.redirected) {

            let error = "Problem logging in: Unknown Error";

            if (response.status === 403) {
               error = "Invalid username or password";
            } else if (response.ok && !response.redirected) {
               error = "Problem logging in: Not Redirected";
            }

            setState({ ...state, error: error, submitting: false });
            return;
         }

         window.location.assign((response.redirected && response.url) ? response.url : "/index");

      } catch (error) {
         setState({ ...state, error: `Problem logging in ${error}`, submitting: false });
      }
   }

   return (<ThemeProvider applyTo='element' theme={dashboard.darktheme ? darkTheme : lightTheme}>
      <div style={{ position: 'absolute', left: '50%', top: '50%', transform: 'translate(-50%, -50%)' }}>
         <Stack>
            <Stack horizontalAlign="center">
               <Stack styles={{ root: { paddingTop: 2 } }}>
                  <Image shouldFadeIn={false} shouldStartVisible={true} src={`/images/${dashboard.darktheme ? "unreal_horde_logo_dark_mode.png" : "unreal_horde_logo.png"}`} />
               </Stack>

               <Stack style={{ width: 420 }} tokens={{ childrenGap: 12 }}>
                  {!!error && <Stack>
                     <MessageBar key={`validation_error`} messageBarType={MessageBarType.error} isMultiline={false}>{error}</MessageBar>
                  </Stack>}

                  <Stack style={{ padding: 8 }}>
                     <TextField disabled={state.submitting} label="Username" autoComplete="on" spellCheck={false} onChange={(ev, value) => { setState({ ...state, username: value ?? "" }) }} />
                  </Stack>
                  <Stack style={{ padding: 8 }}>
                     <TextField disabled={state.submitting} label={"Password"} autoComplete="off" spellCheck={false} type="password" canRevealPassword onChange={(ev, value) => { setState({ ...state, password: value ?? "" }) }} />
                  </Stack>
                  <Stack style={{ padding: 8 }}>
                     <PrimaryButton disabled={state.submitting} text="Login" onClick={() => {
                        setState({ ...state, error: undefined, submitting: true });
                        onLogin();
                     }} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </div>
   </ThemeProvider>);

}