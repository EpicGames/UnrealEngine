// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, DefaultButton, DetailsList, DetailsListLayoutMode, Dialog, DialogFooter, DialogType, IColumn, ITag, Icon, Label, MessageBar, MessageBarType, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, TagPicker, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import backend from "../../backend";
import { AccountClaimMessage, CreateServiceAccountRequest, GetServiceAccountResponse, UpdateServiceAccountRequest } from "../../backend/Api";
import { PollBase } from "../../backend/PollBase";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import { errorDialogStore } from "../error/ErrorStore";
import moment from "moment";

class ServiceAccountHandler extends PollBase {

   constructor(pollTime = 60000) {

      super(pollTime);

   }

   clear() {
      this.loaded = false;
      this.accounts = [];
      this.accountGroups = [];
      super.stop();
   }

   async poll(): Promise<void> {

      try {

         this.accounts = (await backend.getServiceAccounts()).sort((a, b) => a.description.localeCompare(b.description));
         this.accountGroups = await backend.getAccountGroups();
         this.loaded = true;
         this.setUpdated();         

      } catch (err) {

      }

   }

   loaded = false;

   accounts: GetServiceAccountResponse[] = [];
   accountGroups: AccountClaimMessage[] = [];
}

const handler = new ServiceAccountHandler();

const AccountPanel: React.FC = observer(() => {

   const [state, setState] = useState<{ showEditor?: boolean, editAccount?: GetServiceAccountResponse }>({});
   const [newToken, setNewToken] = useState("");

   // subscribe
   if (handler.updated) { };

   const columns: IColumn[] = [
      { key: 'column_id', name: 'Id', minWidth: 240, maxWidth: 240, isResizable: false },
      { key: 'column_status', name: 'Status', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'column_description', name: 'Description', minWidth: 440, isResizable: false },
      { key: 'column_edit', name: 'Edit', minWidth: 48, maxWidth: 48, isResizable: false, onRenderHeader: () => null },
   ];

   let accounts = [...handler.accounts];

   const renderItem = (item: GetServiceAccountResponse, index?: number, column?: IColumn) => {
      if (!column) {
         return null;
      }

      if (column.name === "Id") {
         return <Stack><Text>{item.id}</Text></Stack>
      }

      if (column.name === "Status") {

         return <Stack>
            <Text>{item.enabled ? "Enabled" : "Disabled"}</Text>
         </Stack>
      }

      if (column.name === "Description") {

         return <Stack><Text>{item.description ?? ""}</Text></Stack>
      }

      if (column.name === "Edit") {

         return <Stack horizontalAlign="end" onClick={() => { setState({ showEditor: true, editAccount: item }) }} style={{ "cursor": "pointer" }}>
            <Icon style={{ paddingTop: 4, paddingRight: 8 }} iconName="Edit" />
         </Stack>
      }

      return null;
   };

   const { hordeClasses } = getHordeStyling();

   return (<Stack>
      {!!newToken && <Dialog
         hidden={false}
         onDismiss={() => setNewToken("")}
         minWidth={560}
         dialogContentProps={{
            type: DialogType.normal,
            title: `New Service Token`,
            subText: `A new service token has been generated:\n\n ${newToken}`
         }}
         modalProps={{ isBlocking: true, topOffsetFixed: true, styles: { main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } } }} >
         <DialogFooter>
            <PrimaryButton onClick={() => { setNewToken("") }} text="Ok" />
         </DialogFooter>
      </Dialog>}
      {!!state.showEditor && <ServiceAccountEditor accountIn={state.editAccount} onClose={(newToken?: string) => {
         setState({})
         if (newToken?.length) {
            setNewToken(newToken);
         }
      }} />}
      <Stack style={{paddingBottom: 12}}>
         <Stack verticalAlign="center">            
            <Stack horizontalAlign="end">
               <PrimaryButton styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }} onClick={() => setState({ showEditor: true, editAccount: undefined })}>Create Account</PrimaryButton>
            </Stack>
            {!accounts.length && handler.loaded && <Stack horizontalAlign="center">
               <Text variant="mediumPlus">No Service Accounts Found</Text>
            </Stack>}                        
         </Stack>
      </Stack>

      {!!accounts.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
            <Stack>
               <DetailsList
                  items={accounts}
                  columns={columns}
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  compact={true}
                  onRenderItemColumn={renderItem}
               />
            </Stack>
         </Stack>
      </Stack>}
   </Stack>);
});


export const ServiceAccountsView: React.FC = () => {

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Service Accounts' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <AccountPanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

const ServiceAccountEditor: React.FC<{ accountIn?: GetServiceAccountResponse, onClose: (newToken?: string) => void }> = ({ accountIn, onClose }) => {

   const [submitting, setSubmitting] = useState(false);
   const [confirmDelete, setConfirmDelete] = useState(false);
   const { hordeClasses } = getHordeStyling();
   const [error, setError] = useState("");
   const [account, setAccount] = useState<GetServiceAccountResponse>(accountIn ? { ...accountIn } : { id: "", claims: [], description: "", enabled: true });
   const [resetToken, setResetToken] = useState(false);

   type ClaimTag = ITag & {
      claimType: string;
   }

   const claimTags: ClaimTag[] = handler.accountGroups.sort((a, b) => a.value.localeCompare(b.value)).map(c => {
      return { key: c.value, name: c.value, claimType: c.type }
   });


   const onValidate = () => {

      if (!account.description?.trim().length) {
         return "Please enter a description";
      }

      return undefined;
   }

   const onSave = async () => {

      const nerror = onValidate();

      if (nerror) {
         setError(nerror);
         return;
      }

      setError("");

      try {

         let newToken: string | undefined;

         setSubmitting(true);
         if (accountIn?.id) {
            const uaccount: UpdateServiceAccountRequest = { ...account, resetToken: resetToken };
            const response = await backend.updateServiceAccount(accountIn.id, uaccount);
            newToken = response?.newSecretToken;
         } else {
            const naccount: CreateServiceAccountRequest = { ...account };
            const response = await backend.createServiceAccount(naccount)
            newToken = response?.secretToken;
         }
         setSubmitting(false);
         onClose(newToken);
         handler.poll();

      } catch (reason) {
         console.error(reason);
         errorDialogStore.set({
            reason: `${reason}`,
            title: `User Modification Error`,
            message: `There was an issue modifying the user.\n\nReason: ${reason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`
         }, true);

         setSubmitting(false);
      }

   }

   const onDelete = async () => {

      try {
         setSubmitting(true);
         await backend.deleteServiceAccount(account.id!);
         setSubmitting(false);
         onClose();
         handler.poll();
      } catch (reason) {
         console.error(reason);
         setSubmitting(false);
      }
   }

   if (submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="mediumPlus">Please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   }

   if (confirmDelete) {
      return <Dialog
         hidden={false}
         onDismiss={() => setConfirmDelete(false)}
         minWidth={512}
         dialogContentProps={{
            type: DialogType.normal,
            title: `Delete Account`,
            subText: `Confirm deletion of account ${account.id}`,
         }}
         modalProps={{ isBlocking: true, topOffsetFixed: true, styles: { main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } } }} >
         <DialogFooter>
            <PrimaryButton onClick={() => { setConfirmDelete(false); onDelete() }} text="Delete" />
            <DefaultButton onClick={() => setConfirmDelete(false)} text="Cancel" />
         </DialogFooter>
      </Dialog>
   }


   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 600, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 16 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>{accountIn ? "Edit Service Account" : "New Service Account"}</Text>
         </Stack>
         {!!error && <Stack>
            <MessageBar key={`validation_error`} messageBarType={MessageBarType.error} isMultiline={false}>{error}</MessageBar>
         </Stack>}

         <Stack style={{ padding: 8 }}>
            <TextField label="Description" autoComplete="off" spellCheck={false} placeholder="Description of the service" defaultValue={account.description} onChange={(ev, value) => { setAccount({ ...account, description: value ?? "" }) }} />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <Label >Groups</Label>
            <TagPicker inputProps={{ placeholder: account.claims.length === 0 ? "Select claims" : undefined }}
               selectedItems={claimTags.filter(c => !!account.claims.find(ac => c.name === ac.value))}
               onResolveSuggestions={(filter) => claimTags.filter(c => !account.claims.find(ac => c.name === ac.value)).filter(c => !filter || c.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1)}
               getTextFromItem={(item) => item.name}
               onEmptyResolveSuggestions={() => claimTags.filter(c => !account.claims.find(ac => c.name === ac.value))}
               onChange={(items) => {
                  setAccount({
                     ...account, claims: items?.map(c => {
                        const ct = c as ClaimTag;
                        return {
                           value: c.name,
                           type: ct.claimType
                        }
                     }) ?? []
                  })
               }}
            />
         </Stack>

         <Stack style={{ padding: 8 }}>
            <Checkbox label="Enabled" checked={account.enabled} onChange={(ev, value) => { setAccount({ ...account, enabled: value ? true : false }) }} />
         </Stack>

         {!!accountIn && <Stack style={{ padding: 8 }}>
            <Checkbox label="Reset Token" checked={resetToken} onChange={(ev, value) => { setResetToken(value ? true : false) }} />
         </Stack>}

         <Stack horizontal style={{ paddingTop: 64 }}>
            {!!accountIn && <PrimaryButton style={{ backgroundColor: "red", border: 0 }} onClick={() => setConfirmDelete(true)} text="Delete Account" />}
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 28 }}>
               <PrimaryButton onClick={() => onSave()} text="Save" />
               <DefaultButton onClick={() => onClose()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>

}