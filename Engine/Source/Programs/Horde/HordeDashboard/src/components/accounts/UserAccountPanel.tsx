// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, ScrollablePane, ScrollbarVisibility, SelectionMode, Stack, Text } from "@fluentui/react";
import dashboard from "../../backend/Dashboard";
import { useState } from "react";
import backend from "../../backend";

export const UserAccountPanel: React.FC<{ mode: string }> = ({ mode }) => {

   const [entitlements, setEntitlements] = useState("");

   if (entitlements === "inflight") { 
      return null;
   }

   if (!entitlements) {
      setEntitlements("inflight");
      backend.getAccountEntitlements().then((d:any) => {
         setEntitlements(JSON.stringify(d, undefined, 4));
      })
      return null;
   }

   
   const columns = [
      { key: 'column1', name: 'Claim', fieldName: 'Claim', minWidth: 100 }
   ];

   type GeneralItem = {
      name: string;
      value?: string;
   }

   let items: GeneralItem[] = [];

   if (mode === "Claims") {

      const claims = dashboard.claims.sort((a, b) => {
         if (a.type !== b.type) {
            return a.type.localeCompare(b.type);
         }

         return a.value.localeCompare(b.value);
      })

      items = claims.map(c => {
         return {
            name: c.type,
            value: c.value
         }
      })
   }

   const onRenderItemColumn = (item: GeneralItem) => {

      if (mode === "Claims") {
         return <Stack horizontal tokens={{ childrenGap: 8 }}><Stack><Text >{item.name}:</Text></Stack>
            <Stack>
               <Text style={{ fontWeight: 600 }} >{item.value}</Text>
            </Stack>
         </Stack>
      }
   }

   return <Stack style={{ padding: 8, position: "relative", height: 'calc(100vh - 380px)' }}>
      <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
         {mode === "Claims" && <DetailsList
            items={items}
            columns={columns}
            compact
            setKey="set"
            layoutMode={DetailsListLayoutMode.justified}
            isHeaderVisible={false}
            selectionMode={SelectionMode.none}
            onRenderItemColumn={onRenderItemColumn}
         />}
         {mode === "Entitlements" && <Stack>
            <Text style={{whiteSpace: "pre"}}>{entitlements}</Text>
         </Stack>}
      </ScrollablePane>
   </Stack>
}