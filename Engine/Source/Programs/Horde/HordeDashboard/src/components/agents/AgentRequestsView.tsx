// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, IColumn, SelectionZone, PrimaryButton, Selection, SelectionMode, Stack, Text, DefaultButton, Dialog, DialogFooter, DialogType, Modal, Spinner, SpinnerSize } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import backend from "../../backend";
import { GetPendingAgentResponse } from "../../backend/Api";
import { PollBase } from "../../backend/PollBase";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import { errorDialogStore } from "../error/ErrorStore";
import { useNavigate } from "react-router-dom";
import moment from "moment";
import { agentStore } from "./AgentStore"

class AgentRequestsHandler extends PollBase {

   constructor(pollTime = 2500) {

      super(pollTime);

   }

   clear() {
      this.initial = true;
      this.requests = [];
      this.selectedAgents = [];
      this.selection = new Selection({ onSelectionChanged: () => { this.onSelectionChanged(this.selection.getSelection() as any) }, selectionMode: SelectionMode.multiple })
      super.stop();
   }

   async poll(): Promise<void> {

      try {
         const requests = await backend.getAgentRegistrationRequests();
         if (requests.agents.length) {
            this.requests = requests.agents;
            this.selectedAgents = this.requests;
            this.selection.setItems(this.selectedAgents);
            this.selection.setRangeSelected(0, this.selectedAgents.length, true, false);
            this.initial = false;
            this.stop();
            this.setUpdated();
         }

      } catch (err) {

      }
   }

   onSelectionChanged(selection: GetPendingAgentResponse[] | undefined) {
      this.selectedAgents = selection ?? [];
      this.setUpdated();
   }

   selection = new Selection({ onSelectionChanged: () => { this.onSelectionChanged(this.selection.getSelection() as any) }, selectionMode: SelectionMode.multiple })

   selectedAgents: GetPendingAgentResponse[] = [];

   requests: GetPendingAgentResponse[] = [];

   initial = true;
}

const handler = new AgentRequestsHandler();

const AgentsPanel: React.FC = observer(() => {

   const [confirmRegister, setConfirmRegister] = useState(true);
   const [submitting, setSubmitting] = useState(false);
   const navigate = useNavigate();

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { modeColors } = getHordeStyling();

   // subscribe
   if (handler.updated) { };

   const onRegister = async () => {

      const agents = handler.selectedAgents.map(a => { return { key: a.key } })

      const queryAgents = async (queryTime: string) => {

         const nagents = await backend.getAgents({ modifiedAfter: queryTime, invalidateCache: true });
         if (nagents.length) {
            await agentStore.update(false);
            navigate("/agents");
         } else {
            setTimeout(queryAgents, 2000);
         }
      }

      try {
         setSubmitting(true);
         const queryTime = moment.utc().toISOString();
         await backend.registerAgents({ agents: agents });
         setTimeout(queryAgents, 3000);
         
      } catch (reason) {
         console.error(reason);

         errorDialogStore.set({
            reason: reason,
            title: `Error Enrolling Agents`,
            message: `There was an error enrolling agents, reason: "${reason}"`

         }, true);

         setSubmitting(false);
      }
   }

   const columns = [
      { key: 'column_hostname', name: 'Hostname', fieldName: 'hostName', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_description', name: 'Description', fieldName: 'description', minWidth: 120, maxWidth: 120, isResizable: false },
   ];

   let requests = [...handler.requests];

   requests = requests.sort((a, b) => a.hostName.localeCompare(b.hostName));

   const renderItem = (item: any, index?: number, column?: IColumn) => {
      if (!column?.fieldName) {
         return null;
      }
      return <Text style={{ color: modeColors.text }}>{item[column?.fieldName]}</Text>
   };

   const { hordeClasses } = getHordeStyling();

   let title = handler.selectedAgents.length > 1 ? "Enroll Agents" : "Enroll Agent";
   if (handler.selectedAgents.length == 1) {
      title += " " + handler.selectedAgents[0].hostName
   }
   let agentText = handler.selectedAgents.length > 1 ? "these agents" : "this agent";

   return <Stack>
      {submitting && <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="mediumPlus">Enrolling, please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>}
      {confirmRegister && handler.selectedAgents.length > 0 &&
         <Dialog
            hidden={false}
            onDismiss={() => setConfirmRegister(false)}
            minWidth={612}
            dialogContentProps={{
               type: DialogType.normal,
               title: title,
               subText: `Do you want to enroll ${agentText}? This will allow ${agentText} to take on work assigned to it by the server: ${handler.selectedAgents.map(a => a.hostName).join(", ")}`
            }}
            modalProps={{ isBlocking: true, topOffsetFixed: true, styles: { main: { padding: 8, width: 400, hasBeenOpened: false, top: "120px", position: "absolute" } } }} >
            <Stack style={{ height: "18px" }} />
            <DialogFooter>
               <PrimaryButton onClick={() => { setConfirmRegister(false); onRegister() }} text="Register" />
               <DefaultButton onClick={() => setConfirmRegister(false)} text="Cancel" />
            </DialogFooter>
         </Dialog>
      }
      {<Stack style={{ paddingBottom: 12 }}>
         <Stack verticalAlign="center">
            {!!requests.length && <Stack horizontalAlign="end">
               <PrimaryButton disabled={!handler.selectedAgents.length} styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }} onClick={() => setConfirmRegister(true)}>Enroll Agents</PrimaryButton>
            </Stack>}
            {!requests.length && <Stack horizontalAlign="center" tokens={{ childrenGap: 12 }}>
               <Stack>
                  <Text variant="mediumPlus">No enrollment requests found, waiting for agents to connect</Text>
               </Stack>
               <Stack>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>}
         </Stack>
      </Stack>}

      {!!requests.length && <Stack className={hordeClasses.raised} >
         <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
            <Stack>
               <SelectionZone selection={handler.selection}>
                  <DetailsList
                     setKey="set"
                     items={requests}
                     columns={columns}
                     layoutMode={DetailsListLayoutMode.justified}
                     compact={true}
                     selectionMode={SelectionMode.multiple}
                     selection={handler.selection}
                     selectionPreservedOnEmptyClick={true}
                     onRenderItemColumn={renderItem}
                     enableUpdateAnimations={false}
                     onShouldVirtualize={() => false}
                  />
               </SelectionZone>
            </Stack>
         </Stack>
      </Stack>}
   </Stack >
});


export const AgentRequestsView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Agent Enrollment' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <AgentsPanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

