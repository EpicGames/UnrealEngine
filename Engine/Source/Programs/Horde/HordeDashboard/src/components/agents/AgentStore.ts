// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from 'mobx';
import backend from '../../backend';
import { AgentData, CategoryAgents, GetAgentResponse, PoolData } from '../../backend/Api';

class AgentStore {

    constructor() {
        makeObservable(this);
    }

    @observable
    agentsUpdated: number = 0

    get agents(): AgentData[] {
        // subscribe in any observers
        if (this.agentsUpdated) { }
        return this._agents;
    }

    private _agents: AgentData[] = [];

    @observable
    poolsUpdated: number = 0;

    get pools(): PoolData[] {
        // subscribe in any observers
        if (this.poolsUpdated) { }
        return this._pools;
    }

    private _pools: PoolData[] = [];

    // don't need this for now
    modifiedAfterDate?: Date;

    inflight?: boolean;

    @action
    private _setPools(data: PoolData[]) {
        this._pools = data;
        this.poolsUpdated++;
    }

    @action
    private _setAgents(data: AgentData[]) {
        if (data.length !== 0) {
            const toSet: AgentData[] = [...this._agents];
            data.forEach(updatedAgent => {
                const newUpdateIdx = toSet.findIndex(existingAgent => existingAgent.id === updatedAgent.id);
                if (newUpdateIdx !== -1) {
                    toSet[newUpdateIdx] = updatedAgent;
                }
                else {
                    toSet.push(updatedAgent);
                }
            });            
            this._agents = toSet.filter(a => true /*!!a.deleted*/);            
        } else {
            this._agents = [];
        }
        this.agentsUpdated++;
    }

    async updateAgent(agentId: string): Promise<GetAgentResponse> {
        return new Promise<GetAgentResponse>(async (resolve, reject) => {
            try {
                const agent = await backend.getAgent(agentId);
                this._agents = this._agents.filter(a => a.id !== agentId);
                this._agents.push(agent);

                const load = agent.pools?.filter(p => !this._pools.find(pool => pool.id === p));
                if (load?.length) {
                    this._pools = await backend.getPools()
                }                

                resolve(agent);
            } catch (error) {
                reject(error);
            }
        })
    }

    categoryAgents = new Map<string, CategoryAgents>();

    private forceUpdate() {
        this._setAgents([...this._agents]);
    }

    getCategory(condition: string): CategoryAgents {

        let agents = this.categoryAgents.get(condition);

        if (agents && ( agents.polling || (Date.now() - agents.lastPoll.getTime()) / 1000 < 60)) {
            return agents;
        }
        
        agents = { ids: agents?.ids ?? [], lastPoll: new Date(), polling: !agents };
        this.categoryAgents.set(condition, agents);

        const filter = "id";
        backend.getAgents({ condition: condition, filter: filter }).then((agents) => {

            const oldAgents = new Set(this.categoryAgents.get(condition)?.ids ?? []);
            const ids = agents.map(a => a.id);        
            this.categoryAgents.set(condition, { ids: ids, lastPoll: new Date() })

            const newAgents = new Set(ids);

            if (oldAgents.size !== newAgents.size || ![...oldAgents].every(x => newAgents.has(x))) {
                // force update
                this.forceUpdate();    
            }
        });
                
        return agents;

    }

    async updatePools(): Promise<void> {
        const pools = await backend.getPools();
        this._setPools(pools);
    }

    async update(slim:boolean = false, invalidateCache: boolean = false): Promise<void> {
        return new Promise<void>((resolve, reject) => {

            if (this.inflight !== undefined && slim) {
                resolve();
                return;
            }

            if (invalidateCache) {
                this.modifiedAfterDate = undefined;
            }

            this.inflight = true;

            const filter = "id,name,sessionId,sessionExpiresAt,online,enabled,ephemeral,comment,version,forceVersion,pools,capabilities,leases,acl,updateTime,deleted,pendingConform,pendingFullConform,conformAttemptCount,lastConformTime,nextConformTime,lastShutdownReason,pendingShutdown,status";
            const promises: any[] = [];
            promises.push(backend.getAgents({ includeDeleted: false, modifiedAfter: this.modifiedAfterDate?.toISOString(), filter:filter,invalidateCache:invalidateCache}));
            if (!slim) {
                promises.push(backend.getPools());
            }
            Promise.all(promises).then(values => {
                this.modifiedAfterDate = new Date();
                this._setAgents(values[0]);
                if (!slim) {
                    this._setPools(values[1]);
                }
                resolve();
            }).catch(reason => {
                console.error(`Error getting agents and pools: ${reason}`);
            }).finally(() => {
                this.inflight = undefined;
            });
        });
    }

    filterValidPools(): PoolData[] {
        return this.pools.filter(pool => pool.name !== "");
    }
}

export const agentStore = new AgentStore();