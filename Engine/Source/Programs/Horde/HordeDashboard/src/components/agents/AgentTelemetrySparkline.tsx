// Copyright Epic Games, Inc. All Rights Reserved.

import { Label, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import * as d3 from "d3";
import { observer } from "mobx-react-lite";
import { Sparklines, SparklinesLine, SparklinesReferenceLine } from "react-sparklines";
import backend from "../../backend";
import { GetAgentResponse, GetAgentTelemetrySampleResponse } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { PollBase } from "../../backend/PollBase";

//  http://localhost:5173/log/6849950e9935f922c0cc2339?batch=e043&agentId=10-99-112-190

const sparkWidth = 1380 / 2;
const sparkHeight = 64;

export class AgentTelemetryHandler extends PollBase {

   constructor() {
      super(15000);

   }

   show(visible: boolean) {

      if (this.visible === visible) {
         return;
      }

      visible ? this.start() : this.stop();

      this.visible = visible;
      this.setUpdated();

   }

   setActive(value: boolean) {
      if (this.active === value) {
         return;
      }

      this.active = value;
   }

   set(agentId: string, startTime: Date, endTime?: Date) {

      this.agentId = agentId;
      this.startTime = startTime;
      this.endTime = endTime;
   }

   setCurrentTime(currentTime?: Date) {

      if (!this.visible) {
         this.currentTime = undefined;
         return;
      }

      this.currentTime = currentTime;

      // chart selectors
      const charts = ["#telemetry_cpu > svg", "#telemetry_ram > svg", "#telemetry_disk > svg"]

      const startTime = this.lastStartTime;
      const endTime = this.lastEndTime;

      if (this.currentTime && startTime && endTime) {

         let x = (this.currentTime.getTime() - startTime.getTime()) / (endTime.getTime() - startTime.getTime());

         if (x < 0 || x > 1) {
            return;
         }

         x *= sparkWidth;
         x = Math.floor(x);

         if (this.lastX === x) {
            return;
         }

         this.lastX = x;

         charts.forEach(chartSelector => {

            const chart = d3.select(chartSelector);

            if (!chart) {
               return;
            }

            chart.selectAll(".cline").remove();

            chart.append("line")
               .attr("class", "cline")
               .attr("x1", () => x)
               .attr("x2", () => x)
               .attr("y1", () => 0)
               .attr("y2", () => sparkHeight)
               .attr("stroke-width", () => 2)
               .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4BAA")

         })

      }

   }

   async poll(): Promise<void> {


      try {

         if (!this.agentId || !this.startTime) {
            return;
         }

         if (!this.active && this.data?.length) {
            return;
         }

         let endTime = this.endTime;
         if (!endTime) {
            endTime = new Date();
         }

         // clamp to 24 hours
         if ((endTime.getTime() - this.startTime.getTime()) > 86400000) {
            endTime = new Date(this.startTime.getTime() + 86400000);
         }

         if (!this.agent) {
            this.agent = await backend.getAgent(this.agentId);
         }

         if (this.startTime === this.lastStartTime && this.endTime === this.lastEndTime) {
            return;
         }

         this.data = await backend.getAgentTelemetry(this.agentId, this.startTime, endTime);

         this.lastStartTime = this.startTime;
         this.lastEndTime = endTime;

         this.setUpdated();

      } catch (err) {
         console.error(err);
      }
   }

   active = true;
   lastX: number = -1;
   lastStartTime?: Date;
   lastEndTime?: Date;

   data?: GetAgentTelemetrySampleResponse[];

   agent?: GetAgentResponse;

   visible = false;
   agentId?: string;
   startTime?: Date;
   endTime?: Date;
   currentTime?: Date;

}

export const AgentTelemetrySparkline: React.FC<{ handler: AgentTelemetryHandler }> = observer(({ handler }) => {


   handler.subscribe();

   const data = handler.data;

   if (data === undefined) {
      return <Stack horizontal horizontalAlign="center" style={{ padding: 32 }}>
         <Spinner size={SpinnerSize.large} />
      </Stack>
   }

   if (!data.length) {
      return <Stack horizontal horizontalAlign="center" style={{ padding: 32 }}>
         <Text variant="mediumPlus">No Telemetry Data</Text>
      </Stack>
   }

   if (!handler.visible || !data?.length) {
      return;
   }

   let cpuText = "";
   let ramText = "";
   let diskText = "";

   let device: any;
   const devices = handler.agent?.capabilities?.devices;
   if (devices?.length) {
      device = devices[0];
   }

   device?.properties?.forEach(v => {
      if (v.startsWith("CPU=")) {
         cpuText = `${v.replace("CPU=", "")}`;
      }

      if (v.startsWith("RAM=")) {
         ramText = `${v.replace("RAM=", "")} GB`;
      }

      if (v.startsWith("WorkingDiskTotalSize=")) {
         const elements = v.split("=");
         if (elements.length === 2) {

            function formatBytes(bytes: number, decimals = 0) {
               if (!+bytes) return '0 Bytes'

               const k = 1024
               const dm = decimals < 0 ? 0 : decimals
               const sizes = ['Bytes', 'KB', 'MB', 'GB', 'TB', 'PB', 'EB', 'ZB', 'YB']

               const i = Math.floor(Math.log(bytes) / Math.log(k))

               return `${parseFloat((bytes / Math.pow(k, i)).toFixed(dm))} ${sizes[i]}`
            }

            diskText = formatBytes(parseInt(elements[1]))
         }
      }
   })

   const ramData = data.map(d => {
      return 1.0 - (d.freeRam / d.totalRam);
   })

   const cpuData = data.map(d => {
      return (d.userCpu + d.systemCpu) / 100;
   })

   const diskData = data.map(d => {
      if (typeof (d.workingTotalDisk) === "number" && d.workingTotalDisk) {
         return 1.0 - (d.workingFreeDisk / d.workingTotalDisk)
      }

      return 0;
   })

   const statusColors = dashboard.getStatusColors();
   const opacity = 0.75;

   return <Stack>
      <Stack horizontal horizontalAlign="center" tokens={{ childrenGap: 24 }} style={{ paddingBottom: 8 }}>
         <Stack style={{ width: sparkWidth }}>
            <Stack horizontal>
               <Label>Used CPU</Label>
               <Stack grow></Stack>
               {!!cpuText && <Label>{cpuText}</Label>}
            </Stack>
            <div id="telemetry_cpu">
               <Sparklines width={sparkWidth} height={sparkHeight} data={cpuData} min={0} max={1} style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
                  <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} style={{ strokeWidth: 2 }} />
                  <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                  <SparklinesReferenceLine type="custom" value={22} style={{ stroke: statusColors.get(StatusColor.Warnings), opacity: opacity }} />
                  <SparklinesReferenceLine type="custom" value={44} style={{ stroke: statusColors.get(StatusColor.Success), opacity: opacity }} />
               </Sparklines>
            </div>
         </Stack>
         <Stack style={{ width: sparkWidth }}>
            <Stack horizontal>
               <Label>Used RAM</Label>
               <Stack grow></Stack>
               {!!ramText && <Label>{ramText}</Label>}
            </Stack>
            <div id="telemetry_ram">
               <Sparklines width={sparkWidth} height={sparkHeight} data={ramData} min={0} max={1} style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
                  <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} style={{ strokeWidth: 2 }} />
                  <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                  <SparklinesReferenceLine type="custom" value={22} style={{ stroke: statusColors.get(StatusColor.Warnings), opacity: opacity }} />
                  <SparklinesReferenceLine type="custom" value={44} style={{ stroke: statusColors.get(StatusColor.Success), opacity: opacity }} />
               </Sparklines>
            </div>
         </Stack>
      </Stack>
      <Stack style={{ width: sparkWidth, paddingLeft: 4, paddingBottom: 4 }}>
         <Stack horizontal>
            <Label>Used Disk</Label>
            <Stack grow></Stack>
            {!!diskText && <Label>{diskText}</Label>}
         </Stack>
         <div id="telemetry_disk" style={{ width: 690 }}>
            <Sparklines width={sparkWidth} height={sparkHeight} data={diskData} min={0} max={1} style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
               <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} style={{ strokeWidth: 2 }} />
               <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
               <SparklinesReferenceLine type="custom" value={22} style={{ stroke: statusColors.get(StatusColor.Warnings), opacity: opacity }} />
               <SparklinesReferenceLine type="custom" value={44} style={{ stroke: statusColors.get(StatusColor.Success), opacity: opacity }} />
            </Sparklines>
         </div>
      </Stack>

   </Stack>
})