// Copyright Epic Games, Inc. All Rights Reserved.
import { Checkbox, CommandButton, ConstrainMode, ContextualMenu, DefaultButton, DetailsHeader, DetailsList, DetailsListLayoutMode, Dialog, DialogType, DirectionalHint, Dropdown, FontIcon, FontSizes, FontWeights, IBasePickerProps, IColumn, IContextualMenuItem, IContextualMenuProps, IDetailsHeaderProps, IDetailsHeaderStyles, IDetailsListProps, ITag, ITagItemStyles, ITooltipHostStyles, Icon, IconButton, Pivot, PivotItem, PrimaryButton, ProgressIndicator, Link as ReactLink, ScrollablePane, ScrollbarVisibility, SearchBox, Selection, SelectionMode, Slider, Spinner, SpinnerSize, Stack, Sticky, StickyPositionType, TagItem, TagPicker, Text, TextField, TooltipHost, mergeStyleSets } from '@fluentui/react';
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { createRef, useEffect, useState } from 'react';
import { Link, useSearchParams } from 'react-router-dom';
import Marquee from 'react-text-marquee';
import backend from '../../backend';
import { agentStore } from './AgentStore';
import { AgentData, BatchUpdatePoolRequest, GetAgentResponse, GetDashboardAgentCategoryResponse, LeaseState, PoolData } from '../../backend/Api';
import dashboard, { StatusColor } from '../../backend/Dashboard';
import { copyToClipboard } from '../../base/utilities/clipboard';
import { useWindowSize } from '../../base/utilities/hooks';
import { getShortNiceTime } from '../../base/utilities/timeUtils';
import { getHordeStyling, hexToRGB, linearInterpolate } from '../../styles/Styles';
import { getHordeTheme } from '../../styles/theme';
import { Breadcrumbs } from '../Breadcrumbs';
import { ConfirmationDialog } from '../ConfirmationDialog';
import { HistoryModal } from './HistoryModal';
import { TopNav } from '../TopNav';

let _agentStyles: any;
const getAgentStyles = () => {
   const theme = getHordeTheme();

   const agentStyles = _agentStyles ?? mergeStyleSets({

      ticker: {
         width: '100%'
      },
      checkboxCell: {
         selectors: {
            '> div': {
               height: '100%'
            }
         }
      },
      dialog: {
         selectors: {
            ".ms-Label,.ms-Button-label": {
               fontWeight: "unset",
               fontFamily: "Horde Open Sans SemiBold"
            }
         }
      },
      detailsList: {
         selectors: {
            ".ms-DetailsHeader-cellName": {
               fontWeight: "unset",
               fontFamily: "Horde Open Sans SemiBold"
            },
            ".ms-DetailsRow #editagent": {
               opacity: 0
            },
            ".ms-DetailsRow:hover #editagent": {
               opacity: 1
            }
         }
      },
      descFont: {
         font: '8pt Horde Open Sans SemiBold !important',
         marginLeft: 12,
         marginRight: 12,
         marginBottom: 4
      },
      buttonFont: {
         height: '26px',
         font: '7pt Horde Open Sans SemiBold !important',
         flexShrink: '0 !important',
         paddingLeft: 6,
         paddingRight: 6,
         selectors: {
            '.ms-Icon': {
               width: 0,
               margin: 0
            },
            ':active': {
               textDecoration: 'none'
            },
            ':hover': {
               textDecoration: 'none'
            },
            ':visited': {
               textDecoration: 'none',
               color: "#FFFFFF"
            }

         }
      },
      modalHeader: [
         {
            font: '24px Horde Open Sans Light',
            flex: '1 1 auto',
            color: theme.palette.neutralPrimary,
            display: 'flex',
            fontSize: FontSizes.xLarge,
            alignItems: 'center',
            fontWeight: FontWeights.semibold,
            padding: '12px 12px 14px 24px'
         }
      ],
      modalBody: {
         flex: '4 4 auto',
         padding: '0 24px 24px 24px',
         overflowY: 'hidden',
         selectors: {
            p: {
               margin: '14px 0'
            },
            'p:first-child': {
               marginTop: 0
            },
            'p:last-child': {
               marginBottom: 0
            }
         }
      },
      ellipsesStackItem: {
         whiteSpace: 'nowrap',
         overflow: 'hidden',
         textOverflow: 'ellipsis'
      }
   });

   _agentStyles = agentStyles;

   return agentStyles;
}



// column in the main table
type ColumnItem = {
   key: string;
   displayText: string;
   colSize: number;
   isChecked: boolean;
   isCheckable: boolean;
   isSorted: boolean;
   isSortedDescending: boolean;
   columnDef: IColumn | undefined;
}

type StorageColumnConfig = {
   key: string;
   displayText: string;
   freeCapability: string;
   totalCapability: string;
   searchStateKey: string;
}

const STORAGE_COLUMNS: Record<string, StorageColumnConfig> = {
   workingStorage: {
      key: 'workingStorage',
      displayText: 'Working Storage',
      freeCapability: 'WorkingDiskFreeSpace',
      totalCapability: 'WorkingDiskTotalSize',
      searchStateKey: 'workingStorage'
   },
   systemStorage: {
      key: 'systemStorage',
      displayText: 'System Storage',
      freeCapability: 'SystemDiskFreeSpace',
      totalCapability: 'SystemDiskTotalSize',
      searchStateKey: 'systemStorage'
   }
};

const BYTES_PER_GIB = 1024 * 1024 * 1024; // 1024^3

function getAgentCapability(agent: AgentData, inProp: string) {
   if (agent.capabilities?.devices) {
      for (const deviceIdx in agent.capabilities?.devices) {
         const device = agent.capabilities?.devices[deviceIdx];
         if (device.properties) {
            for (const propIdx in device.properties) {
               const prop = device.properties[propIdx];
               if (prop.indexOf(inProp) !== -1) {
                  return prop.split('=')[1];
               }
            }
         }
      }
   }
   return null;
}

function getAgentOs(agent: AgentData) {
   const osDist = getAgentCapability(agent, "OSDistribution");
   const osFamily = getAgentCapability(agent, "OSFamily");
   return osDist ?? osFamily;
}

function getTaskTime(agent: GetAgentResponse): number {

   let max: moment.Duration | undefined
   if (agent.leases) {
      agent.leases.forEach(lease => {
         const start = moment(lease.startTime);
         const end = moment(Date.now());
         const duration = moment.duration(end.diff(start));
         if (!max) {
            max = duration;
         }
         else {
            if (duration.asSeconds() > max.asSeconds()) {
               max = duration;
            }
         }
      });
   }

   return max?.asSeconds() ?? 0;

}

type SearchState = {
   agentId?: string;
   agentSearch?: string;
   exactSearch?: boolean;
   filter?: string[];
   columnMode?: string;
   category?: string;
}

class LocalState {

   // agent update function params
   editingPools = false;  // if editing pools dialog is open

   // text in the filter search
   @observable agentFilter = "";
   @observable filterExactMatch: boolean = false;
   @observable agentStatusFilter = new Set<string>();

   // table selection
   selection: Selection = new Selection({ onSelectionChanged: () => { this.setAgentsSelectedCount(this.selection.getSelectedCount()); } });
   currentSelection: AgentData[] = [];
   // number of agents selected for the button
   @observable agentsSelectedCount = 0;

   @observable deleteAgentDialogIsOpen = false;
   @observable restartAgentDialogIsOpen = false;
   @observable shutdownAgentDialogIsOpen = false;
   @observable disableAgentDialogIsOpen = false;
   @observable cancelLeasesDialogIsOpen = false;
   @observable editCommentDialogIsOpen = false;

   // column state for the table
   @observable columnsState: ColumnItem[];
   columnSearchState: any = {};
   @observable columnMenuProps: IContextualMenuItem[];

   @observable headerContextMenuOpen = false;
   @observable agentContextMenuOpen = false;
   contextMenuRef: any = null;
   contextMenuTargetRef: any = createRef<Element>();
   @observable mouseX = 0;
   @observable mouseY = 0;

   @action
   resetState() {
      this.agentFilter = "";
      this.filterExactMatch = false;
      this.selection = new Selection({ onSelectionChanged: () => { this.setAgentsSelectedCount(this.selection.getSelectedCount()); } });
      this.currentSelection = [];
      this.deleteAgentDialogIsOpen = false;
      this.searchState = {};
   }

   @action
   setRightClickDiv(x: number, y: number) {
      this.mouseX = x;
      this.mouseY = y;
   }

   @action
   setAgentFilter(filter: string) {
      this.searchState.agentSearch = filter?.trim() ? filter.trim() : undefined;
      this.updateSearch();
      this.agentFilter = filter;
   }

   @action
   setAgentStaus(filter: Set<string>) {
      const filters = Array.from(filter);
      this.searchState.filter = filters.length ? filters : undefined;
      this.updateSearch();
      this.agentStatusFilter = filter;
   }

   @action
   setExactMatch(match: boolean) {
      this.searchState.exactSearch = match ? true : undefined;
      this.updateSearch();
      this.filterExactMatch = match;
   }

   @action
   setAgentId(id?: string) {
      this.searchState.agentId = id;
      this.updateSearch();
   }

   @action
   setAgentCategory(category?: string) {
      this.searchState.category = category?.trim();
      this.updateSearch();
   }

   @action
   private _onColumnClick(ev: React.MouseEvent<HTMLElement>, column: IColumn) {
      if (column.key === "pools") {
         return;
      }
      this.columnsState.forEach(colState => {
         if (colState.columnDef!.key === column.key) {
            colState.isSortedDescending = !colState.isSortedDescending;
            colState.isSorted = true;
         }
         else {
            colState.isSorted = false;
            colState.isSortedDescending = false;
         }
      });
      this._updateColumnDefs();
   }

   // enable or disable builders.
   async changeBuilderEnabled(enabled: string, disableText?: string) {
      let that = this;
      const enabledVal: boolean = enabled === "enable" ? true : false;
      const allUpdates: any[] = [];
      const selectedAgents = this.currentSelection;
      selectedAgents.forEach(agent => {
         // add to update queue if the agent isnt set to the value
         if (agent.enabled !== enabledVal) {
            allUpdates.push(backend.updateAgent(agent.id, { enabled: enabledVal, comment: disableText ?? "" }));
         }
      });
      await Promise.all(allUpdates).then(function (responses) {
         if (!enabledVal) {
            that.setDisableBuilderDialogOpen(false);
         }
      }).catch(function (errors) {
      }).finally(function () {
         agentStore.update();
      });
   }

   // enable or disable builders.
   async changeCommment(commentText?: string) {
      let that = this;
      const comment = commentText ?? "";
      const allUpdates: any[] = [];
      const selectedAgents = this.currentSelection;
      selectedAgents.forEach(agent => {
         // add to update queue if the agent isnt set to the value
         if (agent.comment !== comment) {
            allUpdates.push(backend.updateAgent(agent.id, { comment: comment }));
         }
      });
      await Promise.all(allUpdates).then(function () {
         that.setEditCommentDialogOpen(false);
      }).catch(function (errors) {
      }).finally(function () {
         agentStore.update();
      });
   }

   async requestBuilderUpdate(conform: boolean, restart: boolean, fullConform?: boolean, shutdown?: boolean, forceRestart?: boolean) {
      let that = this;
      const allUpdates: any[] = [];
      const selectedAgents = this.currentSelection;
      selectedAgents.forEach(agent => {
         allUpdates.push(backend.updateAgent(agent.id, { requestConform: conform, requestFullConform: fullConform, requestRestart: restart, requestShutdown: shutdown, requestForceRestart: !!forceRestart }));
      });
      await Promise.all(allUpdates).then(function (responses) {
         if (restart) {
            that.setRestartBuilderDialogOpen(false);
         }
         if (shutdown) {
            that.setShutdownBuilderDialogOpen(false);
         }
      }).catch(function (errors) {
      }).finally(function () {
         agentStore.update();
      });
   };

   async cancelLeases() {

      const selectedAgents = this.currentSelection;

      const leases = new Set<string>();

      selectedAgents.forEach(agent => {
         agent.leases?.forEach(lease => {
            if (!lease.finishTime) {
               leases.add(lease.id);
            }
         })
      });

      let requests = Array.from(leases).map(id => {
         return backend.updateLease(id, { aborted: true });
      });

      while (requests.length) {

         const batch = requests.slice(0, 5);

         await Promise.all(batch).then(() => {

         }).catch((errors) => {
            console.log(errors);
            // eslint-disable-next-line
         }).finally(() => {

            requests = requests.slice(5);
         });
      }

      agentStore.update();

   };


   @action setDisableBuilderDialogOpen(isOpen: boolean) {
      this.disableAgentDialogIsOpen = isOpen;
   }

   @action setEditCommentDialogOpen(isOpen: boolean) {
      this.editCommentDialogIsOpen = isOpen;
   }

   @action setRestartBuilderDialogOpen(isOpen: boolean) {
      this.restartAgentDialogIsOpen = isOpen;
   }

   @action setShutdownBuilderDialogOpen(isOpen: boolean) {
      this.shutdownAgentDialogIsOpen = isOpen;
   }

   @action setCancelLeasesDialogOpen(isOpen: boolean) {
      this.cancelLeasesDialogIsOpen = isOpen;
   }

   @action setDeleteBuilderDialogOpen(isOpen: boolean) {
      this.deleteAgentDialogIsOpen = isOpen;
   }

   async deleteBuilder() {
      let that = this;
      const allUpdates: any[] = [];
      const selectedAgents = this.currentSelection;
      selectedAgents.forEach(agent => {
         allUpdates.push(backend.deleteAgent(agent.id));
      });
      await Promise.all(allUpdates).then(function (responses) {
         that.setDeleteBuilderDialogOpen(false);
      }).catch(function (errors) {
      }).finally(function () {
         that.selection.setAllSelected(false);
         that.currentSelection = [];
         agentStore.update(false, true);
      });
   };

   // sets a "Columns..." button item checked or not by updating the col state
   @action
   setColumnItemChecked(key: string) {
      this.columnsState.filter(colState => { return colState.isCheckable; }).forEach(colState => {
         if (colState.key === key) {
            colState.isChecked = true;
            this.searchState.columnMode = key ? key : undefined;
            this.updateSearch();

         }
         else {
            colState.isChecked = false;
         }
      });

      //this.searchState.columnMode = key;
      //this.updateSearch();

      this.columnMenuProps = this._updateColumnProps();
   }

   private _updateColumnDefs() {
      this.columnsState.forEach(colState => {

         let colSize = colState.colSize;
         
         // True when colState is for a column type that can be selected for the last column
         const bIsSwapColumn: boolean = (
            colState.key === "software"
         || colState.key === "workingStorage"
         || colState.key === "systemStorage"
         || colState.key.indexOf("systemInfo") !== -1
         || colState.key === "taskTime"
         || colState.key === "comment"
         );

         if (!this.agentView && colState.displayText === "Pools") {
            colSize = 480;
         }

         colState.columnDef = {
            key: colState.key,
            name: colState.displayText,
            minWidth: colSize,
            maxWidth: colSize,
            isResizable: false,
            isSorted: colState.key === "pools" ? undefined : colState.isSorted,
            isSortedDescending: colState.key === "pools" ? undefined : colState.isSortedDescending,
            onColumnClick: this._onColumnClick.bind(this),
            styles: bIsSwapColumn ? {
                  cellName: {
                     flexGrow: '1' // Grow the cell name element so there can be space between it's elements
                  }
            } : undefined,
            onRenderHeader: bIsSwapColumn ? () => {
               return <Stack horizontal>
                  <span style={{flexGrow: 1, textAlign: 'center'}}>{colState.displayText}</span>
                  <TooltipHost
                     content="Change column content"
                  >
                     <IconButton
                        iconProps={{ iconName: 'Edit' }}
                        onClick={(ev) => {
                           ev.preventDefault();
                           ev.stopPropagation(); // Consume the input before it causes a sort re-order
                           localState.contextMenuTargetRef = ev?.nativeEvent!;
                           localState.setHeaderContextMenuOpen(true);
                        }
                     }/>
                  </TooltipHost>
               </Stack>
            }
            : undefined,
            onRender: colState.key === "editAgent" ? () => {
               return <Stack style={{ cursor: "pointer" }} verticalFill verticalAlign='center' horizontalAlign='start' onClick={(ev) => {
                  localState.setRightClickDiv(ev?.clientX, ev?.clientY);
                  localState.setAgentContextMenuOpen(true);
               }
               }><FontIcon id="editagent" style={{ fontSize: 14 }} iconName="Edit" />
               </Stack>
            } : undefined
         };

      });
   }

   private _updateColumnProps() {
      return [
         {
            canCheck: true,
            key: 'software',
            checked: this.columnsState.find(c => c.key === 'software')!.isChecked,
            text: this.columnsState.find(c => c.key === 'software')!.displayText,
            onClick: () => this.setColumnItemChecked('software')
         },
         {
            canCheck: true,
            key: 'taskTime',
            checked: this.columnsState.find(c => c.key === 'taskTime')!.isChecked,
            text: this.columnsState.find(c => c.key === 'taskTime')!.displayText,
            onClick: () => this.setColumnItemChecked('taskTime')
         },
         {
            canCheck: true,
            key: 'workingStorage',
            checked: this.columnsState.find(c => c.key === 'workingStorage')!.isChecked,
            text: this.columnsState.find(c => c.key === 'workingStorage')!.displayText,
            onClick: () => this.setColumnItemChecked('workingStorage')
         },
         {
            canCheck: true,
            key: 'systemStorage',
            checked: this.columnsState.find(c => c.key === 'systemStorage')!.isChecked,
            text: this.columnsState.find(c => c.key === 'systemStorage')!.displayText,
            onClick: () => this.setColumnItemChecked('systemStorage')
         },
         {
            canCheck: true,
            key: 'comment',
            checked: this.columnsState.find(c => c.key === 'comment')!.isChecked,
            text: this.columnsState.find(c => c.key === 'comment')!.displayText,
            onClick: () => this.setColumnItemChecked('comment')
         },
         {
            canCheck: true,
            key: 'OS',
            checked: this.columnsState.find(c => c.key === 'systemInfoOS')!.isChecked,
            text: this.columnsState.find(c => c.key === 'systemInfoOS')!.displayText,
            onClick: () => this.setColumnItemChecked('systemInfoOS')
         },
         {
            canCheck: true,
            key: 'CPU',
            checked: this.columnsState.find(c => c.key === 'systemInfoCPU')!.isChecked,
            text: this.columnsState.find(c => c.key === 'systemInfoCPU')!.displayText,
            onClick: () => this.setColumnItemChecked('systemInfoCPU')
         },
         {
            canCheck: true,
            key: 'RAM',
            checked: this.columnsState.find(c => c.key === 'systemInfoRAM')!.isChecked,
            text: this.columnsState.find(c => c.key === 'systemInfoRAM')!.displayText,
            onClick: () => this.setColumnItemChecked('systemInfoRAM')
         }
      ];
   }

   // sets the number of agents selected
   @action
   setAgentsSelectedCount(count: number) {
      this.agentsSelectedCount = count;
   }

   @action
   setHeaderContextMenuOpen(isOpen: boolean) {
      this.agentContextMenuOpen = false;
      this.headerContextMenuOpen = isOpen;
   }

   @action
   setAgentContextMenuOpen(isOpen: boolean) {
      if (isOpen) {
         this.currentSelection = [...this.selection.getSelection() as AgentData[]];
      }
      this.agentContextMenuOpen = isOpen;
      this.headerContextMenuOpen = false;
   }


   @observable
   searchUpdated: number = 0;

   @action
   setSearchUpdated() {
      this.searchUpdated++;
   }


   searchState: SearchState = {};
   search: URLSearchParams = new URLSearchParams();

   updateSearch(): boolean {

      const state = { ...this.searchState } as SearchState;

      state.filter = state.filter?.sort((a, b) => a.localeCompare(b));

      const search = new URLSearchParams();
      const csearch = this.search.toString();

      if (state.agentSearch) {
         search.append("agent", state.agentSearch);
      }

      if (state.exactSearch) {
         search.append("exact", "true");
      }

      if (state.agentId) {
         search.append("agentId", state.agentId);
      }

      if (state.category) {
         search.append("category", encodeURIComponent(state.category));
      }

      state.filter?.forEach(f => {
         if (f) {
            search.append("filter", f);
         }
      });

      if (state.columnMode) {
         search.append("mode", state.columnMode);
      }

      if (search.toString() !== csearch) {
         this.search = search;
         this.setSearchUpdated();
         return true;
      }

      return false;
   }

   stateFromSearch(search: URLSearchParams) {

      const state: SearchState = {};

      const filters = search.getAll("filter") ?? undefined;
      const agentSearch = search.get("agent") ?? undefined;
      const agentId = search.get("agentId") ?? undefined;
      const exact = search.get("exact") ?? undefined;
      const mode = search.get("mode") ?? undefined;
      const category = search.get("category") ?? undefined;

      state.filter = filters?.sort((a, b) => a.localeCompare(b));
      state.columnMode = mode?.length ? mode : undefined;
      state.agentSearch = agentSearch?.length ? agentSearch : undefined;
      state.exactSearch = exact?.trim() === "true" ? true : undefined;
      state.agentId = agentId?.trim() ? agentId : undefined;
      state.category = category?.trim() ? category : undefined;

      this.search = search;
      this.searchState = state;

      if (state.agentSearch) {
         this.agentFilter = state.agentSearch;
      }
      if (state.exactSearch) {
         this.filterExactMatch = true;
      }
      if (state.filter?.length) {
         this.agentStatusFilter = new Set<string>(state.filter);
      }

      if (state.columnMode) {
         this.setColumnItemChecked(state.columnMode);
      }

      return state;
   }

   agentView: boolean | undefined = true;

   constructor() {
      makeObservable(this);
      this.columnsState = [
         {
            key: 'name',
            displayText: 'Name',
            colSize: 170,
            isChecked: true,
            isCheckable: false,
            isSorted: true,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'pools',
            displayText: 'Pools',
            colSize: 476,
            isChecked: true,
            isCheckable: false,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'status',
            displayText: 'Status',
            colSize: 300,
            isChecked: true,
            isCheckable: false,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'software',
            displayText: 'Software',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'taskTime',
            displayText: 'Task Time',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'workingStorage',
            displayText: 'Working Storage',
            colSize: 220,
            isChecked: true,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'systemStorage',
            displayText: 'System Storage',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'comment',
            displayText: 'Comment',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'systemInfoOS',
            displayText: 'OS',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'systemInfoCPU',
            displayText: 'CPU',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'systemInfoRAM',
            displayText: 'RAM',
            colSize: 220,
            isChecked: false,
            isCheckable: true,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         },
         {
            key: 'editAgent',
            displayText: '',
            colSize: 24,
            isChecked: true,
            isCheckable: false,
            isSorted: false,
            isSortedDescending: false,
            columnDef: undefined
         }
      ];
      this.columnSearchState = {
         'pools': {},
         'status': {},
         'workingStorage': {},
         'systemStorage': {},
         'comment': {},
         'taskTime': {},
         'taskTimeDescription': {}
      };
      this.columnMenuProps = this._updateColumnProps();
      this._updateColumnDefs();
   }

   updateColumns() {
      this._updateColumnDefs();
   }
}

// pool editor list item
type PoolEditorItem = {
   key: string;
   pool: PoolData;
   deleted: boolean;
   sortKey: string;
   numAgentsAssigned: number;
   configManaged?: boolean;
}

// state class for the edit pools modal
class EditPoolsModalState {
   constructor() {
      makeObservable(this);
   }
   // pool editor observables
   newIdSuffix = 0;
   @observable isOpen = false;
   @observable isPoolValueValid = true;
   @observable isEditorOpen = false;
   @observable modifiedPools: PoolEditorItem[] = [];
   @observable selectedColor = "";

   @observable isConfirmationOpen = false;

   // last selected color of the color modal
   @observable lastSelectedPool: PoolEditorItem | undefined = undefined;
   isDirectEdit = false;

   numAgentsAssigned: Record<string, number> = {}
   private _getNumAgentsAssigned() {
      this.numAgentsAssigned = {};
      agentStore.agents.forEach(agent => {
         agent.pools?.forEach(pool => {
            if (!(pool in this.numAgentsAssigned)) {
               this.numAgentsAssigned[pool] = 0;
            }
            this.numAgentsAssigned[pool]++;
         });
      });
   }

   private _poolToPoolItemCopy(pool: PoolData) {
      return {
         key: pool.id,
         sortKey: pool.name,
         deleted: false,
         pool: {
            id: pool.id,
            name: pool.name,
            colorValue: pool.colorValue,
            properties: { ...pool.properties },
            enableAutoscaling: pool.enableAutoscaling,
            workspaces: pool.workspaces
         },
         selected: false,
         numAgentsAssigned: this.numAgentsAssigned[pool.id] ?? 0,
         configManaged: pool.configManaged
      };
   }

   @action
   showConfirmation(show: boolean) {
      this.isConfirmationOpen = show;
   }

   // sets the pool editor dialog open
   @action
   setOpen(isEditorOpen?: boolean, selectedItem?: PoolData) {
      this._getNumAgentsAssigned();
      this.modifiedPools = [...agentStore.filterValidPools().map(pool => { return this._poolToPoolItemCopy(pool); })];
      this.newIdSuffix = 0;
      this.isOpen = true;
      this.isDirectEdit = isEditorOpen ?? false;

      if (this.isDirectEdit && selectedItem) {
         this.setSelectedPool(this.modifiedPools[this.modifiedPools.findIndex(item => item.key === selectedItem.id)]);
         this.isOpen = false;
      }
      else {
         this.isEditorOpen = false;
      }
   }

   @action
   setClose() {
      this.isOpen = false;
      this.isEditorOpen = false;
   }

   @action
   private _syncLastSelectedToModified() {
      if (this.lastSelectedPool) {
         this.modifiedPools[this.modifiedPools.findIndex(pool => pool.key === this.lastSelectedPool!.key)] = { ...this.lastSelectedPool! };
      }
   }

   @action
   setEditorOpen(isOpen: boolean, cancel: boolean) {
      this.isEditorOpen = isOpen;
      if (isOpen === false) {
         if (this.lastSelectedPool!.pool.id.indexOf(`zzzz_newPool_${this.newIdSuffix - 1}`) !== -1 && cancel) {
            this.modifiedPools = [...this.modifiedPools.slice(0, this.modifiedPools.length - 1)];
         }
         else if (!cancel) {
            this._syncLastSelectedToModified();
         }
         this.lastSelectedPool = undefined;
      }
   }

   newPoolInputId = "zzzz_newPool_";
   // adds a new pool to the collection
   @action
   addNewTempPool() {
      const newId = this.newPoolInputId + this.newIdSuffix;
      const newPool: PoolData = { id: newId, name: "", properties: { Color: "0" }, colorValue: "#aaaaaa", enableAutoscaling: false, workspaces: [] };
      const newItem: PoolEditorItem = { key: newId, sortKey: newId, pool: newPool, numAgentsAssigned: 0, deleted: false };
      this.newIdSuffix++;
      this.modifiedPools.push(newItem);
      this.setSelectedPool(this.modifiedPools[this.modifiedPools.findIndex(item => item.key === newItem.key)]);
   }

   // sets the selected pool
   @action
   setSelectedPool(item: PoolEditorItem | undefined) {
      this.modifiedPools.forEach(existing => {
         if (existing.pool.name !== "") {
            existing.sortKey = existing.pool.name;
         }
      });
      if (item) {
         this.lastSelectedPool = this._poolToPoolItemCopy(item.pool);
         this.isEditorOpen = true;
      }
   }

   // sets the selected pool's name
   @action
   setSelectedName(name: string) {
      if (this.lastSelectedPool) {
         this.lastSelectedPool.sortKey = name;
         this.lastSelectedPool.pool.name = name;
      }
   }

   @action
   setSelectedPoolColor(range: string) {
      if (this.lastSelectedPool) {
         this.selectedColor = linearInterpolate(range);
         this.lastSelectedPool.pool.colorValue = this.selectedColor;
         this.lastSelectedPool.pool.properties!["Color"] = range;
      }
   }

   @action
   setPoolDeleted(selectedItem: PoolEditorItem) {
      selectedItem.deleted = true;
   }

   @action
   setPoolValueValid(value: boolean) {
      this.isPoolValueValid = value;
   }

   getPoolModifications(): { deletedPools: string[], newPools: string[], modifiedPools: string[] } {


      const value = { deletedPools: [] as string[], newPools: [] as string[], modifiedPools: [] as string[] };

      this.modifiedPools.forEach(item => {
         // Skip config-managed pools — they cannot be modified
         if (item.configManaged === true) {
            return;
         }
         if (item.deleted) {
            value.deletedPools.push(item.pool.name);
         }
         else if (item.key.indexOf(this.newPoolInputId) !== -1) {
            value.newPools.push(item.pool.name);
         } else {
            const pool = agentStore.pools.find(pool => { return pool.id === item.pool.id; })!;
            if (pool) {
               if (pool.name !== item.pool.name || pool.properties!["Color"] !== item.pool.properties!["Color"]) {
                  value.modifiedPools.push(item.pool.name);
               }
            }
         }

      })

      return value;
   }

   @action
   async saveChanges() {
      // save off pool changes
      // 1. add any new pools into the list
      // 2. update any existing ids with new names
      // 3. remove any pools on agents with deleted flags
      const allUpdates: any[] = [];
      const batchUpdatePools: BatchUpdatePoolRequest[] = [];
      const that = this;

      this._syncLastSelectedToModified();

      this.modifiedPools.forEach(item => {
         // Skip config-managed pools — they cannot be modified through the API
         if (item.configManaged === true) {
            return;
         }
         if (item.deleted) {
            allUpdates.push(backend.deletePool(item.pool.id));
         }
         else {
            // create new if this has a new name prefix
            if (item.key.indexOf(this.newPoolInputId) !== -1) {
               allUpdates.push(backend.createPool({ name: item.pool.name, properties: { Color: item.pool.properties!["Color"] } }));
            }
            // otherwise update the name or deleted state if changed
            else {
               const pool = agentStore.pools.find(pool => { return pool.id === item.pool.id; })!;
               if (pool.name !== item.pool.name || pool.properties!["Color"] !== item.pool.properties!["Color"]) {
                  batchUpdatePools.push({ id: item.pool.id, name: item.pool.name, properties: { Color: item.pool.properties!["Color"] } });
               }
            }
         }
      });
      // remove any deleted pools from agents
      agentStore.agents.forEach(agent => {
         const numPools = agent.pools!.length;
         agent.pools = agent.pools!.filter(agentPool => {
            // if the pool doesn't exist in the pools list, remove it
            if (agentStore.pools.findIndex(item => item.id === agentPool) === -1) {
               return false;
            }
            const modified = that.modifiedPools.find(modified => modified.pool.id === agentPool);
            if (modified) {
               if (modified.deleted) {
                  return false;
               }
               return true;
            }
            return false;
         });
         // don't update the agent if nothing was removed
         if (agent.pools!.length !== numPools) {
            allUpdates.push(backend.updateAgent(agent.id, { pools: agent.pools }));
         }
      });
      // take all the batch update pools and add them to the last promise
      allUpdates.push(backend.batchUpdatePools(batchUpdatePools));
      await Promise.all(allUpdates).then(function (responses) {
      }).catch(function (errors) {
      }).finally(function () {
         that.setClose();
         agentStore.update();
      });
   }
}

type SelectPoolItem = {
   key: string;
   name: string;
   pool: PoolData;
   agentsAssigned: string[];
}

type AgentDataSlim = {
   Id: string;
   Name: string;
   Pools: string[];
}

// state class for the select pools modal
class SelectPoolsModalState {

   constructor() {
      makeObservable(this);
   }

   @observable isOpen = false;
   @observable.shallow availablePools: SelectPoolItem[] = [];
   @observable.shallow selectedPools: SelectPoolItem[] = [];
   agentChangeState: AgentDataSlim[] = [];
   agentsSelectedTooltip = "";

   @observable confirmDialogIsOpen = false;
   confirmKey: "discard" | "no-pools" = "discard";
   confirmDialogTitle = "Discard Changes";
   confirmDialogText = "There are unsaved changes, discard?";
   confirmConfirmButtonText = "Discard";
   confirmCancelButtonText = "Cancel";

   // sets the pool editor dialog open
   @action
   setIsOpen(isOpen: boolean) {
      // if we're closing don't reset
      if (isOpen) {
         // first get all nondeleted pools
         this.availablePools = agentStore.filterValidPools().map(pool => { return { key: pool.id, name: pool.name, pool: pool, agentsAssigned: [] }; });
         this.agentChangeState = [];
         this.selectedPools = [];
         this.agentsSelectedTooltip = "";
         this.confirmDialogIsOpen = false;
         // then loop through each agent and split them up into selected at all and available
         if (localState.currentSelection) {
            (localState.currentSelection).forEach(agent => {
               this.agentChangeState.push({ Id: agent.id, Name: agent.name, Pools: [...agent.pools!] });
               this.agentsSelectedTooltip += agent.name + "\n";
               agent.pools!.forEach(pool => {
                  const index = this.availablePools.findIndex(available => available.pool.id === pool);
                  if (index !== -1) {
                     const item = this.availablePools.splice(index, 1)[0];
                     item.agentsAssigned.push(agent.id);
                     this.selectedPools.push(item);
                  }
                  else {
                     this.selectedPools.find(selected => selected.pool.id === pool)?.agentsAssigned.push(agent.id);
                  }
               });
            });
         }
         this.availablePools = this.availablePools.sort((a, b) => { return a.pool.name.localeCompare(b.pool.name); });
         this.selectedPools = this.selectedPools.sort((a, b) => { return a.pool.name.localeCompare(b.pool.name); });
      }
      this.isOpen = isOpen;
   }

   @action
   transitionPool(item: SelectPoolItem) {
      // look for the item in available pools
      let foundIdx = this.availablePools.findIndex(c => c.key === item.key);
      // if not found, this is an UNASSIGN
      if (foundIdx === -1) {
         foundIdx = this.selectedPools.findIndex(c => c.key === item.key);
         const itemTransfer = this.selectedPools.splice(foundIdx, 1)[0];
         itemTransfer.agentsAssigned = [];
         this.availablePools = [...this.availablePools, itemTransfer].sort((a, b) => { return a.pool.name.localeCompare(b.pool.name); });
         // remove pools from existing temp agents
         this.agentChangeState.forEach(agent => {
            const existingIndex = agent.Pools!.indexOf(itemTransfer.pool.id);
            // possible an agent isn't assigned to this pool, so check
            if (existingIndex !== -1) {
               agent.Pools!.splice(existingIndex, 1);
            }
         });
         // force selectedPools to itself so we trigger a re-render (not sure why setting to not shallow and just splicing it doesn't achieve this, but here we are.)
         this.selectedPools = [...this.selectedPools];
      }
      // if it is, its an ASSIGN
      else {
         const itemTransfer = this.availablePools.splice(foundIdx, 1)[0];
         itemTransfer.agentsAssigned = this.agentChangeState.map(agent => { return agent.Id; });
         this.selectedPools = [...this.selectedPools, itemTransfer];
         // change the temp agents to contain the new assignments
         this.agentChangeState.forEach(agent => {
            agent.Pools!.push(itemTransfer.pool.id);
         });
      }
      // always return null, the observable changing should rerender the list
      return null;
   }

   @action
   setConfirmOrCloseModal(type: "save" | "cancel") {
      if (this.selectedPools.length === 0 && type === "save") {
         this.confirmKey = "no-pools";
         this.confirmDialogTitle = "No Pools";
         this.confirmDialogText = "The agent(s) have no pools. Save anyway?";
         this.confirmConfirmButtonText = "Save";
         this.confirmCancelButtonText = "Cancel";
         this.confirmDialogIsOpen = true;
      }
      else {
         let dirty = false;
         if (localState.currentSelection) {
            for (const idx in localState.currentSelection) {
               let original = agentStore.agents.find(agent => agent.id === localState.currentSelection[idx].id)?.pools!;
               let newPools = this.agentChangeState.find(a => a.Id === localState.currentSelection[idx].id)!.Pools!;
               // if they're not the same length we know they're different.
               if (original.length !== newPools.length) {
                  dirty = true;
               }
               else {
                  // if they are the same length, ensure all the items are the same
                  original = original.sort();
                  newPools = newPools.sort();
                  for (let jdx = 0; jdx < original.length; jdx++) {
                     if (original[jdx] !== newPools[jdx]) {
                        dirty = true;
                        break;
                     }
                  }
               }
               if (dirty) {
                  break;
               }

            }

         }
         if (dirty) {
            if (type === "cancel") {
               this.confirmKey = "discard";
               this.confirmDialogTitle = "Discard Changes";
               this.confirmDialogText = "There are unsaved changes, discard?";
               this.confirmConfirmButtonText = "Discard";
               this.confirmCancelButtonText = "Cancel";
               this.confirmDialogIsOpen = true;
            }
            else {
               this.saveChanges();
            }
         }
         else {
            this.confirmDialogIsOpen = false;
            this.setIsOpen(false);
            return;
         }
      }
   }

   @action
   setConfirmModalClosed() {
      this.confirmDialogIsOpen = false;
   }

   @action
   async saveChanges(conform?: boolean | undefined) {
      // save off pool changes
      // 3. remove any pools on agents with deleted flags
      const allUpdates: any[] = [];
      const that = this;

      this.confirmDialogIsOpen = false;
      this.agentChangeState.forEach(agent => {
         allUpdates.push(backend.updateAgent(agent.Id, { pools: agent.Pools, requestFullConform: conform }));
      });
      // take all the batch update pools and add them to the last promise
      await Promise.all(allUpdates).then(function (responses) {
      }).catch(function (errors) {
      }).finally(function () {
         that.setIsOpen(false);
         agentStore.update();
      });
   }
}

const localState = new LocalState();
const editPoolsModalState = new EditPoolsModalState();
const selectPoolsModalState = new SelectPoolsModalState();

// these are static and dont need to be re-init'd on every draw
const agentSelectedProps: IContextualMenuProps = {
   onItemClick: (ev?: React.MouseEvent<HTMLElement, MouseEvent> | React.KeyboardEvent<HTMLElement> | undefined, item?: IContextualMenuItem | undefined) => {
      if (item) {
         localState.currentSelection = [...localState.selection.getSelection() as AgentData[]];
         if (item.key === "disable") {
            localState.setDisableBuilderDialogOpen(true);
         }
         else if (item.key === "enable") {
            localState.changeBuilderEnabled("enable");
         }
         else if (item.key === "changePools") {
            selectPoolsModalState.setIsOpen(true);
         }
         else if (item.key === "requestConform") {
            localState.requestBuilderUpdate(true, false);
         }
         else if (item.key === "requestFullConform") {
            localState.requestBuilderUpdate(false, false, true);
         }
         else if (item.key === "clearConform") {
            localState.requestBuilderUpdate(false, false, false);
         }            
         else if (item.key === "restart") {
            localState.setRestartBuilderDialogOpen(true);
         }
         else if (item.key === "delete") {
            localState.setDeleteBuilderDialogOpen(true);
         }
         else if (item.key === "editcomment") {
            localState.setEditCommentDialogOpen(true);
         }

      }
   },
   items: [
      {
         key: 'enable',
         text: 'Enable',
      },
      {
         key: 'disable',
         text: 'Disable',
      },
      {
         key: 'audit',
         text: 'Audit',
      },
      {
         key: 'editcomment',
         text: 'Edit Comment',
      },
      {
         key: 'remotedesktop',
         text: 'Remote Desktop',
      },
      {
         key: 'changePools',
         text: 'Edit Pools'
      },
      {
         key: 'cancelleases',
         text: 'Cancel Leases',
      },
      {
         key: 'clearConform',
         text: 'Clear Pending Conform'
      },
      {
         key: 'requestConform',
         text: 'Request Conform'
      },
      {
         key: 'requestFullConform',
         text: 'Request Full Conform'
      },
      {
         key: 'restart',
         text: 'Request Restart'
      },
      {
         key: 'shutdown',
         text: 'Request Shutdown'
      },
      {
         key: 'delete',
         text: 'Delete'
      }
   ]
};

// these are static and dont need to be re-init'd on every draw
const agentContextMenuProps: IContextualMenuItem[] = [
   {
      key: 'enable',
      text: 'Enable',
      onClick: () => localState.changeBuilderEnabled('enable')
   },
   {
      key: 'disable',
      text: 'Disable',
      onClick: () => localState.setDisableBuilderDialogOpen(true)
   },
   {
      key: 'remotedesktop',
      text: 'Remote Desktop',
      onClick: () => {
         if (localState.currentSelection?.length && localState.currentSelection[0].id) {
            // @todo: this assumes the id is the ip, should have a private ip property on agents instead
            let ip = localState.currentSelection[0].id;
            if (ip.startsWith("10-")) {
               ip = ip.replaceAll("-", ".")
            }
            window.open(`ugs://rdp?host=${ip}`, "_self")
         }
      }
   },
   {
      key: 'editcomment',
      text: 'Edit Comment',
      onClick: () => localState.setEditCommentDialogOpen(true)
   },
   {
      key: 'audit',
      text: 'Audit',
      onClick: () => {
         if (localState.currentSelection?.length && localState.currentSelection[0].id) {

            const href = `/audit/agent/${encodeURIComponent(localState.currentSelection[0].id)}`;
            window.open(href, "_blank")
         }
      }
   },
   {
      key: 'editPools',
      text: 'Edit Pools',
      onClick: () => selectPoolsModalState.setIsOpen(true)
   },
   {
      key: 'cancelleases',
      text: 'Cancel Leases',
      onClick: () => localState.setCancelLeasesDialogOpen(true)
   },
   {
      key: 'clearConform',
      text: 'Clear Pending Conform',
      onClick: () => localState.requestBuilderUpdate(false, false, false)
   },
   {
      key: 'requestConform',
      text: 'Request Conform',
      onClick: () => localState.requestBuilderUpdate(true, false)
   },
   {
      key: 'requestFullConform',
      text: 'Request Full Conform',
      onClick: () => localState.requestBuilderUpdate(false, false, true)
   },
   {
      key: 'requestRestart',
      text: 'Request Restart',
      onClick: () => localState.setRestartBuilderDialogOpen(true)
   },
   {
      key: 'requestShutdown',
      text: 'Request Shutdown',
      onClick: () => localState.setShutdownBuilderDialogOpen(true)
   },
   {
      key: 'delete',
      text: 'Delete',
      onClick: () => localState.setDeleteBuilderDialogOpen(true)
   },
];

const agentStatus = ["Active", "Ready", "Disabled", "Ephemeral", "No Pools", "Pending Conform", "Pending Shutdown", "Offline", "Offline (Autoscaler)", "Offline (Manual)", "Offline (Unexpected)"];


const AgentMenuBar: React.FC<{ agentView?: boolean }> = observer(({ agentView }) => {

   let selectedButton: any = <div></div>;
   if (localState.agentsSelectedCount !== 0) {
      selectedButton = <PrimaryButton
         disabled={localState.agentsSelectedCount === 0 ? true : false}
         styles={{ root: { fontFamily: 'Horde Open Sans SemiBold !important' } }}
         menuProps={agentSelectedProps}
         onClick={() => { }}>
         {`${localState.agentsSelectedCount} Agent${localState.agentsSelectedCount > 1 ? "s" : ""} Selected`}
      </PrimaryButton>;
   }

   const agentStatusItems = agentStatus.map(status => {
      return {
         text: status,
         key: `dropdown_agent_status_${status}_key`,
         status: status
      }
   });

   return (
      <Stack horizontal verticalAlign='center' tokens={{ childrenGap: 24 }}>
         {!!agentView && <Stack horizontal tokens={{ childrenGap: 12 }} verticalAlign='center'>
            {!!dashboard.user?.dashboardFeatures?.showPoolEditor && <CommandButton
               onClick={() => { editPoolsModalState.setOpen(); }}
               iconProps={{ iconName: 'Edit' }}
               styles={{ root: { fontFamily: 'Horde Open Sans SemiBold !important' } }}>
               {`Pools`}
            </CommandButton>}
            {selectedButton}
            <PoolEditorModal></PoolEditorModal>
            <PoolSelectionModal></PoolSelectionModal>
         </Stack>
         }
         <Stack>
            <Dropdown
               placeholder="Filter Status"
               style={{ width: 200 }}
               selectedKeys={Array.from(localState.agentStatusFilter).map(status => `dropdown_agent_status_${status}_key`)}
               multiSelect
               options={agentStatusItems}
               onChange={(event, option, index) => {

                  if (option) {

                     // new set instance to update observable action update
                     const newFilter = new Set(localState.agentStatusFilter);

                     if (option.selected) {
                        newFilter.add((option as any).status);
                     } else {
                        newFilter.delete((option as any).status);
                     }

                     localState.setAgentStaus(newFilter);

                  }
               }}
            />
         </Stack>
         <Stack>
            <Checkbox label={"Exact Match"} checked={localState.filterExactMatch} onChange={(ev, checked) => localState.setExactMatch(checked!)} />
         </Stack>
         <Stack>
            <SearchBox
               showIcon={true}
               disableAnimation={true}
               placeholder="Search Agents"
               value={localState.agentFilter}
               styles={{ root: { width: 240 } }}
               onChange={(event?: React.ChangeEvent<HTMLInputElement> | undefined, newValue?: string | undefined) => { localState.setAgentFilter(newValue ?? ""); }}
               onClear={() => { localState.setAgentFilter(""); }}
            />
         </Stack>

      </Stack>
   );
});


const AgentPivot: React.FC = () => {

   const { hordeClasses, modeColors } = getHordeStyling();

   const categories = dashboard.agentCategories;

   const pivotItems = categories.map(tab => {
      return <PivotItem headerText={tab.name} itemKey={tab.name} key={tab.name} style={{ color: modeColors.text }} />;
   });

   pivotItems.unshift(<PivotItem headerText="All" itemKey="all" key={"all"} style={{ color: modeColors.text }} />);

   return <Stack grow>
      <Pivot className={hordeClasses.pivot}
         overflowBehavior='menu'
         selectedKey={localState.searchState.category ?? "all"}
         linkSize="normal"
         linkFormat="links"
         onLinkClick={(item) => {
            if (item) {
               localState.setAgentCategory(item.props.itemKey === "all" ? undefined : item.props.itemKey);
            }
         }}>
         {pivotItems}
      </Pivot>
   </Stack>

}

const PoolEditorConfirmation: React.FC = observer(() => {


   if (!editPoolsModalState.isConfirmationOpen) {
      return null;
   }

   const mods = editPoolsModalState.getPoolModifications();

   if (mods.deletedPools.length === 0 && mods.newPools.length === 0 && mods.modifiedPools.length === 0) {
      return null;
   }

   let title = "Modify Pool" + (mods.modifiedPools.length > 1 ? "s" : "");
   let subText = "";

   if (mods.deletedPools.length && !mods.newPools.length) {
      title = "Delete Pool";
      if (mods.deletedPools.length > 1) {
         title += "s";
      }
   } else if (!mods.deletedPools.length && mods.newPools.length) {
      title = "Create Pool";
      if (mods.newPools.length > 1) {
         title += "s";
      }
   }

   title += "?";

   if (mods.deletedPools.length) {
      subText += "Delete: " + mods.deletedPools.join(", ") + " ";
   }

   if (mods.newPools.length) {
      subText += "Create: " + mods.newPools.join(", ") + " ";
   }

   if (mods.modifiedPools.length) {
      subText += "Modified: " + mods.modifiedPools.join(", ");
   }

   return <ConfirmationDialog
      title={title}
      subText={subText}
      isOpen={true}
      confirmText="Save"
      cancelText="Cancel"
      onConfirm={() => { editPoolsModalState.showConfirmation(false); editPoolsModalState.saveChanges(); }}
      onCancel={() => editPoolsModalState.showConfirmation(false)}
   />

});

const PoolEditorModal: React.FC = observer(() => {

   const [state, setState] = useState<{ sortBy: "Agents" | "Name" }>({ sortBy: "Name" });
   const { hordeClasses } = getHordeStyling();
   const agentStyles = getAgentStyles();

   let selectedColor = "#ffffff";
   let selectedColorValue = "0";
   if (editPoolsModalState.lastSelectedPool) {
      selectedColorValue = editPoolsModalState.lastSelectedPool.pool.properties!["Color"];
      selectedColor = linearInterpolate(selectedColorValue);
   }

   function onRenderPoolModalItem(item: PoolEditorItem, index?: number, column?: IColumn) {
      switch (column!.key) {
         case 'name':
            const color = item.pool.colorValue;
            const textColor = "white";
            if (item.configManaged === true) {
               return (
                  <Stack styles={{ root: { height: '100%', } }} horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                     <Stack.Item align={"center"}>
                        <Text styles={{ root: { padding: '4px 8px', backgroundColor: color, color: textColor, borderRadius: 2, fontSize: '8pt', fontFamily: 'Horde Open Sans SemiBold' } }}>{item.pool.name}</Text>
                     </Stack.Item>
                     <TooltipHost content="Managed via config files">
                        <Icon iconName="Lock" styles={{ root: { fontSize: 10 }}} />
                     </TooltipHost>
                  </Stack>
               );
            }
            return (
               <Stack styles={{ root: { height: '100%', } }} horizontal>
                  <Stack.Item align={"center"}>
                     <PrimaryButton key={"pooleditormodal_item_" + item.pool.id}
                        className={agentStyles.buttonFont}
                        onClick={() => {
                           editPoolsModalState.setSelectedPool(item);
                        }}
                        styles={{
                           root: { borderColor: color, backgroundColor: color, color: textColor },
                           rootHovered: { borderColor: color, backgroundColor: color, color: textColor, },
                           rootPressed: { borderColor: color, backgroundColor: color, color: textColor, }
                        }}>{item.pool.name}
                     </PrimaryButton>
                  </Stack.Item>
               </Stack>
            );
         case 'numMachines':
            return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{item.numAgentsAssigned}</Stack.Item></Stack>;
         case 'del':
            if (item.configManaged === true) {
               return null;
            }
            return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={"center"}>
               <Stack.Item grow={0} align={"center"}>
                  <IconButton iconProps={{ iconName: 'Cancel' }} onClick={(event: any) => { editPoolsModalState.setPoolDeleted(item); }} />
               </Stack.Item>
            </Stack>;
         default:
            return <div></div>;
      }
   }

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      const customStyles: Partial<IDetailsHeaderStyles> = {};
      if (props) {
         return (
            <Sticky stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
               <DetailsHeader {...props} styles={customStyles} onColumnClick={(ev: React.MouseEvent<HTMLElement>, column: IColumn) => {
                  if (column.name === "Agents") {
                     if (state.sortBy !== 'Agents') {
                        setState({ ...state, sortBy: "Agents" });
                     }

                  } else if (column.name === "Name") {
                     if (state.sortBy !== 'Name') {
                        setState({ ...state, sortBy: "Name" });
                     }
                  }
               }} onRenderColumnHeaderTooltip={onRenderColumnHeaderTooltip} />
            </Sticky>
         );
      }
      return null;
   };

   const onRenderColumnHeaderTooltip: IDetailsHeaderProps['onRenderColumnHeaderTooltip'] = (props: any) => {
      const customStyles: Partial<ITooltipHostStyles> = {};
      if (props) {
         customStyles.root = { selectors: { "> span": { paddingLeft: '8px !important', display: 'flex', justifyContent: 'center' } } };
         /*
          if (props.children._owner.child?.child.child.child.elementType === "span") {
              const data = props.children._owner.child.child.child.child.child.child.stateNode.data;
              if (data === "Name") {
                  customStyles.root = {};
              }
          }
          */
         return <Text styles={customStyles}>{props.children}</Text>;
      }
      return null;
   };

   const pools = editPoolsModalState.modifiedPools.filter(item => item.deleted === false).sort((a, b) => {
      if (state.sortBy === "Name")
         return a.sortKey.localeCompare(b.sortKey);
      else
         return b.numAgentsAssigned - a.numAgentsAssigned;
   });

   // 75vh for max
   const viewportHeight = document.documentElement.clientHeight * .75;
   const modalHeight = Math.min(viewportHeight, 80 + (44 * pools.length));

   return (
      <Stack>
         <PoolEditorConfirmation />
         <Dialog
            modalProps={{
               isBlocking: false,
               dragOptions: {
                  closeMenuItemText: "Close",
                  moveMenuItemText: "Move",
                  menu: ContextualMenu
               }
            }}
            onDismiss={() => { editPoolsModalState.setClose(); }}
            className={agentStyles.dialog}
            minWidth={440}
            hidden={!editPoolsModalState.isOpen}
            dialogContentProps={{
               type: DialogType.close,
               onDismiss: () => { editPoolsModalState.setClose(); },
               title: "Pool Editor",
            }}
         >
            <Stack>
               <Stack horizontal horizontalAlign="space-evenly">
                  <Stack.Item className={hordeClasses.relativeWrapper} styles={{ root: { height: modalHeight, width: 400 } }}>
                     <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
                        <DetailsList
                           className={agentStyles.detailsList}
                           compact={true}
                           items={pools}
                           columns={[{ key: 'name', name: 'Name', minWidth: 200, maxWidth: 500 }, { key: 'numMachines', name: 'Agents', minWidth: 60, maxWidth: 60 }, { key: 'del', name: '', minWidth: 55, maxWidth: 55 }]}
                           onRenderDetailsHeader={onRenderDetailsHeader}
                           onRenderItemColumn={onRenderPoolModalItem}
                           layoutMode={DetailsListLayoutMode.justified}
                           constrainMode={ConstrainMode.horizontalConstrained}
                           selectionMode={SelectionMode.none}
                        />
                     </ScrollablePane>
                  </Stack.Item>
               </Stack>
               <Stack styles={{ root: { paddingTop: 10 } }}>
                  <Stack horizontal>
                     <Stack>
                        <DefaultButton onClick={() => editPoolsModalState.addNewTempPool()}>New Pool</DefaultButton>
                     </Stack>
                     <Stack grow />
                     <Stack>
                        <PrimaryButton disabled={editPoolsModalState.getPoolModifications().deletedPools.length === 0 && editPoolsModalState.getPoolModifications().newPools.length === 0 && editPoolsModalState.getPoolModifications().modifiedPools.length === 0} onClick={() => editPoolsModalState.showConfirmation(true)} styles={{ root: { marginRight: "10px" } }}>Save</PrimaryButton>
                     </Stack>
                     <Stack>
                        <DefaultButton onClick={() => editPoolsModalState.setClose()} styles={{ root: { marginRight: "10px" } }}>Cancel</DefaultButton>
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Dialog>

         <Dialog
            modalProps={{
               isBlocking: false,
               dragOptions: {
                  closeMenuItemText: "Close",
                  moveMenuItemText: "Move",
                  menu: ContextualMenu
               }
            }}
            onDismiss={() => { editPoolsModalState.setEditorOpen(false, true); }}
            className={agentStyles.dialog}
            minWidth={440}
            hidden={!editPoolsModalState.isEditorOpen}
            dialogContentProps={{
               type: DialogType.close,
               onDismiss: () => { editPoolsModalState.setEditorOpen(false, true); },
               title: "Edit Pool",
            }}
         >
            <Stack>
               <Stack horizontal horizontalAlign="space-evenly">
                  <Stack styles={{ root: { width: 400 } }}>
                     <TextField
                        spellCheck={false}
                        label="Name"
                        borderless={false}
                        autoFocus={true}
                        styles={{
                           root: { paddingLeft: 10, paddingRight: 10 },
                        }}
                        onChange={(event: React.FormEvent<HTMLInputElement | HTMLTextAreaElement>, newValue?: string | undefined) => {
                           editPoolsModalState.setSelectedName(newValue!);
                        }}
                        placeholder={`Enter pool name...`}
                        defaultValue={editPoolsModalState.lastSelectedPool?.pool.name}
                        onGetErrorMessage={(value: string) => {
                           let editingExistingPool = agentStore.pools.find(pool => pool.name === value);
                           if (!editingExistingPool) {
                              editingExistingPool = editPoolsModalState.modifiedPools.find(pool => pool.pool.name === value)?.pool;
                           }
                           if (value.length < 1 || value.length > 64) {
                              editPoolsModalState.setPoolValueValid(false);
                              return "Length must be between 1 and 64 characters.";
                           }
                           if (editingExistingPool && editingExistingPool.id !== editPoolsModalState.lastSelectedPool?.pool.id) {
                              editPoolsModalState.setPoolValueValid(false);
                              return "Name is already taken.";
                           }
                           editPoolsModalState.setPoolValueValid(true);
                           return "";
                        }}
                     />
                     <Stack horizontal styles={{ root: { marginBottom: 40 } }}>
                        <Stack grow>
                           <Slider
                              styles={{ root: { paddingTop: 15, paddingLeft: 10, paddingRight: 10 } }}
                              label="Color"
                              onChange={(value: number) => editPoolsModalState.setSelectedPoolColor(value.toString())}
                              defaultValue={parseInt(selectedColorValue)}
                              max={599}
                              showValue={false}
                           />
                        </Stack>
                        <Stack>
                           <div className={hordeClasses.colorPreview} style={{ backgroundColor: selectedColor }} />
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>
               <Stack styles={{ root: { paddingTop: 10 } }}>
                  <Stack horizontal>
                     <Stack style={{ paddingLeft: 12 }}>
                        <DefaultButton href={`/pools?pool=${editPoolsModalState.lastSelectedPool?.pool?.id}`} target="_blank">Details</DefaultButton>
                     </Stack>
                     <Stack grow />
                     <PrimaryButton disabled={!editPoolsModalState.isPoolValueValid} onClick={() => { editPoolsModalState.isDirectEdit ? editPoolsModalState.saveChanges() : editPoolsModalState.setEditorOpen(false, false); }} styles={{ root: { marginRight: "10px" } }}>
                        {editPoolsModalState.isDirectEdit ? "Update" : "Save"}
                     </PrimaryButton>
                     <DefaultButton onClick={() => { editPoolsModalState.isDirectEdit ? editPoolsModalState.setClose() : editPoolsModalState.setEditorOpen(false, true); }} styles={{ root: { marginRight: "10px" } }}>Cancel</DefaultButton>
                  </Stack>
               </Stack>
            </Stack>
         </Dialog>
      </Stack>
   );
});

const PoolSelectionModal: React.FC = observer(() => {

   const agentStyles = getAgentStyles();

   function onResolveSuggestions(filter: string, selectedItems?: ITag[] | undefined) {
      return selectPoolsModalState.availablePools.filter(item => item.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1);
   }

   // renderitem override
   const onTagRenderItem: IBasePickerProps<ITag>['onRenderItem'] = (props) => {
      const item = props.item as SelectPoolItem;
      let color = item.pool.colorValue;
      const hoverColor = hexToRGB(color);
      const hoverCloseColor = hexToRGB(color);
      hoverColor.r *= .8;
      hoverColor.g *= .8;
      hoverColor.b *= .8;
      hoverCloseColor.r *= .7;
      hoverCloseColor.g *= .7;
      hoverCloseColor.b *= .7;
      const textColor = "white";
      let name = item.pool.name;
      let title = "";
      if (item.agentsAssigned.length !== localState.agentsSelectedCount && item.agentsAssigned.length !== 0) {
         color += "80";
         name += ` (${item.agentsAssigned.length}/${localState.agentsSelectedCount})`;
         title = "Assigned to: \n\t" + item.agentsAssigned.map(agent => { return selectPoolsModalState.agentChangeState.find(found => found.Id === agent)!.Name; }).join('\n\t');
      }

      const customStyles: Partial<ITagItemStyles> = {};
      customStyles.root = {
         borderColor: color, backgroundColor: color, textAlign: 'center',
         selectors: {
            ':hover': { backgroundColor: `rgb(${hoverColor.r},${hoverColor.g},${hoverColor.b})` },
            ':hover .ms-TagItem-close': { color: 'white' },
            ':active': { color: 'white' }
         }
      };
      customStyles.text = { font: '8pt Horde Open Sans SemiBold !important', color: textColor, position: 'relative', top: 6, paddingLeft: 16, paddingRight: 4, marginRight: 0 };
      customStyles.close = { color: 'white', selectors: { ':hover': { color: 'white', background: `rgb(${hoverCloseColor.r},${hoverCloseColor.g},${hoverCloseColor.b})` } } };
      return <TagItem
         key={`tagItem_pools_${item.pool.id}`}
         title={title}
         index={props.index}
         item={props.item}
         styles={customStyles}
         onRemoveItem={() => { selectPoolsModalState.transitionPool(item); }}>
         {name}
      </TagItem>;
   };

   const [conform, setConform] = useState<boolean | undefined>(false);

   return (
      <Dialog
         hidden={!selectPoolsModalState.isOpen}
         dialogContentProps={{
            type: DialogType.close,
            onDismiss: () => { selectPoolsModalState.setConfirmOrCloseModal("cancel"); },
            title: 'Pool Assignments',
         }}
         className={agentStyles.dialog}
         minWidth={800}
         onDismiss={() => { selectPoolsModalState.setConfirmOrCloseModal("cancel"); }}
         modalProps={{
            isBlocking: false,
            dragOptions: {
               closeMenuItemText: "Close",
               moveMenuItemText: "Move",
               menu: ContextualMenu
            }
         }}
      >
         <TagPicker
            onResolveSuggestions={onResolveSuggestions}
            onEmptyResolveSuggestions={() => { return selectPoolsModalState.availablePools; }}
            onItemSelected={(selectedItem: ITag | undefined) => { return selectPoolsModalState.transitionPool(selectedItem as SelectPoolItem); }}
            onRenderItem={onTagRenderItem}
            selectedItems={selectPoolsModalState.selectedPools}
            pickerSuggestionsProps={{
               noResultsFoundText: 'No Pools Found'
            }}
            inputProps={{
               'aria-label': 'Tag Picker'
            }}
         />
         <Stack horizontal styles={{ root: { marginTop: 16 } }}>
            <Stack>
               <Checkbox label="Conform Immediately" onChange={(ev?: any, checked?: boolean | undefined) => setConform(checked)} />
            </Stack>
            <Stack grow />
            <Stack horizontal>
               <PrimaryButton styles={{ root: { marginRight: 8 } }} onClick={() => selectPoolsModalState.setConfirmOrCloseModal("save")} >Update</PrimaryButton>
               <DefaultButton onClick={() => selectPoolsModalState.setConfirmOrCloseModal("cancel")}>Cancel</DefaultButton>
            </Stack>
         </Stack>
         <ConfirmationDialog
            title={selectPoolsModalState.confirmDialogTitle}
            subText={selectPoolsModalState.confirmDialogText}
            isOpen={selectPoolsModalState.confirmDialogIsOpen}
            confirmText={selectPoolsModalState.confirmConfirmButtonText}
            cancelText={selectPoolsModalState.confirmCancelButtonText}
            onConfirm={() => (selectPoolsModalState.confirmKey === "no-pools" ? selectPoolsModalState.saveChanges(conform ?? undefined) : selectPoolsModalState.setIsOpen(false))}
            onCancel={() => selectPoolsModalState.setConfirmModalClosed()}
         />
      </Dialog>
   );
});

const SearchUpdate: React.FC = observer(() => {

   const [searchParams, setSearchParams] = useSearchParams();
   const [state, setState] = useState<{ search?: string }>({ search: searchParams.toString() });

   // subscribe
   if (localState.searchUpdated) { }

   const csearch = localState.search.toString();

   if (state.search !== csearch) {
      setSearchParams(csearch, { replace: true });
      setState({ search: csearch });
   }

   return null;
});

const AgentViewInner: React.FC<{ agentId?: string, poolId?: string, searchParams?: URLSearchParams, agentView?: boolean, agents?: AgentData[] }> = observer(({ agentId, poolId, searchParams, agentView, agents: externalAgents }) => {

   poolId = poolId?.toLowerCase();

   const hasExternalAgents = !!externalAgents;

   const [initAgentUpdater, setInitAgentUpdater] = useState(false);

   useEffect(() => {
      if (hasExternalAgents) return;
      const interval = setInterval(() => {
         agentStore.update(true)
      }, 15000);
      return () => clearInterval(interval);
   }, [hasExternalAgents]);

   // subscribe
   if (localState.searchUpdated) { }

   const agentStyles = getAgentStyles();

   if (!initAgentUpdater) {
      const initPromise = hasExternalAgents ? agentStore.updatePools() : agentStore.update();
      initPromise.then(() => {
         localState.resetState();
         if (searchParams) {
            localState.stateFromSearch(searchParams);
         }
         setInitAgentUpdater(true);
      });
      return <Stack>
         <Stack style={{ position: "relative", height: "calc(100vh - 240px)" }} horizontalAlign="center">
            <Stack horizontal tokens={{ childrenGap: 24 }}>
               <Text variant="mediumPlus">Loading Agents</Text>
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Stack>
   }

   const effectiveAgents = externalAgents ?? agentStore.agents;

   let activeAgent: AgentData | undefined = undefined;
   if (agentId) {
      activeAgent = effectiveAgents.find(agent => agent.id === agentId);
   }

   // if(activeAgent) {
   //     if(historyModalState.selectedAgent?.id !== activeAgent.id){
   //         historyModalState.setSelectedAgent(activeAgent.id);
   //         historyModalState.setIsOpen(true, activeAgent);
   //     }
   // }
   // else {
   //     historyModalState.setIsOpen(false);
   // }

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      if (props) {
         return (
            <Sticky stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
               <DetailsHeader {...props} onRenderColumnHeaderTooltip={onRenderColumnHeaderTooltip} />
            </Sticky>
         );
      }
      return null;
   };

   // everything about this is no.
   const onRenderColumnHeaderTooltip: IDetailsHeaderProps['onRenderColumnHeaderTooltip'] = (props: any) => {
      const customStyles: Partial<ITooltipHostStyles> = {};
      if (props) {
         customStyles.root = { selectors: { "> span": { paddingLeft: '8px !important', display: 'flex', justifyContent: 'center' } } };

         /*
          // no other way to filter children being centered other than to drill into private members??
          if (props.children._owner.child?.child.child.child.elementType === "span") {
              const data = props.children._owner.child.child.child.child.child.child.stateNode.data;
              // if this cluster happens to be true, reset back to the default, because doing this the other way around
              // takes too long when the columns update on details switch
              // ugh :(
              if (data === "Name" || data === "Pools") {
                  customStyles.root = {};
              }
          }
          */
         return <Text styles={customStyles}>{props.children}</Text>;
      }
      return null;
   };

   function filterAgents(this: string, agent: AgentData) {
      const filter = this.toLowerCase();
      if (agent.deleted) {
         return null;
      }

      let category: GetDashboardAgentCategoryResponse | undefined;

      if (localState.searchState.category) {
         category = dashboard.agentCategories.find(c => c.name === localState.searchState.category);
      }

      if (category && category.condition) {
         const cat = agentStore.getCategory(category.condition);
         if (!cat.ids.length && cat.polling) {
            return null;
         } else {
            if (!cat.ids.find((id) => id === agent.id)) {
               return null;
            }
         }
      }

      if (filter !== "") {
         if (localState.filterExactMatch) {
            // add if there's a match to name or version
            if (agent.name.toLowerCase() === filter) {
               return agent;
            }

            // check status
            if (localState.columnSearchState['status'][agent.id]) {
               for (const idx in localState.columnSearchState['status'][agent.id]) {
                  if (localState.columnSearchState['status'][agent.id][idx].toLowerCase() === filter) {
                     return agent;
                  }
               }
            }

            // also add if there's a pool match
            if (localState.columnSearchState['pools'][agent.id]) {
               for (const idx in localState.columnSearchState['pools'][agent.id]) {
                  if (localState.columnSearchState['pools'][agent.id][idx].toLowerCase() === filter) {
                     return agent;
                  }
               }
            }
            // revert back to checking for individual pools if there's no cached version
            else {
               if (agent.pools) {
                  for (const idx in agent.pools) {
                     const pool = agentStore.pools.find(p => { return p.id === agent.pools![idx]; });
                     if (pool && pool.name.toLowerCase() === filter) {
                        return agent;
                     }
                  }
               }
            }
         }
         else {
            // add if there's a match to name or version
            if ((agent.name.toLowerCase().indexOf(filter) !== -1)) {
               return agent;
            }

            // check status
            if (localState.columnSearchState['status'][agent.id]) {
               if (localState.columnSearchState['status'][agent.id].join('').toLowerCase().indexOf(filter) !== -1) {
                  return agent;
               }
            }

            // also add if there's a pool match
            if (localState.columnSearchState['pools'][agent.id]) {
               if (localState.columnSearchState['pools'][agent.id].join('').toLowerCase().indexOf(filter) !== -1) {
                  return agent;
               }
            }
            // revert back to checking for individual pools if there's no cached version
            else {
               if (agent.pools) {
                  for (const idx in agent.pools) {
                     const pool = agentStore.pools.find(p => { return p.id === agent.pools![idx]; });
                     if (pool) {
                        if (pool.name.toLowerCase().indexOf(filter) !== -1) {
                           return agent;
                        }
                     } else {
                        console.error(`No matching pool id for ${JSON.stringify(agent.pools![idx])} while filtering pools`);
                     }
                  }
               }
            }
         }

         let toCheck = "";
         // also check currently selected othercolumn
         switch (localState.columnMenuProps.find(prop => prop.checked)!.key) {
            case 'software':
               if (agent.version) {
                  toCheck = agent.version;
               }
               else {
                  toCheck = "Unknown";
               }
               break;
            case 'taskTime':
               if (agent.version) {
                  if (localState.columnSearchState['taskTimeDescription'][agent.id]) {
                     toCheck = localState.columnSearchState['taskTimeDescription'][agent.id];
                  }
               }
               else {
                  toCheck = "Unknown";
               }
               break;
            case 'workingStorage':
            case 'systemStorage':
               const filterConfig = STORAGE_COLUMNS[localState.searchState?.columnMode ?? 'workingStorage'];
               if (localState.columnSearchState[filterConfig.searchStateKey][agent.id]) {
                  toCheck = String(localState.columnSearchState[filterConfig.searchStateKey][agent.id]);
               } else {
                  const diskFree = getAgentCapability(agent, filterConfig.freeCapability);
                  const diskTotal = getAgentCapability(agent, filterConfig.totalCapability);
                  toCheck = (diskFree !== null && diskTotal !== null)
                     ? String(getFreeDiskSpaceInGB(Number(diskFree), Number(diskTotal)))
                     : 'Unknown';
               }
               break;
            case 'comment':
               if (localState.columnSearchState['comment'][agent.id]) {
                  toCheck = localState.columnSearchState['comment'][agent.id];
               }
               else {
                  toCheck = "Unknown";
               }
               break;
            case 'OS':
               toCheck = getAgentOs(agent) ?? "Unknown";
               break;
            case 'CPU':
               const CPU = getAgentCapability(agent, "CPU");
               toCheck = CPU ?? "Unknown";
               break;
            case 'RAM':
               const RAM = getAgentCapability(agent, "RAM");
               toCheck = RAM ?? "Unknown";
               break;
            default:
               toCheck = "Unknown";
               return null;
         }
         if (localState.filterExactMatch) {
            if (toCheck.toString().toLowerCase() === filter) {
               return agent;
            }
         }
         else {
            if (toCheck.toString().toLowerCase().indexOf(filter) !== -1) {
               return agent;
            }
         }
      }
      else {
         return agent;
      }
   }

   function getStorageMetrics(agent: AgentData, config: StorageColumnConfig): {
      freeGiB: number;
      percentUsed: number;
      displayText: string;
   } {
      const diskFree = getAgentCapability(agent, config.freeCapability);
      const diskTotal = getAgentCapability(agent, config.totalCapability);

      if (diskFree === null || diskTotal === null) {
         return { freeGiB: 0, percentUsed: 0, displayText: 'Unknown' };
      }

      const freeGiB = Number(diskFree) / BYTES_PER_GIB;
      const totalGiB = Number(diskTotal) / BYTES_PER_GIB;
      const percentUsed = (totalGiB - freeGiB) / totalGiB;

      return {
         freeGiB,
         percentUsed,
         displayText: freeGiB.toLocaleString(undefined, { maximumFractionDigits: 0 }) + ' GiB Free'
      };
   }

   function renderStorageCell(agent: AgentData, columnKey: string): JSX.Element {
      const config = STORAGE_COLUMNS[columnKey];
      const metrics = getStorageMetrics(agent, config);

      // Update search state for sorting/filtering
      localState.columnSearchState[config.searchStateKey][agent.id] = metrics.freeGiB;

      // Nudge percentage slightly for progress bar visual
      const nudgedPercent = metrics.percentUsed * 0.965;

      return (
         <Stack horizontal horizontalAlign={"center"}>
            <Stack horizontal horizontalAlign="end" styles={{ root: { minWidth: 80, maxWidth: 80 } }}>
               <Text styles={{ root: { marginRight: 10, marginTop: 7 } }}>
                  {metrics.displayText}
               </Text>
            </Stack>
            <Stack horizontal styles={{ root: { minWidth: 110, maxWidth: 110 } }}>
               <ProgressIndicator
                  barHeight={15}
                  styles={{
                     progressBar: {
                        height: dashboard.darktheme ? 13 : 16,
                        marginLeft: dashboard.darktheme ? 1 : 0,
                        marginTop: dashboard.darktheme ? 2 : 0
                     },
                     progressTrack: { width: '98%', border: '1px solid !important' },
                     root: { width: 110 }
                  }}
                  percentComplete={nudgedPercent}
               />
            </Stack>
         </Stack>
      );
   }

   function getFreeDiskSpaceInGB(diskFree: number, diskCapacity: number): string | undefined {
      if (diskFree === null || diskCapacity === null) {
         return undefined;
      }

      const freeGB = diskFree / BYTES_PER_GIB;
      return freeGB.toFixed(0);
   }

   function sortAgents(a: AgentData, b: AgentData) {
      // sort differently depending on which column this is
      const sortedCol = localState.columnsState.find(c => c.isSorted === true)!;
      let left = a;
      let right = b;
      if (sortedCol.isSortedDescending) {
         left = b;
         right = a;
      }
      switch (sortedCol.key) {
         case 'name':
            return left.name.toLocaleLowerCase().localeCompare(right.name, undefined, { numeric: true, sensitivity: 'base' });
         case 'pools':
            if (localState.columnSearchState['pools'][left.id] && localState.columnSearchState['pools'][right.id]) {
               return localState.columnSearchState['pools'][left.id].join('').localeCompare(localState.columnSearchState['pools'][right.id].join(''));
            }
            return left.name.localeCompare(right.name);
         case 'status':
            let [leftAgentStatus, leftAgentDate] = getAgentStatus(left);
            let [rightAgentStatus, rightAgentDate] = getAgentStatus(right);

            // sort by date if agents have date data and the same status
            if(leftAgentDate && rightAgentDate && (leftAgentStatus == rightAgentStatus)) {
               return leftAgentDate.getTime() - rightAgentDate.getTime();
            }
            return (leftAgentStatus ?? "").localeCompare(rightAgentStatus ?? "");
         case 'software':
            return (left.version ?? "").toLocaleLowerCase().localeCompare(right.version ?? "");
         case 'taskTime':
            return getTaskTime(left) - getTaskTime(right);
         case 'workingStorage':
         case 'systemStorage':
            const sortConfig = STORAGE_COLUMNS[sortedCol.key];
            const leftVal = localState.columnSearchState[sortConfig.searchStateKey][left.id];
            const rightVal = localState.columnSearchState[sortConfig.searchStateKey][right.id];
            if (leftVal !== undefined && rightVal !== undefined) {
               return leftVal - rightVal;
            }
            return Number(getAgentCapability(left, sortConfig.freeCapability) ?? 0) - Number(getAgentCapability(right, sortConfig.freeCapability) ?? 0);
         case 'comment':
            if (localState.columnSearchState['comment'][left.id] && localState.columnSearchState['comment'][right.id]) {
               return localState.columnSearchState['comment'][left.id].localeCompare(localState.columnSearchState['comment'][right.id]);
            }
            return (left.comment ?? "").toLocaleLowerCase().localeCompare(right.comment ?? "");
         case 'systemInfoOS':
            return ((getAgentOs(left) ?? "Unknown").toLocaleLowerCase().localeCompare((getAgentOs(right) ?? "Unknown")));
         case 'systemInfoCPU':
            return (getAgentCapability(left, "CPU") ?? "Unknown").toLocaleLowerCase().localeCompare((getAgentCapability(right, "CPU") ?? "Unknown"));
         case 'systemInfoRAM':
            return parseInt((getAgentCapability(left, "RAM") ?? "0").replace(/\D/g, "")) - parseInt((getAgentCapability(right, "RAM") ?? "0").replace(/\D/g, ""));
         default:
            return left.name.localeCompare(right.name);
      }
   }

   function getAgentStatus(agent: AgentData): [string, Date | undefined] {
      let agentStatus: string = '';
      let dateInfo: Date | undefined = undefined;

      // get status from agent's leases
      if (agent.leases && agent.leases.length > 0) {
         agent.leases.forEach(lease => {
            let statusText = lease.name;

            if (lease.state === LeaseState.Cancelled) {
               if (lease.finishTime) {
                  statusText += " (Canceled)"
               } else {
                  statusText += " (Cancelling)"
               }
            }
            
           agentStatus += statusText;
         });
      }
      // if there are no leases, get status from other state
      if (agentStatus.length === 0) {
         agentStatus = agent.status ?? "";

         if (agentStatus === "Ok") {
            agentStatus = "Ready";
         }
         if (!agent.online) {
            agentStatus = `Offline - ${agent.lastShutdownReason}`
            dateInfo = new Date(agent.updateTime);
         }
         else if (agent.pendingShutdown) {
            agentStatus = `Pending Shutdown ${agent.lastShutdownReason}`;
         }
         else if (agent.pendingConform || agent.pendingFullConform) {
            if (agent.conformAttemptCount && agent.conformAttemptCount > 0) {
               agentStatus = `${agent.pendingFullConform ? "Full " : ""}Conform failed (${agent.conformAttemptCount} attempts)`;
               if (agent.nextConformTime) {
                  dateInfo = new Date(agent.nextConformTime);
               }
            }
            else {
               agentStatus = "Pending Conform";
               if (agent.pendingFullConform) {
                  agentStatus += " (Full)";
               }
            }
         }
         else if (!agent.enabled) {
            agentStatus = "Disabled";
         }
      }

      return [agentStatus, dateInfo];
   }

   function onContextMenu(item?: any, index?: number | undefined, ev?: Event | undefined) {
      const event = ev as MouseEvent;
      localState.setRightClickDiv(event?.clientX, event?.clientY);
      localState.setAgentContextMenuOpen(true);
   }

   function onHistoryModalDismiss() {
      localState.setAgentId(undefined);
   }

   const totalPoolSize = effectiveAgents.filter((a) => {
      if (!a.pools) {
         return false;
      }
      return !poolId || a.pools.indexOf(poolId) !== -1
   }).length;

   let agentItems = effectiveAgents.filter(filterAgents, localState.agentFilter).sort(sortAgents);

   if (poolId) {
      agentItems = agentItems.filter((item => {
         return !!item.pools && item.pools.indexOf(poolId!) !== -1;
      }))
   }

   agentItems = agentItems.filter(item => {
      const filter = localState.agentStatusFilter;
      if (!filter.size) {
         return true;
      }

      let filtered = true;

      if (filter.has("Active")) {
         if (item.leases?.length) {
            filtered = false;
         }
      }

      if (filter.has("No Pools")) {
         if (!item.pools?.length) {
            filtered = false;
         } else {
            let anyInvalid = false;
            item.pools.forEach(pid => {

               if (anyInvalid) {
                  return;
               }

               if (!agentStore.pools.find(pool => { return pool.id === pid; })) {
                  anyInvalid = true;
               }

            });

            if (anyInvalid) {
               filtered = false;
            }
         }

      }

      if (filter.has("Ephemeral")) {
         if (item.ephemeral) {
            filtered = false;
         }
      }

      if (filter.has("Offline")) {
         if (!item.online) {
            filtered = false;
         }
      }

      if (filter.has("Offline (Autoscaler)")) {
         if (!item.online && item.lastShutdownReason === "Autoscaler") {
            filtered = false;
         }
      }

      if (filter.has("Offline (Unexpected)")) {
         if (!item.online && item.lastShutdownReason === "Unexpected") {
            filtered = false;
         }
      }

      if (filter.has("Offline (Manual)")) {
         if (!item.online && item.lastShutdownReason?.startsWith("Manual")) {
            filtered = false;
         }
      }

      if (filter.has("Pending Shutdown")) {

         if (item.online && item.pendingShutdown) {
            filtered = false;
         }
      }

      if (filter.has("Disabled")) {
         if (!item.enabled) {
            filtered = false;
         }
      }

      if (filter.has("Pending Conform")) {

         if (item.pendingConform || item.pendingFullConform) {
            filtered = false;
         }
      }

      if (filter.has("Ready")) {
         if (!item.leases?.length && item.online && item.enabled && !item.pendingConform && !item.pendingFullConform) {
            filtered = false;
         }
      }

      return !filtered;
   });

   const height = Math.min(((totalPoolSize < 5 ? 5 : totalPoolSize) * 48) + 48, 480)

   return (<Stack>
      {!!agentView && <SearchUpdate />}
      <Stack horizontal style={{ paddingBottom: !agentView ? 18 : 0 }}>
         {!!agentView && <AgentPivot />}
         <Stack grow />
         <AgentMenuBar agentView={agentView} />
      </Stack>
      <Stack style={{ position: "relative", height: agentView ? "calc(100vh - 240px)" : height }}>
         <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always}>
            <DetailsList
               className={agentStyles.detailsList}
               onItemContextMenu={onContextMenu}
               checkboxCellClassName={agentStyles.checkboxCell}
               compact={true}
               setKey="set"
               items={agentItems}
               columns={localState.columnsState.filter(colState => { return colState.isChecked; }).map(colState => { return colState.columnDef!; })}
               selection={localState.selection}
               onRenderDetailsHeader={onRenderDetailsHeader}
               onColumnHeaderContextMenu={onColumnHeaderContextMenu}
               onRenderItemColumn={onRenderAgentListItem}
               layoutMode={DetailsListLayoutMode.justified}
               constrainMode={ConstrainMode.horizontalConstrained}
               selectionMode={SelectionMode.multiple}
            />
            <ContextualMenu
               items={localState.columnMenuProps}
               onItemClick={() => localState.setHeaderContextMenuOpen(false)}
               onDismiss={() => localState.setHeaderContextMenuOpen(false)}
               isBeakVisible={true}
               hidden={!localState.headerContextMenuOpen}
               target={localState.contextMenuTargetRef}
               directionalHint={DirectionalHint.bottomLeftEdge}
               directionalHintFixed={true}
            />
            <ContextualMenu
               items={agentContextMenuProps}
               onItemClick={() => localState.setAgentContextMenuOpen(false)}
               onDismiss={() => localState.setAgentContextMenuOpen(false)}
               isBeakVisible={true}
               target={{ x: localState.mouseX, y: localState.mouseY }}
               hidden={!localState.agentContextMenuOpen}
               directionalHint={DirectionalHint.bottomLeftEdge}
               directionalHintFixed={false}
            />
         </ScrollablePane>
         <Stack grow />
      </Stack>
      <ConfirmationDialog
         title={`Delete Agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}`}
         subText={`Really delete selected agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}?`}
         isOpen={localState.deleteAgentDialogIsOpen}
         confirmText={"Delete"}
         cancelText={"Cancel"}
         onConfirm={() => { localState.deleteBuilder() }}
         onCancel={() => { localState.setDeleteBuilderDialogOpen(false) }}
      />
      <ConfirmationDialog
         title={`Restart Agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}`}
         subText={`Really restart selected agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}?`}
         isOpen={localState.restartAgentDialogIsOpen}
         confirmText={"Restart"}
         cancelText={"Cancel"}
         checkBoxText='Force Restart'
         onConfirm={(_, force) => { localState.requestBuilderUpdate(false, !force ? true : false, false, false, !!force) }}
         onCancel={() => { localState.setRestartBuilderDialogOpen(false) }}
      />
      <ConfirmationDialog
         title={`Shutdown Agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}`}
         subText={`Really shutdown selected agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}?`}
         isOpen={localState.shutdownAgentDialogIsOpen}
         confirmText={"Shutdown"}
         cancelText={"Cancel"}
         textBoxLabel={"Type Confirm to confirm"}
         isTextBoxSpawned={true}
         onConfirm={() => { localState.requestBuilderUpdate(false, false, false, true) }}
         onCancel={() => { localState.setShutdownBuilderDialogOpen(false) }}
      />

      <ConfirmationDialog
         title={`Cancel Leases`}
         subText={`Really cancel selected agent leases?`}
         isOpen={localState.cancelLeasesDialogIsOpen}
         confirmText={"Yes"}
         cancelText={"No"}
         onConfirm={() => { localState.setCancelLeasesDialogOpen(false); localState.cancelLeases() }}
         onCancel={() => { localState.setCancelLeasesDialogOpen(false) }}
      />
      <ConfirmationDialog
         title={`Disable Agent${localState.selection.getSelectedCount() > 1 ? "s" : ""}`}
         isOpen={localState.disableAgentDialogIsOpen}
         confirmText={"Disable"}
         cancelText={"Cancel"}
         textBoxLabel={"Enter Disable Reason - Required"}
         isTextBoxSpawned={true}
         isTextRequired={true}
         onConfirm={(textFieldText: string) => { localState.changeBuilderEnabled("disable", textFieldText) }}
         onCancel={() => { localState.setDisableBuilderDialogOpen(false) }}
      />
      <ConfirmationDialog
         title={`Edit Agent Comment${localState.selection.getSelectedCount() > 1 ? "s" : ""}`}
         isOpen={localState.editCommentDialogIsOpen}
         confirmText={"Edit Comment"}
         cancelText={"Cancel"}
         textBoxLabel={"New Comment"}
         isTextBoxSpawned={true}
         onConfirm={(textFieldText: string) => { localState.changeCommment(textFieldText) }}
         onCancel={() => { localState.setEditCommentDialogOpen(false) }}
      />

      <HistoryModal agentId={activeAgent?.id} onDismiss={onHistoryModalDismiss}></HistoryModal>
   </Stack>
   );


   function onColumnHeaderContextMenu(column?: IColumn | undefined, ev?: React.MouseEvent<HTMLElement, MouseEvent> | undefined) {
      const key = column!.key;
      if (key === "software" || key === "storage" || key.indexOf("systemInfo") !== -1 || key === "taskTime" || key === "comment") {
         localState.contextMenuTargetRef = ev?.nativeEvent!;
         localState.setHeaderContextMenuOpen(true);
      }
   }

   function getAgentStatusIcon(agent: AgentData) {

      const statusColors = dashboard.getStatusColors();

      let color = statusColors.get(StatusColor.Success);
      if (agent.enabled === false) {
         color = statusColors.get(StatusColor.Warnings);
      }
      if (!agent.online) {
         color = statusColors.get(StatusColor.Skipped);
      }
      return <Icon iconName="FullCircle" style={{ color: color, fontSize: 16, marginRight: "13px", paddingTop: "2px" }} />;
   }

   function getPropFromDevice(agent: AgentData, propKey: string, propValue: string | null = null) {
      const unknownElement = <Stack styles={{ root: { height: '100%' } }} horizontal horizontalAlign={"center"}><Stack.Item align={"center"}>Unknown</Stack.Item></Stack>;
      let prop = propValue ?? getAgentCapability(agent, propKey);
      if (prop === null)
         return unknownElement;

      if (propKey.indexOf("RAM") !== -1) {
         prop += " GB";
      }
      return (
         <Stack styles={{ root: { height: '100%' } }} horizontal horizontalAlign={"center"}>
            <Stack.Item align={"center"} className={agentStyles.ellipsesStackItem}>
               <Text title={prop} key={`sysInfo_${agent.id}_${propKey}`}>{`${prop}`}</Text>
            </Stack.Item>
         </Stack>
      );
   }

   // when an actual item is drawn
   function onRenderAgentListItem(agent: AgentData, index?: number, column?: IColumn) {
      switch (column!.key) {
         case 'name':
            return (
               <Stack styles={{ root: { height: 31, } }} horizontal horizontalAlign={"start"}>
                  <Stack.Item align={"center"}>
                     {getAgentStatusIcon(agent)}
                  </Stack.Item>
                  <Stack.Item align={"center"}>
                     <ReactLink styles={{ root: { paddingTop: '1px', fontSize: "12px" } }} title="Lease and Session History" onClick={() => localState.setAgentId(agent.id)}>{agent.name}</ReactLink>
                  </Stack.Item>
               </Stack>
            );
         case 'pools':
            const poolItems: any = [];
            const poolSearchNames: string[] = [];
            const invalidPoolIds = new Set<string>();
            if (agent.pools) {
               const poolObjs: PoolData[] = [];
               // get actual pool objects
               for (const key in agent.pools) {
                  const id = agent.pools[key];
                  const pool = agentStore.pools.find(pool => { return pool.id === id; });
                  if (pool) {
                     poolObjs.push(pool);
                  } else {
                     invalidPoolIds.add(id);
                     poolObjs.push({
                        id: id,
                        name: id ?? "",
                        colorValue: "#797979",
                        enableAutoscaling: false,
                        workspaces: []
                     })
                  }
               }
               // sort them by name
               poolObjs.sort((a, b) => {
                  return a.name.localeCompare(b.name);
               });
               for (let idx = 0; idx < poolObjs.length; idx++) {
                  let color = poolObjs[idx].colorValue;
                  const textColor = invalidPoolIds.has(poolObjs[idx]?.id) ? "#DDDDDD" : "white";
                  if (agent.pendingConform || agent.pendingFullConform) {
                     const pendingConformColor = hexToRGB(color);
                     color = `rgb(${pendingConformColor.r},${pendingConformColor.g},${pendingConformColor.b}, .5)`;
                  }
                  const menuProps: IContextualMenuProps = {
                     items: [
                        {
                           key: 'agent_pool_view',
                           text: 'View Pool',
                           onClick: (ev) => {
                              window.open(`/pools?pool=${poolObjs[idx]?.id}`, '_blank')
                           }
                        },
                        {
                           key: 'agent_pool_filter',
                           text: 'Filter to Pool',
                           onClick: (ev) => {
                              localState.setExactMatch(true);
                              localState.setAgentFilter(poolObjs[idx]?.name);
                           }
                        },
                        {
                           key: 'agent_pool_copy_to_clipboard',
                           text: 'Copy to Clipboard',
                           onClick: (ev) => {
                              copyToClipboard(poolObjs[idx].name)
                           }
                        }
                     ],
                  };

                  const disabled = invalidPoolIds.has(poolObjs[idx].id);


                  poolItems.push(
                     <Stack.Item align={"center"} key={"pool_" + agent.id + "_" + poolObjs[idx].id}>
                        <DefaultButton key={poolObjs[idx].id}
                           text={poolObjs[idx].name}
                           primary
                           menuProps={!disabled ? menuProps : undefined}
                           menuIconProps={{ iconName: "" }}
                           className={agentStyles.buttonFont}
                           onClick={(ev) => { ev.preventDefault(); ev.stopPropagation() }}
                           styles={{
                              root: { border: '0px', backgroundColor: color, color: textColor },
                              rootHovered: { border: '0px', backgroundColor: color, color: textColor, },
                              rootPressed: { border: '0px', backgroundColor: color, color: textColor, }
                           }} />

                     </Stack.Item>
                  );
                  poolSearchNames.push(poolObjs[idx].name);
               }
            }
            localState.columnSearchState['pools'][agent.id] = poolSearchNames;
            return <Stack horizontal horizontalAlign={"start"} styles={{ root: { overflow: "auto", height: '100%' } }} tokens={{ childrenGap: 6 }}>{poolItems}</Stack>;
         case 'status':
            const leases: any = [];
            const leaseSearchItems: string[] = [];
            if (agent.leases) {
               agent.leases.forEach(lease => {

                  let statusText = lease.name;

                  if (lease.state === LeaseState.Cancelled) {
                     if (lease.finishTime) {
                        statusText += " (Canceled)"
                     } else {
                        statusText += " (Cancelling)"
                     }
                  }

                  let link = "";
                  if (lease.details) {
                     if ('jobId' in lease.details) {
                        link = `/job/${lease.details['jobId']}`;
                        if ('batchId' in lease.details) {
                           link += `?batch=${lease.details['batchId']}`;
                        }
                     }
                     else if ('logId' in lease.details) {
                        link = `/log/${lease.details['logId']}?agentId=${agent.id}`;
                     }
                  }
                  if (link !== "") {
                     // marquees dont play nice unless the parent container is 100% of the width. regular centered stack items dont play nice if
                     // their root width's are set...
                     if (statusText!.length > 46) {
                        leases.push(<Stack.Item align={"center"} key={"statusText_item_" + agent.id + "_" + lease.id} styles={{ root: { width: '100%' } }}>
                           <Link key={"statusText_" + agent.id + "_" + lease.id} to={link}>
                              <Marquee loop={true} trailing={2000} text={statusText} />
                           </Link>
                        </Stack.Item>);
                     }
                     else {
                        leases.push(<Stack.Item align={"center"} key={"statusText_item_" + agent.id + "_" + lease.id}>
                           <Link key={"statusText_" + agent.id + "_" + lease.id} to={link}>{statusText}</Link>
                        </Stack.Item>);
                     }
                     leases.push();
                  }
                  else {
                     leases.push(<Stack.Item align={"center"} key={"statusText_item_" + agent.id + "_" + lease.id} className={agentStyles.ellipsesStackItem}>
                        <Text key={"statusText_" + agent.id + "_" + lease.id}>{statusText}</Text>
                     </Stack.Item>);
                  }
                  if (lease.name) {
                     leaseSearchItems.push(lease.name);
                  }
               });
            }
            // if there are no leases, we'll push some other state.
            if (leases.length === 0) {
               let title = agent.status ?? "";
               if (title === "Ok") {
                  title = "Ready";
               }
               let subtitle = "";
               if (!agent.online) {
                  title = `Offline - ${agent.lastShutdownReason}`;
                  subtitle = `Last online at ${getShortNiceTime(agent.updateTime, true, true, true)}`;
               }
               else if (agent.pendingShutdown) {
                  title = `Pending Shutdown`;
                  subtitle = agent.lastShutdownReason;
               }
               else if (agent.pendingConform || agent.pendingFullConform) {
                  if (agent.conformAttemptCount && agent.conformAttemptCount > 0) {
                     title = `${agent.pendingFullConform ? "Full " : ""}Conform failed (${agent.conformAttemptCount} attempts)`;
                     if (agent.nextConformTime) {
                        subtitle = `Next attempt at ${getShortNiceTime(agent.nextConformTime, true, true, true)}`;
                     }
                  }
                  else {
                     title = "Pending Conform";
                     if (agent.pendingFullConform) {
                        title += " (Full)";
                     }
                  }
               }
               else if (!agent.enabled) {
                  title = "Disabled";
               }

               leases.push(<Stack.Item key={"itemStatus_" + agent.id} align={"center"} className={agentStyles.ellipsesStackItem}>
                  <Text key={"statusText_" + agent.id}>{title}</Text>
               </Stack.Item>);
               leaseSearchItems.push(title);
               if (subtitle !== "") {
                  leases.push(<Stack.Item key={"itemSubtitleStatus_" + agent.id} align={"center"} className={agentStyles.ellipsesStackItem}>
                     <Text variant="small" key={"statusSubtitleText_" + agent.id}>{subtitle}</Text>
                  </Stack.Item>);
                  leaseSearchItems.push(subtitle);
               }

            }

            localState.columnSearchState['status'][agent.id] = leaseSearchItems;
            return <Stack styles={{ root: { height: '100%', } }} verticalAlign={"center"}>{leases}</Stack>;
         case 'software':
            return (<Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={"center"}>
               <Stack.Item align={"center"}>
                  <Text key={"softwareText_" + agent.id} className={agentStyles.ellipsesStackItem}>{agent.version ?? "Unknown"}
                  </Text>
               </Stack.Item>
            </Stack>);
         case 'taskTime':
            let longestDuration: moment.Duration | undefined;
            let taskText = "Agent Idle";
            if (agent.leases) {
               agent.leases.forEach(lease => {
                  const start = moment(lease.startTime);
                  const end = moment(Date.now());
                  const duration = moment.duration(end.diff(start));
                  if (!longestDuration) {
                     longestDuration = duration;
                  }
                  else {
                     if (duration.asSeconds() > longestDuration.asSeconds()) {
                        longestDuration = duration;
                     }
                  }
               });
            }

            if (longestDuration) {
               taskText = longestDuration!.humanize();
               localState.columnSearchState['taskTime'][agent.id] = longestDuration!.asSeconds();
            }
            else {
               localState.columnSearchState['taskTime'][agent.id] = 0;
            }

            localState.columnSearchState['taskTimeDescription'][agent.id] = taskText;


            return (<Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={"center"}>
               <Stack.Item align={"center"}>
                  <Text key={"taskTimeText_" + agent.id} className={agentStyles.ellipsesStackItem}>{taskText}
                  </Text>
               </Stack.Item>
            </Stack>);
         case 'workingStorage':
         case 'systemStorage':
            return renderStorageCell(agent, column!.key);
         case 'comment':
            localState.columnSearchState['comment'][agent.id] = agent.comment ?? "";
            return (<Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={"center"}>
               <Stack.Item align={"center"}>
                  <Text key={"commentText_" + agent.id} className={agentStyles.ellipsesStackItem}>{agent.comment ?? ""}
                  </Text>
               </Stack.Item>
            </Stack>);
         case 'systemInfoOS':
            return getPropFromDevice(agent, "OSDistribution", getAgentOs(agent));
         case 'systemInfoCPU':
            return getPropFromDevice(agent, "CPU");
         case 'systemInfoRAM':
            return getPropFromDevice(agent, "RAM");
         default:
            return <span>{agent[column!.fieldName as keyof AgentData] as string}</span>;
      }
   }
});

export const AgentPanel: React.FC<{ agentId?: string, poolId?: string, agentView?: boolean, agents?: AgentData[] }> = observer(({ agentId, agentView, poolId, agents }) => {

   // adjust automatically to viewport changes
   useWindowSize();

   // subscribe
   if (localState.searchUpdated) { }

   localState.agentView = agentView;
   localState.updateColumns();

   return <Stack>
      <AgentViewInner agentId={localState.searchState?.agentId} agentView={agentView} poolId={poolId} agents={agents} />
   </Stack>

})

export const AgentView: React.FC = () => {

   const [searchParams] = useSearchParams();
   const hordeTheme = getHordeTheme();
   const { hordeClasses } = getHordeStyling();

   const agentId = searchParams.get("agentId") ? searchParams.get("agentId") : undefined;

   localState.agentView = true;
   localState.updateColumns();

   // adjust automatically to viewport changes
   useWindowSize();

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Agents' }]} />
      <Stack horizontal>
         <Stack grow styles={{ root: { backgroundColor: hordeTheme.horde.neutralBackground } }} />
         <Stack tokens={{ maxWidth: 1440, childrenGap: 4 }} styles={{ root: { width: 1440, height: '100vh', backgroundColor: hordeTheme.horde.neutralBackground, paddingTop: 18, paddingLeft: 12 } }}>
            <Stack className={hordeClasses.raised} styles={{ root: { paddingRight: '40px' } }}>
               <AgentViewInner agentView={true} agentId={agentId ? agentId : undefined} searchParams={searchParams} />
            </Stack>
         </Stack>
         <Stack grow styles={{ root: { backgroundColor: hordeTheme.horde.neutralBackground } }} />
      </Stack>
   </Stack>

}
