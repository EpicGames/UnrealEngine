// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, ComboBox, ConstrainMode, DefaultButton, DetailsHeader, DetailsList, DetailsListLayoutMode, DetailsRow, Dialog, DialogFooter, DialogType, FontIcon, GroupedList, GroupHeader, ICheckbox, IColumn, IComboBoxOption, IContextualMenuItem, IContextualMenuProps, IDetailsHeaderProps, IDetailsHeaderStyles, IDetailsListProps, IGroup, ITextField, ITooltipHostStyles, Label, mergeStyleSets, Modal, Pivot, PivotItem, PrimaryButton, ScrollablePane, ScrollbarVisibility, Selection, SelectionMode, Spinner, SpinnerSize, Stack, Sticky, StickyPositionType, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import { Link } from "react-router-dom";
import { Sparklines, SparklinesLine, SparklinesReferenceLine } from "react-sparklines";
import backend from "../../backend";
import { agentStore } from "./AgentStore";
import { GetAgentLeaseResponse, GetAgentTelemetrySampleResponse, JobStepBatchError, JobStepOutcome, LeaseData, SessionData, UpdateAgentRequest } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { getNiceTime, getShortNiceTime } from "../../base/utilities/timeUtils";
import { getHordeStyling } from "../../styles/Styles";
import { getHordeTheme } from "../../styles/theme";
import { BatchStatusIcon, LeaseStatusIcon, StepStatusIcon } from "./../StatusIcon";
import { historyModalStore, processLeaseData, ProcessedLease } from "./HistoryModalStore";
import { formatBytes } from "horde/base/utilities/stringUtills";

const historyStyles = mergeStyleSets({
   dialog: {
      selectors: {
         ".ms-Label,.ms-Button-label": {
            fontWeight: "unset",
            fontFamily: "Horde Open Sans SemiBold"
         }
      }
   },
   detailsList: {
      selectors: {
         ".ms-DetailsHeader-cellName": {
            fontWeight: "unset",
            fontFamily: "Horde Open Sans SemiBold"
         }
      }
   }
});

const state = historyModalStore;

const TelemetryTooltip: React.FC<{ id: "cpu" | "ram" | "workingDisk" | "systemDisk", cpuKey?: string }> = observer(({ id, cpuKey }) => {

   const { modeColors } = getHordeStyling();

   const tooltip = state.currentLeaseTooltip;

   if (!tooltip?.lease || tooltip.id !== id) {
      return null;
   }

   let tipX = tooltip.x;
   let offsetX = 32;
   let translateX = "0%";

   if ((tipX ?? 0) > 600) {
      offsetX = -32;
      translateX = "-100%";
   }

   const translateY = "-50%";

   const textSize = 11;
   const titleWidth = 58;

   const dataElement = (title: string, value: string, link?: string, agent?: string, linkTarget?: string) => {

      let valueElement = <Stack>
         <Stack style={{ fontSize: textSize }}>
            {value}
         </Stack>
      </Stack>

      if (link) {
         valueElement = <Link className="horde-link" to={link} target={linkTarget}>
            {valueElement}
         </Link>
      }

      let component = <Stack style={{ cursor: (link || agent) ? "pointer" : undefined }} onClick={() => {
      }}>
         <Stack horizontal>
            <Stack style={{ width: titleWidth }}>
               {!!title && <Text style={{ fontFamily: "Horde Open Sans SemiBold", fontSize: textSize }} >{title}</Text>}
            </Stack>
            {valueElement}
         </Stack>
      </Stack>

      return component;
   }

   const elements: JSX.Element[] = [];

   const lease = tooltip.lease;

   if (lease?.details?.type === "Job") {
      elements.push(dataElement("Job:", tooltip.lease?.name ?? "Unknown Job Lease", `/job/${lease.details.jobId}`));
      const job = state.jobData.get(lease.details.jobId);
      if (job && job !== true && job.batches) {
         const step = job.batches.map(b => b.agentId === state.selectedAgent?.id ? b.steps : []).flat().find(s => {
            if (s.startTime && tooltip.time) {
               if (tooltip.time >= new Date(s.startTime) && (!s.finishTime || tooltip.time <= new Date(s.finishTime))) {
                  return true;
               }
            }
            return false;
         });
         if (step) {
            elements.push(dataElement("Step:", step.name ?? "Unknown Job Step", `/job/${lease.details.jobId}?step=${step.id}`));
            if (step.logId)
               elements.push(dataElement("Log:", `Step`, `/log/${step.logId}`));
         }
      }
   } else {
      elements.push(dataElement("Lease:", tooltip.lease?.name ?? "Unknown Lease"));
   }

   if (lease?.details?.logId) {
      elements.push(dataElement("Log:", `Lease (${lease.id})`, `/log/${lease.details.logId}`));
   }

   if (tooltip.time) {
      elements.push(dataElement("Time:", getNiceTime(tooltip.time)));
   }

   // Calculate and display lease duration
   if (lease) {
      const startMs = new Date(lease.startTime).getTime();
      const endMs = lease.finishTime ? new Date(lease.finishTime).getTime() : Date.now();
      const durationMs = endMs - startMs;

      const totalSeconds = Math.floor(durationMs / 1000);
      const days = Math.floor(totalSeconds / 86400);
      const hours = Math.floor((totalSeconds % 86400) / 3600);
      const minutes = Math.floor((totalSeconds % 3600) / 60);
      const seconds = totalSeconds % 60;

      const parts: string[] = [];
      if (days > 0) parts.push(`${days}d`);
      if (hours > 0 || days > 0) parts.push(`${hours}h`);
      if (minutes > 0 || hours > 0 || days > 0) parts.push(`${minutes}m`);
      parts.push(`${seconds}s`);

      let durationStr = parts.join(" ");
      if (!lease.finishTime) {
         durationStr += " (ongoing)";
      }

      elements.push(dataElement("Duration:", durationStr));
   }

   if (tooltip.sample) {

      const d = tooltip.sample;
      let title = "Unknown";
      let value = "";

      switch (id) {
         case "cpu":
            title = "CPU";
            if (cpuKey == "user") value = `${Math.ceil(d.userCpu)}%`;
            if (cpuKey == "system") value = `${Math.ceil(d.systemCpu)}%`;
            if (cpuKey == "combined") value = `${Math.ceil(d.userCpu + d.systemCpu)}%`;
            break;
         case "workingDisk":
            title = "Disk"
            value = formatBytes((d.workingTotalDisk - d.workingFreeDisk) * 1024 * 1024) + " Used";
            break;
         case "systemDisk":
            title = "Disk"
            value = formatBytes((d.systemTotalDisk - d.systemFreeDisk) * 1024 * 1024) + " Used";
            break;
         case "ram":
            title = "Ram"
            value = `${formatBytes(d.usedRam * 1024 * 1024)} Used`;
            break;
      }

      elements.push(dataElement(`${title}:`, value));
   }


   return <div style={{
      position: "absolute",
      display: "block",
      top: `${tooltip.y}px`,
      left: `${(tooltip.x ?? 0) + offsetX}px`,
      backgroundColor: modeColors.background,
      zIndex: 1,
      border: "solid",
      borderWidth: "1px",
      borderRadius: "3px",
      width: "max-content",
      borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F",
      pointerEvents: tooltip.frozen ? undefined : "none",
      transform: `translate(${translateX}, ${translateY})`
   }}>
      <Stack horizontal>
         <Stack tokens={{ childrenGap: 6, padding: 12 }}>
            {elements}
         </Stack>
         {!!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12, paddingTop: 12, cursor: "pointer" }} onClick={() => { state.setLeaseTooltip(undefined) }}>
            <FontIcon iconName="Cancel" />
         </Stack>}
         {!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12 + 16, paddingTop: 12 }} />}

      </Stack>
   </div>

});

type TimeSelection = {
   text: string;
   key: string;
   minutes: number;
}

const timeSelections: TimeSelection[] = [
   {
      text: "Past 1 Hour", key: "time_1_hour", minutes: 60
   },
   {
      text: "Past 2 Hours", key: "time_2_hours", minutes: 60 * 2
   },
   {
      text: "Past 4 Hours", key: "time_4_hours", minutes: 60 * 4
   },
   {
      text: "Past 8 Hours", key: "time_8_hours", minutes: 60 * 8
   },
   {
      text: "Past 12 Hours", key: "time_12_hours", minutes: 60 * 12
   },
   {
      text: "Past 1 Day", key: "time_1_day", minutes: 60 * 24
   },
   {
      text: "Past 2 Days", key: "time_2_days", minutes: 60 * 24 * 2
   },
   {
      text: "Past 1 Week", key: "time_1_week", minutes: 60 * 24 * 7
   },
   {
      text: "Past 2 Weeks", key: "time_2_weeks", minutes: 60 * 24 * 7 * 2
   },
   {
      text: "Past Month", key: "time_4_weeks", minutes: 60 * 24 * 7 * 4
   }
]

const cpuSelections: IComboBoxOption[] = [
   {
      text: "Combined", key: "combined"
   },
   {
      text: "User", key: "user"
   },
   {
      text: "System", key: "system"
   }
]

// 10 distinct colors for lease overlays - more transparent to not occlude graph data
const leaseColorsDark = [
   "rgba(100, 149, 237, 0.15)",  // Cornflower blue
   "rgba(144, 238, 144, 0.15)",  // Light green
   "rgba(255, 182, 108, 0.15)",  // Peach
   "rgba(221, 160, 221, 0.15)",  // Plum
   "rgba(64, 224, 208, 0.15)",   // Turquoise
   "rgba(255, 218, 185, 0.15)",  // Peach puff
   "rgba(135, 206, 235, 0.15)",  // Sky blue
   "rgba(255, 160, 122, 0.15)",  // Light salmon
   "rgba(152, 251, 152, 0.15)",  // Pale green
   "rgba(216, 191, 216, 0.15)",  // Thistle
];

const leaseColorsLight = [
   "rgba(65, 105, 225, 0.12)",   // Royal blue
   "rgba(34, 139, 34, 0.12)",    // Forest green
   "rgba(255, 140, 0, 0.12)",    // Dark orange
   "rgba(148, 0, 211, 0.12)",    // Dark violet
   "rgba(0, 139, 139, 0.12)",    // Dark cyan
   "rgba(184, 134, 11, 0.12)",   // Dark goldenrod
   "rgba(70, 130, 180, 0.12)",   // Steel blue
   "rgba(205, 92, 92, 0.12)",    // Indian red
   "rgba(60, 179, 113, 0.12)",   // Medium sea green
   "rgba(186, 85, 211, 0.12)",   // Medium orchid
];

/**
 * Renders colored overlay rectangles for leases on a Sparklines chart.
 * Each lease gets a distinct color. Overlays are hidden entirely if all leases are too narrow.
 */
const LeaseOverlayRects: React.FC<{
   processedLeases: ProcessedLease[];
   sparkHeight: number;
   sparkWidth: number;
   chartId: string;
}> = ({ processedLeases, sparkHeight, sparkWidth, chartId }) => {
   const colors = dashboard.darktheme ? leaseColorsDark : leaseColorsLight;
   const margin = 2; // Sparklines default margin
   const minWidthThreshold = 4; // Hide all overlays if every lease is narrower than this

   const sortedLeases = [...processedLeases].sort((a, b) => a.startX - b.startX);

   // Check if any lease is wide enough to show overlays
   const hasWideEnoughLease = sortedLeases.some(pl => {
      const endX = pl.endX ?? sparkWidth - margin;
      return (endX - pl.startX) >= minWidthThreshold;
   });

   if (!hasWideEnoughLease) {
      return null;
   }

   const elements: JSX.Element[] = [];

   sortedLeases.forEach((pl, idx) => {
      const x = pl.startX;
      const endX = pl.endX ?? sparkWidth - margin;
      const width = endX - x;

      if (width < 1) {
         return;
      }

      const color = colors[processedLeases.indexOf(pl) % colors.length];

      elements.push(
         <rect key={`${chartId}-lease-${idx}`} x={x} y={0} width={width} height={sparkHeight} fill={color} />
      );
   });

   return <>{elements}</>;
};

const TelemetryPanel: React.FC<{}> = observer(({ }) => {

   const [timeKey, setTimeKey] = useState("time_2_days");
   const [cpuKey, setCPUKey] = useState("combined");

   useEffect(() => {

      const time = timeSelections.find(t => t.key === timeKey);
      if (time) {
         state.UpdateTelemetry(time.minutes);
      }

      return () => {
      };

   }, []);


   // subscribe
   if (state.currentData) { }

   const currentData = (state.currentData ?? []);
   const isLoading = !currentData.length;

   const statusColors = dashboard.getStatusColors();

   const sparkWidth = 1100;
   const sparkHeight = 128;
   const data = isLoading ? [] : (currentData[0] as GetAgentTelemetrySampleResponse[]);
   const leases = (!isLoading && currentData.length > 1) ? currentData[1] as GetAgentLeaseResponse[] : [];

   const hasData = data.length > 0;

   const processedData = hasData ? processLeaseData(data, leases, sparkWidth) : null;
   const processedLeases = processedData?.processedLeases ?? [];

   const cpuData = hasData ? data.map(d => {
      if (cpuKey == "user") return d.userCpu;
      if (cpuKey == "system") return d.systemCpu;
      if (cpuKey == "combined") return d.userCpu + d.systemCpu;
      return 1;
   }) : [];

   const min = hasData ? data[0].time : "";
   const max = hasData ? data[data.length - 1].time : "";

   const minTime = hasData ? getShortNiceTime(min, true) : "";
   const maxTime = hasData ? getShortNiceTime(max, true) : "";

   const calculateDiskUtilization = (
      data: GetAgentTelemetrySampleResponse[], 
      totalKey: keyof GetAgentTelemetrySampleResponse, 
      freeKey: keyof GetAgentTelemetrySampleResponse
   ) => 
      data.map(d => {
         const total = d[totalKey];
         const free = d[freeKey];

         if (typeof total == "number" && typeof free == "number" && total > 0) {
            return 1.0 - (free / total);
         }
   
         return 0;
      })

   const diskData = calculateDiskUtilization(data, 'workingTotalDisk', 'workingFreeDisk');
   const systemDiskData = calculateDiskUtilization(data, 'systemTotalDisk', 'systemFreeDisk');

   let cpuText = "";
   let ramText = "";
   let diskText = "";
   let systemDiskText = "";

   let cpuLabel = "";
   if (cpuKey == "user") cpuLabel = "CPU - User";
   if (cpuKey == "system") cpuLabel = "CPU - System";
   if (cpuKey == "combined") cpuLabel = "CPU - Combined";

   const formatDiskSize = (bytes: number, decimals = 0) => {
      if (!+bytes) return '0 Bytes'

      const k = 1024
      const dm = decimals < 0 ? 0 : decimals
      const sizes = ['Bytes', 'KB', 'MB', 'GB', 'TB', 'PB', 'EB', 'ZB', 'YB']

      const i = Math.floor(Math.log(bytes) / Math.log(k))

      return `${parseFloat((bytes / Math.pow(k, i)).toFixed(dm))} ${sizes[i]}`
   }

   const devices = state.selectedAgent?.capabilities?.devices;
   if (devices?.length) {
      const device = devices[0];
      device.properties?.forEach(v => {
         if (v.startsWith("CPU=")) {
            cpuText = `${v.replace("CPU=", "")}`;
         }

         if (v.startsWith("RAM=")) {
            ramText = `${v.replace("RAM=", "")} GB`;
         }

         if (v.startsWith("WorkingDiskTotalSize=")) {
            const elements = v.split("=");
            if (elements.length === 2) {
               diskText = formatDiskSize(parseInt(elements[1]))
            }
         }

         if (v.startsWith("SystemDiskTotalSize=")) {
            const elements = v.split("=");
            if (elements.length === 2) {
               systemDiskText = formatDiskSize(parseInt(elements[1]))
            }
         }

      })
   }

   const ramData = data.map(d => {
      return 1.0 - (d.freeRam / d.totalRam);
   })


   // Padding value used in Sparklines style - must match for coordinate calculations
   const sparkPadding = 8;
   const sparkMargin = 2; // Sparklines internal margin

   function relativeCoords(event: any, id: string) {

      if (state.currentLeaseTooltip?.frozen || !data?.length) {
         return;
      }

      // Use currentTarget to always get the event handler element, not whatever child is under the cursor
      let bounds = event.currentTarget.getBoundingClientRect();
      let x = event.clientX - bounds.left;
      let y = event.clientY - bounds.top;

      // Adjust for padding and border to get position within the actual chart area
      // Structure: handler div > tooltip div > styled container (border 1px, padding 8px) > inner div > SVGs
      const chartX = x - sparkPadding - 1; // -1 for border, now in SVG coordinates (0-1100)

      // Sparklines draws data from x=sparkMargin to x=(sparkWidth-sparkMargin)
      // Must account for the margin to correctly map to sample indices
      const drawWidth = sparkWidth - 2 * sparkMargin;
      const dataX = chartX - sparkMargin; // Position within drawable area
      const percent = dataX / drawWidth;

      // Clamp and calculate sample index
      // Use (data.length - 1) to match how processLeaseData calculates positions
      const clampedPercent = Math.max(0, Math.min(1, percent));
      const sampleIdx = Math.min(Math.round(clampedPercent * (data.length - 1)), data.length - 1);

      if (sampleIdx < 0 || sampleIdx >= data.length) {
         return;
      }

      const sample = data[sampleIdx];
      const time = sample.time;

      // Find lease by visual X position (using processedLeases which have padding applied)
      // This ensures the tooltip matches what the user sees visually
      // Sort by startX to match drawing order - leases drawn later are on top
      // When leases overlap, use the LAST matching one (drawn on top, visually visible)
      const sortedLeases = [...processedLeases].sort((a, b) => a.startX - b.startX);
      let lease: GetAgentLeaseResponse | undefined;
      for (const pl of sortedLeases) {
         const leaseEndX = pl.endX ?? sparkWidth; // ongoing leases extend to end
         if (chartX >= pl.startX && chartX <= leaseEndX) {
            lease = pl.lease; // Keep updating to get the last (topmost) match
         }
      }

      if (lease?.details?.type === "Job") {
         const jobId = lease.details.jobId;
         if (jobId) {
            if (!state.jobData.has(jobId)) {
               state.jobData.set(jobId, true);
               backend.getJob(jobId, { filter: "id,batches" }).then(job => {
                  state.jobData.set(jobId, job);
                  state.setLeaseTooltip(state.currentLeaseTooltip ? { ...state.currentLeaseTooltip } : undefined);
               })
            }
         }
      }

      state.setLeaseTooltip({ lease: lease, x: x, y: y, id: id, time: time, sample: sample });
   };

   return <Stack tokens={{ childrenGap: 24 }} style={{ paddingBottom: 32, paddingLeft: 8 }}>
      <Stack horizontal verticalAlign="center" verticalFill>
         <Text variant="mediumPlus">{hasData ? `${minTime} - ${maxTime}` : "Select a time range"}</Text>
         <Stack grow />
         <Stack horizontal tokens={{ childrenGap: 18 }}>

            <ComboBox
               label={"CPU"}
               styles={{ root: { width: 128 } }}
               options={cpuSelections}
               selectedKey={cpuKey}
               onChange={(ev, option, index, value) => {
                  if (!option) {
                     return;
                  }
                  setCPUKey(option.key as string);
               }}
            />

            <ComboBox
               label="Time"
               styles={{ root: { width: 180 } }}
               options={timeSelections}
               selectedKey={timeKey}
               onChange={(ev, option, index, value) => {
                  const select = option as TimeSelection;
                  setTimeKey(select.key);
                  state.UpdateTelemetry(select.minutes);
               }}
            />
         </Stack>
      </Stack>

      {/* Show loading spinner when data is being fetched */}
      {isLoading && <Stack horizontalAlign="center" style={{ paddingTop: 48 }}>
         <Spinner size={SpinnerSize.large} label="Loading telemetry data..." />
      </Stack>}

      {/* Show message when no data is available for the selected time range */}
      {!isLoading && !hasData && <Stack horizontalAlign="center" style={{ paddingTop: 48 }}>
         <Text variant="large">No telemetry data available for this time range.</Text>
         <Text variant="medium" style={{ paddingTop: 8 }}>Try selecting a different time range above.</Text>
      </Stack>}

      {/* Render charts only when data is available */}
      {hasData && <>
      <Stack verticalAlign="center" verticalFill>
         <Stack>
            <Label>{cpuLabel} Utilization {cpuText ? `- ${cpuText}` : ""}</Label>
         </Stack>
         <div onMouseLeave={() => state.setLeaseTooltip(undefined)} onMouseMove={(ev) => { relativeCoords(ev, "cpu") }} onClick={() => {

            if (!state.currentLeaseTooltip?.frozen) {
               if (state.currentLeaseTooltip) {
                  state.setLeaseTooltip({ ...state.currentLeaseTooltip, frozen: true });
               }
            }
         }}>
            <div style={{ position: "relative" }}>
               <TelemetryTooltip id={"cpu"} cpuKey={cpuKey} />
            </div>

            <div style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
               <div style={{ position: "relative" }}>
                  <Sparklines width={sparkWidth} height={sparkHeight} svgWidth={sparkWidth} svgHeight={sparkHeight} data={cpuData} min={0} max={100}>
                     <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} />
                     <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                     <SparklinesReferenceLine type="custom" value={32} style={{ stroke: statusColors.get(StatusColor.Warnings) }} />
                     <SparklinesReferenceLine type="custom" value={64} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                     <SparklinesReferenceLine type="custom" value={96} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                  </Sparklines>
                  <svg style={{ position: "absolute", top: 0, left: 0, pointerEvents: "none" }} width={sparkWidth} height={sparkHeight} viewBox={`0 0 ${sparkWidth} ${sparkHeight}`} preserveAspectRatio="none">
                     <LeaseOverlayRects processedLeases={processedLeases} sparkHeight={sparkHeight} sparkWidth={sparkWidth} chartId="cpu" />
                  </svg>
               </div>
            </div>
         </div>
      </Stack>
      <Stack verticalAlign="center" verticalFill>
         <Stack>
            <Label>RAM Utilization {ramText ? `- ${ramText}` : ""}</Label>
         </Stack>
         <div onMouseLeave={() => state.setLeaseTooltip(undefined)} onMouseMove={(ev) => { relativeCoords(ev, "ram") }} onClick={() => {

            if (!state.currentLeaseTooltip?.frozen) {
               if (state.currentLeaseTooltip) {
                  state.setLeaseTooltip({ ...state.currentLeaseTooltip, frozen: true });
               }
            }
         }}>
            <div style={{ position: "relative" }}>
               <TelemetryTooltip id="ram" />
            </div>
            <div style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
               <div style={{ position: "relative" }}>
                  <Sparklines width={sparkWidth} height={sparkHeight} svgWidth={sparkWidth} svgHeight={sparkHeight} data={ramData} min={0} max={1}>
                     <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} />
                     <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                     <SparklinesReferenceLine type="custom" value={32} style={{ stroke: statusColors.get(StatusColor.Warnings) }} />
                     <SparklinesReferenceLine type="custom" value={64} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                     <SparklinesReferenceLine type="custom" value={96} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                  </Sparklines>
                  <svg style={{ position: "absolute", top: 0, left: 0, pointerEvents: "none" }} width={sparkWidth} height={sparkHeight} viewBox={`0 0 ${sparkWidth} ${sparkHeight}`} preserveAspectRatio="none">
                     <LeaseOverlayRects processedLeases={processedLeases} sparkHeight={sparkHeight} sparkWidth={sparkWidth} chartId="ram" />
                  </svg>
               </div>
            </div>
         </div>
      </Stack>

      {!!diskText && <Stack verticalAlign="center" verticalFill>
         <Stack>
            <Label>Agent Working Drive Utilization - {diskText}</Label>
         </Stack>
         <div onMouseLeave={() => state.setLeaseTooltip(undefined)} onMouseMove={(ev) => { relativeCoords(ev, "workingDisk") }} onClick={() => {

            if (!state.currentLeaseTooltip?.frozen) {
               if (state.currentLeaseTooltip) {
                  state.setLeaseTooltip({ ...state.currentLeaseTooltip, frozen: true });
               }
            }
         }}>
            <div style={{ position: "relative" }}>
               <TelemetryTooltip id="workingDisk" />
            </div>
            <div style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
               <div style={{ position: "relative" }}>
                  <Sparklines width={sparkWidth} height={sparkHeight} svgWidth={sparkWidth} svgHeight={sparkHeight} data={diskData} min={0} max={1}>
                     <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} />
                     <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                     <SparklinesReferenceLine type="custom" value={32} style={{ stroke: statusColors.get(StatusColor.Warnings) }} />
                     <SparklinesReferenceLine type="custom" value={64} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                     <SparklinesReferenceLine type="custom" value={96} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                  </Sparklines>
                  <svg style={{ position: "absolute", top: 0, left: 0, pointerEvents: "none" }} width={sparkWidth} height={sparkHeight} viewBox={`0 0 ${sparkWidth} ${sparkHeight}`} preserveAspectRatio="none">
                     <LeaseOverlayRects processedLeases={processedLeases} sparkHeight={sparkHeight} sparkWidth={sparkWidth} chartId="workingDisk" />
                  </svg>
               </div>
            </div>
         </div>
      </Stack>}

      {!!systemDiskText && <Stack verticalAlign="center" verticalFill>
         <Stack>
            <Label>System Drive Utilization - {systemDiskText}</Label>
         </Stack>
         <div onMouseLeave={() => state.setLeaseTooltip(undefined)} onMouseMove={(ev) => { relativeCoords(ev, "systemDisk") }} onClick={() => {

            if (!state.currentLeaseTooltip?.frozen) {
               if (state.currentLeaseTooltip) {
                  state.setLeaseTooltip({ ...state.currentLeaseTooltip, frozen: true });
               }
            }
         }}>
            <div style={{ position: "relative" }}>
               <TelemetryTooltip id="systemDisk" />
            </div>
            <div style={{ backgroundColor: dashboard.darktheme ? "#060709" : "#F3F2F1", padding: 8, border: "solid 1px #181A1B" }}>
               <div style={{ position: "relative" }}>
                  <Sparklines width={sparkWidth} height={sparkHeight} svgWidth={sparkWidth} svgHeight={sparkHeight} data={systemDiskData} min={0} max={1}>
                     <SparklinesLine color={dashboard.darktheme ? "lightblue" : "#1E90FF"} />
                     <SparklinesReferenceLine type="custom" value={0} style={{ stroke: statusColors.get(StatusColor.Failure) }} />
                     <SparklinesReferenceLine type="custom" value={32} style={{ stroke: statusColors.get(StatusColor.Warnings) }} />
                     <SparklinesReferenceLine type="custom" value={64} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                     <SparklinesReferenceLine type="custom" value={96} style={{ stroke: statusColors.get(StatusColor.Success) }} />
                  </Sparklines>
                  <svg style={{ position: "absolute", top: 0, left: 0, pointerEvents: "none" }} width={sparkWidth} height={sparkHeight} viewBox={`0 0 ${sparkWidth} ${sparkHeight}`} preserveAspectRatio="none">
                     <LeaseOverlayRects processedLeases={processedLeases} sparkHeight={sparkHeight} sparkWidth={sparkWidth} chartId="systemDisk" />
                  </svg>
               </div>
            </div>
         </div>
      </Stack>}
      </>}

   </Stack>
})

export const HistoryModal: React.FC<{ agentId: string | undefined, onDismiss: (...args: any[]) => any; }> = observer(({ agentId, onDismiss }) => {

   const [selectedAgent, setSelectedAgent] = useState<string | undefined>(undefined);
   const [actionState, setActionState] = useState<{ action?: string, confirmed?: boolean, comment?: string }>({});
   const actionTextInputRef = React.useRef<ITextField>(null);
   const forceRestartCheckboxRef = React.useRef<ICheckbox>(null);
   const [agentError, setAgentError] = useState(false);

   const { hordeClasses, modeColors } = getHordeStyling();
   const theme = getHordeTheme();

   if (agentError) {
      return <Dialog hidden={false} onDismiss={onDismiss} dialogContentProps={{
         type: DialogType.normal,
         title: 'Missing Agent',
         subText: `Unable to find agent ${agentId}`
      }}
         modalProps={{ styles: { main: { width: "640px !important", minWidth: "640px !important", maxWidth: "640px !important" } } }}>
         <DialogFooter>
            <PrimaryButton onClick={() => onDismiss()} text="Ok" />
         </DialogFooter>
      </Dialog>
   }

   //  subscribe to updates
   if (state.selectedAgent) { }

   if (!agentId) {
      if (selectedAgent) {
         setSelectedAgent(undefined);
      }
      return null;
   }

   if (selectedAgent !== agentId) {

      agentStore.updateAgent(agentId).then(agent => {

         state.setSelectedAgent(agent);

      }).catch(error => {
         console.error(`Unable to find agent id: ${agentId} ${error}`);
         setAgentError(true);
         return;
      })

      setSelectedAgent(agentId);
   }

   if (!state.selectedAgent) {
      if (agentId) {
         return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 16, width: 1200, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
            <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
               <Stack grow verticalAlign="center">
                  <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold", fontSize: "20px" } }}>Agent {agentId}</Text>
               </Stack>
               <Stack verticalAlign="center">
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Modal>
      }

      return null;
   }

   type AgentAction = {
      name: string;
      confirmText: string;
      textInput?: boolean;
      update?: (request: UpdateAgentRequest, comment?: string) => void;
   }
   const actions: AgentAction[] = [
      {
         name: 'Enable',
         confirmText: "Are you sure you would like to enable this agent?",
         update: (request) => { request.enabled = true }
      },
      {
         name: 'Disable',
         confirmText: "Are you sure you would like to disable this agent?",
         textInput: true,
         update: (request, comment) => { request.enabled = false; request.comment = comment }
      },
      {
         name: 'Cancel Leases',
         confirmText: "Are you sure you would like to cancel this agent's leases?"
      },
      {
         name: 'Request Conform',
         confirmText: "Are you sure you would like to request an agent conform?",
         update: (request) => { request.requestConform = true }
      },
      {
         name: 'Request Full Conform',
         confirmText: "Are you sure you would like to request a full agent conform?",
         update: (request) => { request.requestFullConform = true }

      },
      {
         name: 'Request Restart',
         confirmText: "Are you sure you would like to request an agent restart?",
         update: (request) => { request.requestRestart = !forceRestartCheckboxRef.current?.checked; request.requestForceRestart = !!forceRestartCheckboxRef.current?.checked; }
      },
      {
         name: 'Edit Comment',
         confirmText: "Please enter new comment",
         textInput: true,
         update: (request, comment) => { request.comment = comment }
      }
   ];

   // cancel leases action
   const cancelLeases = async () => {

      const leases = new Set<string>();

      state.selectedAgent?.leases?.forEach(lease => {
         if (!lease.finishTime) {
            leases.add(lease.id);
         }
      });

      let requests = Array.from(leases).map(id => {
         return backend.updateLease(id, { aborted: true });
      });

      while (requests.length) {

         const batch = requests.slice(0, 5);

         await Promise.all(batch).then(() => {

         }).catch((errors) => {
            console.log(errors);
            // eslint-disable-next-line
         }).finally(() => {

            requests = requests.slice(5);
         });
      }
      agentStore.update();
   };


   const currentAction = actions.find(a => a.name === actionState.action);

   if (currentAction && actionState.confirmed) {
      if (currentAction.update) {
         const request: UpdateAgentRequest = {};
         currentAction.update(request, actionState.comment);
         backend.updateAgent(agentId, request).then(() => {
            if (!agentStore.pools?.length) {
               agentStore.update(false).then(() => {
                  state.setSelectedAgent(agentStore.agents.find(agent => agent.id === agentId));
                  setActionState({});
               });
            } else {
               agentStore.updateAgent(agentId).then(() => {
                  state.setSelectedAgent(agentStore.agents.find(agent => agent.id === agentId));
                  setActionState({});
               });
            }
         }).catch((reason) => {
            console.error(reason);
         });
      } else if (currentAction.name === "Cancel Leases") {
         (async () => {
            await cancelLeases();
            setActionState({});
         })();
      }
   }

   const actionMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      items: actions.map(a => {
         return {
            key: a.name.toLowerCase(),
            text: a.name,
            onClick: () => {
               setActionState({ action: a.name })
            }
         }
      })
   };

   const poolItems: IContextualMenuItem[] = [];;
   const agent = state.selectedAgent;
   agent.pools?.forEach(poolId => {

      const pool = agentStore.pools?.find(pool => poolId === pool.id);
      if (pool) {
         poolItems.push({
            key: `pools_${poolId}`,
            text: pool.name,
            href: `/pools?pool=${pool.id}`,
            target: "_blank"
         })
      }
   })

   function onColumnClick(ev: React.MouseEvent<HTMLElement>, column: IColumn) {
      //historyModalState.setSorted(column.key);
   }

   function generateColumns() {
      let columns: IColumn[] = [];
      if (state.mode === "leases") {
         columns = [
            {
               key: 'type',
               name: 'Type',
               minWidth: 125,
               maxWidth: 125,
               isResizable: false,
               isSorted: false,
               isSortedDescending: false,
               onColumnClick: onColumnClick
            },
            {
               key: 'id',
               name: 'ID',
               minWidth: 185,
               maxWidth: 185,
               isResizable: false,
               isSorted: false,
               isSortedDescending: false,
               onColumnClick: onColumnClick
            },
            {
               key: 'description',
               name: 'Description',
               minWidth: 350,
               isResizable: false,
               isSorted: false,
               isSortedDescending: false,
               onColumnClick: onColumnClick
            }
         ];
      }
      else if (state.mode === "sessions") {
         columns = [
            {
               key: 'id',
               name: 'ID',
               minWidth: 692,
               maxWidth: 692,
               isResizable: false,
               isSorted: false,
               isSortedDescending: false,
               onColumnClick: onColumnClick
            }
         ];
      }

      if (state.mode === "leases" || state.mode === "sessions") {


         columns.push({
            key: 'startTime',
            name: 'Start Time',
            minWidth: 200,
            maxWidth: 200,
            isResizable: false,
            isSorted: false,
            isSortedDescending: false,
            onColumnClick: onColumnClick
         });
         columns.push({
            key: 'endTime',
            name: 'Finish Time',
            minWidth: 200,
            maxWidth: 200,
            isResizable: false,
            isSorted: false,
            isSortedDescending: false,
            onColumnClick: onColumnClick
         });
      }

      if (state.mode === "leases") {
         columns.find(col => col.key === state.sortedLeaseColumn)!.isSorted = true;
         columns.find(col => col.key === state.sortedLeaseColumn)!.isSortedDescending = state.sortedLeaseColumnDescending;
      }
      else if (state.mode === "sessions") {
         columns.find(col => col.key === state.sortedSessionColumn)!.isSorted = true;
         columns.find(col => col.key === state.sortedSessionColumn)!.isSortedDescending = state.sortedSessionColumnDescending;
      }

      return columns;
   }

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      const customStyles: Partial<IDetailsHeaderStyles> = {};
      customStyles.root = { paddingTop: 0 };
      if (props) {
         return (
            <Sticky stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
               <DetailsHeader {...props} styles={customStyles} onRenderColumnHeaderTooltip={onRenderColumnHeaderTooltip} />
            </Sticky>
         );
      }
      return null;
   };

   const onRenderColumnHeaderTooltip: IDetailsHeaderProps['onRenderColumnHeaderTooltip'] = (props: any) => {
      const customStyles: Partial<ITooltipHostStyles> = {};
      if (props) {
         customStyles.root = { selectors: { "> span": { paddingLeft: '8px !important', display: 'flex', justifyContent: 'center' } } };

         /*
          // no other way to filter children being centered other than to drill into private members??
          if (props.children._owner.child?.child.child.child.elementType === "span") {
              const data = props.children._owner.child.child.child.child.child.child.stateNode.data;
              // if this cluster happens to be true, reset back to the default, because doing this the other way around
              // takes too long when the columns update on details switch
              // ugh :(
              if (data === "ID" || data === "Description") {
                  customStyles.root = {};
              }
          }
          */
         return <Text styles={customStyles}>{props.children}</Text>;
      }
      return null;
   };

   function onRenderHistoryItem(item: LeaseData | SessionData, index?: number, column?: IColumn) {

      let link = "";

      if (state.mode === "leases") {
         const lease = item as LeaseData;
         switch (column!.key) {
            case 'type':

               if (lease.batch) {

                  const step = lease.batch.steps.find(s => s.outcome === JobStepOutcome.Failure);

                  let name = lease.type;

                  if (lease.batch.error !== JobStepBatchError.None) {
                     name = lease.batch.error;
                  } else if (step) {
                     name = "StepError";
                  }

                  return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'start'} verticalAlign="center" tokens={{ childrenGap: 4 }}>
                     <Stack >
                        {!step && <BatchStatusIcon batch={lease.batch} />}
                        {!!step && <StepStatusIcon step={step} />}
                     </Stack>
                     <Stack >{name}</Stack>
                  </Stack>
               } else if (lease.outcome) {
                  return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'start'} verticalAlign="center" tokens={{ childrenGap: 4 }}>
                     <Stack >
                        <LeaseStatusIcon lease={lease} />
                     </Stack>
                     <Stack >{lease.type}</Stack>
                  </Stack>
               }
               return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{lease.type}</Stack.Item></Stack>
            case 'id':
               link = "";
               if (lease.details && 'LogId' in lease.details) {
                  link = '/log/' + lease.details['LogId'];
               }
               else if (lease.logId) {
                  link = `/log/${lease.logId}`;
               }
               if (link) {
                  return <Stack style={{ height: "100%" }} verticalAlign="center"><Link to={link}>{lease.id}</Link></Stack>
               }
               return <Stack style={{ height: "100%" }} verticalAlign="center">{lease.id}</Stack>;
            case 'name':
               return <Stack>{lease.name}</Stack>;
            case 'startTime':
               return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{getShortNiceTime(lease.startTime, false, true, true)}</Stack.Item></Stack>
            case 'endTime':
               return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{getShortNiceTime(lease.finishTime, false, true, true)}</Stack.Item></Stack>
            case 'description':
               link = "";
               let name = lease.name;
               if (lease.details) {
                  if ('jobId' in lease.details) {
                     link = `/job/${lease.details['jobId']}`;
                     if ('batchId' in lease.details) {
                        link += `?batch=${lease.details['batchId']}`
                     }
                  }
               }
               else if (lease.jobId) {
                  link = `/job/${lease.jobId}`;
               }

               if (!link && (lease.parentId && lease.details && lease.details["parentLogId"])) {
                  link = `/log/${lease.details["parentLogId"]}`
                  return <Stack styles={{ root: { height: '100%' } }} horizontal><Stack.Item align={"center"}><Text style={{ fontSize: "12px" }}>{name} (parent: </Text><Link style={{ fontSize: "12px" }} to={link}>{lease.parentId}</Link><Text style={{ fontSize: "12px" }}>)</Text></Stack.Item></Stack>;
               }

               if (link !== "") {
                  return <Stack styles={{ root: { height: '100%' } }} horizontal><Stack.Item align={"center"}><Link style={{ fontSize: 12 }} key={"leaseText_" + lease.id} to={link}>{name}</Link></Stack.Item></Stack>;
               }
               else {
                  return <Stack styles={{ root: { height: '100%', } }} horizontal><Stack.Item align={"center"}><Text styles={{ root: { fontSize: 12 } }} key={"leaseText_" + lease.id}>{name}</Text></Stack.Item></Stack>;
               }
            default:
               return <span>{lease[column!.fieldName as keyof LeaseData] as string}</span>;
         }
      }
      else if (state.mode === "sessions") {
         const session = item as SessionData;
         switch (column!.key) {
            case 'id':
               return <Stack styles={{ root: { height: '100%', } }} horizontal><Stack.Item align={"center"}>{session.id}</Stack.Item></Stack>
            case 'startTime':
               return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{getShortNiceTime(session.startTime, false, true)}</Stack.Item></Stack>
            case 'endTime':
               return <Stack styles={{ root: { height: '100%', } }} horizontal horizontalAlign={'center'}><Stack.Item align={"center"}>{getShortNiceTime(session.finishTime, false, true)}</Stack.Item></Stack>
            default:
               return <span>{session[column!.fieldName as keyof SessionData] as string}</span>;
         }
      }
   }

   const onRenderInfoDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      const customStyles: Partial<IDetailsHeaderStyles> = {};
      customStyles.root = { paddingTop: 0 };
      if (props) {
         return (
            <Sticky stickyPosition={StickyPositionType.Header} isScrollSynced={true}>
               <DetailsHeader {...props} styles={customStyles} />
            </Sticky>
         );
      }
      return null;
   };

   const c1 = theme.horde.darkTheme ? theme.horde.neutralBackground : "#f3f2f1";
   const c2 = theme.horde.darkTheme ? theme.horde.contentBackground : "#ffffff";

   function onRenderBuilderInfoCell(nestingDepth?: number | undefined, item?: any, index?: number | undefined) {
      return (
         <Stack horizontal onClick={(ev) => { state.setInfoItemSelected(item); ev.preventDefault(); }} styles={{
            root: {
               background: item!.selected ? c1 : c2,
               paddingLeft: 48 + (10 * nestingDepth!),
               paddingTop: 8,
               paddingBottom: 8,
               selectors: {
                  ":hover": {
                     background: c1,
                     cursor: 'pointer'
                  }
               }
            }
         }}>
            <Stack>
               <Link to="" onClick={(ev) => { state.setInfoItemSelected(item); ev.preventDefault(); }}><Text>{item!.name}</Text></Link>
            </Stack>
         </Stack>
      );
   }

   const groups: IGroup[] = [
      {
         count: state.agentItemCount,
         key: "agent",
         name: "Agent",
         startIndex: 0,
         level: 0,
         isCollapsed: false,
      },
      {
         count: state.devicesItemCount,
         key: "devices",
         name: "Devices",
         startIndex: state.agentItemCount,
         level: 0,
         isCollapsed: false,
      },
      {
         count: state.workspaceItemCount,
         key: "workspaces",
         name: "Workspaces",
         startIndex: state.agentItemCount + state.devicesItemCount,
         level: 0,
         isCollapsed: false,
      },

   ];

   const onDismissHistoryModal = function (ev?: React.MouseEvent<HTMLButtonElement, MouseEvent> | undefined) {
      state.setSelectedAgent(undefined);
      onDismiss();
   };

   return (
      <Dialog
         modalProps={{
            isBlocking: false,
            topOffsetFixed: true,
            styles: {
               main: {
                  hasBeenOpened: false,
                  top: "80px",
                  position: "absolute"
               },
               root: {
                  selectors: {
                     ".ms-Dialog-title": {
                        paddingTop: '24px',
                        paddingLeft: '32px'
                     }
                  }
               }
            }
         }}
         onDismiss={onDismissHistoryModal}
         className={historyStyles.dialog}
         hidden={!state.selectedAgent}
         minWidth={1200}
         dialogContentProps={{
            type: DialogType.close,
            onDismiss: onDismissHistoryModal,
            title: `Agent ${state.selectedAgent?.name}`,
         }}
      >
         <Stack>
            {!!currentAction && <Dialog
               hidden={false}
               onDismiss={() => { setActionState({}) }}
               minWidth={512}
               dialogContentProps={{
                  type: DialogType.normal,
                  title: `Confirm Action`,
               }}
               modalProps={{ isBlocking: true }} >
               <Stack style={{ paddingBottom: 18, paddingLeft: 4 }}>
                  <Text>{currentAction.confirmText}</Text>
               </Stack>
               {!!currentAction.textInput && <TextField componentRef={actionTextInputRef} label={currentAction.name === "Edit Comment" ? "New Comment" : "Disable Reason - Required"} onChange={(ev, value) => setActionState({ ...actionState, comment: value })} spellCheck={false} autoComplete="false" />}
               {currentAction.name === "Request Restart" && <Checkbox componentRef={forceRestartCheckboxRef} label={"Force Restart"} />}
               <DialogFooter>
                  <PrimaryButton disabled={actionState.confirmed || (!!currentAction.textInput && currentAction.name !== "Edit Comment" && !actionState.comment?.length)} onClick={() => { setActionState({ ...actionState, confirmed: true, comment: actionTextInputRef.current?.value }) }} text={currentAction.name} />
                  <DefaultButton disabled={actionState.confirmed} onClick={() => { setActionState({}) }} text="Cancel" />
               </DialogFooter>
            </Dialog>
            }
            <Stack horizontal styles={{ root: { paddingLeft: 2 } }}>
               <Stack grow>
                  <Stack horizontalAlign={"start"}>
                     <Pivot className={hordeClasses.pivot}
                        onLinkClick={(item?: PivotItem | undefined) => { state.setMode(item?.props.itemKey); }}
                        linkSize="normal"
                        linkFormat="links"
                        defaultSelectedKey={state.mode ?? "info"}
                     >
                        <PivotItem headerText="Info" itemKey="info" />
                        <PivotItem headerText="Sessions" itemKey="sessions" />
                        <PivotItem headerText="Leases" itemKey="leases" />
                        <PivotItem headerText="Telemetry" itemKey="telemetry" />
                     </Pivot>
                  </Stack>
               </Stack>
               {!!poolItems.length && <Stack style={{ paddingRight: 24 }}><DefaultButton text="Pools" menuProps={{ shouldFocusOnMount: true, items: poolItems }} /></Stack>}
               {!!dashboard.user?.dashboardFeatures?.showRemoteDesktop && <Stack> <Stack horizontal tokens={{ childrenGap: 24 }}>
                  <Stack>
                     <DefaultButton text="Actions" menuProps={actionMenuProps} />
                  </Stack>
                  <Stack>
                     <Link to={`/audit/agent/${encodeURIComponent(agentId)}`} ><DefaultButton text="Audit" onClick={(ev) => { }} /></Link>
                  </Stack>

                  <Stack>
                     <DefaultButton text="Remote Desktop" onClick={() => {
                        if (state.selectedAgent?.id) {
                           // @todo: this assumes the id is the ip, should have a private ip property on agents instead
                           let ip = state.selectedAgent?.id;
                           if (ip.startsWith("10-")) {
                              ip = ip.replaceAll("-", ".")
                           }
                           window.open(`ugs://rdp?host=${ip}`, "_self")
                        }
                     }} />
                  </Stack>
               </Stack>
               </Stack>}
            </Stack>
            <Stack styles={{ root: { paddingTop: 30 } }}>
               {state.mode === "info" && <Stack horizontal tokens={{ childrenGap: 20 }}>
                  <Stack styles={{ root: { width: 300 } }}>
                     <GroupedList
                        items={state.infoItems}
                        compact={true}
                        onRenderCell={onRenderBuilderInfoCell}
                        groups={groups}
                        selection={new Selection()}
                        selectionMode={SelectionMode.none}
                        groupProps={{
                           showEmptyGroups: true,
                           onRenderHeader: (props) => {
                              return <Link to="" onClick={(ev) => { ev.preventDefault(); props!.onToggleCollapse!(props!.group!); }}> <GroupHeader {...props} /></Link>;
                           },
                           headerProps: {
                              styles: {
                                 title: {
                                    fontFamily: "Horde Open Sans Semibold",
                                    color: modeColors.text,
                                    paddingLeft: 0

                                 },
                                 headerCount: {
                                    display: 'none'
                                 }
                              }
                           }
                        }}
                     />
                  </Stack>
                  <Stack styles={{ root: { width: '100%' } }}>
                     <Stack.Item className={hordeClasses.relativeModalSmall}>
                        <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto} styles={{ contentContainer: { overflowX: 'hidden' } }}>
                           <DetailsList
                              className={historyStyles.detailsList}
                              compact={true}
                              items={state.infoSubItems}
                              columns={[
                                 { key: 'column1', name: 'Name', fieldName: 'name', minWidth: 200, maxWidth: 200, isResizable: false },
                                 { key: 'column2', name: 'Value', fieldName: 'value', minWidth: 200, isResizable: false }
                              ]}
                              layoutMode={DetailsListLayoutMode.justified}
                              onRenderDetailsHeader={onRenderInfoDetailsHeader}
                              constrainMode={ConstrainMode.unconstrained}
                              selectionMode={SelectionMode.none}
                              onRenderRow={(props) => {

                                 if (props) {
                                    return <DetailsRow styles={{ cell: { whiteSpace: "pre-line", overflowWrap: "break-word" } }} {...props} />
                                 }

                                 return null;
                              }}
                           />
                        </ScrollablePane>
                     </Stack.Item>
                  </Stack>
               </Stack>
               }
               {(state.mode === "leases" || state.mode === "sessions") &&
                  <Stack.Item className={hordeClasses.relativeModalSmall}>
                     <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto} styles={{ contentContainer: { overflowX: 'hidden' } }}>
                        <DetailsList
                           className={historyStyles.detailsList}
                           compact={true}
                           items={state.currentData}
                           columns={generateColumns()}
                           onRenderDetailsHeader={onRenderDetailsHeader}
                           onRenderItemColumn={onRenderHistoryItem}
                           layoutMode={DetailsListLayoutMode.justified}
                           constrainMode={ConstrainMode.unconstrained}
                           selectionMode={SelectionMode.none}
                           listProps={{ renderedWindowsAhead: 1, renderedWindowsBehind: 1 }}
                           onRenderMissingItem={() => { state.nullItemTrigger(); return <div></div> }}
                        />
                     </ScrollablePane>
                  </Stack.Item>}
               {state.mode === "telemetry" && <TelemetryPanel />}
            </Stack>
         </Stack>
         <DialogFooter>
            <PrimaryButton onClick={() => { state.setSelectedAgent(undefined); onDismiss() }}>Close</PrimaryButton>
         </DialogFooter>
      </Dialog>
   );
});