// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { AgentData, GetAgentLeaseResponse, GetAgentSessionResponse, GetAgentTelemetrySampleResponse, GetJobResponse, LeaseData, SessionData } from "../../backend/Api" 
import backend from "../../backend";

type LeaseTooltip = {
   lease?: GetAgentLeaseResponse;
   time?: Date;
   id?: string;
   x?: number;
   y?: number;
   frozen?: boolean;
   sample?: GetAgentTelemetrySampleResponse;
}

export type ProcessedLease = {
   lease: GetAgentLeaseResponse;
   startX: number;
   endX: number | null;
   startPercent: number;
   endPercent: number | null;
}

export type ProcessedTelemetryData = {
   samples: GetAgentTelemetrySampleResponse[];
   leases: GetAgentLeaseResponse[];
   processedLeases: ProcessedLease[];
   minTime: Date;
   maxTime: Date;
}

type InfoPanelItem = {
   key: string;
   name: string;
   data?: string;
   selected: boolean;
};

type InfoPanelSubItem = {
   name: string;
   value: string;
}

/**
 * Processes lease data to calculate X positions for overlay rendering on Sparklines charts.
 *
 * Sparklines renders data points by INDEX (evenly spaced), not by time. This function maps
 * lease time ranges to sample indices, then calculates pixel positions that align with
 * how Sparklines positions its data points.
 *
 * Leases are padded by one sample width on each side to fully enclose their samples.
 * Very short leases (those that fall between sample intervals) are filtered out.
 */
export function processLeaseData(
   samples: GetAgentTelemetrySampleResponse[],
   leases: GetAgentLeaseResponse[],
   sparkWidth: number
): ProcessedTelemetryData | null {
   if (!samples.length) {
      return null;
   }

   const minTime = samples[0].time;
   const maxTime = samples[samples.length - 1].time;
   const minTimeMs = new Date(minTime).getTime();
   const maxTimeMs = new Date(maxTime).getTime();

   if (maxTimeMs - minTimeMs <= 0) {
      return null;
   }

   // Sparklines uses margin=2, drawing points from x=2 to x=(sparkWidth-2)
   const sparkMargin = 2;
   const drawWidth = sparkWidth - 2 * sparkMargin;
   const numSamples = samples.length;
   const sampleWidth = drawWidth / (numSamples - 1);

   // Find first sample index on or after target time
   const findFirstSampleOnOrAfter = (targetMs: number): number => {
      if (targetMs <= minTimeMs) return 0;
      if (targetMs >= maxTimeMs) return numSamples - 1;
      for (let i = 0; i < numSamples; i++) {
         if (new Date(samples[i].time).getTime() >= targetMs) return i;
      }
      return numSamples - 1;
   };

   // Find last sample index on or before target time
   const findLastSampleOnOrBefore = (targetMs: number): number => {
      if (targetMs <= minTimeMs) return 0;
      if (targetMs >= maxTimeMs) return numSamples - 1;
      for (let i = numSamples - 1; i >= 0; i--) {
         if (new Date(samples[i].time).getTime() <= targetMs) return i;
      }
      return 0;
   };

   // Convert sample index to X position
   const indexToX = (idx: number): number => {
      return sparkMargin + (idx / (numSamples - 1)) * drawWidth;
   };

   // Sort leases by start time to properly detect adjacent leases
   const sortedLeases = [...leases].sort((a, b) =>
      new Date(a.startTime).getTime() - new Date(b.startTime).getTime()
   );

   // First pass: calculate raw positions for all leases
   type IntermediateLease = {
      lease: GetAgentLeaseResponse;
      rawStartX: number;
      rawEndX: number | null;
      startX: number;
      endX: number | null;
      startIdx: number;
      endIdx: number | null;
   };
   const intermediateLeases: IntermediateLease[] = [];

   for (const lease of sortedLeases) {
      const leaseStartMs = new Date(lease.startTime).getTime();
      const leaseEndMs = lease.finishTime ? new Date(lease.finishTime).getTime() : null;

      // Skip leases outside visible range
      if ((leaseEndMs !== null && leaseEndMs < minTimeMs) || leaseStartMs > maxTimeMs) {
         continue;
      }

      const startIdx = findFirstSampleOnOrAfter(leaseStartMs);
      const endIdx = leaseEndMs !== null ? findLastSampleOnOrBefore(leaseEndMs) : null;

      // Skip leases with no samples in their timeframe
      if (endIdx !== null && startIdx > endIdx) {
         continue;
      }

      // Calculate raw positions (without padding) and initial padded positions
      const rawStartX = indexToX(startIdx);
      const rawEndX = endIdx !== null ? indexToX(endIdx) : null;
      const startX = Math.max(sparkMargin, rawStartX - sampleWidth);
      const endX = rawEndX !== null ? Math.min(sparkMargin + drawWidth, rawEndX + sampleWidth) : null;

      // Skip leases with no visual width
      if (endX !== null && endX <= startX) {
         continue;
      }

      intermediateLeases.push({
         lease,
         rawStartX,
         rawEndX,
         startX,
         endX,
         startIdx,
         endIdx
      });
   }

   // Second pass: adjust boundaries for adjacent leases to prevent visual overlap
   // Adjacent leases are those where the current lease starts near where the previous ends
   const processedLeases: ProcessedLease[] = [];

   for (let i = 0; i < intermediateLeases.length; i++) {
      const current = intermediateLeases[i];
      const prev = i > 0 ? intermediateLeases[i - 1] : null;

      let adjustedStartX = current.startX;
      let adjustedEndX = current.endX;

      // Check if this lease is adjacent to the previous one (their padded regions overlap)
      if (prev && prev.endX !== null) {
         // Calculate where their raw boundaries meet
         const prevRawEnd = prev.rawEndX!;
         const currRawStart = current.rawStartX;

         // If the raw positions are close (within 2 sample widths), they're adjacent
         // This means their padded regions would overlap
         if (currRawStart - prevRawEnd < sampleWidth * 2) {
            // Calculate the midpoint between the two raw positions
            const midpoint = (prevRawEnd + currRawStart) / 2;

            // Adjust the previous lease's end to the midpoint
            if (processedLeases.length > 0) {
               processedLeases[processedLeases.length - 1].endX = midpoint;
            }

            // Adjust current lease's start to the midpoint
            adjustedStartX = midpoint;
         }
      }

      const processedLease: ProcessedLease = {
         lease: current.lease,
         startX: adjustedStartX,
         endX: adjustedEndX,
         startPercent: current.startIdx / (numSamples - 1),
         endPercent: current.endIdx !== null ? current.endIdx / (numSamples - 1) : null
      };

      processedLeases.push(processedLease);
   }

   return { samples, leases, processedLeases, minTime, maxTime };
}

class HistoryModalStore {
   constructor() {
      makeObservable(this);
   }

   @observable
   selectedAgentUpdated = 0;

   @observable
   currentLeaseTooltip?: LeaseTooltip;

   get selectedAgent(): AgentData | undefined {
      // subscribe in any observers
      if (this.selectedAgentUpdated) { }
      return this._selectedAgent;
   }

   private _selectedAgent: AgentData | undefined = undefined;
   @observable.shallow currentData: any = [];
   @observable.shallow infoItems: InfoPanelItem[] = [];
   @observable.shallow infoSubItems: InfoPanelSubItem[] = [];
   agentItemCount: number = 0;
   devicesItemCount: number = 0;
   workspaceItemCount: number = 0;
   sortedLeaseColumn = "startTime";
   sortedLeaseColumnDescending = true;
   sortedSessionColumn = "startTime";
   sortedSessionColumnDescending = true;

   @observable mode: string | undefined = undefined;
   modeCurrentIndex: number = 0;
   bUpdatedQueued: boolean = false;

   // sets the pool editor dialog open
   @action
   setSelectedAgent(selectedAgent?: AgentData | undefined) {
      // if we're closing don't reset
      this._selectedAgent = selectedAgent;
      if (this.selectedAgent) {
         // set date to now
         this._initBuilderItems();
         if (!this.mode) {
            this.setMode("info");
         }
         else {
            this.setMode(this.mode, true);
         }
         this._doUpdate();
      }

      this.selectedAgentUpdated++;
   }

   @action
   setMode(newMode?: string, force?: boolean) {
      if (newMode) {
         newMode = newMode.toLowerCase();
         if (this.mode !== newMode || force) {
            this.mode = newMode;
            this.currentData = [];
            this.modeCurrentIndex = 0;
            this._doUpdate();
         }
      }
   }

   @action
   setLeaseTooltip(tooltip?: LeaseTooltip) {
      this.currentLeaseTooltip = tooltip;
   }

   setInfoItemSelected(item: InfoPanelItem) {
      this.infoItems.forEach(infoItem => {
         if (infoItem.key === item.key) {
            infoItem.selected = true;
         }
         else {
            infoItem.selected = false;
         }
      });
      this.setInfoItems(this.infoItems);
   }

   private _initBuilderItems() {
      this.agentItemCount = 0;
      this.devicesItemCount = 0;
      this.workspaceItemCount = 0;
      const items: InfoPanelItem[] = [
         {
            key: "overview",
            name: "Overview",
            selected: false
         }
      ];
      this.agentItemCount = 1;
      if (this.selectedAgent) {
         if (this.selectedAgent.capabilities?.devices) {
            for (const deviceIdx in this.selectedAgent.capabilities?.devices) {
               const device = this.selectedAgent.capabilities?.devices[deviceIdx];
               if (!device["name"]) {
                  device["name"] = "Primary";
               }
               if (device.properties) {
                  items.push({ key: `device${deviceIdx}`, name: device["name"], selected: device["name"] === "Primary" ? true : false });
                  this.devicesItemCount++;
               }
            }
         }
         if (this.selectedAgent.workspaces) {
            const streamCount = new Map<string, number>();
            for (const workspaceIdx in this.selectedAgent.workspaces) {
               const workspace = this.selectedAgent.workspaces[workspaceIdx];
               let count = streamCount.get(workspace.stream) ?? 0;
               if (!count) {
                  count++;
                  streamCount.set(workspace.stream, 1);
               } else {
                  count++;
                  streamCount.set(workspace.stream, count);
               }

               const name: string = count > 1 ? `${workspace.stream} (${count})` : workspace.stream;

               (workspace as any)._hackName = name;
               ;
               items.push({ key: `workspace${workspaceIdx}`, name: name, selected: false, data: name });
               this.workspaceItemCount++;
            }
         }
      }
      this.setInfoItems(items);
   }

   @action
   setInfoItems(items: InfoPanelItem[]) {
      const subItems: InfoPanelSubItem[] = [];
      if (this.selectedAgent) {
         const selectedItem = items.find(item => item.selected);
         if (selectedItem) {
            if (selectedItem.key === "overview") {
               subItems.push({ name: 'Enabled', value: this.selectedAgent.enabled.toString() });
               subItems.push({ name: 'Comment', value: this.selectedAgent.comment ?? "None" });
               subItems.push({ name: 'Ephemeral', value: this.selectedAgent.ephemeral.toString() });
               subItems.push({ name: 'ForceVersion', value: this.selectedAgent.forceVersion ?? "None" });
               subItems.push({ name: 'Id', value: this.selectedAgent.id });
               subItems.push({ name: 'Online', value: this.selectedAgent.online.toString() });
               subItems.push({ name: 'Last Update', value: this.selectedAgent.updateTime.toString() });
               subItems.push({ name: 'Version', value: this.selectedAgent.version ?? "None" });
            }
            else if (selectedItem.key.indexOf("workspace") !== -1) {
               if (this.selectedAgent.workspaces) {
                  for (const workspaceIdx in this.selectedAgent.workspaces) {
                     const workspace = this.selectedAgent.workspaces[workspaceIdx];

                     if ((workspace as any)._hackName === selectedItem.data) {
                        subItems.push({ name: 'Identifier', value: workspace.identifier });
                        subItems.push({ name: 'Stream', value: workspace.stream });
                        subItems.push({ name: 'Incremental', value: workspace.bIncremental.toString() });
                        workspace.serverAndPort && subItems.push({ name: 'Server and Port', value: workspace.serverAndPort });
                        workspace.userName && subItems.push({ name: 'Username', value: workspace.userName });
                        workspace.password && subItems.push({ name: 'Password', value: workspace.password });
                        workspace.view && subItems.push({ name: 'View', value: workspace.view?.join("\n") });
                     }
                  }
               }
            }
            else if (selectedItem.key.indexOf("device") !== -1) {
               if (this.selectedAgent.capabilities?.devices) {
                  for (const deviceIdx in this.selectedAgent.capabilities?.devices) {
                     const device = this.selectedAgent.capabilities?.devices[deviceIdx];
                     if (device.name === selectedItem.name) {
                        if (device.properties) {
                           for (const propIdx in device.properties) {
                              const prop = device.properties[propIdx];
                              const subItemData = prop.split('=');
                              if (subItemData[0].indexOf("RAM") !== -1) {
                                 subItemData[1] += " GB";
                              }
                              else if (subItemData[0].indexOf("Disk") !== -1) {
                                 subItemData[1] = (Number(subItemData[1]) / 1073741824).toLocaleString(undefined, { maximumFractionDigits: 0 }) + " GiB";
                              }
                              subItems.push({ name: subItemData[0], value: subItemData[1] });
                           }
                        }
                     }
                  }
               }
            }
         }
      }
      this.infoItems = [...items];
      this.infoSubItems = subItems;
   }

   private _doUpdate() {
      if (this.mode === "leases") {
         this.UpdateLeases();
      }
      else if (this.mode === "sessions") {
         this.UpdateSessions();
      }
   }

   nullItemTrigger() {
      if (!this.bUpdatedQueued) {
         this.bUpdatedQueued = true;
         this._doUpdate();
      }
   }

   UpdateLeases() {
      const data: LeaseData[] = [];
      backend.getLeases(this.selectedAgent!.id, this.modeCurrentIndex, 30, true).then(responseData => {
         responseData.forEach((dataItem: GetAgentLeaseResponse) => {
            data.push(dataItem);
         });
         this.appendData(data);
      });
   }

   UpdateSessions() {
      const data: SessionData[] = [];
      backend.getSessions(this.selectedAgent!.id, this.modeCurrentIndex, 30).then(responseData => {
         responseData.forEach((dataItem: GetAgentSessionResponse) => {
            data.push(dataItem);
         });
         this.appendData(data);
      });
   }

   jobData = new Map<string, GetJobResponse | boolean>();

   UpdateTelemetry(minutes: number) {

      const requests = [backend.getAgentTelemetry(this.selectedAgent!.id, new Date(Date.now() - 1000 * 60 * minutes), new Date()), backend.getAgentLeases(this.selectedAgent!.id, new Date(Date.now() - 1000 * 60 * minutes), new Date())]
      Promise.all(requests).then((data) => {
         this.setTelemetryData((data[0] as GetAgentTelemetrySampleResponse[]).reverse(), data[1] as GetAgentLeaseResponse[]);
      })
   }

   @action
   setTelemetryData(telemetryData: GetAgentTelemetrySampleResponse[], sessionData: GetAgentLeaseResponse[]) {

      this.currentData = [telemetryData, sessionData];

   }

   @action
   appendData(newData: any[]) {

      if (this.mode === "telemetry") {
         return;
      }

      // if there's any data, there might be more data next time, so add another callback.
      if (newData.length > 0) {
         newData.push(null);
      }

      let combinedData = [...this.currentData];

      // remove previous null if it exists
      if (combinedData[combinedData.length - 1] === null) {
         combinedData.splice(-1, 1);
      }
      // add all the new data
      Array.prototype.push.apply(combinedData, newData);

      const dedupe = new Set<string>();

      this.currentData = combinedData.filter(d => {
         if (dedupe.has(d?.id)) {
            return false;
         }
         dedupe.add(d?.id);
         return true;
      });

      this.modeCurrentIndex += newData.length;
      this.bUpdatedQueued = false;
   }

}

export const historyModalStore = new HistoryModalStore();