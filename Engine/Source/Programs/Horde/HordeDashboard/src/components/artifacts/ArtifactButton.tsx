// Copyright Epic Games, Inc. All Rights Reserved.

import { IContextualMenuProps, DirectionalHint, PrimaryButton, Modal, Stack, Text, DefaultButton, Checkbox } from "@fluentui/react";
import backend from "horde/backend";
import { ArtifactBrowserItem, ArtifactBrowserType, ArtifactSelection, CreateZipRequest, DownloadFormat, GetArtifactResponse } from "horde/backend/Api";
import dashboard, { UserPlatform } from "horde/backend/Dashboard";
import { copyToClipboard } from "horde/base/utilities/clipboard";
import { formatBytes } from "horde/base/utilities/stringUtills";
import { getHordeStyling } from "horde/styles/Styles";
import { useState } from "react";
import { useNavigate } from "react-router-dom";


export const ArtifactButton: React.FC<{ jobId?: string, stepId?: string, artifact?: GetArtifactResponse, pathIn?: string, selection?: ArtifactSelection, openArtifactInfo?: () => void, disabled?: boolean, minWidth?: number }> = ({ jobId, stepId, artifact, pathIn, selection, openArtifactInfo, disabled, minWidth }) => {

   const [state, setState] = useState<{ showZipDownloadWarning?: boolean }>({});

   const navigate = useNavigate();

   const path = pathIn ? pathIn + "/" : "";
   const items = selection?.items;

   let sizeText = "0KB";
   if (selection?.size) {
      sizeText = formatBytes(selection.size, (selection.size < (1024 * 1024)) ? 0 : 1)
   }

   const zenArtifact = !!artifact?.metadata?.find(m => m.toLowerCase() === "backend=zen");

   const format = zenArtifact ? DownloadFormat.UGS : dashboard.downloadFormat;

   let buttonText = "Download";

   if (selection?.directoriesSelected || (selection?.filesSelected ?? 0) > 1) {
      buttonText = `Download (${sizeText})`;
   } else if (selection?.filesSelected === 1) {
      buttonText = `Download (${sizeText})`;
   }

   let jobUrl = "";
   if (jobId) {
      jobUrl = `/job/${jobId}`;
      if (stepId) {
         jobUrl += `?step=${stepId}`;
      }
   }

   const downloadProps: IContextualMenuProps = {
      items: [],
      directionalHint: DirectionalHint.bottomRightEdge
   };

   if (openArtifactInfo) {
      downloadProps.items.push({
         key: 'view_artifact_info',
         text: 'View Artifact Info',
         disabled: !artifact,
         onClick: () => {
            openArtifactInfo();
         }
      })
   }

   if (jobUrl) {
      {
         downloadProps.items.push({
            key: 'navigate_to_job',
            text: 'Navigate to Job',
            onClick: () => {
               navigate(jobUrl);
            }
         })
      }
   }

   function downloadToolbox() {
      if (artifact?.id) {
         window.location.assign(`horde-artifact://${window.location.hostname}:${window.location.port}/api/v2/artifacts/${artifact.id}`);
      }
   }

   function downloadUGS() {
      if (artifact?.id) {
         window.location.assign(`/api/v2/artifacts/${artifact.id}/download?format=ugs`);
      }
   }

   function openInBuildStorageTool(zenUri: string) {
      if (zenUri) {
         window.location.assign(`ugs://runcustomtool?toolName=build%20storage%20tool&args=RunBuildStorageTool.bat%20-GoToBuild%20${encodeURIComponent(zenUri)}`);
      }
   }

   function downloadZip() {

      if (!artifact) {
         return;
      }

      if (!items?.length) {
         window.location.assign(`/api/v2/artifacts/${artifact.id}/download?format=zip`);
         return;
      }

      // download a single file
      if (items.length === 1) {
         const item = items[0];
         if (item.type === ArtifactBrowserType.File) {

            try {
               backend.downloadArtifactV2(artifact.id, path + item.text);
            } catch (err) {
               console.error(err);
            } finally {

            }

            return;
         }
      }

      let zipRequest: CreateZipRequest | undefined;

      if ((items?.length ?? 0) > 0) {

         const filter = items.map(s => {
            const item = s as ArtifactBrowserItem;

            if (item.type === ArtifactBrowserType.NavigateUp) {
               return "";
            }

            if (item.type === ArtifactBrowserType.Directory) {
               return `${path}${item.text}/...`;
            }
            return `${path}${item.text}`;
         }).filter(f => !!f);

         zipRequest = {
            filter: filter
         }
      } else {
         if (path) {
            zipRequest = {
               filter: [path + "..."]
            }
         }
      }

      try {
         backend.downloadArtifactZipV2(artifact.id, zipRequest);
      } catch (err) {
         console.error(err);
      } finally {

      }
   }

   if (artifact?.id) {

      if (!zenArtifact) {
         downloadProps.items.push(
            {
               key: 'download_zip',
               text: 'Download Zip',
               onClick: () => {
                  showDownloadWarning() ? setState({ showZipDownloadWarning: true }) : downloadZip()
               }
            }
         )
      }

      downloadProps.items.push(
         {
            key: 'download_ugs',
            text: 'Download with UGS',
            onClick: () => {
               downloadUGS();
            }
         }
      )

      if (!zenArtifact) {
         downloadProps.items.push(
            {
               key: 'download_toolbox',
               text: 'Download with Toolbox',
               onClick: () => {
                  downloadToolbox();
               }
            }
         )

         downloadProps.items.push(
            {
               key: 'download_copy_to_clipboard',
               text: 'Copy Artifact ID',
               onClick: () => {
                  copyToClipboard(artifact.id);
               }
            }
         )
      } else {
         const zenUri = artifact?.metadata?.find(m => m.toLowerCase().startsWith("zenbuilduri="))?.slice(12)?.trim();
         if (zenUri) {
            downloadProps.items.push(
               {
                  key: 'download_copy_zen_uri_to_clipboard',
                  text: 'Copy Cloud Artifact URL',
                  onClick: () => {
                     copyToClipboard(zenUri);
                  }
               }
            )

            if (dashboard.platform === UserPlatform.Windows) {
               downloadProps.items.push(
                  {
                     key: 'open_build_storage_tool',
                     text: 'Open in Build Storage Tool',
                     onClick: () => {
                        openInBuildStorageTool(zenUri);
                     }
                  }
               )
            }
         }
      }
   }

   function showDownloadWarning() {
      let hide_warning = localStorage.getItem("horde_hide_zip_warning")
      if (hide_warning === "true") {
         return false
      }

      if (selection && selection.size < 52428800) {
         return false
      }

      return true
   }

   return <Stack>{!!state.showZipDownloadWarning && <ZipDownloadWarning onContinue={() => { downloadZip(); setState({ showZipDownloadWarning: false }); }} onClose={() => { setState({ showZipDownloadWarning: false }) }} />}
      <PrimaryButton split menuProps={downloadProps} text={buttonText} styles={{ root: { minWidth: minWidth ?? 104, fontFamily: 'Horde Open Sans SemiBold !important' } }} disabled={disabled} onClick={() => {
         if (format === DownloadFormat.Zip) {
            showDownloadWarning() ? setState({ showZipDownloadWarning: true }) : downloadZip()
         }
         else if (format === DownloadFormat.UGS) {
            downloadUGS();
         }
         else if (format === DownloadFormat.UTOOLBOX) {
            downloadToolbox();
         }
      }} />
   </Stack>

}

const ZipDownloadWarning: React.FC<{ onContinue: () => void, onClose: () => void }> = ({ onContinue, onClose }) => {

   const { hordeClasses } = getHordeStyling();

   return <Modal className={hordeClasses.modal} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, height: "fit-content", width: 600, hasBeenOpened: false, top: 0, bottom: 150, left: 0, right: 0, margin: "auto", position: "absolute" } }} >

      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 24 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>Warning: Slow Download</Text>
         </Stack>
         <Stack style={{ paddingBottom: 32 }} tokens={{childrenGap: 16}}>
            <Text variant="medium" style={{ fontFamily: "Horde Open Sans SemiBold" }}>Downloading artifacts in zip format is significantly slower than downloading with UGS.</Text>
            <Text variant="medium" style={{ fontFamily: "Horde Open Sans SemiBold" }}>Click the download button dropdown and select "Download with UGS." Open the downloaded uartifact file with UnrealGameSync for faster download speeds.</Text>
         </Stack>
         <Stack>
            <Checkbox label="Don't show this warning again" onChange={(ev?: any, checked?: boolean | undefined) => { localStorage.setItem("horde_hide_zip_warning", checked!.toString()) }} />
         </Stack>
         <Stack horizontal style={{ paddingTop: 16 }}>
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 16 }}>
               <PrimaryButton onClick={() => onContinue()} text="Continue" />
               <DefaultButton onClick={() => onClose()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>
}
