// Copyright Epic Games, Inc. All Rights Reserved.

import { Callout, DetailsList, DetailsListLayoutMode, DirectionalHint, FontIcon, IColumn, IContextualMenuProps, IconButton, Modal, NavBase, PrimaryButton, ScrollablePane, Selection, SelectionMode, SelectionZone, Spinner, SpinnerSize, Stack, Text, TooltipHost, mergeStyleSets } from "@fluentui/react";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import { useEffect, useState } from "react";
import { NavigateFunction, useNavigate } from "react-router-dom";
import backend from "../../backend";
import { ArtifactBrowserItem, ArtifactBrowserType, ArtifactContextType, CreateZipRequest, GetArtifactDirectoryEntryResponse, GetArtifactDirectoryResponse, GetArtifactFileEntryResponse, GetArtifactResponse } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { getHordeStyling } from "../../styles/Styles";
import { formatBytes } from "horde/base/utilities/stringUtills";
import { ArtifactButton } from "./ArtifactButton";


class ArtifactsHandler {

   constructor(jobId: string, stepId: string, contextType: ArtifactContextType, artifactPath?: string, artifacts?: GetArtifactResponse[], artifactId?: string) {
      makeObservable(this);
      this.jobId = jobId;
      this.stepId = stepId;
      this.context = contextType;
      this.artifacts = artifacts;
      this.artifactId = artifactId;
      this.set(artifactPath);

      const params = new URLSearchParams(window.location.search);
      params.delete("artifactPath");
      this.baseSearch = `?${params.toString()}`;

      ArtifactsHandler.current = this;


   }

   @observable
   updated = 0

   @action
   updateReady() {
      this.updated++;
   }

   private async set(artifactPath?: string) {

      let artifacts: GetArtifactResponse[] | undefined = this.artifacts;

      if (!artifacts) {

         const key = `job:${this.jobId}/step:${this.stepId}`;
         try {
            const v = await backend.getJobArtifactsV2(undefined, [key]);
            artifacts = v.artifacts;
         } catch (err) {
            console.error(err);
         }

         if (!artifacts) {
            console.error(`Missing artifacts for job: ${this.jobId} step: ${this.stepId}`);
            return;
         }

         this.artifacts = artifacts;
      }

      if (!artifacts) {
         console.error(`Missing artifacts for job: ${this.jobId} step: ${this.stepId}`);
         return;
      }

      if (this.artifactId && !artifacts.find(a => a.id === this.artifactId)) {
         try {
            const artifact = await backend.getArtifactData(this.artifactId);
            artifacts.push(artifact);
         } catch (err) {
            console.log(err);
         }
      }

      this.artifacts = artifacts;

      if (!this.context) {
         console.error("Artifact browser has no context");
         this.updateReady();
         return;
      }

      let a: GetArtifactResponse | undefined;

      if (this.artifactId) {
         a = artifacts.find(a => a.id === this.artifactId)
      }

      if (!a) {
         a = artifacts.find(a => a.type === this.context)!;
      }

      if (!a) {
         console.error("Unable to find artifact for context", this.context, artifacts);
         this.artifactMissing = true;
         this.updateReady();
         return;
      }

      this.artifact = a;

      this.loading = true;
      this.updateReady();

      if (!artifactPath) {
         this.browse = await backend.getBrowseArtifacts(a.id);
         if (this.browse.directories?.length === 1 && !this.browse.files?.length) {
            let d = this.browse.directories[0] as GetArtifactDirectoryEntryResponse | undefined;
            const path: string[] = [];
            while (d) {
               path.push(d.name);
               d = d.directories?.length ? d.directories[0] : undefined;
            }
            const dpath = path.join("/");
            this.browse = await backend.getBrowseArtifacts(a.id, dpath);
            this.path = dpath;
         }
         this.loading = false;
      } else {
         this.browse = await backend.getBrowseArtifacts(this.artifact.id, artifactPath);
         this.path = artifactPath;
         this.loading = false;
      }

      this.updateReady();

   }

   hasContext(c: ArtifactContextType) {

      return !!this.artifacts?.find(a => a.type === c);

   }

   get contextName(): string {
      switch (this.context) {
         case "step-output":
            return "Temp Storage";
         case "step-saved":
            return "Log";
         case "step-trace":
            return "Trace";
         default:
            return this.artifact?.description ?? this.artifact?.name ?? "Unknown";
      }
   }

   async browseTo(path: string, navigate: NavigateFunction, push = true) {

      if (!this.artifact) {
         return;
      }

      this.loading = true;
      this.updateReady();
      this.browse = await backend.getBrowseArtifacts(this.artifact.id, path);

      this.path = path;

      if (push) {
         this.history.push(path);
      }

      if (this.baseSearch) {
         let url = `${window.location.pathname}${this.baseSearch}`;
         if (path?.length) {
            url += `&artifactPath=${encodeURIComponent(path.replaceAll("+", "%2B"))}`;
         }

         navigate(url, { replace: true });
         console.log(this.artifact.id, path);
      }

      this.loading = false;
      this.updateReady();

   }

   get currentSelection(): { filesSelected: number, directoriesSelected: number, size: number, items: ArtifactBrowserItem[] } {

      let result = {
         filesSelected: 0,
         directoriesSelected: 0,
         size: 0,
         items: [] as ArtifactBrowserItem[]
      }

      const browse = this.browse;
      if (!browse) {
         return result;
      }

      let selection = this.selection?.getSelection() as (ArtifactBrowserItem[] | undefined);

      if (!selection?.length) {
         selection = [];

         browse.directories?.forEach(d => {
            selection!.push({ key: d.hash, text: d.name, icon: "Folder", type: ArtifactBrowserType.Directory, size: d.length });
         });

         browse.files?.forEach(d => {
            selection!.push({ key: d.hash, text: d.name, icon: "Document", type: ArtifactBrowserType.File, size: d.length });
         });
      }

      selection.forEach(b => {
         result.items.push(b);
         if (b.type === ArtifactBrowserType.Directory) {
            result.directoriesSelected++;
            result.size += b.size ?? 0;
         }
         if (b.type === ArtifactBrowserType.File) {
            result.filesSelected++;
            result.size += b.size ?? 0;
         }
      });

      return result;

   }

   get directories(): GetArtifactDirectoryEntryResponse[] {
      return this.browse?.directories ?? [];
   }

   get files(): GetArtifactFileEntryResponse[] {
      return this.browse?.files ?? [];
   }

   clear() {
      this.selection = undefined;
      this.selectionCallback = undefined;
      this.path = undefined;
      this.browse = undefined;
      this.artifact = undefined;
      this.artifacts = undefined;
      this.history = [];
      this.stepId = "";
      this.loading = false;
      this.baseSearch = undefined;
      this.artifactMissing = undefined;
      ArtifactsHandler.current = undefined;
   }


   selectionCallback?: () => void;

   path?: string;

   selection?: Selection = new Selection({ canSelectItem: (item: any) => { return item.type !== ArtifactBrowserType.NavigateUp; }, onSelectionChanged: () => { if (this.selectionCallback) this.selectionCallback() } });

   browse?: GetArtifactDirectoryResponse;

   artifact?: GetArtifactResponse;
   artifacts?: GetArtifactResponse[];

   artifactId?: string;

   readonly context: ArtifactContextType;

   jobId: string;
   stepId: string;

   history: string[] = [];

   loading = false;

   baseSearch?: string;

   artifactMissing?: boolean;

   static current?: ArtifactsHandler;
}

let _styles: any;

const getStyles = () => {

   const border = `1px solid ${dashboard.darktheme ? "#2D2B29" : "#EDEBE9"}`

   const styles = _styles ?? mergeStyleSets({
      list: {
         selectors: {
            'a': {
               height: "unset !important",
            },
            '.ms-List-cell': {

               borderTop: border,
               borderRight: border,
               borderLeft: border
            },
            '.ms-List-cell:nth-last-child(-n + 1)': {
               borderBottom: border
            },
            ".ms-DetailsRow #artifactview": {
               opacity: 0
            },
            ".ms-DetailsRow:hover #artifactview": {
               opacity: 1
            },
         }
      }
   });

   _styles = styles;

   return styles;

}

const BrowseHistory: React.FC<{ handler: ArtifactsHandler }> = observer(({ handler }) => {

   const navigate = useNavigate();

   // subscribe
   if (handler.updated) { };

   let backDisabled = !handler.history.length;

   return <Stack style={{ paddingRight: 12 }}>
      <Stack horizontal tokens={{ childrenGap: 8 }}>
         <Stack>
            <IconButton disabled={backDisabled} style={{ fontSize: 14, paddingTop: 1 }} iconProps={{ iconName: 'ArrowLeft' }} onClick={() => {
               if (handler.history.length === 1) {
                  handler.history = [];
                  handler.browseTo("", navigate, false);
               } else {
                  handler.history.pop();
                  handler.browseTo(handler.history[handler.history.length - 1], navigate, false);
               }
            }} />
         </Stack>
      </Stack>
   </Stack>

});

const BrowseBreadCrumbs: React.FC<{ handler: ArtifactsHandler }> = observer(({ handler }) => {

   const navigate = useNavigate();

   // subscribe
   if (handler.updated) { }

   const fontSize = 13;

   let rootName = handler.context;

   if (handler.context === "step-saved") {
      rootName = "Saved";
   }
   else if (handler.context === "step-output") {
      rootName = "Output";
   }
   else if (handler.context === "step-trace") {
      rootName = "Trace";
   } else {
      rootName = handler.contextName
   }

   if (!handler.path) {
      return <Stack horizontal verticalAlign="center" styles={{ root: { height: 40 } }}>
         <Stack>
            <BrowseHistory handler={handler} />
         </Stack>
         <Stack>
            <Text style={{ fontSize: fontSize, fontWeight: 600, paddingLeft: 2, paddingRight: 4 }}>{rootName} /</Text>
         </Stack>
      </Stack>

   }

   const ppath = handler.path.split("/");

   const elements = ppath.map((e, index) => {
      const path = ppath.slice(0, index + 1).join("/");

      const last = index === (ppath.length - 1);

      const color = last ? undefined : (dashboard.darktheme ? "#55B7FF" : "#0078D4");
      const cursor = last ? undefined : "pointer";

      return <Stack horizontal onClick={() => handler.browseTo(path, navigate)} style={{ cursor: cursor }}>
         <Stack>
            <Text style={{ fontSize: fontSize, color: color, fontWeight: last ? 600 : undefined }}>{e}</Text>
         </Stack>
         {index !== (ppath.length - 1) && <Stack style={{ paddingLeft: 4, paddingRight: 4 }}><Text style={{ fontSize: fontSize, fontWeight: 600 }}>/</Text></Stack>}
      </Stack>
   });

   elements.unshift(<Stack style={{ cursor: "pointer" }} onClick={() => handler.browseTo("", navigate)}>
      <Stack style={{ paddingLeft: 2, paddingRight: 4 }}>
         <Text style={{ fontSize: fontSize, color: (dashboard.darktheme ? "#55B7FF" : "#0078D4"), fontWeight: 600 }}>{rootName} /</Text>
      </Stack>
   </Stack>);

   return <Stack horizontal verticalAlign="center" styles={{ root: { height: 40 } }}>
      <Stack>
         <BrowseHistory handler={handler} />
      </Stack>
      <Stack horizontal wrap style={{ width: 800 }}>
         {elements}
      </Stack>
   </Stack>

})


const DownloadButton: React.FC<{ handler: ArtifactsHandler, openArtifactInfo: () => void }> = observer(({ handler, openArtifactInfo }) => {

   const [selectKey, setSelectionKey] = useState(0);   

   // subscribe
   if (handler.updated) { };

   const browse = handler.browse;

   if (!browse) {
      return null;
   }

   handler.selectionCallback = () => {
      setSelectionKey(selectKey + 1);
   }

   const selection = handler.currentSelection;

   return <Stack id="callout_target_artifactinfo" horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
      <ArtifactButton minWidth={150} jobId={handler?.jobId} stepId={handler?.stepId} artifact={handler.artifact} selection={selection} pathIn={handler.path} openArtifactInfo={openArtifactInfo} />
   </Stack>

});

let idcounter = 0;

const JobDetailArtifactsInner: React.FC<{ jobId: string; stepId: string, artifacts?: GetArtifactResponse[], contextType: ArtifactContextType, artifactPath?: string, artifactId?: string }> = observer(({ jobId, stepId, artifacts, contextType, artifactPath, artifactId }) => {

   // eslint-disable-next-line
   const handler = ArtifactsHandler.current ?? new ArtifactsHandler(jobId, stepId, contextType, artifactPath, artifacts, artifactId);

   const [viewArtifactInfo, setViewArtifactInfo] = useState(false);

   useEffect(() => {
      return () => {
         handler?.clear();
      };
   }, [handler]);

   const navigate = useNavigate();

   const styles = getStyles();

   // subscribe
   if (handler.updated) { }

   if (handler.artifactMissing) {

      let text = `${handler.contextName} artifact was not found on server.`;

      if (handler.contextName === "Temp Storage") {
         text += "  Temporary storage artifacts expire due to high volume.";
      }

      return <Stack>
         <Stack horizontal verticalAlign="center" verticalFill tokens={{ childrenGap: 12 }}>
            <FontIcon style={{ paddingTop: 1, fontSize: 17, color: dashboard.getStatusColors().get(StatusColor.Failure) }} iconName="Error" />
            <Text variant="mediumPlus">{text}</Text>
         </Stack>
      </Stack>;
   }

   const browse = handler.browse;

   if (!browse) {
      return <Stack><Spinner size={SpinnerSize.large} /></Stack>;
   }

   const items: ArtifactBrowserItem[] = [];

   const getFileHRef = (item: ArtifactBrowserItem) => {

      if (item.type !== ArtifactBrowserType.File) {
         return undefined;
      }

      let path = (handler.path ? handler.path + "/" : "") + encodeURIComponent(item.text);
      path = path.replaceAll("+", "%2B");
      const server = backend.serverUrl;
      return `${server}/api/v2/artifacts/${handler.artifact!.id}/file?path=${path}`;

   }

   // use the up arrow instead
   if (handler.path?.length) {
      items.push({ key: "navigate up", text: "..", type: ArtifactBrowserType.NavigateUp });
   }

   browse.directories?.forEach(d => {

      function recurseDirectories(dir: GetArtifactDirectoryEntryResponse, flattened: GetArtifactDirectoryEntryResponse[]) {
         if (!dir.directories || dir.directories.length > 1 || dir.files?.length) {
            const name = flattened.length ? flattened.map(d => d.name).join("/") + "/" + dir.name : dir.name;
            items.push({ key: dir.hash, text: name, icon: "Folder", type: ArtifactBrowserType.Directory, size: dir.length, dirResponse: dir });
         } else {
            flattened.push(dir);
            dir.directories.forEach(d => recurseDirectories(d, [...flattened]));
         }
      }

      recurseDirectories(d, []);


   });

   browse.files?.forEach(d => {
      items.push({ key: d.hash, text: d.name, icon: "Document", type: ArtifactBrowserType.File, size: d.length, fileResponse: d });
   });

   const columns: IColumn[] = [
      { key: 'column1', name: 'Name', minWidth: 794 - 32, maxWidth: 794 - 32, isResizable: false, isPadded: false },
      { key: 'column2', name: 'Size', minWidth: 128, maxWidth: 128, isResizable: false, isPadded: false },
      { key: 'column3', name: 'View_Download', minWidth: 64 + 32, maxWidth: 64 + 32, isResizable: false, isPadded: false }
   ];

   const renderItem = (item: any, index?: number, column?: IColumn) => {

      if (!column) {
         return null;
      }

      if (column.name === "Size") {
         if (!item.size) {
            return null;
         }
         return <Stack horizontalAlign="end" verticalAlign="center" verticalFill>
            <Text>{formatBytes(item.size, (item.size < (1024 * 1024)) ? 0 : 1)}</Text>
         </Stack>

      }

      if (column.name === "View_Download") {

         if (item.text === "..") {
            return null;
         }

         const isFile = item.type === ArtifactBrowserType.File;

         const href = getFileHRef(item);

         return <Stack>
            <Stack data-selection-disabled verticalAlign="center" verticalFill horizontal horizontalAlign="end" style={{ paddingTop: 0, paddingBottom: 0 }}>
               {isFile && <IconButton id="artifactview" href={`${href}&inline=true`} target="_blank" style={{ paddingTop: 1, color: "#106EBE" }} iconProps={{ iconName: "Eye", styles: { root: { fontSize: "14px" } } }} />}
               {isFile && <IconButton id="artifactview" href={href} target="_blank" style={{ paddingTop: 1, color: "#106EBE" }} iconProps={{ iconName: "CloudDownload", styles: { root: { fontSize: "14px" } } }} />}
            </Stack>
         </Stack>
      }

      if (column.name === "Name") {

         const path = (item.text as string).split("/");
         const fullName = item.text as string;

         const pathElements = path.map((t, index) => {
            const last = index === (path.length - 1);
            let color = last ? (dashboard.darktheme ? "#FFFFFF" : "#605E5C") : undefined;
            const font = last ? undefined : "Horde Open Sans Light";
            const sep = last ? undefined : "/"
            return <Text styles={{ root: { fontFamily: font } }} style={{ color: color }}>{t}{sep}</Text>
         })

         // Wrap in tooltip if the name is long
         const needsTooltip = fullName.length > 100;

         const wrapWithTooltip = (content: JSX.Element) => {
            if (needsTooltip) {
               return <TooltipHost content={fullName} calloutProps={{ styles: { root: { maxWidth: 700 } } }}>{content}</TooltipHost>;
            }
            return content;
         };

         return wrapWithTooltip(<Stack verticalFill verticalAlign="center" style={{ cursor: "pointer" }} onClick={(ev) => {

            if (item.type === ArtifactBrowserType.Directory) {
               const nbrowse = handler.path ? `${handler.path}/${item.text}` : item.text;
               handler.browseTo(nbrowse, navigate);
            }
            if (item.type === ArtifactBrowserType.NavigateUp && handler.path) {
               const nbrowse = handler.path.split("/")
               nbrowse.pop();
               handler.browseTo(nbrowse.join("/"), navigate);
            }

         }}>
            {item.type !== ArtifactBrowserType.NavigateUp && <Stack data-selection-disabled={item.type !== ArtifactBrowserType.File} horizontal tokens={{ childrenGap: 8 }}>
               <Stack>
                  <FontIcon style={{ paddingTop: 1, fontSize: 16 }} iconName={item.icon} />
               </Stack>
               <Text style={{ overflow: "hidden", textOverflow: "ellipsis", whiteSpace: "nowrap", display: "block" }}>
                  {fullName}
               </Text>
            </Stack>}
            {item.type === ArtifactBrowserType.NavigateUp && <Stack data-selection-disabled verticalFill horizontal verticalAlign="center" tokens={{ childrenGap: 9 }}>
               <Stack>
                  <FontIcon style={{ paddingTop: 1, fontSize: 15 }} iconName="ArrowUp" />
               </Stack>
               <Stack>
                  <Text>..</Text>
               </Stack>
            </Stack>
            }
         </Stack>);

      }

      return null;

   }

   return <Stack key={`jobdetailartifacts_${idcounter++}`}>
      {viewArtifactInfo && <Callout
         styles={{ root: { padding: "32px 24px", maxWidth: 1300 } }}
         role="dialog"
         gapSpace={12}
         target={`#callout_target_artifactinfo`}
         isBeakVisible={true}
         beakWidth={12}
         onDismiss={() => {
            setViewArtifactInfo(false);
         }}
         directionalHint={DirectionalHint.leftTopEdge}
         setInitialFocus>
         <Stack style={{ maxWidth: 1140 }}>
            <Stack style={{ paddingBottom: 12 }}>
               <Stack horizontal verticalAlign="center" verticalFill={true}>
                  <Stack>
                     <Text style={{ fontSize: 14, fontFamily: "Horde Open Sans SemiBold" }}>{`Artifact ID ${handler.artifact?.id}`}</Text>
                  </Stack>
                  <Stack grow />
                  <Stack>
                     <IconButton
                        iconProps={{ iconName: 'Cancel', styles: { root: { fontSize: "14px" } } }}
                        onClick={() => { setViewArtifactInfo(false); }}
                     />
                  </Stack>
               </Stack>
            </Stack>
            <Stack style={{ paddingLeft: 0 }}>
               <Text style={{ fontSize: 11, whiteSpace: "pre-wrap", fontFamily: "Horde Cousine Regular" }}>{handler.artifact ? JSON.stringify(handler.artifact, undefined, 2).replaceAll("\\r", "").replaceAll("\\n", "\n") : ""}</Text></Stack>
         </Stack>
      </Callout>}

      <Stack tokens={{ childrenGap: 12 }}>
         <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 18 }} style={{ paddingBottom: 12 }}>
            <Stack>
               <BrowseBreadCrumbs handler={handler} />
            </Stack>
            <Stack grow />
            <DownloadButton handler={handler} openArtifactInfo={() => setViewArtifactInfo(true)} />
         </Stack >
         {handler.loading && <Stack>
            <Spinner styles={{ root: { opacity: 0, animation: "hordeFadeIn 1s ease-in-out 2s forwards" } }} size={SpinnerSize.large} />
         </Stack>}
         {!handler.loading && <Stack style={{ height: 492 + 160, position: "relative" }}>
            <ScrollablePane style={{ height: 492 + 160 }}>
               <SelectionZone selection={handler.selection!}>
                  <DetailsList
                     styles={{ root: { overflowX: "hidden" } }}
                     className={styles.list}
                     isHeaderVisible={false}
                     compact={true}
                     items={items}
                     columns={columns}
                     layoutMode={DetailsListLayoutMode.fixedColumns}
                     selectionMode={SelectionMode.multiple}
                     enableUpdateAnimations={false}
                     selection={handler.selection}
                     selectionPreservedOnEmptyClick={true}
                     onShouldVirtualize={() => false}
                     onItemInvoked={(item: ArtifactBrowserItem) => {
                        if (item?.type !== ArtifactBrowserType.File) {
                           return;
                        }
                        const href = getFileHRef(item);
                        if (href) {
                           window.open(href + "&inline=true", "_blank");
                        }
                     }}
                     onRenderItemColumn={renderItem}
                  />
               </SelectionZone>

            </ScrollablePane>
         </Stack>}
      </Stack >
   </Stack>

})



export const JobArtifactsModal: React.FC<{ jobId: string; stepId: string, artifacts?: GetArtifactResponse[], contextType: ArtifactContextType, artifactPath?: string, artifactId?: string, onClose: () => void }> = ({ jobId, stepId, artifacts, contextType, artifactPath, artifactId, onClose }) => {

   const { hordeClasses } = getHordeStyling();

   return <Stack>
      <Modal isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1180, height: 820, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onClose()} className={hordeClasses.modal}>
         <Stack className="horde-no-darktheme" styles={{ root: { paddingTop: 10, paddingRight: 12 } }}>
            <Stack style={{ paddingLeft: 24, paddingRight: 24 }}>
               <Stack tokens={{ childrenGap: 12 }} style={{ height: 800 }}>
                  <Stack horizontal verticalAlign="start">
                     <Stack style={{ paddingTop: 3 }}>
                        <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Artifacts</Text>
                     </Stack>
                     <Stack grow />
                     <Stack horizontalAlign="end">
                        <IconButton
                           iconProps={{ iconName: 'Cancel' }}
                           onClick={() => { onClose() }}
                        />
                     </Stack>
                  </Stack>
                  <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
                     <JobDetailArtifactsInner stepId={stepId} jobId={jobId} contextType={contextType} artifactPath={artifactPath} artifacts={artifacts} artifactId={artifactId} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   </Stack>;
};