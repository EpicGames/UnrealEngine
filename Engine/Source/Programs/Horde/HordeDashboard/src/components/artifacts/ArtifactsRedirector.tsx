// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { Navigate, useNavigate, useParams } from 'react-router-dom';
import { errorDialogStore } from '../error/ErrorStore';
import backend from '../../backend';

function setError(artifactId: string, message: string) {

   console.error(message);

   message = `${message}\n\nReferring URL: ${encodeURI(window.location.href)}`

   errorDialogStore.set({

      title: `Error Getting Artifact ${artifactId}`,
      message: message

   }, true);

}

let redirecting = false;

// redirect from external source, where horde stream id, etc are not known by that application
export const ArtifactRedirector: React.FC = () => {

   const { artifactId } = useParams<{ artifactId: string }>();
   const navigate = useNavigate();

   const getData = async () => {
      try {
         const data = await backend.getArtifactData(artifactId!);
         let key = data.keys.find(k => k.startsWith("job:") && k.indexOf("/step:") !== -1);
         let jobId = "";
         let stepId = "";
         if (key) {
            jobId = key.slice(4, 28);
            stepId = key.slice(-4)
         } else {
            key = data.keys.find(k => k.startsWith("job:"));
            if (key) {
               jobId = key.slice(4, 28);
            }
         }

         if (!jobId) {
            setError(artifactId!, "Unable to get job data");
         } else {
            let url = `/job/${jobId}`;
            if (stepId) {
               url += `?step=${stepId}&artifactId=${artifactId}`;
            } 
            navigate(url, { replace: true });
         }

      } catch (reason) {
         setError(artifactId!, "Error getting artifact data");
      }
   }

   if (!artifactId) {
      setError("None", "No artifact id specified");
   } else {
      if (!redirecting) {
         redirecting = true;
         getData();
      }
   }

   return null;

}