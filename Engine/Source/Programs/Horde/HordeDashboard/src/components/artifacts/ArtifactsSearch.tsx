// Copyright Epic Games, Inc. All Rights Reserved.

import { ComboBox, DefaultButton, DetailsList, DetailsListLayoutMode, DirectionalHint, IComboBox, IComboBoxOption, IContextualMenuProps, IDetailsListProps, IconButton, Label, Modal, PrimaryButton, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import { useConst } from '@fluentui/react-hooks';
import React, { useState } from "react";
import { NavigateFunction, useNavigate } from "react-router-dom";
import backend from "../../backend";
import { GetArtifactResponse } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { projectStore } from "../../backend/ProjectStore";
import { getActiveStreamId } from "../../base/utilities/streamUtils";
import { getHordeStyling } from "../../styles/Styles";
import { StreamChooser, streamIdAll } from "../projects/StreamChooser";
import { JobArtifactsModal } from "./ArtifactsModal";
import { useQuery } from "horde/base/utilities/hooks";
import { ArtifactButton } from "./ArtifactButton";

type ArtifactSearchState = {
   streamId?: string;
   minChange?: string;
   maxChange?: string;
   name?: string;
   typeKey?: string;
   sort?: string;
   browseArtifactId?: string;
   browseJobId?: string;
   browseStepId?: string;
   browseType?: string;
}

export class ArtifactQueryState {

   constructor(search: URLSearchParams) {
      this.originalSearch = new URLSearchParams(search);
      const state = this.state = this.fromSearch(search);

      if (state.minChange || state.maxChange || state.name || state.typeKey) {
         this.autoLoad = true;
      }
   }

   fromSearch(search: URLSearchParams): ArtifactSearchState {

      const state: ArtifactSearchState = {};

      state.streamId = search.get("artifactStreamId") ?? undefined;
      state.minChange = search.get("artifactMinChange") ?? undefined;
      state.maxChange = search.get("artifactMaxChange") ?? undefined;
      state.name = search.get("artifactName") ?? undefined;
      state.typeKey = search.get("artifactTypeKey") ?? undefined;
      state.sort = search.get("artifactSort") ?? undefined;
      state.browseArtifactId = search.get("artifactId") ?? undefined;
      state.browseJobId = search.get("artifactJobId") ?? undefined;
      state.browseStepId = search.get("artifactStepId") ?? undefined;
      state.browseType = search.get("artifactType") ?? undefined;
      return state;

   }

   static clearSearch(search: URLSearchParams) {
      search.delete("artifactStreamId");
      search.delete("artifactMinChange");
      search.delete("artifactMaxChange");
      search.delete("artifactName");
      search.delete("artifactTypeKey");
      search.delete("artifactSort");
      search.delete("artifactId");
      search.delete("artifactJobId");
      search.delete("artifactStepId");
      search.delete("artifactType");
   }

   reset(navigate: NavigateFunction) {
      const search = new URLSearchParams(this.originalSearch);
      ArtifactQueryState.clearSearch(search);
      this.state = this.fromSearch(search);
      const url = `${window.location.pathname}?` + search.toString();
      navigate(url, { replace: true })
   }

   updateSearch(navigate?: NavigateFunction, replace: boolean = true) {

      const search = new URLSearchParams(this.originalSearch);
      ArtifactQueryState.clearSearch(search);
      const state = this.state;

      if (state.streamId) {
         search.append("artifactStreamId", state.streamId)
      }

      if (state.minChange) {
         search.append("artifactMinChange", state.minChange)
      }

      if (state.maxChange) {
         search.append("artifactMaxChange", state.maxChange)
      }

      if (state.name) {
         search.append("artifactName", state.name)
      }

      if (state.typeKey) {
         search.append("artifactTypeKey", state.typeKey)
      }

      if (state.sort) {
         search.append("artifactSort", state.sort)
      }

      if (state.browseArtifactId) {
         search.append("artifactId", state.browseArtifactId)
      }

      if (state.browseJobId) {
         search.append("artifactJobId", state.browseJobId)
      }

      if (state.browseStepId) {
         search.append("artifactStepId", state.browseStepId)
      }

      if (state.browseType) {
         search.append("artifactType", state.browseType)
      }

      if (navigate) {
         const url = `${window.location.pathname}?` + search.toString();
         navigate(url, { replace: replace })
      }
   }

   get streamId(): string | undefined {
      return this.state.streamId;
   }

   set streamId(streamId: string | undefined) {
      this.state.streamId = streamId;
   }

   get name(): string | undefined {
      return this.state.name;
   }

   set name(name: string | undefined) {
      this.state.name = name;
   }

   get sort(): string | undefined {
      return this.state.sort;
   }

   set sort(sort: string | undefined) {
      this.state.sort = sort;
   }

   get typeKey(): string | undefined {
      return this.state.typeKey;
   }

   set typeKey(typeKey: string | undefined) {
      this.state.typeKey = typeKey;
   }

   get minChangeList(): string | undefined {
      return this.state.minChange;
   }

   set minChangeList(minChange: string | undefined) {
      this.state.minChange = minChange;
   }

   get maxChangeList(): string | undefined {
      return this.state.maxChange;
   }

   set maxChangeList(maxChange: string | undefined) {
      this.state.maxChange = maxChange;
   }

   get browseArtifactId(): string | undefined {
      return this.state.browseArtifactId;
   }

   set browseArtifactId(browseArtifactId: string | undefined) {
      this.state.browseArtifactId = browseArtifactId;
   }

   get browseJobId(): string | undefined {
      return this.state.browseJobId;
   }

   set browseJobId(browseJobId: string | undefined) {
      this.state.browseJobId = browseJobId;
   }

   get browseStepId(): string | undefined {
      return this.state.browseStepId;
   }

   set browseStepId(browseStepId: string | undefined) {
      this.state.browseStepId = browseStepId;
   }

   get browseType(): string | undefined {
      return this.state.browseType;
   }

   set browseType(browseType: string | undefined) {
      this.state.browseType = browseType;
   }

   autoLoad = false;
   private state: ArtifactSearchState = {}
   private originalSearch: URLSearchParams;

}

const ArtifactBrowser: React.FC<{ state: ArtifactQueryState }> = ({ state }) => {

   const navigate = useNavigate();
   useQuery();

   // sync with search, needed for browser hisgtory navigation
   const search = new URLSearchParams(window.location.search);

   const artifactId = search.get("artifactId") ?? undefined;
   const jobId = search.get("artifactJobId") ?? undefined;
   const stepId = search.get("artifactStepId") ?? undefined;
   const type = search.get("artifactType") ?? undefined;

   state.browseArtifactId = artifactId;
   state.browseJobId = jobId;
   state.browseStepId = stepId;
   state.browseType = type;

   if (!artifactId || !jobId || !stepId || !type) {
      return null;
   }

   return <Stack>
      <JobArtifactsModal jobId={jobId} stepId={stepId!} artifactId={artifactId} contextType={type} onClose={() => {
         state.browseArtifactId = undefined;
         state.browseJobId = undefined;
         state.browseStepId = undefined;
         state.browseType = undefined;
         state.updateSearch(navigate)
      }} />
   </Stack>
}

const ArtifactsList: React.FC<{ state: ArtifactQueryState, artifacts?: GetArtifactResponse[], sortBy?: string }> = ({ state, artifacts, sortBy }) => {

   const navigate = useNavigate();

   if (!artifacts?.length) {
      return null;
   }

   sortBy = sortBy ?? "sort-name";

   const sorted = artifacts.filter(a => {
      return !!a.keys.find(k => k.startsWith("job:") && k.indexOf("/step:") !== -1)
   }).sort((a, b) => {

      if (a.streamId !== b.streamId) {
         return a.streamId!.localeCompare(b.streamId!);
      }

      const changeA = a.change ?? 0;
      const changeB = b.change ?? 0;

      if (sortBy === "sort-name") {

         if (a.name !== b.name) {
            return a.name.localeCompare(b.name);
         }

         if (changeA !== changeB) {
            return changeA - changeB;
         }
      } else {

         if (changeA !== changeB) {
            return changeA - changeB;
         }

         if (a.name !== b.name) {
            return a.name.localeCompare(b.name);
         }
      }


      return 0;

   })

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as GetArtifactResponse;

         let background: string | undefined;
         if (props.itemIndex % 2 === 0) {
            background = dashboard.darktheme ? "#1D2021" : "#EAE9E9";
         }

         const stream = projectStore.streamById(item.streamId);

         return <Stack horizontal verticalAlign="center" verticalFill tokens={{ childrenGap: 24 }} styles={{ root: { backgroundColor: background, paddingLeft: 12, paddingRight: 12, paddingTop: 8, paddingBottom: 8 } }}>
            <Stack style={{ width: 360, paddingLeft: 8 }}>
               <Text style={{ fontWeight: 600 }}>{item.name}</Text>
            </Stack>
            <Stack style={{ width: 180 }}>
               <Text>{stream ? (stream.fullname ?? stream.name) : item.streamId}</Text>
            </Stack>
            <Stack style={{ width: 72 }}>
               <Text>{item.change}</Text>
            </Stack>
            <Stack style={{ width: 144 }}>
               <Text>{item.type}</Text>
            </Stack>
            <Stack horizontal tokens={{ childrenGap: 18 }}>
               <DefaultButton style={{ width: 90 }} text="Browse" onClick={() => {

                  const key = item.keys.find(k => k.startsWith("job:") && k.indexOf("/step:") !== -1);
                  if (!key) {
                     return;
                  }

                  state.browseArtifactId = item.id;
                  state.browseJobId = key.slice(4, 28);
                  state.browseStepId = key.slice(-4);
                  state.browseType = item.type;
                  state.updateSearch(navigate, false);

               }} />
               <ArtifactButton 
                  disabled={!item.id}                  
                  artifact={item}
               />
            </Stack>
         </Stack>
      }
      return null;
   };

   return <Stack styles={{ root: { position: "relative", height: 590, marginRight: 8 } }}>
      <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always}>
         <DetailsList
            isHeaderVisible={false}
            styles={{
               root: {
                  paddingRight: 12
               },
               headerWrapper: {
                  paddingTop: 0
               }
            }}
            items={sorted}
            selectionMode={SelectionMode.single}
            layoutMode={DetailsListLayoutMode.justified}
            compact={false}
            onRenderRow={renderRow}
         />
      </ScrollablePane>
   </Stack>

}

const sortOptions: IComboBoxOption[] = [
   {
      key: `sort-name`,
      text: `Name`
   }, {
      key: `sort-change`,
      text: `Change`
   }
]

let streamChooserId = 0;

export const FindArtifactsModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const navigate = useNavigate();
   const searchState = useConst(new ArtifactQueryState(new URLSearchParams(window.location.search)));
   const [state, setState] = useState<{ searching?: boolean, artifacts?: GetArtifactResponse[], noResults?: boolean }>({});

   const { hordeClasses } = getHordeStyling();

   if (!searchState.streamId) {

      if (!searchState.streamId) {
         const streamId = getActiveStreamId();
         if (streamId) {
            searchState.streamId = streamId;
            searchState.updateSearch(navigate);
         }
      }
   }

   const artifactTypes: IComboBoxOption[] = [
      {
         key: `step-all`,
         text: `All`
      }
   ]

   dashboard.artifactTypes.sort((a, b) => a.localeCompare(b)).forEach(t => {
      artifactTypes.push({ key: t, text: t });
   });


   const queryArtifacts = async () => {

      setState({ ...state, searching: true });

      try {

         let minChange: number | undefined = parseInt(searchState.minChangeList?.trim() ?? "0");
         if (!minChange || isNaN(minChange)) {
            minChange = undefined;
         }

         let maxChange: number | undefined = parseInt(searchState.maxChangeList?.trim() ?? "0");
         if (!maxChange || isNaN(maxChange)) {
            maxChange = undefined;
         }

         let name: string | undefined = searchState.name?.trim();
         if (!name) {
            name = undefined;
         }

         let type: string | undefined = searchState.typeKey?.trim();

         if (type === "step-all") {
            type = undefined;
         } else if (type) {
            const existing = artifactTypes.find(t => t.key === type);
            if (!existing) {
               artifactTypes.push({ key: type, text: type });
            }
         }


         let streamId = searchState?.streamId?.trim();
         if (!streamId || streamId === streamIdAll) {
            streamId = undefined;
         }

         const mongoId = /^[a-fA-F0-9]{24}$/i;

         let id: string | undefined;

         if (name?.length) {
            if (name.match(mongoId)?.length) {
               id = name;
            }
         }

         let artifacts: GetArtifactResponse[] = [];

         if (id) {
            try {
               const artifact = await backend.getArtifactData(id);
               if (artifact?.id) {
                  artifacts = [artifact];
               }
            } catch (reason) {
               console.error(reason)
            }

         } else {
            const find = await backend.getArtifacts(streamId, minChange, maxChange, name, type);
            artifacts = find.artifacts;
         }


         streamChooserId++;
         setState({
            searching: false, artifacts: artifacts, noResults: !artifacts.length
         });

      } catch (reason) {
         console.error(reason);
         streamChooserId++;
         setState({ searching: false });
      }
   }

   const Searching = () => {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 320, height: 128, hasBeenOpened: false, top: "128px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack horizontalAlign="center">
            <Stack styles={{ root: { padding: 8, paddingBottom: 32 } }}>
               <Text variant="mediumPlus">Searching for Artifacts</Text>
            </Stack>
            <Stack horizontalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>
   }

   const filterTypes = [...artifactTypes].filter(t => !!t.text?.trim());

   let typeText = "";
   let key = searchState.typeKey ?? "step-all"
   const option = artifactTypes.find(o => o.key === key);
   if (option) {
      typeText = option.text;
   } else {
      typeText = searchState.typeKey ?? "";
   }

   if (searchState.autoLoad) {
      searchState.autoLoad = false;
      setTimeout(() => {
         queryArtifacts();
      })
   }

   return <Stack key="artifact_search_modal">
      <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1200, height: 820, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => {
         onClose()
      }} className={hordeClasses.modal}>
         {!!state.searching && <Searching />}
         <ArtifactBrowser state={searchState} />
         <Stack className="horde-no-darktheme" styles={{ root: { paddingTop: 10, paddingRight: 12 } }}>
            <Stack style={{ paddingLeft: 24, paddingRight: 12 }}>
               <Stack tokens={{ childrenGap: 12 }} style={{ height: 800 }}>
                  <Stack horizontal verticalAlign="start">
                     <Stack style={{ paddingTop: 3 }}>
                        <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Find Artifacts</Text>
                     </Stack>
                     <Stack grow />
                     <Stack horizontalAlign="end">
                        <IconButton
                           iconProps={{ iconName: 'Cancel' }}
                           onClick={() => {
                              onClose()
                           }}
                        />
                     </Stack>
                  </Stack>
                  <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 20 }}>
                     <Stack key={`stream_chooser_${streamChooserId}`}>
                        <Label>Stream</Label>
                        <StreamChooser defaultStreamId={searchState.streamId} allowAll={true} onChange={(streamId) => {
                           searchState.streamId = streamId;
                           setState({ ...state });
                        }} />
                     </Stack>
                     <Stack >
                        <TextField key={`min_change_option_${streamChooserId}`} defaultValue={searchState.minChangeList} value={searchState.minChangeList} autoComplete="off" spellCheck={false} style={{ width: 92 }} label="Min Changelist" onChange={(ev, newValue) => {
                           let change: number | undefined;
                           newValue = newValue ?? "";
                           change = parseInt(newValue);
                           if (isNaN(change)) {
                              change = undefined;
                           }

                           if (typeof (change) === "number") {
                              searchState.minChangeList = change.toString();
                           } else {
                              searchState.minChangeList = undefined;
                           }

                           setState({ ...state });
                        }} />
                     </Stack>
                     <Stack >
                        <TextField key={`max_change_option_${streamChooserId}`} defaultValue={searchState.maxChangeList} value={searchState.maxChangeList} autoComplete="off" spellCheck={false} style={{ width: 92 }} label="Max Changelist" onChange={(ev, newValue) => {
                           let change: number | undefined;
                           newValue = newValue ?? "";
                           change = parseInt(newValue);
                           if (isNaN(change)) {
                              change = undefined;
                           }

                           if (typeof (change) === "number") {
                              searchState.maxChangeList = change.toString();
                           } else {
                              searchState.maxChangeList = undefined;
                           }
                           setState({ ...state });

                        }} />
                     </Stack>
                     <Stack >
                        <TextField key={`name_option_${streamChooserId}`} defaultValue={searchState.name} value={searchState.name} style={{ width: 232 }} label="Name / Artifact Id" spellCheck={false} autoComplete="off" onChange={(ev, newValue) => {
                           searchState.name = newValue;
                           setState({ ...state });
                        }} />
                     </Stack>
                     <Stack>
                        <Label>Artifact Type</Label>
                        <ComboBox key={`type_option_${streamChooserId}`} allowFreeform={true} autoComplete="off" text={typeText} spellCheck={false} style={{ width: 144, textAlign: "left" }} selectedKey={searchState.typeKey ?? "step-all"} options={filterTypes} calloutProps={{ doNotLayer: true }} onChange={(event: React.FormEvent<IComboBox>, option?: IComboBoxOption, index?: number, value?: string) => {
                           if (option) {
                              searchState.typeKey = option.key as string;
                           } else if (value) {

                              searchState.typeKey = value;

                           }

                           setState({ ...state });


                        }} />
                     </Stack>
                     <Stack>
                        <Label>Sort By</Label>
                        <ComboBox key={`sort_option_${streamChooserId}`} style={{ width: 144, textAlign: "left" }} selectedKey={searchState.sort ?? "sort-name"} options={sortOptions} calloutProps={{ doNotLayer: true }} onChange={(event: React.FormEvent<IComboBox>, option?: IComboBoxOption, index?: number, value?: string) => {
                           if (option) {
                              searchState.sort = option.key as string;
                              setState({ ...state })
                           }
                        }} />
                     </Stack>
                  </Stack>
                  <Stack horizontal style={{ paddingTop: 12, paddingRight: 8 }}>
                     <Stack grow />
                     <Stack horizontal tokens={{ childrenGap: 18 }}>
                        <Stack>
                           <DefaultButton disabled={!!state.searching} text="Reset" onClick={() => {
                              streamChooserId++;
                              searchState.reset(navigate);
                              setState({ ...state })
                           }} />
                        </Stack>
                        <Stack>
                           <PrimaryButton disabled={!!state.searching} text="Find" onClick={() => {
                              searchState.updateSearch(navigate);
                              queryArtifacts()
                           }} />
                        </Stack>
                     </Stack>
                  </Stack>
                  <Stack key={`artifact_list_${streamChooserId}`} styles={{ root: { paddingTop: 12 } }}>
                     {!state.noResults && <ArtifactsList state={searchState} artifacts={state.artifacts} sortBy={searchState.sort} />}
                     {state.noResults && <Stack grow horizontalAlign="center"><Text variant="mediumPlus">No Results Found</Text></Stack>}
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   </Stack>;
};
