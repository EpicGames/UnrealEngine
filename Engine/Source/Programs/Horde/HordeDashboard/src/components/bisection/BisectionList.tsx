// Copyright Epic Games, Inc. All Rights Reserved.

import { Callout, DefaultButton, DetailsList, DetailsListLayoutMode, DirectionalHint, FontIcon, IDetailsListProps, IconButton, MessageBar, MessageBarType, Modal, PrimaryButton, SelectionMode, Stack, Text, mergeStyleSets } from '@fluentui/react';
import { useConst } from '@fluentui/react-hooks';
import * as d3 from "d3";
import { action, makeObservable, observable } from 'mobx';
import { observer } from 'mobx-react-lite';
import { default as React, useState } from 'react';
import { Link } from "react-router-dom";
import backend from "../../backend";
import { BisectTaskState, GetBisectTaskResponse, JobStepOutcome } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { projectStore } from '../../backend/ProjectStore';
import { getMongoIdDate, getShortNiceTime } from '../../base/utilities/timeUtils';
import { ChangeButton, JobParameters } from '../ChangeButton';
import { getHordeStyling } from '../../styles/Styles';

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

class Tooltip {

   constructor() {
      makeObservable(this);
   }

   subscribe() { if (this.updated) { } }

   @observable
   updated = 0

   @action
   update(bisection?: GetBisectTaskResponse, change?: number, x?: number, y?: number) {
      this.bisection = bisection;
      this.change = change ?? 0;
      this.x = x ?? 0;
      this.y = y ?? 0;
      this.updated++;
   }

   @action
   freeze(frozen: boolean) {
      this.frozen = frozen;
      this.updated++;
   }

   get step() {
      return this.bisection?.steps?.find((s => s.change === this.change));
   }

   bisection?: GetBisectTaskResponse;
   change: number = 0;
   x: number = 0;
   y: number = 0;
   frozen = false;
}


const GraphTooltip: React.FC<{ renderer: BisectionRenderer }> = observer(({ renderer }) => {

   const { modeColors } = getHordeStyling();

   const tooltip = renderer.tooltip;
   tooltip.subscribe();

   const bisection = tooltip.bisection!;
   if (!bisection) {
      return null;
   }

   let tipX = tooltip.x;
   let offsetX = 32;

   if (tipX > 1000) {
      offsetX = -32;
   }

   const titleWidth = 48;

   const dataElement = (title: string, value: string, linkText?: string, linkTarget?: string, valueElementIn?: JSX.Element) => {

      const textSize = 12;

      let valueElement = valueElementIn;

      if (!valueElement) {
         valueElement = <Stack>
            <Stack style={{ fontSize: textSize }}>
               {value}
            </Stack>
         </Stack>
      }

      if (linkText) {
         valueElement = <Link className="horde-link" to={linkText} target={linkTarget}>
            {valueElement}
         </Link>
      }


      let component = <Stack style={{ cursor: (linkText) ? "pointer" : undefined }} onClick={() => {
         /*
         if (agent) {
            setViewAgent(agent);
         }
         */
      }}>
         <Stack horizontal>
            <Stack style={{ width: titleWidth }}>
               {!!title && <Text style={{ fontFamily: "Horde Open Sans SemiBold", fontSize: textSize }} >{title}</Text>}
            </Stack>
            {valueElement}
         </Stack>
      </Stack>


      return component;
   }

   const elements: JSX.Element[] = [];
   const step = tooltip.step;
   if (step) {

      const jobParams: JobParameters = {
         id: step.jobId,
         change: tooltip.change,
         streamId: bisection.streamId
      }


      elements.push(dataElement("Step:", bisection.nodeName, `/job/${step.jobId}?step=${step.stepId}`));
      elements.push(dataElement("", "", undefined, undefined, <ChangeButton job={jobParams} hideAborted={true} />));
      if (step.logId) {
         elements.push(dataElement("", "View Log", `/log/${step.logId}`, "_blank"));
      }

   } else {

      if (tooltip.change === bisection.minChange) {

         const jobParams: JobParameters = {
            id: bisection.minJobId!,
            change: bisection.minChange!,
            streamId: bisection.streamId
         }

         elements.push(dataElement("Step:", bisection.nodeName, `/job/${bisection.minJobId!}?step=${bisection.minStepId!}`));
         elements.push(dataElement("", "", undefined, undefined, <ChangeButton job={jobParams} hideAborted={true} />));

      } else if (bisection.nextJobChange === tooltip.change) {

         const jobParams: JobParameters = {
            id: bisection.nextJobId,
            change: bisection.nextJobChange,
            streamId: bisection.streamId
         }

         elements.push(dataElement("Setup:", "View Job", `/job/${bisection.nextJobId}`, "_blank"));
         elements.push(dataElement("", "", undefined, undefined, <ChangeButton job={jobParams} hideAborted={true} />));
      } else {
         elements.push(dataElement("Task:", bisection.state));
      }

   }

   const target = `bisection_callout_target_${bisection.id}_${tooltip.change}`;


   return <div id={target} style={{
      position: "absolute",
      display: "block",
      top: `${tooltip.y}px`,
      left: `${tooltip.x + offsetX}px`,
      backgroundColor: modeColors.background,
      width: "max-content",
      borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F",
      pointerEvents: tooltip.frozen ? undefined : "none",
   }}>
      <Callout target={`#${target}`} isBeakVisible={false} gapSpace={24} directionalHint={DirectionalHint.bottomCenter}>
         <Stack horizontal>
            <Stack tokens={{ childrenGap: 6, padding: 12 }}>
               {elements}
            </Stack>
            {!!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12, paddingTop: 12, cursor: "pointer" }} onClick={() => { tooltip.freeze(false); tooltip.update() }}>
               <FontIcon iconName="Cancel" />
            </Stack>}
            {!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12 + 16, paddingTop: 12 }} />}
         </Stack>
      </Callout>
   </div>

})

type BisectionResult = {
   change: number
}


let id_counter = 0;
class BisectionRenderer {

   constructor() {

      this.margin = { top: 0, right: 32, bottom: 0, left: 32 };

   }

   render(container: HTMLDivElement) {

      if (!this.bisection) {
         return;
      }

      const width = 760;
      const height = 64;
      const margin = this.margin;
      const scolors = dashboard.getStatusColors();
      const bisection = this.bisection!;

      const changeSet = new Set<number>();
      const changeColors = new Map<number, string>();

      const maxCL = bisection.initialChange;
      let minCL = Math.min(bisection.currentChange, bisection.nextJobChange ?? Number.MAX_SAFE_INTEGER);
      bisection.steps?.forEach(s => minCL = Math.min(minCL, s.change));

      // bisection result
      let result: BisectionResult | undefined;
      if (bisection.state === BisectTaskState.Succeeded) {
         result = {
            change: bisection.currentChange
         }
      }

      const svgId = `suite_svg_${id_counter++}`;

      const xScale = d3.scaleLinear()
         .domain([minCL - 1, maxCL + 1])
         .range([margin.left, width - margin.right])

      changeColors.set(maxCL, bisection.outcome === JobStepOutcome.Failure ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Warnings)!);

      bisection.steps?.forEach(s => {

         changeColors.set(s.change, scolors.get(StatusColor.Running)!);
         if (s.outcome === JobStepOutcome.Failure) {
            changeColors.set(s.change, scolors.get(StatusColor.Failure)!);
         } else if (s.outcome === JobStepOutcome.Warnings) {
            changeColors.set(s.change, scolors.get(StatusColor.Warnings)!);
         } else if (s.outcome === JobStepOutcome.Success) {
            changeColors.set(s.change, scolors.get(StatusColor.Success)!);
         }
      });

      if (bisection.state === BisectTaskState.Running) {
         if (bisection.nextJobChange) {
            changeSet.add(bisection.nextJobChange);
            changeColors.set(bisection.nextJobChange, scolors.get(StatusColor.Running)!);
         }
      }

      changeSet.add(minCL);
      changeSet.add(maxCL);
      bisection.steps?.forEach(s => {
         changeSet.add(s.change);
      });

      // if we only have one change set, bisection is still initializing so don't show it
      if (!bisection.minChange) {
         changeSet.clear();
      }

      const X = _d3.map(Array.from(changeSet).sort((a, b) => a - b), (t) => t);
      const I = d3.range(X.length);

      let svg = this.svg;

      if (!svg) {
         svg = d3.select(container)
            .append("svg") as any as SelectionType;

         svg.attr("id", svgId)
            .attr("width", width + 48)
            .attr("height", height)
            .attr("viewBox", [0, 0, width, height] as any)

         this.svg = svg;
      } else {
         // remove tooltip
         d3.select(container).selectAll('div').remove();
         svg.selectAll("*").remove();
      }

      const g = svg.append("g");

      // Bisection Line

      const yoffset = 36;

      g.append("line")
         .attr("stroke", () => {
            return dashboard.darktheme ? "#6D6C6B" : "#4D4C4B";
         })
         .attr("stroke-width", 1)
         .attr("stroke-linecap", 4)
         .attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
         .attr("x1", () => this.margin.left)
         .attr("x2", () => width - this.margin.right)
         .attr("y1", () => yoffset)
         .attr("y2", () => yoffset)

      const radius = 3.5
      g.selectAll("circle")
         .data(I)
         .join("circle")
         .attr("id", i => `id_${bisection.id}_${X[i]}`)
         .attr("cx", i => {
            return xScale(X[i]);
         })
         .attr("cy", i => yoffset)
         .attr("fill", i => changeColors.get(X[i])!)
         .attr("r", i => X[i] === result?.change ? radius * 1.5 : radius);

      if (result) {
         svg.append("line")
            .attr("class", "cline")
            .attr("x1", () => xScale(result!.change))
            .attr("x2", () => xScale(result!.change))
            .attr("y1", () => 8)
            .attr("y2", () => 68)
            .attr("stroke-width", () => 2)
            .style("stroke-dasharray", "2, 2")
            //.attr("marker-end", "url(#cmarker)")
            .attr("stroke", bisection.outcome === JobStepOutcome.Failure ? scolors.get(StatusColor.Failure)! : scolors.get(StatusColor.Warnings)!)
      }

      const closestCL = (x: number, y: number): number | undefined => {

         if (!changeSet.size) {
            return undefined;
         }

         const changes = Array.from(changeSet);

         return Array.from(changeSet).reduce<number>((bestCL: number, change: number) => {

            const bs = Math.abs(xScale(bestCL)! - x);
            const cs = Math.abs(xScale(change)! - x);

            if (cs > bs) {
               return bestCL;
            }

            return change;

         }, changes[0]);
      }

      const handleMouseMove = (event: any) => {

         if (this.tooltip.frozen) {
            return;
         }

         let mouseX = _d3.pointer(event)[0];
         let mouseY = _d3.pointer(event)[1];
         const change = closestCL(mouseX, mouseY);
         this.tooltip.update(this.bisection, change, Math.floor(_d3.pointer(event, container)[0]), Math.floor(_d3.pointer(event, container)[1]));
      }

      const handleMouseLeave = (event: any) => {
         if (!this.tooltip.frozen) {
            this.tooltip.update();
         }
      }

      const handleMouseClick = (event: any) => {

         if (this.tooltip.frozen) {
            this.tooltip.freeze(false);
            return;
         }

         if (this.tooltip.change) {
            this.tooltip.freeze(true);
         }

      }

      svg.on("click", (event) => handleMouseClick(event))
      svg.on("mousemove", (event) => handleMouseMove(event));
      svg.on("mouseleave", (event) => { handleMouseLeave(event); })

   }

   bisection?: GetBisectTaskResponse;
   margin: { top: number, right: number, bottom: number, left: number }
   svg?: SelectionType;
   tooltip = new Tooltip();
}

const BisectionGraph: React.FC<{ bisection: GetBisectTaskResponse }> = ({ bisection }) => {

   const { hordeClasses } = getHordeStyling();

   const graph_container_id = `timeline_graph_container`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const graph = useConst(new BisectionRenderer());
   graph.bisection = bisection;

   if (container) {
      try {
         graph.render(container);

      } catch (err) {
         console.error(err);
      }
   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8 }}>
         <div style={{ position: "relative" }}>
            <GraphTooltip renderer={graph} />
         </div>
         <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </Stack>;
}

export const BisectionList: React.FC<{ bisections?: GetBisectTaskResponse[] }> = observer(({ bisections }) => {

   const [showCancelModal, setShowCancelModal] = useState("");

   dashboard.subscribe();

   if (!bisections?.length) {
      return null;
   }

   type BisectionItem = {
      bisection: GetBisectTaskResponse;
   }

   const items: BisectionItem[] = [];

   bisections.sort((a, b) => {
      return getMongoIdDate(b.id)!.getTime() - getMongoIdDate(a.id)!.getTime();
      
   }).forEach((b) => {

      items.push({
         bisection: b
      })

   })

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (!props) {
         return null;
      }

      const item = items[props.itemIndex];
      const bisection = item?.bisection;

      if (!bisection) {
         return null;
      }

      const streamId = bisection.streamId;
      const templateId = bisection.templateId;

      const stream = projectStore.streamById(streamId);
      const template = stream?.templates?.find(t => t.id === templateId);
      const nodeName = bisection.nodeName;

      const colors = dashboard.getStatusColors();
      let statusColor = "#FF0000";
      if (bisection.state === BisectTaskState.Running) {
         statusColor = colors.get(StatusColor.Running)!;
      } else if (bisection.state === BisectTaskState.Succeeded) {
         statusColor = colors.get(StatusColor.Success)!;
      } else if (bisection.state === BisectTaskState.Cancelled) {
         statusColor = colors.get(StatusColor.Skipped)!;
      } else {
         statusColor = colors.get(StatusColor.Failure)!;
      }

      const timestamp = bisection.id.substring(0, 8)
      const time = getShortNiceTime(new Date(parseInt(timestamp, 16) * 1000), true);

      const running = bisection.state === BisectTaskState.Running;
      const succeeded = bisection.state === BisectTaskState.Succeeded;
      const statusWidth = 64;

      return <Stack /*style={{ cursor: "pointer" }} onClick={() => { setShowBisectionModal(bisection.id) }}*/>
         <Stack style={{ backgroundColor: props.itemIndex % 2 ? undefined : "#E9E8E7", height: 96, padding: "12px 18px" }}>
            <Stack verticalAlign='center' verticalFill>
               <Stack horizontal style={{ paddingTop: 8, paddingBottom: 8 }}>
                  <Stack style={{ width: 280 }} horizontal verticalAlign='center' verticalFill tokens={{ childrenGap: 12 }}>
                     <Stack className="horde-no-darktheme">
                        <FontIcon style={{ color: statusColor }} iconName={"Square"} />
                     </Stack>
                     <Stack>
                        <Stack tokens={{ childrenGap: 8 }}>
                           <Stack>
                              <Text style={{ fontWeight: 600 }}>{nodeName}</Text>
                           </Stack>
                           <Stack>
                              <Text variant='small'>{stream?.fullname} - {template?.name}</Text>
                           </Stack>
                           <Stack>
                              {running && <Stack horizontal tokens={{ childrenGap: 8 }}>
                                 <Text style={{ fontWeight: 600, width: statusWidth }}>{!bisection.minChange ? "Starting:" : "Running:"}</Text>
                                 <DefaultButton style={{ height: "20px", fontSize: "10px", width: "80px" }} text="Cancel" onClick={() => setShowCancelModal(bisection.id)} />
                              </Stack>}
                              {succeeded && <Stack horizontal tokens={{ childrenGap: 8 }}>
                                 <Stack>
                                    <Text style={{ fontWeight: 600, width: statusWidth }}>Suspect:</Text>
                                 </Stack>
                                 <Stack>
                                    <ChangeButton job={{ streamId: bisection.streamId, id: bisection.currentJobId, change: bisection.currentChange }} />
                                 </Stack>
                              </Stack>}
                              {!succeeded && !running && <Text style={{ fontWeight: 600, width: statusWidth }}>{bisection.state}</Text>}
                           </Stack>
                        </Stack>

                     </Stack>
                  </Stack>
                  <Stack>
                     <BisectionGraph bisection={bisection} />
                  </Stack>
                  <Stack grow />
                  <Stack horizontal tokens={{ childrenGap: 12 }}>
                     <Stack verticalAlign='center' horizontalAlign="center" verticalFill tokens={{ childrenGap: 4 }}>
                        {!!bisection.owner.name && <Text variant='small'>{bisection.owner.name}</Text>}
                        <Text variant='small'>{time}</Text>
                     </Stack>
                     {dashboard.bisectPinned(bisection.id) && <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"} onClick={(ev) => {
                        ev.preventDefault();
                        ev.stopPropagation();
                        dashboard.unpinBisect(bisection.id);
                     }}>
                        <IconButton iconProps={{ iconName: 'Unpin' }} />
                     </Stack>}
                     {!dashboard.bisectPinned(bisection.id) && <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"end"} onClick={(ev) => {
                        ev.preventDefault();
                        ev.stopPropagation();
                        dashboard.pinBisect(bisection.id);
                     }}>
                        <IconButton iconProps={{ iconName: 'Pin' }} style={{ color: dashboard.getStatusColors().get(StatusColor.Skipped!) }} />
                     </Stack>}

                  </Stack>

               </Stack>
            </Stack>
         </Stack>
      </Stack >
   }

   return <Stack>
      {!!showCancelModal && <CancelBisectionModal bisectTaskId={showCancelModal} onClose={(canceled) => { if (canceled) { /*window.location.reload();*/ setShowCancelModal("") } }} />}
      <DetailsList
         styles={{ root: { overflow: "hidden" } }}
         className={customStyles.details}
         compact={true}
         isHeaderVisible={false}
         indentWidth={0}
         items={items}
         setKey="set"
         selectionMode={SelectionMode.none}
         layoutMode={DetailsListLayoutMode.justified}
         onRenderRow={renderRow}
      />
   </Stack>;
});


const customStyles = mergeStyleSets({
   details: {
      selectors: {
         '.ms-GroupHeader,.ms-GroupHeader:hover': {
            background: "#DFDEDD",
         },
         '.ms-GroupHeader-title': {
            cursor: "default"
         },
         '.ms-GroupHeader-expand,.ms-GroupHeader-expand:hover': {
            background: "#DFDEDD"
         },
         '.ms-DetailsRow': {
            animation: "none",
            background: "unset"
         },
         '.ms-DetailsRow:hover': {
            background: "#F3F2F1"
         }
      },
   }
});


const CancelBisectionModal: React.FC<{ bisectTaskId: string, onClose: (canceled: boolean) => void }> = ({ bisectTaskId, onClose }) => {

   const [state, setState] = useState<{ submitting?: boolean, error?: string }>({});

   const { hordeClasses } = getHordeStyling();

   const onCancelTask = async () => {

      setState({ ...state, submitting: true });

      try {

         await backend.updateBisectTask(bisectTaskId, {
            cancel: true
         });

         setState({ ...state, submitting: false, error: "" });
         onClose(true);

      } catch (reason) {

         setState({ ...state, submitting: false, error: reason as string });
      }

   };

   const height = state.error ? 200 : 180;

   return <Modal className={hordeClasses.modal} isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height, hasBeenOpened: false, top: "24px", position: "absolute" } }} onDismiss={() => { onClose(false) }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">Cancel Bisection</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(false); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { paddingLeft: 8, width: 540, paddingTop: 12 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            {!!state.error && <MessageBar
               messageBarType={MessageBarType.error}
               isMultiline={false}> {state.error} </MessageBar>}

         </Stack>
      </Stack>

      <Stack tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack horizontal>
            <Stack>
            </Stack>
            <Stack grow />
            <Stack>
               <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingRight: 24 }}>
                  <PrimaryButton text="Cancel Task" disabled={state.submitting ?? false} onClick={() => { onCancelTask() }} />
                  <DefaultButton text="Go Back" disabled={state.submitting} onClick={() => { onClose(false); }} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>;

};