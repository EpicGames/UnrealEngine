// Copyright Epic Games, Inc. All Rights Reserved.

import { Modal, Stack, Text, IconButton, TextField, PrimaryButton, DefaultButton, MessageBar, MessageBarType } from "@fluentui/react";
import { useState } from "react";
import backend from "../../backend";
import { CommitTag, CreateBisectTaskResponse } from "../../backend/Api";
import { getHordeStyling } from "../../styles/Styles";

export const BisectionCreateModal: React.FC<{ jobId: string, nodeName: string, onClose: (response?: CreateBisectTaskResponse) => void }> = ({ jobId, nodeName, onClose }) => {

   const [state, setState] = useState<{ submitting?: boolean, errorMsg?: string, commitTags?: string, ignoreChanges?: string, ignoreJobs?: string }>({});

   const { hordeClasses } = getHordeStyling();

   const onBisect = async () => {

      setState({ ...state, submitting: true });

      try {

         let commitTags: CommitTag[] | undefined;
         let ignoreChanges: number[] | undefined;
         let ignoreJobs: string[] | undefined;

         if (state.commitTags) {
            commitTags = state.commitTags.split(',').map(s => { return { text: s.trim() } }).filter(s => !!s.text)
         }

         if (state.ignoreChanges) {
            ignoreChanges = state.ignoreChanges.split(',').map(s => { return parseInt(s.trim()) }).filter(s => !isNaN(s));
         }

         if (state.ignoreJobs) {
            ignoreJobs = state.ignoreJobs.split(',').map(s => { return s.trim() }).filter(s => !!s.length);
         }

         const request = {
            jobId: jobId,
            nodeName: nodeName,
            commitTags: commitTags,
            ignoreChanges: ignoreChanges,
            ignoreJobs: ignoreJobs
         };

         const response = await backend.createBisectTask(request);
         console.log(response);

         setState({ ...state, submitting: false, errorMsg: "" });
         onClose(response);

      } catch (reason) {

         setState({ ...state, submitting: false, errorMsg: reason as string });
      }

   };


   return <Modal isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 640, height: '570px', backgroundColor: '#FFFFFF', hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
      <Stack styles={{ root: { paddingTop: 8, paddingLeft: 24, paddingRight: 12, paddingBottom: 8 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal styles={{ root: { padding: 0 } }}>
               <Stack style={{ paddingLeft: 0, paddingTop: 4 }} grow>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Step Bisection (Experimental)</Text>
               </Stack>
               <Stack grow horizontalAlign="end">
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     onClick={() => { onClose(); }}
                  />
               </Stack>
            </Stack>
            <Stack style={{ paddingRight: 12 }} tokens={{ childrenGap: 12 }}>
               <Stack>
                  {!!state.errorMsg && <MessageBar
                     messageBarType={MessageBarType.error}
                     isMultiline={false}> {state.errorMsg}
                  </MessageBar>}
               </Stack>
               <Stack>
                  <TextField label="JobId" value={jobId} disabled={true} />
               </Stack>
               <Stack>
                  <TextField label="Node Name" value={nodeName} disabled={true} />
               </Stack>
               <Stack>
                  <TextField label="Commit Tags" multiline spellCheck={false} placeholder="Comma delimited list of commit tags" onChange={(event, newValue) => setState({ ...state, commitTags: newValue })} />
               </Stack>
               <Stack>
                  <TextField label="Ignore Changes" multiline spellCheck={false} placeholder="Comma delimited list of changes to ignore" onChange={(event, newValue) => setState({ ...state, ignoreChanges: newValue })} />
               </Stack>
               <Stack>
                  <TextField label="Ignore Jobs" multiline spellCheck={false} placeholder="Comma delimited list of job ids to ignore" onChange={(event, newValue) => setState({ ...state, ignoreJobs: newValue })} />
               </Stack>
            </Stack>
            <Stack>
               <Stack horizontal>
                  <Stack grow />
                  <Stack horizontal style={{ paddingTop: 24 }}>
                     <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                        <PrimaryButton disabled={state.submitting} text="Bisect" onClick={() => { onBisect() }} />
                     </Stack>
                     <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                        <DefaultButton disabled={state.submitting} text="Cancel" onClick={() => { onClose() }} />
                     </Stack>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Modal>
}