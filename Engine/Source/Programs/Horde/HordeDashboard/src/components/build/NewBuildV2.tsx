// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, ComboBox, ContextualMenuItemType, DefaultButton, DirectionalHint, Dropdown, DropdownMenuItemType, IComboBoxOption, IContextualMenuItem, IContextualMenuProps, IDropdownOption, ITag, Icon, IconButton, Label, MessageBar, MessageBarType, Modal, Pivot, PivotItem, PrimaryButton, ScrollablePane, ScrollbarVisibility, Spinner, SpinnerSize, Stack, TagPicker, Text, TextField, TooltipHost, ValidationState } from "@fluentui/react";
import { ITextField } from "@fluentui/react/lib-commonjs/TextField";
import Markdown from "markdown-to-jsx";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import React, { useEffect, useState } from "react";
import backend, { useBackend } from "../../backend";
import { BoolParameterData, ChangeQueryConfig, CreateJobRequest, GetJobsTabResponse, GetTemplateRefResponse, JobsTabData, ListParameterData, ListParameterItemData, ListParameterStyle, ParameterData, ParameterType, Priority, StreamData, TabType, TextParameterData } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { ProjectStore } from "../../backend/ProjectStore";
import templateCache from "../../backend/TemplateCache";
import { copyToClipboard, copyToClipboardAsync } from "../../base/utilities/clipboard";
import { getHordeStyling } from "../../styles/Styles";
import { errorDialogStore } from "../error/ErrorStore";
import { NewBuild } from "../NewBuild";
import { JobDetailsV2 } from "../jobDetailsV2/JobDetailsViewCommon";

type ValidationError = {
   paramString: string;
   error: string;
   value: string;
   context?: string;
   param?: TextParameterData;
};

const parameterWidth = 667;
const parameterGap = 6;

const LONG_LIST_PARAMETER_THRESHOLD = 50; // List parameters with more than this many items will become a `LongListParamater` automatically

type NewBuildMode = "Basic" | "Advanced";

type OptionsConstruction = {
   streamId: string;
   projectStore: ProjectStore;
   onClose: (newJobId: string | undefined) => void;
   jobDetails?: JobDetailsV2;
   jobKey?: string;
   readOnly?: boolean;
   aliasTemplates?: string[];
}

class BuildOptions {

   constructor(streamId: string, projectStore: ProjectStore, onClose: (newJobId: string | undefined) => void, jobDetails?: JobDetailsV2, jobKey?: string, readOnly?: boolean, aliasTemplates?: string[]) {

      if (BuildOptions.instance) {
         // can happen with hot reload
         //throw `Failed to clean up build options`;
      }

      BuildOptions.instance = this;
      makeObservable(this);

      this.construction = {
         streamId: streamId,
         projectStore: projectStore,
         onClose: onClose,
         jobDetails: jobDetails,
         jobKey: jobKey,
         readOnly: readOnly,
         aliasTemplates: aliasTemplates
      }

      this.init();

   }

   getQuotedArg(arg: string) {
      // Ignore args that don't follow "-set:XXX=YYY"
      // Also prevents overquoting incorrectly recorded args 
      // i.e., forgetting "" in -set:XXX=YYY ZZZ AAA BBB => -set:XXX="YYY" "ZZZ" "AAA" "BBB"
      if (!arg.startsWith("-set")) return arg;

      // Replace any existing quotes with escaped quotes
      arg = arg.replace(/"/g, '\\"');

      // Split argument on space to extract left side assignment
      const splitOnSpace = arg.split(" ");

      // Ignore args without spaces
      if (splitOnSpace.length === 1) return arg;

      // Split assignment on equals, getting the last = sign. Which equals sign doesn't necessarily matter.
      // It looks best/is more semantic to do the inner most one, though.
      const splitOnEquals = splitOnSpace[0].split("=");

      // Add double quote just before value
      splitOnEquals[splitOnEquals.length - 1] = `"${splitOnEquals[splitOnEquals.length - 1]}`;

      // Replace splitOnSpace[0], Restore string, and add " to end
      splitOnSpace[0] = splitOnEquals.join("=");
      let finalString = splitOnSpace.join(" ") + '"';

      // -set:XXX=YYY=AAA BBB CCC --> -set:XXX=YYY="AAA BBB CCC"
      return finalString;
   }

   init() {

      this.change = undefined;
      this.changeOption = undefined;
      this.changeCommitId = undefined;
      this.preflightChange = undefined;
      this.preflightCommitId = undefined;
      this.queryShelvedChange = undefined;
      this.queryShelvedCommitId = undefined;
      this.autoSubmit = undefined;
      this.advJobName = undefined;
      this.updateIssues = undefined;
      this.advJobPriority = undefined;
      this.advAdditionalArgs = undefined;
      this.advTargets = undefined;
      this.template = undefined;
      this.parameters = {};
      this.disabledParameters = new Set();

      const streamId = this.construction.streamId;
      const projectStore = this.construction.projectStore;
      const onClose = this.construction.onClose;
      const jobDetails = this.construction.jobDetails;
      const jobKey = this.construction.jobKey;
      const readOnly = this.construction.readOnly;

      const query = new URLSearchParams(window.location.search);

      this.isPreflightSubmit = !!query.get("shelvedchange");
      this.isFromP4V = !!query.get("p4v");
      this.queryTemplateId = !query.get("templateId") ? "" : query.get("templateId")!;
      const queryShelvedParam = query.get("shelvedchange") ?? "";
      if (queryShelvedParam) {
         const parsed = parseInt(queryShelvedParam);
         if (!isNaN(parsed) && parsed) {
            this.queryShelvedChange = parsed;
         } else if (/^[a-f0-9]+$/i.test(queryShelvedParam)) {
            this.queryShelvedCommitId = queryShelvedParam.trim().toLowerCase();
         }
      }

      this.preflightChange = jobDetails?.jobData?.preflightChange;
      this.preflightCommitId = jobDetails?.jobData?.preflightCommitId;
      this.change = jobDetails?.jobData?.change;

      if (!this.preflightChange && !this.preflightCommitId) {
         this.preflightChange = this.queryShelvedChange;
         this.preflightCommitId = this.queryShelvedCommitId;
      }

      this.advJobPriority = jobDetails?.jobData?.priority;

      if (query.get("autosubmit") === "true") {
         this.autoSubmit = true;
      }

      this.streamId = streamId;
      this.onClose = onClose;
      this.readOnly = readOnly ?? false;
      this.projectStore = projectStore;
      this.jobKey = jobKey;
      this.jobDetails = jobDetails;
      this.updateIssues = jobDetails?.jobData?.updateIssues;

      this.showAllTemplates = jobKey === "all" || jobKey === "summary"

      const stream = projectStore.streamById(streamId);
      if (!stream) {
         throw `Unable to get stream ${streamId}`;
      }

      this.stream = stream;

      this.advAdditionalArgs = jobDetails?.jobData?.additionalArguments?.map(this.getQuotedArg).join(" ");

      this.load();

   }

   construction: OptionsConstruction;

   onClose: (newJobId?: string) => void;

   loaded = false;

   jobKey?: string;
   jobDetails?: JobDetailsV2;
   readOnly: boolean;
   currentRenderKey = 1;

   isPreflightSubmit?: boolean;
   isFromP4V?: boolean;
   queryTemplateId?: string;

   submitting?: boolean;

   mode: NewBuildMode = "Basic";

   // advanced options
   advJobName?: string;
   advJobPriority?: Priority;
   advAdditionalArgs?: string;
   advTargets?: string[];

   updateIssues?: boolean;
   runAsScheduler?: boolean;

   parameters: Record<string, string> = {};
   disabledParameters: Set<string> = new Set();

   validationErrors: ValidationError[] = [];

   get defaultPreflightQuery(): ChangeQueryConfig[] | undefined {

      const template = this.template;
      let defaultPreflightQuery: ChangeQueryConfig[] | undefined;


      if (template?.defaultChange) {
         defaultPreflightQuery = template.defaultChange;
      } else {
         const defaultChange = this.stream?.defaultPreflight?.change;

         if (defaultChange) {
            if (!defaultChange.name && defaultChange.templateId) {
               const defaultTemplate = this.allTemplates.find(t => t.id === defaultChange.templateId);
               if (defaultTemplate) {
                  defaultChange.name = "Latest Success - " + defaultTemplate.name;
               }
            }

            if (defaultChange.name) {
               defaultPreflightQuery = [defaultChange];
            }
         }
      }

      const defaultStreamPreflightTemplate = this.stream.templates.find(t => t.id === this.stream.defaultPreflight?.templateId);

      if (!defaultPreflightQuery && this.stream.defaultPreflight?.templateId) {

         if (defaultStreamPreflightTemplate) {
            defaultPreflightQuery = defaultStreamPreflightTemplate.defaultChange;
         }

         if (!defaultPreflightQuery) {
            console.error(`Unable to find default stream preflight template ${this.stream.defaultPreflight?.templateId} in stream templates`);
         }
      }

      return defaultPreflightQuery;

   }

   get advancedModified(): boolean {

      return !!((this.advJobPriority && this.advJobPriority !== Priority.Normal) || !!this.advJobName || !!this.advAdditionalArgs || !!this.advTargets)
   }

   change?: number;
   changeOption?: string;
   changeCommitId?: string;
   preflightChange?: number;
   preflightCommitId?: string;
   queryShelvedChange?: number;
   queryShelvedCommitId?: string;
   autoSubmit?: boolean;

   static initialized() {
      return !!BuildOptions.instance;
   }

   static get() {
      if (!BuildOptions.instance) {
         throw `Build Options have not been instantiated`;
      }
      return BuildOptions.instance!;
   }

   template?: GetTemplateRefResponse;
   templates: GetTemplateRefResponse[] = [];

   clear() {
      BuildOptions.instance = undefined;
   }

   streamId: string;
   stream: StreamData;
   projectStore: ProjectStore;

   preview: boolean = false;
   previewSource?: string;
   previewRestoreId?: string;

   showAllTemplates = false;

   setSubmitting(submitting?: boolean) {
      if (this.submitting === submitting) {
         return;
      }

      this.submitting = submitting;
      this.setChanged();

   }

   onAutoSubmitChanged(value: boolean) {
      if (this.autoSubmit === value) {
         return;
      }

      this.autoSubmit = value;
      this.setChanged();
   }

   onPreviewChanged(preview: boolean) {
      if (this.preview == preview) {
         return;
      }

      if (preview) {
         this.previewRestoreId = this.template?.id;
      } else {
         const template = this.allTemplates.find(t => t.id === this.previewRestoreId);
         this.template = undefined;
         if (template) {
            this.setTemplate(template.id);
         } else {
            this.selectDefaultTemplate();
         }
      }

      this.preview = preview;

      this.setChanged();
   }

   onPreviewSourceChanged(source: string) {
      this.previewSource = source;
      this.setPreviewTemplate();
   }

   onModeChanged(mode: NewBuildMode) {
      if (mode === this.mode) {
         return;
      }

      this.mode = mode;
      this.setChanged();
   }

   onPreflightChange(preflightCL?: number) {

      if (this.preflightChange === preflightCL) {
         return;
      }

      this.preflightChange = preflightCL;
      if (preflightCL) {
         this.preflightCommitId = undefined;
      }

      this.setChanged();
   }

   onPreflightCommitIdChange(commitId?: string) {
      if (this.preflightCommitId === commitId) {
         return;
      }
      this.preflightCommitId = commitId;
      if (commitId) {
         this.preflightChange = undefined;
      }

      this.setChanged();
   }

   onTemplateChange(templateId: string) {
      this.setTemplate(templateId, true);
   }

   onBooleanChanged(id: string, value: boolean) {
      this.parameters[id] = value.toString();
      this.setParameterChanged();
   }

   onTextChanged(id: string, value: string) {
      this.parameters[id] = value;
      this.setParameterChanged();
   }

   onListItemChanged(id: string, value: boolean) {
      this.parameters[id] = value.toString();
   }

   onListItemsUpdated() {
      this.setParameterChanged();
   }

   templateIdOverride?: string;

   onResetToDefaults() {

      this.currentRenderKey++;

      const template = this.template;

      if (!template) {
         return;
      }

      this.templateIdOverride = template.id;
      this.init();
   }

   setValidationErrors(errors: ValidationError[]) {
      this.validationErrors = errors;
      this.setChanged();
   }

   subscribe() {
      if (this.changed) { }
   }

   // so every parameter change
   subscribeToParameterChange() {
      if (this.parameterChanged) { }
   }

   setShowAllTemplates(value: boolean) {
      if (this.showAllTemplates === value) {
         return;
      }
      this.showAllTemplates = value;
      this.filterTemplates();
      this.setChanged();
   }

   @observable
   private changed = 0;

   @action
   setChanged() {
      this.changed++;
   }

   @observable
   private parameterChanged = 0;

   @action
   private setParameterChanged() {
      this.parameterChanged++;
   }

   validateParameter(p: ParameterData, errors: ValidationError[]): void {

      if (p.type === ParameterType.Text) {

         const param = p as TextParameterData;

         if (!param.validation) {
            return;
         }

         let value = this.parameters[param.id] ?? param.default;

         let regex: RegExp | undefined;

         try {
            regex = new RegExp(param.validation, 'gm');
         } catch (reason: any) {

            errors.push({
               param: param,
               paramString: param.label,
               value: value,
               error: `Invalid validation regex: ${param.validation}`,
               context: reason.toString()
            });

            return;
         }

         const match = value.match(regex);

         if (!match || !match.length || match[0]?.trim() !== value.trim()) {
            errors.push({
               param: param,
               paramString: param.label,
               value: value,
               error: `Did not match regex: ${param.validation}`,
               context: param.validationError ?? param.hint
            });
            return;
         }
      }
   }


   validate(errors: ValidationError[]): boolean {
      if (!this.template) {
         return true;
      }

      if (this.advTargets && !this.advTargets?.length) {
         errors.push({ error: `No targets specified`, paramString: "Target", value: "" })
      }

      if (this.preflightChange && !this.template.allowPreflights) {
         errors.push({ error: `Template "${this.template.name}" does not allow preflights`, paramString: "Preflight Change", value: this.preflightChange.toString() })
      }

      if (this.change !== undefined && isNaN(this.change)) {
         errors.push({
            paramString: "Change",
            value: this.change.toString(),
            error: "Invalid changelist"
         });
      }

      if (this.preflightChange !== undefined && isNaN(this.preflightChange)) {
         errors.push({
            paramString: "Preflight Change",
            value: this.preflightChange.toString(),
            error: "Invalid shelved changelist"
         });
      }

      if (this.preflightCommitId && !this.template.allowPreflights) {
         errors.push({ error: `Template "${this.template.name}" does not allow preflights`, paramString: "Preflight Change", value: this.preflightCommitId })
      }

      this.template.parameters.forEach(p => {

         this.validateParameter(p, errors);
      });

      if (errors.length) {
         this.setChanged();
      }

      this.validationErrors = errors;

      return !errors.length;

   }

   estimateHeight(previewPanel?: boolean): number {

      const template = this.template;
      if (!template) {
         return 0;
      }
      let estimatedHeight = parameterGap * template.parameters.length;

      const estimateHeight = (param: ParameterData) => {

         switch (param.type) {

            case ParameterType.List:
               estimatedHeight += 54;
               break;
            case ParameterType.Bool:
               estimatedHeight += 20;
               break;
            case ParameterType.Text:
               estimatedHeight += 54;
               break;
         }

      }

      if (this.mode === "Basic") {
         template.parameters.forEach(p => {
            estimateHeight(p);
         });
      }

      if (!!this.preflightChange) {
         estimatedHeight += 32;
      }

      if (this.preview && estimatedHeight < 600) {
         estimatedHeight = 600
      }

      if (estimatedHeight > 600) {
         estimatedHeight = 600
      }

      if (this.preview) {
         if (!previewPanel) {
            if (this.mode === "Basic") {
               estimatedHeight -= 64;
            } else {
               estimatedHeight -= 22;
            }
         }

      } else if (this.mode === "Advanced") {
         if (estimatedHeight < 280) {
            estimatedHeight = 280
         }
      }


      return estimatedHeight;

   }

   private setTemplateParameters() {

      const template = this.template;
      if (!template) {
         return;
      }

      this.parameters = {};

      template.parameters.forEach(param => {

         switch (param.type) {
            case ParameterType.Bool:
               const b = param as BoolParameterData;
               this.parameters[b.id] = this.jobDetails?.jobData?.parameters[b.id] ?? b.default.toString();
               break;
            case ParameterType.Text:
               const t = param as TextParameterData;
               this.parameters[t.id] = (this.jobDetails?.jobData?.parameters[t.id] ?? t.default) ?? "";
               break;
            case ParameterType.List:
               const list = param as ListParameterData;
               list.items.forEach(item => {
                  this.parameters[item.id] = (this.jobDetails?.jobData?.parameters[item.id] ?? item.default.toString()) ?? "false";
               });
               break;
            default:
               console.error(`Unknown parameter type: ${param.type}`)
         }
      });

      if (this.jobDetails?.jobData?.targets?.length) {
         this.setTargets(this.jobDetails?.jobData?.targets);
      }

   }

   private setPreviewTemplate() {
      const source = this.previewSource ?? ""
      const template = JSON.parse(source) as GetTemplateRefResponse;
      this.template = template;

      this.setTemplateParameters();

      this.setChanged();

   }

   setTargets(targets: string[]) {
      if (!this.advTargets) {
         this.initOverrideTargets();
      }
      this.advTargets = targets.sort((a, b) => a.localeCompare(b));
      this.setChanged();
   }

   addTarget(target: string) {
      if (!this.advTargets) {
         this.initOverrideTargets();
         this.advTargets = this.targets;
      }
      this.advTargets.push(target);
      this.advTargets = Array.from(new Set(this.advTargets)).sort((a, b) => a.localeCompare(b));

      this.setChanged();
   }

   get overrideTargets(): boolean {
      return !!this.advTargets;
   }

   get targets(): string[] {

      if (this.advTargets) {
         return this.advTargets
      }

      const template = this.template;

      if (!template) {
         return [];
      }

      let targets: Set<string> = new Set();

      template?.arguments.forEach(a => {
         if (a.toLowerCase().startsWith("-target=")) {
            targets.add(a.slice(8).trim())
         }
      });

      template.parameters.forEach(p => {

         let allArgs: string[] = [];

         switch (p.type) {
            case ParameterType.Bool:
               const b = p as BoolParameterData;
               if (this.parameters[b.id]?.toLowerCase() === "true") {
                  if (b.argumentIfEnabled) {
                     allArgs.push(b.argumentIfEnabled);
                  }
                  else if (b.argumentsIfEnabled?.length) {
                     allArgs.push(...b.argumentsIfEnabled);
                  }
               } else {
                  if (b.argumentIfDisabled) {
                     allArgs.push(b.argumentIfDisabled);
                  }
                  else if (b.argumentsIfDisabled?.length) {
                     allArgs.push(...b.argumentsIfDisabled);
                  }
               }

               allArgs.filter(a => a.toLowerCase().startsWith("-target=")).forEach(t => {
                  targets.add(t.slice(8).trim())
               })

               break;
            case ParameterType.Text:
               const t = p as TextParameterData;
               if (t.argument?.toLowerCase().startsWith("-target=")) {
                  if (this.parameters[t.id]?.length) {
                     targets.add(this.parameters[t.id]);
                  }
               }
               break;
            case ParameterType.List:
               const list = p as ListParameterData;
               list.items.forEach(i => {
                  allArgs = [];

                  if (this.parameters[i.id]?.toLowerCase() === "true") {
                     if (i.argumentIfEnabled) {
                        allArgs.push(i.argumentIfEnabled);
                     }
                     else if (i.argumentsIfEnabled?.length) {
                        allArgs.push(...i.argumentsIfEnabled);
                     }
                  } else {
                     if (i.argumentIfDisabled) {
                        allArgs.push(i.argumentIfDisabled);
                     }
                     else if (i.argumentsIfDisabled?.length) {
                        allArgs.push(...i.argumentsIfDisabled);
                     }
                  }

                  allArgs.filter(a => a.toLowerCase().startsWith("-target=")).forEach(t => {
                     targets.add(t.slice(8).trim())
                  })

               });
               break;
         }
      })

      return Array.from(targets).filter(t => !!t).sort((a, b) => a.localeCompare(b));
   }


   private initOverrideTargets() {

      const template = this.template;

      if (!template) {
         return;
      }

      template.parameters.forEach(p => {

         let allArgs: string[] = [];

         switch (p.type) {
            case ParameterType.Bool:
               const b = p as BoolParameterData;

               if (b.argumentIfEnabled) {
                  allArgs.push(b.argumentIfEnabled);
               }
               else if (b.argumentsIfEnabled?.length) {
                  allArgs.push(...b.argumentsIfEnabled);
               }

               if (b.argumentIfDisabled) {
                  allArgs.push(b.argumentIfDisabled);
               }
               else if (b.argumentsIfDisabled?.length) {
                  allArgs.push(...b.argumentsIfDisabled);
               }

               if (allArgs.find(a => a.toLowerCase().startsWith("-target="))) {
                  this.disabledParameters.add(b.id);
               }

               break;
            case ParameterType.Text:
               const t = p as TextParameterData;
               if (t.argument?.toLowerCase().startsWith("-target=")) {
                  this.disabledParameters.add(t.id);
               }
               break;
            case ParameterType.List:
               const list = p as ListParameterData;
               list.items.forEach(i => {
                  allArgs = [];
                  if (i.argumentIfEnabled) {
                     allArgs.push(i.argumentIfEnabled);
                  }
                  else if (i.argumentsIfEnabled?.length) {
                     allArgs.push(...i.argumentsIfEnabled);
                  }

                  if (i.argumentIfDisabled) {
                     allArgs.push(i.argumentIfDisabled);
                  }
                  else if (i.argumentsIfDisabled?.length) {
                     allArgs.push(...i.argumentsIfDisabled);
                  }

                  if (allArgs.find(a => a.toLowerCase().startsWith("-target="))) {
                     this.disabledParameters.add(i.id);
                  }

               });
               break;
         }

      })
   }

   private setTemplate(templateId: string, notifyChanged = true) {

      const template = this.allTemplates.find(t => t.id === templateId);
      if (!template || template.id === this.template?.id) {
         return;
      }

      if (!template) {
         throw `Unable to find template ${templateId}`;
      }

      this.advTargets = undefined;
      this.disabledParameters.clear();

      const source = { ...template };
      source.hash = "";

      this.previewSource = JSON.stringify(source, null, 4);

      if (this.preview) {

         this.setPreviewTemplate();
         return;
      }

      this.template = template;

      this.setTemplateParameters();

      if (notifyChanged) {
         this.setChanged();
      }
   }

   private setDefaultParameters() {

      if (!this.template) {
         return;
      }

      if (Object.keys(this.jobDetails?.jobData?.parameters ?? {}).length) {
         this.parameters = { ...this.parameters, ...this.jobDetails!.jobData!.parameters };
      }

      const query = new URLSearchParams(window.location.search);

      this.template.parameters.forEach(p => {
         let qid = "";
         switch (p.type) {
            case ParameterType.Bool:
               const b = p as BoolParameterData;
               qid = `id-${b.id}`;
               if (query.get(qid)) {
                  this.parameters[b.id] = query.get(qid) as string;
               }
               break;
            case ParameterType.Text:
               const t = p as TextParameterData;
               qid = `id-${t.id}`;
               if (query.get(qid)) {
                  this.parameters[t.id] = query.get(qid) as string;
               }
               break;
            case ParameterType.List:
               const list = p as ListParameterData;
               let any = false;
               list.items.forEach(i => {
                  qid = `id-${i.id}`;
                  if (query.get(qid)) {
                     any = true;
                  }
               });

               if (any) {

                  list.items.forEach(i => {
                     (this.parameters as any)[i.id] = undefined;
                     qid = `id-${i.id}`;
                     if (query.get(qid)) {
                        this.parameters[i.id] = query.get(qid) as string;
                     }
                  });
               }
               break;
            default:
               console.error(`Unknown parameter type ${p.type}`)
         }

      })

   }

   private selectDefaultTemplate() {


      const jobDetails = this.jobDetails;
      const stream = this.stream;
      const queryTemplateId = this.queryTemplateId;

      let t: GetTemplateRefResponse | undefined;

      if (this.templateIdOverride) {
         t = this.allTemplates.find(t => t.id === this.templateIdOverride);
      }
      this.templateIdOverride = undefined;

      if (!t && jobDetails?.template) {
         t = this.allTemplates.find(t => t.name === jobDetails.template!.name);
      }
      // handle preflight redirect case
      if (!t && this.isPreflightSubmit && !jobDetails) {

         if (queryTemplateId) {
            let errorReason = "";
            t = this.allTemplates?.find(t => t.id === queryTemplateId);
            if (!t) {
               errorReason = `Unable to find queryTemplateId ${queryTemplateId} in stream ${this.streamId}`;
               console.error(errorReason);

            } else if (!t.allowPreflights) {
               errorReason = `Template does not allow preflights: queryTemplateId ${queryTemplateId} in stream ${this.streamId}`;
               console.error(errorReason);
               t = undefined;
            }

            if (errorReason) {
               errorDialogStore.set({
                  reason: `${errorReason}`,
                  title: `Preflight Template Error`,
                  message: `There was an issue with the specified preflight template.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`
               }, true);
            }
         }

         if (!t) {
            const defaultStreamPreflightTemplate = stream.templates.find(t => t.id === stream.defaultPreflight?.templateId);
            t = defaultStreamPreflightTemplate;
            if (!t) {
               console.error(`Stream default preflight template cannot be found for stream ${stream.fullname} : stream defaultPreflightTemplate ${stream.defaultPreflight?.templateId}, will use first template in list`);
            }
         }
      }

      if (!t && queryTemplateId) {
         t = this.allTemplates.find(template => template.id === queryTemplateId);
      }

      if (!t) {
         const pref = dashboard.getLastJobTemplateSettings(stream.id, this.templates.map(t => t.id));
         if (pref) {
            t = this.templates.find(t => stream.id === pref.streamId && t.id === pref.templateId)
         }
      }
      // default to sane template when all are shown
      if (!t && this.showAllTemplates && stream.tabs.length > 0) {
         const stab = stream.tabs[0] as JobsTabData;
         if (stab.templates && stab.templates.length > 0) {
            t = this.allTemplates.find(t => t.id === stab.templates![0]);
         }
      }

      if (!t) {
         t = this.templates.length > 0 ? this.templates[0] : undefined;
         if (!t) {
            t = this.allTemplates[0];
         }
      }

      if (!t) {
         throw `Unable to get default template for jobKey: ${this.jobKey}, check that tab has a valid template list`;
      }

      // when coming from P4V, always clear the base CL
      if (this.isFromP4V) {
         this.change = undefined;
         this.changeOption = undefined;
      }

      this.setTemplate(t.id, false);

   }

   private filterTemplates() {

      let templates: GetTemplateRefResponse[] = [];

      if (!this.jobDetails) {
         if (this.showAllTemplates) {
            templates = [...this.allTemplates].sort((a, b) => a.name.localeCompare(b.name));
         } else {
            const tab = this.stream.tabs.find(tab => tab.title === this.jobKey) as JobsTabData | undefined;
            if (!tab) {
               throw `No stream tab ${this.jobKey}`;
            }

            if (tab.type !== TabType.Jobs) {
               throw `Tab is not of Jobs type: ${tab.title}`;
            }

            tab.templates?.forEach(templateName => {
               const t = this.allTemplates.find(t => t.id === templateName);
               if (!t) {
                  console.error(`Could not find template ${templateName}`);
                  return;
               }
               templates.push(t);
            });
         }

         templates = templates.filter(t => !!t.canRun);

      } else {
         const query = new URLSearchParams(window.location.search);
         const allowtemplatechange = query.get("allowtemplatechange") === "true";
         if (!allowtemplatechange) {
            const template = this.allTemplates.find(t => t.name === this.jobDetails?.template?.name);
            if (template) {
               templates = [template];
            }
         }

         if (!templates.length) {
            if (!this.showAllTemplates) {
               const tab = this.stream.tabs.filter(t => t.type === TabType.Jobs).find(t => !!(t as JobsTabData).templates?.find(template => template === this.jobDetails?.template?.id));

               (tab as JobsTabData | undefined)?.templates?.forEach(tid => {
                  const t = this.allTemplates.find(t => t.id === tid);
                  if (!t) {
                     console.error(`Could not find template ${tid}`);
                     return;
                  }
                  templates.push(t);
               });
            }
         }

         if (!templates.length) {
            templates = [...this.allTemplates].sort((a, b) => a.name.localeCompare(b.name));
         }

      }

      // Filter by alias templates if present
      const aliasTemplates = this.construction.aliasTemplates;
      if (aliasTemplates && aliasTemplates.length > 0) {
         const aliasTemplateSet = new Set(aliasTemplates);
         templates = templates.filter(t => aliasTemplateSet.has(t.id));
      }

      this.templates = templates;

   }

   private allTemplates: GetTemplateRefResponse[] = [];

   private async load() {
      this.allTemplates = await templateCache.getStreamTemplates(this.stream);

      this.filterTemplates();

      this.selectDefaultTemplate();
      this.setDefaultParameters();

      this.loaded = true;
      this.setChanged();
   }

   private static instance?: BuildOptions;

}

const BoolParameter: React.FC<{ param: BoolParameterData, disabled?: boolean }> = observer(({ param, disabled }) => {

   const options = BuildOptions.get();

   options.subscribeToParameterChange();

   const key = param.id.replaceAll(".", "-") + `_${options.currentRenderKey}`;

   return <Stack>
      <Checkbox key={key}
         label={param.label}
         disabled={options.readOnly || disabled}
         checked={options.parameters[param.id]?.toLowerCase() === "true"}
         onChange={(ev, value) => {
            ev?.preventDefault();
            options.onBooleanChanged(param.id, value ?? false);
         }}
      />
   </Stack>

})

const TextParameter: React.FC<{ param: TextParameterData, disabled?: boolean }> = observer(({ param, disabled }) => {

   const options = BuildOptions.get();

   options.subscribeToParameterChange();

   const key = param.id.replaceAll(".", "-") + `_${options.currentRenderKey}`;

   let value = options.parameters[param.id] ?? "";
   if (options.disabledParameters.has(param.id)) {
      value = "";
   }

   return <Stack>
      <TextField key={key}
         placeholder={options.jobDetails ? "" : param.hint}
         label={param.label}
         spellCheck={false}
         autoComplete="off"
         value={value}
         readOnly={options.readOnly || disabled}
         onChange={(ev, value) => {
            options.onTextChanged(param.id, value ?? "");
         }}
      />
   </Stack>
})


const TagPickerParameter: React.FC<{ param: ListParameterData, disabled?: boolean }> = observer(({ param, disabled }) => {

   const options = BuildOptions.get();

   options.subscribeToParameterChange();

   const key = `parameter_key_${param.label}` + `_${options.currentRenderKey}`;

   type PickerItem = {
      itemData: ListParameterItemData;
      key: string;
      name: string;
   }

   const allItems: PickerItem[] = param.items.map(item => {
      return {
         itemData: item,
         key: item.text,
         name: item.text
      };
   });

   const selectedItems = allItems.filter(item => {
      return options.parameters[item.itemData.id]?.toLowerCase() === "true";
   });

   // tag picker
   return <Stack key={key}>
      <Label> {param.label}</Label>
      <TagPicker
         disabled={options.readOnly || disabled}
         onResolveSuggestions={(filter, selected) => {
            return allItems.filter(i => {
               return !selected?.find(s => i.key === s.key) && i.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1;
            });
         }}

         onEmptyResolveSuggestions={(selected) => {
            return allItems.filter(i => {
               return !selected?.find(s => i.key === s.key);
            });
         }}

         selectedItems={selectedItems}

         onChange={(items?) => {

            allItems.forEach(item => {
               if (!items) {
                  return;
               }

               options.onListItemChanged(item.itemData.id, items.find(i => i.key === item.key) ? true : false);

            });

            options.onListItemsUpdated();

         }}
      />
   </Stack>;
})

const BasicListParameter: React.FC<{ param: ListParameterData, disabled?: boolean }> = observer(({ param, disabled }) => {

   const options = BuildOptions.get();
   options.subscribeToParameterChange();

   const key = `parameter_key_${param.label}` + `_${options.currentRenderKey}`;

   const doptions: IDropdownOption[] = [];

   param.items.forEach((item, index) => {
      doptions.push({
         key: item.id,
         text: item.text,
         selected: options.parameters[item.id]?.toLowerCase() === "true"
      });
   });

   return <Dropdown key={key}
      label={param.label}
      options={doptions}
      disabled={options.readOnly || disabled}
      placeholder={options.jobDetails ? "" : "Select option"}
      onChange={(ev, option, index) => {

         param.items.forEach(item => {
            options.onListItemChanged(item.id, false);
         });

         options.onListItemChanged(param.items[index!].id, true);
         options.onListItemsUpdated();

      }} />
})

const MultiListParameter: React.FC<{ param: ListParameterData, disabled?: boolean }> = observer(({ param, disabled }) => {

   const options = BuildOptions.get();
   options.subscribeToParameterChange();

   const { modeColors } = getHordeStyling();

   const jobDetails = options.jobDetails;

   const key = `parameter_key_${param.label}` + `_${options.currentRenderKey}`;

   const gset: Set<string> = new Set();

   param.items.forEach(item => {
      if (!item.group) {
         item.group = "__nogroup";
      }
      if (item.group) {
         gset.add(item.group);
      }
   });

   const groups = Array.from(gset).sort((a, b) => a.localeCompare(b));

   const doptions: IDropdownOption[] = [];

   const selectedKeys: string[] = [];

   groups.forEach(group => {

      if (group !== "__nogroup") {
         doptions.push({
            key: `group_${group}`,
            text: group,
            itemType: DropdownMenuItemType.Header
         });
      }

      const dupes = new Map<string, number>();

      param.items.forEach(item => {
         const v = dupes.get(item.text);
         dupes.set(item.text, !v ? 1 : v + 1);
      });

      param.items.forEach(item => {
         if (item.group === group) {
            const key = item.id;
            const selected = options.parameters[item.id]?.toLowerCase() === "true"
            if (selected) {
               selectedKeys.push(key);
            }
            doptions.push({
               key: key,
               data: item,
               text: (item.group !== "__nogroup" && dupes.get(item.text)! > 1) ? `${item.text} - ${item.group}` : item.text,
               selected: selected
            });
         }
      });
   });

   return <Dropdown
      key={key}
      disabled={options.readOnly || disabled}
      placeholder={jobDetails ? "" : "Select options"}
      styles={{
         callout: {
            selectors: {
               ".ms-Callout-main": {
                  padding: "4px 4px 12px 12px",
                  overflow: "hidden"
               }
            }
         },
         dropdownItemHeader: { fontSize: 12, color: modeColors.text },
         dropdownOptionText: { fontSize: 12 },
         dropdownItem: {
            minHeight: 18, lineHeight: 18, selectors: {
               '.ms-Checkbox-checkbox': {
                  width: 14,
                  height: 14,
                  fontSize: 11
               }
            }
         },
         dropdownItemSelected: {
            minHeight: 18, lineHeight: 18, backgroundColor: "inherit",
            selectors: {
               '.ms-Checkbox-checkbox': {
                  width: 14,
                  height: 14,
                  fontSize: 11
               }
            }
         }
      }}
      label={param.label}
      defaultSelectedKeys={selectedKeys}
      onChange={(event, option, index) => {

         if (!option) {
            return;
         }

         options.onListItemChanged((option.data as ListParameterItemData).id, option.selected ? true : false)
         options.onListItemsUpdated();
      }}
      multiSelect
      options={doptions}
   />;
})

type PickerItem = {
   itemData: ListParameterItemData;
   key: string;
   name: string;
}
const LongListParameter: React.FC<{ param: ListParameterData, disabled?: boolean, allowSelectFilter?: (item: PickerItem) => boolean, allowMultiple: boolean }> = observer(({ param, disabled, allowSelectFilter, allowMultiple }) => {

   const options = BuildOptions.get();

   options.subscribeToParameterChange();

   const key = `parameter_key_${param.label}` + `_${options.currentRenderKey}`;

   const allItems: Map<ITag['key'], PickerItem> = new Map(param.items.map(item => {
      return [
         item.text,
         {
            itemData: item,
            key: item.text,
            name: item.text
         }
      ]
   }));

   const selectFilter = allowSelectFilter ?? ((item: PickerItem) => {
      return !(item.name.startsWith("**") && item.name.endsWith("**"));
   });
   const selectedItems = allItems.values().filter(item => {
      return options.parameters[item.itemData.id]?.toLowerCase() === "true" && selectFilter(item);
   })
      .toArray();

   // tag picker
   return <Stack key={key}>
      <Label>
         {param.label}
      </Label>
      <TagPicker
         disabled={options.readOnly || disabled}
         onResolveSuggestions={(filter, selected) => {
            return allItems.values().filter(i => {
               return !selected?.find(s => i.key === s.key) && i.name.toLowerCase().indexOf(filter.toLowerCase()) !== -1;
            })
               .toArray();
         }}

         onEmptyResolveSuggestions={(selected) => {
            return allItems.values().filter(i => {
               return !selected?.find(s => i.key === s.key);
            })
               .toArray();
         }}

         selectedItems={selectedItems}

         onChange={(item_tags?) => {

            if (item_tags) {

               // Get the items for all selected tags for filtering 
               const selected_items = item_tags.map(tag => allItems.get(tag.key)).filter(item => item !== undefined);

               let filtered_items = selected_items.filter(item => selectFilter(item));
               if (!allowMultiple) {
                  // Keep the last item which passed the filter list as it's most likely to be the newest (need more selection metadata to perform accurately)
                  filtered_items = [filtered_items[filtered_items.length - 1]];
               }

               // Update all tags
               for (const [key, item] of allItems) {
                  const was_selected = filtered_items.find(selected_item => key === selected_item?.key) !== undefined;
                  const can_be_selected = selectFilter(item);

                  options.onListItemChanged(item.itemData.id, was_selected && can_be_selected);
               }
            }
            options.onListItemsUpdated();
         }}
      />
   </Stack>;
})

const ListParameter: React.FC<{ param: ListParameterData, disabled?: boolean }> = ({ param, disabled }) => {

   if (param.style === ListParameterStyle.TagPicker) {
      return <TagPickerParameter param={param} disabled={disabled} />
   } else if (param.items.length > LONG_LIST_PARAMETER_THRESHOLD) {
      return <LongListParameter param={param} disabled={disabled} allowMultiple={param.style === ListParameterStyle.MultiList} />
   } else if (param.style === ListParameterStyle.MultiList) {
      return <MultiListParameter param={param} disabled={disabled} />
   } else if (param.style === ListParameterStyle.List) {
      return <BasicListParameter param={param} disabled={disabled} />
   }

   return null;

}

let toolTipId = 0;

const BuildParametersPanel: React.FC = observer(() => {

   const options = BuildOptions.get();
   options.subscribe();

   const template = options.template;

   if (!template) {
      console.error("Build Modal has no template");
      return null;
   }

   let estimatedHeight = options.estimateHeight();

   if (estimatedHeight > window.innerHeight - 360) {
      estimatedHeight = window.innerHeight - 360;
   }

   const renderParameter = (param: ParameterData, disabled?: boolean) => {

      switch (param.type) {
         case ParameterType.Bool:
            const b = param as BoolParameterData;
            return <BoolParameter key={`parameter_key_${b.id}_${options.currentRenderKey}`} param={b} disabled={disabled} />
         case ParameterType.Text:
            const t = param as TextParameterData;
            return <TextParameter key={`parameter_key_${t.id}_${options.currentRenderKey}`} param={t} disabled={disabled} />
         case ParameterType.List:
            const list = param as ListParameterData;
            return <ListParameter key={`parameter_key_${list.label}_${options.currentRenderKey}`} param={list} disabled={disabled} />
         default:
            return <Text>Unknown Parameter Type</Text>;
      }
   };

   return <Stack>
      {!!template?.description && <Stack>
         <Stack style={{ width: parameterWidth }}>
            <Stack style={{ paddingTop: "4px", paddingBottom: "20px" }}>
               <Markdown styles={{ root: { maxHeight: 240, overflow: "auto", th: { fontSize: 12 } } }}>{template.description}</Markdown>
            </Stack>
         </Stack>
      </Stack>}

      <Stack style={{
         height: estimatedHeight,
         position: 'relative',
         width: parameterWidth
      }}><ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
            <Stack tokens={{ childrenGap: parameterGap }}>
               {template.parameters.map((p) => {


                  let key = "";
                  let disabled = false;

                  switch (p.type) {
                     case ParameterType.Bool:
                        const b = p as BoolParameterData;
                        key = `boolean_parameter_key_${b.id}`;
                        if (options.disabledParameters.has(b.id)) {
                           disabled = true;
                        }
                        break;
                     case ParameterType.Text:
                        const t = p as TextParameterData;
                        key = `text_parameter_key_${t.id}`;
                        if (options.disabledParameters.has(t.id)) {
                           disabled = true;
                        }
                        break;
                     case ParameterType.List:
                        const list = p as ListParameterData;
                        key = `list_parameter_key_${list.label}`;
                        if (list.items.find(i => options.disabledParameters.has(i.id))) {
                           disabled = true;
                        }
                        break;
                     default:
                        return <Text>Unknown Parameter Type</Text>;
                  }

                  let toolTip = (p as any).toolTip ?? "";
                  if (disabled) {
                     if (toolTip) {
                        toolTip += "\n(Disabled due to advanced option target modification)";
                     } else {
                        toolTip = "Disabled due to advanced option target modification";
                     }

                  }

                  if (toolTip) {
                     return <TooltipHost key={key} content={toolTip} id={`unique_tooltip_${toolTipId++}`} directionalHint={DirectionalHint.leftCenter}>
                        {renderParameter(p, disabled)}
                     </TooltipHost>
                  } else {
                     return renderParameter(p, disabled);
                  }

               })}
               {!!options.preflightChange && <Stack style={{ paddingTop: 8 }}><Checkbox label={`Automatically submit changelist ${options.preflightChange} upon preflight success`}
                  checked={options.autoSubmit}
                  onChange={(ev, checked) => {
                     options.onAutoSubmitChanged(checked ? true : false);
                  }}
               /></Stack>}
            </Stack>
         </ScrollablePane>
      </Stack>
   </Stack>
})

const SubmittingModal: React.FC = () => {

   const { hordeClasses } = getHordeStyling();

   return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
      <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
         <Stack grow verticalAlign="center">
            <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Creating Job</Text>
         </Stack>
         <Stack horizontalAlign="center">
            <Text variant="mediumPlus">The job is being created and will be available soon.</Text>
         </Stack>
         <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Stack>
   </Modal>
}

const TemplateSelector: React.FC = () => {

   const options = BuildOptions.get();

   let templateOptions: IContextualMenuItem[] = [];

   if (!options.showAllTemplates) {
      templateOptions = options.templates.map(t => {
         return { key: t.id, text: t.name, onClick: () => options.onTemplateChange(t.id) };
      }).sort((a, b) => a.text < b.text ? -1 : 1);
   } else {

      const sorted = new Map<string, GetTemplateRefResponse[]>();

      options.templates.forEach(t => {

         options.stream.tabs.forEach(tab => {
            if (tab.type !== TabType.Jobs) {
               return;
            }

            const jtab = tab as GetJobsTabResponse;
            if (!jtab.templates?.find(template => template === t.id)) {
               return;
            }

            if (!sorted.has(jtab.title)) {
               sorted.set(jtab.title, []);
            }

            sorted.get(jtab.title)!.push(t);

         })
      })

      Array.from(sorted.keys()).sort((a, b) => a < b ? -1 : 1).forEach(cat => {

         const templates = sorted.get(cat);
         if (!templates?.length) {
            return;
         }

         const subItems = templates.sort((a, b) => a.name < b.name ? -1 : 1).map(t => {
            return { key: t.id, text: t.name, onClick: () => options.onTemplateChange(t.id) };
         })

         templateOptions.push({ key: `${cat}_category`, text: cat, subMenuProps: { items: subItems } });

      })
   }

   if (options.jobKey !== "all" && options.jobKey !== "summary") {

      templateOptions.push({ key: `show_all_templates_divider`, itemType: ContextualMenuItemType.Divider });

      if (!options.showAllTemplates) {
         templateOptions.unshift({ key: `show_all_templates`, text: "Show All Templates", onClick: (ev) => { ev?.stopPropagation(); ev?.preventDefault(); options.setShowAllTemplates(true) } });
      } else {
         templateOptions.unshift({ key: `show_all_templates`, text: "Filter Templates", onClick: (ev) => { ev?.stopPropagation(); ev?.preventDefault(); options.setShowAllTemplates(false) } });
      }
   }

   const templateMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: templateOptions,
   };

   let templateName = options?.template?.name;
   if (templateName && templateName?.length > 72) {
      templateName = templateName.slice(0, 72);
      templateName += "...";
   }

   return <Stack>
      <Label>Template</Label>
      <DefaultButton style={{ width: 280, textAlign: "left" }} disabled={options.readOnly} menuProps={templateMenuProps} text={templateName ?? "Error"} />
   </Stack>
}

let textCounter = 0;

const PreviewPanel: React.FC = observer(() => {

   const options = BuildOptions.get();

   const textRef = React.useRef<ITextField>(null);

   options.subscribe();

   const onUpdate = (source?: string) => {
      const csource = source ?? textRef.current?.value ?? "";
      options.onPreviewSourceChanged(csource);
   }

   let estimatedHeight = options.estimateHeight(true);

   if (estimatedHeight > window.innerHeight - 360) {
      estimatedHeight = window.innerHeight - 360;
   }

   return <Stack>
      <Stack style={{ paddingTop: 64, paddingBottom: 12 }}>
         <Label>
            Template Source
         </Label>
         <Stack>
            <TextField key={`text_counter_${textCounter++}`} componentRef={textRef} style={{ width: 670, height: estimatedHeight }} multiline spellCheck={false} resizable={false} defaultValue={options.previewSource} />
         </Stack>
         <Stack>
         </Stack>
      </Stack>
      <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
         <Stack grow />
         <DefaultButton text="Copy" disabled={false} onClick={() => {

            copyToClipboard(textRef.current?.value ?? "");

         }} />
         <DefaultButton text="Paste" disabled={false} onClick={async () => {
            const text = await navigator.clipboard.readText();
            onUpdate(text);
         }} />
         <DefaultButton text="Format" disabled={false} onClick={async () => {
            let text = textRef.current?.value ?? "{}";
            const value = JSON.parse(text);
            onUpdate(JSON.stringify(value, null, 4));
         }} />
         <PrimaryButton text="Preview" onClick={() => { onUpdate(); }} />
      </Stack>

   </Stack>

});

const AdvancedPanel: React.FC = observer(() => {

   const targetPicker = React.useRef(null)

   const options = BuildOptions.get();
   options.subscribe();

   const stream = options.stream;

   const template = options.template;

   if (!template) {
      console.error("Build Modal has no template");
      return null;
   }

   const priorityOptions: IDropdownOption[] = [];

   for (const p in Priority) {
      priorityOptions.push({
         text: p,
         key: p,
         isSelected: options!.advJobPriority ? p === options!.advJobPriority : Priority.Normal === p
      });
   }

   let height = options.estimateHeight();

   const a = options.jobDetails?.jobData?.arguments ?? [];
   const b = options.jobDetails?.jobData?.parameters ?? {};

   let farguments = "";
   let fargumentsHeight = 0;

   if (options.readOnly && a.length > 0) {

      farguments = a.join("\n")
      const flines = (farguments.match(/\n/g) || '').length + 1
      fargumentsHeight = Math.min(flines * 18, 240) + 16
      height += (fargumentsHeight + 24)
   }

   let showAdditionalArgs = !options.readOnly || !!options.advAdditionalArgs;
   if (showAdditionalArgs) {
      height += 52;
   }

   let fparameters = "";
   let fparametersHeight = 0;

   if (options.readOnly && (Object.keys(b).length)) {
      fparameters = (JSON.stringify(b, null, 2));
      const plines = (fparameters.match(/\n/g) || '').length + 1
      fparametersHeight = Math.min(plines * 18, 240) + 16
      height += (fparametersHeight + 24)
   }

   let showArgumentClipboardButton = !!options.jobDetails?.jobData?.arguments?.length;
   if (showArgumentClipboardButton) {
      height += 32;
   }

   // targets
   height += 92;

   if (height > window.innerHeight - 360) {
      height = window.innerHeight - 360;
   }

   // Run as scheduler
   if (dashboard.hordeAdmin) {
      height += 32;
   }

   // target options for picker
   type TargetPickerItem = {
      key: string;
      name: string;
   }

   const targetItems: TargetPickerItem[] = options.targets.map(t => {
      return {
         key: t,
         name: t
      }
   });

   return <Stack style={{
      height: height,
      position: 'relative',
      width: parameterWidth,
   }}>
      <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
         <Stack style={{ paddingBottom: 12, paddingRight: 12 }} tokens={{ childrenGap: 12 }}>
            {!!stream.configRevision && <Stack>
               <TextField label="Template Path" readOnly={true} value={stream.configPath ?? ""} />
            </Stack>
            }
            <Stack>
               <Dropdown disabled={options.readOnly} key={`key_adv_priority_${options.currentRenderKey}`} defaultValue={options.advJobPriority} label="Priority" options={priorityOptions} onChange={(ev, option) => {
                  options.advJobPriority = option?.key as Priority;
                  options.setChanged();
               }
               } />
            </Stack>
            <Stack>
               <Checkbox disabled={options.readOnly} key={`key_adv_update_issues_${options.currentRenderKey}`}
                  label="Update Build Health Issues"
                  checked={options.updateIssues ?? false} onChange={(ev, checked) => {
                     options.updateIssues = checked ?? false;
                     options.setChanged();
                  }} />
            </Stack>
            {!options.readOnly && dashboard.hordeAdmin && <Stack>
               <Checkbox key={`key_adv_run_as_scheduler_${options.currentRenderKey}`}
                  label="Run As Scheduler"
                  checked={options.runAsScheduler ?? false} onChange={(ev, checked) => {
                     options.runAsScheduler = checked ?? false;
                     options.setChanged();
                  }} />
            </Stack>}
            <Stack>
               <TextField key={`key_adv_job_name_${options.currentRenderKey}`} disabled={options.readOnly} spellCheck={false} defaultValue={options.advJobName} label="Job Name" onChange={(ev, newValue) => {
                  options.advJobName = newValue;
                  options.setChanged();
               }} />
            </Stack>
            <Stack>
               <Label>Targets</Label>
               <TagPicker

                  componentRef={targetPicker}

                  disabled={options.readOnly}

                  onBlur={(ev) => {
                     if (ev.target.value && ev.target.value.trim()) {
                        options.addTarget(ev.target.value.trim())
                     }

                     // This is using undocumented behavior to clear the input when you lose focus
                     // It could very well break
                     if (targetPicker.current) {
                        try {
                           (targetPicker.current as any).input.current._updateValue("");
                        } catch (reason) {
                           console.error("There was an error adding target to list and clearing the input in process\n" + reason);
                        }

                     }

                  }}

                  onResolveSuggestions={(filter, selected) => {
                     return [];
                  }}

                  onEmptyResolveSuggestions={(selected) => {
                     return [];
                  }}

                  onRemoveSuggestion={(item) => { }}

                  createGenericItem={(input: string, ValidationState: ValidationState) => {
                     return {
                        item: { name: input, key: input },
                        selected: true
                     };
                  }}

                  onChange={(items) => {
                     if (items) {
                        options.setTargets(items.map(i => i.name))
                     }
                  }}

                  onValidateInput={(input?: string) => input ? ValidationState.valid : ValidationState.invalid}

                  selectedItems={targetItems}

                  onItemSelected={(item) => {
                     if (!item || !item.name?.trim()) {
                        return null;
                     }
                     options.addTarget(item.name.trim());
                     return null;
                  }}
               />
            </Stack>
            {showAdditionalArgs && <TextField key={`key_adv_add_args_${options.currentRenderKey}`} multiline style={{ height: 48 }} resizable={false} readOnly={options.readOnly} spellCheck={false} defaultValue={options.advAdditionalArgs} label="Additional Arguments" onChange={(ev, newValue) => options.advAdditionalArgs = newValue} />}
            {showArgumentClipboardButton && <DefaultButton text="Copy Job Arguments to Clipboard" style={{ width: 240 }} onClick={() => copyToClipboard(
               options.jobDetails?.jobData?.arguments.map(options.getQuotedArg).join(" ")
            )} />}
            {!!farguments && <TextField style={{ height: fargumentsHeight }} key={`key_adv_job_arguments_${options.currentRenderKey}`} defaultValue={farguments} readOnly={true} label="Job Arguments" multiline resizable={false} />}
            {!!fparameters && <TextField style={{ height: fparametersHeight }} key={`key_adv_job_parameters_${options.currentRenderKey}`} defaultValue={fparameters} readOnly={true} label="Job Parameters" multiline resizable={false} />}

            <Stack>
               <Checkbox disabled={options.readOnly}
                  label="Template Editor"
                  checked={options.preview}
                  onChange={(ev, checked) => {
                     if (checked) {
                        options.onModeChanged("Basic");
                     }
                     options.onPreviewChanged(checked ?? false);
                  }} />
            </Stack>

         </Stack>
      </ScrollablePane>
   </Stack>

})

async function buildParametersUrl(template: GetTemplateRefResponse): Promise<boolean> {

   // Note: can't rely on current path for template as user can change that via All Templates and URL won't update.
   const setParams = new URLSearchParams(`${location.search}&newbuild=true&templateId=${template.id}`);
   const args = BuildOptions.get().parameters;

   for (const p of template.parameters) {
      switch (p.type) {
         case ParameterType.Bool: {
            const b = p as BoolParameterData;
            if (args[b.id] && args[b.id] !== b.default.toString()) {
               setParams.set(`id-${b.id}`, args[b.id]);
            }
            break;
         }
         case ParameterType.Text: {
            const t = p as TextParameterData;
            if (args[t.id] && args[t.id] !== t.default) {
               setParams.set(`id-${t.id}`, args[t.id]);
            }
            break;
         }
         case ParameterType.List: {
            const list = p as ListParameterData;
            for (const i of list.items) {
               // Note: need to include only true values for lists, as setDefaultParameters clears all values if any are present in the url.
               if (args[i.id] && args[i.id].toLowerCase() === "true") {
                  setParams.set(`id-${i.id}`, args[i.id]);
               }
            }
            break;
         }
         default:
            console.error(`Unknown parameter type ${p.type}`);
      }
   }
   
   const query = new URL(window.location.href);
   query.search = setParams.toString();
   return await copyToClipboardAsync(query.toString());
}

const BuildModal: React.FC<{ setUseLegacyDialog: (value: boolean) => void }> = observer(({ setUseLegacyDialog }) => {

   const [parametersUrlCopied, setParametersUrlCopied] = useState(false);

   const options = BuildOptions.get();
   options.subscribe();

   const template = options.template;

   if (!template) {
      return null;
   }

   const { hordeClasses, modeColors } = getHordeStyling();

   const changeOptions = ["Latest Change"];

   const defaultPreflightQuery = options.defaultPreflightQuery;

   if (defaultPreflightQuery) {
      changeOptions.push(...defaultPreflightQuery.filter(n => !!n.name).map(n => n.name!));
   }

   const changeItems: IComboBoxOption[] = changeOptions.map(name => { return { key: `key_change_option_${name}`, text: name } });

   let changeText: string | undefined;

   if (options.change) {
      changeText = options.change.toString();
   }
   if (options.changeCommitId) {
      changeText = options.changeCommitId;
   }
   if (options.changeOption) {
      changeText = options.changeOption;
   }

   const onSubmit = () => {
      const errors: ValidationError[] = [];
      if (!options.validate(errors)) {
         return;
      }

      const templateId = template.id;

      let changeQueries: ChangeQueryConfig[] | undefined;

      if (typeof options.change !== 'number' && !options.changeCommitId) {
         const changeOption = options.changeOption ?? "Latest Change";
         if (changeOption !== "Latest Change" && defaultPreflightQuery) {
            changeQueries = defaultPreflightQuery;
         }
         else {
            const changeQuery = defaultPreflightQuery?.find(p => p.name === options.changeOption);
            if (changeQuery) {
               changeQueries = [{ ...changeQuery, condition: undefined }];
            }
         }
      }

      let additionalArgs: string[] = [];
      if (options!.advAdditionalArgs) {
         // Match either:
         // 1. A key-value pair where the value is a quoted string that may contain escaped characters
         //    - Starts with one or more non-whitespace chars followed by '='
         //    - Then a quoted string starting and ending with "
         //    - Inside the quotes, allow any characyer except " and \
         //    - However, allow any escaped character
         // 2. A standalone non-whitespace character sequence
         const argRegex = /\S+="(?:[^"\\]|\\.)*"|\S+/g
         // Match input with above regex, replacing any non-escaped quotes with nothing and escaped quotes with just a quote.
         options.advAdditionalArgs.trim().replaceAll("\n", " ").match(argRegex)?.forEach(arg => additionalArgs.push(arg.replace(/(?<!\\)"/g, '').replace(/\\"/g, '"')));
      }

      // trim all parameters
      const parameters: Record<string, string> = {};
      for (const [id, value] of Object.entries(options.parameters ?? {})) {
         parameters[id?.trim()] = value?.trim()
      }

      const data: CreateJobRequest = {
         streamId: options.streamId,
         templateId: templateId,
         name: options.advJobName,
         priority: options.advJobPriority,
         changeQueries: changeQueries,
         parameters: parameters,
         additionalArguments: additionalArgs?.length ? additionalArgs : undefined,
         targets: options.advTargets,
         updateIssues: options.updateIssues,
         runAsScheduler: options.runAsScheduler
      };

      if (typeof (options.change) === 'number') {
         data.change = options.change;
      }

      if (options.changeCommitId) {
         data.commitId = options.changeCommitId;
      }

      if (typeof (options.preflightChange) === 'number') {

         data.preflightChange = options.preflightChange;

         data.updateIssues = false;

         if (options.autoSubmit) {
            data.autoSubmit = true;
         }

      }

      if (options.preflightCommitId) {
         data.preflightCommitId = options.preflightCommitId;
         data.updateIssues = false;
      }

      const submit = async () => {

         if (!!import.meta.env.VITE_HORDE_DEBUG_NEW_JOB) {

            console.log("Debug Job Submit");
            console.log(data);
            console.log(JSON.stringify(data));
            return;
         }

         let errorReason: any = undefined;

         options.setSubmitting(true);

         console.log("Submitting job");
         console.log(JSON.stringify(data));

         let redirected = false;

         await backend.createJob(data).then(async (data) => {
            console.log(`Job created: ${JSON.stringify(data)}`);
            console.log("Updating notifications")
            try {
               await backend.updateNotification({ slack: true }, "job", data.id);
               const user = await backend.getCurrentUser();
               if (user.jobTemplateSettings) {
                  dashboard.jobTemplateSettings = user.jobTemplateSettings;
               }

            } catch (reason) {
               console.log(`Error on updating notifications: ${reason}`);
            }

            redirected = true;
            options.onClose(data.id);
         }).catch(reason => {
            // "Not Found" is generally a permissions error
            errorReason = reason ? reason : "Unknown";
            console.log(`Error on job creation: ${errorReason}`);

         }).finally(() => {

            if (!redirected) {
               options.setSubmitting(false);
               options.onClose(undefined);

               if (errorReason) {

                  if (errorReason?.trim().endsWith("does not exist") || errorReason?.trim().endsWith("does not contain any shelved files")) {
                     errorReason += ".\nPerforce edge server replication for the change may be in progress. Please try again in a few minutes."
                  }

                  errorDialogStore.set({

                     reason: `${errorReason}`,
                     title: `Error Creating Job`,
                     message: `There was an issue creating the job.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`

                  }, true);

               }
            }
         });

      }

      submit();

   }

   const advancedPivotRenderer = (link: any, defaultRenderer: any): JSX.Element => {
      return (
         <Stack horizontal>
            {defaultRenderer(link)}
            {options.advancedModified && <Icon iconName="Issue" style={{ color: '#EDC74A', paddingLeft: 8 }} />}
         </Stack>
      );
   }

   const pivotItems = ["Basic", "Advanced"].map(tab => {
      return <PivotItem headerText={tab} itemKey={tab} key={tab} onRenderItemLink={tab === "Advanced" ? advancedPivotRenderer : undefined} />;
   });

   let name = template?.name ? template.name : options.jobDetails?.jobData?.name;
   if (options.preview) {
      name += " (Preview)";
   }

   return <Modal isModeless={false} key="new_build_key" isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700 * (options.preview ? 2 : 1), hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { }}>
      <Stack horizontal tokens={{ childrenGap: 18 }}>
         <Stack>
            <Stack horizontal styles={{ root: { paddingLeft: 8, paddingTop: 8, paddingBottom: 8 } }}>
               <Stack grow verticalAlign="center">
                  <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{name}</Text>
               </Stack>


               <Stack >
                  {<Pivot className={hordeClasses.pivot}
                     style={{ paddingLeft: 32 }}
                     selectedKey={options.mode}
                     linkSize="normal"
                     linkFormat="links"
                     onLinkClick={(item) => {
                        options.onModeChanged(item!.props!.itemKey as NewBuildMode)
                     }}>
                     {pivotItems}
                  </Pivot>
                  }
               </Stack>
            </Stack>
            <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8, paddingTop: 4 } }}>
               <Stack tokens={{ childrenGap: 8 }}>
                  <Stack horizontal tokens={{ childrenGap: 18 }} style={{ paddingBottom: 4 }}>
                     <Stack verticalAlign="center" verticalFill={true}>
                        <TemplateSelector />
                     </Stack>
                     <Stack>
                        <ComboBox style={{ width: 204 }} key={`build_change_key_${options.currentRenderKey}`} label="Change" text={changeText} options={changeItems} disabled={options.readOnly} allowFreeform autoComplete="off" placeholder="Latest Change" defaultValue={"-1"} onChange={(ev, option, index, value) => {
                           ev.preventDefault();
                           if (option) {
                              options.change = undefined;
                              options.changeCommitId = undefined;
                              options.changeOption = option.text;
                           } else {
                              options.changeOption = undefined;
                              if (!value) {
                                 options.change = undefined;
                                 options.changeCommitId = undefined;
                              } else {
                                 const nvalue = parseInt(value);
                                 if (!isNaN(nvalue)) {
                                    options.change = nvalue;
                                    options.changeCommitId = undefined;
                                 } else {
                                    options.change = undefined;
                                    options.changeCommitId = value.trim().toLowerCase();
                                 }
                              }
                           }
                           options.setChanged();

                        }}
                        />
                     </Stack>

                     <Stack>
                        <TextField
                           key={`build_preflight_key_${options.currentRenderKey}`}
                           label="Shelved Change"
                           style={{ width: 120 }}
                           placeholder={!template!.allowPreflights ? "Disabled" : ""}
                           autoComplete="off"
                           spellCheck={false}
                           value={options.preflightChange?.toString() ?? options.preflightCommitId ?? ""}
                           disabled={!template!.allowPreflights || options.readOnly || options.isPreflightSubmit}
                           onChange={(ev, newValue) => {
                              ev.preventDefault();
                              const trimmed = newValue?.trim() ?? "";
                              if (!trimmed) {
                                 options.onPreflightChange(undefined);
                                 options.onPreflightCommitIdChange(undefined);
                                 return;
                              }
                              if (/^\d+$/.test(trimmed)) {
                                 options.onPreflightChange(parseInt(trimmed));
                              } else {
                                 options.onPreflightCommitIdChange(trimmed.toLowerCase());
                              }
                           }}
                        />
                     </Stack>
                  </Stack>
               </Stack>
               <Stack>
                  {(options.mode === "Basic") && <BuildParametersPanel />}
                  {options.mode === "Advanced" && <AdvancedPanel />}
               </Stack>
               <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 32, paddingLeft: 8, paddingBottom: 8 } }}>
                  {!options.jobDetails && !options.readOnly && <Stack>
                     <DefaultButton text="Reset" style={{ width: 110 }} onClick={() => options.onResetToDefaults()} />
                  </Stack>}
                  <DefaultButton text={parametersUrlCopied ? "URL copied!": "Create URL"} disabled={!template}
                     onClick={async () => { 
                        const copySuccess = await buildParametersUrl(template);
                        setParametersUrlCopied(copySuccess);
                        if (copySuccess) {
                           setTimeout(() => setParametersUrlCopied(false), 5000);
                        }
                  }} />

                  <Stack grow />
                  <PrimaryButton text="Start Job" disabled={!template || !template.canRun || options.readOnly || options.preview} onClick={() => { onSubmit(); }} />
                  <DefaultButton text="Cancel" disabled={false} onClick={() => { options.onClose(); }} />
               </Stack>
            </Stack>
         </Stack>
         {options.preview && <PreviewPanel />}
      </Stack>
   </Modal>
})


const NewBuildV2Inner: React.FC<{ setUseLegacyDialog: (value: boolean) => void }> = observer(({ setUseLegacyDialog }) => {

   const options = BuildOptions.get();
   options.subscribe();

   return <Stack>
      {!!options.validationErrors.length && <ValidationErrorModal errors={options.validationErrors} onClose={() => options.setValidationErrors([])} />}
      {!options.submitting && <BuildModal setUseLegacyDialog={setUseLegacyDialog} />}
      {!!options.submitting && <SubmittingModal />}
   </Stack>

})

export const NewBuildV2: React.FC<{ streamId: string; show: boolean; onClose: (newJobId: string | undefined) => void, jobKey?: string; jobDetails?: JobDetailsV2, readOnly?: boolean, aliasTemplates?: string[] }> = ({ streamId, jobKey, show, onClose, jobDetails, readOnly, aliasTemplates }) => {

   const { projectStore } = useBackend();
   const [useNewBuildV1, setNewBuildV1] = useState<boolean>(false);

   // initially had this using useConst, though not working
   if (!BuildOptions.initialized()) {
      new BuildOptions(streamId, projectStore, onClose, jobDetails, jobKey, readOnly, aliasTemplates);
   }

   const options = BuildOptions.get();

   useEffect(() => {
      return () => {
         options.clear();
      };
   }, [options]);

   if (useNewBuildV1) {
      return <NewBuild show={true} streamId={streamId} onClose={onClose} jobKey={jobKey} jobDetails={jobDetails} readOnly={readOnly} />
   }

   return <Stack>
      <NewBuildV2Inner setUseLegacyDialog={(value: boolean) => { setNewBuildV1(value) }} />
   </Stack>
}

const ValidationErrorModal: React.FC<{ errors: ValidationError[], onClose: () => void }> = ({ errors, onClose }) => {

   const { hordeClasses } = getHordeStyling();

   const close = () => {
      onClose();
   };

   const messages = errors.map((e, idx) => {
      return <MessageBar key={`validation_error_${idx}`} messageBarType={MessageBarType.error} isMultiline={true}>
         <Stack>
            <Stack grow tokens={{ childrenGap: 12 }}>
               <Stack tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Parameter:</Text>
                  <Text nowrap >{e.paramString}</Text>
               </Stack>
               <Stack tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Value:</Text>
                  <Text nowrap >{e.value ? e.value : "no value"}</Text>
               </Stack>
               <Stack grow tokens={{ childrenGap: 12 }} horizontal>
                  <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold", width: 80 }}>Error:</Text>
                  <Text>{e.error}</Text>
               </Stack>
            </Stack>
            {
               e.context && <Stack style={{ paddingTop: 18, paddingBottom: 18 }}>
                  <Stack grow tokens={{ childrenGap: 12 }}>
                     <Text nowrap style={{ fontFamily: "Horde Open Sans SemiBold" }}>Context:</Text>
                     <Text>{e.context}</Text>
                  </Stack>
               </Stack>
            }

         </Stack>
      </MessageBar>
   })

   return <Modal isOpen={true} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 700 } }} onDismiss={() => { close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack style={{ paddingLeft: 8, paddingTop: 4 }} grow>
            <Text variant="mediumPlus">Validation Errors</Text>
         </Stack>
         <Stack grow horizontalAlign="end">
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack>
      </Stack>

      <Stack style={{ paddingLeft: 20, paddingTop: 8, paddingBottom: 8 }}>
         <Text style={{ fontSize: 15 }}>Please correct the following errors and try again.</Text>
      </Stack>

      <Stack tokens={{ childrenGap: 8 }} styles={{ root: { paddingLeft: 20, paddingTop: 18, paddingBottom: 24, width: 660 } }}>
         {messages}
      </Stack>


      <Stack horizontal styles={{ root: { padding: 8, paddingTop: 8 } }}>
         <Stack grow />
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <PrimaryButton text="Ok" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};