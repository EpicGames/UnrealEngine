// Copyright Epic Games, Inc. All Rights Reserved.

import backend from "horde/backend";
import { GetBuildHealthFilterResponse, JobData, JobStreamQuery, ProjectData, StreamData } from "horde/backend/Api";
import { action, makeObservable, observable, runInAction } from "mobx";
import { BuildHealthOptionsState } from "./BuildHealthOptions";
import { LabelRefData, StepRefData, TemplateRefData } from "./BuildHealthDataTypes";
import { decodeTemplateKey, encodeFullyQualifiedLabelName, encodeTemplateKeyFromStrings } from "./BuildHealthUtilities";
import { BuildHealthOptionsController, IHierarchyValidator, StreamDescriptor, TemplateDescriptor } from "./BuildHealthOptionsController";

// #region -- API Data Types & Utilities --

/**
 * Request object that controls what items to refresh upon Data Handler refresh.
 */
export type DataHandlerRefreshRequest = {
    projects?: boolean;
    streams?: boolean;
    jobs?: boolean;
    steps?: boolean;
}

export const EMPTY_REFRESH_REQUEST: DataHandlerRefreshRequest = { projects: false, streams: false, jobs: false, steps: false };
export const DEFAULT_REFRESH_REQUEST: DataHandlerRefreshRequest = { projects: true, streams: true, jobs: true, steps: true };

/**
 * Utility function that will expand a Data Handler Refresh Request to include all the dependents in it's refresh.
 * @param request The request to expand.
 * @returns The expanded request.
 */
export function expandRefreshRequest(request: DataHandlerRefreshRequest): DataHandlerRefreshRequest {
    const expanded: DataHandlerRefreshRequest = { ...request };
    if (request.projects) {
        expanded.streams = true;
    }

    if (expanded.streams) {
        expanded.jobs = true;
    }

    if (expanded.jobs) {
        expanded.steps = true;
    }

    return expanded;
}

// #endregion -- API Data Types & Utilities --

/**
 * Represents a cache entry for step ref data.
 */
interface StepCacheEntry {
    timestamp: number;
    steps: StepRefData[];
}

/**
 * Represents a cache entry for label ref data.
 */
interface LabelCacheEntry {
    timestamp: number;
    labels: LabelRefData[];
}

/**
 * Data handler for all build health data used for dropdowns.
 * @todo Data sharing & caching would ideally occur between the BuildHealth View & Step Outcome.
 */
export class BuildHealthDataHandler implements IHierarchyValidator {
    private readonly STREAMS_FILTER = "id,state,streamId,name,labels";
    private readonly STREAM_JOBS_BATCH_STEP_FILTER = "batches.steps.id,batches.steps.name,batches.steps.state,batches.steps.outcome,batches.steps.finishTime"
    private readonly STEP_DATA_SAMPLE_SIZE = 5;
    private readonly ACL_SCOPE = "horde";
    private readonly ACL_ACTION = "AddUpdateDeleteBuildHealthFilter";

    // #region -- Private Members --

    private lastRefresh = -1;
    private buildHealthOptions: BuildHealthOptionsState;
    private buildHealthOptionsController: BuildHealthOptionsController;
    private cachedProjectToStreamData: Map<string, StreamData[]> = new Map<string, StreamData[]>();
    private cachedStreamToTemplatesData: Map<string, TemplateRefData[]> = new Map<string, TemplateRefData[]>();
    private cachedStreamTempalteIdToSteps: Map<string, StepCacheEntry> = new Map<string, StepCacheEntry>();
    private cachedStreamTemplateIdToLabels: Map<string, LabelCacheEntry> = new Map<string, LabelCacheEntry>();

    // Loading state for initial data fetch
    @observable
    private _isInitializing: boolean = false;

    @observable
    private _initializingMessage: string = "";

    // #endregion -- Private Members --

    // #region -- Public Members --

    get isInitializing(): boolean {
        return this._isInitializing;
    }

    get initializingMessage(): string {
        return this._initializingMessage;
    }

    @action
    setInitializing(isInitializing: boolean, message: string = ""): void {
        this._isInitializing = isInitializing;
        this._initializingMessage = message;
    }

    @observable.shallow
    projectsData: ProjectData[] = [];

    @observable.shallow
    projectFilterData: Map<string, GetBuildHealthFilterResponse[]> = new Map<string, GetBuildHealthFilterResponse[]>();

    @observable
    hasBuildHealthFilterWriteAccess: boolean = false;

    @observable.shallow
    streamsData: StreamData[] = [];

    @observable.shallow
    templatesData: TemplateRefData[] = [];

    @observable.shallow
    stepData: StepRefData[] = [];

    @observable.shallow
    labelData: LabelRefData[] = [];

    // #endregion -- Public Members --

    // #region -- Constructor --

    constructor(buildHealthOptionsState: BuildHealthOptionsState, buildHealthOptionsController: BuildHealthOptionsController) {
        makeObservable(this);

        this.buildHealthOptions = buildHealthOptionsState;
        this.buildHealthOptionsController = buildHealthOptionsController;
    }

    //#endregion -- Constructor --

    // #region -- Private API --

    private async refreshData(request: DataHandlerRefreshRequest) {
        return this.getProjectsData(request);
    }

    // #endregion -- Private API --

    // #region -- Public API --

    /**
     * Resets the handler, retunring it to a project-only data state.
     */
    @action
    reset() {
        this.streamsData = [];
        this.templatesData = [];
        this.stepData = [];
        this.labelData = [];
    }

    /**
     * Requests a hierarchical data refresh. Refreshw ill occur if the options have changed since last refresh.
     * @param request The data refresh request message to use in order to control refresh depth.
     * @param force Whether to force the teh refresh or not. If false, will only refresh if the options have changed since last refresh.
     * @param onComplete Delegate that will be invoked upon completion of the hierarchical refresh.
     */
    async requestHierarchicalRefresh(request: DataHandlerRefreshRequest = DEFAULT_REFRESH_REQUEST, force?: boolean, onComplete?: (request: DataHandlerRefreshRequest) => void) {
        if (this.lastRefresh < this.buildHealthOptionsController.optionsChangeVersion || force) {
            this.lastRefresh = this.buildHealthOptionsController.optionsChangeVersion;
            await this.refreshData(request);
            onComplete?.(request);
        }
    }

    /**
     * Obtains the project data given the current buildHealthOptions.
     * @param request The data refresh request message to use in order to control refresh depth.
     */
    async getProjectsData(request: DataHandlerRefreshRequest) {
        if (this.projectsData.length === 0 || request.projects) {
            const projectDataResult = (await backend.getProjects());

            runInAction(() => {
                this.projectsData = projectDataResult;
            });

            const accessResults: { access: boolean, scopes: any[] } = await backend.getAccountAccess(this.ACL_SCOPE, this.ACL_ACTION);

            runInAction(() => {
                this.hasBuildHealthFilterWriteAccess = accessResults.access;
            })

            await this.getProjectFiltersData();
        }
        await this.getStreamData(request);
    }

    /**
     * Get the project filters data.
     * @param projectId the project to get filter data for. If undefined, obtain for all known projects.
     */
    async getProjectFiltersData(projectId?: string) {
        let buildHealthFilterData: Map<string, GetBuildHealthFilterResponse[]> = new Map<string, GetBuildHealthFilterResponse[]>;

        for (let idx: number = 0; idx < this.projectsData.length; ++idx) {
            if (projectId === undefined || projectId === this.projectsData[idx].id) {
                const projectFilterDataResult: GetBuildHealthFilterResponse[] = (await backend.getBuildHealthProjectFilters({ projectId: this.projectsData[idx].id }));
                buildHealthFilterData.set(this.projectsData[idx].id, projectFilterDataResult);
            }
        }

        runInAction(() => {
            this.projectFilterData = buildHealthFilterData;
        })
    }

    /**
     * Obtains the stream data given the current buildHealthOptions.
     * @param request The data refresh request message to use in order to control refresh depth.
     */
    async getStreamData(request: DataHandlerRefreshRequest) {
        if (request.streams && Object.keys(this.buildHealthOptions.enabledProjects).length > 0) {
            let streamsData: StreamData[] = [];
            this.cachedStreamToTemplatesData.clear();

            this.projectsData.forEach((project: ProjectData) => {
                const isProjectEnabled = this.buildHealthOptions.enabledProjects.hasOwnProperty(project.id);

                if (!this.cachedProjectToStreamData.has(project.id)) {
                    this.cachedProjectToStreamData.set(project.id, []);
                }

                if (project.streams === undefined) {
                    return;
                }

                this.cachedProjectToStreamData.get(project.id)?.push(...project.streams!);

                if (!isProjectEnabled) {
                    return;
                }

                project.streams?.forEach((stream: StreamData) => {
                    streamsData.push(stream);
                    this.cachedStreamToTemplatesData.set(stream.id, stream.templates);
                });
            });

            runInAction(() => {
                this.streamsData = streamsData;
            });
        }

        await this.getTemplateData(request);
    }

    /**
     * Obtains the template data given the current buildHealthOptions.
     * @param request The data refresh request message to use in order to control refresh depth.
     */
    async getTemplateData(request: DataHandlerRefreshRequest) {
        if (request.jobs && Object.keys(this.buildHealthOptions.enabledStreams).length > 0) {
            let templatesData: TemplateRefData[] = [];
            for (const key of Object.keys(this.buildHealthOptions.enabledStreams)) {
                if (this.cachedStreamToTemplatesData.has(key)) {
                    const templates = this.cachedStreamToTemplatesData.get(key);
                    if (templates) {
                        templatesData.push(
                            ...templates.map(t => ({ ...t, streamId: key }))
                        );
                    }
                }
            }

            runInAction(() => {
                this.templatesData = templatesData;
            });
        } else if (request.jobs) { // We have no enabled streams, so we should have no template data showing.
            runInAction(() => {
                this.templatesData = [];
            });
        }

        await this.getStepData(request);
    }

    /**
     * Obtains the step data given the current buildHealthOptions.
     * @param request The data refresh request message to use in order to control refresh depth.
     */
    async getStepData(request: DataHandlerRefreshRequest) {
        const entries = Object.entries(this.buildHealthOptions.enabledTemplates);
        if (request.steps && entries.length > 0) {
            const enabledTemplateKey = new Set(entries.map(([enabledTemplateKey]) => enabledTemplateKey));

            await Promise.all(entries.filter(([key, _value]) => !this.cachedStreamTempalteIdToSteps.has(key)).map(async ([key]) => {
                const templateData = decodeTemplateKey(key);

                const query: JobStreamQuery = {
                    template: [templateData.templateId],
                    count: this.STEP_DATA_SAMPLE_SIZE,
                    filter: `${this.STREAMS_FILTER},${this.STREAM_JOBS_BATCH_STEP_FILTER}`
                };

                const jobDatas: JobData[] = await backend.getStreamJobs(templateData.streamId, query);
                const recentJobs = jobDatas.filter(job => job.batches);
                const stepMap = new Map<string, StepRefData>();
                const labelMap = new Map<string, LabelRefData>();

                for (const job of recentJobs) {
                    for (const batch of job.batches!) {
                        for (const step of batch.steps) {
                            const stepKey = step.name;
                            if (!stepMap.has(stepKey)) {
                                stepMap.set(stepKey, { ...step, streamId: templateData.streamId, templateId: decodeTemplateKey(key).templateId });
                            }
                        }
                    }
                    for (const label of job.labels!) {
                        const labelKey = encodeFullyQualifiedLabelName(label.dashboardCategory, label.dashboardName);
                        if (!labelMap.has(labelKey)) {
                            labelMap.set(labelKey, { ...label, streamId: templateData.streamId, templateId: decodeTemplateKey(key).templateId })
                        }
                    }
                }

                const distinctSteps = [...stepMap.values()];
                const distintcLabels = [...labelMap.values()];

                this.cachedStreamTempalteIdToSteps.set(key, { timestamp: Date.now(), steps: distinctSteps });
                this.cachedStreamTemplateIdToLabels.set(key, { timestamp: Date.now(), labels: distintcLabels });

                return distinctSteps;
            }));

            runInAction(() => {
                this.stepData = Array.from(this.cachedStreamTempalteIdToSteps.entries())
                    .filter(([templateId]) => enabledTemplateKey.has(templateId))
                    .flatMap(([_, entry]) => entry.steps);

                this.labelData = Array.from(this.cachedStreamTemplateIdToLabels.entries())
                    .filter(([templateId]) => enabledTemplateKey.has(templateId))
                    .flatMap(([_, entry]) => entry.labels);
            });
        } else if (request.steps) { // We have no enabled templates, so we should have no step data showing.

            runInAction(() => {
                this.stepData = [];
                this.labelData = [];
            });
        }
    }

    /**
     *  Obtains all the currently known streams for a given project.
     * @param projectId The project.
     * @returns An array representing all the known streams for a given project.
     */
    getCachedStreams(projectId: string): StreamDescriptor[] {
        let returnArray: StreamDescriptor[] = [];

        if (this.cachedProjectToStreamData.has(projectId)) {
            returnArray = this.cachedProjectToStreamData.get(projectId)!.map(x => ({ id: x.id, name: x.name }));
        }

        return returnArray;
    }


    /**
     * Obtains all the currently known templates for a given stream.
     * @param streamId The stream.
     * @returns An array representing all the known templates for a given stream.
     */
    getCachedTemplates(streamId: string): TemplateDescriptor[] {
        let returnArray: TemplateDescriptor[] = [];

        if (this.cachedStreamToTemplatesData.has(streamId)) {
            returnArray = this.cachedStreamToTemplatesData.get(streamId)!.map(x => ({ id: x.id, name: x.name }));
        }

        return returnArray;
    }

    /**
     * Obtains all the currently known step names for a given stream & template.
     * @param streamId The stream.
     * @param templateId The template.
     * @returns An array representing all the known step names for a given stream & template pair.
     */
    getCachedTemplateStepNames(streamId: string, templateId: string): string[] {
        let returnArray: string[] = [];

        let encodedTemplateKey: string = encodeTemplateKeyFromStrings(streamId, templateId);

        if (this.cachedStreamTempalteIdToSteps.has(encodedTemplateKey)) {
            returnArray = this.cachedStreamTempalteIdToSteps.get(encodedTemplateKey)!.steps.map(x => x.name);
        }

        return returnArray;
    }

    /**
     * Obtains all the currently known labels for a given stream & template.
     * @param streamId The stream.
     * @param templateId The template.
     * @returns An array representing all the known labels for a given stream & template pair.
     */
    getCachedTemplateLabelNames(streamId: string, templateId: string): string[] {
        let returnArray: string[] = [];

        let encodedTemplateKey: string = encodeTemplateKeyFromStrings(streamId, templateId);

        if (this.cachedStreamTemplateIdToLabels.has(encodedTemplateKey)) {
            returnArray = this.cachedStreamTemplateIdToLabels.get(encodedTemplateKey)!.labels.map(x => encodeFullyQualifiedLabelName(x.dashboardCategory, x.dashboardName));
        }

        return returnArray;
    }

    /**
     * Obtains all the StepRefData for a given stream & template.
     * @param streamId The stream.
     * @param templateId The template.
     * @returns An array representing all the StepRefData for a given stream & template pair.
     */
    getCachedTemplateStepData(streamId: string, templateId: string): StepRefData[] | undefined {
        let encodedTemplateKey: string = encodeTemplateKeyFromStrings(streamId, templateId);
        return this.cachedStreamTempalteIdToSteps.get(encodedTemplateKey)?.steps;
    }

    /**
     * @inheritdoc
     */
    isValidTemplate(streamId: string, templateId: string): boolean {
        if (!this.cachedStreamToTemplatesData.has(streamId)) {
            return false;
        }
        if (this.cachedStreamToTemplatesData.get(streamId)?.find(x => x.id === templateId) === undefined) {
            return false;
        }

        return true;
    }

    /**
     * @inheritdoc
     */
    isValidStep(streamId: string, templateId: string, stepName: string): boolean {
        if (!this.cachedStreamToTemplatesData.has(streamId)) {
            return false;
        }

        let template = this.cachedStreamToTemplatesData.get(streamId)?.find(x => x.id === templateId)

        if (template === undefined) {
            return false;
        }

        let encodedTemplateKey: string = encodeTemplateKeyFromStrings(streamId, templateId);
        if (!this.cachedStreamTempalteIdToSteps.get(encodedTemplateKey)?.steps.find(x => x.name.toLocaleLowerCase() == stepName.toLocaleLowerCase())) {
            return false;
        }

        return true;
    }

    /**
    * @inheritdoc
    */
    isValidLabel(streamId: string, templateId: string, fullyQualifiedLabelName: string): boolean {
        if (!this.cachedStreamToTemplatesData.has(streamId)) {
            return false;
        }

        let template = this.cachedStreamToTemplatesData.get(streamId)?.find(x => x.id === templateId)

        if (template === undefined) {
            return false;
        }

        let encodedTemplateKey: string = encodeTemplateKeyFromStrings(streamId, templateId);
        if (!this.cachedStreamTemplateIdToLabels.get(encodedTemplateKey)?.labels.find(x => encodeFullyQualifiedLabelName(x.dashboardCategory, x.dashboardName).toLocaleLowerCase() === fullyQualifiedLabelName.toLocaleLowerCase())) {
            return false;
        }

        return true;
    }

    // #endregion -- Public API --
}