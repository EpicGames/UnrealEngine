// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyleSets } from "@fluentui/react";
import { GetLabelStateResponse, GetStepResponse, GetTemplateRefResponse } from "horde/backend/Api";

export const QUERY_SCHEMA_PARAM_NAME: string = "query_schema";
export const PROJECTS_URL_PARAM_NAME: string = "projects";
export const STREAMS_URL_PARAM_NAME: string = "streams";
export const TEMPLATES_URL_PARAM_NAME: string = "templates";
export const LABELS_URL_PARAM_NAME: string = "labels";
export const STEPS_URL_PARAM_NAME: string = "stepNames";
export const STATES_URL_PARAM_NAME: string = "states";
export const JOB_STARTS_URL_PARAM_NAME: string = "jobStarts";
export const PARAMETER_KEY_PREFIX: string = "parameter_key";
export const TIME_SPAN_URL_PARAM: string = "lastTimeRange";
export const INCLUDE_PREFLIGHT_PARAM: string = "includePreflight";
export const INCLUDE_CANCELLED_PARAM: string = "includeCancelled";
export const DEBUG_MODE_PARAM: string = "debugMode";
export const FILTER_ID_URL_PARAM: string = "filterId";
export const KEY_SEPARATOR: string = "::";

export const UI_SUMMARY_WARNING_AS_ERROR_PARAM: string = "warningAsError";
export const UI_DATE_ANCHORS_PARAM: string = "dateAnchors";

export const LABELS_URL_PARAM_ALL_LABELS: string = "labels_AllKnownValues";
export const STEPS_URL_PARAM_ALL_STEPS: string = "stepNames_AllKnownValues";
export const STATES_URL_PARAM_ALL_STATES: string = "stateNames_AllKnownValues";
export const JOB_START_URL_PARAM_ALL_NAMES: string = "jobStart_akv";

// #region -- Styles --

export const dropdownNavigationButtons = mergeStyleSets({
    base: {
        marginLeft: 4,
        padding: "2px 6px",
        fontSize: 12,
        cursor: "pointer",
    }
});

export const dropdownOptions = mergeStyleSets({
    base: {
        padding: "4px 8px",
    }
});

// #endregion -- Styles --

/**
 * Convenience type to fold in extra metadata relevant for the Template. 
 */
export type TemplateRefData = GetTemplateRefResponse & {
    streamId?: string;
    fullname?: string;
}

/**
 * Convenience type to fold in extra metadata relevant for the Step. 
 */
export type StepRefData = GetStepResponse & {
    templateId?: string;
    streamId?: string;
}

/**
 * Convenience type to fold in extra metadata relevant for the Label. 
 */
export type LabelRefData = GetLabelStateResponse & {
    templateId?: string;
    streamId?: string;
}