import { DirectionalHint, IconButton, IDropdownOption, ISelectableOption, Label, Stack, TooltipHost } from "@fluentui/react";
import { GetBuildHealthFilterResponse, GetThinUserInfoResponse } from "horde/backend/Api";
import { BuildHealthDataHandler } from "./BuildHealthDataHandler";
import { BuildHealthOptionsController } from "./BuildHealthOptionsController";
import { useState } from "react";
import { observer } from "mobx-react-lite";
import { BuildHealthFilterModal, BuildHealthFilterModalProps } from "./BuildHealthFilterModal";
import { parseBuildHealthQueryParams, upgradeBuildHealthQueryParams, loadBuildHealthOptionsFromParams, toNavigationQuery } from "./BuildHealthOptions";
import { FILTER_ID_URL_PARAM } from "./BuildHealthDataTypes";
import { getHordeStyling } from "horde/styles/Styles";
import { getDateTimeString } from "./stepoutcome/StepOutcomeUtilities";
import { SearchableDropdown } from "./SearchableDropdownComponent";
import { copyToClipboardAsync } from "../../base/utilities/clipboard";

// #region -- Helper Functions --

function constructEditableBuildHealthFilter(buildHealthFilter: GetBuildHealthFilterResponse | null, buildHealthOptions: BuildHealthOptionsController, handler: BuildHealthDataHandler): EditableBuildHealthFilter {
    if (buildHealthFilter !== null) {
        let currentOptionsString = toNavigationQuery(buildHealthOptions.state, buildHealthOptions.uiState, buildHealthOptions.querySchemaVersion, handler).toString();
        let currentHumanReadableQuery = parseBuildHealthQueryParams(new URLSearchParams(currentOptionsString));
        let humanReadableQuery = parseBuildHealthQueryParams(new URLSearchParams(buildHealthFilter.filterQuery));
        return EditableBuildHealthFilter.produceFromExisting(buildHealthFilter, currentOptionsString, JSON.stringify(humanReadableQuery, null, 2), JSON.stringify(currentHumanReadableQuery, null, 2))
    }
    else {
        let currentProject: string | null = buildHealthOptions.getCurrentProject()!;
        let currentOptionsString = toNavigationQuery(buildHealthOptions.state, buildHealthOptions.uiState, buildHealthOptions.querySchemaVersion, handler).toString();
        let humanReadableQuery = parseBuildHealthQueryParams(new URLSearchParams(currentOptionsString));
        return EditableBuildHealthFilter.produceNew(currentProject, currentOptionsString, JSON.stringify(humanReadableQuery, null, 2))
    }
}

function requestFilterApplication(filterQuery: string, buildHealthOptions: BuildHealthOptionsController, handler: BuildHealthDataHandler) {
    let searchParams: URLSearchParams = new URLSearchParams(filterQuery);
    const params = parseBuildHealthQueryParams(searchParams);
    const upgradedParams = upgradeBuildHealthQueryParams(params.buildHealthQueryParams, buildHealthOptions.querySchemaVersion);
    loadBuildHealthOptionsFromParams(upgradedParams, params.buildHealthUIParams, buildHealthOptions, handler);
}

// #endregion -- Helper Functions --

// #region -- Public Types --

/**
 * Data structure that represents an editable Build Health Filter.
 */
export class EditableBuildHealthFilter {
    /// The filter id.
    filterId: string | null;

    /// The project the filter belongs to.
    filterProjectId: string;

    /// The name of the filter.
    filterName: string;

    /// The description of the filter.
    filterDescription?: string;

    /// The query of the filter. If null, there is no prior state (new filter).
    oldFilterQuery: string;

    /// The query of the filter.
    newFilterQuery: string;

    /// The last updated time of the filter. If null, there is no prior state (new filter).
    lastUpdated: Date | null;

    /// The filter owner. If null, there is no prior state (new filter).
    filterOwner: GetThinUserInfoResponse | null;

    /// The old query, in human readable form. If null, there is no prior state (new filter).
    oldHumanReadableQuery: string | null;

    /// The new query, in human readable form.
    newHumanReadableQuery: string;

    private constructor(filterName: string, filterQuery: string, newFilterQuery: string, oldHumanReadableQuery: string | null, newHumanReadableQuery: string, filterId: string | null, filterProject: string, owner: GetThinUserInfoResponse | null, lastUpdated: Date | string | null, filterDescription?: string) {
        this.filterName = filterName;
        this.filterDescription = filterDescription;
        this.oldFilterQuery = filterQuery;
        this.newFilterQuery = newFilterQuery;
        this.filterId = filterId;
        this.oldHumanReadableQuery = oldHumanReadableQuery;
        this.newHumanReadableQuery = newHumanReadableQuery;
        this.filterProjectId = filterProject;
        this.filterOwner = owner;
        this.lastUpdated = lastUpdated === null ? null : new Date(lastUpdated);
    }

    /**
     * Creates an editable Build Health Filter from an existing filter. 
     * @param existingFilter The existing filter.
     * @param newFilterQuery The new filter query to apply.
     * @param oldHumanReadableQuery The old human readable query filter.
     * @param newHumanReadableQuery The new human readable query filter.
     * @returns The editable form of the existing filter.
     */
    static produceFromExisting(existingFilter: GetBuildHealthFilterResponse, newFilterQuery: string, oldHumanReadableQuery: string, newHumanReadableQuery: string): EditableBuildHealthFilter {
        return new EditableBuildHealthFilter(existingFilter.filterName, existingFilter.filterQuery, newFilterQuery, oldHumanReadableQuery, newHumanReadableQuery, existingFilter.id, existingFilter.filterProject, existingFilter.owner, existingFilter.updateTimeUtc, existingFilter.filterDescription);
    }

    /**
     * Creates an editable Build Health Filter from new data.
     * @param projectId The project the Build Health Filter belongs to.
     * @param newFilterQuery The new filter query to apply.
     * @param newHumanReadableQuery The new human readable query filter.
     * @returns The editable form of the new filter.
     */
    static produceNew(projectId: string, newFilterQuery: string, newHumanReadableQuery: string) {
        return new EditableBuildHealthFilter("New Filter", newFilterQuery, newFilterQuery, null, newHumanReadableQuery, null, projectId, null, null);
    }
}

// #endregion -- Public Types --

// #region -- Private Types --

type BuildHealthFilterOptionData = {
    id: string,
    text: string,
    query: string,
    underylingObject: GetBuildHealthFilterResponse | undefined
}

// #endregion -- Private Types --

// #region -- Private Components --

const ProjectFilterDropdownSingle: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, onUploadBuildFilter: (fitler: GetBuildHealthFilterResponse | null) => void }> = observer(function ConstructProjectDropdownSingle({ handler, buildHealthOptions, onUploadBuildFilter }) {
    let selectedFilterId = buildHealthOptions.getBuildHealthFilterId();
    let selectedFilter: GetBuildHealthFilterResponse | null = null;
    let selectedProject: string | null = buildHealthOptions.getCurrentProject();
    const [filterText, setFilterText] = useState("");
    const [linkCopied, setLinkCopied] = useState(false);

    let generateToolTip = (castedOption: IDropdownOption<BuildHealthFilterOptionData>): JSX.Element | undefined => {
        if (!castedOption.data || castedOption.data.underylingObject === undefined) {
            return handler.hasBuildHealthFilterWriteAccess ? (
                <div>
                    Select to make a new filter.
                </div>
            ) : undefined;
        }

        const owner = castedOption.data.underylingObject.owner?.name ?? "N/A";
        const description = castedOption.data.underylingObject.filterDescription ?? "N/A";
        const lastUpdated = getDateTimeString(new Date(castedOption.data.underylingObject.updateTimeUtc));

        return (
            <div>
                <b>Owner</b>: {owner}
                <br />
                <b>Description</b>: {description}
                <br />
                <b>Last Updated</b>: {lastUpdated}
                <br />
            </div>
        );
    };

    if (selectedProject === null) {
        return (null);
    }

    let buildHealthFilters: GetBuildHealthFilterResponse[] | undefined = handler.projectFilterData.get(selectedProject);
    if (buildHealthFilters === undefined) {
        return (null)
    }

    buildHealthFilters.sort((a, b) => a.filterName.localeCompare(b.filterName));

    const doptions: IDropdownOption<BuildHealthFilterOptionData>[] = [];
    const defaultSelectLabel: string = handler.hasBuildHealthFilterWriteAccess ? "(New)" : "Select a Filter";

    doptions.push({
        key: "empty",
        text: defaultSelectLabel,
        data: undefined
    });

    buildHealthFilters.map((projectFilter: GetBuildHealthFilterResponse) => {
        if (projectFilter.id === selectedFilterId) {
            selectedFilter = projectFilter;
        }

        let listParam: BuildHealthFilterOptionData = {
            id: `${projectFilter.id}`,
            text: projectFilter.filterName,
            query: projectFilter.filterQuery,
            underylingObject: projectFilter
        };

        doptions.push({
            key: listParam.id,
            text: listParam.text,
            data: listParam
        });
    });

    let iconName: string = selectedFilterId === null ? "Save" : "Edit";
    let computedSelectedKey: string = selectedFilterId === null ? "empty" : selectedFilterId;

    const lowerFilter = filterText.toLowerCase();
    const filteredOptions = filterText
        ? doptions.filter(o => o.text.toLowerCase().includes(lowerFilter))
        : doptions;

    return (
        <Stack>
            <TooltipHost
                content="Saved filters for the selected project. Write access required to create new filters."
                calloutProps={{ directionalHint: DirectionalHint.rightCenter }}>
                <Label>Project Filters</Label>
            </TooltipHost>
            <Stack horizontal tokens={{ childrenGap: 8 }}>
                <SearchableDropdown
                    options={filteredOptions}
                    styles={{ dropdown: { width: 300 } }}
                    selectedKey={computedSelectedKey}
                    placeholder={defaultSelectLabel}
                    calloutProps={{
                        directionalHint: DirectionalHint.rightTopEdge,
                        alignTargetEdge: true,
                    }}
                    onDismiss={() => {
                        setFilterText("");
                    }}
                    onSearchValueChanged={(newValue) => {
                        // Mirror BuildHealthView: on deletion, blank then re-set on the next frame so the
                        // underlying Fluent Dropdown re-indexes options and onChange picks the right row.
                        if (newValue.length < filterText.length) {
                            setFilterText("");
                            requestAnimationFrame(() => {
                                setFilterText(newValue);
                            });
                        } else {
                            setFilterText(newValue);
                        }
                    }}
                    onRenderOption={(option?: ISelectableOption) => {
                        if (!option) {
                            return null;
                        }
                        let castedOption: IDropdownOption<BuildHealthFilterOptionData> = option as IDropdownOption<BuildHealthFilterOptionData>;
                        if (!castedOption) {
                            return null;
                        }

                        return (
                            <TooltipHost content={generateToolTip(castedOption)}
                                calloutProps={{ directionalHint: DirectionalHint.rightCenter }}
                                styles={{ root: { display: 'block', width: '100%', height: '100%' } }}>
                                <div
                                    style={{
                                        display: 'block',
                                        overflow: 'hidden',
                                        textOverflow: 'ellipsis',
                                        whiteSpace: 'nowrap',
                                        width: '100%',
                                        maxWidth: '280px',
                                    }}
                                >
                                    {castedOption.text}
                                </div>
                            </TooltipHost>
                        )
                    }}
                    onChange={(_ev, option: IDropdownOption<BuildHealthFilterOptionData>) => {
                        if (!option.data) {

                            buildHealthOptions.setBuildHealthFilterId(null);
                            return;
                        }

                        // Simply re-issue our parse on the stored query.
                        const query = option.data!.query;
                        buildHealthOptions.setBuildHealthFilterId(option.data!.underylingObject!.id);
                        requestFilterApplication(query, buildHealthOptions, handler);
                    }}>
                </SearchableDropdown>
                {/* Only show the button if the user has write access for Build Health Filters. */}
                {
                    handler.hasBuildHealthFilterWriteAccess
                    && (
                        <TooltipHost
                            content="Promotes or updates the current filter selection to the server."
                            directionalHint={DirectionalHint.rightTopEdge}
                        >
                            <IconButton
                                iconProps={{ iconName: iconName }}
                                styles={{
                                    root: {
                                        width: 30,
                                        height: 30,
                                        minWidth: 0,
                                        padding: 0,
                                    },
                                }}
                                onClick={() => {
                                    // get build filter from data structs
                                    onUploadBuildFilter(selectedFilter);
                                }}
                            />
                        </TooltipHost>
                    )
                }
                {/* Share Filter Link — copies a URL that, when opened, will apply this saved filter. */}
                {
                    selectedFilterId !== null
                    && (
                        <TooltipHost
                            content={linkCopied ? "Link copied!" : "Copy a shareable link to this filter"}
                            directionalHint={DirectionalHint.rightTopEdge}
                        >
                            <IconButton
                                iconProps={{ iconName: linkCopied ? "CheckMark" : "Link" }}
                                styles={{
                                    root: {
                                        width: 30,
                                        height: 30,
                                        minWidth: 0,
                                        padding: 0,
                                    },
                                }}
                                onClick={async () => {
                                    if (!selectedFilterId) return;
                                    const shareParams = new URLSearchParams();
                                    shareParams.set(FILTER_ID_URL_PARAM, selectedFilterId);
                                    const shareUrl = `${window.location.origin}/buildhealth?${shareParams.toString()}`;
                                    const copied = await copyToClipboardAsync(shareUrl);
                                    if (copied) {
                                        setLinkCopied(true);
                                        window.setTimeout(() => setLinkCopied(false), 1500);
                                    }
                                }}
                            />
                        </TooltipHost>
                    )
                }
            </Stack>
        </Stack>
    );
});

// #endregion -- Private Components --

// #region -- Public Components --

/**
 * Toplevel Build Health Filter component.
 * @returns The component.
 */
export const BuildHealthFilterComponent: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController }> = observer(function ConstructBuildHealthFilterComponent({ handler, buildHealthOptions }) {
    const [selectedFilter, setSelectedFilter] = useState<EditableBuildHealthFilter | undefined>();
    const [modalVisible, setModalVisible] = useState(false);
    const { hordeClasses, modeColors } = getHordeStyling();

    let modalProperties: BuildHealthFilterModalProps = {
        buildHealthFilter: selectedFilter,
        onDismiss: () => {
            setModalVisible(false);
        },
        onSaveSuccess: (savedFilter: GetBuildHealthFilterResponse) => {
            setModalVisible(false);
            buildHealthOptions.setBuildHealthFilterId(savedFilter.id);
            handler.getProjectFiltersData(savedFilter.filterProject);
        },
        onDeleteSuccess: (oldFilterProjectId?: string) => {
            setModalVisible(false);
            buildHealthOptions.setBuildHealthFilterId(null);
            handler.getProjectFiltersData(oldFilterProjectId);
        },
        onRevert: () => {
            setModalVisible(false);

            if (selectedFilter === undefined) {
                return;
            }

            requestFilterApplication(selectedFilter.oldFilterQuery, buildHealthOptions, handler);
        }
    }
    return (
        <>
            {buildHealthOptions.getCurrentProject() !== null && (
                <Stack
                    styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                        },
                    }}>
                    <ProjectFilterDropdownSingle
                        handler={handler}
                        buildHealthOptions={buildHealthOptions}
                        onUploadBuildFilter={(selectedItem: GetBuildHealthFilterResponse | null) => {
                            setModalVisible(true);
                            let editableBuildHealthFilter: EditableBuildHealthFilter = constructEditableBuildHealthFilter(selectedItem, buildHealthOptions, handler);
                            setSelectedFilter(editableBuildHealthFilter);
                        }} />
                    {modalVisible && <BuildHealthFilterModal {...modalProperties} />}
                </Stack>
            )
            }
        </>
    );
});

// #endregion -- Public Components --