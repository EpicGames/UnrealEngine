// Copyright Epic Games, Inc. All Rights Reserved.

import { Dialog, DialogType, Label, PrimaryButton, Stack, TextField } from "@fluentui/react";
import backend from "horde/backend";
import { AddBuildHealthFiltersRequest, GetBuildHealthFilterResponse, UpdateBuildHealthFiltersRequest } from "horde/backend/Api";
import { useState } from "react";
import dashboard from "horde/backend/Dashboard";
import { EditableBuildHealthFilter } from "./BuildHealthFilterComponent";
import { getHordeTheme } from "horde/styles/theme";
import { getDateTimeString } from "./stepoutcome/StepOutcomeUtilities";

// #region -- Private Helpers --

function addLineNumbers(text: string): string {
    return text
        .split("\n")
        .map((line, i) => `${i + 1}: ${line}`)
        .join("\n");
}

type JsonValue = null | boolean | number | string | JsonValue[] | { [k: string]: JsonValue };

type DiffEntry =
    | { kind: "added"; path: string; newVal: JsonValue }
    | { kind: "removed"; path: string; oldVal: JsonValue }
    | { kind: "changed"; path: string; oldVal: JsonValue; newVal: JsonValue };

function tryParseJson(text: string): JsonValue | undefined {
    try {
        return JSON.parse(text) as JsonValue;
    } catch {
        return undefined;
    }
}

function isPlainObject(v: JsonValue): v is { [k: string]: JsonValue } {
    return typeof v === "object" && v !== null && !Array.isArray(v);
}

function formatValue(v: JsonValue): string {
    if (v === undefined) return "<missing>";
    return JSON.stringify(v);
}

function joinPath(base: string, key: string | number): string {
    if (base === "") return typeof key === "number" ? `[${key}]` : key;
    return typeof key === "number" ? `${base}[${key}]` : `${base}.${key}`;
}

function collectJsonDiffs(oldVal: JsonValue, newVal: JsonValue, path: string, out: DiffEntry[]): void {
    if (oldVal === newVal) return;

    const oldIsObj = isPlainObject(oldVal);
    const newIsObj = isPlainObject(newVal);
    if (oldIsObj && newIsObj) {
        const keys = Array.from(new Set([...Object.keys(oldVal), ...Object.keys(newVal)])).sort();
        for (const key of keys) {
            const childPath = joinPath(path, key);
            const hasOld = Object.prototype.hasOwnProperty.call(oldVal, key);
            const hasNew = Object.prototype.hasOwnProperty.call(newVal, key);
            if (hasOld && !hasNew) {
                out.push({ kind: "removed", path: childPath, oldVal: oldVal[key] });
            } else if (!hasOld && hasNew) {
                out.push({ kind: "added", path: childPath, newVal: newVal[key] });
            } else {
                collectJsonDiffs(oldVal[key], newVal[key], childPath, out);
            }
        }
        return;
    }

    const oldIsArr = Array.isArray(oldVal);
    const newIsArr = Array.isArray(newVal);
    if (oldIsArr && newIsArr) {
        const maxLen = Math.max(oldVal.length, newVal.length);
        for (let i = 0; i < maxLen; i++) {
            const childPath = joinPath(path, i);
            if (i >= oldVal.length) {
                out.push({ kind: "added", path: childPath, newVal: newVal[i] });
            } else if (i >= newVal.length) {
                out.push({ kind: "removed", path: childPath, oldVal: oldVal[i] });
            } else {
                collectJsonDiffs(oldVal[i], newVal[i], childPath, out);
            }
        }
        return;
    }

    if (JSON.stringify(oldVal) !== JSON.stringify(newVal)) {
        out.push({ kind: "changed", path: path || "<root>", oldVal, newVal });
    }
}

function renderDiffEntries(entries: DiffEntry[]): string {
    const pathWidth = Math.min(60, entries.reduce((w, e) => Math.max(w, e.path.length), 0));
    return entries.map(e => {
        const path = e.path.padEnd(pathWidth);
        switch (e.kind) {
            case "added":
                return `+ ${path}  ${formatValue(e.newVal)}`;
            case "removed":
                return `- ${path}  ${formatValue(e.oldVal)}`;
            case "changed":
                return `~ ${path}  ${formatValue(e.oldVal)}  →  ${formatValue(e.newVal)}`;
        }
    }).join("\n");
}

function allLineDiffs(a: string, b: string): string {
    const aLines = a.split("\n");
    const bLines = b.split("\n");
    const max = Math.max(aLines.length, bLines.length);
    const lineNumWidth = String(max).length;
    const out: string[] = [];

    for (let i = 0; i < max; i++) {
        if (aLines[i] !== bLines[i]) {
            const ln = String(i + 1).padStart(lineNumWidth, " ");
            out.push(`L${ln}  OLD: ${aLines[i] ?? "<missing>"}`);
            out.push(`L${ln}  NEW: ${bLines[i] ?? "<missing>"}`);
        }
    }

    return out.length === 0 ? "No differences found." : out.join("\n");
}

function diffQueries(a: string, b: string): string {
    if (a === b) return "No differences found.";

    const oldJson = tryParseJson(a);
    const newJson = tryParseJson(b);

    if (oldJson !== undefined && newJson !== undefined) {
        const entries: DiffEntry[] = [];
        collectJsonDiffs(oldJson, newJson, "", entries);
        if (entries.length === 0) {
            return "No structural differences found (whitespace or key-order only).";
        }
        return renderDiffEntries(entries);
    }

    return allLineDiffs(a, b);
}

// #endregion -- Private Helpers --

// #region -- Private Types --

/**
 * Datastructure for build health modal props.
 */
export type BuildHealthFilterModalProps = {
    // The filter to use for the modal.
    buildHealthFilter: EditableBuildHealthFilter | undefined;

    // The delegate to execute when the modal is dismissed.
    onDismiss?: (_ev?: React.MouseEvent<HTMLButtonElement>) => any;

    // The delegate to execute when the Build Health Filter has been saved.
    onSaveSuccess?: (savedFilter: GetBuildHealthFilterResponse) => any;

    // The delegate to execute when changes to the current Build Health Filter has been reverted.
    onRevert?: () => any;

    // The delegate to execute when the Build Health Filter has been deleted.
    onDeleteSuccess?: (oldFilterProjectId?: string) => any;
};

// #endregion -- Private Types --

// #region -- Components --

/**
 * Build Health Filter modal, that is responsible for the visualization and editing of a Build Health Filter
 * @param Props that describe the Build Health Filter to use for the modal, and all delegates to use for operations. 
 * @returns The component.
 */
export const BuildHealthFilterModal: React.FC<BuildHealthFilterModalProps> = ({ buildHealthFilter, onDismiss, onSaveSuccess, onDeleteSuccess, onRevert }) => {
    const theme = getHordeTheme();

    if (buildHealthFilter === undefined) {
        return;
    }

    const [state, setState] = useState<{ filterName: string, filterDescription?: string }>(buildHealthFilter !== null && (buildHealthFilter.filterName !== undefined || buildHealthFilter.filterDescription !== undefined) ? { filterName: buildHealthFilter.filterName, filterDescription: buildHealthFilter.filterDescription } : { filterName: "", filterDescription: undefined });
    const isNewFilter: boolean = buildHealthFilter.filterId === null;
    const isAdmin: boolean = dashboard.hordeGenericAdmin;
    const isOwner: boolean = buildHealthFilter.filterOwner !== null && buildHealthFilter.filterOwner.id == dashboard.userId;
    const isEditorOwner: boolean = isOwner || isAdmin;
    const isOldAndNewDiff: boolean = buildHealthFilter.oldHumanReadableQuery !== buildHealthFilter.newHumanReadableQuery || buildHealthFilter.filterName !== state.filterName || buildHealthFilter.filterDescription !== state.filterDescription;
    const hasHumanReadanbleOldQuery: boolean = buildHealthFilter.oldHumanReadableQuery !== null;

    // #region -- Action Handlers --

    async function Save(): Promise<GetBuildHealthFilterResponse | null> {
        if (buildHealthFilter?.filterId === null) {
            let request: AddBuildHealthFiltersRequest = { filterName: state.filterName, filterDescription: state.filterDescription, filterProject: buildHealthFilter.filterProjectId, filterQuery: buildHealthFilter.oldFilterQuery };
            let response: GetBuildHealthFilterResponse = await backend.addBuildHealthProjectFilter(request);

            return response;
        } else {
            let request: UpdateBuildHealthFiltersRequest = { filterName: state.filterName, filterDescription: state.filterDescription, filterQuery: buildHealthFilter!.newFilterQuery };
            let response: GetBuildHealthFilterResponse = await backend.updateBuildHealthProjectFilter(buildHealthFilter?.filterId!, request);
            return response;
        }
    };

    async function SaveAsCopy(): Promise<GetBuildHealthFilterResponse | null> {
        if (buildHealthFilter?.filterId === null || buildHealthFilter === undefined) {
            return null;
        }

        let request: AddBuildHealthFiltersRequest = { filterName: state.filterName, filterDescription: state.filterDescription, filterProject: buildHealthFilter.filterProjectId, filterQuery: buildHealthFilter.newFilterQuery };
        let response: GetBuildHealthFilterResponse = await backend.addBuildHealthProjectFilter(request);

        return response;
    };

    async function Delete(): Promise<boolean> {
        if (buildHealthFilter?.filterId === null || buildHealthFilter === undefined) {
            return false;
        }

        let response: boolean = await backend.deleteBuildHealthProjectFilter(buildHealthFilter?.filterId);

        return response;
    };

    // #endregion -- Action Handlers --

    // #region -- Button Delegates --

    const handleSaveClick = async () => {
        const savedFilter = await Save();
        if (savedFilter) {
            onSaveSuccess?.(savedFilter);
        }
    };

    const handleSaveAsCopyClick = async () => {
        const savedFilter = await SaveAsCopy();
        if (savedFilter) {
            onSaveSuccess?.(savedFilter);
        }
    };

    const handleDeleteClick = async (oldFilterProjectId?: string) => {
        const savedFilter = await Delete();
        if (savedFilter) {
            onDeleteSuccess?.(oldFilterProjectId);
        }
    };

    const handleRevertClick = () => {
        onRevert?.();
    }

    //#endregion -- Button Delegates --

    return (
        <Dialog
            modalProps={{
                isBlocking: false,
                topOffsetFixed: true,
                styles: {
                    root: {
                        selectors: {
                            ".ms-Dialog-title": {
                                paddingTop: '24px',
                                paddingLeft: '32px'
                            }
                        }
                    }
                }
            }}
            onDismiss={onDismiss}
            hidden={false}
            minWidth={1400}
            dialogContentProps={{
                type: DialogType.close,
                onDismiss: onDismiss,
                title: "Upload Build Health Filter Preset",
            }}>
            <TextField
                label="Project ID"
                value={buildHealthFilter.filterProjectId}
                readOnly
                styles={{ root: { maxWidth: "64ch" } }}
            />
            <TextField
                label="Filter Name"
                maxLength={64}
                value={state.filterName}
                onChange={(_, newValue) => {
                    setState({ ...state, filterName: (newValue ?? "").slice(0, 64) })
                }}
                styles={{ root: { maxWidth: "64ch" } }}
            />
            <TextField
                label="Owner"
                value={buildHealthFilter.filterOwner !== null ? buildHealthFilter.filterOwner.name : dashboard.username}
                readOnly
                styles={{
                    root: { maxWidth: "64ch" },
                    field: {
                        textOverflow: "ellipsis",
                        overflow: "hidden",
                        whiteSpace: "nowrap",
                    }
                }}
            />
            <TextField
                label="Description"
                multiline
                rows={5}
                maxLength={500}
                value={state.filterDescription ?? ""}
                onChange={(_, newValue) => {
                    setState({ ...state, filterDescription: newValue ?? undefined })
                }}
            />
            <Stack horizontal tokens={{ childrenGap: 16 }}>
                {hasHumanReadanbleOldQuery && <Stack.Item grow>
                    <TextField
                        label="Old Query"
                        value={addLineNumbers(buildHealthFilter.oldHumanReadableQuery!)}
                        multiline
                        rows={15}
                        readOnly
                        resizable={false}
                        styles={{ root: { width: "100%" } }}
                    />
                </Stack.Item>}
                {(!hasHumanReadanbleOldQuery || isOldAndNewDiff) && <Stack.Item grow>
                    <TextField
                        label="New Query"
                        value={addLineNumbers(buildHealthFilter.newHumanReadableQuery)}
                        multiline
                        rows={15}
                        readOnly
                        resizable={false}
                        styles={{ root: { width: "100%" } }}
                    />
                </Stack.Item>}
            </Stack>
            {
                (hasHumanReadanbleOldQuery && isOldAndNewDiff) && <TextField
                    label="Differences"
                    value={diffQueries(buildHealthFilter.oldHumanReadableQuery!, buildHealthFilter.newHumanReadableQuery)}
                    multiline
                    rows={10}
                    readOnly
                    resizable={false}
                    styles={{
                        field: { fontFamily: "Consolas, 'Courier New', monospace", whiteSpace: "pre" }
                    }}
                />
            }
            {!isNewFilter && (
                <Label styles={{
                    root: {
                        width: '100%',
                        textAlign: 'right',
                        fontStyle: 'italic',
                        color: theme.semanticColors.bodyBackgroundHovered,
                    }
                }}>
                    Filter Last Updated: {getDateTimeString(buildHealthFilter.lastUpdated)}
                </Label>
            )
            }

            <Stack horizontal tokens={{ childrenGap: 6 }} styles={{ root: { paddingTop: 12 } }}>
                {(isNewFilter || isEditorOwner) && <PrimaryButton disabled={(!isNewFilter && !isOldAndNewDiff)} onClick={() => { handleSaveClick(); }} text="Save" />}
                {!isNewFilter && <PrimaryButton onClick={() => { handleSaveAsCopyClick(); }} text="Save As Copy" />}
                {!isNewFilter && isOldAndNewDiff && <PrimaryButton onClick={() => { handleRevertClick(); }} text="Revert" />}
                {!isNewFilter && <PrimaryButton disabled={!isOwner && !isAdmin} onClick={() => { handleDeleteClick(buildHealthFilter?.filterProjectId); }} text="Delete" />}
            </Stack>
        </Dialog >
    );
};

// #endregion -- Components --