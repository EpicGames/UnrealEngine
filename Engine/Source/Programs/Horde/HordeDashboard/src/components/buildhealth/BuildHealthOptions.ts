// Copyright Epic Games, Inc. All Rights Reserved.

import { makeObservable, observable, runInAction } from "mobx";
import { DEBUG_MODE_PARAM, FILTER_ID_URL_PARAM, INCLUDE_CANCELLED_PARAM, INCLUDE_PREFLIGHT_PARAM, JOB_START_URL_PARAM_ALL_NAMES, JOB_STARTS_URL_PARAM_NAME, LABELS_URL_PARAM_ALL_LABELS, LABELS_URL_PARAM_NAME, PROJECTS_URL_PARAM_NAME, QUERY_SCHEMA_PARAM_NAME, STATES_URL_PARAM_ALL_STATES, STATES_URL_PARAM_NAME, STEPS_URL_PARAM_ALL_STEPS, STEPS_URL_PARAM_NAME, STREAMS_URL_PARAM_NAME, TEMPLATES_URL_PARAM_NAME, TIME_SPAN_URL_PARAM, UI_DATE_ANCHORS_PARAM, UI_SUMMARY_WARNING_AS_ERROR_PARAM } from "../buildhealth/BuildHealthDataTypes";
import { BuildHealthDataHandler } from "./BuildHealthDataHandler";
import { decodeFullyQualifiedLabelName, decodeLabelKey, decodeStateKey, decodeStepKey, decodeTemplateKey, encodeFullyQualifiedLabelName, encodeLabelKey, encodeLabelKeyFromStrings, encodeStateKeyFromString, encodeStepKey, encodeStepNameFromStringsPreserveCase, encodeTemplateKey, encodeTemplateKeyFromStrings } from "./BuildHealthUtilities";
import { Location } from 'react-router-dom';
import LZString from "lz-string"
import { BuildHealthOptionsController, IBuildHealthOptionWriter } from "./BuildHealthOptionsController";
import { JobStepState } from "horde/backend/Api";

// #region -- Options Utilities & Data Types --

/**
 * Perfoms a deep copy of a provided build health query params.
 * @param params The params to perform a deep clone on.
 * @returns The deep clone.
 */
export function cloneBuildHealthQueryParams(params: BuildHealthQueryParams): BuildHealthQueryParams {
    return {
        ...params,
        projectIds: [...params.projectIds],
        streamIds: [...params.streamIds],
        templateIds: [...params.templateIds],
        stepNames: [...params.stepNames],
    };
}

/**
 * Data structure used to pass state from location query, to Build Health Options.
 */
type BuildHealthQueryParams = {
    projectIds: string[];
    streamIds: string[];
    templateIds: string[];
    labels: string[];
    stepNames: string[];
    stateNames: string[];
    jobStartNames: string[];
    timeSpanKey: string;
    includePreflight: boolean;
    includeCancelledJobs: boolean;
    debugMode: boolean;
    querySchemaVersion: number;
    filterId: string | null;
};

/**
 * Data structure used to pass state from query location, to Build Health UI Options.
 */
type BuildHealthUIParams = {
    showDateAnchor: boolean;
    summaryWarningAsErrors: boolean;
}

/**
 * Data type used to specify time span from now to consider for jobs.
 */
export type TimeSpan = {
    text: string;
    key: string;
    minutes: number;
}

/**
* List of currently supported time spans.
*/
export const JobHistoryTimeSpans: TimeSpan[] = [
    {
        text: "Past 15 Minutes", key: "time_15_minutes", minutes: 15
    },
    {
        text: "Past 1 Hour", key: "time_1_hour", minutes: 60
    },
    {
        text: "Past 4 Hours", key: "time_4_hours", minutes: 60 * 4
    },
    {
        text: "Past 1 Day", key: "time_1_day", minutes: 60 * 24
    },
    {
        text: "Past 2 Days", key: "time_2_days", minutes: 60 * 24 * 2
    },
    {
        text: "Past 1 Week", key: "time_1_week", minutes: 60 * 24 * 7
    },
    {
        text: "Past 2 Weeks", key: "time_2_week2", minutes: 60 * 24 * 7 * 2
    },
    {
        text: "Past 1 Month", key: "time_1_month", minutes: 60 * 24 * 31
    }
]

export const DEFAULT_JOB_HISTORY_TIME_SPAN: TimeSpan = JobHistoryTimeSpans[1];

/**
 * Obtains a URLSearchParams representation of the current options.
 * @returns The URLSearchParams representation of the current options.
 */
export function toNavigationQuery(optionsState: BuildHealthOptionsState, uiOptionsState: BuildHealthUIOptionsState, querySchemaVersion: number, buildHealthDataHandler: BuildHealthDataHandler): URLSearchParams {

    // Internal helper method to compress keys to the comrpessed representation.
    // e.g. stream-id::template-id::stepname-1,stream-id::template-id::stepname-2 -> stream-id::template-id(stepname-1;stepname-2)
    function compressKeys(keys: string[], encode: (k: string) => { prefix: string, suffix: string }, groupSummarizer?: (prefix: string, suffixes: string[]) => string | undefined): string {
        const groups: Record<string, string[]> = {};

        for (const key of keys) {
            const { prefix, suffix } = encode(key);
            if (!groups[prefix]) {
                groups[prefix] = [];
            }
            groups[prefix].push(suffix);
        }

        return Object.entries(groups)
            .map(([prefix, suffixes]) => {
                let summarizedSuffix = groupSummarizer ? groupSummarizer(prefix, suffixes) : undefined;
                let finalSuffix = summarizedSuffix ?? suffixes.join(",");

                return `${prefix}(${finalSuffix})`;
            })
            .join(";");
    }

    const params = new URLSearchParams(location.search);
    let projects: string = "";
    let streams: string = "";
    let jobs: string = "";
    let labels: string = "";
    let steps: string = "";
    let states: string = "";
    let jobStarts: string = "";

    projects = Object.keys(optionsState.enabledProjects).join(",");
    streams = optionsState.stepOutcomeEnabledStreamKeys.join(",");
    const lastTimeRange = optionsState.jobHistoryTimeSpan.key.toString();
    const includePreflight = optionsState.includePreflight;
    const includeCancelledJobs = optionsState.includeCancelledJobs;

    // Job & Steps 
    if (querySchemaVersion === 1) {
        jobs = optionsState.stepOutcomeEnabledJobKeys.join(",");
        steps = optionsState.stepOutcomeEnabledStepKeys.join(",");
    }
    else if (querySchemaVersion >= 2) {

        jobs = compressKeys(optionsState.stepOutcomeEnabledJobKeys, (key: string) => {
            let { streamId, templateId } = decodeTemplateKey(key);

            return { prefix: streamId, suffix: templateId };
        });

        steps = compressKeys(optionsState.stepOutcomeEnabledStepKeys,
            (key: string) => {
                let { streamId, templateId, stepName } = decodeStepKey(key);

                return { prefix: encodeTemplateKeyFromStrings(streamId, templateId), suffix: stepName };
            },
            (prefix: string, suffixes: string[]) => {
                let { streamId, templateId } = decodeTemplateKey(prefix);
                let summarized = buildHealthDataHandler.getCachedTemplateStepData(streamId, templateId)?.length === suffixes.length ? STEPS_URL_PARAM_ALL_STEPS : undefined;

                return summarized;
            });
    }

    // States
    if (querySchemaVersion >= 3) {
        states = optionsState.stepOutcomeEnabledsStates.length === Object.keys(JobStepState).length ? STATES_URL_PARAM_ALL_STATES : optionsState.stepOutcomeEnabledsStates.join(",");
    }

    // Labels
    if (querySchemaVersion >= 4) {
        labels = compressKeys(optionsState.stepOutcomeEnabledLabelKeys,
            (key: string) => {
                let { streamId, templateId, dashboardCategory, dashboardName } = decodeLabelKey(key);

                return { prefix: encodeTemplateKeyFromStrings(streamId, templateId), suffix: encodeFullyQualifiedLabelName(dashboardCategory, dashboardName) };
            },
            (prefix: string, suffixes: string[]) => {
                let { streamId, templateId } = decodeTemplateKey(prefix);
                let summarized = buildHealthDataHandler.getCachedTemplateLabelNames(streamId, templateId)?.length === suffixes.length ? LABELS_URL_PARAM_ALL_LABELS : undefined;

                return summarized;
            });
    }

    // Job Starts
    if (querySchemaVersion >= 5) {
        jobStarts = optionsState.stepOutcomeEnabledJobStartMethods.length === ALL_JOB_START_METHODS_LIST.length ? JOB_START_URL_PARAM_ALL_NAMES : optionsState.stepOutcomeEnabledJobStartMethods.join(",");
    }

    params.set(PROJECTS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(projects));
    params.set(STREAMS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(streams));
    params.set(TEMPLATES_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(jobs));
    params.set(LABELS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(labels));
    params.set(STEPS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(steps));
    params.set(STATES_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(states));
    params.set(JOB_STARTS_URL_PARAM_NAME, LZString.compressToEncodedURIComponent(jobStarts));
    params.set(TIME_SPAN_URL_PARAM, LZString.compressToEncodedURIComponent(lastTimeRange));
    params.set(INCLUDE_PREFLIGHT_PARAM, String(includePreflight));
    params.set(INCLUDE_CANCELLED_PARAM, String(includeCancelledJobs));
    params.set(QUERY_SCHEMA_PARAM_NAME, querySchemaVersion.toString());

    // Filter ID is intentionally NOT written here. The URL captures literal/verbose state; a saved
    // filter is shared via the explicit "Share Filter Link" button (in BuildHealthFilterComponent).
    // We do strip any inherited filterId from the seeded location.search so that loading a
    // `?filterId=X` link rewrites to the verbose form after the filter is applied on mount.
    params.delete(FILTER_ID_URL_PARAM);

    // Handle UI Option State
    params.set(UI_DATE_ANCHORS_PARAM, String(uiOptionsState.includeDateAnchors));
    params.set(UI_SUMMARY_WARNING_AS_ERROR_PARAM, String(uiOptionsState.warningsAsSummaryFailure));

    return params;
}

/**
 * Parses the Build Health options from the query params.
 * @param location Location object.
 * @returns Parsed Build Health Query Params.
 */
export function parseBuildHealthQueryParams(searchParams: URLSearchParams): { buildHealthQueryParams: BuildHealthQueryParams, buildHealthUIParams: BuildHealthUIParams } {

    // Internal helper method to expand keys from the comrpessed representation.
    // e.g. stream-id::template-id(stepname-1;stepname-2) -> stream-id::template-id::stepname-1,stream-id::template-id::stepname-2
    function expandKeys(encoded: string, decode: (prefix: string, suffix: string) => string): string[] {
        if (!encoded || !encoded.trim()) return [];

        const result: string[] = [];

        const groups = encoded.split(";");

        for (const group of groups) {
            const match = group.match(/^([^()]+)\((.*)\)$/);
            if (match) {
                const prefix = match[1];
                const suffixes = match[2].split(",");
                for (const suffix of suffixes) {
                    result.push(decode(prefix, suffix));
                }
            } else {
                console.warn("Failed to decode parameter: " + group);
            }
        }

        return result;
    }

    let projectIds: string[] = [];
    let streamIds: string[] = [];
    let templateIds: string[] = [];
    let labels: string[] = [];
    let stepNames: string[] = [];
    let stateNames: string[] = [];
    let jobStartNames: string[] = [];

    const getList = (key: string) => {
        const raw = searchParams.get(key);
        if (!raw) return [];

        const decompressed = LZString.decompressFromEncodedURIComponent(raw);
        if (!decompressed) return [];

        return decompressed.split(",").filter(Boolean);
    };

    const getBoolParam = (key: string): boolean => {
        return searchParams.get(key)?.toLowerCase() === "true";
    }

    const querySchemaVersion = Number(searchParams.get(QUERY_SCHEMA_PARAM_NAME)) || 1;
    projectIds = getList(PROJECTS_URL_PARAM_NAME);
    streamIds = getList(STREAMS_URL_PARAM_NAME);

    // Job & steps
    if (querySchemaVersion === 1) {
        templateIds = getList(TEMPLATES_URL_PARAM_NAME);
        stepNames = getList(STEPS_URL_PARAM_NAME);
    } else if (querySchemaVersion >= 2) {
        const rawTemplates = searchParams.get(TEMPLATES_URL_PARAM_NAME);
        if (!rawTemplates) {
            templateIds = [];
        }
        else {
            let decompressedTemplates = LZString.decompressFromEncodedURIComponent(rawTemplates);
            templateIds = expandKeys(decompressedTemplates, (prefix, suffix) => {
                return encodeTemplateKeyFromStrings(prefix, suffix);
            });
        }

        const rawSteps = searchParams.get(STEPS_URL_PARAM_NAME);
        if (!rawSteps) {
            stepNames = [];
        }
        else {
            let decompressedSteps = LZString.decompressFromEncodedURIComponent(rawSteps);
            stepNames = expandKeys(decompressedSteps, (prefix, suffix) => {
                let { streamId, templateId } = decodeTemplateKey(prefix);

                return encodeStepNameFromStringsPreserveCase(streamId, templateId, suffix);
            });
        }
    }

    // States
    if (querySchemaVersion >= 3) {
        stateNames = getList(STATES_URL_PARAM_NAME);
    }

    // Labels
    if (querySchemaVersion >= 4) {
        const rawLabels = searchParams.get(LABELS_URL_PARAM_NAME);
        if (!rawLabels) {
            labels = [];
        }
        else {
            let decompressedLabels = LZString.decompressFromEncodedURIComponent(rawLabels);
            labels = expandKeys(decompressedLabels, (prefix, suffix) => {
                let { streamId, templateId } = decodeTemplateKey(prefix);
                let { labelCategory, label } = decodeFullyQualifiedLabelName(suffix);
                return encodeLabelKeyFromStrings(streamId, templateId, labelCategory, label);
            });
        }
    }

    // Job Starts
    if (querySchemaVersion >= 5) {
        jobStartNames = getList(JOB_STARTS_URL_PARAM_NAME);
    }

    // Time Spans
    let timeSpanKey: string = (() => {
        const raw = searchParams.get(TIME_SPAN_URL_PARAM);
        return raw ? LZString.decompressFromEncodedURIComponent(raw) ?? "" : "";
    })();

    if (!JobHistoryTimeSpans.some(ts => ts.key === timeSpanKey)) {
        timeSpanKey = DEFAULT_JOB_HISTORY_TIME_SPAN.key;
    }

    // Toggle Query Params
    const includePreflight = getBoolParam(INCLUDE_PREFLIGHT_PARAM);
    const includeCancelledJobs = getBoolParam(INCLUDE_CANCELLED_PARAM);

    // UI params
    const uiWarningsAsErrors = getBoolParam(UI_SUMMARY_WARNING_AS_ERROR_PARAM);
    const uiDateAnchors = getBoolParam(UI_DATE_ANCHORS_PARAM);

    // Debug Modes
    const debugMode = getBoolParam(DEBUG_MODE_PARAM);

    // Filter ID — when present, the URL is asking to load a saved filter by id and populate all controls from it.
    const filterId = searchParams.get(FILTER_ID_URL_PARAM) || null;

    return {
        buildHealthQueryParams: { projectIds, streamIds, templateIds: templateIds, labels, stepNames, stateNames, jobStartNames, timeSpanKey, includePreflight, includeCancelledJobs, debugMode, querySchemaVersion, filterId },
        buildHealthUIParams: { showDateAnchor: uiDateAnchors, summaryWarningAsErrors: uiWarningsAsErrors }
    };
}

/**
 * Performs an upgrade on a provided build healthy query params object. 
 * @param params The params object to attempt to upgrade.
 * @param currentSchemaVersion The target schema version to attempt an upgrade for.
 * @returns If applicable, a cloned params object that has been upgraded. Otherwise returns the original object.
 */
export function upgradeBuildHealthQueryParams(params: BuildHealthQueryParams, currentSchemaVersion: number): BuildHealthQueryParams {
    if (params.querySchemaVersion === currentSchemaVersion) {
        return params;
    }

    let clonedParams = cloneBuildHealthQueryParams(params);
    if (clonedParams.querySchemaVersion === 1 && currentSchemaVersion >= 2) {
        // Upgrades from query structure to 1 -> 2+ we adopted explciit select-all instead of inferred step all
        // - template ids in the query param used to be enough to signify "use all"; we have since moved away from this to a more explicit standard
        // - if there is a template id in the query params, and it has no matching entry in the step filters, infer that it is the STEPS_URL_PARAM_ALL_STEPS case.
        let upgradedSteps: string[] = [];

        for (let idx: number = 0; idx < clonedParams.templateIds.length; idx++) {

            const templateIdKey = clonedParams.templateIds[idx];
            let { streamId, templateId } = decodeTemplateKey(templateIdKey);

            // Create an encoded step key with no step name so we can prefix match against the query param's encoded step keys to see if the template is represented.
            let encodedStepKeyWithNoName = encodeStepNameFromStringsPreserveCase(streamId, templateId, "");

            const hasPrefix = clonedParams.stepNames.some((encodedStep: string) => encodedStep.startsWith(encodedStepKeyWithNoName));

            if (!hasPrefix) {
                upgradedSteps.push(encodeStepNameFromStringsPreserveCase(streamId, templateId, STEPS_URL_PARAM_ALL_STEPS));
            }
        }

        clonedParams.stepNames.push(...upgradedSteps);
    }

    // We must add in the state information, as no such filter existed before
    if (clonedParams.querySchemaVersion < 3 && currentSchemaVersion >= 3) {
        clonedParams.stateNames.push(encodeStateKeyFromString(-1, STATES_URL_PARAM_ALL_STATES));
    }

    // We must add in the job start names as no such filter existed before
    if (clonedParams.querySchemaVersion < 5 && currentSchemaVersion >= 5) {
        clonedParams.jobStartNames.push(...ALL_JOB_START_METHODS_LIST);
    }

    return clonedParams;
}

/**
 * Processes provided Build Health Query Params for the Build Health View.
 * @param params The params to process.
 * @param options The Build Health Option controller to use to update options from the params.
 * @param handler The Build Health Data Handler to use to obtain data based on the params.
 */
export async function loadBuildHealthOptionsFromParams(
    params: BuildHealthQueryParams,
    uiParams: BuildHealthUIParams,
    options: BuildHealthOptionsController,
    handler: BuildHealthDataHandler) {
    let optionsWriter: IBuildHealthOptionWriter = options.getTransactionSession();

    // Determine if we have meaningful URL filters (not just states/jobStarts which are always present)
    const hasUrlFilters = params.projectIds.length > 0 ||
        params.streamIds.length > 0 ||
        params.templateIds.length > 0 ||
        params.labels.length > 0 ||
        params.stepNames.length > 0;

    // Set initializing state with appropriate message
    const message = hasUrlFilters
        ? "Loading filters from URL..."
        : "Loading initial build health data...";
    handler.setInitializing(true, message);

    // Handle UI first
    options.setWarningsAsSummaryFailure(uiParams.summaryWarningAsErrors);
    options.setHideDateAnchors(uiParams.showDateAnchor);

    runInAction(() => {
        optionsWriter.setDebugMode(params.debugMode);
    });

    runInAction(() => {
        optionsWriter.setIncludePreflight(params.includePreflight);
    });

    runInAction(() => {
        optionsWriter.setIncludeCancelledJobs(params.includeCancelledJobs);
    });

    if (params.timeSpanKey) {
        runInAction(() => {
            optionsWriter.setJobHistoryTimeSpan(JobHistoryTimeSpans.find(ts => ts.key === params.timeSpanKey)!);
        });
    }

    // Set States
    if (params.stateNames) {
        runInAction(() => {
            optionsWriter.clearStates();
        });

        if (params.stateNames.length === 1 && params.stateNames[0].includes(STATES_URL_PARAM_ALL_STATES)) {
            optionsWriter.toggleAllStates(ALL_STATES_TUPLE_LIST.map(entry => entry.key), true);
        } else if (params.stateNames.length > 0) {
            for (let idx: number = 0; idx < params.stateNames.length; ++idx) {
                let { index, stateName } = decodeStateKey(params.stateNames[idx]);
                optionsWriter.toggleState(params.stateNames[idx], stateName, true);
            }
        }
    }

    // Set Job Starts
    if (params.jobStartNames) {
        runInAction(() => {
            optionsWriter.clearJobStartMethods();
        });

        if (params.jobStartNames.length === 1 && params.jobStartNames[0].includes(JOB_START_URL_PARAM_ALL_NAMES)) {
            optionsWriter.toggleAllJobStartMethods(ALL_JOB_START_METHODS_LIST, true);
        } else if (params.jobStartNames.length > 0) {
            for (let idx: number = 0; idx < params.jobStartNames.length; ++idx) {
                optionsWriter.toggleJobStartMethod(params.jobStartNames[idx], params.jobStartNames[idx], true);
            }
        }
    }

    // Get baseline project data first
    await handler.requestHierarchicalRefresh();

    runInAction(() => {
        optionsWriter.clearProjects();
        params.projectIds.forEach(pid => {
            const project = handler.projectsData.find(p => p.id === pid);
            if (project) {
                optionsWriter.toggleProject(pid, project.name, true);
            }
            else {
                console.warn(`Project parameter provided in URL query (${pid}) that could not be matched to underlying data - discarding.`);
            }
        });
    });

    await handler.getStreamData({ streams: true });

    runInAction(() => {
        optionsWriter.clearStreams();
        params.streamIds.forEach(sid => {
            const stream = handler.streamsData.find(s => s.id === sid);
            if (stream) {
                optionsWriter.toggleStream(sid, stream.name, true);
            }
            else {
                console.warn(`Stream parameter provided in URL query (${sid}) that could not be matched to underlying data - discarding.`);
            }
        });
    });

    await handler.getTemplateData({ jobs: true });

    runInAction(() => {
        optionsWriter.clearTemplates();
        const decodedTemplates = params.templateIds.map(templateKey => decodeTemplateKey(templateKey));

        decodedTemplates.forEach(tid => {
            const template = handler.templatesData.find(t => t.id === tid.templateId && t.streamId == tid.streamId);
            if (template) {
                optionsWriter.toggleTemplate(encodeTemplateKey(template), template.name, true);
            }
            else {
                console.warn(`Template parameter provided in URL query (${tid}) that could not be matched to underlying data - discarding.`);
            }
        });
    });

    await handler.getStepData({ steps: true });

    runInAction(() => {
        optionsWriter.clearSteps();
        const decodedStepNames = params.stepNames.map(stepNameKey => decodeStepKey(stepNameKey));

        decodedStepNames.forEach(stepData => {
            const step = handler.stepData.find(s => s.name.toLowerCase() === stepData.stepName.toLowerCase() && s.streamId == stepData.streamId && s.templateId == stepData.templateId);
            if (step) {
                optionsWriter.toggleStep(encodeStepKey(step), step.name, true);
            } else if (stepData.stepName.toLocaleLowerCase() === STEPS_URL_PARAM_ALL_STEPS.toLocaleLowerCase()) {
                let stepNames = handler.getCachedTemplateStepNames(stepData.streamId, stepData.templateId)
                optionsWriter.toggleAllSteps(stepData.streamId, stepData.templateId, stepNames, true);
            }
            else {
                console.warn(`Step name parameter provided in URL query (${stepData.stepName}) that could not be matched to underlying data - discarding.`);
            }
        });
    });

    runInAction(() => {
        optionsWriter.clearLabels();
        const decodedLabels = params.labels.map(labelKey => decodeLabelKey(labelKey));

        decodedLabels.forEach(labelData => {
            const label = handler.labelData.find(l => l.streamId == labelData.streamId
                && l.templateId == labelData.templateId
                && l.dashboardCategory?.toLowerCase() === labelData.dashboardCategory.toLowerCase()
                && l.dashboardName?.toLowerCase() === labelData.dashboardName.toLowerCase());
            if (label) {
                optionsWriter.toggleLabel(encodeLabelKey(label), encodeFullyQualifiedLabelName(labelData.dashboardCategory, labelData.dashboardName), true);
            } else if (labelData.dashboardCategory?.toLocaleLowerCase() === LABELS_URL_PARAM_ALL_LABELS.toLocaleLowerCase()) {
                let labelNames = handler.getCachedTemplateLabelNames(labelData.streamId, labelData.templateId)
                optionsWriter.toggleAllLabels(labelData.streamId, labelData.templateId, labelNames, true);
            }
            else {
                console.warn(`Label parameter provided in URL query (${encodeFullyQualifiedLabelName(labelData.dashboardCategory, labelData.dashboardName)}) that could not be matched to underlying data - discarding.`);
            }
        });
    });

    options.commitTransactionSession();

    // Clear initializing state
    handler.setInitializing(false);
}

// #endregion -- Options Utilities & Data Types --

/**
 * Build Health View Options.
 */
export class BuildHealthUIOptionsState {
    @observable includeDateAnchors: boolean = false;
    @observable warningsAsSummaryFailure: boolean = false;
    @observable linkedSelectionModeEnabled: boolean = false;
    @observable validStates: Set<string> = new Set<string>();
    @observable debugMode: boolean = false;

    constructor() {
        makeObservable(this);
    }
}

/**
 * Interface that describes the difference between a property of type T.
 */
export interface FlagDiff<T> {
    old: T;
    new: T;
}

/**
 * Interface that describes the difference between two option states.
 */
export interface BuildHealthOptionStateDiff {
    added?: BuildHealthOptionStateReceipt;
    removed?: BuildHealthOptionStateReceipt;
    enabledPreflight?: FlagDiff<boolean>;
    timespan?: FlagDiff<TimeSpan>;
    hasDiff?: boolean;
}

/**
 * A snapshot of the build health options.
 */
export class BuildHealthOptionStateReceipt {
    enabledProjects: ReadonlySet<string>;
    enabledStreams: ReadonlySet<string>;
    enabledTemplates: ReadonlySet<string>;
    enabledLabels: ReadonlySet<string>;
    enabledSteps: ReadonlySet<string>;
    enabledStates: ReadonlySet<string>;
    enabledJobStartMethods: ReadonlySet<string>;
    enabledPreflight: boolean;
    enabledCancelledJobs: boolean;
    timespan: TimeSpan;

    constructor(
        projects: Iterable<string> = [],
        streams: Iterable<string> = [],
        templates: Iterable<string> = [],
        labels: Iterable<string> = [],
        steps: Iterable<string> = [],
        states: Iterable<string> = [],
        jobStartMethods: Iterable<string> = [],
        enabledPreflight: boolean,
        enabledCancelledJobs: boolean,
        timespan: TimeSpan
    ) {
        this.enabledProjects = new Set(projects);
        this.enabledStreams = new Set(streams);
        this.enabledTemplates = new Set(templates);
        this.enabledLabels = new Set(labels);
        this.enabledSteps = new Set(steps);
        this.enabledStates = new Set(states);
        this.enabledJobStartMethods = new Set(jobStartMethods);
        this.enabledPreflight = enabledPreflight;
        this.enabledCancelledJobs = enabledCancelledJobs;
        this.timespan = timespan;
    }

    /**
     * Produces a build health option state diff between two provided receipts.
     * @param prev The previous option state.
     * @param next The next option state.
     * @returns The diff between the two receipts.
     */
    static diff(
        prev: BuildHealthOptionStateReceipt,
        next: BuildHealthOptionStateReceipt
    ): BuildHealthOptionStateDiff {

        const added = new BuildHealthOptionStateReceipt(
            BuildHealthOptionStateReceipt.difference(next.enabledProjects, prev.enabledProjects),
            BuildHealthOptionStateReceipt.difference(next.enabledStreams, prev.enabledStreams),
            BuildHealthOptionStateReceipt.difference(next.enabledTemplates, prev.enabledTemplates),
            BuildHealthOptionStateReceipt.difference(next.enabledLabels, prev.enabledLabels),
            BuildHealthOptionStateReceipt.difference(next.enabledSteps, prev.enabledSteps),
            BuildHealthOptionStateReceipt.difference(next.enabledStates, prev.enabledStates),
            BuildHealthOptionStateReceipt.difference(next.enabledJobStartMethods, prev.enabledJobStartMethods),
            next.enabledPreflight,
            next.enabledCancelledJobs,
            next.timespan
        );

        const removed = new BuildHealthOptionStateReceipt(
            BuildHealthOptionStateReceipt.difference(prev.enabledProjects, next.enabledProjects),
            BuildHealthOptionStateReceipt.difference(prev.enabledStreams, next.enabledStreams),
            BuildHealthOptionStateReceipt.difference(prev.enabledTemplates, next.enabledTemplates),
            BuildHealthOptionStateReceipt.difference(prev.enabledLabels, next.enabledLabels),
            BuildHealthOptionStateReceipt.difference(prev.enabledSteps, next.enabledSteps),
            BuildHealthOptionStateReceipt.difference(prev.enabledStates, next.enabledStates),
            BuildHealthOptionStateReceipt.difference(prev.enabledJobStartMethods, next.enabledJobStartMethods),
            prev.enabledPreflight,
            prev.enabledCancelledJobs,
            prev.timespan
        );

        const enabledPreflight: FlagDiff<boolean> = { old: prev.enabledPreflight, new: next.enabledPreflight };
        const timespan: FlagDiff<TimeSpan> = { old: prev.timespan, new: next.timespan };
        const preflightChanged = prev.enabledPreflight !== next.enabledPreflight;
        const timespanChanged = prev.timespan !== next.timespan;
        const hasDiff = this.receiptHasChanges(added) || this.receiptHasChanges(removed) || preflightChanged || timespanChanged;

        return { added, removed, enabledPreflight, timespan, hasDiff };
    }

    private static receiptHasChanges(r: BuildHealthOptionStateReceipt): boolean {
        return (
            r.enabledProjects.size > 0 ||
            r.enabledStreams.size > 0 ||
            r.enabledTemplates.size > 0 ||
            r.enabledLabels.size > 0 ||
            r.enabledSteps.size > 0 ||
            r.enabledStates.size > 0 ||
            r.enabledJobStartMethods.size > 0
        );
    }

    private static difference<T>(a: ReadonlySet<T>, b: ReadonlySet<T>): Set<T> {
        const out = new Set<T>();
        for (const item of a) {
            if (!b.has(item)) out.add(item);
        }
        return out;
    }
}

export const ALL_STATES_TUPLE_LIST = Object.values(JobStepState).map(
    (stateName, idx): { key: string; stateName: string } => ({
        key: encodeStateKeyFromString(idx, stateName),
        stateName: stateName
    })
);

export const SCHEDULED_START_METHOD = "Scheduled";
export const MANUAL_START_METHOD = "Manual";
export const ALL_JOB_START_METHODS_LIST = [MANUAL_START_METHOD, SCHEDULED_START_METHOD];

/**
 * Build Health Filter Options.
 */
export class BuildHealthOptionsState {

    // #region -- Published Options --

    @observable stepOutcomeEnabledStreamKeys: string[] = [];
    @observable stepOutcomeEnabledJobKeys: string[] = [];
    @observable stepOutcomeEnabledLabelKeys: string[] = [];
    @observable stepOutcomeEnabledStepKeys: string[] = [];
    @observable stepOutcomeEnabledsStates: string[] = [];
    @observable stepOutcomeEnabledJobStartMethods: string[] = [];
    @observable startDate!: Date;
    @observable endDate?: Date;

    // #endregion -- Published Options --

    // #region -- User Selected Options --
    
    @observable buildHealthFilterId : string | null = null; 
    @observable enabledProjects: Record<string, string> = {};
    @observable enabledProjectFilter: string;
    @observable enabledStreams: Record<string, string> = {};
    @observable enabledTemplates: Record<string, string> = {};
    @observable enabledLabels: Record<string, string> = {};
    @observable enabledSteps: Record<string, string> = {};
    @observable enabledStates: Record<string, string> = {};
    @observable enabledJobStartMethods: Record<string, string> = {};
    @observable jobHistoryTimeSpan: TimeSpan = DEFAULT_JOB_HISTORY_TIME_SPAN;
    @observable includePreflight: boolean = false;
    @observable includeCancelledJobs: boolean = true;
    @observable debugMode: boolean = false;

    // #endregion -- User Selected Options --

    // #region -- Constructor --

    constructor() {
        makeObservable(this);
        this.endDate = new Date();

        const d = new Date();
        d.setDate(d.getDate() - 1);
        this.startDate = d;
    }

    // #endregion -- Constructor --

    // #region -- Public API --

    /**
     * Generates a state receipt for the options in it's current state.
     * @returns A receipt representation of the current build health options.
     */
    generateStateReceipt(): BuildHealthOptionStateReceipt {
        let receipt: BuildHealthOptionStateReceipt = new BuildHealthOptionStateReceipt(
            Object.keys(this.enabledProjects),
            Object.keys(this.enabledStreams),
            Object.keys(this.enabledTemplates),
            Object.keys(this.enabledLabels),
            Object.keys(this.enabledSteps),
            Object.values(this.enabledStates),
            Object.values(this.enabledJobStartMethods),
            this.includePreflight,
            this.includeCancelledJobs,
            this.jobHistoryTimeSpan
        );

        return receipt;
    }

    // #endregion -- Public API --
}