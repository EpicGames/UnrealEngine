import { action, makeObservable } from "mobx";
import { BuildHealthOptionsState, BuildHealthOptionStateDiff, BuildHealthOptionStateReceipt, BuildHealthUIOptionsState, TimeSpan } from "./BuildHealthOptions";
import { decodeFullyQualifiedLabelName, decodeLabelKey, decodeStateKey, decodeStepKey, decodeTemplateKey, encodeFullyQualifiedLabelName, encodeLabelKeyFromStrings, encodeStepNameFromStringsPreserveCase, encodeTemplateKey, encodeTemplateKeyFromStrings } from "./BuildHealthUtilities";
import { BuildHealthDataHandler } from "./BuildHealthDataHandler";

/**
 * An interface that can provide hierarchy checks on stream, template, and step relationships.
 */
export interface IHierarchyValidator {
    /**
     * Determines whether a stream & template combination is valid (in that it exists).
     * @param streamId The stream.
     * @param templateId The template.
     * @returns True if this stream has a template of the given id, false otherwise.
     */
    isValidTemplate(streamId: string, templateId: string): boolean;

    /**
     * Determines whether a stream, template, and step combination is valid (in that it exists).
     * @param streamId The stream.
     * @param templateId The template.
     * @param stepName The step name.
     * @returns True if this stream has a template of the given id, which has a step of the given name, false otherwise.
     */
    isValidStep(streamId: string, templateId: string, stepName: string): boolean;

    /**
     * Determines whether a stream, template, and label combination is valid (in that it exists).
     * @param streamId The stream.
     * @param templateId The template.
     * @param fullyQualifiedLabelName The fully qualified label name.
     * @returns True if this stream has a template of the given id, which has a label of the given name, false otherwise.
     */
    isValidLabel(streamId: string, templateId: string, fullyQualifiedLabelName: string): boolean;
}

/**
 * Data interface to describe a stream.
 */
export interface StreamDescriptor {
    /**
     * The id of the stream.
     */
    id: string;

    /**
     * The name of the stream.
     */
    name: string;
}

/**
 * Data interface to describe a template.
 */
export interface TemplateDescriptor {
    /**
     * The id of the template.
     */
    id: string;

    /**
     * The name of the template.
     */
    name: string;
}

/**
 * Interface that describes all the writeable operations on the build health options.
 */
export interface IBuildHealthOptionWriter {

    /**
     * Produces the set of additions and removals for all options.
     */
    produceReceiptDiff(): BuildHealthOptionStateDiff;

    /**
     * Sets whether to filter out cancelled jobs, or not.
     * @param includeCancelledJobs True to include cancelled jobs, false otherwise.
     */
    setIncludeCancelledJobs(includeCancelledJobs: boolean): void;

    /**
     * Sets whether to filter out preflight jobs, or not.
     * @param includePreflight True to include preflight job instances, false otherwise.
     */
    setIncludePreflight(includePreflight: boolean): void;

    /**
     * Sets whether the system is in debug mode or not.
     * @param debugMode True to indicate debug mode, false otherwise.
     */
    setDebugMode(debugMode: boolean): void;

    /**
     * Sets the current job history time span to utilize.
     * @param time The time span to utilize.
     */
    setJobHistoryTimeSpan(time: TimeSpan): void;

    /**
     * Toggles the state of the provided project.
     * @param id The id of the project to toggle.
     * @param name The name of the project.
     * @param enabled Whether it's enabled or not.
     */
    toggleProject(id: string, name: string, enabled: boolean): void;

    /**
     * Toggles the state of the provided project. Will clear all other set projects.
     * @param id The id of the project to toggle.
     * @param name The name of the project.
     * @param enabled Whether it's enabled or not.
     */
    toggleSingleProject(id: string, name: string): void;

    /**
     * Toggles all provided streams.
     * @param streamDescriptors The list of streams to attempt to toggle. 
     * @param enabled Whether it's enabled or not.
     */
    toggleAllStreams(streamDescriptors: StreamDescriptor[], enabled: boolean): void;

    /**
     * Toggles the state of the provided stream.
     * @param id The id of the stream to toggle.
     * @param name The name of the stream.
     * @param enabled Whether it's enabled or not.
     */
    toggleStream(id: string, name: string, enabled: boolean): void;

    /**
     * Toggles the state of the provided stream. Will clear all other set streams.
     * @param id The id of the stream to toggle.
     * @param name The name of the stream.
     * @param enabled Whether it's enabled or not.
     */
    toggleSingleStream(id: string, name: string): void;

    /**
     * Toggles all linkable templates, for all active streams.
     * @param enabled Whether it's enabled or not.
     * @param queryOnly Whether to query the linkability of templates solely, or not. If false, apply the toggle action.
     * @returns Whether a linking action is required happened.
     */
    toggleAllLinkableTemplates(enabled: boolean, queryOnly?: boolean): boolean;

    /**
     * Toggles all provided templates for a given stream.
     * @param streamId The stream.
     * @param templateDescriptors The list of templates to attempt to toggle. 
     * @param enabled  Whether it's enabled or not.
     */
    toggleAllTemplates(streamId: string, templateDescriptors: TemplateDescriptor[], enabled: boolean): void;

    /**
     * Toggles the state of the provided template.
     * @param id The id of the template to toggle.
     * @param name The name of the template.
     * @param enabled Whether it's enabled or not.
     */
    toggleTemplate(id: string, name: string, enabled: boolean): void;

    /**
     * Toggles all linkable steps, for all active templates.
     * @param enabled True if we want to enable, false otherwise.
     * @param queryOnly Whether to query the linkability of steps solely, or not. If false, apply the toggle action.
     * @returns Whether a linking action is required happened.
     */
    toggleAllLinkableSteps(enabled: boolean, queryOnly?: boolean): boolean;

    /**
     * Toggles all provided steps for a given stream & template, given they are valid.
     * @param streamId The stream.
     * @param templateId The template.
     * @param stepNames The list of steps to toggle.
     * @param enabled True to enable, false otherwise.
     * @param cascadeLinkedAction True to enable linked action propagation (default behaviour), false or otherwise.
     */
    toggleAllSteps(streamId: string, templateId: string, stepNames: string[], enabled: boolean, cascadeLinkedAction?: boolean): void;

    /**
     * Toggles all linkable labels, for all active templates.
     * @param enabled True if we want to enable, false otherwise.
     * @param queryOnly Whether to query the linkability of labels solely, or not. If false, apply the toggle action.
     * @returns Whether a linking action is required happened.
     */
    toggleAllLinkableLabels(enabled: boolean, queryOnly?: boolean): boolean;

    /**
     * Toggles all provided labels for a given stream & template, given they are valid.
     * @param streamId The stream.
     * @param templateId The template.
     * @param labels The list of labels to toggle.
     * @param enabled True to enable, false otherwise.
     * @param cascadeLinkedAction True to enable linked action propagation (default behaviour), false or otherwise.
     */
    toggleAllLabels(streamId: string, templateId: string, labels: string[], enabled: boolean, cascadeLinkedAction?: boolean): void;

    /**
     * Toggles the state of the provided step.
     * @param id The id of the step to toggle.
     * @param name The name of the step.
     * @param enabled Whether it's enabled or not.
     */
    toggleStep(id: string, name: string, enabled: boolean): void;

    /**
     * Toggles the state of the provided label.
     * @param id The id of the label to toggle.
     * @param name The fully qualified name of the label.
     * @param enabled Whether it's enabled or not.
     */
    toggleLabel(id: string, name: string, enabled: boolean): void;

    /**
     * Toggles all provided states.
     * @param states List of states to enable.
     * @param enabled True to enable, false otherwise.
     */
    toggleAllStates(states: string[], enabled: boolean): void;

    /**
     * Toggles the state of the provided JobStepState.
     * @param id The id of the state to toggle.
     * @param name The name of the state.
     * @param enabled Whether it's enabled or not.
     */
    toggleState(id: string, state: string, enabled: boolean): void;

    /**
     * Toggles all provided methods.
     * @param methods List of methods to enable.
     * @param enabled True to enable, false otherwise.
     */
    toggleAllJobStartMethods(methods: string[], enabled: boolean): void;

    /**
     * Toggles the state of the job start method.
     * @param id The id of the start method to toggle.
     * @param method The name of the start method.
     * @param enabled Whether it's enabled or not.
     */
    toggleJobStartMethod(id: string, method: string, enabled: boolean): void;

    /**
     * Clears all currently selected options.
     */
    clearAll(): void;

    /**
      * Clear all the projects tracked by the options.
      * @param cascade Whether to cascade this down to dependent options.
      */
    clearProjects(cascade?: boolean): void;

    /**
      * Clear all the streams tracked by the options.
      * @param cascade Whether to cascade this down to dependent options.
      */
    clearStreams(cascade?: boolean): void;

    /**
      * Clear all the templates tracked by the options.
      * @param cascade Whether to cascade this down to dependent options.
      * @param owningStreamId Whether to clear an item if it belongs to a stream. If not provided, will remove any template.
      */
    clearTemplates(cascade?: boolean, owningStreamId?: string): void;

    /**
     * Clear all the labels tracked by the options.
     * @param owningHierarchy Whether to clear an item if it belongs to a stream & template. If not provided, will remove any label.
     */
    clearLabels(owningHierarchy?: { streamId: string; templateId: string }): void;

    /**
     * Clear all the steps tracked by the options.
     * @param owningHierarchy Whether to clear an item if it belongs to a stream & template. If not provided, will remove any step.
     */
    clearSteps(owningHierarchy?: { streamId: string; templateId: string }): void;

    /**
     * Clear all the states tracked by the options.
     */
    clearStates(): void;

    /**
     * Clear all the job start methods tracked by the options.
     */
    clearJobStartMethods(): void;
}

export class BuildHealthOptionsWriter implements IBuildHealthOptionWriter {
    readonly state: BuildHealthOptionsState;
    readonly uiState: BuildHealthUIOptionsState;
    readonly hierarchyValidator?: IHierarchyValidator;
    private activeTransactionReceipt: BuildHealthOptionStateReceipt;

    constructor(state: BuildHealthOptionsState, uiState: BuildHealthUIOptionsState, hierarhcyValidator?: IHierarchyValidator) {
        makeObservable(this);
        this.state = state;
        this.uiState = uiState;
        this.activeTransactionReceipt = this.state.generateStateReceipt();
        this.hierarchyValidator = hierarhcyValidator;
    }

    /**
     * Produces the current receipt diff between the receipt from the start of the session, and the current receipt.
     * @returns The diff receipts: added items & deleted items.
     */
    produceReceiptDiff(): BuildHealthOptionStateDiff {
        let transactionStartReceipt = this.state.generateStateReceipt();
        let returnSet = BuildHealthOptionStateReceipt.diff(this.activeTransactionReceipt, transactionStartReceipt);
        return returnSet;
    }

    @action
    clearAll() {
        Object.keys(this.state.enabledProjects).forEach(key => {
            delete this.state.enabledProjects[key];
        });
        this.clearStreams(true);
        this.clearStates();
        this.clearJobStartMethods();
        this.state.buildHealthFilterId = null;
    }

    /**
     * @inheritdoc
     */
    @action
    toggleSingleProject(id: string, name: string) {
        this.state.enabledProjects = { [id]: name };
        this.clearStreams(true);
    }

    /**
     * @inheritdoc
     */
    @action
    toggleProject(id: string, name: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledProjects[id] = name;
        } else {
            delete this.state.enabledProjects[id];
            this.clearStreams(true);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllStreams(streamDescriptors: StreamDescriptor[], enabled: boolean): void {
        for (let idx: number = 0; idx < streamDescriptors.length; ++idx) {
            let streamDescriptor: StreamDescriptor = streamDescriptors[idx];
            this.toggleStream(streamDescriptor.id, streamDescriptor.name, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleSingleStream(id: string, name: string) {
        this.clearStreams(true);
        this.state.enabledStreams[id] = name;
    }

    /**
     * @inheritdoc
     */
    @action
    toggleStream(id: string, name: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledStreams[id] = name;
        } else {
            delete this.state.enabledStreams[id];
            this.clearTemplates(true, id);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllTemplates(streamId: string, templateDescriptors: StreamDescriptor[], enabled: boolean, cascadeLinkedAction?: boolean): void {
        for (let idx: number = 0; idx < templateDescriptors.length; ++idx) {
            let templateDescriptor: StreamDescriptor = templateDescriptors[idx];
            let encodedTemplateKey = encodeTemplateKeyFromStrings(streamId, templateDescriptor.id);
            if (cascadeLinkedAction) {
                this.toggleTemplate(encodedTemplateKey, templateDescriptor.name, enabled);
            } else {
                this.toggleTemplateInternal(encodedTemplateKey, templateDescriptor.name, enabled);
            }
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleTemplate(id: string, name: string, enabled: boolean) {
        this.toggleTemplateInternal(id, name, enabled);
        if (this.uiState.linkedSelectionModeEnabled) {
            this.toggleLinkedTemplates(id, name, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllLinkableTemplates(enabled: boolean, queryOnly: boolean = false) {
        let requiresLinking: boolean = false;
        if (this.uiState.linkedSelectionModeEnabled) {
            Object.keys(this.state.enabledTemplates).forEach(templateKey => {
                let templateName = this.state.enabledTemplates[templateKey];
                requiresLinking ||= this.toggleLinkedTemplates(templateKey, templateName, enabled, queryOnly);
            });
        }

        return requiresLinking;
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllLinkableLabels(enabled: boolean, queryOnly: boolean = false): boolean {
        let requiresLinking: boolean = false;
        if (this.uiState.linkedSelectionModeEnabled) {
            Object.keys(this.state.enabledLabels).forEach(labelKey => {
                let fullyQualifiedLabelName = this.state.enabledLabels[labelKey];
                requiresLinking ||= this.toggleLinkedLabels(labelKey, fullyQualifiedLabelName, enabled, queryOnly);
            });
        }

        return requiresLinking;
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllLinkableSteps(enabled: boolean, queryOnly: boolean = false): boolean {
        let requiresLinking: boolean = false;
        if (this.uiState.linkedSelectionModeEnabled) {
            Object.keys(this.state.enabledSteps).forEach(stepKey => {
                let stepName = this.state.enabledSteps[stepKey];
                requiresLinking ||= this.toggleLinkedSteps(stepKey, stepName, enabled, queryOnly);
            });
        }

        return requiresLinking;
    }

    /**
     * @inheritdoc
     */
    @action
    toggleLabel(id: string, name: string, enabled: boolean) {
        this.toggleLabelInternal(id, name, enabled);
        if (this.uiState.linkedSelectionModeEnabled) {
            this.toggleLinkedLabels(id, name, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleStep(id: string, name: string, enabled: boolean) {
        this.toggleStepInternal(id, name, enabled);
        if (this.uiState.linkedSelectionModeEnabled) {
            this.toggleLinkedSteps(id, name, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleState(id: string, state: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledStates[id] = state;
        } else {
            delete this.state.enabledStates[id];
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleJobStartMethod(id: string, method: string, enabled: boolean): void {
        if (enabled) {
            this.state.enabledJobStartMethods[id] = method;
        } else {
            delete this.state.enabledJobStartMethods[id];
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllSteps(streamId: string, templateId: string, stepNames: string[], enabled: boolean, cascadeLinkedAction: boolean = true) {
        for (let idx: number = 0; idx < stepNames.length; ++idx) {
            let stepName = stepNames[idx];
            let toggleAllStepId = encodeStepNameFromStringsPreserveCase(streamId, templateId, stepName);

            if (cascadeLinkedAction) {
                this.toggleStep(toggleAllStepId, stepName, enabled);
            } else {
                this.toggleStepInternal(toggleAllStepId, stepName, enabled);
            }
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllLabels(streamId: string, templateId: string, labels: string[], enabled: boolean, cascadeLinkedAction: boolean = true) {
        for (let idx: number = 0; idx < labels.length; ++idx) {
            let label = labels[idx];
            let decodedFullyQualifiedLabelName = decodeFullyQualifiedLabelName(label);
            let toggleAllLabelId = encodeLabelKeyFromStrings(streamId, templateId, decodedFullyQualifiedLabelName.labelCategory, decodedFullyQualifiedLabelName.label);

            if (cascadeLinkedAction) {
                this.toggleLabel(toggleAllLabelId, label, enabled);
            }
            this.toggleLabelInternal(toggleAllLabelId, label, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllStates(states: string[], enabled: boolean) {
        for (let idx: number = 0; idx < states.length; ++idx) {
            let { index, stateName } = decodeStateKey(states[idx]);
            this.toggleState(states[idx], stateName, enabled);
        }
    }

    /**
     * @inheritdoc
     */
    @action
    toggleAllJobStartMethods(methods: string[], enabled: boolean) {
        for (let idx: number = 0; idx < methods.length; ++idx) {
            this.toggleJobStartMethod(methods[idx], methods[idx], enabled);
        }
    }


    /**
     * @inheritdoc
     */
    @action
    setJobHistoryTimeSpan(time: TimeSpan) {
        this.state.jobHistoryTimeSpan = time;
    }

    /**
     * @inheritdoc
     */
    @action
    setIncludeCancelledJobs(includeCancelledJobs: boolean) {
        this.state.includeCancelledJobs = includeCancelledJobs;
    }

    /**
     * @inheritdoc
     */
    @action
    setIncludePreflight(includePreflight: boolean) {
        this.state.includePreflight = includePreflight;
    }

    /**
     * @inheritdoc
     */
    @action
    setDebugMode(debugMode: boolean) {
        this.state.debugMode = debugMode;
    }

    /**
     * @inheritdoc
     */
    clearProjects(cascade?: boolean) {
        Object.keys(this.state.enabledProjects).forEach(key => {
            delete this.state.enabledProjects[key];
        });

        if (cascade) {
            this.clearStreams(cascade);
        }
    }

    /**
     * @inheritdoc
     */
    clearStreams(cascade?: boolean) {
        Object.keys(this.state.enabledStreams).forEach(key => {
            delete this.state.enabledStreams[key];

            if (cascade) {
                this.clearTemplates(cascade, key);
            }
        });
    }

    /**
     * @inheritdoc
     */
    clearTemplates(cascade?: boolean, owningStreamId?: string) {
        Object.keys(this.state.enabledTemplates).forEach(key => {
            let { streamId, templateId } = decodeTemplateKey(key);
            if (!owningStreamId || (streamId === owningStreamId)) {
                delete this.state.enabledTemplates[key];

                if (cascade) {
                    this.clearLabels(owningStreamId ? { streamId: owningStreamId!, templateId: templateId } : undefined);
                    this.clearSteps(owningStreamId ? { streamId: owningStreamId!, templateId: templateId } : undefined);
                }
            }
        });
    }

    /**
     * @inheritdoc
     */
    clearLabels(owningHierarchy?: { streamId: string; templateId: string }) {
        Object.keys(this.state.enabledLabels).forEach(key => {
            let { streamId, templateId, dashboardCategory, dashboardName } = decodeLabelKey(key);
            if ((!owningHierarchy) || (streamId === owningHierarchy.streamId && templateId === owningHierarchy.templateId)) {
                delete this.state.enabledLabels[key];
            }
        });
    }

    /**
     * @inheritdoc
     */
    clearSteps(owningHierarchy?: { streamId: string; templateId: string }) {
        Object.keys(this.state.enabledSteps).forEach(key => {
            let { streamId, templateId, stepName } = decodeStepKey(key);
            if ((!owningHierarchy) || (streamId === owningHierarchy.streamId && templateId === owningHierarchy.templateId)) {
                delete this.state.enabledSteps[key];
            }
        });
    }

    /**
     * @inheritdoc
     */
    clearStates() {
        Object.keys(this.state.enabledStates).forEach(key => {
            delete this.state.enabledStates[key];
        });
    }

    /**
     * @inheritdoc
     */
    clearJobStartMethods() {
        Object.keys(this.state.enabledJobStartMethods).forEach(key => {
            delete this.state.enabledJobStartMethods[key];
        });
    }

    // #region -- Private API --

    private toggleTemplateInternal(id: string, name: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledTemplates[id] = name;
        } else {
            delete this.state.enabledTemplates[id];
            let { streamId, templateId } = decodeTemplateKey(id);
            this.clearLabels({ streamId, templateId });
            this.clearSteps({ streamId, templateId });
        }
    }

    private toggleLinkedTemplates(sourceId: string, name: string, enabled: boolean, queryOnly: boolean = false): boolean {
        let { streamId, templateId } = decodeTemplateKey(sourceId);
        let requiresLink: boolean = false;
        Object.keys(this.state.enabledStreams).forEach(streamKey => {
            if (streamKey !== streamId) {
                let linkedTempalteId = encodeTemplateKeyFromStrings(streamKey, templateId);
                if (this.hierarchyValidator?.isValidTemplate(streamKey, templateId)) {
                    let areDifferentStates: boolean = this.state.enabledTemplates[linkedTempalteId] != this.state.enabledTemplates[sourceId];
                    if (areDifferentStates) {
                        if (!queryOnly) {
                            this.toggleTemplateInternal(linkedTempalteId, name, enabled);
                        } else {
                            requiresLink ||= true;
                        }
                    }
                }
            }
        });
        return requiresLink;
    }

    private toggleLabelInternal(id: string, name: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledLabels[id] = name;
        } else {
            delete this.state.enabledLabels[id];
        }
    }

    private toggleLinkedLabels(sourceId: string, name: string, enabled: boolean, queryOnly: boolean = false): boolean {
        let sourceDecodedLabelKey = decodeLabelKey(sourceId);
        let requiresLink: boolean = false;
        Object.keys(this.state.enabledTemplates).forEach(key => {
            let targetDecodedTempalteKey = decodeTemplateKey(key);
            // Set this label on every other enabled template, except the narrow case of the stream-template-label - we've just come from there.
            if (targetDecodedTempalteKey.templateId !== sourceDecodedLabelKey.templateId || targetDecodedTempalteKey.streamId !== sourceDecodedLabelKey.streamId) {
                let linkedEncodedLabelId = encodeLabelKeyFromStrings(targetDecodedTempalteKey.streamId, targetDecodedTempalteKey.templateId, sourceDecodedLabelKey.dashboardCategory, sourceDecodedLabelKey.dashboardName);
                if (this.hierarchyValidator?.isValidLabel(targetDecodedTempalteKey.streamId, targetDecodedTempalteKey.templateId, name)) {
                    let areDifferentStates: boolean = this.state.enabledLabels[linkedEncodedLabelId] != this.state.enabledLabels[sourceId];
                    if (areDifferentStates) {
                        if (!queryOnly) {
                            this.toggleLabelInternal(linkedEncodedLabelId, name, enabled);
                        } else {
                            requiresLink ||= true;
                        }
                    }
                }
            }
        });
        return requiresLink;
    }

    private toggleStepInternal(id: string, name: string, enabled: boolean) {
        if (enabled) {
            this.state.enabledSteps[id] = name;
        } else {
            delete this.state.enabledSteps[id];
        }
    }

    private toggleLinkedSteps(sourceId: string, name: string, enabled: boolean, queryOnly: boolean = false): boolean {
        let sourceDecodedStepKey = decodeStepKey(sourceId);
        let requiresLink: boolean = false;
        Object.keys(this.state.enabledTemplates).forEach(key => {
            let targetDecodedTempalteKey = decodeTemplateKey(key);
            // Set this step on every other enabled template, except the narrow case of the stream-template-step - we've just come from there.
            if (targetDecodedTempalteKey.templateId !== sourceDecodedStepKey.templateId || targetDecodedTempalteKey.streamId !== sourceDecodedStepKey.streamId) {
                let linkedEncodedStepId = encodeStepNameFromStringsPreserveCase(targetDecodedTempalteKey.streamId, targetDecodedTempalteKey.templateId, sourceDecodedStepKey.stepName);
                if (this.hierarchyValidator?.isValidStep(targetDecodedTempalteKey.streamId, targetDecodedTempalteKey.templateId, sourceDecodedStepKey.stepName)) {
                    let areDifferentStates: boolean = this.state.enabledSteps[linkedEncodedStepId] != this.state.enabledSteps[sourceId];
                    if (areDifferentStates) {
                        if (!queryOnly) {
                            this.toggleStepInternal(linkedEncodedStepId, name, enabled);
                        } else {
                            requiresLink ||= true;
                        }
                    }
                }
            }
        });
        return requiresLink;
    }

    // #endregion -- Private API --
}

/**
 * Controller responsible for modifying the underlying BuildHealthOptionsState.
 */
export class BuildHealthOptionsController {
    // #region -- Private Members --

    readonly state: BuildHealthOptionsState;
    readonly uiState: BuildHealthUIOptionsState;
    readonly handler: BuildHealthDataHandler;

    private writer: IBuildHealthOptionWriter | null;
    private hierarchyValidator?: IHierarchyValidator;
    private lastSynchronize = -1;

    // #endregion -- Private Members --

    // #region -- Public Members --

    optionsChangeVersion = 0;
    uiOptionsChangeVersion = 0;
    querySchemaVersion: number = 5;

    // #endregion -- Public Members --

    // #region -- Constructor --

    constructor(state: BuildHealthOptionsState, uiState: BuildHealthUIOptionsState) {
        this.state = state;
        this.uiState = uiState;
        makeObservable(this);
    }

    private setOptionsChanged() {
        this.optionsChangeVersion++;
    }

    private setUIOptionsChanged() {
        this.uiOptionsChangeVersion++;
    }

    // #endregion -- Constructor --

    // #region -- Public API --

    /**
     * Gets the currently selected project for the entire options set.
     */
    getCurrentProject(): string | null {
        const keys = Object.keys(this.state.enabledProjects);
        return keys.length > 0 ? keys[0] : null;
    }

    /**
     * Sets the hierarchy validator for the controller.
     * @param validator The hierarchy valildator to us.
     */
    setHierarchyValidator(validator: IHierarchyValidator) {
        this.hierarchyValidator = validator;
    }

    /**
     * Sets the current selected Build Health Filter Id.
     * @param buildHealthFilterId The Build Health Filter Id to set to. Null if no active filter is set.
     */
    @action
    setBuildHealthFilterId(buildHealthFilterId: string | null) {
        this.state.buildHealthFilterId = buildHealthFilterId;
    }

    /**
     * Obtains the currently selected Build Health Filter Id.
     * @returns The currently selected Build Health Filter id.
     */
    getBuildHealthFilterId() {
        return this.state.buildHealthFilterId;
    }

    /**
     * Sets whether the linked selection mode is on when editing.
     * @param linkedSelectionModeEnabled True to enable linked selection, false otherwise.
     */
    @action
    setLinkedSelectionMode(linkedSelectionModeEnabled: boolean) {
        this.uiState.linkedSelectionModeEnabled = linkedSelectionModeEnabled;
        this.setUIOptionsChanged();
    }

    /**
     * Sets whether the date anchors should be used when visualizing the table.
     * @param includeDateAnchors True to show date anchors, false otherwise.
     */
    @action
    setHideDateAnchors(includeDateAnchors: boolean) {
        this.uiState.includeDateAnchors = includeDateAnchors;
        this.setUIOptionsChanged();
    }

    /**
     * Sets whether summary rows should consider warnings as a success, or failure.
     * @param warningsAsSummaryFailure False to include warnings in the numerator of success ratio, true otherwise.
     */
    @action
    setWarningsAsSummaryFailure(warningsAsSummaryFailure: boolean) {
        this.uiState.warningsAsSummaryFailure = warningsAsSummaryFailure;
        this.setUIOptionsChanged();
    }

    /**
     * Gets a options transaction session to make changes to.
     * @returns The @see IBuildHealthOptionWriter that will enact options changes.
     */
    @action
    getTransactionSession(): IBuildHealthOptionWriter {
        if (!this.writer) {
            this.writer = new BuildHealthOptionsWriter(this.state, this.uiState, this.hierarchyValidator);
        }

        return this.writer;
    }

    /**
     * Commits the current options transaction session if one is active.
     * @returns The receipts for the current transaction.
     */
    @action
    commitTransactionSession(): BuildHealthOptionStateDiff {
        if (this.writer === null) {
            return {
                added: undefined,
                removed: undefined,
                enabledPreflight: undefined,
                timespan: undefined
            };
        }

        let finalReceipts = this.writer.produceReceiptDiff();
        this.writer = null;

        // Only request a synchronization if we have produced diffs in our receipt.
        if (finalReceipts.hasDiff) {
            this.setOptionsChanged();
            this.synchronizeDerivedKeys();
        }

        return finalReceipts;
    }

    // #endregion -- Public API --

    // #region -- Private API -- 

    /**
     * Sycnrhonizes the options that are bound to UI elements, with the data observed by consumers of the option set.
     * This is important to keep separate as it allows us to control when we signal/flush a finalized set of options to the consumers.
     */
    @action
    private synchronizeDerivedKeys() {
        if (this.lastSynchronize < this.optionsChangeVersion) {
            this.state.stepOutcomeEnabledStreamKeys = Object.keys(this.state.enabledStreams);
            this.state.stepOutcomeEnabledJobKeys = Object.keys(this.state.enabledTemplates);
            this.state.stepOutcomeEnabledLabelKeys = Object.keys(this.state.enabledLabels);
            this.state.stepOutcomeEnabledStepKeys = Object.keys(this.state.enabledSteps);
            this.state.stepOutcomeEnabledsStates = Object.keys(this.state.enabledStates);
            this.state.stepOutcomeEnabledJobStartMethods = Object.keys(this.state.enabledJobStartMethods);
            this.state.startDate = new Date(new Date().valueOf() - (this.state.jobHistoryTimeSpan.minutes * 60000));
            this.state.endDate = new Date();

            this.uiState.validStates.clear();
            Object.values(this.state.enabledStates).forEach(x => this.uiState.validStates.add(x));

            this.lastSynchronize = this.optionsChangeVersion;
        }
    }

    // #endregion -- Private API --
}