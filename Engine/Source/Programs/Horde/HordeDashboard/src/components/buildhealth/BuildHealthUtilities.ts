// Copyright Epic Games, Inc. All Rights Reserved.

import { KEY_SEPARATOR, LabelRefData, StepRefData, TemplateRefData } from "./BuildHealthDataTypes";

/**
 * Produces an encoded fully qualified label name.
 * @param labelCategory The label category.
 * @param label The label name.
 * @returns The fully qualified label name.
 */
export function encodeFullyQualifiedLabelName(labelCategory: string | undefined, label: string | undefined) {
    return `${labelCategory}${KEY_SEPARATOR}${label}`;
}

/**
 * Decodes a fully qualified label name to it's constituent parts.
 * @param fullyQualifiedLabelName 
 * @returns The constituent parts.
 */
export function decodeFullyQualifiedLabelName(fullyQualifiedLabelName: string): { labelCategory: string, label: string } {
    let [labelCategory, label] = fullyQualifiedLabelName.split(KEY_SEPARATOR);
    return { labelCategory, label };
}

/**
 * Produces an encoded label key for unique identification
 * @param streamId The streamId to encode.
 * @param templateId The templateId to encode.
 * @param labelCategory 
 * @param labelName 
 * @returns The key representing the label.
 */
export function encodeLabelKeyFromStrings(streamId: string, templateId: string, labelCategory: string | undefined, labelName: string | undefined) {
    return `${streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${templateId}${KEY_SEPARATOR}${labelCategory?.toLocaleLowerCase()}${KEY_SEPARATOR}${labelName?.toLocaleLowerCase()}`;
}

/**
 * Produces an encoded label key for unique identification
 * @param streamId The streamId to encode.
 * @param templateId The templateId to encode.
 * @param labelCategory 
 * @param labelName 
 * @returns The key representing the label.
 */
export function encodeLabelKeyFromStringsPreserveCase(streamId: string, templateId: string, labelCategory: string | undefined, labelName: string | undefined) {
    return `${streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${templateId}${KEY_SEPARATOR}${labelCategory}${KEY_SEPARATOR}${labelName}`;
}

/**
 * Produces an encoded label key for unique identification.
 * @returns The key representing the state.
 */
export function encodeLabelKey(lableStateReponse: LabelRefData): string {
    return `${lableStateReponse.streamId}${KEY_SEPARATOR}${lableStateReponse.templateId}${KEY_SEPARATOR}${lableStateReponse.dashboardCategory?.toLocaleLowerCase()}${KEY_SEPARATOR}${lableStateReponse.dashboardName?.toLocaleLowerCase()}`;
}

/**
 * Decodes an encoded label key, providing it's constituent parts.
 * @returns The constituent parts.
 */
export function decodeLabelKey(encodedStateKey: string): { streamId: string, templateId: string, dashboardCategory: string, dashboardName: string } {
    const [streamId, templateId, dashboardCategory, dashboardName] = encodedStateKey.split(KEY_SEPARATOR);
    return { streamId, templateId, dashboardCategory, dashboardName };
}

/**
 * Produces an encoded state key for unique identification.
 * @returns The key representing the state.
 */
export function encodeStateKeyFromString(index: number, stateName: string): string {
    return `${index}${KEY_SEPARATOR}${stateName}`;
}

/**
 * Decodes an encoded state key, providing it's constituent parts.
 * @returns The constituent parts.
 */
export function decodeStateKey(encodedStateKey: string): { index: number; stateName: string } {
    const [indexStr, stateName] = encodedStateKey.split(KEY_SEPARATOR);
    return { index: Number(indexStr) || 0, stateName: stateName ?? '' };
}

/**
 * Produces an encoded template key for unique identification.
 * @param template The template to produce the key for.
 * @returns The key representing the template.
 */
export function encodeTemplateKey(template: TemplateRefData): string {
    return `${template.streamId}${KEY_SEPARATOR}${template.id}`;
}

/**
 * Produces an encoded template key for unique identification.
 * @param streamId The stream id to use in the key.
 * @param templateId The template id to use in the key.
 * @returns 
 */
export function encodeTemplateKeyFromStrings(streamId: string, templateId: string): string {
    return `${streamId}${KEY_SEPARATOR}${templateId}`;
}

/**
 * Decodes an encoded template key, providing it's constituent parts.
 * @param templateKey The template key to decode.
 * @returns The constituent parts.
 */
export function decodeTemplateKey(templateKey: string): { streamId: string; templateId: string } {
    const [streamId, templateId] = templateKey.split(KEY_SEPARATOR);
    return { streamId, templateId };
}

/**
 * Produces an encoded step key for unique identification
 * @param streamId The streamId to encode.
 * @param templateId The templateId to encode.
 * @param stepName The stepName to encode.
 * @returns The key representing the step.
 * @remark We don't want toLocalLowerCase the templateId until UE-315953 is completed in order to bump the url encoding query.
 */
export function encodeStepNameFromStrings(streamId: string, templateId: string, stepName: string) {
    return `${streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${templateId}${KEY_SEPARATOR}${stepName.toLocaleLowerCase()}`;
}

/**
 * Produces an encoded step key for unique identification. Preserves case of step name.
 * @param streamId The streamId to encode.
 * @param templateId The templateId to encode.
 * @param stepName The stepName to encode.
 * @returns The key representing the step.
 */
export function encodeStepNameFromStringsPreserveCase(streamId: string, templateId: string, stepName: string) {
    return `${streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${templateId}${KEY_SEPARATOR}${stepName}`;
}

/**
 * Produces an encoded step key for unique identification.
 * @param step The step to produce the key for.
 * @returns The key representing the step.
 */
export function encodeStepKey(step: StepRefData) {
    return `${step.streamId}${KEY_SEPARATOR}${step.templateId}${KEY_SEPARATOR}${step.name}`;
}

/**
 * Decodes an encoded step key, providing it's constituent parts.
 * @param templateKey The step key to decode.
 * @returns The constituent parts.
 */
export function decodeStepKey(stepKey: string): { streamId: string, templateId: string, stepName: string } {
    const [streamId, templateId, stepName] = stepKey.split(KEY_SEPARATOR);
    return { streamId, templateId, stepName };
}