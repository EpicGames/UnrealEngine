// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, ComboBox, DefaultButton, DirectionalHint, Dropdown, DropdownMenuItemType, IconButton, IDropdownOption, ISelectableOption, Label, PrimaryButton, SelectableOptionMenuItemType, Spinner, SpinnerSize, Stack, TooltipHost } from "@fluentui/react";
import { GetBuildHealthFilterResponse, GetProjectResponse, GetStreamResponse } from "horde/backend/Api";
import { observer } from "mobx-react-lite";
import { useCallback, useEffect, useLayoutEffect, useMemo, useRef, useState } from "react";
import { useLocation, useNavigate } from "react-router";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import { StepOutcomeView } from "../buildhealth/stepoutcome/StepOutcome";
import { BuildHealthDataHandler, DataHandlerRefreshRequest, EMPTY_REFRESH_REQUEST, expandRefreshRequest } from "./BuildHealthDataHandler";
import { dropdownOptions, KEY_SEPARATOR, LabelRefData, PARAMETER_KEY_PREFIX, StepRefData, TemplateRefData } from "./BuildHealthDataTypes";
import { ALL_JOB_START_METHODS_LIST, ALL_STATES_TUPLE_LIST, BuildHealthOptionsState, BuildHealthOptionStateDiff, BuildHealthUIOptionsState, JobHistoryTimeSpans, loadBuildHealthOptionsFromParams, MANUAL_START_METHOD, parseBuildHealthQueryParams, SCHEDULED_START_METHOD, TimeSpan, toNavigationQuery, upgradeBuildHealthQueryParams } from "./BuildHealthOptions";
import { decodeStepKey, decodeTemplateKey, encodeFullyQualifiedLabelName, encodeLabelKey, encodeStepKey, encodeTemplateKey } from "./BuildHealthUtilities";
import { stepOutcomeTableClasses } from "./stepoutcome/StepOutcomeSharedUIComponents";
import { makeAutoObservable, observable, runInAction } from "mobx";
import { BuildHealthOptionsController, StreamDescriptor, TemplateDescriptor } from "./BuildHealthOptionsController";
import { StepOutcomeLegend } from "./stepoutcome/StepOutcomeLegend";
import { StepOutcomeDataHandler, StepOutcomeFilters } from "./stepoutcome/StepOutcomeDataHandler";
import { BuildHealthFilterComponent } from "./BuildHealthFilterComponent";
import { SearchableDropdown } from "./SearchableDropdownComponent";

// #region -- Dropdown Components --

// #region -- Dropdown Data Types --

type BuildHealthOptionData = {
    id: string,
    text: string,
    group?: string
}

type BuildHealthOptionList = {
    items: BuildHealthOptionData[]
    label: string,
    toolTip: string
}

const SELECT_CLEAR_ALL_SPECIAL_GROUP: string = "select_clear_all";

// #endregion -- Dropdown Data Types --

// #region -- Base Dropdown Components --

/**
 * React Component that is a single select drop down.
 * @returns React Component.
 */
const BasicListParameter: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, param: BuildHealthOptionList, disabled?: boolean, selectedKey?: string, request: DataHandlerRefreshRequest, onChange: (option: IDropdownOption<BuildHealthOptionData>) => void }> = function ConstructBasicListParamater({ param, handler, buildHealthOptions, selectedKey, request, onChange }) {
    const key = `${PARAMETER_KEY_PREFIX}${KEY_SEPARATOR}${param.label}`;
    const changeVersion = buildHealthOptions.optionsChangeVersion;
    const doptions: IDropdownOption<BuildHealthOptionData>[] = [];

    param.items.forEach((item) => {
        doptions.push({
            key: item.id,
            text: item.text,
        });
    });

    return (<Dropdown id={changeVersion.toString() + key}
        key={key}
        label={param.label}
        placeholder={"Select Project"}
        options={doptions}
        selectedKey={selectedKey}
        dropdownWidth={200}
        styles={{ dropdown: { width: 300 } }}
        calloutProps={{
            directionalHint: DirectionalHint.rightTopEdge,
            alignTargetEdge: true,
        }}
        onChange={(_ev, option) => {
            let castedOption = option as IDropdownOption<BuildHealthOptionData>;
            if (!castedOption) {
                return;
            }
            onChange(castedOption as IDropdownOption<BuildHealthOptionData>);
        }}
        onDismiss={() => {
            buildHealthOptions.commitTransactionSession();
            handler.requestHierarchicalRefresh(request, undefined, (_request: DataHandlerRefreshRequest) => {
            });
        }}
    />);
};

/**
 * React Component that is a multi select drop down.
 * @returns React Component.
 */
const MultiListParameter: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, param: BuildHealthOptionList, disabled?: boolean, selectedKeys: string[], request: DataHandlerRefreshRequest, onChange: (option: IDropdownOption<BuildHealthOptionData>) => void, enabledSelectAll?: boolean, onSelectAll?: (groupOption: IDropdownOption<BuildHealthOptionData>, optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]) => void, onHierarchicalRefreshCompleted?: (receipts?: BuildHealthOptionStateDiff) => void }> = function ConstructMultilistParamter({ param, handler, buildHealthOptions, disabled, selectedKeys, request, onChange, enabledSelectAll = false, onSelectAll, onHierarchicalRefreshCompleted }) {
    const [filterText, setFilterText] = useState("");
    const changeVersion = buildHealthOptions.optionsChangeVersion;
    const key = `${PARAMETER_KEY_PREFIX}${KEY_SEPARATOR}${param.label}`;
    const selectedSet = useMemo(() => new Set(selectedKeys), [selectedKeys]);
    const groupRefs = useRef<(HTMLDivElement | null)[]>([]);

    useEffect(() => { groupRefs.current = []; }, [param.items]);

    // #region -- Data Memos --

    const doptions = useMemo(() => {
        const groupMap = new Map<string, BuildHealthOptionData[]>();

        for (const item of param.items) {
            const group = item.group ?? "__nogroup";
            if (!groupMap.has(group)) groupMap.set(group, []);
            groupMap.get(group)!.push(item);
        }

        const groups = Array.from(groupMap.keys()).sort();
        const options: IDropdownOption<BuildHealthOptionData>[] = [];

        for (const group of groups) {
            if (group !== "__nogroup") {
                options.push({ key: `group_${group}`, text: group, itemType: DropdownMenuItemType.Header });
                if (enabledSelectAll) {
                    options.push({
                        key: `group_select_all${KEY_SEPARATOR}${group}`,
                        itemType: DropdownMenuItemType.Header,
                        text: SELECT_CLEAR_ALL_SPECIAL_GROUP,
                    });
                }
            }

            for (const item of groupMap.get(group)!) {
                options.push({
                    key: item.id,
                    text: item.text,
                    data: item,
                    selected: selectedSet.has(item.id),
                });
            }
        }

        return options;
    }, [param.items, selectedSet, enabledSelectAll]);

    // #endregion -- Data Memos --
    const filteredOptions = useMemo(() => {
        const lowerFilter = filterText.toLowerCase();
        return doptions.filter(o =>
            o.itemType === DropdownMenuItemType.Header || !filterText
                ? true
                : o.text.toLowerCase().includes(lowerFilter)
        );
    }, [doptions, filterText]);

    const filteredOptionsRef = useRef<IDropdownOption<BuildHealthOptionData>[]>([]);

    useLayoutEffect(() => {
        filteredOptionsRef.current = filteredOptions;
    }, [filteredOptions]);

    // #region -- Dropdown Callbacks --

    const onRenderOption = useCallback(
        (option?: ISelectableOption) => {
            if (!option) {
                return null;
            }

            // Select & Clear all for the given filtered list.
            if (option.itemType === DropdownMenuItemType.Header && option.text === SELECT_CLEAR_ALL_SPECIAL_GROUP) {
                let fitleredOptionList = filteredOptionsRef.current.length == doptions.length ? undefined : filteredOptionsRef.current.filter(x => x.itemType == undefined || x.itemType != SelectableOptionMenuItemType.Header);
                let selectAllLabel = (fitleredOptionList !== undefined && fitleredOptionList.length > 0) ? "Select Filtered" : "Select All";
                let showButton = fitleredOptionList == undefined || fitleredOptionList.length > 0;
                return (
                    <Stack horizontal tokens={{ childrenGap: 8 }} styles={{
                        root:
                        {
                            width: "100%",
                            height: "100%",
                            alignItems: "center",
                            justifyContent: "center",
                        }
                    }}>
                        {/* We have opted to use buttons here as we: 1. don't want to persist the state of a select all option; 2. the semantics of storing a "Select/Clear All" value gets complicated when the action is applied to a filtered set; */}
                        {showButton &&
                            <DefaultButton title="Selects all options (or currently filtered) within the group." styles={{ root: { flex: 1, height: 20 } }} text={selectAllLabel}
                                onClick={() => {
                                    if (!option) return;

                                    const synthesizedOption: IDropdownOption<BuildHealthOptionData> = {
                                        ...option,
                                        selected: true,
                                    };

                                    // Only send a suggested list if we actually have filtered something.
                                    let fitleredOptionList = filteredOptionsRef.current.length == doptions.length ? undefined : filteredOptionsRef.current;
                                    option && onSelectAll?.(synthesizedOption, fitleredOptionList)
                                }}
                            />
                        }
                        <DefaultButton title="Clears all options within the group." styles={{ root: { flex: 1, height: 20 } }} text="Clear All"
                            onClick={() => {
                                if (!option) return;

                                const synthesizedOption: IDropdownOption<BuildHealthOptionData> = {
                                    ...option,
                                    selected: false,
                                };

                                // We do not send a suggested list for clear all; we just clear all.
                                option && onSelectAll?.(synthesizedOption, undefined)
                            }} />
                    </Stack>
                );
            }
            if (option.itemType === DropdownMenuItemType.Header) {
                return <div ref={el => groupRefs.current[option.key as string] = el} style={{ fontWeight: "bold", padding: "4px 8px" }}>{option.text}</div>;
            }

            return (
                // Due to some load testing, when we get 1000s of entires we don't want to bloat the load time with ToolTipHost construction.
                // The inline width/height: 100% expands the title-bearing element to fill the option row so the native browser tooltip
                // fires anywhere on the row, not only over the text characters themselves.
                <div
                    title={`Group: ${option.data?.group || ""}\n${option.text}`}
                    className={dropdownOptions.base}
                    style={{ display: "block", width: "100%", height: "100%", boxSizing: "border-box" }}
                >
                    {option.text}
                </div>
            );
        }, [filteredOptionsRef.current]);

    const handleDropdownChange = useCallback((_ev, option) => {
        if (option && option.itemType !== DropdownMenuItemType.Header) {
            onChange(option);
        }

    }, [onChange]);

    const handleMenuDismiss = useCallback(() => {
        setFilterText("");
        let buildHealthOptionStateReceipts = buildHealthOptions.commitTransactionSession();
        handler.requestHierarchicalRefresh(request, undefined,
            // Upon completion of the hierarchical refresh, we want to commit the transaction session for the options. 
            // We do this because the data handler can populate the template & step cache which may be useful for any downstream receipt processing, and actions that depend on the side-effects of updated hierarchy data.
            (_request: DataHandlerRefreshRequest) => {
                if (onHierarchicalRefreshCompleted) {
                    onHierarchicalRefreshCompleted(buildHealthOptionStateReceipts);
                }
            });

    }, [request, buildHealthOptions]);

    // #endregion -- Dropdown Callbacks --

    return (
        <SearchableDropdown
            id={changeVersion.toString() + key}
            key={key}
            placeholder={param.label}
            options={filteredOptions}
            multiSelect
            groupRefs={groupRefs}
            selectedKeys={selectedKeys}
            onChange={handleDropdownChange}
            onDismiss={handleMenuDismiss}
            onSearchValueChanged={(newValue) => {
                // This stems from a very awkward bug where we need to force the underlying dropdown to resynchronize 
                // on deletions by sending a empty filter - forcing the whole collection to be re-populated.
                // We also need to force a complete animation frame in order to propagate this through, resulting in the
                // underyling Dropdown component internal index synchronization.
                // Without doing so, upon "deletion" changes the onChange will receive the wrong DropdownOption.
                if (newValue.length < filterText.length) {
                    setFilterText("");
                    requestAnimationFrame(() => {
                        setFilterText(newValue);
                    });
                } else {
                    setFilterText(newValue);
                }
            }}
            onRenderOption={onRenderOption}
            styles={{
                dropdown: {
                    width: 300, maxWidth: 900,
                },
            }}
            disabled={disabled}
        />
    );
};

// #endregion -- Base Dropdown Components --

// #region -- Option Specific Dropdown Components --

const ProjectDropdownSingle: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController }> = observer(function ConstructProjectDropdownSingle({ handler, buildHealthOptions }) {
    const projectDropdownParams: BuildHealthOptionList = {
        label: "",
        items: [
        ],
        toolTip: "Select an option from the dropdown."
    };
    projectDropdownParams.items = handler.projectsData.map((project: GetProjectResponse) => {
        let listParam: BuildHealthOptionData = {
            id: `${project.id}`,
            text: project.name,
        };

        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    let selectedProjectKey = Object.keys(buildHealthOptions.state.enabledProjects)[0] ?? null;

    // Note: This may look odd, but there is really no reason to force a full hierarchy of project refresh; we just want to pulse streams
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ streams: true });

    return (
        <Stack>
            <Label>Project</Label>
            <BasicListParameter handler={handler} buildHealthOptions={buildHealthOptions} param={projectDropdownParams} selectedKey={selectedProjectKey} request={request} onChange={(option: IDropdownOption<BuildHealthOptionData>) => {
                const key = option.key as string;
                buildHealthOptions.getTransactionSession().toggleSingleProject(key, option.text);
            }} />
        </Stack>
    );
});

const StreamDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, buildHealthLinkedModeNotepad: BuildHealthLinkNotepad }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions, buildHealthLinkedModeNotepad }) {
    const streamDropdownParams: BuildHealthOptionList = {
        label: "Select Stream(s)",
        items: [
        ],
        toolTip: "Select an option from the dropdown."
    };

    streamDropdownParams.items = handler.streamsData.map((stream: GetStreamResponse) => {
        let listParam: BuildHealthOptionData = {
            id: stream.id,
            text: stream.name,
            group: stream.projectId,
        };
        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            const key = option.key as string;
            const [, projectId] = key.split(KEY_SEPARATOR);
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known streams of the (project).
            const streamIds: StreamDescriptor[] =
                optionSelectAllContext === undefined
                    ? handler.getCachedStreams(projectId)
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;

                            return (
                                opt.data.group === projectId
                            );
                        })
                        .map((x) => ({ id: x.data!.id, name: x.data!.text! }));

            // Do not close the transaction session; the user still has the modal open
            buildHealthOptions.getTransactionSession().toggleAllStreams(streamIds, isSelected);

            buildHealthLinkedModeNotepad.dirtyTemplate(true);
        },
        [buildHealthOptions, handler]
    );

    let selectedStreamKeys: string[] = Object.keys(buildHealthOptions.state.enabledStreams);
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ streams: true });

    return streamDropdownParams.items.length > 0 && (
        <Stack>
            <Label>Stream(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} buildHealthOptions={buildHealthOptions} param={streamDropdownParams} selectedKeys={selectedStreamKeys} request={request}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                    onChange={(option: IDropdownOption<BuildHealthOptionData>) => {
                        const key = option.key as string;
                        const isSelected = !!option.selected;

                        buildHealthOptions.getTransactionSession().toggleStream(key, option.text, isSelected);

                        // Workflow special case to cascade all links, and only in the affirmitive case otherwise we will cascade disables
                        if (buildHealthOptions.uiState.linkedSelectionModeEnabled && isSelected) {
                            let linkRequest: DataHandlerRefreshRequest = expandRefreshRequest({ jobs: true });

                            // We do not want to re-issue select all steps for all newly linked templates, as it can be quite destructive.
                            buildHealthOptions.getTransactionSession().toggleAllLinkableTemplates(true);

                            // We commit a hierarchical refresh so we can build any newly added template's step & label list
                            handler.requestHierarchicalRefresh(linkRequest, true, (request) => {
                                buildHealthOptions.getTransactionSession().toggleAllLinkableSteps(true);
                                buildHealthOptions.getTransactionSession().toggleAllLinkableLabels(true);

                                buildHealthLinkedModeNotepad.dirtyTemplate(false);
                            });
                        }
                    }} />
            </Stack>
        </Stack>
    );
});

const TemplateDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, buildHealthLinkedModeNotepad: BuildHealthLinkNotepad }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions, buildHealthLinkedModeNotepad }) {
    const templateDropdownParams: BuildHealthOptionList = {
        label: "Select Template(s)",
        items: [
        ],
        toolTip: "Select an option from the dropdown."
    };

    templateDropdownParams.items = handler.templatesData.map((template: TemplateRefData) => {
        let listParam: BuildHealthOptionData = {
            id: encodeTemplateKey(template),
            text: template.name,
            group: template.streamId,
        };
        return listParam;
    }).sort((a, b) => a.text.localeCompare(b.text));

    // Note: We are keeping our select all logic here, but not currently using it. 
    //       We would like to be prepared for the use case, however, a select all on templates will be excessive load on the server.
    //       This should be revisited after UE-349992
    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            const key = option.key as string;
            const [, streamId] = key.split(KEY_SEPARATOR);
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known templates of the (stream).
            const templateIds: TemplateDescriptor[] =
                optionSelectAllContext === undefined
                    ? handler.getCachedTemplates(streamId)
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;
                            const decoded = decodeTemplateKey(opt.data.id!);
                            return (
                                decoded.streamId === streamId
                            );
                        })
                        .map((x) => ({ id: x.data!.id, name: x.data!.text! }));

            // Do not close the transaction session; the user still has the modal open
            buildHealthOptions
                .getTransactionSession()
                .toggleAllTemplates(streamId, templateIds, isSelected);
        },
        [buildHealthOptions, handler]
    );

    let selectedTemplateKeys: string[] = Object.keys(buildHealthOptions.state.enabledTemplates);
    let request: DataHandlerRefreshRequest = expandRefreshRequest({ jobs: true });
    let linkString = buildHealthLinkedModeNotepad.labelDirty ? "Current State: Unlinked - Some templates, steps, and labels may require linking." : "Current State: Linked";

    return templateDropdownParams.items.length > 0 && (
        <Stack id={`job_dropdown`}>
            <Label>Template(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} buildHealthOptions={buildHealthOptions} param={templateDropdownParams} selectedKeys={selectedTemplateKeys} request={request}
                    onChange={(option: IDropdownOption<BuildHealthOptionData>) => {
                        const key = option.key as string;
                        const isSelected = !!option.selected;
                        buildHealthOptions.getTransactionSession().toggleTemplate(key, option.text, isSelected);
                        buildHealthLinkedModeNotepad.dirtyTemplateDependents(true);
                    }} />
                {
                    buildHealthOptions.uiState.linkedSelectionModeEnabled &&
                    (
                        <TooltipHost
                            content={
                                <span>
                                    Link templates: Apply selected templates across selected streams, as well as steps and labels.
                                    <br /><br />
                                    {linkString}
                                </span>
                            }
                            calloutProps={{ directionalHint: DirectionalHint.rightCenter }}>
                            <IconButton
                                iconProps={{ iconName: buildHealthLinkedModeNotepad.templateDirty ? 'UnGrouped' : 'Grouped' }}
                                styles={{ root: { width: 15, minWidth: 15, padding: 0 } }} onClick={() => {
                                    let linkRequest: DataHandlerRefreshRequest = expandRefreshRequest({ streams: true });
                                    // We do not want to re-issue select all steps for all newly linked templates, as it can be quite destructive.
                                    buildHealthOptions.getTransactionSession().toggleAllLinkableTemplates(true);

                                    // We commit a hierarchical refresh so we can build any newly added template's step & label list
                                    handler.requestHierarchicalRefresh(linkRequest, true, (request) => {
                                        buildHealthOptions.getTransactionSession().toggleAllLinkableSteps(true);
                                        buildHealthOptions.getTransactionSession().toggleAllLinkableLabels(true);

                                        buildHealthLinkedModeNotepad.dirtyTemplate(false);

                                        buildHealthOptions.commitTransactionSession();
                                    });
                                }
                                } />
                        </TooltipHost>
                    )
                }
            </Stack>
        </Stack>
    );
});

const StepDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, buildHealthLinkedModeNotepad: BuildHealthLinkNotepad }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions, buildHealthLinkedModeNotepad }) {
    let selectedStepKeys: string[] = Object.keys(buildHealthOptions.state.enabledSteps);
    let request: DataHandlerRefreshRequest = { steps: false };
    // #region -- Data Memos --

    const items = useMemo<BuildHealthOptionData[]>(
        () => {
            let transformedStepData = handler.stepData
                .map((step: StepRefData) => ({
                    id: encodeStepKey(step),
                    text: step.name,
                    group: `${step.streamId} - ${step.templateId}`,
                })).sort((a, b) => a.text.localeCompare(b.text));

            return transformedStepData;
        },
        [handler.stepData]
    );

    // #endregion -- Data Memos --

    // #region -- Callbacks --

    const handleChange = useCallback(
        (option: IDropdownOption<BuildHealthOptionData>) => {
            const key = option.key as string;
            const isSelected = !!option.selected;
            buildHealthOptions
                .getTransactionSession()
                .toggleStep(key, option.text, isSelected);
        },
        [buildHealthOptions]
    );

    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            const key = option.key as string;
            const [, data] = key.split(KEY_SEPARATOR);
            const [streamId, templateId] = data.split(" - ");
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known steps of the (stream, template).
            const stepNames: string[] =
                optionSelectAllContext === undefined
                    ? handler.getCachedTemplateStepNames(streamId, templateId)
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;
                            const decoded = decodeStepKey(opt.data.id!);
                            return (
                                decoded.streamId === streamId &&
                                decoded.templateId === templateId
                            );
                        })
                        .map((x) => x.data!.text!);

            // Do not close the transaction session; the user still has the modal open
            buildHealthOptions
                .getTransactionSession()
                .toggleAllSteps(streamId, templateId, stepNames, isSelected);

        },
        [buildHealthOptions, handler]
    );

    const handleLinkSteps = useCallback(
        () => {
            // Close the transaction session; the modal is not open so there will be no such transaction completion otherwise.
            buildHealthOptions.getTransactionSession().toggleAllLinkableSteps(true);
            buildHealthOptions.commitTransactionSession();
            buildHealthLinkedModeNotepad.stepDirty = false;
        },
        [buildHealthOptions, handler]
    );

    // #endregion -- Callbacks --

    const stepDropdownParams: BuildHealthOptionList = {
        label: "Select Step(s)",
        items: [
        ],
        toolTip: "Select an option from the dropdown."
    };

    stepDropdownParams.items = items;
    let linkString = buildHealthLinkedModeNotepad.stepDirty ? "Current State: Unlinked - steps may require linking." : "Current State: Linked";

    return stepDropdownParams.items.length > 0 && (
        <Stack id={`step_dropdown`}>
            <Label>Step(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} buildHealthOptions={buildHealthOptions} param={stepDropdownParams} selectedKeys={selectedStepKeys} request={request}
                    onChange={handleChange}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll} />
                {
                    buildHealthOptions.uiState.linkedSelectionModeEnabled &&
                    (
                        <TooltipHost
                            content={
                                <span>
                                    Link steps: Apply selected steps across selected templates.
                                    <br /><br />
                                    {linkString}
                                </span>
                            }
                            calloutProps={{ directionalHint: DirectionalHint.rightCenter }}>
                            <IconButton
                                iconProps={{ iconName: buildHealthLinkedModeNotepad.stepDirty ? 'UnGrouped' : 'Grouped' }}
                                styles={{ root: { width: 15, minWidth: 15, padding: 0 } }}
                                onClick={handleLinkSteps} />
                        </TooltipHost>
                    )
                }
            </Stack>
        </Stack>
    );
});

const LabelDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, buildHealthLinkedModeNotepad: BuildHealthLinkNotepad }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions, buildHealthLinkedModeNotepad }) {
    let selectedLabelKeys: string[] = Object.keys(buildHealthOptions.state.enabledLabels);
    let request: DataHandlerRefreshRequest = { steps: false };

    // #region -- Data Memos --

    const items = useMemo<BuildHealthOptionData[]>(
        () => {
            let transformedLabelData = handler.labelData
                .map((label: LabelRefData) => ({
                    id: encodeLabelKey(label),
                    text: encodeFullyQualifiedLabelName(label.dashboardCategory, label.dashboardName),
                    group: `${label.streamId} - ${label.templateId}`,
                })).sort((a, b) => a.text.localeCompare(b.text));

            return transformedLabelData;
        },
        [handler.labelData]
    );

    // #endregion -- Data Memos --

    // #region -- Callbacks --

    const handleChange = useCallback(
        (option: IDropdownOption<BuildHealthOptionData>) => {
            const key = option.key as string;
            const isSelected = !!option.selected;
            buildHealthOptions
                .getTransactionSession()
                .toggleLabel(key, option.text, isSelected);
        },
        [buildHealthOptions]
    );

    const handleSelectAll = useCallback(
        (
            option: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            const key = option.key as string;
            const [, data] = key.split(KEY_SEPARATOR);
            const [streamId, templateId] = data.split(" - ");
            const isSelected = !!option.selected;

            // If there is no provided context, simply obtain all the known steps of the (stream, template).
            const labelNames: string[] =
                optionSelectAllContext === undefined
                    ? handler.getCachedTemplateLabelNames(streamId, templateId)
                    : optionSelectAllContext
                        .filter((opt) => {
                            if (!opt.data) return false;
                            const decoded = decodeStepKey(opt.data.id!);
                            return (
                                decoded.streamId === streamId &&
                                decoded.templateId === templateId
                            );
                        })
                        .map((x) => x.data!.text!);

            // Do not close the transaction session; the user still has the modal open
            buildHealthOptions
                .getTransactionSession()
                .toggleAllLabels(streamId, templateId, labelNames, isSelected);
        },
        [buildHealthOptions, handler]
    );

    const handleLinkLabels = useCallback(
        () => {
            // Close the transaction session; the modal is not open so there will be no such transaction completion otherwise.
            buildHealthOptions.getTransactionSession().toggleAllLinkableLabels(true);
            buildHealthOptions.commitTransactionSession();
            buildHealthLinkedModeNotepad.labelDirty = false;
        },
        [buildHealthOptions, handler]
    );

    // #endregion -- Callbacks --

    const labelDropdownParams: BuildHealthOptionList = {
        label: "Select Label(s)",
        items: [
        ],
        toolTip: "Select an option from the dropdown."
    };

    labelDropdownParams.items = items;
    let linkString = buildHealthLinkedModeNotepad.templateDirty ? "Current State: Unlinked - labels may require linking." : "Current State: Linked";

    return labelDropdownParams.items.length > 0 && (
        <Stack id={`label_dropdown`}>
            <Label>Label(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter handler={handler} buildHealthOptions={buildHealthOptions} param={labelDropdownParams} selectedKeys={selectedLabelKeys} request={request}
                    onChange={handleChange}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll} />
                {
                    buildHealthOptions.uiState.linkedSelectionModeEnabled &&
                    (
                        <TooltipHost
                            content={
                                <span>
                                    Link labels: Apply selected labels across selected templates.
                                    <br /><br />
                                    {linkString}
                                </span>
                            }
                            calloutProps={{ directionalHint: DirectionalHint.rightCenter }}>
                            <IconButton
                                iconProps={{ iconName: buildHealthLinkedModeNotepad.labelDirty ? 'UnGrouped' : 'Grouped' }}
                                styles={{ root: { width: 15, minWidth: 15, padding: 0 } }}
                                onClick={handleLinkLabels} />
                        </TooltipHost>
                    )
                }
            </Stack>
        </Stack>
    );
});

const StateDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions }) {
    let selectedStateKeys: string[] = Object.keys(buildHealthOptions.state.enabledStates);

    // #region -- Data Memos --

    const items = useMemo<BuildHealthOptionData[]>(() =>
        ALL_STATES_TUPLE_LIST.map((entry) => ({
            id: entry.key,
            text: entry.stateName,
            group: "States"
        })),
        []
    );

    const dropdownOptions = useMemo<IDropdownOption<BuildHealthOptionData>[]>(() =>
        items.map(item => ({
            key: item.id,
            text: item.text,
            data: item,
            group: item.group
        })),
        [items]
    );

    // #endregion

    // #region -- Callbacks --

    const handleChange = useCallback(
        (option?: IDropdownOption<BuildHealthOptionData>) => {
            if (!option) return;

            const stateKey = option.key as string;
            const stateName = option.text;
            const isSelected = !!option.selected;

            buildHealthOptions
                .getTransactionSession()
                .toggleState(stateKey, stateName, isSelected);
        },
        [buildHealthOptions]
    );

    const handleSelectAll = useCallback(
        (
            option?: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            if (!option) return;

            const isSelected = !!option.selected;
            const stateKeys = optionSelectAllContext === undefined ? dropdownOptions.map(x => x.key as string) : optionSelectAllContext.map(x => x.key as string);

            buildHealthOptions
                .getTransactionSession()
                .toggleAllStates(stateKeys, isSelected);
        },
        [buildHealthOptions, dropdownOptions]
    );

    // #endregion

    // Dropdown params
    const stateDropdownParams: BuildHealthOptionList = {
        label: "Select State(s)",
        items: items,
        toolTip: "Select an option from the dropdown."
    };

    return stateDropdownParams.items.length > 0 && (
        <Stack id={`state_dropdown`}>
            <Label>State(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter
                    handler={handler}
                    buildHealthOptions={buildHealthOptions}
                    param={stateDropdownParams}
                    selectedKeys={selectedStateKeys}
                    request={EMPTY_REFRESH_REQUEST}
                    onChange={handleChange}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                />
            </Stack>
        </Stack>
    );
});

const StartDropdown: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController }> = observer(function ConstructProjectDropdown({ handler, buildHealthOptions }) {
    let selectedJobStartKeys: string[] = Object.keys(buildHealthOptions.state.enabledJobStartMethods);

    // #region -- Data Memos --

    const items = useMemo<BuildHealthOptionData[]>(() =>
        ALL_JOB_START_METHODS_LIST.map((entry) => ({
            id: entry,
            text: entry,
            group: "Job Start Methods"
        })),
        []
    );

    const dropdownOptions = useMemo<IDropdownOption<BuildHealthOptionData>[]>(() =>
        items.map(item => ({
            key: item.id,
            text: item.text,
            data: item,
            group: item.group
        })),
        [items]
    );

    // #endregion

    // #region -- Callbacks --

    const handleChange = useCallback(
        (option?: IDropdownOption<BuildHealthOptionData>) => {
            if (!option) return;

            const invocationKey = option.key as string;
            const invocationName = option.text;
            const isSelected = !!option.selected;

            buildHealthOptions
                .getTransactionSession()
                .toggleJobStartMethod(invocationKey, invocationName, isSelected);
        },
        [buildHealthOptions]
    );

    const handleSelectAll = useCallback(
        (
            option?: IDropdownOption<BuildHealthOptionData>,
            optionSelectAllContext?: IDropdownOption<BuildHealthOptionData>[]
        ) => {
            if (!option) return;

            const isSelected = !!option.selected;
            const invocationkeys = optionSelectAllContext === undefined ? dropdownOptions.map(x => x.key as string) : optionSelectAllContext.map(x => x.key as string);

            buildHealthOptions
                .getTransactionSession()
                .toggleAllJobStartMethods(invocationkeys, isSelected);
        },
        [buildHealthOptions, dropdownOptions]
    );

    // #endregion

    // Dropdown params
    const startDropdownParams: BuildHealthOptionList = {
        label: "Select Job Start Methods(s)",
        items: items,
        toolTip: "Select an option from the dropdown."
    };

    return startDropdownParams.items.length > 0 && (
        <Stack id={`start_dropdown`}>
            <Label>Job Start(s)</Label>
            <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="end">
                <MultiListParameter
                    handler={handler}
                    buildHealthOptions={buildHealthOptions}
                    param={startDropdownParams}
                    selectedKeys={selectedJobStartKeys}
                    request={EMPTY_REFRESH_REQUEST}
                    onChange={handleChange}
                    enabledSelectAll={true}
                    onSelectAll={handleSelectAll}
                />
            </Stack>
        </Stack>
    );
});

// #endregion -- Option Specific Dropdown Components --

// #endregion -- Dropdown Components --

// #region -- Primary View Components --

/**
 * React Component used to manage the URL query params, and refresh it based on build health options. This is separate from the main component to scope rerenders to solely this component.
 * @returns React Component.
 */
const BuildHealthUrlSync: React.FC<{ buildHealthController: BuildHealthOptionsController, handler: BuildHealthDataHandler }> = function ConstrucBuildHealthSidebar({ buildHealthController, handler }) {
    const location = useLocation();
    const navigate = useNavigate();

    // We only ever receive from location search once.
    useEffect(() => {
        const initializeFromUrl = async () => {
            let searchParams: URLSearchParams = new URLSearchParams(location.search)
            const parsed = parseBuildHealthQueryParams(searchParams);

            let appliedQueryParams = parsed.buildHealthQueryParams;
            let appliedUIParams = parsed.buildHealthUIParams;
            let resolvedFilterId: string | null = appliedQueryParams.filterId;

            // If a filterId is in the URL, look up the saved filter and load its stored query
            // so every control is populated from the saved configuration.
            if (resolvedFilterId) {
                // Ensure project filter data is loaded before lookup.
                await handler.requestHierarchicalRefresh();

                let foundFilter: GetBuildHealthFilterResponse | undefined;
                for (const filters of handler.projectFilterData.values()) {
                    const match = filters.find(f => f.id === resolvedFilterId);
                    if (match) { foundFilter = match; break; }
                }

                if (foundFilter) {
                    const filterParsed = parseBuildHealthQueryParams(new URLSearchParams(foundFilter.filterQuery));
                    appliedQueryParams = filterParsed.buildHealthQueryParams;
                    appliedUIParams = filterParsed.buildHealthUIParams;
                } else {
                    console.warn(`Build Health filter id '${resolvedFilterId}' from URL was not found; falling back to remaining URL parameters.`);
                    resolvedFilterId = null;
                }
            }

            const upgradedParams = upgradeBuildHealthQueryParams(appliedQueryParams, buildHealthController.querySchemaVersion);
            await loadBuildHealthOptionsFromParams(upgradedParams, appliedUIParams, buildHealthController, handler);

            // Set after load so the URL push effect rewrites the URL with the resolved filter id (or clears it).
            buildHealthController.setBuildHealthFilterId(resolvedFilterId);
        };

        initializeFromUrl();
    }, []);

    useEffect(() => {
        const desiredSearch = toNavigationQuery(buildHealthController.state, buildHealthController.uiState, buildHealthController.querySchemaVersion, handler).toString();
        if (desiredSearch && location.search !== `?${desiredSearch}`) {
            navigate(`${location.pathname}?${desiredSearch}`, { replace: true });
        }
    }, [navigate, buildHealthController.optionsChangeVersion, buildHealthController.uiOptionsChangeVersion, location.pathname, location.search]);

    return null;
};

/**
 * Data structure used to keep track of the current linked selection mode session.
 */
class BuildHealthLinkNotepad {
    @observable releaseDirty: boolean = true;
    @observable templateDirty: boolean = true;
    @observable stepDirty: boolean = true;
    @observable labelDirty: boolean = true;

    constructor() {
        makeAutoObservable(this);
    }

    dirtyTemplate(isDirty: boolean): void {
        this.templateDirty = isDirty;
        this.stepDirty = isDirty;
        this.labelDirty = isDirty;
    }

    dirtyStep(isDirty: boolean): void {
        this.stepDirty = isDirty;
    }

    dirtyLabel(isDirty: boolean): void {
        this.labelDirty = isDirty;
    }

    dirtyTemplateDependents(isDirty: boolean) {
        this.stepDirty = isDirty;
        this.labelDirty = isDirty;
    }
}

/**
 * React Component that represents the Build Health sidebar. All filters are present here.
 * @returns React Component.
 */
const BuildHealthSidebar: React.FC<{ handler: BuildHealthDataHandler, buildHealthOptions: BuildHealthOptionsController, buildHealthLinkedModeNotepad: BuildHealthLinkNotepad }> = observer(function ConstrucBuildHealthSidebar({ handler, buildHealthOptions, buildHealthLinkedModeNotepad }) {
    const { hordeClasses, modeColors } = getHordeStyling();
    const [timeSpanKey, setTimeSpanKey] = useState(() => buildHealthOptions.state.jobHistoryTimeSpan.key);
    const [includeCancelledJobs, setIncludeCancelledJobs] = useState(() => buildHealthOptions.state.includeCancelledJobs)
    const [includePreflight, setIncludePreflight] = useState(() => buildHealthOptions.state.includePreflight)
    const [linkedSelectionMode, setLinkedSelectionMode] = useState(() => buildHealthOptions.uiState.linkedSelectionModeEnabled)

    // #region -- Visual Options -- 

    const [hideDateAnchors, setHideDateAnchors] = useState(() => buildHealthOptions.uiState.includeDateAnchors)
    const [warningsAsSummaryFailure, setWarningsAsSummaryFailure] = useState(() => buildHealthOptions.uiState.warningsAsSummaryFailure)

    // #ednregion -- Visual Options --

    // #region -- Property Callbacks --

    const updateJobHistoryTimeSpan = (_ev, option) => {
        const select = option as TimeSpan;
        setTimeSpanKey(select.key);
        buildHealthOptions.getTransactionSession().setJobHistoryTimeSpan(select);
        buildHealthOptions.commitTransactionSession();
    };

    const updateIncludeCancelledJobs = (isChecked) => {
        setIncludeCancelledJobs(isChecked);
        buildHealthOptions.getTransactionSession().setIncludeCancelledJobs(isChecked);
        buildHealthOptions.commitTransactionSession();
    }

    const updateIncludePreflight = (isChecked) => {
        setIncludePreflight(isChecked);
        buildHealthOptions.getTransactionSession().setIncludePreflight(isChecked);
        buildHealthOptions.commitTransactionSession();
    }

    const updateLinkedSelectionMode = (isChecked) => {
        setLinkedSelectionMode(isChecked);
        buildHealthOptions.setLinkedSelectionMode(isChecked);

        // Determine the linkable state of each of the primary sets.
        let dirtyTemplates = buildHealthOptions.getTransactionSession().toggleAllLinkableTemplates(true, true);
        let dirtyLabels = buildHealthOptions.getTransactionSession().toggleAllLinkableLabels(true, true);
        let dirtySteps = buildHealthOptions.getTransactionSession().toggleAllLinkableSteps(true, true);

        buildHealthLinkedModeNotepad.dirtyTemplate(dirtyTemplates || dirtyLabels || dirtySteps);
        buildHealthLinkedModeNotepad.dirtyLabel(dirtyLabels);
        buildHealthLinkedModeNotepad.dirtyStep(dirtySteps);

        buildHealthOptions.commitTransactionSession();
    }

    const updateIncludeDateAnchors = (isChecked) => {
        setHideDateAnchors(isChecked);
        buildHealthOptions.setHideDateAnchors(isChecked);
    }

    const updateWarningsAsSummaryFailure = (isChecked) => {
        setWarningsAsSummaryFailure(isChecked);
        buildHealthOptions.setWarningsAsSummaryFailure(isChecked);
    }

    // #endregion -- Property Callbacks --

    // #region -- Property Use Effects --

    useEffect(() => {
        setTimeSpanKey(buildHealthOptions.state.jobHistoryTimeSpan.key);
    }, [buildHealthOptions.state.jobHistoryTimeSpan.key]);

    useEffect(() => {
        setIncludeCancelledJobs(buildHealthOptions.state.includeCancelledJobs);
    }, [buildHealthOptions.state.includeCancelledJobs]);

    useEffect(() => {
        setIncludePreflight(buildHealthOptions.state.includePreflight);
    }, [buildHealthOptions.state.includePreflight]);

    useEffect(() => {
        setHideDateAnchors(buildHealthOptions.uiState.includeDateAnchors);
    }, [buildHealthOptions.uiState.includeDateAnchors]);

    useEffect(() => {
        setWarningsAsSummaryFailure(buildHealthOptions.uiState.warningsAsSummaryFailure);
    }, [buildHealthOptions.uiState.warningsAsSummaryFailure]);

    // #endregion -- Property Use Effects --

    let timeComboWidth = 150;

    // Show blocking spinner while initializing
    if (handler.isInitializing) {
        return (
            <Stack style={{ minWidth: 415, maxWidth: 500, paddingRight: 18, flexShrink: 0 }}>
                <Stack className={hordeClasses.modal} tokens={{ childrenGap: 12 }}>
                    <Stack
                        styles={{
                            root: {
                                border: `1px solid ${modeColors.content}`,
                                borderRadius: 4,
                                padding: 8,
                            },
                        }}
                    >
                        <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                            <Label>Step Outcome Filters</Label>
                        </Stack>
                        <Stack styles={{
                            root: {
                                border: `1px solid ${modeColors.content}`,
                                borderRadius: 4,
                                padding: 8,
                                minHeight: 200,
                            },
                        }} horizontalAlign="center" verticalAlign="center">
                            <Spinner size={SpinnerSize.large} label={handler.initializingMessage} />
                        </Stack>
                    </Stack>
                </Stack>
            </Stack>
        );
    }

    return (
        <Stack style={{ minWidth: 375, maxWidth: 500, paddingRight: 18, flexShrink: 0 }}>
            <Stack className={hordeClasses.modal} tokens={{ childrenGap: 12 }}>
                <Stack
                    styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                        },
                    }}
                >
                    <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                        <Label>Step Outcome Filters</Label>
                    </Stack>
                    <Stack styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                            overflow: "hidden",
                        },
                    }}>
                        <ProjectDropdownSingle handler={handler} buildHealthOptions={buildHealthOptions} />
                        <StreamDropdown handler={handler} buildHealthOptions={buildHealthOptions} buildHealthLinkedModeNotepad={buildHealthLinkedModeNotepad} />
                        <TemplateDropdown handler={handler} buildHealthOptions={buildHealthOptions} buildHealthLinkedModeNotepad={buildHealthLinkedModeNotepad} />
                        <LabelDropdown handler={handler} buildHealthOptions={buildHealthOptions} buildHealthLinkedModeNotepad={buildHealthLinkedModeNotepad} />
                        <StepDropdown handler={handler} buildHealthOptions={buildHealthOptions} buildHealthLinkedModeNotepad={buildHealthLinkedModeNotepad} />
                        <StateDropdown handler={handler} buildHealthOptions={buildHealthOptions} />
                        <StartDropdown handler={handler} buildHealthOptions={buildHealthOptions} />
                        <TooltipHost content={
                            <Stack style={{ maxWidth: 300 }}>
                                <Stack>
                                    Clears the current Project/Stream/Template/Step/State selection.
                                </Stack>
                            </Stack>
                        }
                            calloutProps={{
                                directionalHint: DirectionalHint.rightCenter
                            }}
                        >
                            <PrimaryButton text="Clear" onClick={() => {

                                runInAction(() => {
                                    buildHealthController.getTransactionSession().clearAll();
                                    handler.reset();
                                    buildHealthOptions.commitTransactionSession();
                                });
                            }} />
                        </TooltipHost>
                        <TooltipHost content={`Adjust the historical job history time span. Note: This is a best effort due to load limitations (we constrain a max job result count: ${StepOutcomeDataHandler.MAX_JOB_COUNT_DEFAULT}).`}
                            calloutProps={{
                                directionalHint: DirectionalHint.rightCenter
                            }}
                        >
                            <Label style={{ paddingTop: 10 }}>Jobs Since:</Label>
                        </TooltipHost>
                        <ComboBox
                            styles={{ root: { width: timeComboWidth } }}
                            options={JobHistoryTimeSpans}
                            selectedKey={timeSpanKey}
                            onChange={(ev, option, _index, _value) => {
                                updateJobHistoryTimeSpan(ev, option);
                                buildHealthOptions.commitTransactionSession();
                            }}
                        />
                    </Stack>
                </Stack>
                <BuildHealthFilterComponent handler={handler} buildHealthOptions={buildHealthOptions} />
                <Stack
                    styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                        },
                    }}>
                    <Stack style={{ paddingTop: 0, paddingBottom: 4 }}>
                        <Label>Options</Label>
                    </Stack>
                    <TooltipHost content={
                        <div style={{ maxWidth: 300 }}>
                            <div>
                                Linked editing mode: Links filter selects across streams & templates where applicable.
                            </div>
                        </div>
                    }
                        calloutProps={{
                            directionalHint: DirectionalHint.rightCenter
                        }}
                    >
                        <Checkbox
                            label="Linked Selection Mode"
                            checked={linkedSelectionMode}
                            onChange={(_, isChecked) => updateLinkedSelectionMode(!!isChecked)}
                        />
                    </TooltipHost>
                    <TooltipHost content={
                        <div style={{ maxWidth: 300 }}>
                            <div>
                                Include change submit date anchors in the header to separate changes.
                            </div>
                        </div>
                    }
                        calloutProps={{
                            directionalHint: DirectionalHint.rightCenter
                        }}
                    >
                        <Checkbox
                            label="Show Date Anchors"
                            checked={hideDateAnchors}
                            onChange={(_, isChecked) => updateIncludeDateAnchors(!!isChecked)}
                        />
                    </TooltipHost>
                    <TooltipHost content={
                        <div style={{ maxWidth: 300 }}>
                            <div>
                                Warnings are considered success in succes/failure ratio for summary.
                            </div>
                        </div>
                    }
                        calloutProps={{
                            directionalHint: DirectionalHint.rightCenter
                        }}
                    >
                        <Checkbox
                            label="Warnings as Errors"
                            checked={warningsAsSummaryFailure}
                            onChange={(_, isChecked) => updateWarningsAsSummaryFailure(!!isChecked)}
                        />
                    </TooltipHost>
                    <TooltipHost content={
                        <div style={{ maxWidth: 300 }}>
                            <div>
                                Include presubmits in the job history results, or not.
                            </div>
                        </div>
                    }
                        calloutProps={{
                            directionalHint: DirectionalHint.rightCenter
                        }}
                    >
                        <Checkbox
                            label="Include Presubmits"
                            checked={includePreflight}
                            onChange={(_, isChecked) => updateIncludePreflight(!!isChecked)}
                        />
                    </TooltipHost>
                    <TooltipHost content={
                        <div style={{ maxWidth: 300 }}>
                            <div>
                                Include cancelled jobs in the job history results, or not.
                            </div>
                        </div>
                    }
                        calloutProps={{
                            directionalHint: DirectionalHint.rightCenter
                        }}
                    >
                        <Checkbox
                            label="Include Cancelled Jobs"
                            checked={includeCancelledJobs}
                            onChange={(_, isChecked) => updateIncludeCancelledJobs(!!isChecked)}
                        />
                    </TooltipHost>
                </Stack>
                <Stack
                    styles={{
                        root: {
                            border: `1px solid ${modeColors.content}`,
                            borderRadius: 4,
                            padding: 8,
                        },
                    }}>
                    <StepOutcomeLegend />
                </Stack>
            </Stack>
        </Stack>
    );
});

/**
 * React Component representing the entire Build Health view.
 * @returns React Component.
 */
export const BuildHealthView: React.FC = observer(function ConstructBuildHealthView() {
    const windowSize = useWindowSize();
    const { hordeClasses, modeColors } = getHordeStyling();
    const [selectedItem, setSelectedItem] = useState<GetBuildHealthFilterResponse | null>(null);

    const jobHistorySpan = useMemo(() => ({
        start: buildHealthState.startDate,
        end: buildHealthState.endDate
    }),
        // eslint-disable-next-line react-hooks/exhaustive-deps -- All necessary dependencies included for MobX observables
        [buildHealthState.startDate, buildHealthState.endDate]);

    const filter: StepOutcomeFilters = useMemo(() => (
        {
            streams: buildHealthState.stepOutcomeEnabledStreamKeys,
            jobs: buildHealthState.stepOutcomeEnabledJobKeys,
            steps: buildHealthState.stepOutcomeEnabledStepKeys,
            labels: buildHealthState.stepOutcomeEnabledLabelKeys,
            jobHistorySpan,
            includePreflights: buildHealthState.includePreflight,
            includeCancelledJobs: buildHealthState.includeCancelledJobs,
            debugMode: buildHealthState.debugMode,
            jobStartFilter: {
                supportsManualStarts: buildHealthState.stepOutcomeEnabledJobStartMethods.includes(MANUAL_START_METHOD), supportsScheduledStarts: buildHealthState.stepOutcomeEnabledJobStartMethods.includes(SCHEDULED_START_METHOD)
            }
        }
    ),
        // eslint-disable-next-line react-hooks/exhaustive-deps -- All necessary dependencies included for MobX observables
        [
            buildHealthState.stepOutcomeEnabledStreamKeys,
            buildHealthState.stepOutcomeEnabledJobKeys,
            buildHealthState.stepOutcomeEnabledStepKeys,
            buildHealthState.stepOutcomeEnabledJobStartMethods,
            jobHistorySpan,
            buildHealthState.includePreflight,
            buildHealthState.includeCancelledJobs
        ]);

    const uiOptions = useMemo(() => ({
        includeDateAnchors: buildHealthUIState.includeDateAnchors,
        warningsAsSummaryFailure: buildHealthUIState.warningsAsSummaryFailure,
        validStates: buildHealthUIState.validStates,
        debugMode: buildHealthUIState.debugMode
    }),
        // eslint-disable-next-line react-hooks/exhaustive-deps -- All necessary dependencies included for MobX observables
        [
            buildHealthUIState.includeDateAnchors,
            buildHealthUIState.warningsAsSummaryFailure,
            buildHealthUIState.validStates,
            buildHealthUIState.debugMode
        ]);

    return (
        <Stack className={hordeClasses.horde}>
            <BuildHealthUrlSync buildHealthController={buildHealthController} handler={handler} />
            <TopNav />
            <Breadcrumbs items={[{ text: 'Build Health' }]} />
            <Stack horizontal>
                <div key={`windowsize_automationview_${windowSize.width}_${windowSize.height}`} style={{ width: 0, flexShrink: 0, backgroundColor: modeColors.background }} />
                <Stack horizontalAlign="center" grow styles={{ root: { width: "100%", padding: 12, backgroundColor: modeColors.background } }}>
                    <Stack styles={{ root: { width: "100%" } }}>
                        <Stack horizontal styles={{ root: { minHeight: '85vh' } }} >
                            <BuildHealthSidebar handler={handler} buildHealthOptions={buildHealthController} buildHealthLinkedModeNotepad={buildHealthLinkedModeNotepad} />
                            <Stack id="parent-buildhealth-stepoutcome" className={stepOutcomeTableClasses.root} grow style={{ overflowX: "auto", overflowY: "visible", minWidth: "800px", position: "relative", border: `${modeColors.content} solid 2px` }}>
                                <StepOutcomeView key="step-outcome-inline" filter={filter} uiOptions={uiOptions} />
                            </Stack>
                        </Stack>
                    </Stack>
                </Stack>
            </Stack>
        </Stack>
    );
});

// #endregion -- Primary View Components --

// #region -- Script --
const buildHealthLinkedModeNotepad = new BuildHealthLinkNotepad();
const buildHealthState = new BuildHealthOptionsState();
const buildHealthUIState = new BuildHealthUIOptionsState();
const buildHealthController = new BuildHealthOptionsController(buildHealthState, buildHealthUIState);
const handler = new BuildHealthDataHandler(buildHealthState, buildHealthController);
buildHealthController.setHierarchyValidator(handler);

// #endregion -- Script --