import { DirectionalHint, Dropdown, IconButton, IDropdownProps, SearchBox, Stack } from "@fluentui/react";
import { getHordeStyling } from "horde/styles/Styles";
import { useRef } from "react";
import { dropdownNavigationButtons } from "./BuildHealthDataTypes";

/**
 * Props for the SearchableDropdown component.
 */
export interface ISearchableDropdownProps extends IDropdownProps {

    // Delegate that is run when search text field is changed.
    onSearchValueChanged?: (searchValue: string) => void;
    // Refs to all groups in the dropdown.
    groupRefs?: React.RefObject<(HTMLDivElement | null)[]>;
}

/**
 * Searchable drop down component.
 * @param props The props that control the component.
 * @returns The React component.
 */
export const SearchableDropdown: React.FC<ISearchableDropdownProps> = (props) => {
    const { modeColors } = getHordeStyling();
    const { onSearchValueChanged, groupRefs, ...dropdownProps } = props;
    const dropdownRef = useRef<HTMLDivElement | null>(null);
    const currentIndex = useRef(0);
    const PIXEL_AFFORDANCE: number = 5;

    // #region -- Internal Dropdown Navigation Functions --

    const recomputeCurrentIndex = (): { currentGroup: number, onHeader: boolean } => {
        if (!dropdownRef.current || !groupRefs?.current) return { currentGroup: 0, onHeader: false };

        const scrollTop = dropdownRef.current.scrollTop;
        const keys = Object.keys(groupRefs.current || []);

        let newIndex: number = 0;
        let onHeader: boolean = false;

        for (let i = 0; i < keys.length; i++) {
            const el = groupRefs.current[keys[i]];
            if (!el) continue;

            const offset = el.offsetTop - 50;
            if (scrollTop >= offset - PIXEL_AFFORDANCE) {
                newIndex = i;

                if (scrollTop >= offset - PIXEL_AFFORDANCE && scrollTop <= offset + PIXEL_AFFORDANCE) {
                    onHeader = true;
                }
            }
        }

        currentIndex.current = newIndex;
        return { currentGroup: newIndex, onHeader: onHeader };
    };

    const scrollToTop = () => {
        currentIndex.current = 0;
        goToTargetGroup(currentIndex.current);
    };

    const scrollToBottom = () => {
        let groupRefKeys = groupRefs?.current ? Object.keys(groupRefs.current) : [];
        currentIndex.current = groupRefKeys.length - 1;
        goToTargetGroup(currentIndex.current);
    };

    const goToTargetGroup = (targetGroupIndex: number) => {
        let groupRefKeys = groupRefs?.current ? Object.keys(groupRefs.current) : [];
        if (!groupRefs?.current || !dropdownRef.current) {
            return;
        }
        if (targetGroupIndex < 0 || targetGroupIndex >= groupRefKeys.length) {
            return;
        }

        currentIndex.current = targetGroupIndex;
        const el = groupRefs.current[groupRefKeys[targetGroupIndex]];
        if (el) {
            const offset = el.offsetTop - 50;
            dropdownRef.current.scrollTop = offset >= 0 ? offset : 0;
        }
    }

    const iterateToGroup = (index: number) => {
        let currentLocation = recomputeCurrentIndex();
        currentIndex.current = currentLocation.currentGroup;

        // Determine how to move the current index.
        // - If we are going "up" (-1 index), we need to see whether we are exactly on the header or not
        // -- If we are exactly on the header, we intend to go to the previous group (so actually go to the previous group)
        // -- If however we are within the group, we intend to snap to the current groups header
        // - If we are going "down" (1 index), we simply just increment on top of the current group.
        let incrementedIndex = index == -1 ? (currentLocation.onHeader ? currentIndex.current + index : currentIndex.current) : currentIndex.current + index;

        goToTargetGroup(incrementedIndex);
    };

    // #endregion -- Internal Dropdown Navigation Functions --

    return (
        <Dropdown
            ref={dropdownRef}
            {...dropdownProps}
            calloutProps={{
                calloutWidth: 510,
                directionalHint: DirectionalHint.rightTopEdge,
                alignTargetEdge: true,
                styles: {
                    calloutMain: {
                        width: 500,
                        minWidth: 500,
                        minHeight: 200,
                        maxHeight: 800
                    }
                },
            }}
            onRenderList={(props, defaultRender) => {
                if (!props) {
                    return null;
                }

                return (
                    <div ref={dropdownRef} style={{ maxHeight: 800, overflowY: 'auto', width: 500 }}>
                        <Stack
                            horizontal
                            verticalAlign="center"
                            styles={{ root: { position: "sticky", top: 0, zIndex: 1, padding: 8, background: modeColors.background } }}
                        >
                            <SearchBox
                                placeholder="Filter options"
                                autoFocus
                                onChange={(_ev, newVal) => onSearchValueChanged?.(newVal !== undefined ? newVal : '')}
                                onClear={() => onSearchValueChanged?.('')}
                                onKeyDown={(ev) => {
                                    if (ev.key === 'Enter' || ev.key === 'Escape') ev.stopPropagation();
                                }}
                                styles={{
                                    root: {
                                        flexGrow: 1,
                                        marginRight: 4,
                                    },
                                    field: {
                                    },
                                }}
                            />
                            <IconButton
                                iconProps={{ iconName: 'ChevronUp' }}
                                title="Jump to previous group."
                                className={dropdownNavigationButtons.base}
                                onClick={() => { iterateToGroup(-1) }} />
                            <IconButton
                                iconProps={{ iconName: 'ChevronDown' }}
                                title="Jump to next group."
                                className={dropdownNavigationButtons.base}
                                onClick={() => { iterateToGroup(1) }} />
                            <IconButton
                                iconProps={{ iconName: 'DoubleChevronUp' }}
                                title="Jump to first group."
                                className={dropdownNavigationButtons.base}
                                onClick={scrollToTop} />
                            <IconButton
                                iconProps={{ iconName: 'DoubleChevronDown' }}
                                title="Jump to last group."
                                className={dropdownNavigationButtons.base}
                                onClick={scrollToBottom} />
                        </Stack>
                        {defaultRender?.(props)}
                    </div>
                );
            }}
        />
    );
};