// Copyright Epic Games, Inc. All Rights Reserved.

import { Dialog, DialogType, Label, mergeStyleSets, Spinner, SpinnerSize, Stack } from "@fluentui/react";
import { Text } from '@fluentui/react/lib/Text';
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from 'react';
import { StepOutcomeD3Table } from "./StepOutcomeD3Component";
import { prettyPrintStepOutcomeFilters, prettyPrintStepOutcomeFiltersSummary, StepOutcomeDataHandler, StepOutcomeFilters, StepOutcomeViewOptions } from "./StepOutcomeDataHandler";
import { AbortedStr, OutcomeTableEntry, isLabelOutcome, isStepOutcome, StepOutcomeTableEntry } from "./StepOutcomeDataTypes";
import { NOT_APPLICABLE_LABEL, sharedBorderStyle } from "./StepOutcomeSharedUIComponents";
import { getDateTimeString, isTerminalState, toDate } from "./StepOutcomeUtilities";
import { getHordeTheme } from "horde/styles/theme";

// #region -- View Components --

type StepOutcomeTableProps = {
    filters: StepOutcomeFilters;
    uiOptions: StepOutcomeViewOptions;

    handler: StepOutcomeDataHandler;
    onCellSelected: (item: StepOutcomeTableEntry) => void;
};

interface StepOutcomeViewProps {
    filter: StepOutcomeFilters;
    uiOptions: StepOutcomeViewOptions;
    refreshTime?: number;
}

// #region -- Styles  --

const stepOutcomeModalClasses = mergeStyleSets(sharedBorderStyle);

// #endregion -- Styles  --

// #region -- Modal Popup View --

type StepOutcomeFocusModalProps = {
    item: OutcomeTableEntry;
    onDismiss?: (_ev?: React.MouseEvent<HTMLButtonElement>) => any;
};

/**
 * React Component representing the Step Outcome focus.
 */
const StepOutcomeFocusModal: React.FC<StepOutcomeFocusModalProps> = ({ item, onDismiss }) => {
    const startDate = toDate(item.startTime);

    const [issueRequestComplete, setIssueRequestComplete] = useState(false);

    useEffect(() => {
        if (isStepOutcome(item) && !issueRequestComplete) {
            handler.populateStepOutcomeTableEntryIssueData(item, setIssueRequestComplete);
        }
    }, []);

    if (item === undefined) { return; }

    let stepLink: string | null = null;
    let stepNoun: string | null = null;
    if (isStepOutcome(item)) {
        stepNoun = "Step";
        stepLink = `/job/${item.jobId}?step=${item.id}`;
    } else if (isLabelOutcome(item)) {
        stepNoun = "Label";
        stepLink = `/job/${item.jobId}?label=${item.id}`;
    }

    const details: (string | JSX.Element)[][] = [
        ['Date (Job)', item.jobCreateTime ? getDateTimeString(new Date(item.jobCreateTime)): NOT_APPLICABLE_LABEL],
        ['Date (Step)', startDate ? getDateTimeString(startDate) : NOT_APPLICABLE_LABEL],
        ['Stream', item.streamId],
        ['Code Change', item.change],
        ['Job', item.jobName],
        [`${stepNoun}`, item.name],
        ['Duration', isStepOutcome(item) ? item.getDurationString() : NOT_APPLICABLE_LABEL],
        ['State', item.state],
        // For simplicity, we will compeltely override the outcome of this step with the job's outcome, as the step was aborted.
        item.state === AbortedStr ? ['Job Outcome', handler.getJobResponseData(item.jobId)?.summarizedOutcome ?? "-"] : isTerminalState(item) ? ['Outcome', item.outcome] : null,
        ['Job Link', <a href={`/job/${item.jobId}`} target="_blank" rel="noopen er noreferrer">View Job</a>],
        [`${stepNoun} Link`, <a href={`${stepLink}`} target="_blank" rel="noopener noreferrer">View {stepNoun}</a>],
    ].filter((d): d is (string | JSX.Element)[] => Boolean(d));

    let shouldRenderDuplicateColumn = item.duplicateEntries && item.duplicateEntries.length > 0;

    return (
        <Dialog
            modalProps={{
                isBlocking: false,
                topOffsetFixed: true,
                styles: {
                    root: {
                        selectors: {
                            ".ms-Dialog-title": {
                                paddingTop: '24px',
                                paddingLeft: '32px'
                            }
                        }
                    }
                }
            }}
            onDismiss={onDismiss}
            hidden={false}
            minWidth={1400}
            dialogContentProps={{
                type: DialogType.close,
                onDismiss: onDismiss,
                title: item.name,
            }}>
            <Stack>
                <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { width: '100%', flexGrow: 0 } }}>
                    {/* Left Column - Step Details */}
                    <Stack styles={{ root: { width: '25%' } }} tokens={{ childrenGap: 4 }}>
                        <Stack horizontal tokens={{ childrenGap: 12 }} className={stepOutcomeModalClasses.bottomBorder}>
                            <Label styles={{ root: { padding: 0 } }}>Step Details:</Label>
                        </Stack>
                        {details.map(([label, value], index) => (
                            <Stack key={`modal_step_item_${index}`} styles={{ root: { paddingLeft: 16 } }} horizontal tokens={{ childrenGap: 12 }}>
                                <Label styles={{ root: { minWidth: 100, padding: 0 } }}>{label}:</Label>
                                <Text>{value}</Text>
                            </Stack>
                        ))}
                    </Stack>

                    {/* Center Column - Issues*/}
                    <Stack style={{ width: shouldRenderDuplicateColumn ? '50%' : '75%', maxHeight: 400, display: 'flex', flexDirection: 'column' }}>
                        <Stack horizontal tokens={{ childrenGap: 12 }} className={stepOutcomeModalClasses.bottomBorder}>
                            <Label styles={{ root: { padding: 0 } }}>Issue Count:</Label>
                            <Text>{item.issuesData.length}</Text>
                            <Text>[{item.issuesData.filter((issue) => !issue.resolvedAt).length} unresolved, {item.issuesData.filter((issue) => issue.resolvedAt).length} resolved]</Text>
                        </Stack>
                        <Stack style={{ flex: 1, overflowY: 'auto' }}>
                            <Stack>
                                {item.issuesData.length > 0 ?
                                    <Stack tokens={{ childrenGap: 10 }} styles={{ root: { paddingLeft: 16 } }}>
                                        {item.issuesData.map((issue, index) => (
                                            <Stack key={`modal_issue_item_${index}`} tokens={{ childrenGap: 4 }} className={stepOutcomeModalClasses.bottomBorder}>
                                                {[
                                                    ['Issue', issue.id],
                                                    ['Issue Link', isStepOutcome(item) ? <a href={`/job/${item.jobId}?step=${item.id}&issue=${issue.id}`} target="_blank" rel="noopener noreferrer">View Issue</a> : NOT_APPLICABLE_LABEL],
                                                    ['Issue Summary', issue.summary],
                                                    ['Status', issue.resolvedAt ? "Resolved" : "Unresolved"],
                                                    ['Resolved Streams', `[${issue.resolvedStreams.join(', ')}]`],
                                                    ['Unresolved Streams', `[${issue.unresolvedStreams.join(', ')}]`]
                                                ].map(([label, value], j) => (
                                                    <Stack key={j} horizontal tokens={{ childrenGap: 12 }}>
                                                        <Label styles={{ root: { minWidth: 150, padding: 0 } }}>{label}:</Label>
                                                        <Text>{value}</Text>
                                                    </Stack>
                                                ))}
                                            </Stack>
                                        ))}
                                    </Stack> : null

                                }
                            </Stack>
                        </Stack>
                    </Stack>

                    {/* Right Column - Related Steps*/}
                    {shouldRenderDuplicateColumn && (
                        <Stack style={{ width: '25%', maxHeight: 400, display: 'flex', flexDirection: 'column' }}>
                            <Stack horizontal tokens={{ childrenGap: 12 }} className={stepOutcomeModalClasses.bottomBorder}>
                                <Label styles={{ root: { padding: 0 } }}>Duplicate Steps:</Label>
                            </Stack>
                            <Stack style={{ flex: 1, overflowY: 'auto' }}>
                                <Stack tokens={{ childrenGap: 10 }} styles={{ root: { paddingLeft: 16 } }}>
                                    {item.duplicateEntries.map((relatedEntry, index) => {
                                        let relatedStepLink: string | null = null;
                                        let stepNoun: string | null = null;
                                        if (isStepOutcome(relatedEntry)) {
                                            stepNoun = "Step";
                                            relatedStepLink = `/job/${relatedEntry.jobId}?step=${relatedEntry.id}`;
                                        } else if (isLabelOutcome(item)) {
                                            stepNoun = "Label";
                                            relatedStepLink = `/job/${relatedEntry.jobId}?label=${relatedEntry.id}`;
                                        }

                                        return (
                                            <Stack
                                                key={`related_job_item${index}`}
                                                tokens={{ childrenGap: 4 }}
                                                className={stepOutcomeModalClasses.bottomBorder}
                                            >
                                                {[
                                                    ['Date (Job)', relatedEntry.jobCreateTime ? new Date(relatedEntry.jobCreateTime).toLocaleString() : 'N/A'],
                                                    ['Date (Step)', relatedEntry.startTime ? new Date(relatedEntry.startTime).toLocaleString() : 'N/A'],
                                                    ['Job', relatedEntry.jobName],
                                                    ['Job Link', <a href={`/job/${relatedEntry.jobId}`} target="_blank" rel="noopener noreferrer">View Related Job</a>],
                                                    [`${stepNoun} Link`, <a href={`${relatedStepLink}`} target="_blank" rel="noopener noreferrer">View Related {stepNoun}</a>],
                                                    ['State', relatedEntry.state],
                                                ].map(([label, value], j) => (
                                                    <Stack key={j} horizontal tokens={{ childrenGap: 4 }}>
                                                        <Label styles={{ root: { minWidth: 80, padding: 0 } }}>{label}:</Label>
                                                        <Text>{value}</Text>
                                                    </Stack>
                                                ))}
                                            </Stack>
                                        )
                                    })}
                                </Stack>
                            </Stack>
                        </Stack>
                    )}
                </Stack>
            </Stack >
        </Dialog >
    );
};

// #endregion -- Modal Popup View --

/**
 * React Component that acts as a observable binder for data handler updates, and allows for easy swapping of other table types.
 */
const StepOutcomeTableComponent: React.FC<StepOutcomeTableProps> = observer(({ handler, filters, uiOptions, onCellSelected }) => {
    handler.updated;
    uiOptions;
    const [dataCount, setDataCount] = useState(() => handler.StepOutcomeTableData.dataCount);
    const theme = getHordeTheme();

    useEffect(() => {
        setDataCount(handler.StepOutcomeTableData.dataCount);
    }), [handler.StepOutcomeTableData.dataCount];

    const [header, subHeader, subSubHeader] = prettyPrintStepOutcomeFiltersSummary(filters);

    return (
        <Stack>
            {handler.isInFullDataRefresh ? (
                <Spinner style={{ paddingTop: 16 }} size={SpinnerSize.large} />
            ) :
                <Stack>
                    {dataCount > 0 ? (
                        <Stack>
                            <Stack horizontalAlign="center" styles={{ root: { paddingTop: 8, paddingBottom: 8 } }}>
                                <Text styles={{ root: { fontSize: "large", fontWeight: 'bold' } }}>{header}</Text>
                                <Text styles={{ root: { fontSize: "small", fontWeight: 'bold' } }} >{subHeader} - {subSubHeader}</Text>
                                <Text styles={{ root: { color: theme.semanticColors.bodyBackgroundHovered } }}>Last refresh: {getDateTimeString(handler.lastRefreshDate)}</Text>
                            </Stack>
                            <StepOutcomeD3Table
                                stepOutcomeTable={handler.StepOutcomeTableData}
                                handler={handler}
                                onCellSelected={onCellSelected}
                            />
                        </Stack>
                    ) : (
                        <Text style={{ display: "block", paddingTop: 16, textAlign: "center" }}>
                            {!handler.isInitialRefresh && !handler.isInFullDataRefresh && !handler.isInIncrementalDataRefresh ? (
                                <>
                                    No results were found with the currently selected filters:
                                    <br />
                                    <pre style={{ textAlign: "left", display: "inline-block", marginTop: "8px" }}>
                                        {prettyPrintStepOutcomeFilters(filters)}
                                    </pre>
                                </>
                            ) : (
                                <>Please select filters and run a search to see results.</>
                            )}
                        </Text>
                    )}
                </Stack>
            }
        </Stack>
    );
});

/**
 * React Component that represents the overall Step Outcome Table view.
 * @param StepOutcomeViewProps The filter to use for the step outcome, and the refresh cadence of the data.
 * @returns React Component.
 */
export const StepOutcomeView: React.FC<StepOutcomeViewProps> = ({ filter, uiOptions, refreshTime }) => {
    useEffect(() => {
        handler.start();

        return () => {
            handler.stop();
        };

    }, []);

    useEffect(() => {
        handler.setFilter(filter);
    }, [filter]);

    useEffect(() => {
        handler.StepOutcomeTableData.setViewModelOptions(uiOptions);
    }, [uiOptions]);

    useEffect(() => {
        if (refreshTime) {
            handler.setRefreshTime(refreshTime);
        }
    }, [refreshTime]);

    const [selectedItem, setSelectedItem] = useState<StepOutcomeTableEntry | null>(null);

    return (
        <Stack>
            <Stack key="step-outcome-table">
                {selectedItem && (<StepOutcomeFocusModal item={selectedItem} onDismiss={() => setSelectedItem(null)} />)}
                <StepOutcomeTableComponent handler={handler} filters={filter} uiOptions={uiOptions} onCellSelected={setSelectedItem} />
            </Stack>
        </Stack>
    );
};

// #endregion -- View Components --

// #region -- Script --

const DEFAULT_POLL_TIME = 120000;
const handler = new StepOutcomeDataHandler(DEFAULT_POLL_TIME);

// #endregion -- Script --