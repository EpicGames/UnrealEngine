// Copyright Epic Games, Inc. All Rights Reserved.

import { DirectionalHint, IconButton, mergeStyleSets, Stack, TooltipHost } from "@fluentui/react";
import * as d3 from "d3";
import { getHordeStyling, preloadFonts } from "horde/styles/Styles";
import { useEffect, useRef } from "react";
import { StepOutcomeDataHandler } from "./StepOutcomeDataHandler";
import { AbortedStr, ChangeHeader, ColumnHeader, encodeStepNameFromStepNameHeader, isChangeHeader, isDateHeader, isLabelOutcome, isStepOutcome, isSummary, isSummaryHeader, StepNameHeader, StepOutcomeTable, TableEntry } from "./StepOutcomeDataTypes";
import { getCellAccents, getCellClasses, getCellSortClasses, getCellStyle, getColorRecords, IncrementalRefreshIndicatorPanel, NOT_APPLICABLE_LABEL, resetCellClasses, sharedBorderStyle, headerStyles } from "./StepOutcomeSharedUIComponents";
import { getHordeTheme } from "horde/styles/theme";
import { observer } from "mobx-react-lite";
import { getDateTimeString, isTerminalState } from "./StepOutcomeUtilities";

// #region -- Styles --

const baseTooltipLine = {
    paddingLeft: 8,
    margin: 0.2,
};

const tooltipClasses = mergeStyleSets({
    toolTipSection: sharedBorderStyle.bottomBorder,
    tooltipLine: baseTooltipLine,
    tooltipLineWithBorder: {
        ...baseTooltipLine,
        ...sharedBorderStyle.bottomBorder,
    }
});

// #endregion -- Styles --

// #region -- Step Outcome Table Download Handlers --

function saveFile(content: string, filename: string) {
    const blob = new Blob([content], { type: "text/csv" });
    const url = URL.createObjectURL(blob);

    const a = document.createElement("a");
    a.href = url;
    a.download = filename;
    a.click();

    URL.revokeObjectURL(url);
}

function exportStepOutcomeDataToCSV(stepOutcomeTable: StepOutcomeTable, handler: StepOutcomeDataHandler) {
    const defaultName = new Date()
        .toLocaleDateString("en-US", {
            year: "numeric",
            month: "long",
            day: "numeric",
        })
        .replace(/,/g, "")
        .replace(/ /g, "-");

    const filename = window.prompt("Enter name of export:", "ProjectExport");
    if (!filename) return;

    const stepOrderedCsvContent = stepOutcomeTable.toRowOrderedCSV(handler, isStepOutcome);
    const changeOrderedCsvContent = stepOutcomeTable.toColOrderedCSV(handler, isStepOutcome);

    saveFile(stepOrderedCsvContent, `${filename}-StepOrderedData-${defaultName}.csv`);
    saveFile(changeOrderedCsvContent, `${filename}-ChangeOrderedData-${defaultName}.csv`);

    const labelOrderedCsvContent = stepOutcomeTable.toRowOrderedCSV(handler, isLabelOutcome);
    const labelchangeOrderedCsvContent = stepOutcomeTable.toColOrderedCSV(handler, isLabelOutcome);

    saveFile(labelOrderedCsvContent, `${filename}-LabelOrderedData-${defaultName}.csv`);
    saveFile(labelchangeOrderedCsvContent, `${filename}-LabelChangeOrderedData-${defaultName}.csv`);

    const summaryOrderedCsvContent = stepOutcomeTable.summaryToRowOrderedCSV();

    saveFile(summaryOrderedCsvContent, `${filename}-SummaryData-${defaultName}.csv`);
}

// #endregion -- Step Outcome Table Download Handlers --

/**
 * Function to create a D3 format table for the step outcome table.
 * @param stepOutcomeTable The data to base the table off of.
 * @param handler The data handler used to query step outcome data.
 * @param onCellSelected Cell selected callback.
 * @returns React Component.
 */
function GenerateD3Table(stepOutcomeTable: StepOutcomeTable, handler: StepOutcomeDataHandler, onCellClick: any,): JSX.Element {
    // #region -- Tooltip Helpers --

    const TOOLTIP_SPACER: number = 4;

    let attachDynamicPositionToolTip = (event: MouseEvent, tooltip: d3.Selection<HTMLDivElement, unknown, HTMLElement, any>, tooltipContent: HTMLDivElement) => {
        const _d3 = d3 as any;

        tooltip.html("");
        const toolTipNode = tooltip.node();
        toolTipNode?.appendChild(tooltipContent);

        const tooltipWidth = toolTipNode?.offsetWidth ?? 0;
        const tooltipHeight = toolTipNode?.offsetHeight ?? 0;

        // Get absolute  mouse position.
        const x = event.pageX;
        const y = event.pageY;

        let leftPos = x + TOOLTIP_SPACER;
        let topPos = y - tooltipHeight / 2;

        // If tooltip goes off right edge, flip to the left.
        if (x + tooltipWidth + TOOLTIP_SPACER > window.innerWidth) {
            leftPos = x - tooltipWidth - TOOLTIP_SPACER;
        }

        // Prevent it from going off the top/bottom.
        if (topPos < 0) topPos = 0;
        if (topPos + tooltipHeight > window.innerHeight) {
            topPos = window.innerHeight - tooltipHeight;
        }

        tooltip
            .style("opacity", 1)
            .style("left", `${leftPos}px`)
            .style("top", `${topPos}px`);
    };

    let attachTargettedToolTip = (element: HTMLElement, tooltip: d3.Selection<HTMLDivElement, unknown, HTMLElement, any>, tooltipContent: HTMLDivElement) => {
        tooltip.html("");
        const toolTipNode = tooltip.node();
        toolTipNode?.appendChild(tooltipContent);

        const tooltipWidth = toolTipNode?.offsetWidth ?? 0;
        const tooltipHeight = toolTipNode?.offsetHeight ?? 0;

        const cell = element as HTMLElement;
        const rect = cell.getBoundingClientRect();

        let leftPos = rect.right + TOOLTIP_SPACER; // default: to the right.

        // if tooltip goes outside window, position to the left of cell.
        if (rect.right + tooltipWidth + TOOLTIP_SPACER > window.innerWidth) {
            leftPos = rect.left - tooltipWidth - TOOLTIP_SPACER;
        }

        tooltip
            .style("opacity", 1)
            .style("left", `${leftPos}px`)
            .style("top", `${rect.bottom - tooltipHeight}px`);
    };

    let generateSummaryTooltipRow = (tooltipContent: HTMLDivElement, labelString: string, valueString: string, explanationString?: string) => {
        const ratio = document.createElement("div");
        ratio.style.display = "flex";
        ratio.style.justifyContent = "space-between";
        ratio.style.alignItems = "flex-start";
        ratio.style.paddingBottom = "2px";

        const leftDiv = document.createElement("div");
        leftDiv.style.display = "flex";
        leftDiv.style.flexDirection = "column";

        const label = document.createElement("strong");
        label.textContent = labelString;
        label.style.paddingLeft = "5px";
        leftDiv.appendChild(label);

        if (explanationString !== undefined) {
            const desc = document.createElement("span");
            desc.style.fontSize = "smaller";
            desc.textContent = explanationString;
            desc.style.paddingLeft = "10px";
            leftDiv.appendChild(desc);
        }

        const rightDiv = document.createElement("span");
        rightDiv.textContent = valueString;

        // Append both sides to the main container
        ratio.appendChild(leftDiv);
        ratio.appendChild(rightDiv);

        tooltipContent.appendChild(ratio);
    }

    let generateSummaryTooltipHeader = (tooltipContent: HTMLDivElement, header: string) => {
        const sectionHeaderDiv = document.createElement("div");
        sectionHeaderDiv.style.display = "flex";
        sectionHeaderDiv.style.paddingTop = "10px";
        sectionHeaderDiv.style.justifyContent = "space-between";
        const generateLabelTooltipRow = document.createElement("strong");
        generateLabelTooltipRow.textContent = header;
        sectionHeaderDiv.appendChild(generateLabelTooltipRow);
        sectionHeaderDiv.className = tooltipClasses.toolTipSection;
        tooltipContent.appendChild(sectionHeaderDiv);
    }

    let generateTableTooltipRow = (tooltipContent: HTMLDivElement, label: string, value: string) => {
        const tooltipRowElement = document.createElement("div");
        const tooltipLabelElement = document.createElement("strong");
        tooltipLabelElement.textContent = `${label}:`;

        const tooltipValueElement = document.createElement("p");
        tooltipValueElement.className = tooltipClasses.tooltipLine;
        tooltipValueElement.textContent = value;

        tooltipRowElement.appendChild(tooltipLabelElement);
        tooltipRowElement.appendChild(tooltipValueElement);
        tooltipContent.appendChild(tooltipRowElement);
    }

    // #endregion -- Tooltip Helpers --

    const containerRef = useRef<HTMLDivElement>(null);
    const { modeColors, _hordeClasses, _detailClasses } = getHordeStyling();
    const theme = getHordeTheme();
    const colorRecords: Record<string, string> = getColorRecords();
    const summaryColorScaleBase = d3.scaleLinear<string>()
        .domain([0, 0.5, 1])
        .range([colorRecords["Failure"], colorRecords["Warnings"], colorRecords["Success"]]);

    const summaryColorScale = (value: number): string => {
        if (value === -1) {
            return colorRecords["Waiting"];
        }
        return summaryColorScaleBase(value);
    };

    const scaffold = useRef<{
        wrapper: d3.Selection<HTMLDivElement, unknown, null, undefined>;
        table: d3.Selection<HTMLTableElement, unknown, null, undefined>;
        thead: d3.Selection<HTMLTableSectionElement, unknown, null, undefined>;
        theaderrow: d3.Selection<HTMLTableRowElement, unknown, null, undefined>;
        tbody: d3.Selection<HTMLTableSectionElement, unknown, null, undefined>;
        tooltip: d3.Selection<HTMLDivElement, unknown, HTMLElement, any>;
    } | null>(null);

    // Single execution useEffect to create static part of table.
    useEffect(() => {
        if (!containerRef.current || scaffold.current) return;

        // Create our base container.
        const wrapper = d3.select(containerRef.current)
            .append("div")
            .attr("id", "step-outcome-table-container");

        // Create the table element.
        const table = wrapper.append("table")
            .style("border-collapse", "separate")
            .style("border-spacing", "0px")
            .style("width", "fit-content")
            .style("table-layout", "fixed")
            .style("font-family", preloadFonts[0])
            .style("font-size", "12px")
            .style("color", modeColors.text);

        // Add header && tbody.
        const thead = table.append("thead");
        const tbody = table.append("tbody");

        // Add tool tip div; will be used throughout.
        d3.selectAll(".d3-tooltip").remove();

        const tooltip = d3.select("body")
            .append("div")
            .attr("class", "d3-tooltip")
            .style("position", "absolute")
            .style("pointer-events", "none")
            .style("padding", "6px 12px")
            .style("background", theme.semanticColors.bodyBackground)
            .style("color", theme.semanticColors.bodyText)
            .style("border", `1px solid ${modeColors.content}`)
            .style("z-index", "100")
            .style("min-width", "250px")
            .style("font-size", "12px")
            .style("border-radius", "2px")
            .style("line-height", "1.4")
            .style("opacity", 0);

        const theaderrow = thead.append("tr").style("height", "100px");

        scaffold.current = { wrapper, table, thead, theaderrow, tbody, tooltip };

        return () => {
            d3.selectAll(".d3-tooltip").remove();
        };
    }, []);

    // Re-apply tooltip colors when the theme changes. The tooltip is a body-appended D3 node
    // outside the React tree, so a Horde light/dark toggle won't otherwise reach it.
    useEffect(() => {
        if (!scaffold.current) return;
        scaffold.current.tooltip
            .style("background", theme.semanticColors.bodyBackground)
            .style("color", theme.semanticColors.bodyText)
            .style("border", `1px solid ${modeColors.content}`);
    }, [theme.semanticColors.bodyBackground, theme.semanticColors.bodyText, modeColors.content]);

    // Dependency array use effect for stepOutcomeTable data.
    useEffect(() => {
        if (!containerRef.current || !scaffold.current) return;

        // First column header; (0,0) - this will be responsible for resetting to step insertion order
        scaffold.current.theaderrow
            .selectAll("th.start-header")
            .data(["start"])
            .join(
                enter => enter.append("th")
                    .attr("class", d => `start-header ${getCellSortClasses("start", stepOutcomeTable.sortField, stepOutcomeTable.sortDirectionAsc)}`)
                    .call(th => th
                        .style("position", "sticky")
                        .style("z-index", "7")
                        .style("top", "0")
                        .style("background-color", modeColors.content)
                        .style("width", "30px")
                        .style("min-width", "30px")
                        .style("max-width", "30px")
                        .style("cursor", "pointer")
                        .style("left", "0")

                    ),
                update => update
                    .attr("class", d => `start-header ${getCellSortClasses("start", stepOutcomeTable.sortField, stepOutcomeTable.sortDirectionAsc)}`)
                    .call(th => th
                        .style("background-color", modeColors.content)
                    ),
                exit => exit.remove()
            )
            .on("mouseover", function (_event, _d) {
                d3.select(this).style("background-color", theme.semanticColors.bodyBackgroundHovered);

                const tooltipContent = document.createElement("div");
                tooltipContent.style.paddingLeft = "4px";
                let tooltipLabel: string = "";
                let tooltipValue: string = "";

                if (stepOutcomeTable.sortField !== "start") {
                    tooltipLabel = "Click to sort by";
                    tooltipValue = "Start";
                } else {
                    tooltipLabel = "Sort by Start";
                    tooltipValue = stepOutcomeTable.sortDirectionAsc ? "Asc" : "Desc"
                }

                generateTableTooltipRow(tooltipContent, tooltipLabel, tooltipValue);
                attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
            })
            .on("mouseout", function () {
                scaffold.current!.tooltip.style("opacity", 0);
                d3.select(this).style("background-color", modeColors.content);
            })
            .on("click", (_event, _d) => {
                scaffold.current!.tooltip.style("opacity", 0);
                stepOutcomeTable.changeOrderingBy("start");
            });


        // Second Column Header; (0, 1) - used to display refresh time - this sits on top of step name; guarantee the width of step name TH.
        scaffold.current.theaderrow
            .selectAll("th.refresh-header")
            .data(d3.range(1))
            .join(enter => enter.append("th")
                .attr("class", d => `refresh-header ${getCellSortClasses("name", stepOutcomeTable.sortField, stepOutcomeTable.sortDirectionAsc)}`),
                update => update.attr("class", d => `refresh-header ${getCellSortClasses("name", stepOutcomeTable.sortField, stepOutcomeTable.sortDirectionAsc)}`),
                exit => exit.remove())
            .style("position", "sticky")
            .style("z-index", "6")
            .style("top", "0")
            .style("left", "30px")
            .style("background-color", modeColors.content)
            .style("width", "320px")
            .style("min-width", "320px")
            .style("max-width", "320px")
            .style("cursor", "pointer")
            .text("Step Name")
            .on("mouseover", function (_event, _d) {
                d3.select(this).style("background-color", theme.semanticColors.bodyBackgroundHovered);
                const tooltipContent = document.createElement("div");
                tooltipContent.style.paddingLeft = "4px";
                let tooltipLabel: string = "";
                let tooltipValue: string = "";

                if (stepOutcomeTable.sortField !== "name") {
                    tooltipLabel = "Click to sort by";
                    tooltipValue = "Step Name";
                } else {
                    tooltipLabel = "Sort by Step Name";
                    tooltipValue = stepOutcomeTable.sortDirectionAsc ? "Asc" : "Desc"
                }

                generateTableTooltipRow(tooltipContent, tooltipLabel, tooltipValue);
                attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
            })
            .on("mouseout", function () {
                scaffold.current!.tooltip.style("opacity", 0);
                d3.select(this).style("background-color", modeColors.content);
            })
            .on("click", (_event, _d) => {
                if (handler.isInFullDataRefresh || handler.isInIncrementalDataRefresh) {
                    return;
                }
                scaffold.current!.tooltip.style("opacity", 0);
                stepOutcomeTable.changeOrderingBy("name");
            });

        // Add all columns to the header row; these will be the changelists & the summary; these control downstream widths
        scaffold.current.theaderrow.selectAll("th.col")
            .data([...stepOutcomeTable.getColumnHeaders()] as ColumnHeader[])
            .join("th")
            .attr("class", d => {
                if (!isSummaryHeader(d)) {
                    return "col";
                }
                const arrowClass = getCellSortClasses("summary", stepOutcomeTable.sortField, stepOutcomeTable.sortDirectionAsc);
                return `col ${arrowClass}`;
            })
            .style("position", "sticky")
            .style("top", "0")
            .style("z-index", "3")
            .style("background-color", modeColors.content)
            .style("width", d => (isSummaryHeader(d) ? "70px" : "30px"))
            .style("font-size", d =>
                isDateHeader(d) || isSummaryHeader(d) ? "large" : "small"
            )
            .style("right", d => (isSummaryHeader(d) ? "0" : null))
            .style("z-index", d => (isSummaryHeader(d) ? 4 : 3))
            .style("box-sizing", "border-box")
            .style("padding-bottom", "5px")
            .style("border", `1px solid ${modeColors.background}`)
            .style("border", d =>
                isSummaryHeader(d)
                    ? `2px solid ${modeColors.content}`
                    : `1px solid ${modeColors.background}`
            )
            .style("cursor", entry => (isSummaryHeader(entry) ? "pointer" : ""))
            .html(entry => `<span class="${headerStyles.verticalText}">${isSummaryHeader(entry) ? entry.name : isChangeHeader(entry) ? entry.change : entry.date ?? ""}</span>`)
            .on("mouseover", function (_event, d) {
                const entry = d as unknown as ChangeHeader;

                if (isDateHeader(entry)) {
                    return;
                }

                if (isSummaryHeader(entry)) {
                    const tooltipContent = document.createElement("div");
                    tooltipContent.style.paddingLeft = "4px";
                    let tooltipLabel: string = "";
                    let tooltipValue: string = "";

                    if (stepOutcomeTable.sortField !== "summary") {
                        tooltipLabel = "Click to sort by";
                        tooltipValue = "Success Ratio";
                    } else {
                        tooltipLabel = "Sort by Success Ratio";
                        tooltipValue = stepOutcomeTable.sortDirectionAsc ? "Asc" : "Desc"
                    }

                    generateTableTooltipRow(tooltipContent, tooltipLabel, tooltipValue);
                    attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
                    return;
                }

                d3.select(this).style("background-color", theme.semanticColors.bodyBackgroundHovered);
                if (!entry) {
                    {
                        return;
                    }
                }

                if (isChangeHeader(entry)) {
                    const tooltipContent = document.createElement("div");
                    tooltipContent.style.paddingLeft = "4px";

                    const changeHeader = { change: entry.change ?? "-", date: entry.date ?? "-" };
                    const original = new Date(changeHeader.date!);

                    generateTableTooltipRow(tooltipContent, "Change", changeHeader.change.toString());
                    generateTableTooltipRow(tooltipContent, "Date", getDateTimeString(original));

                    attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
                }
            })
            .on("mouseout", function () {
                scaffold.current!.tooltip.style("opacity", 0);
                d3.select(this).style("background-color", modeColors.content);
            })
            .on("click", (_event, d) => {
                const entry = d as unknown as ChangeHeader;
                if (isSummaryHeader(entry)) {
                    scaffold.current!.tooltip.style("opacity", 0);
                    stepOutcomeTable.changeOrderingBy("summary");
                }
            });

        const rows = scaffold.current.tbody.selectAll("tr")
            .data([...stepOutcomeTable.getStepRowHeaders()] as StepNameHeader[])
            .join("tr")
            .style("height", "30px")
            .style("padding-top", "4px")
            .style("padding-bottom", "4px");

        // Add-Update all of the stream names as groups
        const streamTracker = new Map<string, boolean>();

        // Add all of our stream groupings
        rows.each(function (rowData: StepNameHeader) {
            const tr = d3.select(this);
            const stream = rowData.streamId;
            let metadata = stepOutcomeTable.tableStreamMetadata.get(stream) ?? { stepCount: 1, labelCount: 0 };
            let rowSpan = metadata?.labelCount! + metadata?.stepCount!;

            // Only add a TH for the first row of each stream   
            const streamData = !streamTracker.has(stream)
                ? [{
                    streamId: stream,
                    rowspan: rowSpan
                }]
                : [];

            tr.selectAll("th.stream")
                .data(streamData)
                .join(
                    enter => enter.insert("th", ":first-child")
                        .attr("class", "stream")
                        .attr("rowspan", d => d.rowspan)
                        .style("position", "sticky")
                        .style("left", "0")
                        .style("z-index", "2")
                        .style("writing-mode", "vertical-rl")
                        .style("transform", "rotate(180deg)")
                        .style("background-color", modeColors.content)
                        .style("white-space", "nowrap")
                        .style("overflow", "hidden")
                        .style("text-overflow", "ellipsis")
                        .style("font-size", "16px")
                        .style("text-align", "right")
                        .style("padding-bottom", "10px")
                        .style("box-sizing", "border-box")
                        .style("border-left", "1px solid")
                        .style("border-right", "1px solid")
                        .style("border-bottom", "1px solid")
                        .style("border-top", "1px solid")
                        .text(d => d.streamId),
                    update => update
                        .attr("rowspan", d => d.rowspan)
                        .text(d => d.streamId),
                    exit => exit.remove()
                );

            // Attach tool-tip handlers.
            tr.selectAll("th.stream").each(function (stepNameHeader: StepNameHeader) {
                const th = d3.select(this);
                th.on("mouseover", (event: MouseEvent) => {
                    d3.select(this).style("background-color", theme.semanticColors.bodyBackgroundHovered);

                    const tooltipContent = document.createElement("div");
                    tooltipContent.style.paddingLeft = "4px";

                    if (stepNameHeader) {
                        generateTableTooltipRow(tooltipContent, "Stream", stepNameHeader.streamId);
                    }
                    attachDynamicPositionToolTip(event, scaffold.current!.tooltip, tooltipContent);
                });

                th.on("mouseout", () => {
                    scaffold.current!.tooltip.style("opacity", 0);
                    d3.select(this).style("background-color", modeColors.content);
                });
            });

            streamTracker.set(stream, true);
        });

        // Accessability row highlights.
        rows.on("mouseover", function (_event, _d) {
            const row = d3.select(this);
            // color the row (optional)
            row.style("background-color", theme.semanticColors.bodyBackgroundHovered, "important");

            // also color the sticky step-name cell in this row so it shows the hover
            row.select("td.step-name")
                .style("background-color", theme.semanticColors.bodyBackgroundHovered, "important");
        })
            .on("mouseout", function () {
                const row = d3.select(this);
                // remove row-level inline background
                row.style("background-color", null);

                // restore the sticky cell background to the normal mode color
                row.select("td.step-name")
                    .style("background-color", modeColors.content, "important");
            });

        // Add the first "step name" column item.
        rows.each(function (rowData: StepNameHeader) {
            const tr = d3.select(this);
            const cells = tr.selectAll("td.step-name")
                .data([rowData])
                .join(
                    enter => enter.append("td").attr("class", "step-name").text(d => d.stepName),
                    update => update,
                    exit => exit.remove()
                );

            cells.each(function (d) {
                const cell = d3.select(this);
                cell.html("");

                const span = cell.append('span').text(d.stepName);

                if (d.isLabel) {
                    span.style('background-color', theme.semanticColors.bodyBackgroundHovered);
                }
            })
                .style("position", "sticky")
                .style("left", "32px")
                .style("z-index", "1")
                .style("background-color", modeColors.content)
                .style("border", `1px solid ${modeColors.background}`)
                .style("white-space", "nowrap")
                .style("overflow", "hidden")
                .style("padding-left", "8px")
                .style("text-overflow", "ellipsis")
                .style("box-sizing", "border-box")
                .on("mouseover", function (_event, stepNameHeader) {
                    d3.select(this).style("background-color", theme.semanticColors.bodyBackgroundHovered);

                    const castedStepNameHeader = stepNameHeader as unknown as StepNameHeader;
                    const tooltipContent = document.createElement("div");
                    tooltipContent.style.paddingLeft = "4px";

                    if (castedStepNameHeader) {
                        let calculatedLabel: string = castedStepNameHeader.isLabel ? "Label" : "Step";
                        generateTableTooltipRow(tooltipContent, calculatedLabel, castedStepNameHeader.stepName);
                        generateTableTooltipRow(tooltipContent, "Stream", castedStepNameHeader.streamId);
                    }

                    attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
                })
                .on("mouseout", function () {
                    scaffold.current!.tooltip.style("opacity", 0);
                    d3.select(this).style("background-color", modeColors.content);
                });
        });

        const styleAndBindStepCell = (
            selection: d3.Selection<HTMLDivElement, TableEntry, any, any>
        ) => {
            selection
                .style("vertical-align", "middle")
                .style("width", "90%")
                .style("height", "100%")
                .style("display", "inline-block")
                .style("text-align", "center")
                .style("border-right", d => isSummary(d) ? `2px solid ${modeColors.background}` : null)
                .style("border-left", d => isSummary(d) ? `2px solid ${modeColors.background}` : null)
                .style("box-shadow", "none")
                .each(function (entry) {
                    const div = d3.select(this);
                    resetCellClasses().forEach(cls => div.classed(cls, false));

                    if (!entry) {
                        div.text("").style("background-color", "transparent").style("cursor", "default");
                        return;
                    }
                    if (isStepOutcome(entry) || isLabelOutcome(entry)) { // IOutcomeTableEtnry

                        // UE-324856 - This is a temporary fix until the horde server is fixed.
                        const original = new Date(entry.changeDate!);
                        div.text(""),
                            div.style("cursor", "pointer")
                                .style("background-color", getCellStyle(entry).bgColor)
                                .style("box-shadow", getCellAccents(entry, handler.getJobResponseData(entry.jobId)).boxShadow)
                                .on("click", () => onCellClick?.(entry))
                                .on("mouseover", function () {
                                    const tooltipContent = document.createElement("div");
                                    tooltipContent.style.paddingLeft = "4px";

                                    generateTableTooltipRow(tooltipContent, "Change", entry.change !== undefined ? entry.change.toString() : "-");
                                    generateTableTooltipRow(tooltipContent, "Date", `${getDateTimeString(original) ?? "-"}`);

                                    if (isStepOutcome(entry)) {
                                        generateTableTooltipRow(tooltipContent, "Duration", `${entry.getDurationString?.() ?? "-"}`);
                                    }

                                    generateTableTooltipRow(tooltipContent, "State", `${entry?.state ?? "-"}`);

                                    if (isTerminalState(entry)) {
                                        // For simplicity, we will compeltely override the outcome of this step with the job's outcome, as the step was aborted.
                                        let calculatedLabel: string = entry.state === AbortedStr ? "Job Outcome" : "Outcome";
                                        let calculatedValue: string = entry.state === AbortedStr ? `${handler.getJobResponseData(entry.jobId)?.summarizedOutcome ?? "-"}` : `${entry?.outcome ?? "-"}`;
                                        generateTableTooltipRow(tooltipContent, calculatedLabel, calculatedValue);
                                    }

                                    generateTableTooltipRow(tooltipContent, "Stream", entry.streamId);
                                    generateTableTooltipRow(tooltipContent, "Job", `${entry.jobName ?? "-"}`);

                                    let calculatedLabel: string = isLabelOutcome(entry) ? "Label" : "Step";
                                    generateTableTooltipRow(tooltipContent, calculatedLabel, `${entry?.name ?? "-"}`);

                                    attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
                                })
                                .on("mouseout", () => {
                                    scaffold.current!.tooltip.style("opacity", 0);
                                })
                        getCellClasses(entry, handler.getJobResponseData(entry.jobId)).forEach(cls => div.classed(cls, true));
                    } else if (isSummary(entry)) { // SummaryTableEntry
                        let { ratio, ratioStr }: { ratio: number; ratioStr: string; } = entry.computePassRatio(stepOutcomeTable.viewModelOptions.warningsAsSummaryFailure);
                        let { skippedRatio, skippedRatioStr }: { skippedRatio: number; skippedRatioStr: string; } = entry.computeSkipRatio();
                        let { truePassRatio, truePassRatioStr }: { truePassRatio: number; truePassRatioStr: string; } = entry.computeCompletedPassRatio(stepOutcomeTable.viewModelOptions.warningsAsSummaryFailure);

                        div.style("cursor", "default")
                            .style("background-color", summaryColorScale(ratio))
                            .style("width", "100%")
                            .style("display", "flex")
                            .style("align-items", "center")
                            .style("justify-content", "center")
                            .style("font-weight", "900")
                            .style("color", "black")
                            .text(entry.steps ? ratioStr : "-")
                            .on("click", null)
                            .on("mouseover", function () {
                                const tooltipContent = document.createElement("div");
                                tooltipContent.style.paddingLeft = "4px";

                                // Show ratios
                                generateSummaryTooltipHeader(tooltipContent, "Success & Failure Ratios");
                                generateSummaryTooltipRow(tooltipContent, "Overall Success Ratio:", ratioStr, "(Success / Included Steps)");
                                generateSummaryTooltipRow(tooltipContent, "Completed Success Ratio:", truePassRatioStr, "(Success / (Included Steps - Skipped Steps))");
                                generateSummaryTooltipRow(tooltipContent, "Upstream Failure Ratio:", skippedRatioStr, "(Skipped / Included Steps)");

                                // Show totals
                                // Included steps
                                generateSummaryTooltipHeader(tooltipContent, "Included Steps");
                                generateSummaryTooltipRow(tooltipContent, "Succeeded Steps", entry.stepsPass?.toString());
                                generateSummaryTooltipRow(tooltipContent, "Warning Steps", entry.stepsWarning?.toString());
                                generateSummaryTooltipRow(tooltipContent, "Skipped Steps", entry.stepsSkipped?.toString());
                                generateSummaryTooltipRow(tooltipContent, "Failed Steps", (entry.stepsFail).toString());
                                generateSummaryTooltipRow(tooltipContent, "Failed Completed Steps", (entry.stepsFail - entry.stepsSkipped).toString(), "(Failed Steps - Skipped Steps)");
                                generateSummaryTooltipRow(tooltipContent, "Included Steps (Total)", entry.stepsIncluded?.toString());

                                // Excluded steps
                                generateSummaryTooltipHeader(tooltipContent, "Excluded Steps");
                                generateSummaryTooltipRow(tooltipContent, "Pending Steps", entry.stepsPending?.toString());

                                attachTargettedToolTip(this as HTMLElement, scaffold.current!.tooltip, tooltipContent);
                            })
                            .on("mouseout", () => {
                                scaffold.current!.tooltip.style("opacity", 0);
                            });
                    }
                });
        };

        // Render step outcome data
        rows.each(function (rowData: StepNameHeader) {
            const tr = d3.select(this);

            const cellsRaw = stepOutcomeTable.getStepOutputTableEntries(encodeStepNameFromStepNameHeader(rowData));
            const cells: TableEntry[] = cellsRaw ? [...cellsRaw] : [];

            const dataCells = tr.selectAll("td.step-outcome-data")
                .data(cells, (d: TableEntry, i) => isSummary(d) ? `%summary-${i}` : `${d?.jobId}-${d?.id}`)
                .join("td")
                .attr("class", "step-outcome-data");

            dataCells.style("text-align", "center")
                .style("border-right", `1px solid ${modeColors.content}`)
                .style("position", d => isSummary(d) ? "sticky" : null)
                .style("right", d => isSummary(d) ? "0" : null)
                .style("z-index", d => isSummary(d) ? 2 : 1);

            dataCells.selectAll("div")
                .data(d => [d])
                .join("div")
                .call(div => styleAndBindStepCell(div as d3.Selection<HTMLDivElement, TableEntry, any, any>))
                .style("height", "25px");
        });

    }, [stepOutcomeTable.tableEntries, stepOutcomeTable.viewModelOptions, handler.lastRefreshDate, stepOutcomeTable.sortingKey, getCellStyle]);

    const gridStyle: React.CSSProperties = {
        display: 'grid',
        gridTemplateRows: 'auto',
        gridTemplateColumns: '1fr auto',
    };

    // Binding to the reference of the StepOutcomeTableData is by design. Once we have performed the initial load, we want to snap to the right side.
    // After the user regains control and can horizontally scroll, we do *not* want to snap to right anymore.
    useEffect(() => {
        if (containerRef.current) {
            const container = containerRef.current;

            // We need to make sure the d3 table is fully drawn just so we get the padding included in the width.
            requestAnimationFrame(() => {
                container.scrollLeft = container.scrollWidth;
            });
        }
    }, [handler.StepOutcomeTableData, stepOutcomeTable.viewModelOptions?.includeDateAnchors]);

    const updateMaxHeight = () => {
        if (containerRef.current) {
            const containerTop = containerRef.current.getBoundingClientRect().top;
            const availableHeight = window.innerHeight - containerTop - 10;
            containerRef.current.style.maxHeight = `${availableHeight}px`;
        }
    };

    // Maximum height of container, reactive to zoom events.
    useEffect(() => {
        updateMaxHeight();

        window.addEventListener("resize", updateMaxHeight);

        return () => {
            window.removeEventListener("resize", updateMaxHeight);
        };
    }, []);
    ""
    return (
        <div style={gridStyle}>
            {/* Left side: table */}
            <div id="d3-container"
                ref={containerRef}
                style={{
                    overflow: 'auto'
                }}
            />
            {/* Right side: incremental spinner */}
            <Stack
                key={"export-incremental-container"}
                verticalAlign="stretch"
                styles={{
                    root: {
                        display: "flex",
                        flexDirection: "column",
                        justifyContent: "space-between",
                        height: "100%",
                    },
                }}
            >
                {/* Top: spinner */}
                {/* SubTop: refresh button*/}
                <IncrementalRefreshIndicatorPanel handler={handler} />
                {/* Bottom: download button */}
                <TooltipHost
                    content="Export the current step outcome history data to CSV."
                    directionalHint={DirectionalHint.leftCenter}
                >
                    <IconButton
                        iconProps={{ iconName: 'Download' }}
                        styles={{
                            root: {
                                width: 30,
                                height: 30,
                                minWidth: 0,
                                padding: 0,
                            },
                        }}
                        onClick={() => exportStepOutcomeDataToCSV(stepOutcomeTable, handler)}
                    />
                </TooltipHost>
            </Stack>
        </div>
    );
};

/**
 * React Component that represents the provided stepOutcomeTable. This is a D3 table.
 * @param stepOutcomeTable The data to base the table off of.
 * @param handler The data handler used to query step outcome data.
 * @param onCellSelected Cell selected callback.
 * @returns React Component.
 */
export const StepOutcomeD3Table: React.FC<{ stepOutcomeTable: StepOutcomeTable; handler: StepOutcomeDataHandler, onCellSelected: any; }> = observer(({ stepOutcomeTable, handler, onCellSelected }) => {
    return GenerateD3Table(stepOutcomeTable, handler, onCellSelected);
});
