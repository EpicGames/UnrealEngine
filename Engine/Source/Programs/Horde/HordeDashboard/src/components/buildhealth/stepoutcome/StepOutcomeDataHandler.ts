// Copyright Epic Games, Inc. All Rights Reserved.

import { GetJobResponse, GetJobStepRefResponse, IssueQuery, JobStepOutcome, JobStepState, JobStreamQuery, LabelState } from "horde/backend/Api";
import { PollBase } from "horde/backend/PollBase";
import { GetJobStepRefResponseWrapper, GetStepResponseWrapper, JobResponseData, LabelOutcomeTableEntry, StepOutcomeTable, StepOutcomeTableEntry, UpdatedChangeSummaryData } from "./StepOutcomeDataTypes";
import backend from "horde/backend";
import { decodeLabelKey, decodeStepKey, decodeTemplateKey, encodeLabelKeyFromStrings, encodeStepNameFromStringsPreserveCase, encodeTemplateKeyFromStrings } from "../BuildHealthUtilities";
import { action, computed, makeObservable, observable, runInAction } from "mobx";
import { summarizeJob, summarizeJobFromJobStepRef } from "./StepOutcomeUtilities";

// #region -- API Data Types & Utilities --

/**
 * Utility function to create a formatted string of the StepOutcomeFilters object for messaging.
 * @param filters The filters object to use in creating the string.
 * @returns A formated string representation of the StepOutcomeFilters.
 */
export function prettyPrintStepOutcomeFilters(filters: StepOutcomeFilters): string {
    const parts: string[] = [];

    parts.push(`Streams: ${filters.streams.join(", ") || "(none)"}`);

    const start = filters.jobHistorySpan.start.toISOString().split("T")[0];
    const end = filters.jobHistorySpan.end ? filters.jobHistorySpan.end.toISOString().split("T")[0] : "(ongoing)";

    parts.push(`Job History: ${start} → ${end}`);

    if (filters.jobs?.length) {
        parts.push(`Jobs: ${filters.jobs.join(", ")}`);
    }

    if (filters.steps?.length) {
        parts.push(`Steps: ${filters.steps.join(", ")}`);
    }

    if (filters.maxJobCount !== undefined) {
        parts.push(`Max Job Count: ${filters.maxJobCount}`);
    }

    return parts.join("\n");
}

/**
 * Utility function to create a formatted string of the StepOutcomeFilters object for table summary.
 * @param filters The filters object to use in creating the strings.
 * @returns A three string representation of the filter as a summary: header, sub header, secondary sub header.
 */
export function prettyPrintStepOutcomeFiltersSummary(filters: StepOutcomeFilters): [string, string, string] {
    // Header: Stream(s)
    const header = `Streams: ${filters.streams.join(", ") || "(none)"}`;

    // Subheader: Job(s) - Steps
    const jobs = filters.jobs?.length
        ? (filters.jobs.length > 3 ? "(many)" : filters.jobs.join(", "))
        : "(all jobs)";

    const steps = filters.steps?.length
        ? (filters.steps.length > 3 ? "(many)" : filters.steps.join(", "))
        : "(all steps)";

    const subHeader = `Jobs: ${jobs} - Steps: ${steps}`;

    // Sub-subheader: Last N Days/Hours
    const start = filters.jobHistorySpan.start;
    const end = filters.jobHistorySpan.end || new Date();

    const diffMs = end.getTime() - start.getTime();
    const diffDays = Math.floor(diffMs / (1000 * 60 * 60 * 24));

    let subSubHeader: string;
    if (diffDays >= 1) {
        subSubHeader = `Last ${diffDays} day${diffDays > 1 ? "s" : ""}`;
    } else {
        const diffHours = Math.ceil(diffMs / (1000 * 60 * 60));
        subSubHeader = `Last ${diffHours} hour${diffHours > 1 ? "s" : ""}`;
    }

    return [header, subHeader, subSubHeader];
}

/**
 * Data structure that controls the visual options for the StepOutcome
 */
export interface StepOutcomeViewOptions {
    includeDateAnchors: boolean;
    warningsAsSummaryFailure: boolean;
    validStates: Set<string>;
}

/**
 * Data structure that controls the job start filter for the StepOutcome
 */
export interface StepOutcomeJobStartFilter {
    supportsScheduledStarts: boolean;
    supportsManualStarts: boolean;
}

/**
 * Data structure that controls the filtering & queries of the Data Handler.
 * @todo UE-309726 - This data structure can be simplified to reduce duplication, and reduce the need for the internal container to decode.
 */
export interface StepOutcomeFilters {
    streams: string[];
    jobHistorySpan: { start: Date; end?: Date };
    jobs?: string[];
    steps?: string[];
    labels?: string[];
    jobStartFilter?: StepOutcomeJobStartFilter;
    maxJobCount?: number;
    includePreflights: boolean;
    includeCancelledJobs: boolean;
    debugMode: boolean;
}

// #endregion -- API Data Types & Utilities --

/**
 * Data handler for all step outcome data used for step outcomes.
 * @todo Data sharing & caching would ideally occur between the BuildHealth View & Step Outcome.
 */
export class StepOutcomeDataHandler extends PollBase {
    private readonly STREAM_JOBS_FILTER = "id,streamId,name,change,templateId,state,createTime,updateTime,startedByUserInfo,abortedByUserInfo,preflightCommitId,preflightDescription,cancellationReason,labels,defaultLabel";
    private readonly STREAM_JOBS_BATCH_STEP_FILTER = "batches.error,batches.steps.id,batches.steps.name,batches.steps.state,batches.steps.outcome,batches.steps.startTime,batches.steps.finishTime,batches.steps.warning,batches.steps.error"
    private readonly STREAM_JOBS_BATCH_ERROR_FILTER = "batches.error"
    private readonly STREAM_CHANGES_FILTER = "id,dateUtc";
    static readonly MAX_JOB_COUNT_DEFAULT = 400;

    // #region -- Private Members -- 

    @observable
    private internalLastRefreshDate?: Date;

    @observable
    private intialRefresh = true;
    @observable
    private activeRefresh: boolean = false;

    private activeFilter: StepOutcomeFilters;
    private jobData: Map<string, JobResponseData> = new Map<string, JobResponseData>();
    private changeData: Map<number, UpdatedChangeSummaryData> = new Map<number, UpdatedChangeSummaryData>();
    private stepOutcomeTableData: StepOutcomeTable = new StepOutcomeTable();

    //#endregion -- Private Members

    // #region -- Constructor --

    /**
     * Cosntructs a StepOutcomeDataHandler.
     * @param pollTime The initial poll refresh time.
     */
    constructor(pollTime: number) {
        super(pollTime);
        makeObservable(this);
    }

    //#endregion -- Constructor --

    // #region -- Private API --

    private constructStreamJobQueries(templateId: string, jobCount: number, modifiedAfter: string, queries: { streamId: string; query: JobStreamQuery; }[], streamId: string, debugMode: boolean) {
        let completeFilter: string = !debugMode ? `${this.STREAM_JOBS_FILTER},${this.STREAM_JOBS_BATCH_STEP_FILTER}` : `${this.STREAM_JOBS_FILTER},${this.STREAM_JOBS_BATCH_ERROR_FILTER}`
        const query: JobStreamQuery = {
            template: [templateId],
            count: jobCount,
            filter: completeFilter,
            modifiedAfter: modifiedAfter
        };

        queries.push({ streamId, query });
    }

    private constructJobStepHistoryQueries(stepMap: Map<string, Map<string, Set<string>>>, streamId: string, templateId: string, jobStepHistoryBatchQueries: { streamId: string; stepNames: string[]; templateId: string; }[]) {
        let templateMap: Map<string, Set<string>> = stepMap.get(streamId)!;
        let stepSet = templateMap?.get(templateId);
        if (stepSet) {
            jobStepHistoryBatchQueries.push({
                streamId: streamId, stepNames: Array.from(stepSet).map(x => {
                    let { streamId, templateId, stepName } = decodeStepKey(x);
                    return stepName;
                }), templateId: templateId
            });
        }
    }

    private async issueAllStreamJobQueries(streamJobQueries: { streamId: string; query: JobStreamQuery; }[]) {
        return await Promise.all(
            streamJobQueries.map(({ streamId, query }) => backend.getStreamJobs(streamId, query))
        );
    }

    private async issueAllJobStepHistoryQueries(jobStepHistoryBatchQueries: { streamId: string; stepNames: string[]; count: number; templateId: string; }[]) {
        return await Promise.all(
            jobStepHistoryBatchQueries.map(({ streamId, stepNames, count, templateId }) => backend.getJobStepHistoryBatch(streamId, stepNames, count, templateId))
        );
    }

    private processJobStepResults(results: GetJobResponse[][], stepMap: Map<string, Map<string, Set<string>>>, labelMap: Map<string, Map<string, Set<string>>>, jobStepKeyToState: Map<string, JobStepState>) {
        for (const streamResult of results) {
            // Process all job responses
            for (const jobResponse of streamResult) {
                // Filter preflights as appropriate
                if (!this.activeFilter.includePreflights && (jobResponse.preflightChange || jobResponse.preflightDescription)) {
                    continue;
                }

                // Filter cancelled jobs as appropriate
                if (!this.activeFilter.includeCancelledJobs && (jobResponse.abortedByUserInfo !== null)) {
                    continue;
                }

                // Filter for job start method
                if (this.activeFilter.jobStartFilter) {
                    const { supportsManualStarts, supportsScheduledStarts } = this.activeFilter.jobStartFilter;

                    const isManual = jobResponse.startedByUserInfo !== null;
                    const isScheduled = !isManual;

                    if (
                        (!supportsManualStarts && isManual) ||
                        (!supportsScheduledStarts && isScheduled)
                    ) {
                        continue;
                    }
                }

                const batchResponses = jobResponse.batches ?? [];
                let streamStepSet = stepMap.get(jobResponse.streamId);
                let streamLabelSet = labelMap.get(jobResponse.streamId);
                let templateStepSet = streamStepSet ? streamStepSet.get(jobResponse.templateId!) : undefined;
                let templateLabelSet = streamLabelSet ? streamLabelSet.get(jobResponse.templateId!) : undefined;

                if (templateStepSet && templateStepSet.size >= 0) {
                    for (const batch of batchResponses) {
                        for (const step of batch.steps) {
                            let unifiedKey: string = jobResponse.id + step.id;
                            if (!jobStepKeyToState.has(unifiedKey)) {
                                jobStepKeyToState.set(unifiedKey, step.state);
                            }

                            let fullyQualifiedStepName = encodeStepNameFromStringsPreserveCase(jobResponse.streamId, jobResponse.templateId!, step.name);

                            if (templateStepSet &&
                                templateStepSet.size >= 0 &&
                                templateStepSet.has(fullyQualifiedStepName)) {
                                this.stepOutcomeTableData.addEntry(new StepOutcomeTableEntry(new GetStepResponseWrapper(step), jobResponse.id, jobResponse.name, jobResponse.streamId, jobResponse.createTime.toString(), jobResponse.change, this.changeData.get(jobResponse.change!)?.dateUtc, batch));
                            }
                        }
                    }
                }

                // Do not compute label steps if there are no steps for the template.
                if (templateLabelSet && templateLabelSet.size >= 0) {
                    const labelResponses = jobResponse.labels ?? [];
                    for (let idx: number = 0; idx < labelResponses.length; ++idx) {
                        let label = labelResponses[idx];
                        let synthesizedStepOutcome: JobStepState | LabelState = LabelState.Unspecified;

                        // Labels lack sufficient cancellation & skipped context, and inferring this from their .steps member is often incorrect (steps may not be present)
                        // We therefore apply a simple heuristic: if there is cancellation info - we anotate the synthesized step as aborted; else skipped.
                        if (label.state === LabelState.Unspecified) {
                            synthesizedStepOutcome = jobResponse.abortedByUserInfo !== undefined ? JobStepState.Aborted : JobStepState.Skipped;
                        }

                        let fullyQualifiedLabelName = encodeLabelKeyFromStrings(jobResponse.streamId, jobResponse.templateId!, label.dashboardCategory, label.dashboardName).toLocaleLowerCase();

                        if (templateLabelSet &&
                            templateLabelSet.size >= 0 &&
                            templateLabelSet.has(fullyQualifiedLabelName)) {
                            this.stepOutcomeTableData.addEntry(new LabelOutcomeTableEntry(label, idx, jobResponse.id, jobResponse.name, jobResponse.streamId, jobResponse.createTime.toString(), jobResponse.change, this.changeData.get(jobResponse.change!)?.dateUtc, synthesizedStepOutcome));
                        }
                    }
                }
            }
        }
    }

    private processJobStepHistoryResults(jobStepHistoryResults: GetJobStepRefResponse[][]) {
        let returnMap: Map<string, GetJobStepRefResponse[]> = new Map<string, GetJobStepRefResponse[]>();
        for (const item of jobStepHistoryResults) {
            for (const subitem of item) {
                if (this.jobData.has(subitem.jobId)) {
                    let jobName: string = this.jobData.get(subitem.jobId)?.name!;
                    let streamId: string = this.jobData.get(subitem.jobId)?.streamId!;
                    this.stepOutcomeTableData.addEntry(new StepOutcomeTableEntry(new GetJobStepRefResponseWrapper(subitem), subitem.jobId, jobName, streamId, subitem.jobStartTime.toString(), subitem.change, this.changeData.get(subitem.change!)?.dateUtc));
                    if (!returnMap.has(subitem.jobId)) {
                        returnMap.set(subitem.jobId, []);
                    }
                    returnMap.get(subitem.jobId)?.push(subitem);
                }
            }
        }

        for (const item of returnMap) {
            this.jobData.get(item[0])!.summarizedOutcome = summarizeJobFromJobStepRef(item[1], this.jobData.get(item[0])!);
        }
    }

    // #endregion -- Private API --

    // #region -- Interface --

    @action
    async poll(): Promise<void> {
        if (this.activeRefresh ||
            this.activeFilter === undefined ||
            !this.activeFilter.streams?.length ||
            !this.activeFilter.jobs?.length ||
            (!this.activeFilter.steps?.length && !this.activeFilter.labels?.length)
        ) {
            this.stepOutcomeTableData.reset();
            this.setUpdated();
            return;
        }
        runInAction(() => {
            this.activeRefresh = true;
        });

        this.stepOutcomeTableData.reset();

        // @todo UE-309726 - The poll should not need to understand internal data structure; the filter object should be already decoded.
        const stepMap: Map<string, Map<string, Set<string>>> = new Map();
        const labelMap: Map<string, Map<string, Set<string>>> = new Map();

        (this.activeFilter.steps ?? []).forEach(step => {
            const { streamId, templateId, stepName } = decodeStepKey(step);

            if (!stepMap.has(streamId)) {
                stepMap.set(streamId, new Map());
            }

            const templateMap = stepMap.get(streamId)!;

            if (!templateMap.has(templateId)) {
                templateMap.set(templateId, new Set());
            }

            templateMap.get(templateId)!.add(step);
        });

        (this.activeFilter.labels ?? []).forEach(label => {
            const { streamId, templateId, dashboardCategory, dashboardName } = decodeLabelKey(label);

            if (!labelMap.has(streamId)) {
                labelMap.set(streamId, new Map());
            }

            const templateMap = labelMap.get(streamId)!;

            if (!templateMap.has(templateId)) {
                templateMap.set(templateId, new Set());
            }

            templateMap.get(templateId)!.add(label.toLocaleLowerCase());
        });

        const streamList = this.activeFilter.streams ?? [];
        const modifiedAfter = this.activeFilter.jobHistorySpan.start.toISOString();
        const jobCount = this.activeFilter.maxJobCount ?? StepOutcomeDataHandler.MAX_JOB_COUNT_DEFAULT;
        const jobStepKeyToState = new Map<string, JobStepState>();

        // Group jobs by streamId
        const jobsByStream: Record<string, string[]> = {};

        (this.activeFilter.jobs ?? []).forEach(jobKey => {
            // @todo UE-309726 - The poll should not need to understand internal data structure; the filter object should be already decoded.
            const { streamId, templateId } = decodeTemplateKey(jobKey);
            if (!jobsByStream[streamId]) {
                jobsByStream[streamId] = [];
            }
            jobsByStream[streamId].push(templateId);
        });

        const changeQueries: { streamId: string; query: any }[] = [];
        const streamJobQueries: { streamId: string; query: JobStreamQuery }[] = [];
        const jobStepHistoryBatchQueries: { streamId: string; stepNames: string[], templateId: string }[] = [];

        // Construct queries
        for (const streamId of streamList) {
            const jobsForStream = jobsByStream[streamId] ?? [];
            for (const templateId of jobsForStream) {
                changeQueries.push({ streamId: streamId, query: { filter: this.STREAM_CHANGES_FILTER } });
                this.constructStreamJobQueries(templateId, jobCount, modifiedAfter, streamJobQueries, streamId, this.activeFilter.debugMode);
                this.constructJobStepHistoryQueries(stepMap, streamId, templateId, jobStepHistoryBatchQueries);
            }
        }

        const streamJobResults = await this.issueAllStreamJobQueries(streamJobQueries);

        // Annotate the changes that we need to request in getChangeSummaries.
        let streamToChangeRequestList: Map<string, number[]> = new Map<string, number[]>();
        let streamJobToRequestSize: Map<string, number> = new Map<string, number>();

        streamJobResults.forEach((jobs, i) => {
            const streamId = streamJobQueries[i].streamId;
            const newChanges: number[] = [];

            for (let idx: number = 0; idx < jobs.length; ++idx) {
                let change: number = jobs[idx].change!;
                let fullyQualifiedId = encodeTemplateKeyFromStrings(streamId, jobs[idx].templateId!);

                if (!this.changeData.has(jobs[idx].change!)) {
                    newChanges.push(change);
                }

                if (!streamJobToRequestSize.has(fullyQualifiedId)) {
                    streamJobToRequestSize.set(fullyQualifiedId, 0);
                }

                streamJobToRequestSize.set(fullyQualifiedId, streamJobToRequestSize.get(fullyQualifiedId)! + 1);
            }

            if (newChanges.length > 0) {
                if (!streamToChangeRequestList.has(streamId)) {
                    streamToChangeRequestList.set(streamId, []);
                }

                streamToChangeRequestList.get(streamId)!.push(...newChanges);
            }
        });

        // Flatten all jobData & memoize the summarized outcome
        const jobStepHistoryResults = this.activeFilter.debugMode ? await this.issueAllJobStepHistoryQueries(jobStepHistoryBatchQueries.map(x => ({ ...x, count: streamJobToRequestSize.get(encodeTemplateKeyFromStrings(x.streamId, x.templateId))! }))) : undefined;

        this.jobData = new Map<string, JobResponseData>(streamJobResults.flat().map((job: GetJobResponse): [string, JobResponseData] => {
            return [
                job.id,
                {
                    ...job,
                    summarizedOutcome: this.activeFilter.debugMode ? JobStepOutcome.Unspecified : summarizeJob(job),
                    isManualJobInvocation: job.startedByUserInfo !== null
                }
            ]
        }
        ));

        if (streamToChangeRequestList.size > 0) {
            // Parallelize backend calls for change summaries
            const changeResults = await Promise.all(
                changeQueries.map(({ streamId, query }) => backend.getBatchChangeSummaries(streamId, [...new Set(streamToChangeRequestList.get(streamId))], query.filter))
            );

            let flattenedResults = changeResults.flat();
            flattenedResults.map((value: UpdatedChangeSummaryData) => {
                if (!this.changeData.has(value.id.order)) {
                    this.changeData.set(value.id.order, value);
                }
            });

            console.info(`Retrieved: ${flattenedResults.length} change summary results for StepOutcome query.`);
        }

        if (this.activeFilter.debugMode && jobStepHistoryResults) {
            this.processJobStepHistoryResults(jobStepHistoryResults);
        }
        else if (streamJobResults) {
            this.processJobStepResults(streamJobResults, stepMap, labelMap, jobStepKeyToState);
        }

        this.StepOutcomeTableData.summarize();

        console.info(`Retrieved: ${this.jobData.size} job instance results for StepOutcome query.`);

        runInAction(() => {
            this.stepOutcomeTableData.orderTableDataByChange(true);
            this.stepOutcomeTableData.changeOrderingBy(this.stepOutcomeTableData.sortField, false);

            this.activeRefresh = false;
            this.intialRefresh = false;
            this.internalLastRefreshDate = new Date();
        });

        this.setUpdated();

        return Promise.resolve();
    }

    // #endregion -- Interface --

    // #region -- Public API  --

    /**
     * Will populate the provided stepOutcomeTableEntry's issue data field,
     * @param stepOutcomeTableEntry The step outcome table entry to attempt to populate with issue data.
     */
    async populateStepOutcomeTableEntryIssueData(stepOutcomeTableEntry: StepOutcomeTableEntry, onCompleteCallback?: (boolean) => void): Promise<void> {
        let unresolvedIssueQuery: IssueQuery = { jobId: stepOutcomeTableEntry.jobId, stepId: stepOutcomeTableEntry.id, resolved: false }
        const unresolvedIssueResults = await backend.getIssues(unresolvedIssueQuery);
        stepOutcomeTableEntry.issuesData = unresolvedIssueResults;

        let resolvedIssueQuery: IssueQuery = { jobId: stepOutcomeTableEntry.jobId, stepId: stepOutcomeTableEntry.id, resolved: true }
        const resolvedIssueResults = await backend.getIssues(resolvedIssueQuery);
        stepOutcomeTableEntry.issuesData.push(...resolvedIssueResults);

        onCompleteCallback?.(true);
        return Promise.resolve();
    }


    /**
     * Gets the last refresh date.
     * @return The last refresh date, if a refresh has occurred. 
     */
    @computed
    get lastRefreshDate(): Date | undefined {
        return this.internalLastRefreshDate;
    }

    /**
     * Gets whether the handler has yet to initiate the initial refresh.
     * @returns True if the handler has not issued a initial refresh, false otherwise.
     */
    @computed
    get isInitialRefresh(): boolean {
        return this.intialRefresh;
    }

    /**
     * Gets whether an full data refresh is occurring.
     * @returns True if an full data refresh is occurring, false otherwise.
     */
    @computed
    get isInFullDataRefresh(): boolean {
        return this.activeRefresh && this.intialRefresh;
    }

    /**
     * Gets whether an incremental data refresh is occurring.
     * @returns True if an incremental data refresh is occurring, false otherwise.
     */
    @computed
    get isInIncrementalDataRefresh(): boolean {
        return this.activeRefresh && !this.intialRefresh;
    }

    /**
     * Gets the step outcome table data.
     * @returns The instance Step Outcome Data Table.
     */
    get StepOutcomeTableData(): StepOutcomeTable {
        return this.stepOutcomeTableData;
    }

    /**
     * Sets a new filter for the handler, which will control the underlying dataset.
     * @param filter The new filter to use.
     */
    @action
    setFilter(filter: StepOutcomeFilters) {
        this.activeFilter = filter;
        this.intialRefresh = true;
        this.poll();
    }

    /**
     * Gets the current refresh cadence of the data handler.
     * @returns The handler refresh cadence in ms.
     */
    get refreshTime(): number {
        return this.pollTime;
    }

    /**
     *  Sets a new refresh time for the handler, in ms.
     * @param newRefreshTime The new refresh rate.
     */
    setRefreshTime(newRefreshTime: number) {
        this.pollTime = newRefreshTime;
    }

    /**
     * Obtains the corresponding JobResponseData given the job id.
     * @param jobId The job id for the corresponding JobResponseData to retrieve.
     * @returns The JobResponseData if it is present in the handler, undefined otherwise.
     */
    getJobResponseData(jobId: string): JobResponseData | undefined {
        return this.jobData.get(jobId);
    }

    // #endregion -- Public API  --
}