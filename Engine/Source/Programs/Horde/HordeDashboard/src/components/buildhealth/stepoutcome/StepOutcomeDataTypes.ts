// Copyright Epic Games, Inc. All Rights Reserved.

import { ChangeSummaryData, GetBatchResponse, GetJobResponse, GetJobStepRefResponse, GetLabelStateResponse, GetStepResponse, IssueData, JobStepBatchError, JobStepOutcome, JobStepState, LabelOutcome, LabelState } from "horde/backend/Api";
import { computeDuration, DURATION_NOT_SET, formatDurationFromMs, toDate } from "./StepOutcomeUtilities";
import { StepOutcomeDataHandler, StepOutcomeViewOptions } from "./StepOutcomeDataHandler";
import { action, computed, makeObservable, observable } from "mobx";
import { NOT_APPLICABLE_LABEL } from "./StepOutcomeSharedUIComponents";

export const WarningStr: string = "Warnings";
export const SuccessStr: string = "Success";
export const SkippedStr: string = "Skipped";
export const FailureStr: string = "Failure";
export const WaitingStr: string = "Waiting";
export const RunningStr: string = "Running";
export const ReadyStr: string = "Ready";
export const AbortedStr: string = "Aborted";
export const CompletedStepStateStr: string = "Completed";
export const CompleteLabelStateStr: string = "Complete";

/**
 * Encodes a step outcome table entry to a stream qualified step name.
 * Note: The template is not included in the hierarchy qualification since two separate templates within the same stream will collocate the same step row.
 * @param stepOutcomeTableEntry The stepOutcomeTableEntry to encode.
 * @returns The hierarchical stream qualified step name.
 */
export function encodeStepName(stepOutcomeTableEntry: OutcomeTableEntry) {
    return `${stepOutcomeTableEntry.streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${stepOutcomeTableEntry.name.toLocaleLowerCase()}`;
}

/**
 * Encodes a step outcome table entry to a stream qualified step name.
 * Note: The template is not included in the hierarchy qualification since two separate templates within the same stream will collocate the same step row.
 * @param stepNameHeader The stepNameHeader to encode.
 * @returns The hierarchical stream qualified step name.
 */
export function encodeStepNameFromStepNameHeader(stepNameHeader: StepNameHeader) {
    return `${stepNameHeader.streamId.toLocaleLowerCase()}${KEY_SEPARATOR}${stepNameHeader.stepName.toLocaleLowerCase()}`;
}

// #region -- Constants --

const KEY_SEPARATOR: string = "::";

// #endregion -- Constants --

// #region -- Data Types --

/**
 * ChangeId type that has the string & numerical representation of a commit id.
 */
export type ChangeId = {
    name: string
    order: number
}

/**
 * Convenience type to fold in change id into the existing ChangeSummaryData.
 */
export type UpdatedChangeSummaryData = ChangeSummaryData & {
    id: ChangeId
}

/*
* Convenience type to fold in extra metadata relevant for the JobResponse.
*/
export type JobResponseData = GetJobResponse & {
    summarizedOutcome: JobStepOutcome;
    isManualJobInvocation: boolean;
}

// #region -- Table Entry Types --

/**
 * Discriminated union which represents a tabel entry. This can be a @see StepOutcomeTableEntry (the core data in the table) or a @see SummaryTableEntry (the summarization of an entire row).
 */
export type TableEntry = OutcomeTableEntry | SummaryTableEntry | undefined;

/**
 * Type predicate to test whether this is a @see StepOutcomeTableEntry .
 * @param entry The entry to test.
 * @returns True if it is a @see StepOutcomeTableEntry - false otherwise.
 */
export function isStepOutcome(entry: TableEntry): entry is StepOutcomeTableEntry {
    return entry instanceof StepOutcomeTableEntry;
}

/**
 * Type predicate to test whether this is a @see LabelOutcomeTableEntry .
 * @param entry The entry to test.
 * @returns True if it is a @see LabelOutcomeTableEntry - false otherwise.
 */
export function isLabelOutcome(entry: TableEntry): entry is LabelOutcomeTableEntry {
    return entry instanceof LabelOutcomeTableEntry;
}

/**
 * Type predicate to test whether this is a @see SummaryTableEntry .
 * @param entry The entry to test.
 * @returns True if it is a @see SummaryTableEntry - false otherwise.
 */
export function isSummary(entry: TableEntry): entry is SummaryTableEntry {
    return entry instanceof SummaryTableEntry;
}

/**
 * Data structure that represents a step summary for a collection of @see StepOutcomeTableEntry data.
 */
export class SummaryTableEntry {
    steps: number = 0;
    stepsIncluded: number = 0;
    stepsPass: number = 0;
    stepsFail: number = 0;
    stepsWarning: number = 0;
    stepsPending: number = 0;
    stepsSkipped: number = 0;

    /**
     * Computes the pass ratio of the summary entry.
     * @param warningsAsSummaryFailre Whether to include warnings as failures.
     * @returns The numerical and string representation of the pass ratio.
     */
    computePassRatio(warningsAsSummaryFailre: boolean) {
        let ratio: number = -1;

        if (this.steps > 0 && this.stepsIncluded > 0) {
            let numerator: number = this.stepsPass;

            if (!warningsAsSummaryFailre) {
                numerator += this.stepsWarning;
            }

            ratio = numerator / this.stepsIncluded;
        }

        let ratioStr: string = ratio === -1 ? NOT_APPLICABLE_LABEL : `${(ratio * 100).toFixed(1)}%`;
        return { ratio, ratioStr };
    }

    /**
     * Computes the skip ratio of the summary entry.
     * @returns The numerical and string representation of the skip ratio.
     */
    computeSkipRatio() {
        let skippedRatio: number = -1;

        if (this.steps > 0 && this.stepsIncluded > 0) {
            let numerator: number = this.stepsSkipped;
            skippedRatio = numerator / this.stepsIncluded;
        }

        let skippedRatioStr: string = skippedRatio === -1 ? NOT_APPLICABLE_LABEL : `${(skippedRatio * 100).toFixed(1)}%`;
        return { skippedRatio, skippedRatioStr };
    }

    /**
     * Computes the completed pass ratio of the summary entry.
     * @param warningsAsSummaryFailre Whether to include warnings as failures.
     * @returns The numerical and string representation of the completed pass ratio.
     */
    computeCompletedPassRatio(warningsAsSummaryFailre: boolean) {
        let truePassRatio: number = -1;

        if (this.steps > 0 && this.stepsIncluded > 0) {
            let numerator: number = this.stepsPass;

            if (!warningsAsSummaryFailre) {
                numerator += this.stepsWarning;
            }

            truePassRatio = (this.stepsIncluded - this.stepsSkipped) === 0 ? -1 : numerator / (this.stepsIncluded - this.stepsSkipped);
        }

        let truePassRatioStr: string = truePassRatio === -1 ? NOT_APPLICABLE_LABEL : `${(truePassRatio * 100).toFixed(1)}%`;
        return { truePassRatio, truePassRatioStr };
    }

    /**
     * The CSV representation of the summary entry.
     * @returns String representation of CSV row.
     */
    toCSVRepresentation() {
        return `${this.steps},${this.stepsIncluded},${this.stepsPass},${this.stepsFail},${this.stepsWarning},${this.stepsPending},${this.stepsSkipped},${this.stepsFail - this.stepsSkipped},${this.computePassRatio(true).ratioStr},${this.computePassRatio(false).ratioStr},${this.computeCompletedPassRatio(true).truePassRatioStr},${this.computeCompletedPassRatio(false).truePassRatioStr},${this.computeSkipRatio().skippedRatioStr}\n`
    }

    /**
     * The CSV header of the summary entry.
     * @returns The CSV header representing the columns.
     */
    static getCSVHeader(): string {
        return "Step,StepCount,StepsIncluded,StepsPassed,StepsFailed,StepsWarning,StepsPending,StepsSkipped,FailedCompletedSteps,PassRatio(WarningsAsError),PassRatio(WarningsNotAsError),CompletedPassRatio(WarningsAsError),ComplatePassRatio,SkipRatio\n";
    }
}

/**
 * Base model for table entries.
 */
export abstract class OutcomeTableEntry {

    // #region -- Public Members --

    /**
     * The change number for the table entry.
     */
    change?: number;

    /**
     * The change date for the table entry.
     */
    changeDate: Date | string;

    /**
     * The owning job's id.
     */
    jobId: string;

    /**
     * The owning job's name.
     */
    jobName: string;

    /**
     * The stream id that this table entry belongs to.
     */
    streamId: string;

    /**
     * The owning job's creation time.
     */
    jobCreateTime: string;

    /**
     * The issue data for the tabel entry.
     */
    issuesData: IssueData[] = [];

    /**
     * Duplicate entries for the table entry.
     */
    duplicateEntries: OutcomeTableEntry[] = [];

    // #endregion -- Public Members --

    // #region -- Abstract Members --

    /**
     * Gets the id of the table entry.
     */
    abstract get id(): string;

    /**
     * Gets the start time of the table entry.
     */
    abstract get startTime(): Date | string | undefined;

    /**
     * Gets the name of the table entry.
     */
    abstract get name(): string;

    /**
     * The state of the table entry.
     */
    abstract get state(): JobStepState | LabelState;

    /**
     * The outcome of the table entry.
     */
    abstract get outcome(): JobStepOutcome | LabelOutcome;

    /**
     * Gets the CSV representation of the the table entry. Property ordering matches @see getCSVHeader .
     * @param relatedJobContext The related job's context object.
     * @returns CSV representation of the table entry.
     */
    abstract toCSVRepresntation(relatedJobContext: JobResponseData | undefined): string;

    /**
     * Gets the CSV- property names of the step outcome entry. Property ordering matches @see toCSVRepresntation .
     * @returns The comma separated properties serialization.
     */
    abstract getCSVHeader(): string;

    // #endregion -- Abstract Members --

    // #region -- Constructor --

    /**
     * Constructor.
     * @param jobId The owning job's Id.
     * @param jobName The owning job's name.
     * @param streamId The owning streams's Id.
     * @param jobCreateTime The owning job's create time.
     * @param change The change associated with the table entry.
     * @param changeDate The change submit date associated with the table entry.
     */
    constructor(jobId: string, jobName: string, streamId: string, jobCreateTime: string, change?: number, changeDate?: Date | string) {
        if (change !== undefined) {
            this.change = change;
        }

        if (changeDate !== undefined) {
            this.changeDate = changeDate;
        }
        else {
            this.changeDate = jobCreateTime;
        }

        this.jobId = jobId;
        this.jobName = jobName;
        this.streamId = streamId;
        this.jobCreateTime = jobCreateTime
    }

    // #endregion -- Constructor --
}

/**
 * Model for a label table entry.
 */
export class LabelOutcomeTableEntry extends OutcomeTableEntry {
    // #region -- Private Members --

    private labelResponse: GetLabelStateResponse;
    private labelIdx: number;
    private synthesizedStepState?: LabelState | JobStepState;

    // #endregion -- Private Members --

    // #region -- Constructor --

    /**
     * Constructor.
     * @param labelResponse The label response object for the table entry.
     * @param labelIdx The label index for the job.
     * @param jobId The owning job's Id.
     * @param jobName The owning job's name.
     * @param streamId The owning streams's Id.
     * @param jobCreateTime The owning job's create time.
     * @param change The change associated with the table entry.
     * @param changeDate The change submit date associated with the table entry.
     * @param synthesizedStepState The synthesized step state if applicable.
     */
    constructor(labelResponse: GetLabelStateResponse, labelIdx: number, jobId: string, jobName: string, streamId: string, jobCreateTime: string, change?: number, changeDate?: Date | string, synthesizedStepState?: LabelState | JobStepState) {
        super(jobId, jobName, streamId, jobCreateTime, change, changeDate);

        this.labelIdx = labelIdx;
        this.labelResponse = labelResponse;
        this.synthesizedStepState = synthesizedStepState;
    }

    // #endregion -- Constructor --

    // #region -- Abstract API --

    /**
     * @inheritdoc
     */
    get id(): string {
        return this.labelIdx.toString();
    }

    /**
     * @inheritdoc
     */
    get name(): string {
        if (this.labelResponse?.dashboardName !== undefined) {
            return `${this.labelResponse?.dashboardCategory}-${this.labelResponse?.dashboardName}`;
        }
        return "";
    }

    /**
     * @inheritdoc
     * 
     * Note: 
     * Labels have a smaller stable state domain than Steps. Labels don't have a concept of skip / cancelled (due to complexities & edge cases), so we attempt to use a synthesized step state in such a case.
     * This is inferred from other metadata associated with the job, and is provided at construction of the lable outcome entry.
     * 
     * Note:
     * An explicit remapping occurs from @see LabelState.Complete to @see JobStepState.Completed to homogenize frontend experience.
     */
    get state(): LabelState | JobStepState {
        if (this.labelResponse.state === LabelState.Unspecified) {
            return this.synthesizedStepState ?? LabelState.Unspecified;
        }
        if (this.labelResponse.state !== undefined && this.labelResponse.state !== null) {
            if (this.labelResponse.state === LabelState.Complete) {
                return JobStepState.Completed;
            }

            return this.labelResponse.state;
        }
        return LabelState.Unspecified;
    }

    /**
     * @inheritdoc
     */
    get outcome(): LabelOutcome {
        return this.labelResponse.outcome ?? LabelOutcome.Unspecified;
    }

    /**
     * @inheritdoc
     */
    get startTime(): Date | string | undefined {
        return undefined;
    }

    /**
     * @inheritdoc
     */
    toCSVRepresntation(relatedJobContext: JobResponseData | undefined): string {
        let baseUrl = window.location.origin;
        let jobUrl = `${baseUrl}/job/${this.jobId}`;
        let labelUrl = `${baseUrl}/job/${this.jobId}?label=${this.labelIdx}`;
        return `${this.change},${this.streamId},${this.jobName},${this.jobId},${this.labelResponse.dashboardCategory},${this.labelResponse.dashboardName},${this.id},${this.state},${this.outcome},${this.jobCreateTime},${relatedJobContext?.summarizedOutcome},${relatedJobContext?.state},${relatedJobContext?.isManualJobInvocation},${jobUrl},${labelUrl}\n`;
    }

    /**
     * @inheritdoc
     */
    getCSVHeader(): string {
        return "Change,StreamId,JobName,JobId,LabelCategory,LabelName,LabelIdx,StepState,StepOutcome,JobStart,JobSummaryOutcome,JobState,JobManualInvocation,JobUrl,LabelUrl\n";
    }

    // #endregion -- Abstract API --
}

/**
 * Ease of use type to support common property access for @see IJobStepAdapter .
 */
export type JobStepResponseUnion = GetJobStepRefResponse | GetStepResponse;

/**
 * Interface that describes the adapter class for the stepId and stepName properties.
 * This is necessary since the @see GetJobStepRefResponse and @see GetStepResponse have disimilar interfaces into the underlying step structure.
 */
export interface IJobStepAdapter {
    /**
     * The step's id.
     */
    stepId: string;

    /**
     * The step's name.
     */
    stepName: string;

    /**
     * Gets the base union type for ease of access for common properties.
     */
    getBase(): JobStepResponseUnion;
}

/**
 * Wrapper implementation for @see GetJobStepRefResponse .
 */
export class GetJobStepRefResponseWrapper implements IJobStepAdapter {
    constructor(private base: GetJobStepRefResponse) { }

    /**
     * @inheritdoc
     */
    get stepId() {
        return this.base.stepId;
    }

    /**
     * @inheritdoc
     */
    get stepName() {
        return this.base.stepName!;
    }

    /**
     * @inheritdoc
     */
    getBase(): GetJobStepRefResponse {
        return this.base;
    }
}

/**
 * Wrapper implementation for @see GetStepResponse .
 */
export class GetStepResponseWrapper implements IJobStepAdapter {
    constructor(private base: GetStepResponse) { }

    /**
     * @inheritdoc
     */
    get stepId() {
        return this.base.id;
    }

    /**
     * @inheritdoc
     */
    get stepName() {
        return this.base.name;
    }

    /**
     * @inheritdoc
     */
    getBase(): GetStepResponse {
        return this.base;
    }
}

/**
 * Model for a table entry.
 */
export class StepOutcomeTableEntry extends OutcomeTableEntry {
    // #region -- Private Members --

    private synthesizedStepOutcome: JobStepOutcome | undefined;
    private stepResponse: IJobStepAdapter;
    private stepDurationMs: number;

    // #endregion -- Private Members --

    // #region -- Constructor --

    /**
     * Constructor.
     * @param stepResponse The step response object for the table entry.
     * @param jobId The owning job's Id.
     * @param jobName The owning job's name.
     * @param streamId The owning streams's Id.
     * @param jobCreateTime The owning job's create time.
     * @param change The change associated with the table entry.
     * @param changeDate The change submit date associated with the table entry.
     * @param batch The batch associated with the step.
     */
    constructor(stepResponse: IJobStepAdapter, jobId: string, jobName: string, streamId: string, jobCreateTime: string, change?: number, changeDate?: Date | string, batch?: GetBatchResponse) {
        super(jobId, jobName, streamId, jobCreateTime, change, changeDate);
        this.synthesizedStepOutcome = batch !== undefined && batch.error !== JobStepBatchError.None ? JobStepOutcome.Failure : undefined;
        this.stepResponse = stepResponse;
        this.stepDurationMs = computeDuration(stepResponse.getBase().startTime, stepResponse.getBase().finishTime)
    }

    // #endregion -- Constructor --

    // #region -- Abstract API --

    /**
     * @inheritdoc
     */
    get id(): string {
        return this.stepResponse.stepId;
    }

    /**
     * @inheritdoc
     */
    get name(): string {
        return this.stepResponse?.stepName ?? "";
    }

    /**
     * @inheritdoc
     */
    get state(): JobStepState {
        return this.stepResponse.getBase().state!;
    }

    /**
     * @inheritdoc
     */
    get outcome(): JobStepOutcome {
        return this.synthesizedStepOutcome !== undefined ? this.synthesizedStepOutcome : this.stepResponse.getBase().outcome!;
    }

    /**
     * @inheritdoc
     */
    get startTime() {
        return this.stepResponse.getBase().startTime;
    }

    /**
     * @inheritdoc
     */
    toCSVRepresntation(relatedJobContext: JobResponseData | undefined): string {
        let baseUrl = window.location.origin;
        let jobUrl = `${baseUrl}/job/${this.jobId}`;
        let stepUrl = `${baseUrl}/job/${this.jobId}?step=${this.stepResponse.stepId}`;
        return `${this.change},${this.streamId},${this.jobName},${this.jobId},${this.stepResponse.stepName!},${this.stepResponse.stepId},${this.stepResponse.getBase().state},${this.stepResponse.getBase().outcome},${this.stepDurationMs / 1000},${this.jobCreateTime},${this.stepResponse.getBase().startTime},${relatedJobContext?.summarizedOutcome},${relatedJobContext?.state},${relatedJobContext?.isManualJobInvocation},${jobUrl},${stepUrl}\n`;
    }

    /**
     * @inheritdoc
     */
    getCSVHeader(): string {
        return "Change,StreamId,JobName,JobId,StepName,StepId,StepState,StepOutcome,Duration(Sec),JobStart,StepStart,JobSummaryOutcome,JobState,JobManualInvocation,JobUrl,StepUrl\n";
    }

    // #endregion -- Abstract API --

    // #region -- Public API --

    /**
     * Gets the duration of the underyling @see StepReponseData in string representation.
     * @returns The string representation of the step duration.
     */
    getDurationString(): string {
        let durationStr: string = this.stepResponse.getBase().state === SkippedStr ? NOT_APPLICABLE_LABEL : (this.stepResponse.getBase().state === WaitingStr || this.stepResponse.getBase().state === ReadyStr ? "Not Started" : "Ongoing");

        if (this.stepDurationMs != DURATION_NOT_SET) {
            durationStr = formatDurationFromMs(this.stepDurationMs);
        }

        return durationStr;
    }

    // #endregion -- Public API --
}

// #endregion -- Table Entry Types --

// #region -- Table Header Types --

/**
 * Discriminated union which represents a tabel entry. This can be a @see ChangeHeader (the fundamental change header in the table) or a @see SummaryHeader (the summarization of an entire row).
 */
export type ColumnHeader = ChangeHeader | SummaryHeader | DateHeader;

/**
 * Type predicate to test whether this is a @see SummaryHeader .
 * @param header the header to apply the predicate to.
 * @returns True if the header was a @see SummaryHeader ; false otherwise.
 */
export function isSummaryHeader(header: ColumnHeader): header is SummaryHeader {
    return header.type === "summary";
}

/**
 * Type predicate to test whether this is a @see DateHeader .
 * @param header the header to apply the predicate to.
 * @returns True if the header was a @see DateHeader ; false otherwise.
 */
export function isDateHeader(header: ColumnHeader): header is DateHeader {
    return header.type === "date";
}

/**
 * Type predicate to test whether this is a @see ChangeHeader .
 * @param header the header to apply the predicate to.
 * @returns True if the header was a @see ChangeHeader ; false otherwise.
 */
export function isChangeHeader(header: ColumnHeader): header is ChangeHeader {
    return header.type === "change";
}

/**
 * Data struct that represents a summary header.
 */
export type SummaryHeader = {
    type: "summary";
    name: string;
}

/**
 * Data struct that represents a change header. This header contains associated data related to rendering & ordering the header.
 */
export type ChangeHeader = {
    type: "change";
    change: number;
    date?: string;
}

/**
 * Data struct that represents a date header. This header contains associated data related to rendering & ordering the header.
 */
export type DateHeader = {
    type: "date";
    date: string;
}

/**
 * Data struct that represents a step name header. This header contains associated data related to rendering & ordering the header.
 */
export type StepNameHeader = {
    stepName: string;
    streamId: string;
    isLabel: boolean;
}

// #region -- Table Header Types --

/**
 * Utility type that contains metadata about the stream.
 */
export type StreamMetadata = {
    stepCount: number;
    labelCount: number;
};

/**
 * Model for the StepOutcomeTable.
 */
export class StepOutcomeTable {
    // #region -- Public Members --

    @observable sortField: "name" | "summary" | "start" | "none" = "none";
    @observable sortDirectionAsc: boolean = true;

    dataCount: number = 0;

    /**
     * Sets whether @see getDateAnchoredChanges is supported for the given data.
     */
    supportsDateAnchoredChanges: boolean = true;

    /**
     * Change column headers.
     */
    changeColLookup: Map<number, number> = new Map<number, number>();
    changeColHeaders: ChangeHeader[] = [];
    changeOrderAscend: boolean = false;

    /**
     * Step name row headers.
     */
    stepRowLookup: Map<string, number> = new Map<string, number>();
    stepRowNumberLookup: Map<number, string> = new Map<number, string>();
    private stepNameRowHeaders: StepNameHeader[] = [];

    /**
     * SummaryLookup maps a normalized step name to a summary entry.
     */
    summaryLookup: Map<string, SummaryTableEntry> = new Map<string, SummaryTableEntry>();

    tableStreamMetadata: Map<string, StreamMetadata> = new Map<string, StreamMetadata>();

    @observable
    viewModelOptions: StepOutcomeViewOptions;

    /**
     * Table data structure
     */
    tableEntries: (OutcomeTableEntry | null)[][] = [];

    // #endregion --  Public Members --

    constructor() {
        makeObservable(this);
    }

    // #region -- Private API --

    private ensureRow(source: (OutcomeTableEntry | null)[], targetSize: number) {
        while (targetSize > source.length) {
            source.push(null);
        }
    }

    private annotateStreamRow(streamId: string, isLabelOutcome: boolean) {
        const metadata = this.tableStreamMetadata.get(streamId) ?? { stepCount: 0, labelCount: 0 };

        if (isLabelOutcome) {
            metadata.labelCount++;
        } else {
            metadata.stepCount++;
        }

        this.tableStreamMetadata.set(streamId, metadata);
    }

    /**
     * Gets the the date anchored changes. 
     * @returns * Generator of all of the @see ChangeHeader items separated by @see DateHeader that represent dates. 
     */
    private *getDateAnchoredChanges(): Generator<ChangeHeader | DateHeader> {
        if (!this.supportsDateAnchoredChanges) {
            for (const header of this.changeColHeaders) {
                yield header;
            }

            return;
        }

        let prevDay: string | null = null;
        for (let i: number = 0; i < this.changeColHeaders.length; ++i) {

            // If the date associated with the changelist header has changed since the last, we are on a new day. Set that as the colimn.
            let changeHeader = this.changeColHeaders[i];
            if (changeHeader.date !== undefined) {
                const headerDate = new Date(changeHeader.date!);

                // Extract the day portion as YYYY-MM-DD
                const dayStr = `${headerDate.getUTCFullYear()}-${headerDate.getUTCMonth() + 1}-${headerDate.getUTCDate()}`;

                if (prevDay !== dayStr) {
                    prevDay = dayStr;
                    let dateAnchorChangelistHeader: DateHeader = {
                        type: "date",
                        date: dayStr
                    }
                    yield dateAnchorChangelistHeader;
                }
                yield changeHeader;
            }
        }
    }

    // #endregion -- Private API --

    // #region -- Public API --

    /**
     * Gets the current sort key.
     * @returns The current sort key of the table.
     */
    @computed
    get sortingKey() {
        return `${this.sortField}:${this.sortDirectionAsc}`;
    }

    /**
     * Resets the table data. This will remove all data from the container.
     */
    reset(): void {
        this.dataCount = 0;
        this.tableEntries = [];
        this.stepNameRowHeaders = [];
        this.stepRowLookup.clear();
        this.stepRowNumberLookup.clear();
        this.summaryLookup.clear();
        this.changeColHeaders = [];
        this.changeColLookup.clear();
        this.tableStreamMetadata.clear();
    }

    /**
     * Adds a new entry to the table data.
     * @param tableEntry The table entry to add to the data table.
     * @returns If tableEntry parameter is invalid, returns.
     */
    addEntry(tableEntry: OutcomeTableEntry): void {
        if (tableEntry.change === undefined) {
            return;
        }

        const normalizedStepName = encodeStepName(tableEntry);

        // Resolve or create row index
        let insertionRow = this.stepRowLookup.get(normalizedStepName);
        if (insertionRow === undefined) {

            this.annotateStreamRow(tableEntry.streamId, isLabelOutcome(tableEntry));
            insertionRow = this.stepNameRowHeaders.length;
            this.stepRowLookup.set(normalizedStepName, insertionRow); // cache the normalized step name -> row number
            this.stepRowNumberLookup.set(insertionRow, normalizedStepName); // cache the row number -> normalized step name
            this.stepNameRowHeaders.push({ streamId: tableEntry.streamId, stepName: tableEntry.name, isLabel: isLabelOutcome(tableEntry) });
        }

        let insertionCol = this.changeColLookup.get(tableEntry.change);
        if (insertionCol === undefined) {
            insertionCol = this.changeColHeaders.length;
            this.changeColLookup.set(tableEntry.change, insertionCol);
            let changeDate = tableEntry.changeDate;
            this.changeColHeaders.push({ type: "change", change: tableEntry.change, date: toDate(changeDate)?.toISOString() });

            for (const row of this.tableEntries) {
                if (row) {
                    this.ensureRow(row, this.changeColHeaders.length);
                }
            }
        }

        // Ensure the row exists and is padded
        if (this.tableEntries[insertionRow] === undefined) {
            this.tableEntries[insertionRow] = Array(this.changeColHeaders.length).fill(null);
        } else {
            this.ensureRow(this.tableEntries[insertionRow], this.changeColHeaders.length);
        }

        const existingEntry = this.tableEntries[insertionRow][insertionCol];

        // If we have an existing entry, replace the existing entry with the newly added one if it is more recent.
        if (existingEntry !== null) {
            // Attempt to sort by step sort time first. This fails under two scenarios:
            // 1. The step has Not Started (so the startTime will be null and thus, a delay in this taking top position)
            // 2. The step has been skipped (and we must fall back to job creation time as a sorting mechanism - startTime will never be !null).
            let startA: number = tableEntry.startTime ? new Date(tableEntry.startTime).getTime() : 0;
            let startB: number = existingEntry.startTime ? new Date(existingEntry.startTime).getTime() : 0;

            // If either is invalid (most likely due to the Not Started case, but secondarily if it's skipped/aborted), fall back to considering the job created time.
            // Note: this will not be captured on the edge case of step-retry only. The best recommendation for this edge case is for users is to review the job details which they will have access
            // to from the modal.
            if (startA === 0 || startB === 0) {
                startA = new Date(tableEntry.jobCreateTime).getTime();
                startB = new Date(existingEntry.jobCreateTime).getTime();
            }

            const isNewer = startA > startB;

            if (isNewer) {
                tableEntry.duplicateEntries.push(existingEntry, ...existingEntry.duplicateEntries);
                existingEntry.duplicateEntries = []; // we can zero this out; we've swapped to the table entry being the primary one.
                this.tableEntries[insertionRow][insertionCol] = tableEntry;
            } else {
                existingEntry.duplicateEntries.push(tableEntry);
            }

            return;
        }

        this.dataCount++;
        this.tableEntries[insertionRow][insertionCol] = tableEntry;
    }

    /**
     * Summarizes all of the data currently stored within the step outcome table.
     * A @see SummaryTableEntry is the step completion (across failure & success states) for all rows in the table.
     */
    summarize() {
        for (let rowIndex = 0; rowIndex < this.tableEntries.length; rowIndex++) {
            const row = this.tableEntries[rowIndex];
            if (!this.stepRowNumberLookup.has(rowIndex)) {
                continue;
            }

            let summaryTableEntry: SummaryTableEntry = new SummaryTableEntry();
            let normalizedTableEntryName: string = this.stepRowNumberLookup.get(rowIndex)!;
            this.summaryLookup.set(normalizedTableEntryName, summaryTableEntry);

            for (let colIndex = 0; colIndex < row.length; colIndex++) {
                const cell = row[colIndex];

                if (cell === null) {
                    continue;
                }

                if (!this.viewModelOptions.validStates.has(cell.state)) {
                    continue;
                }

                summaryTableEntry.steps++;

                if (cell.state === WaitingStr || cell.state === RunningStr) {
                    summaryTableEntry.stepsPending++;
                    continue;
                }

                summaryTableEntry.stepsIncluded++;

                if (cell.state == SkippedStr) {
                    summaryTableEntry.stepsSkipped++;
                }

                if (cell.outcome == WarningStr) {
                    summaryTableEntry.stepsWarning++;
                }
                else if (cell.outcome == FailureStr || cell.state == AbortedStr) {
                    summaryTableEntry.stepsFail++;
                }
                else if (cell.outcome == SuccessStr) {
                    summaryTableEntry.stepsPass++;
                }
            }
        }
    }

    /**
     * Retrieves a step row summary for a provided row.
     * @param row The normalized row name, or number.
     * @returns The step summary for that row if it exists, undefined otherwise.
     */
    getStepRowSummary(row: string | number): TableEntry {
        let rowString: string | undefined = undefined;
        if (typeof row === 'number') {
            rowString = this.stepRowNumberLookup.get(row);
        } else {
            rowString = row;
        }

        let summaryTableEntry: SummaryTableEntry | undefined = this.summaryLookup.has(rowString!) ? this.summaryLookup.get(rowString!) : undefined;

        return summaryTableEntry;
    }

    /**
     * Gets all of the Table Entries in a row.
     * @param * Generator of table entries.
     */
    * getStepOutputTableEntries(row: string | number): Generator<TableEntry> {
        yield* this.getStepOutputRow(row);
        yield this.getStepRowSummary(row);
    }

    /**
     * Gets the Step Outcome Table Entries for a given step name.
     * @param rowName The step name to obtain.
     * @returns * Generator of TableEntries.
     */
    getStepOutputRow(rowName: string): Generator<OutcomeTableEntry>;
    getStepOutputRow(rowIndex: number): Generator<OutcomeTableEntry>;
    getStepOutputRow(row: string | number): Generator<OutcomeTableEntry>;
    *getStepOutputRow(row: string | number): Generator<OutcomeTableEntry | null | undefined> {
        let rowIdx: number | undefined;

        if (typeof row === 'string') {
            rowIdx = this.stepRowLookup.get(row);
        } else {
            rowIdx = row;
        }

        if (rowIdx === undefined || rowIdx < 0 || rowIdx >= this.tableEntries.length) {
            return;
        }

        const rowEntries = this.tableEntries[rowIdx];

        for (const entry of this.getDateAnchoredChanges()) {
            if (isDateHeader(entry)) {
                yield undefined;
            }
            else {
                const colIdx = this.changeColLookup.get(entry.change);
                const tableEntry = colIdx !== undefined ? rowEntries[colIdx] : null;

                yield tableEntry !== null && this.viewModelOptions.validStates.has(tableEntry.state) ? tableEntry : undefined;
            }
        }
    }

    /**
     * Gets the Table Entries for the given change.
     * @param changeNumber The change number.
     * @returns * Generator of all Table Entries for the specific change.
     */
    *getChangeColumn(changeNumber: number): Generator<OutcomeTableEntry> {

        let colIdx: (number | undefined) = this.changeColLookup.get(changeNumber);
        if (colIdx === undefined) {
            return;
        }

        for (const entry of this.getStepRowHeaders()) {
            let rowIdx: (number | undefined) = this.stepRowLookup.get(encodeStepNameFromStepNameHeader(entry));
            if (rowIdx === undefined) {
                continue;
            }

            let tableEntry: (OutcomeTableEntry | null) = this.tableEntries[rowIdx][colIdx];
            if (tableEntry !== null && this.viewModelOptions.validStates.has(tableEntry.state)) {
                yield tableEntry;
            }
        }
    }

    /**
     * Sets the table data's change (column) order.
     * @param ascending True for ascending order; false for descending order.
     */
    orderTableDataByChange(ascending: boolean): void {
        this.changeColHeaders.sort((a, b) => ascending ? a.change - b.change : b.change - a.change);
        this.changeOrderAscend = ascending;
    }

    /**
     * Requests a change of ordering type.
     * @param requestedOrdering The requested ordering field type.
     * @param toggleIfSameOrdering If true, will togggle the sort direction if the sort field hasn't changed types.
     */
    changeOrderingBy(requestedOrdering: "name" | "summary" | "start" | "none", toggleIfSameOrdering: boolean = true): void {
        let newSortDirectionAsc: boolean = toggleIfSameOrdering && this.sortField === requestedOrdering ? !this.sortDirectionAsc : this.sortDirectionAsc;
        switch (requestedOrdering) {
            case "name":
                this.orderTableDataByStepName(newSortDirectionAsc);
                break;
            case "summary":
                this.orderTableDataBySummaryPassRatio(newSortDirectionAsc);
                break;
            case "start":
                this.orderTableDataByStepStart(newSortDirectionAsc);
                break;
            case "none":
                this.sortField = "none";
                break;
        }
    }

    /**
     * Orders table data based on step "start".
     * @param ascending True for ascending order; false for descending order.
     */
    @action
    orderTableDataByStepStart(ascending: boolean): void {
        this.sortField = "start";
        this.sortDirectionAsc = ascending;

        this.stepNameRowHeaders.sort((a, b) => {
            let aInsertionRow = this.stepRowLookup.get(encodeStepNameFromStepNameHeader(a))!;
            let bInsertionRow = this.stepRowLookup.get(encodeStepNameFromStepNameHeader(b))!;
            if (aInsertionRow < bInsertionRow) return ascending ? -1 : 1;
            if (aInsertionRow > bInsertionRow) return ascending ? 1 : -1;
            return 0;
        });
    }

    /**
     * Orders table data based on step name.
     * @param ascending True for ascending order; false for descending order.
     */
    @action
    orderTableDataByStepName(ascending: boolean): void {
        this.sortField = "name";
        this.sortDirectionAsc = ascending;

        this.stepNameRowHeaders.sort((a, b) => {
            if (a.stepName < b.stepName) return ascending ? -1 : 1;
            if (a.stepName > b.stepName) return ascending ? 1 : -1;
            return 0;
        });
    }

    /**
     * Orders table by the summary pass ratio.
     * @param ascending True for ascending order; false for descending order.
     */
    @action
    orderTableDataBySummaryPassRatio(ascending: boolean): void {
        this.sortField = "summary";
        this.sortDirectionAsc = ascending;
        this.stepNameRowHeaders.sort((a, b) => {
            let aSummary = this.summaryLookup.get(encodeStepNameFromStepNameHeader(a));
            let bSummary = this.summaryLookup.get(encodeStepNameFromStepNameHeader(b));

            if (aSummary?.computePassRatio(false).ratio! < bSummary?.computePassRatio(false).ratio!) return ascending ? -1 : 1;
            if (aSummary?.computePassRatio(false).ratio! > bSummary?.computePassRatio(false).ratio!) return ascending ? 1 : -1;
            return 0;
        });
    }

    /**
     * Gets all of the column headers for the table.
     * @returns * Generator of all the column headers.
     */
    *getColumnHeaders(): Generator<ColumnHeader> {
        yield* this.getDateAnchoredChanges();
        yield {
            type: "summary",
            name: "Summary"
        }
    }

    /**
     * Gets all of the step row headers for the table.
     * @returns * Generator of all the step row headers. If @see this.supportsLabelEntries , these items will also be returned.
     */
    *getStepRowHeaders(): Generator<StepNameHeader> {
        for (let idx: number = 0; idx < this.stepNameRowHeaders.length; ++idx) {
            yield this.stepNameRowHeaders[idx];
        }
    }

    /**
     * Generates the csv string representation of the step outcome table, in row order.
     * @param contextHandler The datahandler used to obtain job response data.
     * @returns A string in CSV format that represents the underlying step outcome table, row order.
     */
    toRowOrderedCSV<U extends OutcomeTableEntry>(contextHandler: StepOutcomeDataHandler, guard: (entry: OutcomeTableEntry) => entry is U): string {
        let resultString: string | undefined;
        for (let rowIndex = 0; rowIndex < this.tableEntries.length; rowIndex++) {
            const row = this.tableEntries[rowIndex];
            for (let i: number = 0; i < row.length; ++i) {
                if (!row[i]) {
                    continue;
                }
                let tableEntry: OutcomeTableEntry = row[i]!;

                if (!guard(tableEntry)) {
                    continue;
                }

                let castedEntry: U = tableEntry as U;

                if (castedEntry === undefined) {
                    continue;
                }

                if (!this.viewModelOptions.validStates.has(castedEntry.state)) {
                    continue;
                }

                if (resultString == undefined) {
                    resultString = castedEntry.getCSVHeader();
                }

                let jobResponseData: JobResponseData | undefined = contextHandler.getJobResponseData(tableEntry.jobId);
                resultString += castedEntry.toCSVRepresntation(jobResponseData);
            }
        }

        return resultString!;
    }

    /**
     * Generates the csv string representation of the step outcome table, in column order.
     * @param contextHandler The datahandler used to obtain job response data.
     * @returns A string in CSV format that represents the underlying step outcome table, column order.
     */
    toColOrderedCSV<U extends OutcomeTableEntry>(contextHandler: StepOutcomeDataHandler, guard: (entry: OutcomeTableEntry) => entry is U): string {
        let resultString: string | undefined;
        for (let changeIdx = 0; changeIdx < this.changeColHeaders.length; changeIdx++) {
            const changeEntries = [...this.getChangeColumn(this.changeColHeaders[changeIdx].change)];
            for (let i: number = 0; i < changeEntries.length; ++i) {

                if (!changeEntries[i]) {
                    continue;
                }

                let tableEntry: OutcomeTableEntry = changeEntries[i]!;

                if (!guard(tableEntry)) {
                    continue;
                }

                let castedEntry: OutcomeTableEntry = tableEntry as OutcomeTableEntry;

                if (castedEntry === undefined) {
                    continue;
                }

                if (!this.viewModelOptions.validStates.has(castedEntry.state)) {
                    continue;
                }

                if (resultString == undefined) {
                    resultString = castedEntry.getCSVHeader();
                }

                let jobResponseData: JobResponseData | undefined = contextHandler.getJobResponseData(tableEntry.jobId);
                resultString += castedEntry.toCSVRepresntation(jobResponseData);
            }
        }

        return resultString!;
    }

    /**
     * Generates the csv string representation of the step outcome table's summary column.
     * @returns A string in CSV format that represents the underyling step outcome table's summary column.
     */
    summaryToRowOrderedCSV() {
        let resultString: string | undefined;

        for (let [stepName, summary] of this.summaryLookup.entries()) {
            if (resultString === undefined) {
                resultString = SummaryTableEntry.getCSVHeader();
            }
            resultString += `${stepName},` + summary.toCSVRepresentation();
        }

        return resultString!;
    }

    /**
     * Sets the view model options to use when interacting with the StepOutcomeTable.
     * @param viewModelOptions The view model options to set.
     */
    @action
    setViewModelOptions(viewModelOptions: StepOutcomeViewOptions) {
        this.viewModelOptions = viewModelOptions;
        this.supportsDateAnchoredChanges = viewModelOptions.includeDateAnchors;
    }

    /**
     * The number of step results stored in the table.
     */
    get StepResultCount(): number {
        let stepCount: number = 0;

        for (const [_streamId, metadata] of this.tableStreamMetadata.entries()) {
            stepCount += metadata.stepCount;
        }

        return stepCount;
    }

    /**
     * The number of label results stored in the table.
     */
    get LabelResultCount(): number {
        let labelCount: number = 0;

        for (const [_streamId, metadata] of this.tableStreamMetadata.entries()) {
            labelCount += metadata.labelCount;
        }

        return labelCount;
    }

    // #endregion -- Public API --
}

// #endregion -- Data Types  --