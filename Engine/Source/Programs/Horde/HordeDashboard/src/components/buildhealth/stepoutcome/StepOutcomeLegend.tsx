// #region -- Legend Component --

import { Stack, Label, TooltipHost, DirectionalHint } from "@fluentui/react";
import { JobStepState, JobStepOutcome } from "horde/backend/Api";
import { JobResponseData, StepOutcomeTableEntry } from "./StepOutcomeDataTypes";
import { getCellStyle, getCellAccents, getCellClasses } from "./StepOutcomeSharedUIComponents";

export const StepOutcomeLegend: React.FC = () => {
    const fakeSuccessEntry = {
        state: JobStepState.Completed,
        outcome: JobStepOutcome.Success
    } as unknown as StepOutcomeTableEntry;

    const fakeSuccessMultiRunEntry = {
        state: JobStepState.Completed,
        outcome: JobStepOutcome.Success,
        duplicateEntries: ["One", "Two", "Three"]
    } as unknown as StepOutcomeTableEntry;

    const fakeFailureEntry = {
        state: JobStepState.Completed,
        outcome: JobStepOutcome.Failure
    } as unknown as StepOutcomeTableEntry;

    const fakeWarningEntry = {
        state: JobStepState.Completed,
        outcome: JobStepOutcome.Warnings
    } as unknown as StepOutcomeTableEntry;

    const fakeRunningEntry = {
        state: JobStepState.Running,
        outcome: JobStepOutcome.Unspecified
    } as unknown as StepOutcomeTableEntry;

    const fakeRunningSummaryData = {
        summarizedOutcome: JobStepOutcome.Unspecified
    } as Partial<JobResponseData> as JobResponseData | undefined;

    const fakeSkippedEntry = {
        state: JobStepState.Skipped,
        outcome: JobStepOutcome.Failure
    } as unknown as StepOutcomeTableEntry;

    const fakeSkippedSummaryData = {
        summarizedOutcome: JobStepOutcome.Failure
    } as Partial<JobResponseData> as JobResponseData | undefined;

    const fakeWaitingEntry = {
        state: JobStepState.Waiting,
        outcome: JobStepOutcome.Success
    } as unknown as StepOutcomeTableEntry;

    const fakeReadyEntry = {
        state: JobStepState.Ready,
        outcome: JobStepOutcome.Success
    } as unknown as StepOutcomeTableEntry;

    const fakeReadyWatingummaryData = {
        summarizedOutcome: JobStepOutcome.Unspecified
    } as Partial<JobResponseData> as JobResponseData | undefined;

    const fakeAbortedEntry = {
        state: JobStepState.Aborted,
        outcome: JobStepOutcome.Failure
    } as unknown as StepOutcomeTableEntry;

    const fakeAbortedFailureSummaryData = {
        summarizedOutcome: JobStepOutcome.Failure
    } as Partial<JobResponseData> as JobResponseData | undefined;

    const fakeAbortedWarningSummaryData = {
        summarizedOutcome: JobStepOutcome.Warnings
    } as Partial<JobResponseData> as JobResponseData | undefined;

    const fakeAbortedSuccessSummaryData = {
        summarizedOutcome: JobStepOutcome.Success
    } as Partial<JobResponseData> as JobResponseData | undefined;

    return (
        <Stack>
            <Label>Legend</Label>
            <Stack>
                <table>
                    <tbody>
                        <tr>
                            <td>
                                <TooltipHost content="When a step has run to completion." calloutProps={{ directionalHint: DirectionalHint.topCenter }}>
                                    <div style={{ width: "100%", height: "100%" }}>Complete</div>
                                </TooltipHost>
                            </td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeFailureEntry).bgColor,
                            }}><TooltipHost content="The step has completed with failures." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeWarningEntry).bgColor,
                            }}><TooltipHost content="The step has completed with warnings." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeSuccessEntry).bgColor,
                            }}><TooltipHost content="The step has completed with success." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                        </tr>
                        <tr>
                            <td>
                                <TooltipHost content="When a step cannot execute (dependency failed), is currently executing, or hasn't yet executed." calloutProps={{ directionalHint: DirectionalHint.topCenter }}>
                                    <div style={{ width: "100%", height: "100%" }}>Incomplete</div>
                                </TooltipHost>
                            </td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeSkippedEntry).bgColor,
                                boxShadow: getCellAccents(fakeSkippedEntry, fakeSkippedSummaryData).boxShadow,
                            }}><TooltipHost content="The step has not run, and has been skipped due to a dependency failing." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeRunningEntry).bgColor,
                                boxShadow: getCellAccents(fakeRunningEntry, fakeRunningSummaryData).boxShadow,
                            }}><TooltipHost content="The step is currently running." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeReadyEntry).bgColor,
                                boxShadow: getCellAccents(fakeReadyEntry, fakeReadyWatingummaryData).boxShadow,
                            }}><TooltipHost content="The step has yet to run, but is in a ready state or waiting on agent." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                        </tr>
                        <tr>
                            <td>
                                <TooltipHost content="When a step as been cancelled." calloutProps={{ directionalHint: DirectionalHint.topCenter }}>
                                    <div style={{ width: "100%", height: "100%" }}>Cancelled</div>
                                </TooltipHost>
                            </td>
                            <td className={getCellClasses(fakeAbortedEntry, fakeAbortedFailureSummaryData).join(" ")}
                                style={{
                                    width: 20,
                                    height: 20,
                                    backgroundColor: getCellStyle(fakeAbortedEntry).bgColor,
                                    boxShadow: getCellAccents(fakeAbortedEntry, fakeAbortedFailureSummaryData).boxShadow,
                                }}><TooltipHost content="The step has been cancelled, and other steps in the job have failed." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td className={getCellClasses(fakeAbortedEntry, fakeAbortedWarningSummaryData).join(" ")}
                                style={{
                                    width: 20,
                                    height: 20,
                                    backgroundColor: getCellStyle(fakeAbortedEntry).bgColor,
                                    boxShadow: getCellAccents(fakeAbortedEntry, fakeAbortedWarningSummaryData).boxShadow,
                                }}><TooltipHost content="The step has been cancelled, and other steps in the job encountered warnings." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                            <td className={getCellClasses(fakeAbortedEntry, fakeAbortedSuccessSummaryData).join(" ")}
                                style={{
                                    width: 20,
                                    height: 20,
                                    backgroundColor: getCellStyle(fakeAbortedEntry).bgColor,
                                    boxShadow: getCellAccents(fakeAbortedEntry, fakeAbortedSuccessSummaryData).boxShadow,
                                }}><TooltipHost content="The step has been cancelled, but all other steps in the job completed with success." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                        </tr>
                        <tr>
                            <td>
                                <TooltipHost content="When a step & changelist have multiple runs." calloutProps={{ directionalHint: DirectionalHint.topCenter }}>
                                    <div style={{ width: "100%", height: "100%" }}>Multi-value</div>
                                </TooltipHost>
                            </td>
                            <td className={getCellClasses(fakeSuccessMultiRunEntry, fakeAbortedSuccessSummaryData).join(" ")}
                            style={{
                                width: 20,
                                height: 20,
                                backgroundColor: getCellStyle(fakeSuccessMultiRunEntry).bgColor,
                            }}><TooltipHost content="The step & change has multiple entires. Most recent result is always shown." ><div style={{ width: "100%", height: "100%" }} /></TooltipHost></td>
                        </tr>
                    </tbody>
                </table>
            </Stack>
        </Stack >)
}

// #endregion -- Legend Component --