// Copyright Epic Games, Inc. All Rights Reserved.

import { DirectionalHint, IconButton, mergeStyleSets, Spinner, SpinnerSize, TooltipHost } from "@fluentui/react";
import dashboard, { StatusColor } from "horde/backend/Dashboard";
import { observer } from "mobx-react-lite";
import { StepOutcomeDataHandler } from "./StepOutcomeDataHandler";
import { AbortedStr, FailureStr, OutcomeTableEntry, JobResponseData, SkippedStr, SuccessStr, WarningStr, CompletedStepStateStr, CompleteLabelStateStr } from "./StepOutcomeDataTypes";
import { isTerminalState } from "./StepOutcomeUtilities";

// #region -- Constants --

export const NOT_APPLICABLE_LABEL: string = "N/A";
export const ROW_HEADER_NAME: string = "stepName" as const;

// #endregion -- Constants --

// #region -- Styles --

/**
 * Step outcome table Header styles.
 */
export const headerStyles = mergeStyleSets({
    verticalText: {
        writingMode: "vertical-rl",
        transform: "rotate(180deg)",
        display: "inline-block",
    }
})

const sortStyles = mergeStyleSets({
    upArrow: {
        '::after': {
            content: '"▲"',
            position: 'absolute',
            bottom: 4,
            right: 4,
            fontSize: 12,
            lineHeight: 1,
        }
    },
    downArrow: {
        position: "relative",
        selectors: {
            '::after': {
                content: '"▼"',
                position: 'absolute',
                bottom: 4,
                right: 4,
                fontSize: 12,
                lineHeight: 1,
            }
        }
    },

    dash: {
        position: "relative",
        selectors: {
            '::after': {
                content: '"-"',
                position: 'absolute',
                bottom: 4,
                right: 4,
                fontSize: 12,
                lineHeight: 1,
            }
        },
    }
});

export const sharedBorderStyle = {
    bottomBorder: {
        borderBottom: `1px solid rgba(80, 80, 80, 1)`
    }
};

// Example: your function returns something like
// { redX: "red", greenX: "green", blueX: "blue" }
const colors: Record<string, string> = getColorRecords();

const baseX = {
    position: "relative",
    overflow: "visible",
    selectors: {
        "&::before": {
            content: "''",
            position: "absolute",
            left: "50%",
            top: "50%",
            width: "calc(80% * 1.414)",
            height: "4px",
            transform: "translate(-50%, -50%) rotate(45deg)",
            transformOrigin: "center",
            pointerEvents: "none",
        },
    },
};

export const stepOutcomeTableClasses = mergeStyleSets({
    root: {
        maxWidth: "min(85vw, 2400px)",
    },
    multiValue: {
        position: "relative",
        selectors: {
            "&::after": {
                content: "'.'",
                position: "absolute",
                top: -9,
                right: -2,
                color: colors["Ready"],
                fontWeight: "bold",
                fontSize: 40,
                lineHeight: 1,
                pointerEvents: "none",
                userSelect: "none",
            },
        },
    },
    ...Object.fromEntries(
        Object.entries(colors).map(([className, color]) => [
            className,
            {
                ...baseX,
                selectors: {
                    "&::before": {
                        ...baseX.selectors["&::before"],
                        background: color,
                    },
                },
            },
        ])
    ),
});

// #endregion -- Styles --

// #region -- Style Helpers --

/**
 * Gets the color records based off of the string representations of @see StatusColor - which maps to the status string.
 * @returns A record that maps sates to their corresponding colours. 
 */
export function getColorRecords(): Record<string, string> {
    const scolors = dashboard.getStatusColors();
    const colors: Record<string, string> = {
        "Success": scolors.get(StatusColor.Success)!,
        "Failure": scolors.get(StatusColor.Failure)!,
        "Warnings": scolors.get(StatusColor.Warnings)!,
        "Skipped": scolors.get(StatusColor.Skipped)!,
        "Running": scolors.get(StatusColor.Running)!,
        "Waiting": scolors.get(StatusColor.Waiting)!,
        "Ready": scolors.get(StatusColor.Ready)!,
        "Unspecified": scolors.get(StatusColor.Skipped)!,
    };

    return colors;
}

/**
 * Gets the set of conditional classes to remove from the cells.
 * @returns The list of all conditional classes to remove from the cell.
 */
export function resetCellClasses(): string[] {
    return Object.values(stepOutcomeTableClasses) as string[];
}

/**
 * Gets the classes for a step outcome table entry cell. These are used to further highlight steps that have not run to completion (Aborted OR Skipped), and are not running.
 * @param entry The step outcome table entry of which to base the classes on.
 * @param jobState The job response data which will be used to style applicable table entries.
 * @returns The classes to apply to the cell.
 */
export function getCellClasses(entry: OutcomeTableEntry | null, jobState: JobResponseData | undefined): string[] {
    const classes: string[] = [];

    if (!entry || jobState?.summarizedOutcome === undefined) {
        return classes;
    }

    if (entry.state === AbortedStr) {
        classes.push(stepOutcomeTableClasses[jobState?.summarizedOutcome]);
    }

    if (entry.duplicateEntries !== undefined && entry.duplicateEntries.length > 0) {
        classes.push(stepOutcomeTableClasses.multiValue);
    }

    return classes;
}

/**
 * Gets the classes for a step outcome table header cell. These are used to add styling to a sortable column.
 * @param sourceSortField The requesting cells sort field type.
 * @param sortField The current sort field type.
 * @param sortDirectionAsc The sort direction.
 * @returns Class string as it relates to the requestor.
 */
export function getCellSortClasses(sourceSortField: "name" | "summary" | "start", sortField: "name" | "summary" | "start" | "none", sortDirectionAsc: boolean): string {
    if (sourceSortField != sortField) {
        return sortStyles.dash;
    }

    return sortDirectionAsc ? sortStyles.upArrow : sortStyles.downArrow;
}


/**
 * Gets the accents for a step outcome table entry cell. These are used to highlight steps that are currently in flight, or have not run to completion (Aborted OR Skipped).
 * @param entry The step outcome table entry of which to base the accents on.
 * @param jobState The job response data which will be used to style applicable table entries.
 * @returns The accents to apply to the cell.
 */
export function getCellAccents(entry: OutcomeTableEntry | null, jobState: JobResponseData | undefined): { boxShadow: string } {
    const standard = { boxShadow: "none" };
    if (!entry || jobState?.summarizedOutcome === undefined) {
        return standard;
    }
    const colors: Record<string, string> = getColorRecords();

    // If we have skipped or aborted the step, we use the summarized outcome to help with coloring.
    if (entry.state == SkippedStr || entry.state == AbortedStr) {
        return { boxShadow: `inset 0 0 0 4px ${colors[jobState?.summarizedOutcome]}` };
    }
    else if (!isTerminalState(entry)) {
        return { boxShadow: `inset 0 0 0 4px ${colors[entry.state]}` };
    }

    return standard;
}

/**
 * Gets the cell style given the provided StepOutcomeTableEntry.
 * @param entry The entry to base the style off of.
 * @param viewModelOptions The view model options to consider while applying styles.
 * @returns The matching style given the state of the entry. 
 */
export function getCellStyle(entry: OutcomeTableEntry | null): { bgColor: string } {
    const colors: Record<string, string> = getColorRecords();

    let resultColor: string = 'transparent';
    if (!entry) {
        return { bgColor: resultColor };
    }

    const { outcome, state } = entry;

    // We do not fill the cells for Aborted, Skipped, or those about to run (or currently running).
    if (state === AbortedStr || state === SkippedStr || !isTerminalState(entry)) {
        return { bgColor: resultColor };
    }

    // There are some interesting mappings going on internally with respect to state & outcome. Outcome & State have overlapping "result view" space, and as such we need to 
    // narrowly select the edge cases (e.g. outcome=success && state=completed uniquely signifies a completed run with no errors, otherwise outcome=success it could mean ready, waiting, or running).
    // Then we can fall back to the state as the truth.

    if (outcome === FailureStr) {
        resultColor = colors[FailureStr];
    }
    // A step can be emitting warnings whilst still running, so we narrowly select this colouring.
    else if (outcome === WarningStr) {
        resultColor = colors[WarningStr];
    }
    // A step can have a completed outcome, but merely be waiting to run, running, or ready.
    else if (outcome === SuccessStr && (state === CompletedStepStateStr || state === CompleteLabelStateStr)) {
        resultColor = colors[SuccessStr];
    }
    // The edge cases have been satisfied, yield to the state to define result view color.
    else {
        resultColor = colors[state];
    }

    return { bgColor: resultColor };
}

// #endregion -- Style Helpers --

/**
 * React Component that provides refresh spinner on incremental refresh. 
 * Note: This is a separate component to prevent rerender of primary table @see GenerateFluentUITable upon refresh spinner.
 * @param handler The data handler used to query step outcome data.
 * @returns React Component.
 */
export const IncrementalRefreshIndicatorPanel: React.FC<{ handler: StepOutcomeDataHandler }> = observer(({ handler }) => {
    const totalSeconds = Math.floor(handler.refreshTime / 1000);
    const minutes = Math.floor(totalSeconds / 60);
    const seconds = totalSeconds % 60;
    const formatted = `${minutes}m ${seconds}s`;

    return (

        <div
            style={{
                gridRow: '1 / span 2',
                gridColumn: 3,
                overflowY: 'auto',
                width: 50,
                height: 50,
            }}
        >
            {handler.isInIncrementalDataRefresh && <Spinner size={SpinnerSize.medium} />}
            {!handler.isInIncrementalDataRefresh &&
                <TooltipHost
                    content={`Refresh the table. (Refresh rate: ${formatted})`}
                    directionalHint={DirectionalHint.leftCenter}
                >
                    <IconButton

                        iconProps={{ iconName: 'Refresh' }}
                        onClick={() => handler.poll()}
                    />
                </TooltipHost>}
        </div>
    )
});