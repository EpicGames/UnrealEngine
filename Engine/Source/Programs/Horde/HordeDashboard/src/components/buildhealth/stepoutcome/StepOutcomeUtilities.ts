// Copyright Epic Games, Inc. All Rights Reserved.

import { GetJobResponse, GetJobStepRefResponse, JobStepBatchError, JobStepOutcome, JobStepState } from "horde/backend/Api";
import { OutcomeTableEntry, ReadyStr, RunningStr, WaitingStr } from "./StepOutcomeDataTypes";
import moment from "moment";
import { displayTimeZone } from "horde/base/utilities/timeUtils";
import dashboard from "horde/backend/Dashboard";

export const DURATION_NOT_SET: number = -1;

/**
 * Gets whether the step outcome table entry is in a terminal state (no more work to be done), or results are still pending.
 * @param stepOutcomeEntry The entry to consider.
 * @returns True if the step has run to completion, false otherwise.
 */
export function isTerminalState(stepOutcomeEntry: OutcomeTableEntry): boolean {
    if (stepOutcomeEntry.state === ReadyStr || stepOutcomeEntry.state === RunningStr || stepOutcomeEntry.state === WaitingStr) {
        return false;
    }

    return true;
}

/**
  * This utility method will summarize the provided job, with respect to the most severe step outcome encountered thus far.
  * @param job The job to summarize.
  * @returns The most severe job step outcome. This is ordered as follows: @see JobStepOutcome.Unspecified , @see JobStepOutcome.Success , @see JobStepOutcome.Warnings , @see JobStepOutcome.Failre .
  */
export function summarizeJob(job: GetJobResponse): JobStepOutcome {
    let accumulatorOutcome: JobStepOutcome = JobStepOutcome.Unspecified;

    if (job.batches === undefined) {
        return accumulatorOutcome;
    }
    accumulatorOutcome = JobStepOutcome.Success;
    for (let response of job.batches) {
        for (let step of response.steps) {
            if (step.state !== JobStepState.Aborted && step.outcome === JobStepOutcome.Failure) {
                accumulatorOutcome = JobStepOutcome.Failure;

                // We cannot become less severe than Failure; exit.
                return accumulatorOutcome;
            } else if (step.outcome === JobStepOutcome.Warnings) {
                accumulatorOutcome = JobStepOutcome.Warnings;
            }
        }

        // Check to see if an entire batch has failed, in spite of the steps declaring a success (due to the absence of steps).
        // This is a common case when a more terminal failure occurs in batch resolution.
        if (accumulatorOutcome === JobStepOutcome.Success && response.error !== JobStepBatchError.None) {
            accumulatorOutcome = JobStepOutcome.Failure;
            return accumulatorOutcome;
        }
    }

    return accumulatorOutcome;
}

/**
  * This utility method will summarize the provided job, with respect to the most severe step outcome encountered thus far.
  * @param jobStepRefs The set of job step refs to use in summarization.  
  * @param job The job to summarize.
  * @returns The most severe job step outcome. This is ordered as follows: @see JobStepOutcome.Unspecified , @see JobStepOutcome.Success , @see JobStepOutcome.Warnings , @see JobStepOutcome.Failre .
  */
export function summarizeJobFromJobStepRef(jobStepRefs: GetJobStepRefResponse[], job: GetJobResponse): JobStepOutcome {
    let accumulatorOutcome: JobStepOutcome = JobStepOutcome.Unspecified;
    accumulatorOutcome = JobStepOutcome.Success;

    for (let response of jobStepRefs) {
        if (response.state !== JobStepState.Aborted && response.outcome === JobStepOutcome.Failure) {
            accumulatorOutcome = JobStepOutcome.Failure;

            // We cannot become less severe than Failure; exit.
            return accumulatorOutcome;
        } else if (response.outcome === JobStepOutcome.Warnings) {
            accumulatorOutcome = JobStepOutcome.Warnings;
        }
    }

    // Check to see if an entire batch has failed, in spite of the steps declaring a success (due to the absence of steps).
    // This is a common case when a more terminal failure occurs in batch resolution.
    if (accumulatorOutcome === JobStepOutcome.Success && job.batches) {
        // do a sanity check for batchEdge case
        for (let response of job.batches) {

            if (response.error !== JobStepBatchError.None) {
                accumulatorOutcome = JobStepOutcome.Failure;
                return accumulatorOutcome;
            }
        }
    }

    return accumulatorOutcome;
}

/**
 * Helper that will return the date-time string formatted according to user preferences (UTC vs Local; 12h vs 24h).
 * @param date THe date object to consider
 * @returns The formatted string.
 */
export function getDateTimeString(date: Date | null | undefined): string {
    if (date === null || date === undefined) {
        return "-";
    }

    const displayTime = moment(date).tz(displayTimeZone());
    const format = dashboard.display24HourClock
        ? "MMM D HH:mm:ss z"
        : "MMM D h:mm:ss A z";

    let displayTimeStr = displayTime.format(format);
    return displayTimeStr;
}

/**
 * Normalizes an ambiguous input (Date | string) to a Date object. 
 * @param input the input string union type.
 * @returns The @see Date standard representation for the time. 
 */
export function toDate(input?: Date | string): Date | undefined {
    if (!input) {
        return undefined;
    }

    if (input instanceof Date) {
        return input;
    }

    const parsed = new Date(input);
    return isNaN(parsed.getTime()) ? undefined : parsed;
}

/**
 * Computes the duration of a window in ms from a common time response format (Date | string).
 * @param startTime The start time of the window.
 * @param finishTime The finish time of the window.
 * @returns The duration of the window in ms, @see DURATION_NOT_SET if an invalid window is provided.
 */
export function computeDuration(startTime?: Date | string, finishTime?: Date | string): number {
    const startDate = toDate(startTime);
    const finishDate = toDate(finishTime);
    return startDate && finishDate
        ? finishDate.getTime() - startDate.getTime()
        : DURATION_NOT_SET;
}

/**
 * Generates a formatted duration provided a time span in milliseconds. 
 * @param ms The duration in milliseconds.
 * @returns The formated duration.
 */
export function formatDurationFromMs(ms: number): string {
    const diffSeconds = Math.floor(ms / 1000);
    const hours = Math.floor(diffSeconds / 3600);
    const minutes = Math.floor((diffSeconds % 3600) / 60);
    const seconds = diffSeconds % 60;

    return `${hours}h ${minutes}m ${seconds}s`;
}

/**
 * Generates a formatted duration provided a start and finish time.
 * @param startISO The start time, in string representation.
 * @param finishISO The finish time, in string representation.
 * @returns The formated duration.
 */
export function formatDuration(startISO: string, finishISO: string): string {
    const start = new Date(startISO);
    const finish = new Date(finishISO);
    const diffMs = finish.getTime() - start.getTime();

    return formatDurationFromMs(diffMs);
}