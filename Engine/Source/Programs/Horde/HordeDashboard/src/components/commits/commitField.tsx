import { Stack, TextField } from "@fluentui/react";
import React from "react";
import { CommitId } from "horde/backend/Api";
import { useImperativeHandle, useState } from "react";

export interface ICommitIdField {
    commitId?: CommitId;
}

export interface ICommitIdFieldProps {
    defaultCommitId?: CommitId;
    label?: string;
    disabled?: boolean;
    placeholder?: string;
    onChange?: (commitId: CommitId | undefined) => void;
}

export const CommitIdField = React.forwardRef<ICommitIdField, ICommitIdFieldProps>((props, ref) => {

    const [commitId, setCommitId] = useState(props?.defaultCommitId);

    useImperativeHandle(ref, () => ({
        commitId: commitId
    }));

    return <Stack>
        <TextField disabled={props?.disabled} label={props?.label}
            spellCheck={false} autoComplete="off"
            placeholder={props?.placeholder}
            value={commitId ?? ""}

            onChange={(ev, newValue) => {
                ev.preventDefault();

                let v = newValue;
                if (v) {
                    const numbers = /-?\d*/;
                    const match = newValue?.match(numbers);

                    if (match) {
                        v = match.join("");
                    } else {
                        v = undefined;
                    }
                }

                if (v && v !== '-') {
                    const n = parseInt(v);

                    if (n < -1) {
                        v = "-1";
                    }
                }

                setCommitId(v?.toString())

                if (props.onChange) {
                    props.onChange(v?.toString());
                }

            }} />
    </Stack>

})