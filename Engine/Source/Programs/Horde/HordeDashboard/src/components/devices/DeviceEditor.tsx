// Copyright Epic Games, Inc. All Rights Reserved.
import { SpinnerSize, DialogType, IDropdownOption, MessageBarType, Text, Checkbox, DefaultButton, Dialog, DialogFooter, Dropdown, MessageBar, Modal, PrimaryButton, Spinner, Stack, TextField, Label, ITag, TagPicker, IPickerItemProps } from "@fluentui/react";
import React, { useState } from "react";
import backend from "../../backend";
import { DashboardPreference, GetDeviceResponse } from "../../backend/Api";
import { getHordeStyling } from "../../styles/Styles";
import { DeviceHandler, DeviceStatus } from "../../backend/DeviceHandler";
import dashboard from "horde/backend/Dashboard";
import { isBright } from "horde/base/utilities/colors";

type DeviceEditData = {
   // if defined, existing device
   id?: string;
   platformId: string;
   poolId: string;
   modelId?: string;
   name: string;
   address?: string;
   enabled: boolean;
   maintenance: boolean;
   notes?: string;
   tags?: string[];
};

const onRenderSuggestionsItem = (item: ITag) => {
    return <Stack style={{height: 24, padding: 4}}>
                <Text title={item.name} style={{textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', maxWidth: 300, fontSize: 12}}>{item.name}</Text>
            </Stack>
}

const onRenderItem = (props: IPickerItemProps<ITag>) => {
    const item = props.item;
    return <Stack style={{ marginTop: 2, marginLeft: 3 }} key={`picker_item_${item.name}`}>
                <PrimaryButton
                    iconProps={{ iconName: "Cancel", styles: { root: { fontSize: 12, margin: 0, padding: 0 } } }}
                    styles={{label: { textOverflow: 'ellipsis', overflow: 'hidden', whiteSpace: 'nowrap', minWidth: 0, maxWidth: 150, margin: 0 }}}
                    style={{ padding: 2, paddingLeft: 5, paddingRight: 5, fontSize: 12, height: "unset"}}
                    text={item.name}
                    title={item.name}
                    onClick={props.onRemoveItem} />
            </Stack>;
}

const formatTag = (tag: string) => {
   const trimmed = tag.trim();
   if (trimmed.length === 0) return ''; 
   return trimmed
      .split(' ')
      .map(word => word.length > 1 
         ? word[0].toUpperCase() + word.slice(1) 
         : word.toUpperCase())
      .join(' ')
}

const getTextFromTag = (item: ITag) => formatTag(item.name);

const TagList: React.FC<{tags: string[], allTags: string[], onChange: (newTags: string[]) => void;}> = ({tags, allTags, onChange}) => {
   const [currentTags, setTags] = useState<string[]>(tags);
 
   const selectedItems: ITag[] = currentTags.map((key) => ({key: key.toLowerCase(), name: key} as ITag));

   const filterSuggestedTags = (filterText: string, _: ITag[]): ITag[] => {
    if (filterText.length < 1) return [];

    const lowerCurrentTags = currentTags.map(currentTag => currentTag.toLowerCase());
    const lowerText = filterText.trim().toLowerCase();
    const lowerTags = allTags.map(tag => tag.toLowerCase()).filter(tag => !lowerCurrentTags.includes(tag));

    let filteredTags: ITag[] = [];

    if (!lowerTags.includes(lowerText) && !lowerCurrentTags.includes(lowerText)) {
      filteredTags = [{key: lowerText, name: formatTag(filterText)}];
    }

    filteredTags = filteredTags.concat(lowerTags.filter(tag => tag.startsWith(lowerText)).map(t => ({key: t, name: formatTag(t)})))

    return filteredTags;
   }
 
   return (
      <Stack>
            <Label>Tags</Label>
            <TagPicker
               styles={{ input: { height: 28, width: 40 }, itemsWrapper: { marginRight: 3, marginBottom: 2 } }}
               onRenderItem={onRenderItem}
               onRenderSuggestionsItem={onRenderSuggestionsItem}
               removeButtonAriaLabel="Remove"
               selectionAriaLabel="Selected tags"
               selectedItems={selectedItems}
               onResolveSuggestions={filterSuggestedTags}
               getTextFromItem={getTextFromTag}
               onChange={(tags) => {
                  let newTagList : string[] = [];
                  if (!!tags) {
                        newTagList = tags!.map(getTextFromTag);
                  }
                  setTags(newTagList);
                  onChange?.(newTagList);
               }}
            />                
      </Stack>
  );
}

export const DeviceEditor: React.FC<{ handler: DeviceHandler, deviceIn?: GetDeviceResponse | undefined, editNote?: boolean, onClose: (device?: GetDeviceResponse) => void }> = ({ handler, deviceIn, editNote, onClose }) => {

   const [state, setState] = useState<{ device?: DeviceEditData, title?: string }>({});
   const [error, setError] = useState<string | undefined>();
   const [submitting, setSubmitting] = useState(false);
   const [confirmDelete, setConfirmDelete] = useState(false);
   
   const { hordeClasses } = getHordeStyling();

   if (submitting) {

      return <Modal isOpen={true} isBlocking={true} styles={{ main: { padding: 8, width: 400 } }} >
         <Stack style={{ paddingTop: 32 }}>
            <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
               <Stack horizontalAlign="center">
                  <Text variant="large">Please wait...</Text>
               </Stack>
               <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
                  <Spinner size={SpinnerSize.large} />
               </Stack>
            </Stack>
         </Stack>
      </Modal>

   }

   const platforms = handler.platforms;
   const pools = handler.pools;

   const device = state.device!;
   const existing = !!deviceIn;

   let deviceStatus = DeviceStatus.Available;

   if (deviceIn) {
      deviceStatus = handler.getDeviceStatus(deviceIn);
   }

   const reservation = deviceStatus === DeviceStatus.Reserved;

   const close = async () => {
      // @todo: state should be squashed

      await handler.forceUpdate();

      setState({});
      setSubmitting(false);
      setError("");
      setConfirmDelete(false);
      const updatedDevice: GetDeviceResponse | undefined = !!deviceIn? {
         ...deviceIn,
            platformId: device.platformId,
            enabled: device.enabled,
            address: device.address,
            name: device.name,
            poolId: device.poolId,
            modelId: device.modelId,
            notes: device.notes,
            tags: device.tags
         }: undefined;
      onClose(updatedDevice);
   };

   const onDelete = async () => {

      try {

         setSubmitting(true);
         await backend.deleteDevice(device.id!);
         await close();

      } catch (reason: any) {
         setError(reason.toString());
         setSubmitting(false);
      }

   }


   if (confirmDelete) {
      return <Dialog
         hidden={false}
         onDismiss={() => setConfirmDelete(false)}
         minWidth={400}
         dialogContentProps={{
            type: DialogType.normal,
            title: `Delete device ${state.device?.name} ?`,
         }}
         modalProps={{ isBlocking: true }} >
         <DialogFooter>
            <PrimaryButton onClick={() => { setConfirmDelete(false); onDelete() }} text="Delete" />
            <DefaultButton onClick={() => setConfirmDelete(false)} text="Cancel" />
         </DialogFooter>
      </Dialog>
   }

   const platformOptions = Array.from(platforms.values()).sort((a, b) => {

      return a.name.localeCompare(b.name);

   }).map(p => {
      return { key: `platform_${p.id}`, text: p.name, selected: device?.platformId === p.id, data: p.id } as IDropdownOption
   });

   let modelOptions: IDropdownOption[] = [];

   const platform = platforms.get(device?.platformId);

   if (platform) {
      modelOptions = platform.modelIds.sort((a, b) => {
         return a.localeCompare(b);
      }).map(model => {
         return { key: `model_${model}`, text: model, selected: device?.modelId === model, data: model } as IDropdownOption
      });
   }

   modelOptions.unshift({ key: `model_base`, text: "Base", selected: device?.modelId === undefined, data: undefined });

   let defaultModelKey = "model_base";
   if (device?.modelId) {
      defaultModelKey = `model_${device.modelId}`;
   }


   const poolOptions: IDropdownOption[] = [];
   pools.forEach(p => {
      poolOptions.push({ key: `pool_${p.id}`, text: p.name, selected: device?.poolId === p.id, data: p.id });
   });

   if (!state.device) {

      if (deviceIn) {
         setState({
            device: {
               id: deviceIn?.id,
               platformId: deviceIn.platformId,
               enabled: deviceIn.enabled,
               address: deviceIn.address,
               name: deviceIn.name,
               poolId: deviceIn.poolId,
               modelId: deviceIn.modelId,
               notes: deviceIn.notes,
               maintenance: !!deviceIn.maintenanceTime,
               tags : deviceIn.tags
            }, title: `Edit ${deviceIn.name}`
         });
      } else {
         setState({
            device: {
               enabled: true,
               maintenance: false,
               address: "",
               name: "",
               platformId: platformOptions.length > 0 ? platformOptions[0].data : "",
               poolId: poolOptions.length > 0 ? poolOptions[0].data : "",
               tags: []
            },
            title: "Add New Device"
         });
      }

      return null;
   }


   const statusOptions: IDropdownOption[] = [];

   if (reservation) {
      statusOptions.push({ key: 'status_reserved', text: "Reserved", selected: true, data: "Reserved" });
   } else {

      statusOptions.push({ key: 'status_available', text: "Available", selected: device.enabled && !device.maintenance, data: "Available" });
      statusOptions.push({ key: 'status_disabled', text: "Disabled", selected: !device.enabled, data: "Disabled" });
      statusOptions.push({ key: 'status_maintenance', text: "Maintenance", selected: device.enabled && device.maintenance, data: "Maintenance" });
   };

   // need to handle delete reservation case
   const onSave = async () => {

      if (!device.name) {
         setError("Device must have a name");
         return;
      }

      if (!device.address) {
         setError("Device must have an address");
         return;
      }

      setSubmitting(true);

      try {

         if (reservation) {

            if (editNote) {

               await backend.modifyDevice(device.id!, {
                  notes: device.notes ? device.notes.trim() : "",
                  tags: device.tags ? device.tags : [""]
               }).then(() => {
                  close();
               }).catch((reason) => {
                  setError(`Problem modifying device, ${reason}`);
               });

            } else {

               await backend.modifyDevice(device.id!, {
                  problem: false,
                  maintenance: device.maintenance,
                  enabled: true,
                  tags: device.tags ? device.tags : [""]
               }).then(() => {
                  close();
               }).catch((reason) => {
                  setError(`Problem modifying device, ${reason}`);
               });
            }

         }
         else if (!existing) {
            const result = await backend.addDevice({
               name: device.name.trim(),
               address: device.address,
               poolId: device.poolId,
               platformId: device.platformId,
               modelId: device.modelId,
               enabled: device.enabled,
               tags: device.tags
            });

            if (!result) {
               setError("Problem adding device");
            } else {
               close();
            }
         } else {

            await backend.modifyDevice(device.id!, {
               name: device.name.trim(),
               address: device.address,
               poolId: device.poolId,
               modelId: device.modelId ?? "Base",
               notes: device.notes,
               problem: false, // always clear automatically generated problem state when saving
               maintenance: device.maintenance,
               enabled: device.enabled,
               tags: device.tags
            }).then(() => {
               close();
            }).catch((reason) => {
               setError(`Problem modifying device,: ${reason}`);
               setSubmitting(false);
            })
         }
         
      } catch (reason: any) {
         setError(reason.toString());
         setSubmitting(false);
      }

   }

   const deleteButtonColor = dashboard.getPreference(DashboardPreference.ColorError) ? dashboard.getPreference(DashboardPreference.ColorError) : "#FF0000";
   const deleteButtonTextColor = isBright(deleteButtonColor!) ? "#2D3F5F" : "#FFFFFF";

   return <Modal className={hordeClasses.modal} isOpen={true} styles={{ main: { padding: 8, width: 700 } }} onDismiss={() => onClose()}>
      {!!error && <MessageBar
         messageBarType={MessageBarType.error}
         isMultiline={false}
         onDismiss={() => setError("")}
      >
         <Text>{error}</Text>
      </MessageBar>}

      <Stack style={{ padding: 8 }}>
         <Stack style={{ paddingBottom: 16 }}>
            <Text variant="mediumPlus" style={{ fontFamily: "Horde Open Sans SemiBold" }}>{state.title}</Text>
         </Stack>
         <Stack tokens={{ childrenGap: 8 }} style={{ padding: 8 }}>
            <TextField label="Name" disabled={existing} defaultValue={device.name} onChange={(ev, value) => {
               device.name = value ?? "";
            }} />
            <Dropdown label="Platform" disabled={existing} options={platformOptions} onChange={(ev, option) => {

               if (option) {
                  device.platformId = option.data as string;
                  device.modelId = undefined;

                  setState({ device: device, title: state.title });
               }

            }} />


            {(!editNote && !reservation) && <Dropdown label="Pool" options={poolOptions} onChange={(ev, option) => {

               if (option) {
                  device.poolId = option.data as string;
               }

            }} />}

            {(!editNote && !reservation) && <Dropdown label="Model" defaultSelectedKey={defaultModelKey} options={modelOptions} onChange={(ev, option) => {

               if (option?.data === undefined) {
                  device.modelId = undefined; // "Base"
               }
               else if (option) {
                  device.modelId = option.data as string;
               }

            }} />}


            <TextField label="Address" defaultValue={device.address} disabled={(editNote || reservation)} onChange={(ev, value) => {
               device.address = value ?? "";
            }} />

            {(!editNote && !reservation) && <Dropdown label="Status" options={statusOptions} onChange={(ev, option) => {

               if (option) {

                  if (option.data === "Available") {
                     device.maintenance = false;
                     device.enabled = true;
                  }

                  if (option.data === "Maintenance") {
                     device.maintenance = true;
                     device.enabled = true;
                  }

                  if (option.data === "Disabled") {
                     device.maintenance = false;
                     device.enabled = false;
                  }

               }

            }} />}

            <Stack>
               <Label></Label>
               <TagList tags={device.tags ? device.tags : []} allTags={handler.tags} onChange={(newTags => {
                  device.tags = newTags;
               })}/>
            </Stack>
            

            <TextField label="Notes" defaultValue={device.notes} multiline rows={5} resizable={false} onChange={(ev, newValue) => {
               device.notes = newValue;
            }} />

            {!!reservation && !editNote && <Stack style={{ paddingTop: 24 }}><Checkbox label="Disable device for maintenance once reservation finishes"
               checked={!!device.maintenance}
               onChange={(ev, checked) => {
                  device.maintenance = checked ? true : false;
                  setState({ device: device, title: state.title });
               }} />
            </Stack>}


         </Stack>

         <Stack horizontal style={{ paddingTop: 64 }}>
            {!!existing && !editNote && !reservation && <PrimaryButton style={{ color: deleteButtonTextColor, backgroundColor: deleteButtonColor, border: 0 }} onClick={() => setConfirmDelete(true)} text="Delete Device" />}
            <Stack grow />
            <Stack horizontal tokens={{ childrenGap: 28 }}>
               <PrimaryButton onClick={() => onSave()} text="Save" />
               <DefaultButton onClick={() => close()} text="Cancel" />
            </Stack>
         </Stack>
      </Stack>
   </Modal>

};