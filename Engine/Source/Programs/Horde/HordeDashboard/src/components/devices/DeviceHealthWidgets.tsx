// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Label, Text, TooltipHost, ITooltipProps, DirectionalHint, IGroup } from "@fluentui/react";
import { useState, useRef } from "react";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { getHordeStyling } from "../../styles/Styles";
import { DeviceHandler, DeviceHealthStatus, DeviceHealthLabels, FilteredDeviceTelemetry } from "../../backend/DeviceHandler";
import { getHumanTime, msecToElapsed } from "../../base/utilities/timeUtils";

import * as d3 from "d3";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;
type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type ScalarTime = d3.ScaleTime<number, number>

export const deviceHealthRateBar = (rate: number, width: number, height: number, text: string): JSX.Element => {
   const dashboardColors = dashboard.getStatusColors();

   return (
      <Stack verticalFill={true} verticalAlign="center" horizontal>
         <div style={{ width: width, height: height, verticalAlign: 'middle', marginRight: '3px' }}>
            <svg width={width} height={height}>
               <defs>
                  <clipPath id="cutout">
                     <rect x={0} y={0} width={width} height={height} rx={4} ry={4} />
                  </clipPath>
               </defs>
               <g clipPath="url(#cutout)">
                  <rect x={0} y={0} width={width} height={height} fill={dashboardColors.get(StatusColor.Success)} />
                  <rect x={0} y={0} width={Math.ceil(width * rate / 100)} height={height} fill={dashboardColors.get(StatusColor.Failure)} />
               </g>
            </svg>
         </div>
         <Text>{text}</Text>
      </Stack>
   );
}

export const deviceHealthBadge = (health: DeviceHealthStatus, hourRange: number): JSX.Element => {
   const dashboardColors = dashboard.getStatusColors();
   const healthColors: Map<DeviceHealthStatus, string> = new Map<DeviceHealthStatus, string>([
      [DeviceHealthStatus.Healthy, dashboardColors.get(StatusColor.Success)!],
      [DeviceHealthStatus.Unreliable, dashboardColors.get(StatusColor.Warnings)!],
      [DeviceHealthStatus.Problematic, dashboardColors.get(StatusColor.Failure)!],
   ]);
   const { modeColors } = getHordeStyling();

   return <TooltipHost content={`${DeviceHealthLabels.get(health)} device < ${hourRange}h`} delay={0} closeDelay={250}
         styles={{root: {display: 'inline-block'}}} directionalHint={DirectionalHint.bottomCenter}>
            <Label style={{ padding: '3px 7px 3px 7px', fontSize: 9, fontFamily: "Horde Open Sans SemiBold", color: "#ffffff", backgroundColor: healthColors.get(health), cursor: 'inherit'}}>Health</Label>
      </TooltipHost>
}

export class PoolTimelineGraph {
   constructor(handler: DeviceHandler) {
      this.handler = handler;
      this.margin = { top: 0, right: 0, bottom: 0, left: 0 };
   }

   set(devices?: string[]) {
      this.data = this.handler.getDevicesTelemetry(devices);
      this.devices = devices;
      this.lastUpdate = this.handler.poolTelemetryLastUpdate;
   }

   draw(container: HTMLDivElement, width: number, height: number, devices?: string[]) {

      if (!!devices && !!this.devices && (devices.length !== this.devices.length || !devices.every(d => this.devices!.includes(d)))) {
         this.forceRender = true;
      } else if (this.lastUpdate != this.handler.poolTelemetryLastUpdate) {
         this.forceRender = true;
      }

      if (this.hasRendered && !this.forceRender) {
         return;
      }
      
      this.container = container;
      this.width = width;
      this.height = height;
      const margin = this.margin;
      this.clear();

      this.set(devices);

      const data = this.data;

      if (!data) {
         return;
      }

      this.hasRendered = true;
      this.forceRender = false;

      const { modeColors } = getHordeStyling();
      const dashboardColors = dashboard.getStatusColors();

      const maxReservations = data.reduce((a, d) => Math.max(a, d.reservations + d.available), 1);
      const peakReservations = data.reduce((a, d) => Math.max(a, d.reservations), 0);
      this.peakReservationsPerc = Math.ceil(peakReservations / maxReservations * 100);
      this.totalSaturatedReservation = peakReservations == maxReservations? data.reduce((a, d) => a + (d.reservations + d.problems == maxReservations? 1 : 0), 0) : 0;

      const svg = d3.select(container)
         .append("svg")
         .attr("width", width)
         .attr("height", height)
         .attr("viewBox", [0, 0, width, height] as any)
         .style("background-color", modeColors.crumbs);

      // plots
      const xPlot = d3.scaleTime()
         .domain(d3.extent(data, (d => d.date)) as any)
         .range([this.margin.left, this.width - this.margin.right]);

      const yPlot = d3.scaleLinear()
         .domain([0, maxReservations])
         .range([height - margin.bottom, margin.top]);


      // bottom axis
      const xAxis = svg.append("g")
         .call(this.xAxisMethod as any, xPlot)
         .style("pointer-events", "none");

      // reservation graph
      const area = d3.area()
         .x((d: any) => xPlot(d.date))
         .y0(yPlot(0))
         .y1((d: any) => yPlot(d.reservations))
         .curve(d3.curveStep);

      const areaColor = dashboard.darktheme ?
         (d3.color(modeColors.text) as any)?.darker(0.2)
         : (d3.color(modeColors.text) as any)?.brighter(0.2);

      svg.append("path")
         .datum(data)
         .attr("fill", areaColor)
         .attr("stroke", modeColors.text).attr("stroke-width", 1.5)
         .attr("opacity", 0.4)
         .attr("d", area as any)
         .style("pointer-events", "none");

      // problems graph
      const areaProblem = d3.area()
         .x((d: any) => xPlot(d.date))
         .y0(yPlot(0))
         .y1((d: any) => yPlot(d.problems))
         .curve(d3.curveStep);

      const colorProblem = dashboardColors.get(StatusColor.Failure)!;

      svg.append("path")
         .datum(data)
         .attr("fill", colorProblem)
         .attr("d", areaProblem as any)
         .style("pointer-events", "none");

      if (maxReservations > 1)
      {
         // saturation
         const lineSaturation = d3.line()
            .defined((d: any) => d.reservations == maxReservations)
            .x((d: any) => xPlot(d.date))
            .y((d: any) => yPlot(d.reservations))
            .curve(d3.curveStep);

         svg.append("path")
            .datum(data)
            .attr("fill", "transparent")
            .attr("stroke", colorProblem).attr("stroke-width", 4)
            .attr("opacity", 0.6)
            .attr("d", lineSaturation as any)
            .style("pointer-events", "none");
      }

   }

   private clear() {
      if (!!this.container) {
         d3.select(this.container).selectAll("*").remove();
      }
   }

   private get xAxisMethod() {
      return (g: SelectionType, x: ScalarTime) => {
         const tickSet: Set<string> = new Set();

         g.attr("transform", `translate(0,${this.height - this.margin.bottom})`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "7px")
            .attr("opacity", dashboard.darktheme? 0.4 : 0.5)
            .call(d3.axisTop(x)
               .ticks(8)
               .tickFormat(d => {
                  const v = (d as Date).toDateString();
                  if (!tickSet.has(v)) {
                     tickSet.add(v)
                     return getHumanTime(d as Date)
                  }

                  return (d as Date).toLocaleString(undefined, { hour: 'numeric', hour12: true })
               })
               .tickSizeOuter(0)
               .tickPadding(10));
      }
   }

   handler: DeviceHandler;

   container?: HTMLDivElement;
   private margin: { top: number, right: number, bottom: number, left: number }
   private width = 0;
   private height = 0;

   private data?: FilteredDeviceTelemetry[];
   private devices?: string[];
   private lastUpdate?: number = 0;

   private hasRendered = false;
   forceRender = false;

   peakReservationsPerc: number = 0;
   totalSaturatedReservation: number = 0;
}

export const DeviceGroupHeader: React.FC<{ group: IGroup, handler: DeviceHandler, devices?: string[] }> = ({ group, handler, devices }) => {

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const graph = useRef<PoolTimelineGraph>(new PoolTimelineGraph(handler));

   if (!!container && !!devices) {
      graph.current.draw(container, 300, 38, devices);
   }

   const tooltipProps: ITooltipProps = {
      onRenderContent: () => (
         <ul style={{margin: 0, padding: 0}}>
            <li>Peak usage: {graph.current.peakReservationsPerc}%</li>
            {graph.current.totalSaturatedReservation > 0 && <li>Saturated for: {msecToElapsed(graph.current.totalSaturatedReservation * 10 * 60 * 1000, true, false)}</li>}
         </ul>
      )
   }

   const backgroundColor = (dashboard.darktheme? d3.gray(10) : d3.gray(80)).toString();

   const title = `${group.name} (${group.count})`;

   return <TooltipHost tooltipProps={tooltipProps} delay={0} closeDelay={250}
         styles={{root: {display: 'inline-block'}}} directionalHint={DirectionalHint.rightCenter}> 
      <Stack style={{ position: "relative" }} horizontal verticalAlign="center">
         <Text style={{width: 120, fontWeight: 'bold'}}>{title}</Text>
         <div id={`pool_graph_container_${group.key}`}
            className="horde-no-darktheme"
            style={{ userSelect: "none", height: 38, backgroundColor: backgroundColor }}
            ref={(ref: HTMLDivElement) => setContainer(ref)}
            onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </TooltipHost>
}
