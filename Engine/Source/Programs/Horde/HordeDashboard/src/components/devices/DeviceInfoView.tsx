// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsRow, IColumn, IconButton, IDetailsListProps, Label, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import moment from "moment";
import { useState } from "react";
import { Link, useNavigate } from "react-router-dom";
import backend from "../../backend";
import { DeviceStatus, DeviceTelemetryQuery, GetDeviceResponse, GetDeviceTelemetryResponse, GetDeviceUtilizationResponse, GetTelemetryInfoResponse, JobData } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { projectStore } from "../../backend/ProjectStore";
import { displayTimeZone } from "../../base/utilities/timeUtils";
import { getHordeStyling } from "../../styles/Styles";
import { DeviceHandler, DeviceStatusLabels } from "../../backend/DeviceHandler";
import { deviceHealthRateBar } from "./DeviceHealthWidgets"
import { DeviceStatusIcon } from "../StatusIcon";

const streamIdToFullname = new Map<string, string>();

export const DeviceInfoModal: React.FC<{ handler: DeviceHandler, deviceIn?: GetDeviceResponse | undefined, onEdit: (device: GetDeviceResponse) => void, onClose: () => void }> = ({ handler, deviceIn, onEdit, onClose }) => {

   type JobItem = {
      job: JobData,
      utilization: GetDeviceUtilizationResponse
   };

   type JobTelemetryItem = {
      job: GetTelemetryInfoResponse,
   }

   const [jobState, setJobState] = useState<{ items: JobItem[], telemetryItems: JobTelemetryItem[], querying: boolean, queried: boolean, modifiedByUser?: string }>({ items: [], telemetryItems: [], querying: true, queried: false });
   const navigate = useNavigate();

   if (!deviceIn) {
      return null;
   }

   const { hordeClasses } = getHordeStyling();

   // get unique ids
   let jobIds = deviceIn.utilization?.map(u => u.jobId).filter(jobId => !!jobId) as string[];
   if (jobIds) {
      jobIds = Array.from(new Set(jobIds)).slice(0, 100);
   }

   const queryJobs = async () => {

      if (!deviceIn.modifiedByUser && !jobIds?.length) {
         setJobState({ ...jobState, queried: true, querying: false });
         return;
      }

      let modifiedByUser = "";

      if (deviceIn.modifiedByUser) {
         const user = await backend.getUsers({ ids: [deviceIn.modifiedByUser!] });
         if (user.length && user[0].name) {
            modifiedByUser = user[0].name;
         }

      }

      let deviceTelemetry: GetDeviceTelemetryResponse[] = [];

      try {

         const maxCount = 512;
         const telemetryQuery: DeviceTelemetryQuery = {
            id: [deviceIn.id],
            count: maxCount,
            minCreateTime: new Date(new Date().getTime() - 86400000 * 14).toISOString()
         };

         deviceTelemetry = await backend.getDeviceTelemetry(telemetryQuery);

      } catch (reason) {

      } finally {

         const streamIds = new Set<string>();

         if (deviceTelemetry.length) {
            deviceTelemetry[0].telemetry.forEach(t => {
               if (t.streamId) {
                  const stream = projectStore.streamById(t.streamId);
                  if (!stream || !stream.fullname) {
                     streamIds.add(t.streamId);
                     streamIdToFullname.set(t.streamId, t.streamId);
                     return;
                  };
                  streamIds.add(t.streamId);
                  streamIdToFullname.set(t.streamId, stream.fullname);
               }
            });
         }

         const items: JobItem[] = [];
         const telemetryItems: JobTelemetryItem[] = [];

         deviceIn.utilization?.forEach(u => {
            const jobsFromTelemetry = deviceTelemetry[0]?.telemetry;
            const telemetryJob = jobsFromTelemetry?.find(j => j.jobId === u.jobId && j.stepId === u.stepId);
            if (!telemetryJob) { return; }
            telemetryItems.push({ job: telemetryJob });
         });

         setJobState({ ...jobState, items: items, telemetryItems: telemetryItems, queried: true, querying: false, modifiedByUser: modifiedByUser });
      }

   }


   if (!jobState.queried) {
      setJobState({ ...jobState, queried: true, querying: true });
      queryJobs();
      return null;
   }

   if (jobState.querying) {

      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Loading Device {deviceIn.name}</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>

         </Stack>
      </Modal>
   }


   // results
   const columns: IColumn[] = [
      //{ key: 'column1', name: 'Change', data: 'string', isRowHeader: true, minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'column1', name: 'Device Health', isRowHeader: true, minWidth: 15, maxWidth: 15, isResizable: false },
      { key: 'column2', name: 'Name', data: 'string', isRowHeader: true, minWidth: 530, maxWidth: 530, isResizable: false },
      { key: 'column3', name: 'Created', data: 'date', isRowHeader: true, minWidth: 120, maxWidth: 120, isResizable: false },
      //{ key: 'column3', name: 'StartedBy', minWidth: 140, maxWidth: 140, isResizable: false },

   ];

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as JobTelemetryItem;

         let url = `/job/${item.job.jobId}`;

         if (item.job?.stepId) {
            url += `?step=${item.job.stepId}`
         }

         const commonSelectors = { ".ms-DetailsRow-cell": { "overflow": "visible", padding: 0 } };

         props.styles = { ...props.styles, root: { selectors: { ...commonSelectors as any } } };

         return <Link to={url} onClick={(ev) => { if (!ev.ctrlKey) onClose() }}><div className="job-item"><DetailsRow {...props} /> </div></Link>;

      }
      return null;
   };



   const renderItem = (item: JobTelemetryItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      if (column.name === "Device Health") {
         let deviceStatus = item.job.problemTimeUtc ? DeviceStatus.Error : DeviceStatus.Normal;
         return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ overflow: "hidden" }} ><DeviceStatusIcon status={deviceStatus} /></Stack>
      }

      if (column.name === "Name") {

         let name = `${streamIdToFullname.get(item.job.streamId ?? "")!}`;
         if (item.job.stepName) {
            name += ` - ${item.job.stepName}`;
         }

         return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ overflow: "hidden" }} ><Text variant="small">{name}</Text></Stack>;
      }

      /*
      if (column.name === "Change") {
          
          const batch = item.job.batches?.find(b => b.steps.find(s => s.id === item.utilization.stepId));
          const step = batch?.steps.find(s => s.id === item.utilization.stepId);

          if (step) {
              return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ paddingTop: 4 }} ><StepStatusIcon step={step} style={{ fontSize: 12, paddingBottom: 4 }} /><ChangeButton job={item.job} /></Stack>;
          }

          

          return <Stack horizontal verticalFill={true} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ paddingTop: 4 }} ><ChangeButton job={item.job} /></Stack>;
      }
      */

      /*
      if (column.name === "StartedBy") {
          let startedBy = item.job.startedByUserInfo;
          if (!startedBy) {
              startedBy = "Scheduler";
          }
          return <Stack verticalAlign="center" verticalFill={true} horizontalAlign={"center"}>{startedBy}</Stack>;
      }
      */


      if (column.name === "Created") {

         if (item.job.createTimeUtc) {

            const displayTime = moment(item.job.reservationStartUtc).tz(displayTimeZone());
            const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

            let displayTimeStr = displayTime.format('MMM Do') + ` at ${displayTime.format(format)}`;


            return <Stack verticalAlign="center" horizontalAlign="end" tokens={{ childrenGap: 0, padding: 0 }} style={{ height: "100%", paddingRight: 18 }}>
               <Text variant="small">{displayTimeStr}</Text>
            </Stack>;

         } else {
            return "???";
         }
      }

      return <Stack />;
   }

   const tags = deviceIn.tags ? deviceIn.tags.join(',') : '';
   const failureRate = jobState.telemetryItems.length == 0? 0 : Math.floor(jobState.telemetryItems.filter(item => !!item.job.problemTimeUtc).length / jobState.telemetryItems.length * 100);
   const status = handler.getDeviceStatus(deviceIn);

   return (<Modal isOpen={true} styles={{ main: { padding: 8, width: 1140, height: '800px', backgroundColor: '#FFFFFF' } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
      <Stack styles={{ root: { paddingTop: 8, paddingLeft: 24, paddingRight: 12, paddingBottom: 8 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal styles={{ root: { padding: 0 } }}>
               <Stack style={{ paddingLeft: 0, paddingTop: 4 }} grow>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{deviceIn.name}</Text>
               </Stack>
               <Stack grow horizontalAlign="end">
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     onClick={() => { onClose(); }}
                  />
               </Stack>
            </Stack>

            <Stack horizontal tokens={{ childrenGap: 48 }} >

               <Stack>
                  <Stack tokens={{ childrenGap: 12 }}>

                     <Stack>
                        <TextField label="Pool" value={deviceIn.poolId?.toUpperCase()} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack>
                        <TextField label="Model" value={deviceIn.modelId ? deviceIn.modelId : "Base"} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack>
                        <TextField label="Address" value={deviceIn.address} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack>
                        <TextField label="Status" value={DeviceStatusLabels.get(status)} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack>
                        <TextField label="Tags" value={tags} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack>
                        <TextField label="Last Modified By" value={jobState.modifiedByUser ? jobState.modifiedByUser : ""} disabled={true} readOnly={true} />
                     </Stack>

                     <Stack horizontal style={{ paddingTop: 24 }}>
                        <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                           <PrimaryButton text="Edit" onClick={() => { onEdit(deviceIn) }} />
                        </Stack>
                        <Stack style={{ paddingTop: 8, paddingRight: 24 }}>
                           <Link to={`/audit/device/${deviceIn.id}`} ><DefaultButton text="Audit" onClick={(ev) => { }} /></Link>
                        </Stack>
                     </Stack>
                  </Stack>
               </Stack>

               <Stack>
                  <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingBottom: 4 } }}>
                     <Stack>
                        <Label>Device Reservations History</Label>
                        {!!failureRate && deviceHealthRateBar(failureRate, 70, 15, `${failureRate}% Failure Rate`)}
                     </Stack>

                     {!jobState.telemetryItems.length && <Stack>
                        <Text>No Results</Text>
                     </Stack>}

                     {!!jobState.telemetryItems.length && <Stack>
                        <div style={{ overflowY: 'auto', overflowX: 'hidden', height: "670px" }} data-is-scrollable={true}>
                           <Stack tokens={{ childrenGap: 12 }} style={{ paddingRight: 12 }}>
                              <DetailsList
                                 compact={true}
                                 isHeaderVisible={false}
                                 indentWidth={0}
                                 items={jobState.telemetryItems}
                                 columns={columns}
                                 setKey="set"
                                 selectionMode={SelectionMode.none}
                                 onRenderItemColumn={renderItem}
                                 onRenderRow={renderRow}

                              />
                           </Stack>
                        </div>
                     </Stack>}
                  </Stack>
               </Stack>

            </Stack>

         </Stack>
      </Stack>

   </Modal >);
}
