// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, Dropdown, FocusZone, FocusZoneDirection, IColumn, IconButton, IDropdownOption, Modal, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { useState, useRef, useEffect, useCallback, useMemo } from "react";
import { Link } from "react-router-dom";
import { deviceHealthRateBar } from "./DeviceHealthWidgets"
import { PoolTelemetryHandler, DeviceProblem } from "../../backend/DeviceTelemetryHandler"
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { getHumanTime } from "../../base/utilities/timeUtils";
import { getHordeStyling } from "../../styles/Styles";
import { mergeStyles } from '@fluentui/react/lib/Styling';
import { PoolTelemetryGraph, PoolDistributionGraph, PoolDistributionTimelineGraph} from "./DeviceTelemetryGraphs";
import { GetDevicePlatformResponse } from "../../backend/Api";

import * as d3 from "d3";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

const scrollableOverflowOverrideClass = mergeStyles({
   // avoid tooltip to be cut by modal
   overflow: "visible !important" as any
});

const HealthGraph: React.FC<{ handler: PoolTelemetryHandler, platform?: string, onChange: (options: object) => void }> = ({ handler, platform, onChange }) => {

   const selectedPlatformItem = Array.from(handler.platforms.values()).find((p) => p.id === platform);
   if (!selectedPlatformItem) {
      return <Stack style={{ paddingLeft: 12, paddingRight: 32 }}>
         <Text>Unknown platform {platform}</Text>
      </Stack>
   }

   const problems = useMemo(() => handler.getProblemDevices(), [handler]);

   const renderPlatform = useCallback((platform: GetDevicePlatformResponse) => {
      let platformProblems = problems.get(platform.id) ?? [];
      platformProblems = platformProblems.filter(p => p.problems > 8 && p.reservable).sort((a, b) => {
         if (a.problemsRate < b.problemsRate) return 1;
         if (a.problemsRate > b.problemsRate) return -1;
         if (a.problems < b.problems) return 1;
         if (a.problems > b.problems) return -1;
         return 0;
      });

      const problemColumns: IColumn[] = [
         { key: 'health-column1', name: 'Problem Devices', minWidth: 60, maxWidth: 150 },
         { key: 'health-column2', name: 'Pool', minWidth: 60, maxWidth: 80 },
         { key: 'health-column3', name: 'Issues/Reservations', minWidth: 100, maxWidth: 150 },
         { key: 'health-column4', name: 'Latest Issue', minWidth: 300, maxWidth: 640 },
      ];

      const schemaColor = d3.scaleOrdinal()
         .domain(Array.from(handler.pools.values()).map((p) => p.name))
         .range(dashboard.darktheme ? d3.schemeDark2 : d3.schemeSet2);

      let column = problemColumns.find(c => c.name === "Pool")!;
      column.onRender = ((item: DeviceProblem) => {

         return <Stack verticalFill={true} verticalAlign="center" horizontal>
            <svg style={{ width: 5, height: 15, verticalAlign: 'middle', marginRight: '3px' }}>
                  <rect x={0} y={0} width={5} height={15} fill={`${schemaColor(item.poolName)}`} />
            </svg>
            <Text>{item.poolName}</Text>
         </Stack>
      });

      column = problemColumns.find(c => c.name === "Issues/Reservations")!;
      column.onRender = ((item: DeviceProblem) => deviceHealthRateBar(item.problemsRate, 60, 15, item.problemsDesc));

      column = problemColumns.find(c => c.name === "Latest Issue")!;
      column.onRender = ((item: DeviceProblem) => {

         const latest = item.latestProblem;

         if (!latest) {
            return null;
         }

         if (!latest.jobId || !latest.stepId) {
            return <Stack verticalFill={true} verticalAlign="center">
               <Text>{getHumanTime(latest.problemTime)}</Text>
            </Stack>
         }

         const url = `/job/${latest.jobId}?step=${latest.stepId}`;

         return <Stack verticalFill={true} verticalAlign="center">
            <Link to={url} target="_blank">
               <Text variant="small">{`${getHumanTime(latest.problemTime!)} - ${latest.jobName ?? "Unknown Job Name"} - ${latest.stepName ?? "Unknown Step Name"}`}</Text>
            </Link>
         </Stack>
      });

      column = problemColumns.find(c => c.name === "Problem Devices")!;
      column.onRender = ((item: DeviceProblem) => {
         const deviceName = item.deviceName;
         const url = `/devices?pivotKey=pivot-key-automation&filter=${deviceName}`;
         return <Stack verticalFill={true} verticalAlign="center">
            <Link to={url} target="_blank"><Text variant="small">{deviceName}</Text></Link>
         </Stack>
      });

      return <Stack style={{ paddingLeft: 12, paddingRight: 32 }}>
         {!!platformProblems?.length &&
            <Stack style={{ paddingLeft: 12, paddingTop: 12, width: 1220 }}>
               <DetailsList
                  isHeaderVisible={true}
                  items={platformProblems}
                  columns={problemColumns}
                  layoutMode={DetailsListLayoutMode.justified}
                  compact={true}
                  selectionMode={SelectionMode.none}
               />
            </Stack>}
      </Stack>
   }, [handler]);

   let title = "Device Health Per Platform";

   if (handler.data && handler.data.length > 1) {
      const minDate = getHumanTime(handler.data[0].date);
      const maxDate = getHumanTime(handler.data[handler.data.length - 1].date);
      title = `Device Health Per Platform / ${minDate} - ${maxDate}`;
   }

   return <Stack style={{ paddingBottom: 18 }}>
         <Stack horizontal verticalAlign="start">
            <Stack style={{ paddingLeft: 4 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{title}</Text>
            </Stack>
            <Stack grow />
            <TelemetryDropdown
               handler={handler}
               isPoolDisabled={true}
               selectedPlatform={platform}
               onChange={(options: object) => {
                  onChange(options);
               }}
            />
         </Stack>
         <Stack style={{ paddingLeft: 12, paddingTop: 12 }}>
            <FocusZone direction={FocusZoneDirection.vertical}>
               <div style={{ position: 'relative', width: 1400, height: 650 }} data-is-scrollable>
                  <ScrollablePane onScroll={() => { }}>
                     <Stack style={{ paddingTop: 12 }} tokens={{ childrenGap: 24 }}>
                        {renderPlatform(selectedPlatformItem)}
                     </Stack>
                  </ScrollablePane>
               </div>
            </FocusZone>
         </Stack>
      </Stack>
}

const DistributionGraph: React.FC<{ handler: PoolTelemetryHandler, platform?: string, onChange: (options: object) => void }> = ({ handler, platform, onChange }) => {
   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const graph = useRef<PoolDistributionGraph>(new PoolDistributionGraph(handler));

   const [timelineContainer, setTimelineContainer] = useState<HTMLDivElement | null>(null);
   const timelineGraph = useRef<PoolDistributionTimelineGraph>(new PoolDistributionTimelineGraph(handler));

   const drawGraph = useCallback(() => {
      if (container && platform) {
         graph.current.draw(container, platform, 1400, 460, 200, timelineGraph.current?.startDate, timelineGraph.current?.endDate);
      }
   }, [container, platform, timelineGraph]);
   drawGraph();

   const drawTimelineGraph = useCallback(() => {
      if (timelineContainer && platform) {
         timelineGraph.current.draw(timelineContainer, platform, 800, 120, () => {
            if (graph.current) {
               graph.current.forceRender = true;
               drawGraph();
            }
         });
      }
   }, [timelineContainer, platform, timelineGraph, drawGraph]);
   drawTimelineGraph();

   let title = "Device Reservation Distribution Per Job Step";

   if (handler.data && handler.data.length > 1) {
      const minDate = getHumanTime(handler.data[0].date);
      const maxDate = getHumanTime(handler.data[handler.data.length - 1].date);
      title = `Device Reservation Distribution Per Job Step / ${minDate} - ${maxDate}`;
   }

   return <Stack>
         <Stack horizontal verticalAlign="start">
            <Stack style={{ paddingLeft: 4 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{title}</Text>
            </Stack>
            <Stack grow />
            <TelemetryDropdown
               handler={handler}
               isPoolDisabled={true}
               selectedPlatform={platform}
               onChange={(options: object) => {
                  if (graph.current) {
                     graph.current.forceRender = true;
                  }
                  if (timelineGraph.current) {
                     timelineGraph.current.forceRender = true;
                  }
                  onChange(options);
               }}
            />
         </Stack>
         <Stack style={{ paddingLeft: 12, paddingTop: 12}}>
            <Stack.Item>
               <div id="pool_distribution_graph_container" className="horde-no-darktheme" style={{ userSelect: "none" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
            </Stack.Item>
            <Stack.Item align="center">
               <div id="pool_distribution_timeline_graph_container" className="horde-no-darktheme" style={{ userSelect: "none" }} ref={(ref: HTMLDivElement) => setTimelineContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
            </Stack.Item>
         </Stack>
      </Stack>
}

const DevicePoolGraph: React.FC<{ handler: PoolTelemetryHandler, pool?: string, platform?: string, onChange: (options: object) => void }> = ({ handler, pool, platform, onChange }) => {

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const graph = useRef<PoolTelemetryGraph>(new PoolTelemetryGraph(handler));

   if (container && pool && platform) {
      graph.current.draw(container, pool, platform, 1400, 600);
   }

   let title = "Device Usage Telemetry";

   if (handler.data && handler.data.length > 1) {
      const minDate = getHumanTime(handler.data[0].date);
      const maxDate = getHumanTime(handler.data[handler.data.length - 1].date);
      title = `Device Usage Telemetry / ${minDate} - ${maxDate}`;
   }

   return <Stack style={{ position: "relative" }}>
         <Stack horizontal verticalAlign="start" style={{ paddingBottom: 18 }}>
            <Stack style={{ paddingLeft: 4 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{title}</Text>
            </Stack>

            <Stack grow />
            <TelemetryDropdown
               handler={handler}
               selectedPool={pool}
               selectedPlatform={platform}
               onChange={(options: object) => {
                  if (graph.current) {
                     graph.current.forceRender = true;
                  }
                  onChange(options);
               }}
            />
         </Stack>

         <Stack style={{ paddingLeft: 12 }}>
            <div id="pool_graph_container" className="horde-no-darktheme" style={{ userSelect: "none" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
         </Stack>
      </Stack>
}

const TelemetryDropdown: React.FC<{ handler: PoolTelemetryHandler, selectedPool?: string, selectedPlatform?: string, isPoolDisabled?: boolean, isPlatformDisabled?: boolean, onChange: (options: object) => void }> = ({ handler, selectedPool, selectedPlatform, isPoolDisabled, isPlatformDisabled, onChange }) => {
   const pools = Array.from(handler.pools.values()).sort((a, b) => a.name.localeCompare(b.name));
   const platforms = Array.from(handler.platforms.values()).sort((a, b) => a.name.localeCompare(b.name));

   if (!isPoolDisabled && !pools.length) {
      return <Stack>No Pools</Stack>
   }
   if (!isPlatformDisabled && !platforms.length) {
      return <Stack>No Platforms</Stack>
   }

   const defaultSelectedPool = selectedPool ?? pools[0].id;
   const defaultSelectedPlatform = selectedPlatform ?? platforms[0].id;

   useEffect(() => {
      // Assign default values on post load
      if ((!isPoolDisabled && !selectedPool) || (!isPlatformDisabled && !selectedPlatform)) {
         onChange({pool: defaultSelectedPool, platform: defaultSelectedPlatform});
      }
   }, [selectedPool, selectedPlatform, defaultSelectedPool, defaultSelectedPlatform, isPoolDisabled, isPlatformDisabled]);

   const poolOptions: IDropdownOption[] = pools.map(p => { return { key: `pool_key_${p.id}`, text: p.name, data: p.id } });
   const platformOptions: IDropdownOption[] = platforms.map(p => { return { key: `platform_key_${p.id}`, text: p.name, data: p.id } });

   return <Stack horizontal tokens={{ childrenGap: 12 }} style={{ paddingTop: 1 }}>
            <Dropdown
               defaultSelectedKey={`platform_key_${defaultSelectedPlatform}`}
               options={platformOptions}
               onChange={(event, option) =>  onChange({platform: option!.data})}
               disabled={!!isPlatformDisabled}
               style={{ width: 120 }}
            />
            <Dropdown
               defaultSelectedKey={`pool_key_${defaultSelectedPool}`}
               options={poolOptions}
               onChange={(event, option) => onChange({pool: option!.data})}
               disabled={!!isPoolDisabled}
               style={{ width: 120 }}
            />
         </Stack>
}

const renderMenuButton = (item: any): JSX.Element => {
   return <Stack key={item.key}><DefaultButton text={item.title} onClick={item.onClick} checked={item.checked} style={{width: '80px', paddingTop: 16 , paddingBottom: 16}} /></Stack>
}

export const DevicePoolTelemetryModal: React.FC<{ poolsIn?: Array<string>, platformsIn?: Array<string>, onClose?: () => void }> = ({ poolsIn, platformsIn, onClose }) => {

   const { hordeClasses, modeColors } = getHordeStyling();
   const [selected, setSelected] = useState<{ pool?: string, platform?: string }>({
      pool: poolsIn?.find((e) => true), platform: platformsIn?.find((e) => true)});
   const [graphOption, setGraphOption] = useState<{ usage?: boolean, health?: boolean, distribution?: boolean }>({usage: true});
   const handler = useRef<PoolTelemetryHandler | undefined>(undefined);

   if (!handler.current) {
      handler.current = new PoolTelemetryHandler();
   }

   const menuItems = [
      {
         key: 'graph1',
         title: 'Usage',
         onClick: () => setGraphOption({usage: true}),
         checked: graphOption.usage
      },
      {
         key: 'graph2',
         title: 'Health',
         onClick: () => setGraphOption({health: true}),
         checked: graphOption.health
      },
      {
         key: 'graph3',
         title: 'Distribution',
         onClick: () => setGraphOption({distribution: true}),
         checked: graphOption.distribution
      }
   ]

   return <Stack>
      <Modal className={hordeClasses.modal} scrollableContentClassName={scrollableOverflowOverrideClass} isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 1560, backgroundColor: modeColors.background, hasBeenOpened: false, top: "64px", position: "absolute", height: "760px", overflow: "visible !important" as any } }} onDismiss={() => { if (onClose) { onClose() } }}>
         <Stack>
            <Stack horizontal>
               <Stack grow />
               <Stack horizontalAlign="end" style={{ paddingTop: 8, paddingRight: 12 }}>
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     ariaLabel="Close popup modal"
                     onClick={() => { if (onClose) onClose() }}
                  />
               </Stack>
            </Stack>
            <Stack style={{ paddingLeft: 24 }} horizontal tokens={{ childrenGap: 6 }}>
               <Stack grow>
                  {!handler.current.loaded &&
                     <Stack horizontalAlign="center" style={{ marginLeft: 12, width: 1400 }} tokens={{ childrenGap: 32 }} grow>
                        <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>Loading Device Pool Telemetry</Text>
                        <Spinner size={SpinnerSize.large} />
                     </Stack>
                  }
                  {handler.current.loaded && !!graphOption.usage &&
                     <DevicePoolGraph
                        handler={handler.current}
                        pool={selected.pool}
                        platform={selected.platform}
                        onChange={(options: object) => {
                           setSelected({ ...selected, ...options });
                        }}
                     />
                  }
                  {handler.current.loaded && !!graphOption.health &&
                     <HealthGraph
                        handler={handler.current}
                        platform={selected.platform}
                        onChange={(options: object) => {
                           setSelected({ ...selected, ...options });
                        }}
                     />
                  }
                  {handler.current.loaded && !!graphOption.distribution &&
                     <DistributionGraph
                        handler={handler.current}
                        platform={selected.platform}
                        onChange={(options: object) => {
                           setSelected({ ...selected, ...options });
                        }}
                     />
                  }
               </Stack>
               <Stack tokens={{ childrenGap: 4 }} style={{ paddingLeft: 5, paddingRight: 24 }}>{menuItems.map((i) => renderMenuButton(i))}</Stack>
            </Stack>
         </Stack>
      </Modal>
   </Stack>
}

