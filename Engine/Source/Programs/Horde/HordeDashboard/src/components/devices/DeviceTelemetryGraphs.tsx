// Copyright Epic Games, Inc. All Rights Reserved.

import { PoolTelemetryHandler, UsagePlatformTelemetry, PlatformTelemetry } from "../../backend/DeviceTelemetryHandler"
import { getHordeStyling } from "../../styles/Styles";
import { getHumanTime, msecToElapsed } from "../../base/utilities/timeUtils";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { TelemetryData } from "../../backend/DeviceTelemetryHandler";
import moment from "moment";

import * as d3 from "d3";

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type ScalarTime = d3.ScaleTime<number, number>
type DivSelectionType = d3.Selection<HTMLDivElement, unknown, null, undefined>;

export class PoolTelemetryGraph {
   constructor(handler: PoolTelemetryHandler) {
      this.handler = handler;
      this.margin = { top: 32, right: 28, bottom: 40, left: 48 };
   }

   set(poolId: string, platformId: string) {

      this.poolId = poolId;
      this.platformId = platformId;
      this.streamIds.clear();
      this.data = undefined;

      const handler = this.handler;
      this.data = handler.getData(poolId, platformId);

   }

   draw(container: HTMLDivElement, poolId: string, platformId: string, width: number, height: number) {

      if (this.hasRendered && !this.forceRender) {
         return;
      }
      
      this.container = container;
      this.width = width;
      this.height = height;
      const margin = this.margin;
      this.clear();

      this.set(poolId, platformId);

      const data = this.data;

      if (!data) {
         return;
      }

      this.hasRendered = true;
      this.forceRender = false;
      const { modeColors } = getHordeStyling();

      const xPlot = this.xPlot = this.xPlotMethod();
      const xPlotAxis = this.xPlotMethod();

      const totalDevices = data.deviceIds.length;

      const streams = Array.from(this.streamIds).sort((a, b) => a.localeCompare(b));

      const schemaColor = d3.scaleOrdinal()
         .domain(streams)
         .range(dashboard.darktheme ? d3.schemeDark2 : d3.schemeSet2);

      const scolors = dashboard.getStatusColors();
      const colors: Record<string, string> = {
         "Available": scolors.get(StatusColor.Success)!,
         "Problem": scolors.get(StatusColor.Failure)!,
         "Maintenance": dashboard.darktheme ? "#413F3D" : "#A19F9D",
         "Disabled": "#715F5D"!
      };

      const y = d3.scaleLinear()
         .domain([0, totalDevices])
         .range([height - margin.top - margin.bottom, 0]);

      const svg = d3.select(container)
         .append("svg")
         .attr("width", width)
         .attr("height", height)
         .attr("viewBox", [0, 0, width, height] as any)
         .style("background-color", modeColors.crumbs);

      // define a clipping region
      svg.append("clipPath")
         .attr("id", this.clipId)
         .append("rect")
         .attr("x", margin.left + 1)
         .attr("y", margin.top)
         .attr("width", width - margin.left - margin.right - 241)
         .attr("height", height - margin.top);

      const zoom = d3.zoom()
         .scaleExtent([1, 12])
         .extent([[margin.left, 0], [width - margin.right - 240, height]])
         .translateExtent([[margin.left, -Infinity], [width - margin.right - 240, Infinity]])
         .on("zoom", zoomed);

      const graph = this;

      function zoomed(event: any) {
         if (graph.freezeTooltip) {
            return;
         }

         // update bottom axis drawing
         xAxis!.call(graph.xAxisMethod as any, event.transform.rescaleX(xPlotAxis));
         // update x plot mapping
         xPlot!.range([margin.left, width - margin.right - 240].map(d => event.transform.applyX(d)));
         // update graph drawing
         areaChart
            .selectAll("path")
            .attr("d", area as any)
         // update markers
         makersChart
            .selectAll("path")
            .attr("transform", plotMarker as any)
      }

      svg.call(zoom as any);

      // left axis
      svg.append("g")
         .attr("transform", `translate(${this.margin.left},${margin.bottom - 8})`)
         .call(d3.axisLeft(y).ticks(totalDevices))

      const area = d3.area()
         .x((d: any) => xPlot!(d.data.date))
         .y0((d) => y(d[0]) + 32)
         .y1((d) => y(d[1]) + 32)
         .curve(d3.curveStep);

      const keys = ["Disabled", "Maintenance", "Problem"];

      data.streamIds.forEach(sid => keys.push(data.streamNames[sid]));

      const stackedData = d3.stack()
         .keys(keys)
         (data.telemetry as any)

      const highlight = (e: any, d: any, fromArea?: boolean) => {
         this.currentTarget = fromArea ? d : undefined;
         if (this.freezeTooltip) {
            return;
         }
         d3.selectAll(".telemetryArea").style("opacity", .2)
         this.freezedHighlights.forEach((s) => {
            d3.selectAll("." + s).style("opacity", 1)
         });
         const stackTarget = data.streamSelectors[d] ?? d;
         if (!this.freezedHighlights.has(stackTarget)) {
            d3.selectAll("." + stackTarget).style("opacity", 1)
         }
      }

      const noHighlight = (e: any, d: any) => {
         this.currentTarget = undefined;
         if (this.freezeTooltip) {
            return;
         }
         if (!this.freezedHighlights.size) {
            d3.selectAll(".telemetryArea").style("opacity", 1)
         } else {
            d3.selectAll(".telemetryArea").style("opacity", .2)
            this.freezedHighlights.forEach((s) => {
               d3.selectAll("." + s).style("opacity", 1)
            });
         }
      }

      const selectHighlight = (e: any, d: any) => {
         if (this.freezeTooltip) {
            return;
         }
         const stackTarget = data.streamSelectors[d] ?? d;
         if (this.freezedHighlights.has(stackTarget)) {
            this.freezedHighlights.delete(stackTarget);
         } else {
            this.freezedHighlights.add(stackTarget);
         }
      }

      const areaChart = svg.append('g')
         .attr("clip-path", `url(#${this.clipId})`);

      areaChart
         .selectAll("telemetrylayers")
         .data(stackedData)
         .enter()
         .append("path")
         .attr("shape-rendering", "criptEdges")
         .attr("class", function (d) { return "telemetryArea " + (data.streamSelectors[d.key] ?? d.key); })
         .style("shapeRendering", "geometricPrecision")
         .style("fill", function (d) { return colors[d.key] ?? schemaColor(d.key) as any })
         .attr("d", area as any)
         .on("mouseover", (e:any) => { highlight(e, e.target.classList[1], true) })
         .on("mouseleave", (e:any) => { noHighlight(e, null) })
         .on("click", (e:any) => { selectHighlight(e, e.target.classList[1]) });

      // problem markers
      const triangleMark = d3.symbol().type(d3.symbolDiamond).size(60)();
      const plotMarker = (d) => `translate(${xPlot!(d.date)},${height - 30})`;
      const markersData = data.telemetry.map(function(d) { 
         return {
            date: d.date,
            value: d["Problem"] > 1? Math.log(d["Problem"]) / Math.log(totalDevices) : 0
         };
      });
      const makersChart = svg.append('g')
         .attr("clip-path", `url(#${this.clipId})`);
      makersChart
         .selectAll("makers")
         .data(markersData)
         .enter()
         .append("path")
         .style("fill", colors["Problem"])
         .attr("d", (d) => d.value > 0? triangleMark : null)
         .attr("opacity", (d) => d.value)
         .attr("transform", plotMarker as any)
         .attr('stroke', 'transparent').attr('stroke-width', 5)
         .on("mouseover", (e, d) => { highlight(e, "Problem", true) })
         .on("mouseleave", (e, d) => { noHighlight(e, null) })
         .on("click", (e, d) => { selectHighlight(e, "Problem") });

      // bottom axis
      const xAxis = this.xAxis = svg.append("g")
         .call(this.xAxisMethod as any, xPlot)
         .style("pointer-events", "none");

      // legend
      const size = 15
      const posx = 1160;
      const posy = margin.top;
      svg.selectAll("legendrect")
         .data(keys)
         .enter()
         .append("rect")
         .attr("class", function (d) { return "telemetryArea " + (data.streamSelectors[d] ?? d) })
         .attr("x", posx)
         .attr("y", function (d, i) { return posy + 10 + i * (size + 5 + 4) })
         .attr("width", size)
         .attr("height", size)
         .style("fill", function (d: any) { return colors[d] ?? schemaColor(d) as any })
         .on("mouseover", highlight as any)
         .on("mouseleave", noHighlight as any)
         .on("click", selectHighlight as any);

      svg.selectAll("legendlabels")
         .data(keys)
         .enter()
         .append("text")
         .attr("x", posx + size * 1.2 + 8)
         .attr("y", function (d, i) { return posy + 10 + i * (size + 5 + 4) + (size / 2) })
         .style("fill", modeColors.text)
         .text(function (d) { return d })
         .attr("text-anchor", "left")
         .attr("class", function (d) { return "telemetryArea " + (data.streamSelectors[d] ?? d) })
         .style("alignment-baseline", "middle")
         .style("font-family", "Horde Open Sans Regular")
         .style("font-size", "13px")
         .on("mouseover", highlight as any)
         .on("mouseleave", noHighlight as any)
         .on("click", selectHighlight as any);

      // tooltip
      this.tooltip = d3.select(container)
         .append("div")
         .style("visibility", "hidden")
         .style("background-color", modeColors.background)
         .style("border", "solid")
         .style("border-width", "1px")
         .style("border-radius", "3px")
         .style("border-color", dashboard.darktheme ? "#413F3D" : "#2D3F5F")
         .style("padding", "8px")
         .style("position", "absolute");

      svg.on("mousemove", (event) => this.handleMouseMove(event));
      // dblclick for double click events
      svg.on("click", (event) => {
         if (this.freezeTooltip) {
            this.freezeTooltip = false;
            if (this.currentTarget !== undefined && !this.freezedHighlights.has(this.currentTarget))
            {
               // force the tooltip to hide
               this.currentTarget = undefined;
            }
            // update the tooltip
            this.handleMouseMove(event);
         }
         if (this.currentTarget) {
            this.freezeTooltip = true;
         }
         // force refreshing hightlights
         noHighlight(event, null);
      });

      // Average load
      const avgLoad = svg.append("g")
         .attr("transform", "translate(8,6)");
      const colorScale = d3.scaleLinear<string>()
         .domain([10, 50, 80])
         .range([scolors.get(StatusColor.Success)!, scolors.get(StatusColor.Warnings)!, scolors.get(StatusColor.Failure)!]);
      const avgLoadWidth = 100;
      const load = totalDevices > 0 ? Math.floor(data.telemetry.reduce((acru, item) => acru + totalDevices - (item.available?.length ?? 0), 0) / (data.telemetry.length * totalDevices) * 100) : 0;
      avgLoad.append("rect")
         .attr("width", avgLoadWidth)
         .attr("height", 20)
         .style("fill", "none")
         .style("stroke", modeColors.text)
         .style("stroke-width", 1);
      avgLoad.append("text")
         .attr("x", avgLoadWidth + 6)
         .attr("y", 10)
         .text(`${load}% Avg Load`)
         .style("fill", modeColors.text)
         .attr("text-anchor", "left")
         .style("alignment-baseline", "middle")
         .style("font-family", "Horde Open Sans Regular")
         .style("font-size", "13px");
      avgLoad.selectAll("averageLoad")
         .data([load])
         .enter()
         .append("rect")
         .attr("width", (d) => Math.floor(d / 100 * (avgLoadWidth-1)))
         .attr("height", 19)
         .style("fill", (d) => colorScale(d));
   }

   private clear() {
      if (!!this.container) {
         d3.select(this.container).selectAll("*").remove();
      }
   }

   private closestData(x: number): PlatformTelemetry | undefined {

      const xPlot = this.xPlot;
      if (!xPlot) {
         return;
      }

      const data = this.data!.telemetry;

      let closest = data.reduce((best, value, i) => {

         let absx = Math.abs(xPlot(value.date) - x)
         if (absx < best.value) {
            return { index: i, value: absx };
         }
         else {
            return best;
         }
      }, { index: 0, value: Number.MAX_SAFE_INTEGER });

      return data[closest.index];
   }

   private updateTooltip(show: boolean, x?: number, y?: number, html?: string) {
      if (!this.tooltip) {
         return;
      }

      x = x ?? 0;
      y = y ?? 0;

      this.tooltip
         .html(html ?? "")
         .style("left", (x + 40) + "px")
         .style("top", y + "px")
         .style("font-family", "Horde Open Sans Regular")
         .style("font-size", "10px")
         .style("line-height", "16px")
         .style("shapeRendering", "crispEdges")
         .style("stroke", "none")
         .style("visibility", show ? "visible" : "hidden");
   }

   private handleMouseMove(event: any) {
      if (this.freezeTooltip) {
         return;
      }

      const target = this.currentTarget;
      if (!target) {
         this.updateTooltip(false);
         return;
      }

      const data = this.data!;
      const streamId = data.streamIds.find(sid => {
         const streamName = data.streamSelectorsReverse[target!];
         return data.streamNamesReverse[streamName] === sid;
      });

      const closest = this.closestData(_d3.pointer(event)[0]);
      if (!closest) {
         this.updateTooltip(false);
         return;
      }

      const displayTime = moment(closest.date);
      const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

      let displayTimeStr = displayTime.format('MMM Do YYYY') + ` - ${displayTime.format(format)}`;

      let targetName = "???";
      if (target) {
         targetName = data.streamSelectorsReverse[target];
         if (!targetName) {
            targetName = target;
         }
      }

      let text = `<div style="padding-left:4px;padding-right:12px;padding-bottom:12px">`;

      text += `<div style="padding-bottom:8px;padding-top:2px"><span style="font-family:Horde Open Sans Semibold;font-size:11px">${targetName}</span></div>`;

      if (streamId && closest.reserved && closest.reserved[streamId]) {
         closest.reserved[streamId].forEach(v => {
            const d = data.devices[v.deviceId]?.name ?? v.deviceId;
            text += `<div style="padding-bottom:8px">`;
            const href = `/job/${v.jobId}?step=${v.stepId}`;
            text += `<a href="${href}" target="_blank"><span style="font-family:Horde Open Sans Semibold;font-size:11px;padding-left:8px">${d}: ${v.stepName}</span></a><br/>`
            if (v.jobId && v.stepId) {
               const metrics = this.handler.getStepMetrics(v.jobId, v.stepId);
               if (metrics && (metrics.duration || metrics.numKits > 1 || metrics.numProblems > 0)) {

                  text += `<div style="padding-left: 16px">`;

                  if (metrics.numKits > 1) {
                     text += `<span style="">Devices:</span> ${metrics.numKits} <br/>`;
                  }

                  if (metrics.numProblems > 0) {
                     text += `<span style="font-family:Horde Open Sans Semibold;color:#FF0000">Problems: ${metrics.numProblems}</span> <br/>`;
                  }

                  if (metrics.duration) {
                     text += `<span style="">Duration:</span> ${metrics.duration} <br/>`;
                  }

                  text += `</div>`;
               }

               text += `</div>`;
            }
         });
      }
      else if (target === "Maintenance" && closest.maintenance) {
         closest.maintenance.forEach(d => {
            d = data.devices[d]?.name ?? d;
            text += `<span style="font-family:Horde Open Sans Semibold;font-size:11px;padding-left:8px">${d}</span><br/>`
         });
         text += `<br/>`;
      }
      else if (target === "Problem") {
         if (!closest.problem?.length) {
            this.updateTooltip(false);
            return;
         }

         const problems = this.handler.getProblemSteps(closest.problem, closest.date);

         closest.problem.sort((a,b) => {
            if (!problems[a] || !problems[b]) return 0;
            // group problems by job step name
            const streamA = !!problems[a].streamId ? data.streamNames[problems[a].streamId] : "Unknown stream";
            const streamB = !!problems[b].streamId ? data.streamNames[problems[b].streamId] : "Unknown stream";
            const nameA = `${streamA} - ${(problems[a].stepName ?? problems[a].jobName)}`;
            const nameB = `${streamB} - ${(problems[b].stepName ?? problems[b].jobName)}`;
            if (nameA < nameB) return -1;
            if (nameA > nameB) return 1;
            return 0;
         }).forEach(d => {
            const problem = problems[d];
            d = data.devices[d]?.name ?? d;

            text += `<span style="font-family:Horde Open Sans Semibold;font-size:11px;padding-left:8px">${d}</span><br/>`

            if (problem && problem.streamId && problem.jobName) {

               const name = problem.stepName ?? problem.jobName;
               let desc = `${data.streamNames[problem.streamId]} - ${name}`;

               const href = `/job/${problem.jobId}?step=${problem.stepId}`;

               text += `<a href="${href}" target="_blank"><div style="padding-bottom:8px;padding-left:8px"><span style="">${desc}</span></div></a>`

            }

         });
      }
      else if (target === "Disabled" && closest.disabled) {
         closest.disabled.forEach(d => {
            d = data.devices[d]?.name ?? d;
            text += `<span style="font-family:Horde Open Sans Semibold;font-size:11px;padding-left:8px">${d}</span><br/>`
         });

         text += `<br/>`;
      }

      text += `${displayTimeStr}<br/>`;

      text += "</div>";

      this.updateTooltip(true, _d3.pointer(event)[0], _d3.pointer(event)[1], text);

   }

   private get xAxisMethod() {
      return (g: SelectionType, x: ScalarTime) => {
         const tickSet: Set<string> = new Set();

         g.attr("transform", `translate(0,${this.height - this.margin.bottom})`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "11px")
            .call(d3.axisBottom(x)
               .ticks(8)
               .tickFormat(d => {
                  const v = (d as Date).toDateString();
                  if (!tickSet.has(v)) {
                     tickSet.add(v)
                     return getHumanTime(d as Date)
                  }

                  return (d as Date).toLocaleString(undefined, { hour: 'numeric', minute: "numeric", hour12: true })
               })
               .tickSizeOuter(0))
      }
   }

   private xPlot?: ScalarTime;

   private xPlotMethod() {

      const data = this.data!.telemetry;

      return d3.scaleTime()
         .domain(d3.extent(data, (d => d.date)) as any)
         .range([this.margin.left, this.width - this.margin.right - 240])
   }

   handler: PoolTelemetryHandler;

   container?: HTMLDivElement;
   private margin: { top: number, right: number, bottom: number, left: number }
   private width = 0;
   private height = 0;

   private data?: TelemetryData;
   private poolId = "";
   private platformId = "";
   private streamIds = new Set<string>();

   private freezeTooltip = false;
   private freezedHighlights = new Set<string>();

   private xAxis?: SelectionType;
   private tooltip?: DivSelectionType;
   private readonly clipId = "pool_history_clip_path";

   private currentTarget?: string;

   private hasRendered = false;
   forceRender = false;

   extent?: any;
}

export class PoolDistributionGraph {
   constructor(handler: PoolTelemetryHandler) {
      this.handler = handler;
      this.margin = { top: 0, right: 0, bottom: 0, left: 0 };
   }

   set(platformId: string, startDate: Date | undefined, endDate: Date | undefined) {
      this.platformId = platformId;
      this.stepIds.clear();
      this.data = this.handler.getSteps(platformId, startDate, endDate);
      this.data.forEach((s) => { this.stepIds.add(s.stepName) });
   }

   draw(container: HTMLDivElement, platformId: string, width: number, height: number, radius: number, startDate: Date | undefined, endDate: Date | undefined) {

      if (this.hasRendered && !this.forceRender) {
         return;
      }
      
      this.container = container;
      this.width = width;
      this.height = height;
      const margin = this.margin;
      this.clear();

      this.set(platformId, startDate, endDate);

      const data = this.data;

      if (!data) {
         return;
      }

      this.hasRendered = true;
      this.forceRender = false;

      const root = this.modelData();
      if (!root) {
         return;
      }
      d3.partition().size([2 * Math.PI, radius])(root);

      const { modeColors } = getHordeStyling();

      const stepIds = Array.from(this.stepIds).sort((a, b) => a.localeCompare(b));

      const schemaColor = d3.scaleOrdinal()
         .domain(stepIds)
         .range(dashboard.darktheme ? d3.schemeDark2 : d3.schemeSet2);

      const svg = d3.select(container)
         .append("svg")
         .attr("width", width)
         .attr("height", height)
         .attr("viewBox", [0, 0, width, height] as any)
         .style("background-color", modeColors.crumbs);

      // define a clipping region
      svg.append("clipPath")
         .attr("id", this.clipId)
         .append("rect")
         .attr("x", margin.left + 1)
         .attr("y", margin.top)
         .attr("width", width - margin.left - margin.right)
         .attr("height", height - margin.top);

      // arc generator
      const arc = d3.arc()
         .cornerRadius(2)
         .startAngle((d: any) => d.x0)
         .endAngle((d: any) => d.x1)
         .padAngle(0.025)
         .padRadius(radius / 4)
         .innerRadius((d: any) => d.y0)
         .outerRadius((d: any) => d.y1 - 2);

      // draw the pieChart
      const center = [width / 2, height / 2];
      const pieChart = svg.append('g')
         .attr("clip-path", `url(#${this.clipId})`)
         .append('g')
         .attr("transform", `translate(${center})`);

      const arcOpacity = (d) => {
         const family = d.ancestors().reverse()[1]?.data.name;
         if (!family || !d.depth) return 0;
         if (d.depth == 1) return 1;
         if (d.depth <= this.displayDepth
               && (!!this.displayFamily && family == this.displayFamily)) return 1;
         if ((!!this.selectedFamily && family == this.selectedFamily)
               || (!!this.displayFamily && family == this.displayFamily)) return 0.3;
         return 0;
      }

      const labelOpacity = (d) => {
         const family = d.ancestors().reverse()[1]?.data.name;
         if (!family || d.depth != 1) return 0;
         const isNarrow = (d.y0 + d.y1) /2 * (d.x1 - d.x0) < 25;
         if (d.depth == this.displayDepth && ((!this.displayFamily && !isNarrow) || family == this.selectedFamily)) return 1;
         if (!isNarrow || family == this.selectedFamily) return 0.5;
         return 0;
      }

      const arcPointerEvent = (d) => {
         return d.depth <= this.displayDepth ? "auto" : "none";
      }

      const highlight = (e: any, d: any) => {
         if (!d.depth) return;
         this.displayDepth = d.depth + 1;
         this.displayFamily = d.ancestors().reverse()[1]?.data.name;
         d3.selectAll(".arcStep")
            .attr("opacity", arcOpacity)
            .style("pointer-events", arcPointerEvent);
         d3.selectAll(".arcStepLabel")
            .attr("opacity", labelOpacity);
         d3.selectAll(`.${d.data.id}`)
            .attr("opacity", 1)
            .raise();
      }

      const noHighlight = (e: any, d: any) => {
         this.displayDepth = this.selectedDepth;
         this.displayFamily = this.selectedFamily;
         d3.selectAll(".arcStep")
            .attr("opacity", arcOpacity)
            .style("pointer-events", arcPointerEvent);
         d3.selectAll(".arcStepLabel")
            .attr("opacity", labelOpacity);
         if (!!this.selectedItem) {
            d3.selectAll(`.${this.selectedItem}`)
               .attr("opacity", 1)
               .raise();
         }
      }

      const selectHighlight = (e: any, d: any) => {
         if (!d.depth) {
            this.selectedDepth = 1;
            this.selectedFamily = "";
            this.selectedItem = "";
         } else {
            this.selectedDepth = d.depth + 1;
            this.selectedFamily = d.ancestors().reverse()[1]?.data.name;
            this.selectedItem = d.data.id;
         }
         noHighlight(e, d);
      }

      const pieceColor = (d) => {
         const pColor = d3.color(schemaColor(d.ancestors().reverse()[1]?.data.name) as any);
         return dashboard.darktheme ? pColor?.brighter(d.depth / 2) : pColor?.darker(d.depth / 2)
      }

      pieChart
         .selectAll("steps")
         .data(root.descendants())
         .enter()
         .append("path")
         .attr("d", arc as any)
         .attr("class", "arcStep")
         .attr("fill", (d: any) => pieceColor(d) as any)
         .attr('stroke', 'transparent').attr('stroke-width', 3)
         .attr("opacity", arcOpacity)
         .style("pointer-events", arcPointerEvent)
         .on("mouseover", highlight as any)
         .on("mouseleave", noHighlight as any)
         .on("click", selectHighlight as any);

      // center label
      const centerLabel = svg.append('g')
         .attr("transform", `translate(${center})`)
         .style("pointer-events", "none");

      centerLabel.append("text")
         .attr("dy", "0em")
         .attr("fill", modeColors.text)
         .style("text-anchor", "middle")
         .text((root.data as any).name)
      centerLabel.append("text")
         .attr("dy", "1em")
         .attr("fill", modeColors.text)
         .style("text-anchor", "middle")
         .text(msecToElapsed((root.data as any).durationMS, true, false));

      // labels
      const labels = svg.append('g')
         .attr("clip-path", `url(#${this.clipId})`)
         .append('g')
         .attr("transform", `translate(${center})`)
         .style("pointer-events", "none");

      const addAlongVector = (vector, factor) => {
         const [x, y] = vector;
         const length = Math.sqrt(x * x + y * y);
         const [xn, yn] = [x / length, y / length];
         return [x + xn * factor, y + yn * factor];
      }

      const labelPosition = (d, offsetx) => {
         const textAnchor = addAlongVector(arc.centroid(d), radius * 0.2);
         if (!!offsetx){
            textAnchor[0] += offsetx * (textAnchor[0] >= 0 ? 1 : -1);
         }
         return textAnchor;
      }

      labels.selectAll("lines")
         .data(root.descendants())
         .enter()
         .append("polyline")
         .attr("class", (d: any) => `arcStepLabel ${d.data.id}`)
         .style("fill", "none")
         .style("stroke", modeColors.background).style("stroke-width", 2)
         .attr("points", (d: any) => {
            const textAnchor = labelPosition(d, 0);
            textAnchor[1] += 4;
            const textUnderline = labelPosition(d, 40);
            textUnderline[1] += 4;
            const startAnchor = addAlongVector(arc.centroid(d), radius * 0.05)
            return ([startAnchor, textAnchor, textUnderline]).toString();
         })
         .attr("opacity", labelOpacity)
         .clone(true)
         .style("stroke", modeColors.text)
         .style("stroke-width", 1);

      labels.selectAll("durations")
         .data(root.descendants())
         .enter()
         .append("text")
         .attr("class", (d: any) => `arcStepLabel ${d.data.id}`)
         .text((d: any) => msecToElapsed((d.data as any).durationMS, true, false))
         .style("fill", modeColors.text)
         .attr("font-weight", "bold")
         .attr("font-size", "smaller")
         .style("stroke", modeColors.background).style("stroke-width", 3)
         .attr("transform", (d: any) => `translate(${arc.centroid(d)})`)
         .style("text-anchor", "middle")
         .attr("opacity", labelOpacity)
         .clone(true)
         .style("fill", modeColors.text)
         .style("stroke", "none");

      labels.selectAll("labels")
         .data(root.descendants())
         .enter()
         .append("text")
         .attr("class", (d: any) => `arcStepLabel ${d.data.id}`)
         .text((d: any) => d.data.name)
         .style("fill", modeColors.text)
         .style("stroke", modeColors.background).style("stroke-width", 3)
         .attr("transform", (d: any) => `translate(${labelPosition(d, 5)})`)
         .style("text-anchor", (d: any) => arc.centroid(d)[0] >= 0 ? "start" : "end")
         .attr("opacity", labelOpacity)
         .clone(true)
         .style("fill", modeColors.text)
         .style("stroke", "none");

      // reset selection highlight
      pieChart.call(() => { noHighlight(null, null) });
   }

   private clear() {
      if (!!this.container) {
         d3.select(this.container).selectAll("*").remove();
      }
   }

   private modelData() {
      if (!this.data) return;

      const items: Map<string, { id: string, name: string, type: string, parentId: string, durationMS: number }> = new Map();
      items.set(this.platformId, {id: this.platformId, name: this.handler.platforms.get(this.platformId)!.name, type: "Platform", parentId: "", durationMS: 0});
      let totalTimeMS = 0;
      const invalidClassCharacters = /[^_a-z0-9-]/gi;

      this.data.forEach(s => {
         totalTimeMS += s.durationMS;
         // step layer
         const stepKey = s.stepName.replaceAll(invalidClassCharacters, "_");
         if (!items.has(stepKey)) {
            items.set(stepKey, { id: stepKey, name: s.stepName, type: "Step", parentId: this.platformId, durationMS: 0 })
         }
         // stream layer
         const streamKey = stepKey + s.stream?.replaceAll(invalidClassCharacters, "_");
         if (!items.has(streamKey)) {
            items.set(streamKey, { id: streamKey, name: s.stream, type: "Stream", parentId: stepKey, durationMS: 0 })
         }
         // pool layer
         const poolKey = streamKey + s.pool?.replaceAll(invalidClassCharacters, "_");
         if (!items.has(poolKey)) {
            items.set(poolKey, { id: poolKey, name: s.pool, type: "Pool", parentId: streamKey, durationMS: s.durationMS })
         } else {
            const item = items.get(stepKey)!;
            item.durationMS += s.durationMS;
         }
      });

      const rootItem = items.get(this.platformId)!;

      const root = d3.stratify()
         .id((d: any) => d.id)
         .parentId((d: any) => d.parentId)
         (Array.from(items.values()));

      // compute time layout
      root.sum((d: any) => d.durationMS);
      root.children = root.children?.filter((d: any) => d.value / totalTimeMS > 0.02);
      root.sum((d: any) => d.durationMS);
      root.each((d: any) => { d.data.durationMS = d.value });
      root.sort((a, b) => b.value! - a.value!);

      return root
   }


   handler: PoolTelemetryHandler;

   container?: HTMLDivElement;
   private margin: { top: number, right: number, bottom: number, left: number }
   private width = 0;
   private height = 0;

   private data?: Map<string, any>;
   private telemetry?: UsagePlatformTelemetry[];
   private platformId = "";
   private stepIds = new Set<string>();

   private displayDepth = 1;
   private displayFamily = "";
   private selectedDepth = 1;
   private selectedFamily = "";
   private selectedItem = "";

   private readonly clipId = "pool_distribution_clip_path";

   private hasRendered = false;
   forceRender = false;
}

export class PoolDistributionTimelineGraph {
   constructor(handler: PoolTelemetryHandler) {
      this.handler = handler;
      this.margin = { top: 10, right: 30, bottom: 20, left: 30 };
   }

   set(platformId: string) {
      this.platformId = platformId;
      this.data = this.handler.getPlatformTelemetry(platformId);
   }

   draw(container: HTMLDivElement, platformId: string, width: number, height: number, onDateChange: () => void) {

      if (this.hasRendered && !this.forceRender) {
         return;
      }
      
      this.container = container;
      this.width = width;
      this.height = height;
      const margin = this.margin;
      this.clear();

      this.set(platformId);

      const data = this.data;

      if (!data) {
         return;
      }

      this.hasRendered = true;
      this.forceRender = false;

      const { modeColors } = getHordeStyling();

      const maxReservations = data.reduce((a, d) => Math.max(a, d.reservations), 1);

      const svg = d3.select(container)
         .append("svg")
         .attr("width", width)
         .attr("height", height)
         .attr("viewBox", [0, 0, width, height] as any)
         .style("background-color", modeColors.crumbs);

      // plots
      const xPlot = d3.scaleTime()
         .domain(d3.extent(data, (d => d.date)) as any)
         .range([this.margin.left, this.width - this.margin.right]);

      const yPlot = d3.scaleLinear()
         .domain([0, maxReservations])
         .range([height - margin.bottom, margin.top]);

      // bottom axis
      const xAxis = svg.append("g")
         .call(this.xAxisMethod as any, xPlot)
         .style("pointer-events", "none");

      // graph
      const area = d3.area()
         .x((d: any) => xPlot(d.date))
         .y0(yPlot(0))
         .y1((d: any) => yPlot(d.reservations));

      const areaColor = dashboard.darktheme ?
         (d3.color(modeColors.text) as any)?.darker(0.5)
         : (d3.color(modeColors.text) as any)?.brighter(0.5)

      svg.append("path")
         .datum(data)
         .attr("fill", areaColor)
         .attr("stroke", modeColors.text).attr("stroke-width", 1.5)
         .attr("opacity", 0.4)
         .attr("d", area as any)
         .style("pointer-events", "none");

      // dragable timeline
      const drag = (handle: any, update: (value: number) => void) => {
         return d3.drag()
            .on("start", (e: any) => { handle.attr("cursor", "grabbing") })
            .on("end", (e: any) => { handle.attr("cursor", "grab") })
            .on("drag", (e: any) => {
               const x = Math.min(Math.max(e.x, xPlot.range()[0]), xPlot.range()[1]);
               handle.attr("transform", `translate(${x},${yPlot(0)})`);
               // update date
               update(x);
               // update time stamp
               handle.selectAll("text").text(toLocaleTimeString(xPlot.invert(x)))
               // update area selected
               updateAreaSelected();
               // trigger callback
               onDateChange();
            });
      }

      [this.startDate, this.endDate] = xPlot.domain();

      const areaSelected = svg.append("g")
         .style("pointer-events", "none");

      const updateAreaSelected = () => {
         areaSelected.selectAll("*").remove();
         areaSelected.append("rect")
            .attr("fill", modeColors.text)
            .attr("opacity", 0.1)
            .attr("x", Math.min(xPlot(this.startDate!), xPlot(this.endDate!)))
            .attr("y", yPlot(maxReservations))
            .attr("width", Math.abs(xPlot(this.endDate!) - xPlot(this.startDate!)))
            .attr("height", yPlot(0) - yPlot(maxReservations));
         areaSelected.append("line")
            .style("stroke", modeColors.text).style("stroke-width", 3)
            .attr("x1", xPlot(this.startDate!))
            .attr("x2", xPlot(this.endDate!))
            .attr("y1", yPlot(0))
            .attr("y2", yPlot(0));
      }

      areaSelected.call(updateAreaSelected as any);

      const toLocaleTimeString = (d: Date) => d.toLocaleTimeString(undefined, { hour: 'numeric', minute: "numeric", hour12: true });

      const constructHandle = (handle, dateValue) => {
         handle.append("circle")
            .attr("r", 6)
            .attr("fill", (d3.color(modeColors.text) as any).darker(0.6))
            .attr("stroke", modeColors.text).attr("stroke-width", 1.5);
         handle.append("text")
            .text(toLocaleTimeString(dateValue))
            .attr("text-anchor", "middle")
            .attr("y", -11)
            .style("fill", modeColors.text)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "11px")
            .style("pointer-events", "none");
      }

      const startHandle = svg.append("g")
         .attr("cursor", "grab")
         .attr("transform", `translate(${xPlot(this.startDate!)},${yPlot(0)})`);
      startHandle.call(
            drag(startHandle, (v) => { this.startDate = xPlot.invert(v) }) as any
         );
      constructHandle(startHandle, this.startDate);

      const endHandle = svg.append("g")
         .attr("cursor", "grab")
         .attr("transform", `translate(${xPlot(this.endDate!)},${yPlot(0)})`);
      endHandle.call(
            drag(endHandle, (v) => { this.endDate = xPlot.invert(v) }) as any
         );
      constructHandle(endHandle, this.endDate);
   }

   private clear() {
      if (!!this.container) {
         d3.select(this.container).selectAll("*").remove();
      }
   }

   private get xAxisMethod() {
      return (g: SelectionType, x: ScalarTime) => {
         const tickSet: Set<string> = new Set();

         g.attr("transform", `translate(0,${this.height - this.margin.bottom})`)
            .style("font-family", "Horde Open Sans Regular")
            .style("font-size", "11px")
            .call(d3.axisBottom(x)
               .ticks(8)
               .tickFormat(d => {
                  const v = (d as Date).toDateString();
                  if (!tickSet.has(v)) {
                     tickSet.add(v)
                     return getHumanTime(d as Date)
                  }

                  return (d as Date).toLocaleString(undefined, { hour: 'numeric', minute: "numeric", hour12: true })
               })
               .tickSizeOuter(0))
      }
   }

   handler: PoolTelemetryHandler;

   container?: HTMLDivElement;
   private margin: { top: number, right: number, bottom: number, left: number }
   private width = 0;
   private height = 0;

   private data?: UsagePlatformTelemetry[];
   private platformId = "";

   startDate?: Date;
   endDate?: Date;

   private hasRendered = false;
   forceRender = false;
}
