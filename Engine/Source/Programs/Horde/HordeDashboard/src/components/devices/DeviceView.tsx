// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, Dropdown, FocusZone, FocusZoneDirection, IColumn, Icon, IconButton, IGroup, GroupHeader, Label, Link as FluentLink, mergeStyleSets, Modal, Pivot, PivotItem, PrimaryButton, ScrollablePane, ScrollbarVisibility, SelectionMode, Spinner, SpinnerSize, Stack, Text, TextField, IDetailsGroupDividerProps, IGroupHeaderProps } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { observable, action, makeObservable } from "mobx";
import moment from "moment";
import React, { useEffect, useState } from "react";
import { Link, useNavigate, useSearchParams } from "react-router-dom";
import backend from "../../backend";
import { DashboardPreference, DevicePoolType, GetDeviceResponse } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { useWindowSize } from "../../base/utilities/hooks";
import { getNiceTime } from "../../base/utilities/timeUtils";
import { Breadcrumbs } from "../Breadcrumbs";
import { DeviceHandler, DeviceStatus, DeviceStatusLabels, DeviceHealthStatus } from "../../backend/DeviceHandler";
import { DeviceEditor } from "./DeviceEditor";
import { DeviceInfoModal } from "./DeviceInfoView";
import { deviceHealthBadge, DeviceGroupHeader } from "./DeviceHealthWidgets";
import { DevicePoolTelemetryModal } from "./DevicePoolTelemetry";
import { TopNav } from "../TopNav";
import { getHordeStyling } from "../../styles/Styles";
import { getHordeTheme } from "../../styles/theme";
import { isBright } from "horde/base/utilities/colors";

const handler = new DeviceHandler();

type DeviceItem = {
   device: GetDeviceResponse;
   // Whether user has this device checked out
   userCheckout?: boolean;
   status?: string;
}

let groups: IGroup[] = [];

type DevicesSearchState = {
   filterPools?: Array<string>;
   filterPlatforms?: Array<string>;
   pivotKey?: string;
   filterString?: string;
   historyItem?: string;
}

class LocalState {

   @observable searchUpdated: number = 0;

   constructor() {
      makeObservable(this);
   }

   @action
   setSearchUpdated() {
      this.searchUpdated++;
   }


   searchState: DevicesSearchState = {};
   search: URLSearchParams = new URLSearchParams(window.location.search);

   updateSearch(): boolean {
      
      const state = { ...this.searchState } as DevicesSearchState;

      state.filterPools = state.filterPools?.sort((a, b) => a.localeCompare(b));

      const search = new URLSearchParams();
      const csearch = this.search.toString();

      state.filterPools?.forEach(f => {
         if (f) {
            search.append("pool", f);
         }
      });

      state.filterPlatforms?.forEach(f => {
         if (f) {
            search.append("platform", f);
         }
      });

      if (state.pivotKey?.length) {
         search.append("pivotKey", state.pivotKey);
      }

      if (state.historyItem?.length) {
         search.append("history-item", state.historyItem);
      }

      if (state.filterString?.length) {
         search.append("filter", state.filterString);
      }

      if (search.toString() !== csearch) {
         this.search = search;
         this.setSearchUpdated();
         return true;
      }

      return false;
   }

   resetState() {
      this.initialized = false;
      this.search = new URLSearchParams();
      this.searchState = {};
   }
   
   private stateFromSearch() {

      const state: DevicesSearchState = {};

      const pools = this.search.getAll("pool") ?? undefined;
      const platforms = this.search.getAll("platform") ?? undefined;
      const pivot = this.search.get("pivotKey") ?? undefined;
      const filter = this.search.get("filter") ?? undefined;
      const historyItem = this.search.get("history-item") ?? undefined;

      state.filterPools = pools?.sort((a, b) => a.localeCompare(b));
      state.filterPlatforms = platforms?.sort((a, b) => a.localeCompare(b));
      state.pivotKey = pivot;
      state.filterString = filter;
      state.historyItem = historyItem;

      this.searchState = state;

      if (state.filterPools?.length) {
         this.searchState.filterPools = state.filterPools;
      }

      if (state.filterPlatforms?.length) {
         this.searchState.filterPlatforms = state.filterPlatforms;
      }

      if (state.pivotKey?.length) {
         this.searchState.pivotKey = state.pivotKey;
      }

      if (state.filterString?.length) {
         this.searchState.filterString = state.filterString;
      }

      return state;
   }

   @action
   setFilterWithoutUrlUpdate(filter: string | undefined) {
      if (filter) {
         this.searchState.filterString = filter;
      }
   }

   @action
   setFilter(filter?: string) {
      this.searchState.filterString = filter;
      this.updateSearch();
   }

   @action
   setPivotKey(pivotKey: string) {
      this.searchState.filterPlatforms = undefined;
      this.searchState.filterPools = undefined;
      this.searchState.filterString = undefined;
      this.searchState.pivotKey = pivotKey;
      this.updateSearch();
   }

   @action
   setPoolFilters(filter: Set<string>) {
      const pools = Array.from(filter);
      this.searchState.filterPools = pools.length ? pools : undefined;
      this.updateSearch();
   }

   @action
   setPlatformFilters(filter: Set<string>) {
      const platforms = Array.from(filter);
      this.searchState.filterPlatforms = platforms.length ? platforms : undefined;
      this.updateSearch();
   }

   @action
   setHistoryItem(item: string) {
      this.searchState.historyItem = item;
      this.updateSearch();
   }

   @action
   resetHistoryItem() {
      this.searchState.historyItem = undefined;
      this.updateSearch();
   }

   initialize() 
   {
      if (!this.initialized) {
         this.initialized = true;
         this.stateFromSearch();
      }
   }

   private initialized = false;
}

const dropDownStyle: any = () => {

   return {
      dropdown: {},
      callout: {
         selectors: {
            ".ms-Callout-main": {
               padding: "4px 4px 12px 12px",
               overflow: "hidden"
            }
         }
      },
      dropdownItemHeader: { fontSize: 12 },
      dropdownOptionText: { fontSize: 12 },
      dropdownItem: {
         minHeight: 28, lineHeight: 28
      },
      dropdownItemSelected: {
         minHeight: 28, lineHeight: 28, backgroundColor: "inherit"
      }
   }
}


const customStyles = mergeStyleSets({
   raised: {
      boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)",
      padding: "25px 30px 25px 30px"
   },
   details: {
      selectors: {
         '.ms-GroupHeader-title': {
            cursor: "default"
         },
         '.ms-DetailsRow': {
            animation: "none",
            background: "unset"
         }
      },
   }
});

const pivotKeyAutomation = "pivot-key-automation";
const pivotKeyShared = "pivot-key-shared";
const localState = new LocalState();

const SearchUpdate: React.FC = observer(() => {

   const [, setSearchParams] = useSearchParams();

   const csearch = localState.search.toString();

   useEffect(() => {
      setSearchParams(csearch, { replace: true });
   }, [csearch, setSearchParams])

   // subscribe
   if (localState.searchUpdated) { }

   return null;
});

const DevicePanel: React.FC = observer(() => {

   const [editState, setEditState] = useState<{ shown?: boolean, infoShown?: boolean, editNote?: boolean, device?: DeviceItem | undefined }>({});
   const [pivotState, setPivotState] = useState<{ key: string, poolFilter: Set<string>, platformFilter: Set<string> }>({ key: localState?.searchState?.pivotKey ?? pivotKeyShared, platformFilter: new Set(localState?.searchState?.filterPlatforms) ?? [], poolFilter: new Set(localState?.searchState?.filterPools ?? []) });
   const [checkoutState, setCheckoutState] = useState<{ checkoutId?: string, checkinId?: string, showConfirm?: "in" | "out" | "error" }>({});
   const [telemState, setTelemState] = useState(false);
   
   const navigate = useNavigate();

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { hordeClasses } = getHordeStyling();
   const hordeTheme = getHordeTheme();

   handler.subscribe();

   let message = "";
   let linkText = "device service configuration.";
   let link = "/docs/Config/Devices.md";
   if (handler.loaded) {

      if (!handler.platforms.size) {
         message = "There are no device platforms configured, please see";
      } else if (!handler.pools.size) {
         message = "There are no device pools configured, please see";
      }
   }

   if (!handler.loaded) {
      return <Stack horizontalAlign="center">
         <Spinner size={SpinnerSize.large} />
      </Stack>
   }

   if (message) {
      return <Stack horizontal tokens={{ childrenGap: 6 }} horizontalAlign="center">
         <Text variant="mediumPlus">{message}</Text>
         {!!linkText && !!link && <a href={link} style={{ fontSize: "18px", "cursor": "pointer" }} onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(link) }}>{linkText}</a>}
      </Stack>
   }


   const warningColorPref = dashboard.getPreference(DashboardPreference.ColorWarning);
   const errorColorPref = dashboard.getPreference(DashboardPreference.ColorError);
   const successColorPref = dashboard.getPreference(DashboardPreference.ColorSuccess);
   const runningColorPref = dashboard.getPreference(DashboardPreference.ColorRunning);

   const StatusColors = new Map<DeviceStatus, string>([
      [DeviceStatus.Available, successColorPref ? successColorPref : "#52C705"],
      [DeviceStatus.Problem, errorColorPref ? errorColorPref : "#DE4522"],
      [DeviceStatus.Reserved, warningColorPref ? warningColorPref : "#00BCF2"],
      [DeviceStatus.Disabled, dashboard.darktheme ? "#D3D2D1" : "#F3F2F1"],
      [DeviceStatus.Maintenance, runningColorPref ? runningColorPref : "#0078D4"]
   ]);


   const InfoModal: React.FC = () => {

      const checkout = !!checkoutState.checkoutId;


      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 840, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 24 }} styles={{ root: { padding: 8 } }}>
            <Stack grow verticalAlign="center">
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{`Checking ${checkout ? "out" : "in"} Device`}</Text>
            </Stack>
            <Stack horizontalAlign="center">
               <Text variant="mediumPlus">{`The device is being checked ${checkout ? "out" : "in"}, one moment please.`}</Text>
            </Stack>
            <Stack verticalAlign="center" style={{ paddingBottom: 32 }}>
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>
   }

   const automationTab = pivotState.key === pivotKeyAutomation;

   const platforms = handler.platforms;

   const poolWidth = automationTab ? 80 : 140;
   const detailsWidth = automationTab ? 160 : 100;

   const columns: IColumn[] = [
      { key: 'column_platform', name: 'Platform', fieldName: 'platformId', minWidth: 72, maxWidth: 72, isResizable: false },
      { key: 'column_pool', name: 'Pool', fieldName: 'poolId', minWidth: poolWidth, maxWidth: poolWidth, isResizable: false },
      { key: 'column_model', name: 'Model', fieldName: 'modelId', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'column_name', name: 'Name', fieldName: 'name', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_address', name: 'Address', fieldName: 'address', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_status', name: 'Status', fieldName: 'status', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_details', name: 'Details', fieldName: 'details', minWidth: detailsWidth, maxWidth: detailsWidth, isResizable: false },
      { key: 'column_notes', name: 'Notes', fieldName: 'notes', minWidth: 240, maxWidth: 240, isResizable: false },
   ];

   columns.forEach(c => {
      c.isPadded = false;
      c.onRender = ((item: DeviceItem, index, column) => {
         return <Stack verticalFill={true} verticalAlign="center">
            <Text>{(item.device as any)[column?.fieldName!]}</Text>
         </Stack>

      });
   });

   let column = columns.find(c => c.name === "Status")!
   column.onRender = ((item: DeviceItem, index, column) => {

      const write = handler.getDeviceWriteAccess(item.device);

      if (!write) {
         return null;
      }

      const device = item.device;
      const status = handler.getDeviceStatus(device);

      let backgroundColor = StatusColors.get(status)!
      let text = DeviceStatusLabels.get(status)!;

      if (device.checkedOutByUserId) {
         if (status === DeviceStatus.Available) {
            text = "Checked Out";
            backgroundColor = StatusColors.get(DeviceStatus.Reserved)!;
         }
      }

      let color = status === DeviceStatus.Disabled ? "#777777 !important" : isBright(backgroundColor) ? "#2D3F5F" : "#FFFFFF";
      const hoveredPressedStyle = { backgroundColor: backgroundColor, border: '0px', filter: isBright(backgroundColor) ? "brightness(0.9)" : "brightness(1.2)" }

      if (item.device.cleanStartTime && !item.device.cleanFinishTime) {
         text = "Cleaning";
      }

      // @todo: support deleting reservations
      return <Stack verticalFill={true} verticalAlign="center" horizontal>
         <PrimaryButton 
            styles={{ root: { border: "0px", width: 84, height: 24, fontSize: 12, fontFamily: "Horde Open Sans SemiBold !important", selectors: { ".ms-Button-label": { color: color } }, backgroundColor: backgroundColor }, rootHovered: hoveredPressedStyle, rootPressed: hoveredPressedStyle }}
            text={text}
            onClick={(ev) => {
               ev.stopPropagation(); ev.preventDefault();
               setEditState({ shown: true, device: item })
            }}
         />
      </Stack>

   })

   column = columns.find(c => c.name === "Platform")!;
   column.onRender = ((item: DeviceItem, index, column) => {

      const platform = handler.platforms.get(item.device.platformId);

      return <Stack verticalFill={true} verticalAlign="center">
         <Text>{platform?.name ?? "???"}</Text>
      </Stack>

   })

   column = columns.find(c => c.name === "Pool")!;
   column.onRender = ((item: DeviceItem, index, column) => {

      const pool = handler.pools.get(item.device.poolId);

      return <Stack verticalFill={true} verticalAlign="center">
         <Text>{pool?.name ?? "???"}</Text>
      </Stack>

   })

   column = columns.find(c => c.name === "Model")!;
   column.onRender = ((item: DeviceItem, index, column) => {


      return <Stack verticalFill={true} verticalAlign="center">
         <Text>{item.device.modelId ?? "Base"}</Text>
      </Stack>

   })

   column = columns.find(c => c.name === "Details")!
   column.onRender = ((item: DeviceItem, index, column) => {

      const device = item.device;

      // Check whether shared kit
      if (!automationTab) {

         const status = handler.getDeviceStatus(item.device);

         const write = handler.getDeviceWriteAccess(item.device);

         const checkOutDisabled = !write || status !== DeviceStatus.Available;

         if (device.id === checkoutState.checkoutId) {
            return <Stack verticalFill={true} verticalAlign="center" horizontalAlign="start"><Spinner size={SpinnerSize.small} /></Stack>;
         }

         if (!device.checkedOutByUserId) {

            return <Stack verticalFill={true} verticalAlign="center">
               {!checkOutDisabled && <PrimaryButton styles={{ root: { border: "0px", width: 90, height: 24, fontSize: 12, fontFamily: "Horde Open Sans SemiBold !important" } }} text="Check Out"
                  onClick={async (ev) => {
                     ev.stopPropagation(); ev.preventDefault();
                     setCheckoutState({ checkoutId: device.id });
                     await backend.checkoutDevice(device.id, true).then(() => {
                        handler.updating = false;
                        handler.update().then(() => {
                           setCheckoutState({ showConfirm: "out" })
                        }).catch((reason) => {
                           console.error(reason);
                           setCheckoutState({ showConfirm: "error" });
                        })
                     });
                  }}
               />}
            </Stack>
         }

         const user = handler.users.get(device.checkedOutByUserId);

         if (!user) {
            return <Text>Error: Missing user {device.checkedOutByUserId} for checkout</Text>
         }

         if (user.id === dashboard.userId) {
            return <Stack verticalFill={true} verticalAlign="center">
               <PrimaryButton styles={{ root: { border: "0px", width: 90, height: 24, fontSize: 12, fontFamily: "Horde Open Sans SemiBold !important" } }} text="Check In"
                  onClick={async (ev) => {
                     ev.stopPropagation(); ev.preventDefault();
                     setCheckoutState({ checkinId: device.id });
                     await backend.checkoutDevice(device.id, false).then(() => {
                        handler.updating = false;
                        handler.update().then(() => setCheckoutState(
                           { showConfirm: handler.getUserDeviceCheckouts(dashboard.userId).length > 0 ? "in" : undefined }))
                     }).catch((reason) => {
                        console.error(reason);
                        setCheckoutState({ showConfirm: "error" });
                     });
                  }}
               />
            </Stack>
         }

         return <Stack verticalFill={true} verticalAlign="center"><Text variant="tiny">{`${user.name}`}</Text></Stack>

      }

      // Automation Kit

      const r = handler.getReservation(device);
      let health = automationTab? handler.getDeviceHealth(device.id) : DeviceHealthStatus.Unknown;
      if (health === DeviceHealthStatus.Healthy && handler.getDeviceStatus(device) === DeviceStatus.Problem) {
         health = DeviceHealthStatus.Unreliable;
      }

      let url = "";

      if (r?.jobId) {
         url = `/job/${r.jobId}`;
         if (r.stepId) {
            url += `?step=${r.stepId}`;
         }
      }
      return <Stack verticalFill={true} verticalAlign="center">
         <Stack horizontal tokens={{ childrenGap: 12 }}>
            {automationTab && <Stack verticalFill={true} verticalAlign="center">
                  <div style={{ cursor: "pointer" }} onClick={(ev) => {
                     ev.stopPropagation(); ev.preventDefault();
                     setEditState({
                        infoShown: true, device: item
                     })
                     localState.setHistoryItem(item.device.id);
                  }}>
                     <Stack horizontal tokens={{ childrenGap: 8 }}>
                        <Icon style={{ padding: 2 }} iconName="History"/>
                        {!!health && deviceHealthBadge(health, handler.telemetryHoursRange)}
                     </Stack>
                  </div>
            </Stack>}
            {!!r && <Stack>
               {!!url && <Link to={url}><Text variant="small">Job Details</Text></Link>}
               {!url && !!r.reservationDetails && r.reservationDetails.startsWith("https://horde") && <FluentLink href={r.reservationDetails} target="_blank" underline={false}><Text variant="small" onClick={(ev) => { ev.stopPropagation(); }}>Job Details</Text></FluentLink>}
               <Text variant="tiny">Host: {r.hostname}</Text>
               <Text variant="tiny">Duration: {Math.floor(moment.duration(moment.utc().diff(moment(r.createTimeUtc))).asMinutes())}m</Text>
            </Stack>}
         </Stack>
      </Stack>

   })

   column = columns.find(c => c.name === "Notes")!
   column.onRender = ((item: DeviceItem, index, column) => {

      const device = item.device;
      let notes = device.notes ?? "";

      if (device.checkOutExpirationTime && device.checkedOutByUserId) {
         notes = `Checked out until ${getNiceTime(device.checkOutExpirationTime)}.  ` + notes;
      }

      return <Stack verticalFill={true} verticalAlign="center">
         <div style={{ cursor: "pointer" }} onClick={(ev) => {
            ev.stopPropagation(); ev.preventDefault();
            setEditState({
               shown: true, device: item, editNote: true
            })
         }}>
            <Stack horizontal tokens={{ childrenGap: 12 }}>
               <Icon style={{ paddingTop: 2 }} iconName="Edit" />
               <Text variant="small" style={{ whiteSpace: "pre-wrap" }}>{notes}</Text>
            </Stack>
         </div>
      </Stack>
   })

   /**
    *
    * @returns List of devices filtered by pool and search string
    */
   const GetFilteredDeviceList = (filterString?: string): [GetDeviceResponse[], boolean] => {

      const devices = handler.getDevices();

      const exactMatch = devices.find(d => {
         return d.name.toLowerCase() === filterString || d.address?.toLowerCase() === filterString;
      })

      if (exactMatch) {

         return [[exactMatch], true];
      }

      let poolDevices = devices.filter((d) => {
         const pool = handler.pools.get(d.poolId);
         if (!pool) {
            return false;
         }

         if (automationTab && pool.poolType !== DevicePoolType.Automation) {
            return false;
         }

         if (!automationTab && pool.poolType === DevicePoolType.Automation) {
            return false;
         }

         if (pivotState.poolFilter.size && !pivotState.poolFilter.has(d.poolId)) {
            return false;
         }

         if (pivotState.platformFilter.size && !pivotState.platformFilter.has(d.platformId)) {
            return false;
         }

         if (filterString) {
            if (!(d.name.toLowerCase().includes(filterString)) && !(d.address?.toLowerCase().includes(filterString))) {
               return false;
            }
         }

         return true;

      });
      return [poolDevices, false];
   }

   /**
    * Function to sort a supplied list of devices based on a specified sort column
    * @param filteredDevices List of "GetDeviceResponse" objects used to create a sorted list of DeviceItems
    * @param sortColumn Defines an optional sort column to use in addition to the primary 'platformId' and 'poolId' columns
    * @returns Sorted list of Device Items
    */
   const GetSortedDeviceList = (filteredDevices: GetDeviceResponse[], sortColumn: undefined | string = undefined): DeviceItem[] => {
      const devices = filteredDevices.sort((a, b) => {

         if (a.platformId === b.platformId) {

            if (a.poolId === b.poolId) {
               if (sortColumn) { // If a sort column is defined, sort by that column
                  if (sortColumn === "status") { // The status column is derived, so it isn't as simple as just comparing property values :(
                     let statusCodeA = handler.getDeviceStatus(a);
                     let statusCodeB = handler.getDeviceStatus(b);
                     // "Checked Out" is defined as 'Available' AND checkedOutByUser
                     let statusA = a.checkedOutByUserId && statusCodeA === DeviceStatus.Available ? 100 : statusCodeA;
                     let statusB = b.checkedOutByUserId && statusCodeB === DeviceStatus.Available ? 100 : statusCodeB;
                     if (statusA === statusB) {
                        return 0; // Statuses are not unique, so we need to explicitly return the matching case
                     }
                     return statusA < statusB ? -1 : 1;
                  }
                  if (a[sortColumn] === b[sortColumn]) {
                     return 0; // Names are always unique, but generic columns could have the same value, so this will account for those.
                  }
                  return (a[sortColumn] ?? "") < (b[sortColumn] ?? "") ? -1 : 1;
               } else { // By default, innermost sort should be done by name
                  return a.name < b.name ? -1 : 1;;
               }
            }

            return a.poolId < b.poolId ? -1 : 1;
         }

         return a.platformId < b.platformId ? -1 : 1;

      }).map(d => {
         return { device: d } as DeviceItem;
      });
      return devices;
   }

   // Subscribe to get a new sorted/filtered device list as soon as the search is updated
   if (localState.searchUpdated) { }

   const [filteredDevices, exactMatch] = GetFilteredDeviceList(localState.searchState.filterString?.toLowerCase());

   // Check for an exact match and switch pivot if necessary
   if (exactMatch) {

      const pool = handler.pools.get(filteredDevices[0].poolId);
      const tabType = automationTab ? DevicePoolType.Automation : DevicePoolType.Shared;

      if (pool && pool.poolType !== tabType) {
         const key = pool.poolType === DevicePoolType.Automation ? pivotKeyAutomation : pivotKeyShared;
         setPivotState({ ...pivotState, key: key });
         return null;
      }
   }

   const devices = GetSortedDeviceList(filteredDevices, !automationTab ? "status" : undefined);
   const newGroups: IGroup[] = [];

   let myDevices: DeviceItem[] = [];
   if (pivotState.key === "pivot-key-shared") {
      myDevices = devices.filter(d => d.device.checkedOutByUserId === dashboard.userId);
      devices.unshift(...myDevices.map(d => { return { ...d, userCheckout: true } }));
   }

   let curPlatform: string | undefined;
   devices.forEach((item, index) => {
      const d = item.device;
      if (item.userCheckout) {
         if (!newGroups.length) {
            const key = `user_checkout`;
            newGroups.push({
               key: key,
               name: "My Checkouts",
               startIndex: 0,
               count: myDevices.length,
               level: 0,
               isCollapsed: groups.find(g => g.key === key)?.isCollapsed
            });
         }
      }
      else if (d.platformId !== curPlatform) {
         curPlatform = d.platformId;
         const key = `group_${d.platformId}`;
         const groupDevices = devices.filter(cd => cd.device.platformId === d.platformId).map(d => d.device.id);
         newGroups.push({
            key: key,
            name: platforms.get(d.platformId)!.name,
            data: {devices: groupDevices},
            startIndex: index,
            count: groupDevices.length,
            level: 0,
            // Collapse if it was collapsed by the user, or filtering is not an exact match, or there is more than one device listed
            isCollapsed: (groups.find(g => g.key === key)?.isCollapsed ?? !exactMatch) && devices.length !== 1
         });
      }
   })

   groups = newGroups;

   const groupTitleRenderer = (props: IGroupHeaderProps): JSX.Element => {
      return <DeviceGroupHeader
         group={props.group!}
         handler={handler}
         devices={props.group!.data!.devices}
      />
   }

   const groupHeaderRenderer = (props: IDetailsGroupDividerProps): JSX.Element => {
      const onRenderName = automationTab ? groupTitleRenderer : undefined;
      return <GroupHeader {...props} onRenderName={onRenderName} />
   }

   const pivotItems: JSX.Element[] = [];

   pivotItems.push(<PivotItem headerText="Shared" itemKey={pivotKeyShared} key={pivotKeyShared} />);
   pivotItems.push(<PivotItem headerText="Automation" itemKey={pivotKeyAutomation} key={pivotKeyAutomation} />);


   const poolItems = Array.from(handler.pools.values()).filter(pool => {

      if (automationTab && pool.poolType !== DevicePoolType.Automation) {
         return false;
      }
      if (!automationTab && pool.poolType === DevicePoolType.Automation) {
         return false;
      }

      return true;

   }).sort((a, b) => {
      return a.name < b.name ? 1 : 0;
   }).map(pool => {
      const name = pool.name;
      return {
         text: name,
         key: `dropdown_${pool.id}_key`,
         poolId: pool.id
      }
   });

   const platformItems = Array.from(platforms.values()).sort((a, b) => {
      if (a.name === b.name) { return 0; }
      return a.name < b.name ? 1 : -1;
   }).map(platform => {
      return {
         text: platform.name,
         key: `dropdown_${platform.id}_key`,
         platformId: platform.id
      }
   })

   const checkedOut = handler.getUserDeviceCheckouts(dashboard.userId);

   const hasDevices = !!handler.getDevices().length;

   return (<Stack>
      {!!telemState && <DevicePoolTelemetryModal poolsIn={localState.searchState.filterPools} platformsIn={localState.searchState.filterPlatforms} onClose={() => setTelemState(false)} />}
      {(!!checkoutState.checkinId || !!checkoutState.checkoutId) && <InfoModal />}
      {!!checkoutState.showConfirm && checkedOut.length > 0 && <CheckoutConfirmModal check={checkoutState.showConfirm} devices={checkedOut} onClose={() => setCheckoutState({})} />}
      {editState.infoShown && <DeviceInfoModal handler={handler} deviceIn={editState.device?.device} onEdit={(device) => { setEditState({ infoShown: true, shown: true, device: { device: device } }) }} onClose={() => { setEditState({}); localState.resetHistoryItem(); }} />}
      {editState.shown && <DeviceEditor handler={handler} deviceIn={editState.device?.device} editNote={editState.editNote} onClose={(device) => { const deviceItem = !!device? { device: device }: undefined; setEditState({ ...editState, shown: false, device: deviceItem }); localState.resetHistoryItem(); }} />}
      <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, width: "100%" } }} >
         <Stack>
            <Stack horizontal verticalAlign="center" style={{ paddingBottom: 12 }} tokens={{ childrenGap: 32 }}>
               {hasDevices && <Stack>
                  <Pivot className={hordeClasses.pivot}
                     selectedKey={pivotState.key}
                     linkSize="normal"
                     linkFormat="links"
                     onLinkClick={(item) => {
                        localState.setPivotKey(item!.props.itemKey!);
                        setPivotState({ key: item!.props.itemKey!, poolFilter: new Set(), platformFilter: new Set() })
                     }}>
                     {pivotItems}
                  </Pivot>
               </Stack>}
               {hasDevices && <Stack>
                  <TextField
                     placeholder="Device Search"
                     spellCheck={false}
                     autoComplete="off"
                     validateOnLoad={false}
                     deferredValidationTime={1000}
                     defaultValue={localState.searchState.filterString}
                     styles={{
                        root: { width: 280, fontSize: 12 }, fieldGroup: {
                           borderWidth: 1
                        }
                     }}
                     onKeyUp={(evt) => {
                        if (evt.key === "Enter") {
                           localState.setFilter(evt.currentTarget.value || undefined)
                        }
                     }}
                     onGetErrorMessage={(newValue) => {
                        localState.setFilter(newValue);
                        return undefined;
                     }}
                  />
               </Stack>}
               {hasDevices && platforms.size > 1 && <Stack style={{ paddingLeft: 15 }}>
                  <Dropdown
                     placeholder="Filter Platforms"
                     style={{ width: 200 }}
                     styles={dropDownStyle}
                     selectedKeys={Array.from(pivotState.platformFilter).map(id => `dropdown_${id}_key`)}
                     multiSelect
                     options={platformItems}
                     onChange={(event, option, index) => {

                        if (option) {
                           if (option.selected) {
                              pivotState.platformFilter.add((option as any).platformId);
                           } else {
                              pivotState.platformFilter.delete((option as any).platformId);
                           }

                           localState.setPlatformFilters(pivotState.platformFilter);
                           setPivotState({...pivotState, platformFilter: new Set(pivotState.platformFilter)})
                        }
                     }}
                  />
               </Stack>}
               {hasDevices && poolItems.length > 1 && <Stack style={{ paddingLeft: 15 }}>
                  <Dropdown
                     placeholder="Filter Pools"
                     style={{ width: 200 }}
                     styles={dropDownStyle}
                     selectedKeys={Array.from(pivotState.poolFilter).map(id => `dropdown_${id}_key`)}
                     multiSelect
                     options={poolItems}
                     onChange={(event, option, index) => {

                        if (option) {

                           if (option.selected) {
                              pivotState.poolFilter.add((option as any).poolId);
                           } else {
                              pivotState.poolFilter.delete((option as any).poolId);
                           }

                           localState.setPoolFilters(pivotState.poolFilter);
                           setPivotState({ ...pivotState })

                        }
                     }}
                  />
               </Stack>}

               {hasDevices && automationTab && <Stack>
                  <DefaultButton text="Pool Telemetry" onClick={() => { setTelemState(true) }} />
               </Stack>}

               <Stack grow />
               <Stack>
                  <PrimaryButton text="Add Device"
                     styles={{ root: { fontFamily: "Horde Open Sans SemiBold !important" } }}
                     onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); setEditState({ shown: true }) }}
                  />
               </Stack>

            </Stack>


         </Stack>

         {!handler.getDevices().length && handler.loaded && <Stack horizontalAlign="center">
            <Text variant="mediumPlus">No Devices Found</Text>
         </Stack>}


         {!!handler.getDevices().length && <Stack tokens={{ childrenGap: 12 }}>
            <FocusZone direction={FocusZoneDirection.vertical}>
               <div className={customStyles.details} style={{ height: "calc(100vh - 280px)", position: 'relative' }} data-is-scrollable>
                  <ScrollablePane scrollbarVisibility={ScrollbarVisibility.always} onScroll={() => { }}>
                     <DetailsList
                        styles={{
                           root: {
                              overflowX: "hidden", width: 1312, selectors: {
                                 '.ms-GroupHeader,.ms-GroupHeader:hover': {
                                    background: dashboard.darktheme ? hordeTheme.horde.dividerColor : "#DFDEDD",
                                 },
                                 '.ms-GroupHeader-expand,.ms-GroupHeader-expand:hover': {
                                    background: dashboard.darktheme ? hordeTheme.horde.dividerColor : "#DFDEDD",
                                 }
                              }
                           }
                        }}
                        items={devices}
                        groups={groups}
                        groupProps={{onRenderHeader: groupHeaderRenderer}}
                        columns={columns}
                        selectionMode={SelectionMode.none}
                        layoutMode={DetailsListLayoutMode.justified}
                        compact={true}
                        onShouldVirtualize={() => false}
                     />
                  </ScrollablePane>
               </div>
            </FocusZone>
         </Stack>}
      </Stack>
   </Stack>
   );
});

const CheckoutConfirmModal: React.FC<{ check: "in" | "out" | "error", devices: GetDeviceResponse[], onClose: () => void }> = ({ check, devices, onClose }) => {

   const { hordeClasses } = getHordeStyling();

   type CheckoutItem = {
      device: GetDeviceResponse;
   }

   const close = () => {
      onClose();
   };

   const columns = [
      { key: 'column1', name: 'Name', fieldName: 'name', minWidth: 240, maxWidth: 240 },
      { key: 'column2', name: 'Date', fieldName: 'value', minWidth: 500, maxWidth: 500 },
   ];

   const deviceItems: CheckoutItem[] = devices.map(d => { return { device: d } });

   const onRenderItemColumn = (item: CheckoutItem, index?: number, columnIn?: IColumn) => {

      const column = columnIn!;

      const device = item.device;

      const platform = handler.platforms.get(device.platformId)?.name;

      // simple cases
      switch (column.name) {
         case 'Name':
            return <Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{platform} : {device.name} : {device.address ?? ""} </Text>
         case 'Date':
            if (device.checkOutExpirationTime) {
               return <Text variant="medium">{`Checked out until ${getNiceTime(device.checkOutExpirationTime)}`}</Text>
            }
            return null;
      }

      return null;
   }

   let text = check === "in" ? "Device checked in, you have the following devices checked out:" : "Device checked out, you have the following devices checked out:";
   if (check === "error") {
      text = "There was an error processing request, please try again.";
   }


   return <Modal isOpen={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 840, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { close() }}>
      <Stack tokens={{ childrenGap: 12 }}>
         <Stack horizontal styles={{ root: { paddingLeft: 12 } }}>
            <Stack grow style={{ paddingTop: 12, paddingLeft: 8 }}>
               <Label style={{ fontSize: 18, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }}>Checked Out Devices</Label>
            </Stack>
            <Stack style={{ paddingRight: 4, paddingTop: 4 }}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { close(); }}
               />
            </Stack>
         </Stack>

         <Stack tokens={{ childrenGap: 32 }} style={{ paddingLeft: 8 }}>
            <Stack style={{ paddingLeft: 18 }}>
               <Stack>
                  <Text variant="medium">{text}</Text>
               </Stack>
            </Stack>
            <Stack horizontal>
               <Stack style={{ minWidth: 700, paddingLeft: 18, paddingRight: 18 }}>
                  {check !== "error" && <DetailsList
                     items={deviceItems}
                     columns={columns}
                     setKey="set"
                     layoutMode={DetailsListLayoutMode.justified}
                     isHeaderVisible={false}
                     selectionMode={SelectionMode.none}
                     onRenderItemColumn={onRenderItemColumn}
                  />}
               </Stack>
            </Stack>

         </Stack>

         <Stack styles={{ root: { padding: 12 } }}>
            <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
               <Stack grow />
               <PrimaryButton text="Ok" disabled={false} onClick={() => { close(); }} />
            </Stack>
         </Stack>
      </Stack>
   </Modal>;

};


export const DeviceView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   localState.initialize();

   useEffect(() => {

      return () => {
         localState.resetState();
      };

   }, []);



   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Devices' }]} />
      <Stack horizontal>
         <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
            <Stack style={{ maxWidth: 1440, paddingTop: 30, marginLeft: 4, height: 'calc(100vh - 8px)' }}>
               <Stack horizontal className={hordeClasses.raised}>
                  <Stack style={{ width: "100%", height: 'calc(100vh - 228px)' }} tokens={{ childrenGap: 18 }}>
                     <DevicePanel />
                     <SearchUpdate />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>

};


