// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, mergeStyleSets } from "@fluentui/react";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import { useEffect, useState } from "react";
import { useLocation } from "react-router-dom";
import { Markdown } from "../../base/components/Markdown";
import { ISideRailLink, SideRail } from "../../base/components/SideRail";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { BreadcrumbItem, Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";

type Anchor = {
   text: string;
   anchor: string;
}

type State = {
   crumbs: BreadcrumbItem[];
   jumpLinks: ISideRailLink[];
}


const docClasses = mergeStyleSets({
   raised: {
      boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)",
      padding: "32px 40px"
   }
});


class LinkState {

   constructor() {
      makeObservable(this);
   }

   @observable
   state: State = { crumbs: [], jumpLinks: [] };

   @action
   setState(crumbs: BreadcrumbItem[], jumpLinks: ISideRailLink[]) {
      this.state = { crumbs: crumbs, jumpLinks: jumpLinks };
   }
}

class DocumentCache {

   constructor() {
      makeObservable(this);
   }

   get(docName: string): { markdown: string, crumbs: BreadcrumbItem[], anchors: Anchor[] } | undefined {

      // implicit subscription
      if (this.available) { }

      const text = this.documentCache.get(docName);
      const crumbs: BreadcrumbItem[] = this.crumbCache.get(docName) ?? [];
      const anchors: Anchor[] = this.anchorCache.get(docName) ?? [];

      if (text) {
         return {
            markdown: text,
            crumbs: crumbs,
            anchors: anchors
         }
      }

      // already in flight
      if (this.documentRequests.has(docName)) {
         return undefined;
      }

      this.documentRequests.add(docName);

      fetch(`/${docName}`, { cache: "no-cache" })
         .then((response) => response.text())
         .then((textContent) => {

            let crumbs: BreadcrumbItem[] = [];
            let anchors: Anchor[] = [];

            textContent = textContent.trim();

            if (textContent?.indexOf("<!doctype html>") === -1) {

               // generate anchors
               let lines = (textContent.match(/[^\r\n]+/g) ?? []) as string[];
               lines = lines.map(i => i.trim()).filter(i => !!i);
               lines.forEach(line => {
                  let anchor = "";

                  //if (line.startsWith("# ")) {
                  //   anchor = line.split("# ")[1];
                  //}

                  if (line.startsWith("## ")) {
                     anchor = line.split("## ")[1];
                  }

                  if (anchor) {
                     anchor = anchor.trim();
                     anchors.push({ text: anchor, anchor: anchor.replace(/[^a-z0-9- ]/gi, '').replace(/ /gi, '-').toLowerCase() })
                  }
               })

               // generate crumbs
               if (lines && lines.length) {
                  const line = lines[0];
                  if (line.startsWith("[Horde]")) {

                     line.split(">").map(c => c.trim()).forEach(c => {
                        const link = c.split("](");
                        if (link.length === 1) {
                           crumbs.push({ text: link[0] })
                        } else {
                           let [cname, clink] = [link[0].replace("[", ""), link[1].replace(")", "")];

                           let docPath = docName.split("/").slice(2);
                           docPath.pop();

                           const relative = (clink.match(/..\//g) || []).length;

                           if (clink.indexOf("README.md") !== -1) {
                              clink = "/docs"
                           } else {
                              docPath = docPath.slice(docPath.length - 1, -relative);
                              const elements = clink.split("/");
                              docPath.push(elements[elements.length - 1]);
                              clink = docPath.join("/");
                           }

                           if (cname !== "Horde") {
                              crumbs.push({ text: cname, link: clink });
                           }

                        }
                     });

                     textContent = textContent.replace(line, "");
                  }
               }

               this.documentCache.set(docName, textContent);
               this.crumbCache.set(docName, crumbs);
               this.anchorCache.set(docName, anchors);

            } else {
               this.documentCache.set(docName, `###Missing document ${docName}`);
               this.crumbCache.set(docName, []);
               this.anchorCache.set(docName, []);
            }

            this.setAvailable();

         });
   }

   @observable
   private available: number = 0;

   @action
   private setAvailable() {
      this.available++;
   }

   private documentCache = new Map<string, string>();
   private documentRequests = new Set<string>();
   private crumbCache = new Map<string, BreadcrumbItem[]>();
   private anchorCache = new Map<string, Anchor[]>();

}


const linkState = new LinkState();
const documentCache = new DocumentCache();

const DocPanel: React.FC<{ docName: string }> = observer(({ docName }) => {
   useLocation();
   const [scrolled, setScrolled] = useState("");

   const cache = documentCache.get(docName);
   const anchor = window.location.hash?.split("?")[0]?.slice(1) ?? "";

   useEffect(() => {

      if (anchor === scrolled || !cache) {
         return;
      }

      // This timeout is horrible, though scrollIntoView is inaccurate until the rendering has "settled"
      // I tried quite a few approaches to this, and had to move on, this works
      setTimeout(() => {         
         if (anchor === (window.location.hash?.split("?")[0]?.slice(1) ?? "")) {
            const element = document.getElementById(anchor);
            element?.scrollIntoView();   
         }
      }, 200)
            
      setScrolled(anchor)
   }, [anchor, scrolled, cache])

   if (!cache) {
      return null;
   }
   const text = cache.markdown;
   let crumbs = cache.crumbs;
   let anchors = cache.anchors;

   const jumpLinks = anchors.map(a => {
      return { text: a.text, url: a.anchor }
   })

   jumpLinks.unshift({ text: "Back to top", url: `page-top` })
   linkState.setState(crumbs, jumpLinks);

   return <Stack styles={{ root: { width: "100%" } }}>
      <div style={{ margin: "16px 32px" }}>
         <Markdown>{text}</Markdown>
      </div>
   </Stack>;
})

const DocRail = observer(() => {
   const state = linkState.state;
   return <Stack style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "calc(100vh - 240px)" }} data-is-scrollable={true}><SideRail jumpLinks={state.jumpLinks} /></Stack>

})

const DocCrumbs: React.FC<{ landingPage: boolean }> = observer(({ landingPage }) => {

   const state = linkState.state;

   const crumbs: BreadcrumbItem[] = [];
   if (!landingPage) {
      crumbs.push({ text: "Documentation", link: "/docs" })
      crumbs.push(...state.crumbs.map(c => { return { text: c.text, link: c.link } }));
   } else {
      crumbs.push({ text: "Home", link: "/index" })
   }

   return <Breadcrumbs items={crumbs} />

})

export const DocView = () => {

   const location = useLocation();

   const { hordeClasses, modeColors } = getHordeStyling();

   let docName = location.pathname.replace("/docs/", "").replace("/docs", "").trim();

   let landingPage = docName === "Landing.md";

   if (!docName || docName.indexOf("README.md") !== -1) {

      if (landingPage) {
         docName = "documentation/Docs/Landing.md";
      } else {
         docName = "documentation/Docs/Home.md";
      }

   } else {

      if (!docName.startsWith("Docs/")) {
         docName = `documentation/Docs/${docName}`;
      } else {
         docName = docName.replace("Docs/", "documentation/Docs/");
      }
   }

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <DocCrumbs landingPage={landingPage} />
      <Stack horizontal>
         <Stack key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%", "position": "relative" } }}>
            <div style={{ overflowY: 'scroll', overflowX: 'hidden', height: "calc(100vh - 162px)" }} data-is-scrollable={true}>
               <Stack horizontal style={{ paddingLeft: "32px", paddingBottom: "16px", paddingRight: 0 }} >
                  <Stack style={{ width: 230 }} />
                  <Stack style={{ width: 900, marginLeft: 4 }}>
                     <Stack style={{ height: "24px", backgroundColor: modeColors.background }} id="page-top" />
                     <Stack className={docClasses.raised} styles={{ root: { backgroundColor: modeColors.content } }}>
                        <DocPanel docName={docName} />
                     </Stack>
                     <Stack style={{ paddingBottom: 24 }} />
                  </Stack>
                  {!landingPage && <Stack style={{ paddingLeft: 1160, paddingTop: 24, position: "absolute", pointerEvents: "none" }}>
                     <Stack style={{ pointerEvents: "all" }} styles={{ root: { selectors: { "*::-webkit-scrollbar": { display: "none" }, "*::-ms-overflow-style": "none", "*::scrollbar-width": "none" } } }}>
                        <DocRail/>
                     </Stack>
                  </Stack>}
               </Stack>
            </div>
         </Stack>
      </Stack>
   </Stack>

}

