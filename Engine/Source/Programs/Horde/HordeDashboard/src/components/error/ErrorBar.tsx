// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { MessageBar, MessageBarType } from '@fluentui/react';
import { errorBarStore } from './ErrorStore';
import { observer } from 'mobx-react-lite';
import dashboard from 'horde/backend/Dashboard';

export const ErrorBar: React.FC = observer(() => {
    const {error, clear} = errorBarStore;

    const helpEmail = error?.helpEmail ?? dashboard.helpEmail;
    const helpSlack = error?.helpSlack ?? dashboard.helpSlack;
    const anyHelp = !!helpSlack || !!helpEmail;

    const getConsoleErrorMessage = (): string => {
        if(!error) return "";

        let msg = `${error.title}: ${errorBarStore.message}`
        msg += error.url ? ` - Occurred while trying to access ${error.url}` : "";

        // Clear formatting for console output
        if(anyHelp) {
            msg += ` - If the issue persists, please contact us for assistance:\n\n`;

            msg += helpEmail ? `Email: ${helpEmail}\n` : "";
            msg += helpSlack ? `Slack: ${helpSlack}\n` : "";

            msg += "\n";
        }

        return msg;
    }

    // Report extra information in the broswer console (leave room to breathe in the UI)
    if(error){
        console.error(getConsoleErrorMessage());
    }

    if(!error) {
        return <div />
    }

    return (
        <MessageBar
            messageBarType={MessageBarType.error} 
            isMultiline={false}
            hidden={!error}
            truncated={true}
            onDismiss={clear}
            dismissButtonAriaLabel="Close"
            actions={
                <div>
                    {/*Opportunity for retry or filter implementation if needed.*/}
                    {/* <MessageBarButton>Never show this again</MessageBarButton> */}
                    {/* <MessageBarButton>Retry</MessageBarButton> */}
                </div>
            }
        >
            <span>{error.title}: {errorBarStore.message}. {(!!error.url || anyHelp) && "Check your browser's developer console."}</span>
        </MessageBar>
    );
});