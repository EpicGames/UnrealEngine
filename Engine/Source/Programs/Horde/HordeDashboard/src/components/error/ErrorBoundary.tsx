// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from "@fluentui/react";
import { Component, ErrorInfo, ReactNode } from "react";
import { errorDialogStore } from "./ErrorStore";

interface Props {
   children?: ReactNode;
}

interface State {
   hasError: boolean;
}

class ErrorBoundary extends Component<Props, State> {
   public state: State = {
      hasError: false
   };

   public static getDerivedStateFromError(_: Error): State {
      return { hasError: true };
   }

   public componentDidCatch(error: Error, errorInfo: ErrorInfo) {      
      console.error("Uncaught Exception:", error, errorInfo);
      errorDialogStore.set({
         reason: `${error}`,
         title: `Uncaught Exception`,
         message: `Error: ${error}\n\n${errorInfo?.componentStack}`

      }, true);
   }

   public render() {

      if (this.state.hasError) {
         return <Stack style={{padding: 128}}><Text variant="mediumPlus">Uncaught Exception, please see console for more information</Text></Stack>;
      }
                  
      return this.props.children;
   }

   static error?: Error;
   static errorInfo?: ErrorInfo;
}

export default ErrorBoundary;