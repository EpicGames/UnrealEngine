// Copyright Epic Games, Inc. All Rights Reserved.

import { observer } from 'mobx-react-lite';
import { BaseButton, Checkbox, DefaultButton, Dialog, DialogType, IButtonStyles, mergeStyleSets, PrimaryButton, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import React from 'react';
import { errorDialogStore } from './ErrorStore';
import dashboard, { StatusColor } from '../../backend/Dashboard';
import { getErrorDialogStyles } from './ErrorDialogStyles';

export const ErrorDialog: React.FC = observer(() => {
   const {
      canRetry,
      retry,
      clear,
      error,
   } = errorDialogStore;

   if (!error) {
      return <div />;
   }

   let message = errorDialogStore.message;
   const unauthorized = errorDialogStore.unauthorized();

   let title = error.title;
   if (!title) {
      title = unauthorized ? "Permission Error" : "Oops, there was a problem...";
   }

   const dialogContentProps = {
      type: DialogType.normal,
      title: title
   };

   const helpEmail = error.helpEmail ?? dashboard.helpEmail;
   const helpSlack = error.helpSlack ?? dashboard.helpSlack;

   let fontSize = 13;

   let width = 800;
   if (message?.length) {
      try {
         const exception = JSON.parse(message);
         if (exception?.exception?.message && exception?.exception?.trace) {
            width = 1380;
            fontSize = 11
            message = `${exception?.exception?.message}\n${exception?.exception?.trace}`
         }
      } catch (error) {

      }
   }

   const anyHelp = !!helpEmail || !!helpSlack;

   const errorDialogStyles = getErrorDialogStyles();

   const {
      isRetrying,
      succeeded: retrySucceeded,
      failed: retryFailed,
      hasResult: retryHasResult,
   } = errorDialogStore.retryInfo;

   const getRetryButtonText = () => {
      if (retrySucceeded) return "Success";
      if (retryFailed) return "Retry Failed";
      if (isRetrying) return "Retrying...";
      if (retryHasResult) return "Try Again?";
      return "Retry";
   };

   const getRetryButtonClass = () => {
      if (retrySucceeded) return errorDialogStyles.successButton;
      if (retryFailed) return errorDialogStyles.errorButtonDisabled;
      if (retryHasResult) return errorDialogStyles.errorButton;
      return errorDialogStyles.button;
   };

   return (
      <Dialog
         minWidth={width}
         hidden={!error}
         onDismiss={clear}
         modalProps={{
            isBlocking: true
         }}
         dialogContentProps={dialogContentProps}>

         <Stack tokens={{ childrenGap: 12 }}>
            <Text styles={{ root: { whiteSpace: "pre", fontSize: fontSize, color: "#EC4C47" } }}>{message}</Text>
            {!!error.url && <Text variant="medium">{`URL: ${error.url}`}</Text>}
            <Stack>
               {anyHelp && <Stack tokens={{ childrenGap: 18 }} style={{ paddingTop: 12 }}>
                  <Text style={{ fontSize: 15 }}>If the issue persists, please contact us for assistance.</Text>
                  <Stack tokens={{ childrenGap: 14 }} style={{ paddingLeft: 12, paddingTop: 2 }}>
                     {!!helpEmail && 
                        <Stack horizontal tokens={{ childrenGap: 12 }}>
                           <Text className={errorDialogStyles.supportTextKey}>Email:</Text>
                           <Text className={errorDialogStyles.supportTextValue}>{helpEmail}</Text>
                        </Stack>
                     }
                     {!!helpSlack && 
                        <Stack horizontal tokens={{ childrenGap: 12 }}>
                           <Text className={errorDialogStyles.supportTextKey}>Slack:</Text>
                           <Text className={errorDialogStyles.supportTextValue}>{helpSlack}</Text>
                        </Stack>
                     }
                  </Stack>
               </Stack>}

               <Stack horizontal style={{ paddingTop: 24, gap: 24, alignItems: "center", justifyContent: "end" }}>
                  <Checkbox 
                     label="Don't show errors like this again" 
                     defaultChecked={false} 
                     onChange={(e, b) => { errorDialogStore.filterError = !!b; }} 
                  />
                  {!canRetry && <PrimaryButton text="Dismiss" onClick={clear} />}
                  {canRetry && (
                     <>
                        <DefaultButton text="Dismiss" onClick={clear} />
                        
                        <PrimaryButton
                           text={getRetryButtonText()}
                           onClick={retry}
                           disabled={isRetrying}
                           className={getRetryButtonClass()}
                        >
                           {(isRetrying && !retryHasResult) && <Spinner size={SpinnerSize.small}/>}
                        </PrimaryButton>
                     </>
                  )}
               </Stack>
               {retrySucceeded && (
                  <Stack.Item align='end'>
                     <p style={{marginBottom: 0}}>You can safely dismiss this dialog.</p>
                  </Stack.Item>
               )}
            </Stack>
         </Stack>
      </Dialog>
   );
});