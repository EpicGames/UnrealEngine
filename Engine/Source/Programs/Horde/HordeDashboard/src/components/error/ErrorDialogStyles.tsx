// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyleSets } from "@fluentui/react"
import dashboard, { StatusColor } from "horde/backend/Dashboard";

export const getErrorDialogStyles = () => {
    const colors = dashboard.getDefaultStatusColors();

    const errorDialogStyles = mergeStyleSets({
        button: {
            width: "117px",
        },
        
        errorButton: {
            backgroundColor: colors.get(StatusColor.Failure),
            borderStyle: "hidden",
            width: "117px",
            color: "#FFFFFF",
            selectors: {
            ':active,:hover': {
                filter: "brightness(120%)",
                backgroundColor: colors.get(StatusColor.Failure),
                width: "117px",
                borderStyle: "hidden",
                color: "#FFFFFF",
            }
            }
        },

        errorButtonDisabled: {
            backgroundColor: colors.get(StatusColor.Failure),
            borderStyle: "hidden",
            width: "117px",
            color: "#FFFFFF",
            selectors: {
            ':active,:hover': {
                backgroundColor: colors.get(StatusColor.Failure),
                borderStyle: "hidden",
                width: "117px",
                color: "#FFFFFF",
            }
            }
        },

        successButton: {
            backgroundColor: colors.get(StatusColor.Success),
            borderStyle: "hidden",
            width: "117px",
            color: "#FFFFFF",
            selectors: {
            ':active,:hover': {
                backgroundColor: colors.get(StatusColor.Success),
                borderStyle: "hidden",
                color: "#FFFFFF",
                width: "117px",
            }
            }
        },

        supportTextKey: {fontSize: 15},

        supportTextValue: { fontSize: 15, fontWeight: 400, fontFamily: "Horde Open Sans SemiBold" }
    })

    return errorDialogStyles;
}