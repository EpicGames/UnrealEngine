// Copyright Epic Games, Inc. All Rights Reserved.

import { getSiteConfig } from "../../backend/Config";
import { action, makeObservable, observable, override, runInAction } from "mobx";

// Explicit error message
export type ErrorFormatted = {
   time: Date | string;
   level: "Error" | "Warning";
   message: string;
   id: number;
   format: string;
   properties: Record<string, string | number | boolean>
}

export type ErrorInfo = {

   // general project rejection
   reason?: any;

   // fetch response object
   response?: Response;

   // request url (should match url in fetch response if any)
   url?: string;

   // REST mode
   mode?: "GET" | "PUT" | "DELETE" | "POST" | "PATCH" | string;

   // custom title override
   title?: string;

   // custom message override
   message?: string;

   format?: ErrorFormatted;

   // custom help contact overrides (falls back to global dashboard config if not set)
   helpSlack?: string;
   helpEmail?: string;

}

export enum ErrorReportType {
   Dialog = "Dialog",
   Bar = "Bar"
}

type RetryInfo = {
   isRetrying: boolean,
   succeeded?: boolean,
   failed?: boolean,
   hasResult: boolean
}

class ErrorStore {

   constructor() {
      makeObservable(this);
   }

   @observable
   error?: ErrorInfo;

   // Retry logic
   @observable
   retryInfo: RetryInfo = {
      isRetrying: false,
      succeeded: undefined,
      failed: undefined,
      hasResult: false,
   };
   
   retryCallback?: () => boolean | Promise<boolean>;

   @action
   set(
      infoIn: ErrorInfo, 
      update?: boolean,
      retryCallback?: () => boolean | Promise<boolean>
   ): void {
      // this.print(infoIn);

      const config = getSiteConfig();

      if (!update) {// && !this.unauthorized(infoIn)) {
         if (config.environment === "production") {
            return;
         }
      }

      if (infoIn.response?.status === 502) {
         return;
      }

      const hash = this.hash(infoIn);
      if (hash && this.filter.has(hash)) {
         return;
      }

      if (this.error && !update) {
         return;
      }

      if(retryCallback) this.retryCallback = retryCallback;

      const info = { ...infoIn };
      this.error = info;
   }

   @action.bound
   async retry() {
      if(!this.retryCallback) return;

      this.retryInfo.isRetrying = true;
      this.retryInfo.succeeded = false;
      this.retryInfo.failed = false;
      this.retryInfo.hasResult = false;

      try {
         // Callbacks may be synchronous or asynchronous.
         const result = this.retryCallback();
         const success = result instanceof Promise ? await result : result;

         // In the case of async callbacks, this kicks the rest of the function out of
         // being a MobX action. Thus, further observable mutations need to be in a runInAction
         runInAction(() => {
            if(success) {
               this.retryInfo.succeeded = true;
               // this.clear();
            } else {
               this.retryInfo.failed = true;
            }
         });
      } catch (e) {
         runInAction(() => this.retryInfo.failed = true)
      } finally {
         runInAction(() => {
               this.retryInfo.hasResult = true;
               // Let the user try again
               if(this.retryInfo.failed) setTimeout(this.clearRetry, 1250);
            }
         )
      }
   }

   get canRetry(): boolean {
      return !!this.retryCallback;
   }

   get message(): string {

      const error = this.error;

      if (!error) {
         return "";
      }

      if (error.format?.message) {
         return error.format?.message;
      }

      if (error.message) {
         // Catch(reason) defaults to type 'any,' missing eroneous assignments of incorrect types.
         if(typeof error.message !== "string"){
            console.error("Error message not set to string. Check error store set arguments.")
            return "[BAD ERROR MESSAGE]"
         }
         return error.message;
      }

      if (error.reason) {
         return error.reason.toString();
      }

      if (error.response) {
         return `${error.response.status}: ${error.response.statusText}`;
      }

      return "";

   }

   unauthorized(infoIn?: ErrorInfo): boolean {

      let error = infoIn;

      if (!error) {
         error = this.error;
      }

      if (!error) {
         return false;
      }

      if (!error.response) {
         return false;
      }

      if (error.response.status === 401 || error.response.status === 403 || error.response.url.toLowerCase().indexOf("accessdenied") !== -1) {
         return true;
      }

      return false;

   }

   get reject(): string {

      return this.message;

   }

   hash(error: ErrorInfo): string {

      const url = error.url ?? error?.response?.url;

      // filter on end point, without query strings
      if (url) {
         return url.split(/[?#]/)[0];
      }

      // filter on reason
      if (error.reason) {
         return error.reason;
      }

      return "";
   }

   print(info: ErrorInfo) {

      // this will also go out to datadog if configured
      const message = `${info.response ? JSON.stringify(info.response) : ""} ${info.reason ? info.reason : ""}`;

      if (!message) {
         return;
      }
      console.error(message);
   }

   @action.bound
   clear(): void {

      if (!this.error) {
         return;
      }

      if (this.filterError) {
         const hash = this.hash(this.error);
         if (hash) {
            this.filter.add(hash);
         }
      }

      this.filterError = false;
      
      this.error = undefined;

      this.clearRetry(true);
   }

   // Can clear retry entirely or just for subsequent attempts
   @action.bound
   clearRetry(clearFull?: boolean): void {
      this.retryInfo.succeeded = undefined;
      this.retryInfo.failed = undefined;
      this.retryInfo.isRetrying = false;
      
      if(clearFull) {
         this.retryCallback = undefined;
         this.retryInfo.hasResult = false;
      }
   }

   filter = new Set<string>();

   filterError = false;

}

export const errorDialogStore = new ErrorStore();
export const errorBarStore = new ErrorStore();