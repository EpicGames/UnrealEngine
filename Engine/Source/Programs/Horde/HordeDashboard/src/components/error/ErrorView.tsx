// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import dashboard from '../../backend/Dashboard';
import { getHordeStyling } from "horde/styles/Styles.tsx";
import { DashboardPreference } from "horde/backend/Api.ts";

export const ErrorView: React.FC<{ title: string, description: string, reason: string }> = ({title, description, reason}) => {
    const {modeColors} = getHordeStyling();
    const errorColor = dashboard.getPreference(DashboardPreference.ColorError) ?? "#c44525";
    return (
        <div style={{
            backgroundColor: modeColors.background,
            padding: '60px 0',
            display: 'flex',
            color: errorColor,
            flexDirection: 'column',
            justifyContent: 'center',
            alignItems: 'center'
        }}>
            <div style={{
                maxWidth: 600,
                width: "90%",
                padding: '0px 15px',
                backgroundColor: `color-mix(in srgb, ${errorColor} 10%, white)`,
                border: "1px solid " + errorColor
            }}>
                <h1 style={{border: 1, marginBottom: '8px'}}>{title}</h1>
                <p>{description}</p>
                <h2 style={{fontSize: 14, margin: '0 0'}}>Reason</h2>
                <p style={{marginTop: 5, wordBreak: 'break-word'}}>{reason}</p>
            </div>
        </div>
    );
}
