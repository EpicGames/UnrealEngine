// Copyright Epic Games, Inc. All Rights Reserved.
import { DefaultButton, ITextField, IconButton, Modal, PrimaryButton, Stack, Text, TextField } from '@fluentui/react';
import React from 'react';
import backend from '../../backend';
import { GetJobResponse } from '../../backend/Api';
import { JobDetailsV2 } from './JobDetailsViewCommon';
import { getHordeStyling } from '../../styles/Styles';


export const AbortJobModal: React.FC<{ jobDetails?: JobDetailsV2; jobDataIn?: GetJobResponse; stepId?: string, show: boolean; onClose: () => void }> = ({ jobDetails, jobDataIn, stepId, show, onClose }) => {

   const reasonRef = React.useRef<ITextField>(null);

   const { hordeClasses } = getHordeStyling();

   const jobData = jobDetails?.jobData ?? jobDataIn;
   if (!jobData) {
      console.error("Attempting to display cancel job modal with no job data")
      return null;
   }

   let headerText = `Cancel ${jobData.name} Job?`;

   if (jobDetails && stepId) {
      headerText = `Cancel ${jobDetails.getStepName(stepId)} Step?`;
   }

   const close = () => {
      onClose();
   };

   const onAbort = async () => {

      const reason = reasonRef.current?.value?.trim();

      if (!stepId) {
         await backend.updateJob(jobData.id, { aborted: true, cancellationReason: reason ? reason : undefined }).then((response) => {
            console.log("Job canceled", response);
         }).catch((reason) => {
            // @todo: error ui
            console.error(reason);
         }).finally(() => {
            close();
         });
      } else {

         const batch = jobDetails?.batchByStepId(stepId);

         if (!batch) {
            console.error("Unable to get batch id for step cancel");
            return;
         }

         await backend.updateJobStep(jobData.id, batch.id, stepId, { abortRequested: true, cancellationReason: reason ? reason : undefined }).then((response) => {
            console.log("Job step canceled", response);
         }).catch((reason) => {
            // @todo: error ui
            console.error(reason);
         }).finally(() => {
            close();
         });

      }

   };

   const height = 300;

   return <Modal isOpen={show} className={hordeClasses.modal} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { if (show) close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack style={{ paddingBottom: 24 }}>
            <TextField style={{height: 120}} componentRef={reasonRef} label="Cancellation Reason" multiline resizable={false} />
         </Stack>
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack grow />
            <PrimaryButton text={(jobDetails && stepId) ? "Cancel Step" : "Cancel Job"} disabled={false} onClick={() => { onAbort(); }} />
            <DefaultButton text="Close" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};