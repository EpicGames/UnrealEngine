// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, DirectionalHint, FontIcon, IColumn, IContextualMenuProps, IGroup, PrimaryButton, SelectionMode, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useEffect, useState } from 'react';
import { GetJobArtifactResponse, JobStepState } from '../../backend/Api';
import dashboard, { StatusColor } from '../../backend/Dashboard';
import { ISideRailLink } from '../../base/components/SideRail';
import { getStepETA, getStepFinishTime } from '../../base/utilities/timeUtils';
import { getHordeStyling } from '../../styles/Styles';
import { getStepStatusColor } from '../../styles/colors';
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { JobArtifactsModal } from '../artifacts/ArtifactsModal';

import { useConst } from '@fluentui/react-hooks';
import { ArtifactButton } from '../artifacts/ArtifactButton';
const sideRail: ISideRailLink = { text: "Artifacts", url: "rail_artifacts" };

class JobArtifactsDataView extends JobDataView {

   filterUpdated() {

   }

   clear() {
      this.initial = true;
      this.hasArtifacts = undefined;
      super.clear();
   }

   set() {

      if (!this.initial) {
         return;
      }

      this.initial = false;

      this.hasArtifacts = !!this.details?.jobData?.artifacts?.length;
      this.initialize(this.hasArtifacts ? [sideRail] : undefined);

      // Test for upating artifacts dynamically
      /*
      setTimeout(() => {
         this.details!.jobData!.artifacts = [
            {artifactId: "abcd", "stepId": "abcd", name: "Test Artifact", type: "test-artifact"}
         ]
         this.detailsUpdated();
      }, 5000)
      */
   }


   detailsUpdated() {

      const hasArtifacts = !!this.details?.jobData?.artifacts?.length;
      if (this.hasArtifacts !== hasArtifacts) {
         this.hasArtifacts = hasArtifacts;
         this.initialize(hasArtifacts ? [sideRail] : undefined);
         this.details?.setRootUpdated();
      }

   }

   initial = true;

   hasArtifacts?: boolean;
   order = 0;

}

JobDetailsV2.registerDataView("JobArtifactsDataView", (details: JobDetailsV2) => new JobArtifactsDataView(details));

export const JobArtifactsPanel: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const constGroups = useConst<IGroup[]>([]);
   const [selected, setSelected] = useState<GetJobArtifactResponse | undefined>(undefined);

   const artifactView = jobDetails.getDataView<JobArtifactsDataView>("JobArtifactsDataView");

   jobDetails.subscribe();

   useEffect(() => {
      return () => {
         artifactView.clear();
      }
   }, [artifactView]);

   const jobData = jobDetails.jobData;

   if (!jobData) {
      return null;
   }

   artifactView.set();

   const hasArtifacts = !!jobData.artifacts?.length;

   if (!hasArtifacts) {
      return null;
   }

   const { hordeClasses, modeColors } = getHordeStyling();

   const columns: IColumn[] = [
      { key: 'column_desc', name: 'Description', minWidth: 580, isResizable: false, isMultiline: true },
      { key: 'column_time', name: 'Time', minWidth: 120, isResizable: false, isMultiline: true },
      { key: 'column_download', name: 'Download', minWidth: 300, isResizable: false, isMultiline: true },
   ];

   let artifacts = [...jobData.artifacts!];

   // artifact name => category (id may be unpopulated)
   const clookup: Map<string, string> = new Map();

   let grouped: GetJobArtifactResponse[] = [];
   const ungrouped: GetJobArtifactResponse[] = [];

   const zenArtifacts = new Set<string>();

   artifacts.forEach(a => {

      let category = "";

      if(a.metadata.find(m => m.toLowerCase() === "backend=zen"))
      {
         zenArtifacts.add(a.id);
      }

      const m = a.metadata.find(m => m.toLowerCase().startsWith("dashboard-category"));

      if (m) {
         const elements = m.split("=");
         if (elements.length > 1) {
            if (elements[1].trim()) {
               category = elements[1].trim();
            }
         }
      }

      if (category) {
         clookup.set(a.name, category);
         grouped.push(a);
      }
      else {
         ungrouped.push(a);
      }


   })

   grouped = grouped.sort((a, b) => {

      const acat = clookup.get(a.name)!;
      const bcat = clookup.get(b.name)!;

      if (acat !== bcat) {

         return acat.localeCompare(bcat);
      }

      return a.name.localeCompare(b.name)
   });


   let groups: IGroup[] = [];

   let cgroup: string = "";

   // emit groups
   for (let i = 0; i < grouped.length; i++) {

      const a = grouped[i];
      const cat = clookup.get(a.name)!;

      if (cat != cgroup) {

         if (cgroup) {
            groups[groups.length - 1].count = i - groups[groups.length - 1].startIndex;
         }

         cgroup = cat;

         if (cgroup) {
            const key = `group_key_${cgroup}`;
            groups.push({ startIndex: i, name: cgroup, key: key, count: 0, isCollapsed: constGroups.find(g => g.key === key)?.isCollapsed ?? true });
         }
      }
   }

   if (cgroup) {
      groups[groups.length - 1].count = grouped.length - groups[groups.length - 1].startIndex;
   }

   if (groups?.length) {
      while (constGroups.length > 0) {
         constGroups.pop();
      }

      constGroups.push(...groups);

   }

   const renderItem = (item: GetJobArtifactResponse, index?: number, column?: IColumn) => {

      if (!column) {
         return null;
      }

      let step = jobDetails.stepById(item.stepId);

      let stepFinished = !!step?.finishTime;

      if (column.key === 'column_download') {
         return <Stack horizontalAlign="end" verticalAlign="center" verticalFill={true} style={{ paddingRight: 8 }}>
            <Stack horizontal tokens={{ childrenGap: 18 }}>
               {!zenArtifacts.has(item.id) && <PrimaryButton text="Browse"
                  disabled={!stepFinished || !item.id}
                  style={{ fontFamily: "Horde Open Sans SemiBold" }}
                  onClick={() => {
                     setSelected(item)
                  }}
               />}
               <ArtifactButton   
                  disabled={!stepFinished || !item.id}
                  artifact={item}
               />
            </Stack>
         </Stack>
      }

      let eta = {
         display: "",
         server: ""
      };

      if (column.key === 'column_time' && !!step) {

         if (step.state === JobStepState.Skipped) {
            return null;
         }

         let finished = { display: "", server: "" };

         eta = getStepETA(step, jobDetails.jobData!);

         finished = getStepFinishTime(step);

         if (finished.display) {
            eta.display = finished.display;
            eta.server = finished.server;
         }

         return <Stack horizontalAlign={"end"} verticalAlign="center" verticalFill={true}>
            <Stack horizontal tokens={{ childrenGap: 2 }}>
               {!!eta.display && !step.finishTime && <Text style={{ fontSize: "11px", paddingTop: 2 }}>~</Text>}
               <Text style={{ fontSize: "13px" }}>
                  {eta.display}
               </Text>
            </Stack>
         </Stack>

      }

      if (column.key === 'column_desc') {

         let color = getStepStatusColor(step?.state, step?.outcome);
         if (step?.state === JobStepState.Waiting || step?.state === JobStepState.Ready || step?.state === JobStepState.Running) {
            const colors = dashboard.getStatusColors();
            color = colors.get(StatusColor.Running)!;
         }

         return <Stack horizontal verticalAlign="center" verticalFill={true} >
            {!!step && <FontIcon iconName="Square" style={{ color: color, paddingTop: 2, fontSize: 13, paddingRight: 8 }} />}
            <Text style={{ color: modeColors.text, fontFamily: "Horde Open Sans SemiBold" }}>{item.description ?? item.name}</Text>
         </Stack>
      }

      return null;
   };


   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      {!!selected && <JobArtifactsModal jobId={jobData.id} stepId={selected.stepId} contextType={selected.type} artifactId={selected.id} onClose={() => setSelected(undefined)} />}
      <Stack className={hordeClasses.raised} >
         <Stack tokens={{ childrenGap: 12 }} grow>
            <Stack horizontal>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Artifacts</Text>
               </Stack>
            </Stack>
            <Stack >
               <Stack style={{ paddingTop: 8 }} tokens={{ childrenGap: 12 }}>
                  {!!ungrouped.length && <DetailsList
                     styles={{ root: { overflowX: "hidden" } }}
                     isHeaderVisible={false}
                     items={ungrouped}
                     columns={columns}
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.justified}
                     compact
                     onRenderItemColumn={renderItem}
                  />}

                  {!!grouped.length && <DetailsList
                     styles={{ root: { overflowX: "hidden" } }}
                     isHeaderVisible={false}
                     items={grouped}
                     groups={groups}
                     columns={columns}
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.justified}
                     compact
                     onRenderItemColumn={renderItem}
                  />}
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>)
});
