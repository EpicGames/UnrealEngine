// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useEffect } from 'react';
import backend from '../../backend';
import { GetBisectTaskResponse } from '../../backend/Api';
import { PollBase } from '../../backend/PollBase';
import { ISideRailLink } from '../../base/components/SideRail';
import { BisectionList } from '../bisection/BisectionList';
import { JobDataView, JobDetailsV2 } from './JobDetailsViewCommon';
import { getHordeStyling } from '../../styles/Styles';

const sideRail: ISideRailLink = { text: "Bisection", url: "rail_detail_bisection" };

class BisectionHandler extends PollBase {

   constructor(view: StepBisectionView, pollTime = 10000) {
      super(pollTime);
      this.view = view;
   }

   clear() {
      super.stop();
   }

   async poll(): Promise<void> {

      let bisections: GetBisectTaskResponse[] = [];

      const view = this.view;

      try {

         const jobId = await view.getBisectionJobId();

         if (!jobId) {
            return;
         }

         bisections = await backend.getBisections({ jobId: jobId });
         if (view.stepName) {
            bisections = bisections.filter(r => r.nodeName === view.stepName);
         }


      } catch (err) {
         console.error(err);
      } finally {

         view.handlerUpdated(bisections);         
      }
   }

   view: StepBisectionView;

}

class StepBisectionView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   bisectionUpdated() {
      this.gotoRail = true;
      this.handler?.stop();
      this.handler?.start();
   }

   handlerUpdated(bisections: GetBisectTaskResponse[]) {
      
      const prev = this.bisections;
      this.bisections = bisections;

      if (!this.initialized) {
         this.initialize(bisections?.length ? [sideRail] : undefined);
      } else {
         if (!prev?.length && bisections.length) {
            // didn't have bisections and now we do
            this.addRail(sideRail);
         } 
      }

      this.updateReady();
   }

   // 
   async getBisectionJobId() {

      if (this.bisectionJobId) {
         return this.bisectionJobId;
      }

      const details = this.details;

      if (!details) {
         return;
      }

      let jobId = details.jobData?.startedByBisectTaskId ? details.jobData?.startedByBisectTaskId : details.jobId!;

      if (jobId !== details.jobId) {
         const task = await backend.getBisectTask(jobId);
         jobId = task.initialJobId;
      }

      this.bisectionJobId = jobId;

      return jobId;

   }

   get stepName(): string | undefined {
      return this.details?.getStepName(this.stepId);
   }

   async set(stepId?: string) {

      if (this.bisectionJobId) {
         return;
      }

      try {

         const bisectionJobId = await this.getBisectionJobId()!;

         if (!bisectionJobId) {
            return;
         }

         this.stepId = stepId;

         this.handler = new BisectionHandler(this);
         this.handler.start();

      } catch (error) {
         console.error(error);
      } finally {

      }
   }

   clear() {
      this.stepId = undefined;
      this.bisections = undefined;
      this.handler?.clear();
      super.clear();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();

   }

   stepId?: string;

   bisections?: GetBisectTaskResponse[];

   handler?: BisectionHandler;

   bisectionJobId?: string;

   gotoRail = false;

   order = 9;

}

JobDetailsV2.registerDataView("StepBisectionView", (details: JobDetailsV2) => new StepBisectionView(details));

export const BisectionPanel: React.FC<{ jobDetails: JobDetailsV2, stepId?: string }> = observer(({ jobDetails, stepId }) => {

   // Bisection is currently disabled via feature flag
   return null;

   const dataView = jobDetails.getDataView<StepBisectionView>("StepBisectionView");

   const { hordeClasses } = getHordeStyling();

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   dataView.subscribe();

   dataView.set(stepId);

   if (!dataView.bisections?.length) {
      return null;
   }

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }


   if (dataView.gotoRail) {
      dataView.gotoRail = false;
      window.location.hash = sideRail.url;
   }

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Bisection</Text>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', minHeight: "400px", maxHeight: "400px" }} data-is-scrollable={true}>
                  <BisectionList bisections={dataView.bisections} />
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});


