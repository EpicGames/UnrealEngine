// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, DetailsRow, IColumn, IDetailsListProps, mergeStyleSets, SelectionMode, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useState } from 'react';
import { Link, useLocation } from 'react-router-dom';
import backend from '../../backend';
import { GetExternalIssueResponse, GetIssueResponse, IssueData } from '../../backend/Api';
import { ISideRailLink } from '../../base/components/SideRail';
import { getElapsedString, getShortNiceTime } from '../../base/utilities/timeUtils';
import { IssueModalV2 } from '../IssueViewV2';
import { useQuery } from "horde/base/utilities/hooks";
import { IssueStatusIcon } from '../StatusIcon';
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from '../../styles/Styles';

const sideRail: ISideRailLink = { text: "Health", url: "rail_health" };

const jiraIssueCache: Map<string, { issue: GetExternalIssueResponse, cacheTime: moment.Moment }> = new Map();


type IssueCacheItem = {
   time: moment.Moment;
   issues: GetIssueResponse[];
}
const issueCache: Map<string, IssueCacheItem> = new Map();

class HealthDataView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   async set(stepId?: string, labelIdx?: number) {

      const details = this.details;

      if (!details) {
         return;
      }

      if (!details.jobId || !details.jobData) {
         this.issues = [];
         return;
      }

      if (this.stepId === stepId && this.labelIdx === labelIdx) {
         return;
      }

      this.stepId = stepId;
      this.labelIdx = labelIdx;

      this.issues = [];

      if (details.jobData) {
         this.updateReady();
      }

      const key = details.jobId + this.stepId + this.labelIdx;

      let issueRequests: Promise<IssueData[]>[] = [];

      let cached = issueCache.get(key);
      if (cached) {
         if (moment.duration(moment(Date.now()).diff(cached.time)).asMinutes() > 2) {
            cached = undefined;
         }
      }

      if (!cached) {
         issueRequests = [backend.getIssues({ jobId: details.jobId, stepId: this.stepId, label: this.labelIdx, count: 50, resolved: false }),
         backend.getIssues({ jobId: details.jobId, stepId: this.stepId, label: this.labelIdx, count: 50, resolved: true })];
      } else {
         this.issues = cached.issues;
      }

      if (issueRequests.length) {

         const responses = await Promise.all(issueRequests);
         let issues: GetIssueResponse[] = [];
         responses.forEach(r => issues.push(...r));
         this.issues = issues;
         issueCache.set(key, { issues: issues, time: moment(Date.now()) });
      }

      let jiraKeys: string[] = [];

      const jiraIssues: Set<number> = new Set();
      this.issues.forEach(issue => {
         if (issue.externalIssueKey) {
            jiraIssues.add(issue.id)
            jiraKeys.push(issue.externalIssueKey);
         }
      });

      const now = moment(Date.now());
      jiraKeys = jiraKeys.filter(j => {
         const issue = jiraIssueCache.get(j);
         if (!issue) {
            return true;
         }
         if (moment.duration(now.diff(issue.cacheTime)).asMinutes() <= 2) {
            return false;
         }
         return true;
      });

      if (jiraKeys.length) {
         const jiras = await backend.getExternalIssues(details.jobData!.streamId, jiraKeys);
         jiras.forEach(j => {
            jiraIssueCache.set(j.key, { issue: j, cacheTime: moment(Date.now()) });
         })
      }


      this.initialize(this.issues?.length ? [sideRail] : undefined)
      this.updateReady();


   }

   clear() {
      this.issues = [];
      this.stepId = null;
      this.labelIdx = null;
      super.clear();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();
   }

   issues: GetIssueResponse[] = [];

   stepId?: string | null = null;
   labelIdx?: number | null = null;

   order = 2;

}

JobDetailsV2.registerDataView("HealthDataView", (details: JobDetailsV2) => new HealthDataView(details));

export const HealthPanel: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const query = useQuery();
   const location = useLocation();
   const [issueHistory, setIssueHistory] = useState(false);

   const stepId = query.get("step") ? query.get("step")! : undefined;
   const labelIdx = query.get("label") ? query.get("label")! : undefined;

   const dataView = jobDetails.getDataView<HealthDataView>("HealthDataView");

   if (stepId) {
      dataView.order = 1;
   }

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   dataView.subscribe();

   const { hordeClasses } = getHordeStyling();

   // query changes and immediately initializes
   dataView.set(stepId, labelIdx ? parseInt(labelIdx) : undefined);

   let issues: GetIssueResponse[] = Object.assign([], dataView.issues);

   if (!jobDetails.jobData || !issues.length) {
      return null;
   }

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }


   type HealthItem = {
      issue: GetIssueResponse;
   };

   let items: HealthItem[] = issues.map(i => {
      return { issue: i }
   });

   items = items.sort((a, b) => {
      if (a.issue.resolvedAt && !b.issue.resolvedAt) {
         return 1;
      }

      if (!a.issue.resolvedAt && b.issue.resolvedAt) {
         return -1;
      }

      return b.issue.id - a.issue.id
   });

   const columns = [
      { key: 'health_column1', name: 'Summary', minWidth: 460, maxWidth: 460, isResizable: false },
      { key: 'health_column2', name: 'Quarantine', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'health_column3', name: 'Jira', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'health_column4', name: 'JiraPriority', minWidth: 64, maxWidth: 64, isResizable: false },
      { key: 'health_column5', name: 'JiraAssignee', minWidth: 180, maxWidth: 180, isResizable: false },
      { key: 'health_column6', name: 'JiraStatus', minWidth: 64, maxWidth: 64, isResizable: false },
      { key: 'health_column7', name: 'Status', minWidth: 140, maxWidth: 140, isResizable: false },
      { key: 'health_column8', name: 'Opened', minWidth: 140, maxWidth: 150, isResizable: false },
   ];

   const renderItem = (item: HealthItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      const issue = item.issue;

      const textSize = "small";

      let ack = issue.acknowledgedAt ? "" : "Pending";
      let status = "Unassigned";
      if (issue.ownerInfo?.name) {
         status = issue.ownerInfo?.name;
      }
      if (issue.resolvedAt) {
         status = "Resolved";
         if (issue.fixChange) {
            status += ` in CL ${issue.fixChange}`;
         }
      }

      if (status === "Unassigned" || issue.resolvedAt) {
         ack = "";
      }

      if (column.name === "Summary") {

         let summary = issue.summary;
         if (summary.length > 100) {
            summary = summary.slice(0, 100) + "...";
         }

         const newQuery = query.getCopy();
         newQuery.set("issue", item.issue.id.toString());
         const href = `${location.pathname}?${newQuery.toString()}` + window.location.hash;

         return <Link to={href} className="job-item" onClick={() => setIssueHistory(true)}><Stack horizontal disableShrink={true}>{<IssueStatusIcon issue={issue} />}<Text variant={textSize} style={{ textDecoration: !!issue.resolvedAt ? "line-through" : undefined }}>{`Issue ${issue.id} - ${summary}`}</Text></Stack></Link>;
      }

      if (column.name === "Quarantine") {

         if (!issue.quarantinedByUserInfo && !issue.workflowThreadUrl) {
            return null;
         }

         if (issue.workflowThreadUrl) {
            return <a onClick={(e) => e.stopPropagation()} href={issue.workflowThreadUrl} target="_blank" rel="noreferrer"><Stack style={{ height: "100%" }} horizontalAlign="center" disableShrink={true}  verticalAlign="center" >Slack Thread</Stack></a>
         }

         return <Stack horizontalAlign="start" disableShrink={true} verticalAlign="center" ><Text variant={textSize}>Quarantined</Text></Stack>;
      }


      if (column.name === "Status") {

         if (ack) {
            status += ` (${ack})`;
         }

         return <Stack horizontalAlign="end" disableShrink={true} verticalAlign="center"><Text variant={textSize}>{status}</Text></Stack>;
      }

      if (issue.externalIssueKey) {

         const jissue = jiraIssueCache.get(issue.externalIssueKey)?.issue;

         const textDecoration = jissue?.resolutionName ? "line-through" : undefined;

         if (column.name === "JiraPriority") {

            if (!jissue || !jissue.priorityName) {
               return null;
            }

            let priority = jissue.priorityName;
            if (priority.indexOf("- ") !== -1) {
               priority = priority.slice(priority.indexOf("- ") + 2);
            }

            return <Stack horizontalAlign="start" disableShrink={true}><a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{priority}</Text> </a></Stack>;

         }


         if (column.name === "JiraAssignee") {

            if (!jissue) {
               return null;
            }

            return <Stack horizontalAlign="center" disableShrink={true}><a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{jissue.assigneeDisplayName ?? "Unassigned (Jira)"}</Text> </a></Stack>;
         }

         if (column.name === "JiraStatus") {

            const desc = jissue?.resolutionName ?? jissue?.statusName;

            if (!jissue) {
               return null;
            }

            return <Stack horizontalAlign="start" disableShrink={true}><a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{desc}</Text> </a></Stack>;
         }

         if (column.name === "Jira") {

            if (!jissue) {
               return null;
            }

            return <Stack horizontalAlign="start" disableShrink={true}><a href={jissue.link} target="_blank" rel="noreferrer" onClick={(ev) => ev.stopPropagation()}><Text style={{ textDecoration: textDecoration }} variant={textSize}>{jissue.key}</Text> </a></Stack>;
         }
      }


      if (column.name === "Ack") {
         return <Text>{ack}</Text>;
      }


      if (column.name === "Opened") {
         const openSince = `${getShortNiceTime(issue.createdAt)} (${getElapsedString(moment(issue.createdAt), moment.utc(), false).trim()})`;
         return <Stack style={{ paddingRight: 8 }} horizontalAlign="end" verticalAlign="center"><Text variant={textSize}>{openSince}</Text></Stack>;
      }

      return null;
   }

   const classes = mergeStyleSets({
      detailsRow: {
         selectors: {
            '.ms-DetailsRow-cell': {
               padding: 8,
               overflow: "hidden",
               whiteSpace: "nowrap"
            }
         }
      }
   });

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         return <DetailsRow className={classes.detailsRow} {...props} />

      }
      return null;
   };

   let title = "Health";

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{title}</Text>
            <Stack styles={{ root: { paddingLeft: 0, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <IssueModalV2 issueId={query.get("issue")} popHistoryOnClose={issueHistory} />
               <DetailsList
                  compact={true}
                  isHeaderVisible={false}
                  indentWidth={0}
                  items={items}
                  columns={columns}
                  setKey="set"
                  selectionMode={SelectionMode.none}
                  layoutMode={DetailsListLayoutMode.justified}
                  onRenderItemColumn={renderItem}
                  onRenderRow={renderRow}
               />
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});