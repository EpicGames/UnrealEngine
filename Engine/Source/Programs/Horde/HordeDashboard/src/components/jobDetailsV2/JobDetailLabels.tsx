// Copyright Epic Games, Inc. All Rights Reserved.

import { ConstrainMode, DefaultButton, DetailsList, DetailsListLayoutMode, DirectionalHint, IColumn, SelectionMode, Stack, Text, TooltipHost } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { useNavigate, useLocation } from "react-router-dom";
import { GetStepResponse, JobLabel, JobStepOutcome, LabelOutcome, LabelState, StepData } from "../../backend/Api";
import { getLabelColor } from "../../styles/colors";
import { useQuery } from "horde/base/utilities/hooks";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from "../../styles/Styles";
import dashboard from "../../backend/Dashboard";
import { Link } from "react-router-dom";
import { StepStatusIcon } from "../StatusIcon";
import { LabelHoverCard } from "../labels/LabelHoverCard";

export const LabelsPanelV2: React.FC<{ jobDetails: JobDetailsV2, dataView: JobDataView }> = observer(({ jobDetails, dataView }) => {

   const query = useQuery();
   const navigate = useNavigate();
   const location = useLocation();

   const { hordeClasses, modeColors } = getHordeStyling();

   dataView.subscribe();

   const labelIdx = query.get("label") ? parseInt(query.get("label")!) : undefined;
   const qlabel = jobDetails.labelByIndex(labelIdx);
   if (typeof (labelIdx) === `number` && !qlabel) {
      return null;
   }

   const jobFilter = jobDetails.filter;

   // subscribe   
   if (jobFilter.inputChanged) { }

   const batchFilter = query.get("batch");
   if (batchFilter) {
      return null;
   }

   let labels = jobDetails.labels.filter(label => label.stateResponse.state !== LabelState.Unspecified);

   if (qlabel) {
      labels = labels.filter(l => l.stateResponse.dashboardCategory === qlabel?.stateResponse.dashboardCategory && l.stateResponse.dashboardName === qlabel?.stateResponse.dashboardName);
   }

   const stateFilter = jobFilter.filterStates;
   labels = labels.filter(label => {

      if (stateFilter.indexOf("All") !== -1) {
         return true;
      }

      let include = false;

      if (stateFilter.indexOf("Failure") !== -1) {
         if (label.stateResponse.outcome === LabelOutcome.Failure) {
            include = true;
         }
      }

      if (stateFilter.indexOf("Warnings") !== -1) {
         if (label.stateResponse.outcome === LabelOutcome.Warnings) {
            include = true;
         }
      }

      if (stateFilter.indexOf("Completed") !== -1) {
         if (label.stateResponse.state === LabelState.Complete) {
            include = true;
         }
      }

      if (stateFilter.indexOf("Running") !== -1) {
         if (label.stateResponse.state === LabelState.Running) {
            include = true;
         }
      }

      return include;

   });

   if (!labels) {
      return null;
   }

   type LabelItem = {
      category: string;
      labels: JobLabel[];
   };

   const categories: Set<string> = new Set();
   labels.forEach(label => { if (label.stateResponse.dashboardName ?? "") { categories.add(label.stateResponse.dashboardCategory!); } });


   let items = Array.from(categories.values()).map(c => {
      return {
         category: c,
         labels: labels.filter(label => label.stateResponse.dashboardName && (label.stateResponse.dashboardCategory === c)).sort((a, b) => {
            return a.stateResponse.dashboardName! < b.stateResponse.dashboardName! ? -1 : 1;
         })
      } as LabelItem;
   }).filter(item => item.labels?.length).sort((a, b) => {
      return a.category < b.category ? -1 : 1;
   });

   const filter = jobDetails.filter;

   if (filter.search) {
      items = items.filter(i => {
         i.labels = i.labels.filter(label => (label.stateResponse.dashboardName ?? "").toLowerCase().indexOf(filter.search!.toLowerCase()) !== -1);
         return i.labels.length !== 0;
      });
   }

   if (filter.currentInput) {
      items = items.filter(i => {
         i.labels = i.labels.filter(label => (label.stateResponse.dashboardName ?? "").toLowerCase().indexOf(filter.currentInput!.toLowerCase()) !== -1);
         return i.labels.length !== 0;
      });
   }

   if (!items.length) {
      return null;
   }


   const label = jobDetails.filter.label;

   const job = jobDetails.jobData!;

   type StepItem = {
      step: StepData;
      name: string;
   }

   const jobId = job.id;

   if (!job.batches) {
      return null;
   }

   let steps: GetStepResponse[] = [];

   steps = job.batches.map(b => b.steps).flat().filter(step => !!step.startTime && (step.outcome === JobStepOutcome.Warnings || step.outcome === JobStepOutcome.Failure));
   
   // sort by severity
   steps = steps.sort((a, b) => a.outcome.localeCompare(b.outcome));

   if (label) {
      steps = steps.filter(s => label?.steps.indexOf(s.id) === -1 ? false : true);
   }

   const renderSteps = () => {

      if (!steps || !steps.length) {
         return null;
      }

      const maxStepsShown = 20
      let remainingSteps
      let remainingErrorsCount = 0
      let remainingWarningsCount = 0

      if (steps.length > maxStepsShown) {
         remainingSteps =  steps.slice(maxStepsShown, steps.length);
         remainingErrorsCount = remainingSteps.filter(s => s.outcome == "Failure").length
         remainingWarningsCount = remainingSteps.filter(s => s.outcome == "Warnings").length
         steps = steps.slice(0, maxStepsShown);
      }

      const onRenderCell = (stepItem: StepItem): JSX.Element => {

         const step = stepItem.step;

         const stepUrl = `/job/${jobId}?step=${step.id}`;

         return <Stack key={`step_${step.id}_job_${job.id}_${stepItem.name}`}>
            <Stack horizontal>
               <Link to={stepUrl} onClick={(ev) => { ev.stopPropagation(); }}><div style={{ cursor: "pointer" }}>
                  <Stack horizontal verticalAlign="center">
                     <StepStatusIcon step={step} style={{ fontSize: 13 }} />
                     <TooltipHost
                        content={stepItem.name.length > 30 ? stepItem.name : ''}
                        directionalHint={DirectionalHint.bottomCenter}>
                           <Text styles={{ root: { fontSize: 12, paddingRight: 4, paddingTop: 2, userSelect: "none" } }}>
                              {stepItem.name.length <= 30 ? stepItem.name : `${stepItem.name.slice(0, 27)}...` }
                           </Text>
                     </TooltipHost>
                  </Stack>
               </div></Link>
            </Stack>
         </Stack>;
      };


      let items = steps.map(step => {


         return {
            step: step,
            name: step.name
         }

      })

      items = items.filter(item => !!item);

      const render = items.map(s => onRenderCell(s!));

      return <Stack>
               <Stack horizontal wrap tokens={{ childrenGap: 12}} style={{paddingBottom: 10}}>{render}</Stack>
               {!!remainingSteps && <Text style={{ fontSize: 13 }}>
                  {!!remainingErrorsCount && <Text>{remainingErrorsCount} {remainingErrorsCount === 1 ? "error" : "errors"} </Text>}
                  {!!remainingErrorsCount && !!remainingWarningsCount ? "and " : ""}
                  {!!remainingWarningsCount && <Text>{remainingWarningsCount} {remainingWarningsCount === 1 ? "warning" : "warnings"} </Text>}
                  not shown. Please select individual labels to view additional issues.</Text>}
            </Stack>;

   }

   const buildColumns = (): IColumn[] => {

      const widths: Record<string, number> = {
         "Name": 120,
         "Labels": 1024
      };

      const cnames = ["Name", "Labels"];

      return cnames.map(c => {
         return { key: c, name: c === "Status" ? "" : c, fieldName: c.replace(" ", "").toLowerCase(), minWidth: widths[c] ?? 100, maxWidth: widths[c] ?? 100, isResizable: false, isMultiline: true } as IColumn;
      });

   };

   function onRenderLabelColumn(item: LabelItem, index?: number, column?: IColumn) {

      switch (column!.key) {

         case 'Name':
            return <Stack verticalAlign="center" verticalFill={true}> <Text style={{ fontFamily: "Horde Open Sans SemiBold" }}>{item.category}</Text> </Stack>;

         case 'Labels':
            return <Stack wrap horizontal tokens={{ childrenGap: 4 }} styles={{ root: { paddingTop: 2 } }}>
               {item.labels.map(label => {
                  const color = getLabelColor(label.stateResponse.state, label.stateResponse.outcome);
                  let textColor: string | undefined = undefined;

                  let filtered = false;
                  if (qlabel) {
                     if (qlabel.stateResponse.dashboardCategory !== label.stateResponse.dashboardCategory || qlabel.stateResponse.dashboardName !== label.stateResponse.dashboardName) {
                        filtered = true;
                     }
                  }

                  const labelIdx = label.stateResponse.dashboardName?.length
                     ? jobDetails.labelIndex(label.stateResponse.dashboardName, label.stateResponse.dashboardCategory)
                     : -1;

                  const badge = <Stack key={`labels_${label.stateResponse.dashboardCategory}_${label.stateResponse.dashboardName}`} className={hordeClasses.badge}>
                     <DefaultButton
                        onClick={() => {
                           if (qlabel?.stateResponse.dashboardCategory === label.stateResponse.dashboardCategory && qlabel?.stateResponse.dashboardName === label.stateResponse.dashboardName) {
                              navigate(location.pathname)
                           } else {
                              if (labelIdx >= 0) {
                                 navigate(location.pathname + `?label=${labelIdx}`)
                              } else {
                                 navigate(location.pathname, { replace: true })
                              }
                           }

                        }}
                        key={label.stateResponse.dashboardName ?? ""} style={{ backgroundColor: color.primaryColor, color: textColor, filter: filtered ? "brightness(0.70)" : undefined }}
                        text={label.stateResponse.dashboardName ?? ""}>
                        {!!color.secondaryColor && <div style={{
                           borderLeft: "10px solid transparent",
                           borderRight: `10px solid ${color.secondaryColor}`,
                           borderBottom: "10px solid transparent",
                           height: 0,
                           width: 0,
                           position: "absolute",
                           right: 0,
                           top: 0,
                           zIndex: 1,
                           filter: filtered ? "brightness(0.70)" : undefined
                        }} />}
                     </DefaultButton>
                  </Stack>;

                  return <LabelHoverCard
                     key={`labelhover_${label.stateResponse.dashboardCategory}_${label.stateResponse.dashboardName}`}
                     jobId={jobDetails.jobId!}
                     labelIndex={labelIdx}
                     streamId={jobDetails.jobData?.streamId ?? ''}
                     templateId={jobDetails.jobData?.templateId ?? ''}
                     templateName={jobDetails.jobData?.name}
                     dashboardName={label.stateResponse.dashboardName}
                     dashboardCategory={label.stateResponse.dashboardCategory}
                     state={label.stateResponse.state}
                     outcome={label.stateResponse.outcome}
                     change={jobDetails.jobData?.change}
                  >
                     {badge}
                  </LabelHoverCard>;
               })}</Stack>;

         default:
            return <span>???</span>;
      }
   }


   return <Stack styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack style={{ paddingLeft: 12 }}>
         <Stack styles={{ root: { paddingBottom: 12, selectors: { '.ms-DetailsRow': { backgroundColor: dashboard.darktheme ? modeColors.content : undefined } } } }}>
            <DetailsList
               isHeaderVisible={false}
               indentWidth={0}
               compact={true}
               selectionMode={SelectionMode.none}
               items={items}
               columns={buildColumns()}
               layoutMode={DetailsListLayoutMode.fixedColumns}
               constrainMode={ConstrainMode.unconstrained}
               onRenderItemColumn={onRenderLabelColumn}
               onShouldVirtualize={() => { return false; }}
            />
         </Stack>
         {!!steps.length && <Stack style={{ paddingLeft: 12 }} horizontal>
            <Stack>
               <Text style={{ fontFamily: "Horde Open Sans SemiBold" }}>Issues</Text>
            </Stack>
            <Stack style={{paddingLeft: 100}}>
               {renderSteps()}
            </Stack>
         </Stack>}
      </Stack>
   </Stack>;

});
