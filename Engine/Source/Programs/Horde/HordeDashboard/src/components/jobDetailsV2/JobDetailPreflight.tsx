// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, IconButton, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useEffect, useState } from 'react';
import backend from '../../backend';
import { JobState } from '../../backend/Api';
import dashboard from '../../backend/Dashboard';
import { ISideRailLink } from '../../base/components/SideRail';
import { getPreflightCL, getPreflightDisplay } from '../../base/utilities/commitUtils';
import { errorDialogStore } from '../error/ErrorStore';
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from '../../styles/Styles';

const sideRail: ISideRailLink = { text: "Preflight", url: "rail_preflight" };

class PreflightDataView extends JobDataView {

   filterUpdated() {

   }

   clear() {
      super.clear();
   }


   detailsUpdated() {

   }

   order = 0;

}

JobDetailsV2.registerDataView("PreflightDataView", (details: JobDetailsV2) => new PreflightDataView(details));

const AutosubmitInfo: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const jobData = jobDetails.jobData!;

   const [state, setState] = useState<{ autoSubmit: boolean, inflight: boolean }>({ autoSubmit: !!jobData.autoSubmit, inflight: false });

   // subscribe
   if (jobDetails.updated) { }

   if (jobData.abortedByUserInfo) {
      return null;
   }   

   if (jobData.autoSubmitChange) {
      if (dashboard.swarmUrl) {
         const url = `${dashboard.swarmUrl}/change/${jobData.autoSubmitChange}`;
         return <Stack tokens={{ childrenGap: 8 }} style={{ paddingBottom: 12, paddingTop: 18 }}>
            <Stack style={{}}>
               <Text>Automatically submitted in <a href={url} target="_blank" rel="noreferrer" onClick={ev => ev?.stopPropagation()}>{`CL ${jobData.autoSubmitChange}`}</a></Text>
            </Stack>
         </Stack>
      } else {
         return <Stack style={{ paddingTop: 24 }}><Text>Automatically submitted in {`CL ${jobData.autoSubmitChange}`}</Text></Stack>
      }
   }

   if (jobData.autoSubmitMessage) {
      return <Stack tokens={{ childrenGap: 8 }} style={{ paddingBottom: 12, paddingTop: 18 }}>
         <Stack style={{}}>
            <Stack style={{ paddingTop: 24, whiteSpace: "pre" }}><Text>{`Unable to submit change: ${jobData.autoSubmitMessage}`}</Text></Stack>
         </Stack>
      </Stack>
   }

   if (jobData.state !== JobState.Complete) {

      const preflightCL = getPreflightCL(jobData.preflightCommitId, jobData.preflightChange);
      if (preflightCL) {
         return <Stack tokens={{ childrenGap: 8 }} style={{ paddingBottom: 12, paddingTop: 18 }}>
            <Stack style={{ paddingTop: 8 }}>
               <Checkbox label={`Automatically submit changelist ${preflightCL} upon preflight success`}
                  checked={state.autoSubmit}
                  disabled={state.inflight || (jobData.startedByUserInfo?.id !== dashboard.userId)}
                  onChange={(ev, checked) => {

                     const value = !!checked;

                     backend.updateJob(jobData.id, { autoSubmit: !!checked }).then(result => {
                        // messing with job object here, not great
                        jobData.autoSubmit = value;
                        setState({ autoSubmit: value, inflight: false })
                     }).catch(reason => {

                        errorDialogStore.set({
                           reason: reason,
                           title: `Error Setting Auto-submit`,
                           message: `There was an error setting job to autosubmit, reason: "${reason}"`

                        }, true);

                        // update UI to previous state                        
                        setState({ autoSubmit: !value, inflight: false })

                     })

                     // update UI
                     setState({ autoSubmit: value, inflight: true })
                  }}
               /></Stack>
         </Stack>
      }
   }

   return null;

});

export const PreflightPanel: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const { hordeClasses } = getHordeStyling();

   if (jobDetails.updated) { }

   const preflightView = jobDetails.getDataView<PreflightDataView>("PreflightDataView");

   const [commitIdCopied, setCommitIdCopied] = React.useState(false);

   useEffect(() => {
      return () => {
         preflightView.clear();
      }
   }, [preflightView]);

   const jobData = jobDetails.jobData;

   if (!jobData) {
      return null;
   }

   const pfDisplayCL = getPreflightCL(jobData.preflightCommitId, jobData.preflightChange);
   const pfDisplayText = getPreflightDisplay(jobData.preflightCommitId, jobData.preflightChange);
   const pfIsHash = jobData.preflightCommitId && !pfDisplayCL;
   const hasPreflight = !!pfDisplayText || !!jobData.preflightDescription;

   if (!preflightView.initialized) {
      preflightView.initialize(hasPreflight ? [sideRail] : undefined);
   }

   if (!hasPreflight) {
      return null;
   }

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised} >
         <Stack tokens={{ childrenGap: 12 }} grow>
            <Stack horizontal>
               <Stack>
                  {pfDisplayCL && (
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{`Preflight (CL ${pfDisplayCL})`}</Text>
                  )}
                  {pfIsHash && (
                     <Stack horizontal tokens={{ childrenGap: 8 }} verticalAlign="center">
                        <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>
                           Preflight Commit:
                        </Text>
                        <Text variant="medium" styles={{ root: { fontFamily: 'monospace' } }}>
                           {jobData.preflightCommitId!.substring(0, 12)}...
                        </Text>
                        <IconButton
                           iconProps={{ iconName: commitIdCopied ? 'CheckMark' : 'Copy' }}
                           title={commitIdCopied ? "Copied!" : "Copy full commit ID"}
                           styles={commitIdCopied ? { icon: { color: 'green' } } : undefined}
                           onClick={async () => {
                              try {
                                 await navigator.clipboard.writeText(jobData.preflightCommitId!);
                                 setCommitIdCopied(true);
                                 setTimeout(() => setCommitIdCopied(false), 2000);
                              } catch (err) {
                                 console.error('Failed to copy commit ID to clipboard:', err);
                              }
                           }}
                        />
                     </Stack>
                  )}
               </Stack>
            </Stack>
            <Stack >
               <Stack style={{ paddingLeft: 12, paddingTop: 8 }} tokens={{ childrenGap: 12 }}>
                  {!!jobData.preflightDescription && <Stack> <Text styles={{ root: { whiteSpace: "pre-wrap", fontFamily: "Horde Cousine Regular, monospace, monospace" } }}>{jobData.preflightDescription}</Text> </Stack>}
               </Stack>
               <Stack>
                  <AutosubmitInfo jobDetails={jobDetails} />
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});
