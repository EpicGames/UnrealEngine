// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, DetailsRow, FontIcon, IColumn, IDetailsListProps, SelectionMode, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useEffect, useState } from "react";
import { Link, useNavigate, useLocation } from "react-router-dom";
import backend from "../../backend";
import { BatchLogEventsEntry, EventData, EventSeverity, GetJobStepRefResponse, JobStepOutcome } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { ISideRailLink } from "../../base/components/SideRail";
import { displayTimeZone, getElapsedString } from "../../base/utilities/timeUtils";
import { ChangeButton } from "../ChangeButton";
import { renderLine } from '../logs/LogRender';
import { StepRefStatusIcon } from "../StatusIcon";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeTheme } from "../../styles/theme";
import { getHordeStyling } from "../../styles/Styles";
import { getStepRefMessageInfo } from "./JobDetailsUtils";

const sideRail: ISideRailLink = { text: "History", url: "rail_step_history" };

/** Maximum number of recent failing/warning steps to fetch log events for */
const MAX_STEPS_FOR_EVENTS = 15;

class StepHistoryDataView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   set(stepId?: string) {

      const details = this.details;

      if (!details) {
         return;
      }

      if (this.stepId === stepId || !details.jobId) {
         return;
      }

      this.stepId = stepId;

      if (!this.stepId) {
         return;
      }

      const stepName = details.getStepName(this.stepId, false);

      if (stepName) {
         this.loadHistory(stepName);
      }

   }

   loadHistory(stepName: string) {
      const jobData = this.details?.jobData;
      if (!jobData) {
         return;
      }
      backend.getJobStepHistory(jobData.streamId, stepName, 1024, jobData.templateId!).then(response => {
         this.history = response;
         this.loadLogEvents();
      }).finally(() => {
         this.initialize(this.history?.length ? [sideRail] : undefined);
         this.updateReady();
      })
   }

   async loadLogEvents() {
      if (!this.history.length) {
         return;
      }

      const candidates = this.history
         .filter(ref => (ref.outcome === JobStepOutcome.Failure || ref.outcome === JobStepOutcome.Warnings) && ref.logId)
         .slice(0, MAX_STEPS_FOR_EVENTS);

      if (!candidates.length) {
         return;
      }

      const failureLogIds = candidates.filter(ref => ref.outcome === JobStepOutcome.Failure).map(ref => ref.logId!);
      const warningLogIds = candidates.filter(ref => ref.outcome === JobStepOutcome.Warnings).map(ref => ref.logId!);

      this.loadingErrors = true;
      this.updateReady();

      try {
         const requests: Promise<void>[] = [];

         if (failureLogIds.length) {
            requests.push(backend.getBatchLogEvents({ logIds: failureLogIds, severity: EventSeverity.Error, count: 20 }).then(response => {
               for (const [logId, entry] of Object.entries(response.logEvents)) {
                  if (entry.events.length) {
                     this.logEvents.set(logId, entry);
                  }
               }
            }));
         }

         if (warningLogIds.length) {
            requests.push(backend.getBatchLogEvents({ logIds: warningLogIds, severity: EventSeverity.Warning, count: 20 }).then(response => {
               for (const [logId, entry] of Object.entries(response.logEvents)) {
                  if (entry.events.length) {
                     this.logEvents.set(logId, entry);
                  }
               }
            }));
         }

         await Promise.all(requests);
      } catch (err) {
         console.error("Failed to fetch batch log events:", err);
      }

      this.loadingErrors = false;
      this.updateReady();
   }

   clear() {
      this.history = [];
      this.stepId = undefined;
      this.logEvents = new Map();
      this.loadingErrors = false;
      super.clear();
   }

   detailsUpdated() {
      if (!this.initialized) {
         const stepName = this.details?.getStepName(this.stepId, false);
         if (stepName) {
            this.loadHistory(stepName);
         }
      }
   }

   history: GetJobStepRefResponse[] = [];

   logEvents: Map<string, BatchLogEventsEntry> = new Map();

   loadingErrors = false;

   stepId?: string;

   order = 5;

}

JobDetailsV2.registerDataView("StepHistoryDataView", (details: JobDetailsV2) => new StepHistoryDataView(details));


export const StepHistoryPanel: React.FC<{ jobDetails: JobDetailsV2; stepId: string }> = observer(({ jobDetails, stepId }) => {

   const navigate = useNavigate();
   const location = useLocation();

   const dataView = jobDetails.getDataView<StepHistoryDataView>("StepHistoryDataView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   dataView.subscribe();

   const { hordeClasses, modeColors } = getHordeStyling();
   const [expandedLogs, setExpandedLogs] = useState<Set<string>>(new Set());

   if (!jobDetails.jobData) {
      return null;
   }

   const theme = getHordeTheme();

   dataView.set(stepId);

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   type HistoryItem =
      | { kind: 'step'; ref: GetJobStepRefResponse }
      | { kind: 'event'; ref: GetJobStepRefResponse; event: EventData }
      | { kind: 'overflow'; ref: GetJobStepRefResponse; remaining: number };

   const items: HistoryItem[] = [];
   for (const h of dataView.history) {
      items.push({ kind: 'step', ref: h });
      if (h.logId && expandedLogs.has(h.logId) && dataView.logEvents.has(h.logId)) {
         const entry = dataView.logEvents.get(h.logId)!;
         for (const event of entry.events) {
            items.push({ kind: 'event', ref: h, event });
         }
         if (entry.total > entry.events.length) {
            items.push({ kind: 'overflow', ref: h, remaining: entry.total - entry.events.length });
         }
      }
   }

   if (!items.length) {
      return null;
   }

   const columns = [
      { key: 'column1', name: 'Name', minWidth: 580, maxWidth: 580, isResizable: false },
      { key: 'column2', name: 'Change', minWidth: 80, maxWidth: 80, isResizable: false },
      { key: 'column3', name: 'Started', minWidth: 180, maxWidth: 180, isResizable: false },
      { key: 'column4', name: 'Agent', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'column5', name: 'Meta', minWidth: 200, maxWidth: 200, isResizable: false },
      { key: 'column6', name: 'Duration', minWidth: 90, maxWidth: 90, isResizable: false },
   ];

   const errorColor = dashboard.getStatusColors().get(StatusColor.Failure)!;
   const warningColor = dashboard.getStatusColors().get(StatusColor.Warnings)!;

   const textColorForBg = (bg: string): string => {
      let r: number, g: number, b: number;
      const rgbMatch = bg.match(/rgb\(\s*(\d+)\s*,\s*(\d+)\s*,\s*(\d+)\s*\)/);
      if (rgbMatch) {
         r = parseInt(rgbMatch[1]);
         g = parseInt(rgbMatch[2]);
         b = parseInt(rgbMatch[3]);
      } else {
         const hex = bg.replace("#", "");
         r = parseInt(hex.substring(0, 2), 16);
         g = parseInt(hex.substring(2, 4), 16);
         b = parseInt(hex.substring(4, 6), 16);
      }
      const luminance = (0.299 * r + 0.587 * g + 0.114 * b) / 255;
      return luminance > 0.5 ? "#000000" : "#FFFFFF";
   };

   const renderItem = (item: HistoryItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      if (item.kind === 'event') {
         if (column.key !== 'column1') {
            return null;
         }

         const event = item.event;
         const gutterColor = event.severity === EventSeverity.Warning ? warningColor : errorColor;
         const logUrl = `/log/${event.logId}?lineIndex=${event.lineIndex + 1}`;
         const lines = event.lines.filter(line => line.message?.trim().length).map((line, lineIdx) => (
            <Stack key={`history_error_${event.logId}_${event.lineIndex}_${lineIdx}`} styles={{ root: { paddingLeft: 8, paddingRight: 8, lineBreak: "anywhere", whiteSpace: "pre-wrap", lineHeight: 18, fontSize: 10, fontFamily: "Horde Cousine Regular, monospace, monospace" } }}>
               <Link style={{ color: modeColors.text }} to={logUrl}>{renderLine(navigate, line, undefined, {})}</Link>
            </Stack>
         ));

         return (
            <Stack style={{ borderLeft: `6px solid ${gutterColor}`, paddingLeft: 14, paddingTop: 4, paddingBottom: 4 }}>
               {lines.length > 0 ? lines : <Text style={{ fontSize: 10 }}>Missing Log Data</Text>}
            </Stack>
         );
      }

      if (item.kind === 'overflow') {
         if (column.key !== 'column1') {
            return null;
         }
         const logUrl = `/log/${item.ref.logId}`;
         return (
            <Stack style={{ paddingLeft: 20, paddingTop: 4, paddingBottom: 4 }}>
               <Link to={logUrl} style={{ fontSize: 10, color: modeColors.text }}>View {item.remaining} more in full log...</Link>
            </Stack>
         );
      }

      const ref = item.ref;

      const step = jobDetails.stepById(stepId);

      if (column.name === "Name") {
         return <Stack horizontal>{<StepRefStatusIcon stepRef={ref} />}<Text>{step?.name}</Text></Stack>;
      }

      if (column.name === "Change") {

         const sindex = dataView.history.indexOf(ref);
         return <ChangeButton job={jobDetails.jobData!} stepRef={ref} hideAborted={true} rangeCL={sindex < (dataView.history.length - 1) ? (dataView.history[sindex + 1].change + 1) : undefined} />;

      }

      if (column.name === "Agent") {

         const agentId = item.ref.agentId;

         if (!agentId) {
            return null;
         }

         let url = `${location.pathname}?agentId=${agentId}`;
         if (location.search) {
            url = `${location.pathname}${location.search}&agentId=${agentId}`;
         }

         return <a href={url} onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(url, { replace: true }); }}><Stack horizontal horizontalAlign={"end"} verticalFill={true} tokens={{ childrenGap: 0, padding: 0 }}><Text>{agentId}</Text></Stack></a>;
      }

      if (column.name === "Meta") {
         let { message: stepMessage, color: stepMessageColor } = getStepRefMessageInfo(ref);
         const logEntry = ref.logId ? dataView.logEvents.get(ref.logId) : undefined;
         const eventCount = logEntry?.total ?? 0;

         if (eventCount > 0) {
            const isExpanded = expandedLogs.has(ref.logId!);
            const btnColor = ref.outcome === JobStepOutcome.Failure ? errorColor : warningColor;
            const btnTextColor = textColorForBg(btnColor);
            return <Stack verticalAlign="center" verticalFill horizontal horizontalAlign="center">
                  <DefaultButton
                     styles={{
                        root: { minWidth: 0, height: 20, padding: "0 6px", backgroundColor: btnColor, borderColor: btnColor, borderRadius: 2, color: btnTextColor },
                        rootHovered: { backgroundColor: btnColor, borderColor: btnColor, color: btnTextColor, opacity: 0.9 },
                        rootPressed: { backgroundColor: btnColor, borderColor: btnColor, color: btnTextColor, opacity: 0.8 },
                        label: { fontSize: 10, fontFamily: "Horde Open Sans SemiBold", color: btnTextColor, lineHeight: "18px", margin: 0, padding: 0 }
                     }}
                     onClick={(ev) => {
                        ev.preventDefault();
                        ev.stopPropagation();
                        const next = new Set(expandedLogs);
                        if (isExpanded) {
                           next.delete(ref.logId!);
                        } else {
                           next.add(ref.logId!);
                        }
                        setExpandedLogs(next);
                     }}
                  ><FontIcon iconName={isExpanded ? "ChevronDown" : "ChevronRight"} style={{ fontSize: 8, marginRight: 4, position: "relative", top: 1, color: btnTextColor }} />{stepMessage} ({eventCount})</DefaultButton>
               </Stack>
         }

         return <Stack verticalAlign="center" horizontal tokens={{ childrenGap: 6 }} horizontalAlign="center">
               {!!stepMessageColor && <FontIcon style={{ fontSize: 13, color: stepMessageColor, paddingTop: 1 }} iconName="Square" />}
               <Text style={{fontSize: 12, overflow: "hidden", textOverflow: "ellipsis"}}>{stepMessage}</Text>
            </Stack>
      }


      if (column.name === "Started") {

         if (ref.startTime) {

            const displayTime = moment(ref.startTime).tz(displayTimeZone());
            const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

            let displayTimeStr = displayTime.format('MMM Do') + ` at ${displayTime.format(format)}`;

            return <Stack horizontal horizontalAlign={"start"}>{displayTimeStr}</Stack>;

         } else {
            return "???";
         }
      }

      if (column.name === "Duration") {

         const start = moment(ref.startTime);
         let end = moment(Date.now());

         if (ref.finishTime) {
            end = moment(ref.finishTime);
         }
         if (item.ref.startTime) {
            const time = getElapsedString(start, end);
            return <Stack horizontal horizontalAlign={"end"} style={{ paddingRight: 8 }}><Text>{time}</Text></Stack>;
         } else {
            return "???";
         }
      }


      return <Stack />;
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as HistoryItem;

         if (item.kind === 'event' || item.kind === 'overflow') {
            const commonSelectors = { ".ms-DetailsRow-cell": { "overflow": "visible" } };
            props.styles = { ...props.styles, root: { background: theme.palette.neutralLighterAlt, selectors: { ...commonSelectors as any, "&:hover": { background: theme.palette.neutralLighter } } } };
            return <div className="job-item"><DetailsRow {...props} /></div>;
         }

         const ref = item.ref;
         const url = `/job/${ref.jobId}?step=${ref.stepId}`;

         const commonSelectors = { ".ms-DetailsRow-cell": { "overflow": "visible" } };

         if (ref.stepId === stepId && ref.jobId === jobDetails.jobId) {
            props.styles = { ...props.styles, root: { background: `${theme.palette.neutralLight} !important`, selectors: { ...commonSelectors as any } } };
         } else {
            props.styles = { ...props.styles, root: { selectors: { ...commonSelectors as any } } };
         }

         return <Link to={url}><div className="job-item"><DetailsRow {...props} /> </div></Link>;

      }
      return null;
   };

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal tokens={{ childrenGap: 18 }}>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{"History"}</Text>
               {dataView.loadingErrors && <Spinner size={SpinnerSize.small} />}
            </Stack>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "400px" }} data-is-scrollable={true}>
                  {!!dataView.history.length && <DetailsList
                     compact={true}
                     isHeaderVisible={false}
                     indentWidth={0}
                     items={items}
                     columns={columns}
                     setKey="set"
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.justified}
                     onRenderItemColumn={renderItem}
                     onRenderRow={renderRow}
                  />}
                  {!dataView.history.length && <Text>No step history</Text>}
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});