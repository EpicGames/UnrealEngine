// Copyright Epic Games, Inc. All Rights Reserved.

import { Slider, Spinner, SpinnerSize, Stack, Text, Toggle } from "@fluentui/react";
import * as d3 from "d3";
import { action, computed, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment, { Moment } from "moment";
import { useEffect, useState } from "react";
import { Link } from "react-router-dom";
import backend from "../../backend";
import { GetJobStepRefResponse, JobStepOutcome } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { ISideRailLink } from "../../base/components/SideRail";
import { displayTimeZone, getElapsedString, getHumanTime, msecToElapsed } from "../../base/utilities/timeUtils";
import { ChangeButton } from "../ChangeButton";
import { HistoryModal } from "../agents/HistoryModal";
import { StepRefStatusIcon } from "../StatusIcon";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from "../../styles/Styles";

const sideRail: ISideRailLink = { text: "Trends", url: "rail_step_trends" };

// Handle bad "@types/d3" types, fix if addressed upstream
const _d3 = d3 as any;

class Tooltip {

   constructor() {
      makeObservable(this);
   }

   @observable
   updated = 0

   @action
   update(ref?: GetJobStepRefResponse, x?: number, y?: number, change?: number) {
      this.ref = ref;
      this.x = x ?? 0;
      this.y = y ?? 0;
      this.change = change ?? 0;
      this.updated++;
   }

   @action
   freeze(frozen: boolean) {
      this.frozen = frozen;
      this.updated++;
   }


   x: number = 0;
   y: number = 0;
   change: number = 0;
   frozen: boolean = false;
   ref?: GetJobStepRefResponse;
}

class StepTrendsDataView extends JobDataView {

   constructor(details: JobDetailsV2) {
      super(details, false)
      makeObservable(this);
   }

   filterUpdated() {
      // this.updateReady();
   }

   initData(history: GetJobStepRefResponse[]) {

      this.minTime = this.maxTime = undefined;
      this.maxMinutes = 0;

      const alltimes: number[] = [];

      history = history.filter(r => {

         if (!r.startTime) {
            return false;
         }

         if (!r.finishTime) {
            return false;
         }

         return true;

      });

      // durations in minutes
      const durations = this.durations;

      durations.clear();

      history.forEach(h => {
         // Cache step timestamps
         this.times.set(h.startTime as string, new Date(h.startTime));
         if (h.finishTime) {
            this.times.set(h.finishTime as string, new Date(h.finishTime));
         }

         const start = moment(this.useFromJobStart ? h.jobStartTime : h.startTime);
         const end = h.finishTime ? moment( h.finishTime) : moment();
         
         // Update oldest start time of any given step
         const stepStart = moment(h.startTime);
         if (!this.minTime || this.minTime.getTime() > (stepStart.unix() * 1000)) {
            this.minTime = stepStart.toDate();
         }

         // Update newest end time of any given step
         if (!this.maxTime || this.maxTime.getTime() < (end.unix() * 1000)) {
            this.maxTime = end.toDate();
         }

         const minutes = moment.duration(end.diff(start)).asMinutes();

         // only add to max minutes if <= 23, otherwise skews graph for steps which "ran" for 24 hours
         if (moment.duration(end.diff(start)).asHours() <= 23) {
            this.maxMinutes = Math.max(minutes, this.maxMinutes);
         }

         // Only add successful steps to the trend data
         if (h.outcome === JobStepOutcome.Success || h.outcome === JobStepOutcome.Warnings) {
            alltimes.push(minutes);
         } else {
            this.skipTrendLine.add(h.jobId);
         }

         durations.set(h.jobId, minutes);
      });

      history = history.sort((a, b) => this.times.get(b.startTime as string)!.getTime() - this.times.get(a.startTime as string)!.getTime())

      const median = (arr: number[]): number | undefined => {
         if (!arr.length) return undefined;
         const s = [...arr].sort((a, b) => a - b);
         const mid = Math.floor(s.length / 2);
         return s.length % 2 ? s[mid] : ((s[mid - 1] + s[mid]) / 2);
      };

      this.median = undefined;

      this.median = median(alltimes);
      if (this.median) {

         this.median *= 3.0;

         if (this.median < 1) {
            this.median = 1;
         }


         if (this.median > this.maxMinutes) {
            this.median = undefined;
         }
      }

      this.history = history;
   }

   set(stepId?: string) {

      const details = this.details;

      if (!details) {
         return;
      }

      if (this.stepId === stepId || !details.jobId) {
         return;
      }

      this.stepId = stepId;

      if (!this.stepId) {
         return;
      }

      const jobData = details.jobData!;
      const stepName = details.getStepName(this.stepId);

      backend.getJobStepHistory(jobData.streamId, stepName, 4096, jobData.templateId!).then(response => {
         this.initData(response);
         this.updateReady();
      }).finally(() => {
         this.initialize(this.history?.length ? [sideRail] : undefined);
      })
   }

   clear() {
      this.history = [];
      this.stepId = undefined;
      this.durations.clear();
      this.minTime = this.maxTime = undefined;
      this.maxMinutes = 0;
      this.renderer = undefined;
      super.clear();
   }

   get lastFailure(): GetJobStepRefResponse | undefined {

      return this.history.find(r => r.outcome === JobStepOutcome.Failure);
   }

   get lastSuccess(): GetJobStepRefResponse | undefined {

      return this.history.find(r => r.outcome === JobStepOutcome.Success);
   }

   get current(): GetJobStepRefResponse | undefined {

      return this.history.find(r => r.stepId === this.stepId);
   }

   detailsUpdated() {

   }

   @observable
   selectedAgentId?: string;

   @observable
   useFromJobStart = false;

   @observable
   timeScales = {
      fromJobStart: 0,
      fromStepStart: 0
   }

   get defaultTimeScale(): number {
      return this.median ?? this.maxMinutes;
   }

   @computed
   get timeScale(): number {
      const scales = this.timeScales;

      if(
         (this.useFromJobStart && !scales.fromJobStart)
         || (!this.useFromJobStart && !scales.fromStepStart)
      ) {
         return this.defaultTimeScale;
      }

      return this.useFromJobStart ? scales.fromJobStart : scales.fromStepStart;
   }
   
   @action
   setTimeScale(time: number) {
      if (this.useFromJobStart) {
         this.timeScales.fromJobStart = time;
      }
      else {
         this.timeScales.fromStepStart = time;
      }

      this.renderer?.onScaleTime?.(time);
   }

   @action
   setUseFromJobStart(use: boolean) {
      const prev = this.useFromJobStart;
      this.useFromJobStart = use;

      if (use !== prev) {
         this.initData(this.history);
         this.updateReady();
      }
   }

   @action
   setSelectedAgentId(agentId?: string) {
      if (this.selectedAgentId === agentId) {
         return;
      }

      this.selectedAgentId = agentId;
   }

   history: GetJobStepRefResponse[] = [];
   times = new Map<string, Date>();

   stepId?: string;

   order = 8;

   durations = new Map<string, number>();
   skipTrendLine = new Set<string>();
   maxMinutes = 0;
   median?: number;

   minTime?: Date;
   maxTime?: Date;

   lastZoom?: d3.ZoomTransform;

   tooltip = new Tooltip();

   renderer?: StepTrendsRenderer;
}

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;
type Zoom = d3.ZoomBehavior<Element, unknown>;
type Scalar = d3.ScaleLinear<number, number>;

class StepTrendsRenderer {

   // find max active and factor in wait time

   constructor(dataView: StepTrendsDataView) {

      this.dataView = dataView;

      this.margin = { top: 16, right: 32, bottom: 0, left: 64 };

   }

   onScaleTime?: (scaleMinutes: number) => void;

   render(container: HTMLDivElement) {

      if (this.hasRendered && !this.forceRender) {
         //return;
      }

      const dataView = this.dataView;
      dataView.renderer = this;

      this.hasRendered = true;

      const width = 1800;
      const margin = this.margin;

      let svg = this.svg;

      let height = 480;

      const scolors = dashboard.getStatusColors();
      const colors: Record<string, string> = {
         "Success": scolors.get(StatusColor.Success)!,
         "Failure": scolors.get(StatusColor.Failure)!,
         "Warnings": scolors.get(StatusColor.Warnings)!,
         "Unspecified": scolors.get(StatusColor.Skipped)!,
      };

      const x = this.scaleX = d3.scaleLinear()
         .domain([dataView.minTime!, dataView.maxTime!].map(d => d.getTime() / 1000))
         .range([margin.left, width - margin.right])

      const y = this.scaleY = d3.scaleLinear()
         .domain([0, dataView.timeScale]).nice()
         .range([height - margin.bottom, margin.top])


      if (!svg) {
         svg = d3.select(container)
            .append("svg") as any as SelectionType

         this.svg = svg;
      } else {
         // remove tooltip
         d3.select(container).selectAll('div').remove();
         svg.selectAll("*").remove();
      }

      svg.attr("viewBox", [0, 0, width, height] as any);

      /*
      svg.append("rect")
         .attr("width", "100%")
         .attr("height", "100%")
         .attr("fill", modeColors.background);
      */

      const clipId = `step_history_clip`;

      svg.append("clipPath")
         .attr("id", clipId)
         .append("rect")
         .attr("x", margin.left)
         .attr("y", 0)
         .attr("width", width - margin.left - margin.right + 2)
         .attr("height", height);

      const data = dataView.history;

      svg.append("g")
         .attr("class", "bars")
         .attr("clip-path", `url(#${clipId})`)
         .selectAll("rect")
         .data(data)
         .join("rect")
         .attr("fill", (d) => {
            if (!d.finishTime) {
               return scolors.get(StatusColor.Running)!;
            }
            if (!d.outcome) {
               return colors["Unspecified"];
            }
            return colors[d.outcome];
         })
         .attr("x", d => x(new Date(d.startTime!).getTime() / 1000))
         .attr("y", d => y(dataView.durations.get(d.jobId)!))
         .attr("height", d => y(0) - y(dataView.durations.get(d.jobId)!))
         .attr("width", 2);

      const arrowPoints = [[0, 0], [0, 10], [10, 5]] as any;
      svg.append("marker")
         .attr("id", "cmarker")
         .attr('viewBox', [0, 0, 10, 10] as any)
         .attr("refX", 5)
         .attr("refY", 5)
         .attr("markerWidth", 5)
         .attr("markerHeight", 5)
         .attr("orient", 'auto-start-reverse')
         .append("path")
         .attr('d', (d3.line() as any)(arrowPoints))
         .style("fill", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B");

      //svg.append("g")
      //  .attr("class", "cline")
      svg.append("line")
         .attr("class", "cline")
         .attr("x1", () => 0)
         .attr("x2", () => 0)
         .attr("y1", () => 0)
         .attr("y2", () => 0)
         .attr("stroke-width", () => 2)
         .style("stroke-dasharray", "3, 3")
         .attr("marker-end", "url(#cmarker)")
         .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")


      svg.append("line")
         .attr("class", "bline")
         .attr("x1", () => margin.left)
         .attr("x2", () => width - margin.right)
         .attr("y1", () => height)
         .attr("y2", () => height)
         .attr("stroke-width", () => 1)
         .attr("stroke", () => dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")

      const lineI: number[] = [];
      data.forEach((d, idx) => {
         if (!dataView.skipTrendLine.has(d.jobId)) {
            lineI.push(idx)
         }
      })

      const showTrendLine = dataView.durations.size >= 10;

      const plotTrendY = (i: any, scaleY?: any) => {
         const idx = i as any as number;

         let sum = 0;
         let count = 0;
         const range = 6;
         for (let j = idx - range; j < idx + range; j++) {

            if (j < 0 || j >= data.length) {
               continue;
            }

            if (dataView.skipTrendLine.has(data[j].jobId)) {
               continue;
            }

            const v = dataView.durations.get(data[j].jobId)!;

            if (dataView.median && v > dataView.median) {
               continue;
            }

            sum += v;
            count++;
         }

         const f = scaleY ?? y;

         if (!count) {
            return f(dataView.durations.get(data[idx].jobId)!) - 2
         }

         return f(sum / count) - 2

      }

      const curve = d3.curveMonotoneX;
      const line: any = d3.line()
         .defined(i => showTrendLine)
         .curve(curve)
         .x(i => { return x(new Date(data[i as any].startTime!).getTime() / 1000) })
         .y(i => {
            return plotTrendY(i);
         })

      svg.append("path")
         .attr("clip-path", `url(#${clipId})`)
         .attr("class", "linechart")
         .attr("fill", "none")
         .attr("stroke", dashboard.darktheme ? "#EEEEEE" : "#035CA1")
         .attr("stroke-width", 3.0)
         .attr("stroke-linecap", "round")
         .attr("stroke-linejoin", "round")
         .attr("stroke-opacity", 1)
         .attr("d", line(lineI as any));

      const xAxis = (g: SelectionType) => {

         const dateMin = dataView.minTime!
         const dateMax = dataView.maxTime!

         let ticks: number[] = [];
         for (const date of d3.timeDays(dateMin, dateMax, 1).reverse()) {
            ticks.push(date.getTime() / 1000);
         }

         if (ticks.length > 14) {
            let nticks = [...ticks];
            // remove first and last, will be readded 
            const first = nticks.shift()!;
            const last = nticks.pop()!;

            const n = Math.floor(nticks.length / 12);

            const rticks: number[] = [];
            for (let i = 0; i < nticks.length; i = i + n) {
               rticks.push(nticks[i]);
            }

            rticks.unshift(first);
            rticks.push(last);
            ticks = rticks;

         }


         g.attr("transform", `translate(0,24)`)
            .style("font-family", "Horde Open Sans SemiBold")
            .style("font-size", "12px")
            .call(d3.axisTop(x)
               .tickValues(ticks)
               //.ticks(d3.timeDays(dateMin, dateMax))
               .tickFormat(d => {
                  return getHumanTime(new Date((d as number) * 1000));
               })
               .tickSizeOuter(0))
            .call(g => g.select(".domain").remove())
            .call(g => g.selectAll(".tick line").attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
               .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
               .attr("y2", height - this.margin.bottom))
      }

      // left axis
      const yAxis = (g: SelectionType) => {

         g.attr("transform", `translate(${this.margin.left},0)`)
            .style("font-family", "Horde Open Sans SemiBold")
            .style("font-size", "12px")
            .call(d3.axisLeft(this.scaleY!)
               .ticks(7)
               .tickFormat((d) => {

                  if (!d) {
                     return "";
                  }

                  return msecToElapsed((d as number) * 60000, true, false);

               }))
      }

      // top axis
      svg.append("g").attr("class", "x-axis").call(xAxis)

      svg.append("g").attr("class", "y-axis").call(yAxis)

      const zoom = this.zoom = d3.zoom()
         .scaleExtent([1, 8])
         .extent([[margin.left, 0], [width - margin.right, height]])
         .translateExtent([[margin.left, -Infinity], [width - margin.right, Infinity]])
         .on("zoom", zoomed);

      const renderer = this;

      function zoomed(event: any) {
         dataView.tooltip.update(undefined)
         dataView.tooltip.freeze(false);

         dataView.lastZoom = event.transform;
         zoomTo(event.transform);
      }

      function zoomTo(transform: d3.ZoomTransform) {
         const zoomLevel = transform.k ? transform.k : 1;
         const barWidth = Math.max(2, zoomLevel / 2);

         x.range([margin.left, width - margin.right].map(d => transform.applyX(d)));
         svg!.selectAll(".bars rect")
            .attr("x", (d: any) => x(new Date(d.startTime!).getTime() / 1000) - (barWidth / 2))
            .attr("width", barWidth);

         const scaledLine: any = d3.line()
            .defined(i => showTrendLine)
            .curve(curve)
            .x(i => { return x(new Date(data[i as any].startTime!).getTime() / 1000) })
            .y(i => plotTrendY(i, renderer.scaleY!))


         svg!.selectAll(".linechart")
            .attr("d", scaledLine(lineI as any));


         svg!.selectAll(".x-axis").call(xAxis as any);
      }

      svg.call(zoom as any);

      this.onScaleTime = (scaleMinutes) => {
         const scaleY = this.scaleY = d3.scaleLinear()
            .domain([0, scaleMinutes]).nice()
            .range([height - margin.bottom, margin.top])

         svg!.selectAll(".bars rect")
            .attr("y", (d: any) => scaleY(dataView.durations.get(d.jobId)!))
            .attr("height", (d: any) => scaleY(0) - scaleY(dataView.durations.get(d.jobId)!))


         const scaledLine: any = d3.line()
            .defined(i => showTrendLine)
            .curve(curve)
            .x(i => { return x(new Date(data[i as any].startTime!).getTime() / 1000) })
            .y(i => { return plotTrendY(i, scaleY) })


         svg!.selectAll(".linechart")
            .attr("d", scaledLine(lineI as any));

         svg!.selectAll(".y-axis").call(yAxis as any);

      }

      const closestData = (xpos: number, ypos: number): GetJobStepRefResponse | undefined => {

         if (!dataView.history.length) {
            return undefined;
         }

         let first = dataView.history[0];

         let closest = dataView.history.reduce<GetJobStepRefResponse>((best: GetJobStepRefResponse, ref: GetJobStepRefResponse) => {

            const bestx = x(dataView.times.get(best.startTime as string)!.getTime() / 1000);
            const refx = x(dataView.times.get(ref.startTime as string)!.getTime() / 1000);

            if (Math.abs(xpos - bestx) > Math.abs(xpos - refx)) {
               return ref
            }

            return best;

         }, first);

         return closest;

      }

      const updateTooltipArrow = (ref: GetJobStepRefResponse): void => {
         const posx = x(new Date(ref.startTime!).getTime() / 1000);
         const posy = this.scaleY!(dataView.durations.get(ref.jobId)!);

         svg!.selectAll(".cline")
            .attr("x1", () => posx)
            .attr("x2", () => posx)
            .attr("y1", d => 0)
            .attr("y2", d => posy - 8);
      }

      const handleMouseMove = (event: any) => {

         if (dataView.tooltip.frozen) {
            return;
         }

         let mouseX = _d3.pointer(event)[0];
         let mouseY = _d3.pointer(event)[1];

         const ref = closestData(mouseX, mouseY);

         if (ref) {
            updateTooltipArrow(ref);
         } else {
            //svg!.selectAll(".cline rect")
            //.attr("width", 0);
         }

         // relative to container
         if (ref) {
            this.showToolTip()
         }
         dataView.tooltip.update(ref, _d3.pointer(event, container)[0], mouseY, ref?.change);
      }

      // events
      svg.on("wheel", (event: any) => { event.preventDefault(); })

      svg.on("mousemove", (event) => { this.showToolTip(true); handleMouseMove(event); });
      svg.on("mouseleave", (event) => { if (!dataView.tooltip.frozen) dataView.tooltip.update(undefined); })

      svg.on("pointerdown", (event) => { });
      svg.on("pointerup", (event) => { });

      svg.on("click", (event) => {
         dataView.tooltip.freeze(!dataView.tooltip.frozen);
      });

      // Preserve some previous state on re-render
      if(dataView.lastZoom) zoomTo(dataView.lastZoom);
      if(dataView.tooltip.ref && dataView.tooltip.frozen) {
         updateTooltipArrow(dataView.tooltip.ref);
      }
   }

   showToolTip(shown: boolean = true) {
      const svg = this.svg;
      if (!svg) {
         return;
      }

      svg!.selectAll(".cline")
         .style("display", shown ? "block" : "none")


   }

   margin: { top: number, right: number, bottom: number, left: number }

   dataView: StepTrendsDataView;

   svg?: SelectionType;

   zoom?: Zoom;
   scaleX?: Scalar;
   scaleY?: Scalar;


   hasRendered = false;
   forceRender = false;
}

const GraphTooltip: React.FC<{ dataView: StepTrendsDataView }> = observer(({ dataView }) => {

   const { modeColors } = getHordeStyling();

   // subscribe
   if (dataView.tooltip.updated) { }

   const tooltip = dataView.tooltip;
   const ref = tooltip.ref;
   const renderer = dataView.renderer;

   if (!ref) {
      if (renderer) {
         renderer.showToolTip(false);
      }
      return null;
   }

   renderer?.showToolTip(true);

   let tipX = tooltip.x;
   let offsetX = 32;
   let translateX = "0%";

   if (tipX > 1000) {
      offsetX = -32;
      translateX = "-100%";
   }

   const translateY = "-50%";

   const sindex = dataView.history.indexOf(ref);

   const start = moment(dataView.useFromJobStart ? ref.jobStartTime : ref.startTime);
   let end = moment(Date.now());

   if (ref.finishTime) {
      end = moment(ref.finishTime);
   }

   const textSize = "small";

   const duration = getElapsedString(start, end);

   const stepDisplayTime = moment(ref.startTime).tz(displayTimeZone());
   const jobDisplayTime = moment(ref.jobStartTime).tz(displayTimeZone());
   const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";
   let displayTimeStr = stepDisplayTime.format('MMM Do') + ` at ${stepDisplayTime.format(format)}`;
   let jobDisplayTimeStr = jobDisplayTime.format('MMM Do') + ` at ${jobDisplayTime.format(format)}`;

   const step = dataView.details?.stepById(dataView.stepId);

   return <div style={{
      position: "absolute",
      display: "block",
      top: `${tooltip.y}px`,
      left: `${tooltip.x + offsetX}px`,
      backgroundColor: modeColors.background,
      zIndex: 1,
      border: "solid",
      borderWidth: "1px",
      borderRadius: "3px",
      width: "max-content",
      borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F",
      padding: 16,
      pointerEvents: tooltip.frozen ? undefined : "none",
      transform: `translate(${translateX}, ${translateY})`
   }}>
      <Stack>
         <Link to={`/job/${ref.jobId}?step=${ref.stepId}`}>
            <Stack horizontal>
               <StepRefStatusIcon stepRef={ref} />
               <Text variant={textSize}>{step?.name}</Text>
            </Stack>
         </Link>
         <Stack style={{ paddingLeft: 2, paddingTop: 8 }} tokens={{ childrenGap: 8 }}>
            <ChangeButton prefix="CL" job={dataView.details!.jobData!} stepRef={ref} hideAborted={true} rangeCL={sindex < (dataView.history.length - 1) ? (dataView.history[sindex + 1].change + 1) : undefined} />
            {dataView.useFromJobStart && <Text variant={textSize}>Job: {jobDisplayTimeStr}</Text>}
            <Text variant={textSize}>{dataView.useFromJobStart && "Step: "}{displayTimeStr}</Text>
            <Text variant={textSize}>Duration: {duration}</Text>
            <Stack style={{ cursor: "pointer" }} onClick={() => dataView.setSelectedAgentId(ref.agentId)}>
               <Text variant={textSize}>{ref.agentId!}</Text>
            </Stack>
         </Stack>
      </Stack>
   </div>
})

const StepTrendGraph: React.FC<{ dataView: StepTrendsDataView }> = ({ dataView }) => {

   const graph_container_id = `timeline_graph_container`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const [state, setState] = useState<{ graph?: StepTrendsRenderer }>({});

   const { hordeClasses, modeColors } = getHordeStyling();

   if (!state.graph) {
      setState({ ...state, graph: new StepTrendsRenderer(dataView) })
      return null;
   }

   if (container) {
      try {
         state.graph?.render(container);

      } catch (err) {
         console.error(err);
      }

   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8, paddingTop: 8 }}>
         <div style={{ position: "relative" }}>
            <GraphTooltip dataView={dataView} />
         </div>
         <Stack style={{ paddingTop: 16, paddingBottom: 16, paddingLeft: 16, backgroundColor: modeColors.background }}>
            <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
         </Stack>
      </Stack>
   </Stack>;

}


JobDetailsV2.registerDataView("StepTrendsDataView", (details: JobDetailsV2) => new StepTrendsDataView(details));

export const StepTrendsPanelV2: React.FC<{ jobDetails: JobDetailsV2; stepId: string }> = observer(({ jobDetails, stepId }) => {

   const dataView = jobDetails.getDataView<StepTrendsDataView>("StepTrendsDataView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   const { hordeClasses } = getHordeStyling();

   dataView.subscribe();

   if (!jobDetails.jobData) {
      return null;
   }

   if (dataView.initialized && !dataView.history?.length) {
      return null;
   }

   dataView.set(stepId);

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         {!!dataView.selectedAgentId && <HistoryModal agentId={dataView.selectedAgentId} onDismiss={() => dataView.setSelectedAgentId(undefined)} />}
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{"Trends"}</Text>
               </Stack>
               <Stack grow />
               {!!dataView.minTime && 
                  <Stack horizontal horizontalAlign="end" verticalAlign="start" tokens={{childrenGap: 24}}>
                     <Slider
                        styles={{ root: { width: 240 } }}
                        label="Scale Time"
                        min={1}
                        max={dataView.maxMinutes}
                        step={dataView.maxMinutes / 20}
                        value={dataView.timeScale}
                        showValue
                        valueFormat={(value) => {
                           if (!value) return ""
                           return msecToElapsed(value * 60 * 1000, true, false)
                        }}
                        onChange={(time) => dataView.setTimeScale(time)}
                     />
                     <Toggle 
                        label="From Job Start"
                        styles={{label: {paddingTop: 0}}}
                        onChange={(_, checked) => {
                           dataView.setUseFromJobStart(!!checked);
                        }}
                     />
                  </Stack>
               }
            </Stack>
            {!!dataView.history.length && <StepTrendGraph dataView={dataView} />}
            {!dataView.history.length && <Spinner size={SpinnerSize.large} />}
         </Stack>
      </Stack>
   </Stack>);
});