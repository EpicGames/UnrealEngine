// Copyright Epic Games, Inc. All Rights Reserved.

import { FontIcon, Slider, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import * as d3 from "d3";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useEffect, useState } from "react";
import { Link, NavigateFunction, useNavigate } from "react-router-dom";
import { GetBatchResponse, GetStepResponse, JobStepBatchState, JobStepOutcome, StepData } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { ISideRailLink } from "../../base/components/SideRail";
import { getShortNiceTime, msecToElapsed } from "../../base/utilities/timeUtils";
import { HistoryModal } from "../agents/HistoryModal";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from "../../styles/Styles";

type SelectionType = d3.Selection<SVGGElement, unknown, null, undefined>;

// There are some bad @type/d3 mappings
const _d3 = d3 as any;

type Batch = GetBatchResponse & {
   utcStart: Date;
   utcFinish: Date;
   utcWaitStart: Date;
   utcWaitFinish: Date;
   utcInitStart: Date;
   utcInitFinish: Date;
}

type Lane = {
   batches: Batch[];
}

enum SpanType {
   Wait,
   Init,
   Step
}

type Span = {

   lane: number;

   batch: Batch;

   type: SpanType;
   utcStart: Date;
   utcFinish: Date;

   cost?: number;

   step?: GetStepResponse;
   stepName?: string;

   filtered?: boolean;
}

const sideRail: ISideRailLink = { text: "Timeline", url: "rail_timeline" };

class Tooltip {

   constructor() {
      makeObservable(this);
   }

   @observable
   updated = 0

   @action
   update(span?: Span, x?: number, y?: number) {
      this.x = x ?? 0;
      this.y = y ?? 0;
      this.span = span;
      this.updated++;
   }

   @action
   freeze(frozen: boolean) {
      this.frozen = frozen;
      this.updated++;
   }

   span?: Span;
   x: number = 0;
   y: number = 0;
   frozen: boolean = false;
}


class TimelineDataView extends JobDataView {

   filterUpdated() {
      this.dirty = true;
   }

   set() {

      const details = this.details;

      if (!details) {
         return;
      }

      this.initialize([sideRail]);

   }

   clear() {
      this.allSpans = undefined;
      this.minTime = undefined;
      this.maxTime = undefined;
      this.anchorMS = undefined;
      this.maxSpanMS = undefined;
      this.filterTime = 0;
      this.filterCost = 0;

      super.clear();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.generateSpans();
      this.updateReady();

   }

   generateSpans() {

      if (!this.details?.jobData?.batches) {
         return;
      }

      const jobDetails = this.details;

      const job = jobDetails.jobData!;

      // patch batches with latest finished step time, this renders the timeline more dynamically than waiting for 
      // complete batches, note: should present batch stopping time, which can be significant

      const nbatches = job.batches?.map(b => {

         const newBatch = {
            ...b,
         } as GetBatchResponse

         if (newBatch.steps.length) {

            let lastStep: GetStepResponse | undefined;
            for (let i = newBatch.steps.length - 1; i >= 0; i--) {
               if (newBatch.steps[i].finishTime) {
                  lastStep = newBatch.steps[i];
                  break;
               }
            }

            if (lastStep) {
               newBatch.state = JobStepBatchState.Complete
               newBatch.finishTime = lastStep.finishTime;
            }
         }

         return newBatch;
      })

      let bresponses = nbatches?.filter(b => !!b.agentId && (!!b.startTime && !!b.readyTime && !!b.finishTime) && b.steps?.length)!.map(b => { return { ...b } as GetBatchResponse })!;

      if (!this.filterStepId && jobDetails.filter) {
         const label = jobDetails.filter.label;
         if (label) {
            bresponses?.forEach(b => {
               b.steps = b.steps?.filter(s => {
                  if (!s.startTime || !s.finishTime) {
                     return false;
                  }
                  return label.steps.indexOf(s?.id ?? "") !== -1;
               });
            });

            bresponses = bresponses.filter(b => !!b.steps?.length);
         }
      }

      let filterStep: StepData | undefined;

      if (this.filterStepId) {
         filterStep = jobDetails.stepById(this.filterStepId);
         const steps = new Set(jobDetails.getStepDependencies(this.filterStepId).map(s => s.id));

         bresponses?.forEach(b => {
            b.steps = b.steps?.filter(s => {
               return steps.has(s.id);
            });
         });

         bresponses = bresponses.filter(b => !!b.steps?.length);
      }

      let maxTime = 0;

      if (filterStep && filterStep.finishTime) {
         maxTime = new Date(filterStep.finishTime).getTime();
      }

      let batches: Batch[] = [];

      bresponses.forEach(b => {

         const utcBatchStart = new Date(b.startTime as string);
         const utcBatchFinish = new Date(b.finishTime as string);
         let utcBatchReady = new Date(b.readyTime as string)
         const utcStepStart = new Date(b.steps[0].startTime as string)

         if (!!b.steps.find( s => jobDetails.getStepRetryNumber(s.id) > 0)) {
            utcBatchReady = utcBatchStart;
         }

         if (!filterStep || !maxTime) {
            if (utcBatchFinish.getTime() > maxTime) {
               maxTime = utcBatchFinish.getTime();
            }
         }

         batches.push({ ...b, utcStart: utcBatchStart, utcFinish: utcBatchFinish, utcWaitStart: utcBatchReady, utcWaitFinish: utcBatchStart, utcInitStart: utcBatchStart, utcInitFinish: utcStepStart });

      });

      if (!batches.length) {
         //console.log("No batches to render in timeline");
         return;
      }

      batches = batches.sort((a, b) => {
         return a.utcWaitStart.getTime() - b.utcWaitStart.getTime();
      });

      this.minTime = batches[0].utcWaitStart;

      this.maxTime = new Date(maxTime);

      const bestLane = (batch: Batch, lanes: Lane[]) => {

         return lanes.find(lane => {
            return batch.utcWaitStart.getTime() > lane.batches[lane.batches.length - 1].utcFinish.getTime();
         });
      }

      const processLanes = (batch: Batch, lanes: Lane[]) => {

         const waitStart = batch.utcWaitStart.getTime();

         const clanes = lanes.filter(lane => {
            return lane.batches[lane.batches.length - 1].utcFinish.getTime() < waitStart;
         });

         if (!clanes.length) {
            // need a new lane
            lanes.push({ batches: [batch] });
         } else {
            const lane = bestLane(batch, clanes);
            console.assert(!!lane);
            lane!.batches.push(batch);
         }

      }

      const allLanes: Lane[] = [];

      this.anchorMS = batches[0].utcWaitStart.getTime();

      batches.forEach(b => {
         processLanes(b, allLanes);
      });

      const spans: Span[] = [];

      allLanes.forEach((lane, index) => {
         lane.batches.forEach(batch => {

            spans.push({
               lane: index,
               batch: batch,
               type: SpanType.Wait,
               utcStart: batch.utcWaitStart,
               utcFinish: batch.utcWaitFinish,
            });

            let initPrice: number | undefined;

            if (batch.agentRate) {
               const start = moment(batch.utcInitStart);
               const end = moment(batch.utcInitFinish);
               const hours = moment.duration(end.diff(start)).asHours();
               initPrice = hours * batch.agentRate;
            }

            spans.push({
               lane: index,
               batch: batch,
               type: SpanType.Init,
               utcStart: batch.utcInitStart,
               utcFinish: batch.utcInitFinish,
               cost: initPrice
            });


            batch.steps.forEach(step => {

               if (!step.startTime || !step.finishTime || step.outcome === JobStepOutcome.Unspecified) {
                  return;
               }

               let stepPrice: number | undefined;

               if (batch.agentRate) {
                  const start = moment(step.startTime);
                  const end = moment(step.finishTime);
                  const hours = moment.duration(end.diff(start)).asHours();
                  stepPrice = hours * batch.agentRate;
               }

               spans.push({
                  lane: index,
                  batch: batch,
                  type: SpanType.Step,
                  utcStart: new Date(step.startTime!),
                  utcFinish: new Date(step.finishTime!),
                  step: step,
                  stepName: jobDetails.getStepName(step.id),
                  cost: stepPrice
               });
            })
         });
      });

      this.maxSpanMS = 0;
      this.maxSpanCost = 0;

      spans.forEach(s => {

         if (s.cost && s.cost > this.maxSpanCost!) {
            this.maxSpanCost = s.cost;
         }

         const t = s.utcFinish.getTime() - s.utcStart.getTime();
         this.maxSpanMS = Math.max(t, this.maxSpanMS!);
      });

      this.allSpans = spans;
   }

   get spans(): Span[] {

      if (this.dirty) {
         this.generateSpans();
         this.dirty = false;
      }

      if (!this.allSpans) {
         return [];
      }

      let fcost = this.filterCost;
      let ftime = this.filterTime;

      this.allSpans.forEach(s => s.filtered = false);

      if (!this.filterCost && !this.filterTime) {
         return this.allSpans
      }

      this.allSpans.forEach(s => {
         s.filtered = false;

         if (fcost) {
            if ((s.type === 0 || !s.cost)) {
               s.filtered = true;
            } else {
               if (s.cost < fcost) {
                  s.filtered = true;
               }
            }
         }

         if (ftime) {
            const t = s.utcFinish.getTime() - s.utcStart.getTime();
            if (t < ftime) {
               s.filtered = true;
            }
         }
      });

      return this.allSpans;

   }

   private allSpans?: Span[];
   minTime?: Date;
   maxTime?: Date;
   maxSpanMS?: number;
   maxSpanCost?: number;
   anchorMS?: number;

   order: number = 7

   filterTime: number = 0;
   filterCost: number = 0;
   filterStepId?: string;

   tooltip = new Tooltip();

   dirty = false;

}

class TimelineRenderer {

   // find max active and factor in wait time

   constructor(dataView: TimelineDataView) {

      this.dataView = dataView;
      this.margin = { top: 0, right: 32, bottom: 0, left: 32 };

   }

   render(container: HTMLDivElement, navigate: NavigateFunction) {

      if (this.hasRendered && !this.forceRender) {
         //return;
      }

      const dataView = this.dataView;

      if (!dataView.spans.length) {
         return;
      }

      this.hasRendered = true;

      const width = 1800;
      const margin = this.margin;

      const spans = dataView.spans;

      const scolors = dashboard.getStatusColors();
      const colors: Record<string, string> = {
         "Success": scolors.get(StatusColor.Success)!,
         "Failure": scolors.get(StatusColor.Failure)!,
         "Warnings": scolors.get(StatusColor.Warnings)!,
         "Unspecified": scolors.get(StatusColor.Skipped)!,
      };


      const X = (d3 as any).map(spans, (s) => s);
      const Y = (d3 as any).map(spans, (s) => s.lane);

      let yDomain: any = Y;
      yDomain = new (d3 as any).InternSet(yDomain);

      const I = d3.range(X.length);

      const yPadding = 1;
      const height = Math.ceil((yDomain.size + yPadding) * 16) + margin.top + margin.bottom;

      let yRange = [margin.top, height - this.margin.bottom];

      const xScale = d3.scaleLinear()
         .domain([dataView.minTime!, dataView.maxTime!].map(d => d.getTime() / 1000))
         .range([margin.left, width - margin.right])

      const yScale = (d3 as any).scalePoint(yDomain, yRange).round(true).padding(yPadding);

      let svg = this.svg;

      if (!svg) {
         svg = d3.select(container)
            .append("svg") as any as SelectionType

         this.svg = svg;
      } else {
         // remove tooltip
         d3.select(container).selectAll('div').remove();
         svg.selectAll("*").remove();
      }

      svg.attr("viewBox", [0, 0, width, height + 24] as any);

      const g = svg.append("g")
         .selectAll()
         .data((d3 as any).group(I, i => Y[i]))
         .join("g")

      const waitColor = dashboard.darktheme ? "#5B6367" : "#D3D2D1";
      const initColor = dashboard.darktheme ? "#2E74B3" : "#7CB5EB";

      g.append("g").selectAll("line")
         .data(([, I]: any) => I)
         .join("line")
         .attr("class", "separator")
         .attr("x1", i => xScale(X[i as any].utcStart.getTime() / 1000) + 1)
         .attr("x2", i => xScale(X[i as any].utcStart.getTime() / 1000) + 1)
         .attr("y1", i => (yScale(X[i as any].lane) as number - 5 + 16))
         .attr("y2", i => (yScale(X[i as any].lane) as number + 5 + 16))
         .attr("stroke-linecap", 0)
         .attr("stroke-width", i => X[i as any].filtered ? 0 : 3)
         .attr("stroke", i => {
            const span = spans[i as any];
            if (span.type === 0) {
               return waitColor;
            }
            if (span.type === 1) {
               return initColor;
            }

            return colors[span.step?.outcome ?? "Unspecified"];
         })


      g.append("g").selectAll("line")
         .data(([, I]: any) => I)
         .join("line")
         .attr("class", "spanline")
         .attr("x1", i => xScale(X[i as any].utcStart.getTime() / 1000) - 1)
         .attr("x2", i => xScale(X[i as any].utcFinish.getTime() / 1000) - 1)
         .attr("stroke-width", i => X[i as any].filtered ? 0 : 5)
         .attr("y1", i => (yScale(X[i as any].lane) as number + 16))
         .attr("y2", i => (yScale(X[i as any].lane) as number + 16))
         .attr("stroke-linecap", 0)
         .attr("stroke", i => {
            const span = spans[i as any];
            if (span.type === 0) {
               return waitColor;
            }
            if (span.type === 1) {
               return initColor;
            }

            return colors[span.step?.outcome ?? "Unspecified"];
         })

      g.append("g").selectAll("line")
         .data(([, I]: any) => I)
         .join("line")
         .attr("class", "cline")
         .attr("x1", 0)
         .attr("x2", 0)
         .attr("stroke-width", 0)
         .attr("y1", i => 0)
         .attr("y2", i => 0)
         .attr("stroke-linecap", 0)
         .attr("stroke", "#FF00FF")


      const xAxis = (g: SelectionType) => {

         g.attr("transform", `translate(0,20)`)
            .style("font-family", "Horde Open Sans SemiBold")
            .style("font-size", "12px")
            .call(d3.axisTop(xScale)
               .tickFormat(d => {
                  return msecToElapsed(((d as number) * 1000) - dataView.anchorMS!, true, false);
               })
               .tickSizeOuter(0))
            .call(g => g.select(".domain").remove())
            .call(g => g.selectAll(".tick line").attr("stroke-opacity", dashboard.darktheme ? 0.35 : 0.25)
               .attr("stroke", dashboard.darktheme ? "#6D6C6B" : "#4D4C4B")
               .attr("y2", height - this.margin.bottom))

      }

      // top axis
      svg.append("g").attr("class", "x-axis").call(xAxis)

      function zoomed(event: any) {
         xScale.range([margin.left, width - margin.right].map(d => event.transform.applyX(d)));
         svg!.selectAll(".separator")
            .attr("x1", i => { if (i as number >= X.length) return 0; return xScale(X[i as any].utcStart.getTime() / 1000) })
            .attr("x2", i => { if (i as number >= X.length) return 0; return xScale(X[i as any].utcStart.getTime() / 1000) })
         svg!.selectAll(".spanline")
            .attr("x1", i => { if (i as number >= X.length) return 0; return xScale(X[i as any].utcStart.getTime() / 1000) })
            .attr("x2", i => { if (i as number >= X.length) return 0; return xScale(X[i as any].utcFinish.getTime() / 1000) })

         svg!.selectAll(".x-axis").call(xAxis as any);

         const span = dataView.tooltip.span;
         if (span) {
            svg!.select(`.cline`)
               .attr("x1", xScale(span.utcStart.getTime() / 1000) - 1)
               .attr("x2", xScale(span.utcFinish.getTime() / 1000) - 1)
               .attr("y1", i => yScale(span.lane) as number + 16)
               .attr("y2", i => yScale(span.lane) as number + 16)
         }

      }

      const zoom = d3.zoom()
         .scaleExtent([1, 48])
         .extent([[margin.left, 0], [width - margin.right, height]])
         .translateExtent([[margin.left, -Infinity], [width - margin.right, Infinity]])
         .on("zoom", zoomed)

      svg.call(zoom as any);

      const closestData = (x: number, y: number): Span | undefined => {

         y -= 16;

         let first = spans.find(s => !s.filtered);

         if (!first) {
            return undefined;
         }

         let closest = spans.reduce<Span>((best: Span, span: Span, i: number, spans: Span[]) => {

            if (span.filtered) {
               return best;
            }

            const by = Math.abs(yScale(best.lane)! - y);
            const cy = Math.abs(yScale(span.lane)! - y);

            if (cy > by) {
               return best;
            }

            const ss = xScale(span.utcStart.getTime() / 1000);
            const sf = xScale(span.utcFinish.getTime() / 1000);

            if (x >= ss && x <= sf) {
               return span;
            }

            return best;

         }, first);

         const ss = xScale(closest.utcStart.getTime() / 1000);
         const sf = xScale(closest.utcFinish.getTime() / 1000);

         if (x < ss || x > sf) {
            return undefined;
         }

         return closest;

      }

      const handleMouseMove = (event: any) => {

         if (dataView.tooltip.frozen) {
            return;
         }

         let mouseX = (d3 as any).pointer(event)[0];
         let mouseY = (d3 as any).pointer(event)[1];

         const span = closestData(mouseX, mouseY);

         if (span) {
            const svg = this.svg!;
            let color = "";
            if (span.type === 0) {
               color = waitColor;
            }
            else if (span.type === 1) {
               color = initColor;
            } else {
               color = colors[span.step?.outcome ?? "Unspecified"];
            }

            if (!dataView.tooltip.frozen) {
               svg.select(`.cline`)
                  .attr("x1", xScale(span.utcStart.getTime() / 1000) - 1)
                  .attr("x2", xScale(span.utcFinish.getTime() / 1000) - 1)
                  .attr("stroke-width", 10)
                  .attr("y1", i => yScale(span.lane) as number + 16)
                  .attr("y2", i => yScale(span.lane) as number + 16)
                  .attr("stroke", color)
            }
         } else {
            this.svg!.select(`.cline`)
               .attr("stroke-width", 0)
         }

         dataView.tooltip.update(span, (d3 as any).pointer(event, container)[0], (d3 as any).pointer(event, container)[1]);

      }

      const handleMouseLeave = (event: any) => {
         if (!dataView.tooltip.frozen) {
            dataView.tooltip.update(undefined);
            this.svg!.select(`.cline`)
               .attr("stroke-width", 0)
         }
      }

      const handleMouseClick = (event: any) => {

         if (dataView.tooltip.frozen) {
            dataView.tooltip.freeze(false);
            return;
         }

         if (dataView.tooltip.span) {
            dataView.tooltip.freeze(true);
         }

      }


      svg.on("click", (event) => handleMouseClick(event))

      svg.on("mousemove", (event) => handleMouseMove(event));
      svg.on("mouseleave", (event) => { handleMouseLeave(event); })
      svg.on("wheel", (event) => { (event as any).preventDefault(); })

   }

   margin: { top: number, right: number, bottom: number, left: number }

   dataView: TimelineDataView;

   svg?: SelectionType;

   hasRendered = false;
   forceRender = false;
}

const GraphTooltip: React.FC<{ dataView: TimelineDataView }> = observer(({ dataView }) => {

   const [viewAgent, setViewAgent] = useState("");
   const { modeColors } = getHordeStyling();

   // subscribe
   if (dataView.tooltip.updated) { }

   const tooltip = dataView.tooltip;
   const span = tooltip.span;

   if (!span) {
      return null;
   }

   const textSize = 11;

   let tipX = tooltip.x;
   let offsetX = 32;
   let translateX = "0%";

   if (tipX > 1000) {
      offsetX = -32;
      translateX = "-100%";
   }

   const translateY = "-50%";

   const details = dataView.details!;

   const pool = details.stream?.agentTypes[span.batch?.agentType!];

   const agentPool = pool?.pool?.toUpperCase() ? pool.pool.toUpperCase() : "Unknown Pool";
   const agent = span.batch.agentId ? span.batch.agentId : "Unknown Agent";

   const titleWidth = 48;

   const dataElement = (title: string, value: string, link?: string, agent?: string, linkTarget?: string) => {

      let valueElement = <Stack>
         <Stack style={{ fontSize: textSize }}>
            {value}
         </Stack>
      </Stack>


      if (link) {
         valueElement = <Link className="horde-link" to={link} target={linkTarget}>
            {valueElement}
         </Link>
      }

      if (agent) {
         valueElement = <button className="horde-link" style={{ borderWidth: 0, padding: 0, backgroundColor: modeColors.background, cursor: "pointer", color: modeColors.text }}>
            {valueElement}
         </button>
      }

      let component = <Stack style={{ cursor: (link || agent) ? "pointer" : undefined }} onClick={() => {
         if (agent) {
            setViewAgent(agent);
         }

      }}>
         <Stack horizontal>
            <Stack style={{ width: titleWidth }}>
               {!!title && <Text style={{ fontFamily: "Horde Open Sans SemiBold", fontSize: textSize }} >{title}</Text>}
            </Stack>
            {valueElement}
         </Stack>
      </Stack>


      return component;
   }

   const elements: JSX.Element[] = [];

   if (span.step) {
      elements.push(dataElement("Step:", span.stepName ?? "Unknown Step", `/job/${details!.jobId}?step=${span.step.id}`));
      if (span.step.logId) {
         elements.push(dataElement("", "View Log", `/log/${span.step.logId}`, undefined, "_blank"));
      }
   }

   elements.push(dataElement("Agent:", agent, undefined, span.batch.agentId ? span.batch.agentId : undefined));
   //elements.push(dataElement("Type:", agentType));

   let poolLink: string | undefined;
   if (pool?.pool) {
      poolLink = `/pools?pool=${pool.pool}`
   }

   elements.push(dataElement("Pool:", agentPool, poolLink, undefined, "_blank"));

   if (span.batch) {
      elements.push(dataElement("Batch:", `View Batch`, `/job/${details!.jobId}?batch=${span.batch.id}`));
   }

   if (span.batch.logId) {
      elements.push(dataElement("", "View Log", `/log/${span.batch.logId}`, undefined, "_blank"));
   }


   if (span.cost) {
      elements.push(dataElement("Cost:", `$${span.cost.toFixed(2)}`));
   }

   let elapsed = msecToElapsed(span.utcFinish.getTime() - span.utcStart.getTime(), true, false);

   let spanType = "Wait:";
   if (span.type === 1) {
      spanType = "Init:";
   }
   else if (span.type === 2) {
      spanType = "Time:";
   }

   const startTime = getShortNiceTime(span.utcStart, false, true, false);
   const finishTime = getShortNiceTime(span.utcFinish, false, true, false);

   elapsed = `${elapsed} / ${startTime} - ${finishTime}`;

   elements.push(dataElement(spanType, elapsed));


   return <div style={{
      position: "absolute",
      display: "block",
      top: `${tooltip.y}px`,
      left: `${tooltip.x + offsetX}px`,
      backgroundColor: modeColors.background,
      zIndex: 1,
      border: "solid",
      borderWidth: "1px",
      borderRadius: "3px",
      width: "max-content",
      borderColor: dashboard.darktheme ? "#413F3D" : "#2D3F5F",
      pointerEvents: tooltip.frozen ? undefined : "none",
      transform: `translate(${translateX}, ${translateY})`
   }}>
      {!!viewAgent && <HistoryModal agentId={viewAgent} onDismiss={() => setViewAgent("")} />}
      <Stack horizontal>
         <Stack tokens={{ childrenGap: 6, padding: 12 }}>
            {elements}
         </Stack>
         {!!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12, paddingTop: 12, cursor: "pointer" }} onClick={() => { tooltip.freeze(false); tooltip.update() }}>
            <FontIcon iconName="Cancel" />
         </Stack>}
         {!tooltip.frozen && <Stack style={{ paddingLeft: 32, paddingRight: 12 + 16, paddingTop: 12 }} />}

      </Stack>
   </div>
})

const TimelineGraph: React.FC<{ dataView: TimelineDataView, filterTime: number, filterCost: number, stepId?: string }> = ({ dataView, filterTime, filterCost, stepId }) => {

   const graph_container_id = `timeline_graph_container`;

   const [container, setContainer] = useState<HTMLDivElement | null>(null);
   const [state, setState] = useState<{ graph?: TimelineRenderer }>({});
   const navigate = useNavigate();

   const { hordeClasses } = getHordeStyling();

   // todo: we can probably cache
   dataView.filterTime = filterTime;
   dataView.filterCost = filterCost;
   if (dataView.filterStepId !== stepId) {
      dataView.filterStepId = stepId;
      dataView.dirty = true;
   }


   if (!state.graph) {
      setState({ ...state, graph: new TimelineRenderer(dataView) })
      return null;
   }

   if (container) {
      try {
         state.graph?.render(container, navigate);

      } catch (err) {
         console.error(err);
      }

   }

   return <Stack className={hordeClasses.horde}>
      <Stack style={{ paddingLeft: 8, paddingTop: 8 }}>
         <div style={{ position: "relative" }}>
            <GraphTooltip dataView={dataView} />
         </div>

         <div id={graph_container_id} className="horde-no-darktheme" style={{ shapeRendering: "crispEdges", userSelect: "none", position: "relative" }} ref={(ref: HTMLDivElement) => setContainer(ref)} onMouseEnter={() => { }} onMouseLeave={() => { }} />
      </Stack>
   </Stack>;

}


JobDetailsV2.registerDataView("TimelineDataView", (details: JobDetailsV2) => new TimelineDataView(details));

export const TimelinePanel: React.FC<{ jobDetails: JobDetailsV2, stepId?: string }> = observer(({ jobDetails, stepId }) => {

   const [state, setState] = useState({ time: 0, cost: 0 });
   const { hordeClasses } = getHordeStyling();

   if (jobDetails.updated) { }

   const dataView = jobDetails.getDataView<TimelineDataView>("TimelineDataView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   dataView.subscribe();

   if (!jobDetails.jobData) {
      return null;
   }

   dataView.set();

   const job = jobDetails.jobData;
   const batches = job.batches?.filter(b => !!b.agentId && (!!b.startTime && !!b.finishTime));

   if (!batches?.length) {
      return null;
   }

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 18 }}>
            <Stack horizontal>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Timeline</Text>
               </Stack>
               <Stack grow />
               {!!dataView.minTime && <Stack horizontal tokens={{ childrenGap: 18 }}>
                  <Stack>
                     <Slider
                        styles={{ root: { width: 240 } }}
                        label="Min Time"
                        min={0}
                        max={dataView.maxSpanMS!}
                        value={state.time}
                        step={10}
                        showValue
                        valueFormat={(value) => {
                           if (!value) return ""
                           return msecToElapsed(value, true, false)
                        }}
                        onChange={(time) => {
                           setState({ ...state, time: time })
                        }}
                     />
                  </Stack>
                  {!!dataView.maxSpanCost && <Stack>
                     <Slider
                        styles={{ root: { width: 240 } }}
                        label="Min Cost"
                        min={0}
                        step={dataView.maxSpanCost! / 10}
                        max={dataView.maxSpanCost!}
                        value={state.cost}
                        showValue
                        valueFormat={(value) => {
                           if (!value) return ""
                           return `$${value.toFixed(2)}`
                        }}
                        onChange={(cost) => {
                           setState({ ...state, cost: cost })
                        }}
                     />
                  </Stack>}
               </Stack>}

            </Stack>
            <Stack>
               {!!dataView.minTime && <TimelineGraph dataView={dataView} stepId={stepId} filterTime={state.time} filterCost={state.cost} />}
               {!dataView.minTime && <Spinner size={SpinnerSize.large} />}
            </Stack>
         </Stack>
      </Stack>
   </Stack>);

})
