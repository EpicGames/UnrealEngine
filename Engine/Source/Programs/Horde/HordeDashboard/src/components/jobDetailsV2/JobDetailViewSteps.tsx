// Copyright Epic Games, Inc. All Rights Reserved.

import { CollapseAllVisibility, ConstrainMode, DetailsHeader, DetailsList, DetailsListLayoutMode, DetailsRow, DirectionalHint, FontIcon, IColumn, Icon, IDetailsListProps, ProgressIndicator, SelectionMode, Stack, Text, TooltipHost } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useId, useState } from "react";
import { Link, useParams } from "react-router-dom";
import { BatchData, GetBatchResponse, JobStepBatchError, JobStepBatchState, JobStepOutcome, JobStepState, StepData } from "../../backend/Api";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { ISideRailLink } from "../../base/components/SideRail";
import { getBatchInitElapsed, getNiceTime, getStepElapsed, getStepETA, getStepFinishTime, getStepPercent, getStepStartTime, getStepTimingDelta, HordeTime } from "../../base/utilities/timeUtils";
import { HistoryModal } from "../agents/HistoryModal";
import { getBatchText, getStepStatusMessage } from "../JobDetailCommon";
import { StepStatusIcon } from "../StatusIcon";
import { LabelsPanelV2 } from "./JobDetailLabels";
import { JobDataView, JobDetailsV2, JobFilterBar, StateFilter } from "./JobDetailsViewCommon";
import { getHordeTheme } from "../../styles/theme";
import { getHordeStyling } from "../../styles/Styles";
import { getStepMessageInfo } from "./JobDetailsUtils";

const stepsSideRail: ISideRailLink = { text: "Steps", url: "rail_steps" };
const depSideRail: ISideRailLink = { text: "Dependencies", url: "rail_dependencies" };

type StepItem = {
   step?: StepData;
   batch?: BatchData;
   agentId?: string;
   agentRow?: boolean;
   agentType?: string;
   agentPool?: string;
};


class StepsDataView extends JobDataView {

   filterUpdated() {
      this.updateReady();
   }


   clear() {
      super.clear();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();

   }

   order = 1;


}

let stepToolTipId = 0;

JobDetailsV2.registerDataView("StepsDataView", (details: JobDetailsV2) => new StepsDataView(details));

const RenderDynamic: React.FC<{ jobDetails: JobDetailsV2, dataView: StepsDataView, step?: StepData, batch?: GetBatchResponse, column: string }> = observer(({ jobDetails, dataView, column, step, batch }) => {

   dataView.subscribe();

   if (step && !step.finishTime && !step.abortRequested && (step.state !== JobStepState.Aborted && step.state !== JobStepState.Completed && step.state !== JobStepState.Skipped)) {
      dataView.subscribeToTick();
   }

   if (batch && !batch.finishTime && (batch.state !== JobStepBatchState.Complete)) {
      dataView.subscribeToTick();
   }

   if (step && column === "Progress") {

      if (step.state === JobStepState.Running && !step.abortRequested) {
         return <Stack horizontalAlign={"center"}> {step.startTime && !step.finishTime && <ProgressIndicator percentComplete={getStepPercent(step)} barHeight={2} styles={{ root: { paddingTop: 2, width: 120 } }} />}</Stack>;
      }

      const retries = jobDetails.getStepRetries(step.id);
      const retry = retries.length > 1 ? retries[retries.length - 1] : undefined;

      if (retry && retry.id !== step.id) {
         return <Stack horizontal horizontalAlign={"center"} tokens={{ childrenGap: 0, padding: 0 }} style={{ fontSize: "13px" }}><Link to={`/job/${jobDetails.jobId!}?step=${retry.id}`}>{`Retried - ${getStepStatusMessage(retry)}`}</Link></Stack>;
      }


      // Handle step message
      const report = step?.reports?.find(r => !!r.message);
      const { message: stepMessage, color: stepMessageColor } = getStepMessageInfo(report);
      if (stepMessage) {
         return <Stack horizontal verticalFill verticalAlign="center" tokens={{ childrenGap: 6 }} horizontalAlign="center">
            {!!stepMessageColor && <FontIcon style={{ fontSize: 13, color: stepMessageColor }} iconName="Square" />}
            <Text variant="tiny" style={{ overflow: "hidden", textOverflow: "ellipsis", wordBreak: "normal" }}>{stepMessage}</Text>
         </Stack>
      }

      return <Stack horizontal horizontalAlign={"center"} tokens={{ childrenGap: 0, padding: 0 }}><Text>{getStepStatusMessage(step)}</Text></Stack>;
   }


   if (column === "Duration") {

      let duration = "";

      if (batch) {

         if (batch?.startTime) {

            duration = getBatchInitElapsed(batch);
         }

      }

      if (step && !duration) {
         if (step.startTime) {
            duration = getStepElapsed(step);
         }
      }

      return <Stack horizontalAlign={"end"}><Text>{duration}</Text></Stack>;

   }

   if (step && column === "ETA") {

      let eta: HordeTime = {
         display: "",
         server: ""
      };

      if (step.state === JobStepState.Skipped) {
         return null;
      }

      let finished: HordeTime = { display: "", server: "" };

      eta = getStepETA(step, jobDetails.jobData!);

      finished = getStepFinishTime(step);

      if (finished.display) {
         eta.display = finished.display;
         eta.server = finished.server;
         eta.displayNice = finished.displayNice;
         eta.serverNice = finished.serverNice;
      }

      let time = eta.display;

      const etaColor = dashboard.darktheme ? "#A9A9A9" : "#999999";

      const color = !step.finishTime ? etaColor : undefined;

      // Open Sans tilde rendering issue at 13px, and not rendering at all at other px: https://github.com/google/fonts/issues/399, do not change from 12px
      return <TooltipHost content={eta.displayNice} directionalHint={DirectionalHint.leftCenter} id={`step_unique_tooltip_${stepToolTipId++}`}><Stack horizontalAlign={"end"}>
         <Stack horizontal tokens={{ childrenGap: 2 }}>
            {!!time && !step.finishTime && <Text style={{ color: color, fontSize: "11px", paddingTop: 2 }}>~</Text>}
            <Text style={{ color: color, fontSize: "13px" }}>
               {time}
            </Text>
         </Stack>
      </Stack>
      </TooltipHost>;
   };

   return null;

});


const StepsPanelInner: React.FC<{ jobDetails: JobDetailsV2, depStepId?: string }> = observer(({ jobDetails, depStepId }) => {

   const { jobId } = useParams<{ jobId: string }>();
   const [lastSelectedAgent, setLastSelectedAgent] = useState<string | undefined>(undefined);

   // IMPORTANT: Do not use useQuery() hook as will negatively impact rendering

   const dataView = jobDetails.getDataView<StepsDataView>("StepsDataView");

   if (depStepId) {
      dataView.order = 3;
   }

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   dataView.subscribe();

   const hordeTheme = getHordeTheme();

   const jobFilter = jobDetails.filter;

   // subscribe to input changes as user types
   if (jobFilter.inputChanged) { }

   let stepId: string | undefined = depStepId ? depStepId : jobFilter.step?.id;
   let agentType: string | undefined;
   let agentPool: string | undefined;
   let singleStep: boolean | undefined;

   const label = jobFilter.label;

   const buildColumns = (): IColumn[] => {

      const widths: Record<string, number> = {
         "Name": 732,
         "Progress": 120,
         "Start": 100,
         "End": 100,
         "Duration": 100,
         "ViewLogColumn": 88
      };

      // ViewLogColumn is tied to css styling to make entire column clickable
      const cnames = ["Name", "Progress", "Start", "End", "Duration", "ViewLogColumn"];

      return cnames.map(c => {
         return { key: c, name: c, fieldName: c.replace(" ", "").toLowerCase(), minWidth: widths[c] ?? 100, maxWidth: widths[c] ?? 100, isResizable: false, isMultiline: true } as IColumn;
      });
   };

   // main header
   const onRenderDetailsHeader: IDetailsListProps['onRenderDetailsHeader'] = (props) => {
      if (props) {
         props.selectionMode = SelectionMode.none;
         props.collapseAllVisibility = CollapseAllVisibility.hidden;
         return <DetailsHeader  {...props} />;
      }
      return null;
   };

   function renderAgentRow(item: StepItem, index?: number, column?: IColumn) {

      let error = !!item.batch?.startTime && item.batch?.error !== JobStepBatchError.None;
      let warning = false; //!!jobDetails.events.find(e => e.severity === EventSeverity.Warning && e.logId === item.batch?.logId);

      if (error) {
         warning = false;
      }

      let url = "";
      if (item.agentType && item.agentPool) {
         url = `/job/${jobId}?agenttype=${encodeURIComponent(item.agentType)}&agentpool=${encodeURIComponent(item.agentPool)}`;
      }

      if (!item.agentId && column!.key === 'ViewLogColumn') {
         return <div />;
      }

      const disabled = !item.batch?.logId;

      let batchText = getBatchText(item);

      if (!batchText) {
         if (item.agentId) {
            batchText = item.agentId;
         } else {
            batchText = "Unassigned";
         }
         if (item.batch?.state) {
            batchText += " - ";
            batchText += item.batch.state;
         }
      }

      const failure = error && item.batch?.error !== JobStepBatchError.NoLongerNeeded;

      const statusColors = dashboard.getStatusColors();
      const errorColor = failure ? (dashboard.darktheme ? "#F88070" : statusColors.get(StatusColor.Failure)) : undefined;
      const errorWeight = failure ? "600" : undefined;

      // Create tooltip content from leaseOutcomeReasons
      const leaseOutcomeReasonsTooltip = item.batch?.leaseOutcomeReasons?.length ? (
         <Stack tokens={{ childrenGap: 4 }}>
            <Text style={{ fontWeight: 600 }}>Cancellation reasons</Text>
            {item.batch.leaseOutcomeReasons.map((reason, idx) => (
               <Text key={idx}>• {reason}</Text>
            ))}
         </Stack>
      ) : undefined;

      // Helper to wrap element with tooltip if reasons exist
      const wrapWithTooltip = (element: JSX.Element) => leaseOutcomeReasonsTooltip
         ? <TooltipHost content={leaseOutcomeReasonsTooltip} id={`batch_reasons_${item.batch?.id}`} directionalHint={DirectionalHint.rightCenter}>{element}</TooltipHost>
         : element;

      switch (column!.key) {

         case 'Name':
            if (!item.agentId) {
               const batchTextElement = <Text styles={{ root: { color: errorColor, fontWeight: errorWeight, paddingRight: leaseOutcomeReasonsTooltip ? 4 : undefined } }} nowrap={true}>{batchText}</Text>;
               if (item.agentPool && (item.batch?.state === JobStepBatchState.Ready || item.batch?.error === JobStepBatchError.NoAgentsOnline || item.batch?.error === JobStepBatchError.NoAgentsInPool)) {
                  return <Stack horizontal disableShrink={true} horizontalAlign="start">
                     {(error || warning) && <Icon styles={{ root: { paddingTop: 3, paddingRight: 8, color: statusColors.get(StatusColor.Failure), fontWeight: errorWeight } }} iconName="Error" />}
                     <Link to={`/pools?pool=${encodeURI(item.agentPool)}`}>{wrapWithTooltip(batchTextElement)}</Link>
                  </Stack>
               }
               return <Stack horizontal disableShrink={true} horizontalAlign="start">
                  {(error || warning) && <Icon styles={{ root: { paddingTop: 3, paddingRight: 8, color: statusColors.get(StatusColor.Failure), fontWeight: errorWeight } }} iconName="Error" />}
                  {wrapWithTooltip(batchTextElement)}
               </Stack>;
            } else {
               const batchTextElement = <Text styles={{ root: { cursor: 'pointer', color: errorColor, fontWeight: errorWeight, paddingRight: leaseOutcomeReasonsTooltip ? 4 : undefined } }}>{batchText}</Text>;
               return <Stack horizontal disableShrink={true}>{(error || warning) && <Icon styles={{ root: { paddingTop: 3, paddingRight: 8, color: statusColors.get(StatusColor.Failure), fontWeight: errorWeight } }} iconName="Error" />}
                  <Stack>
                     <Link to={url} onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); setLastSelectedAgent(item.agentId); }}>{wrapWithTooltip(batchTextElement)}</Link>
                  </Stack>
               </Stack>
            }
         case 'Duration':
            return <RenderDynamic jobDetails={jobDetails} batch={item.batch} dataView={dataView} column="Duration" />
         case 'Progress':
            return null;

         case 'ViewLogColumn':
            return <React.Fragment>
               {!disabled && <Link onClick={(ev) => { ev.stopPropagation(); }} to={`/log/${item.batch?.logId}`}> <Stack horizontal horizontalAlign={"end"} verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}>
                  <Text styles={{ root: { margin: '0px', padding: '0px', paddingRight: '14px' } }} className={"view-log-link"}>View Log</Text>
               </Stack></Link>}
            </React.Fragment>;

         default:
            return <div />;
      }
   }

   function renderStepRow(item: StepItem, index?: number, column?: IColumn) {

      const step = item.step!;
      const disabled = !item.step?.logId;
      const spawnedJobs = step.spawnedJobs ?? [];
      const spawnedJobUrl = spawnedJobs.length > 0 ? `/job/${spawnedJobs[0]}` : undefined;

      const renderViewLogColumn = () => {

         // see data-automation-key in onRenderRow selectors
         return <React.Fragment>
            {!disabled && <Link onClick={(ev) => { ev.stopPropagation(); }} to={`/log/${item.step!.logId}`}> <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ paddingLeft: 33, width: "100%", height: "100%" }}>
               <Text styles={{ root: { margin: '0px', padding: '0px', paddingRight: '14px' } }} className={"view-log-link"}>View Log</Text>
            </Stack></Link>}
         </React.Fragment>;
      };


      const renderStart = () => {

         const started = getStepStartTime(step);
         if (!started.display || !started.server) {
            return null;
         } else {
            return <TooltipHost content={started.displayNice ?? started.display} directionalHint={DirectionalHint.leftCenter} id={`step_unique_tooltip_${stepToolTipId++}`}><Stack horizontalAlign={"end"}><Text style={{ fontSize: "13px" }}>{started.display}</Text></Stack></TooltipHost>;
         }
      };

      const stepName = jobDetails.getStepName(step.id) ?? "Unnown Step Name";
      const stepUrl = `/job/${jobId}?step=${step.id}`;


      switch (column!.key) {

         case 'Name':
            // see data-automation-key in onRenderRow selectors
            return <Stack horizontal horizontalAlign="start" verticalAlign="center" verticalFill tokens={{ childrenGap: 24 }}>
               <Stack><Link to={stepUrl}><Stack horizontal horizontalAlign="start" verticalAlign="center" style={{ width: "100%", height: "100%", paddingLeft: 8 }}><StepStatusIcon step={step} />
                  <Text style={{ paddingTop: 1 }}>{stepName}</Text>
               </Stack>
               </Link>
               </Stack>
               {!!spawnedJobUrl && <Stack><Text>(<a href={spawnedJobUrl}> Spawned Job </a>)</Text></Stack>}
            </Stack>;
         case 'Progress':
            return <RenderDynamic jobDetails={jobDetails} dataView={dataView} step={step} column="Progress" />
         case 'Duration':
            return <RenderDynamic jobDetails={jobDetails} dataView={dataView} step={step} column="Duration" />
         case 'Start':
            return renderStart();
         case 'End':
            return <RenderDynamic jobDetails={jobDetails} dataView={dataView} step={step} column="ETA" />
         case 'ViewLogColumn':
            return renderViewLogColumn();
         default:
            return <div />;
      }

   }

   // apply background styling
   const onRenderRow: IDetailsListProps['onRenderRow'] = (props) => {
      if (props) {
         const item = props.item as StepItem;
         if (item.agentRow) {
            props.styles = { ...props.styles, root: { background: `${hordeTheme.horde.dividerColor} !important`, selectors: { ".ms-DetailsRow-cell": { "overflow": "visible" } } } };
         } else {
            props.styles = { ...props.styles, root: { selectors: { ".ms-DetailsRow-cell": { "overflow": "visible" }, "div[data-automation-key=\"Name\"],div[data-automation-key=\"ViewLogColumn\"]": { padding: 0 } } } };
         }

         return <div className="job-item" ><DetailsRow {...props} /> </div>;

      }
      return null;
   };

   function onRenderItemColumn(item: StepItem, index?: number, column?: IColumn) {
      // data-automation-key="Name"
      return item.step ? renderStepRow(item, index, column) : renderAgentRow(item, index, column);
   }

   let items: StepItem[] = [];

   const stepFilter: StepData[] = [];

   const jobStateFilter = new Set<string>();

   const stateFilter = jobFilter.filterStates;
   jobDetails.getSteps().forEach(step => {

      if (stateFilter.indexOf("All") !== -1) {
         return;
      }

      if (stateFilter.indexOf("Skipped") === -1 && step.state === JobStepState.Skipped) {
         jobStateFilter.add(step.id);
         return;
      }

      if (stateFilter.indexOf("Aborted") === -1 && step.state === JobStepState.Aborted) {
         jobStateFilter.add(step.id);
         return;
      }

      if (stateFilter.indexOf(step.outcome as StateFilter) === -1 && stateFilter.indexOf(step.state) === -1) {
         jobStateFilter.add(step.id);
      }
   })

   if (stepId) {

      const getStepsRecursive = (stepId: string) => {
         const rstep = jobDetails.stepById(stepId);
         if (!rstep) {
            return;
         }
         [rstep.inputDependencies, rstep.optionalInputDependencies, rstep.orderDependencies].flat().forEach(id => {
            const s = jobDetails.stepById(id);
            if (s && !stepFilter.find(s => s.id === id)) {
               stepFilter.push(s);
               getStepsRecursive(s.id);
            }
         });
      };

      getStepsRecursive(stepId);

      const step = jobDetails.stepById(stepId);
      if (step) {
         stepFilter.push(step);
      }

      if (!stepFilter.length) {
         return null;
      }

   }

   // do not use useQuery() hook as will negatively impact rendering   
   const query = new URLSearchParams(window.location.search);
   const batchFilter = query.get("batch");

   let jobBatches = jobDetails.batches.sort((a, b) => a.groupIdx - b.groupIdx);

   jobBatches.forEach(b => {

      if (batchFilter && b.id !== batchFilter) {
         return;
      }

      if (agentType) {

         if (b.agentId) {
            return;
         }

         const p = jobDetails.stream!.agentTypes[b.agentType];

         if (!p || b.agentType !== agentType || p.pool !== agentPool) {
            return;
         }
      }

      const steps = b.steps.filter(step => {

         if (jobStateFilter.has(step.id)) {
            return false;
         }

         if (jobFilter.currentInput) {
            if (jobDetails.getStepName(step.id, false)?.toLowerCase().indexOf(jobFilter.currentInput?.toLowerCase()) === -1) {
               return false;
            }
         }

         if (jobFilter.search) {
            if (jobDetails.getStepName(step.id, false)?.toLowerCase().indexOf(jobFilter.search.toLowerCase()) === -1) {
               return false;
            }
         }

         let filter = true;

         if (stepId) {
            if (!stepFilter.find(s => s.id === step.id)) {
               filter = false;
            }
         } else if (label) {

            filter = false;

            if (label.steps.indexOf(step?.id ?? "") !== -1) {
               filter = true;
            }

         }

         return filter;

      });

      // if the batch has steps and they are all filtered, don't show batch
      if (!b.steps.length || (b.steps.length && !steps.length)) {
         return;
      }

      const pool = jobDetails.stream?.agentTypes[b?.agentType!];

      items.push({
         agentId: b.agentId,
         batch: b,
         agentRow: true,
         agentType: b.agentType.toUpperCase(),
         agentPool: pool?.pool?.toUpperCase(),
      });

      steps.forEach(stepData => {
         const id = stepData.id;
         items.push({
            step: jobDetails.stepById(id)
         });
      });
   });

   if (singleStep) {
      items = items.filter(item => (item.step?.id === stepId) || (item.agentRow && item.batch?.steps.find(s => s.id === stepId)));
   }

   // get the current groups based on filtering
   const stepBatches: Set<string> = new Set();
   items.forEach(item => {
      const step = item.step;
      if (!step) {
         return;
      }

      const b = jobDetails.jobData?.batches?.find(b => b.steps.findIndex(s => s.id === step.id) !== -1);
      if (b)
         stepBatches.add(b.id);
   });

   let batches = jobBatches.filter(b => {
      if ((stepBatches.size && !stepBatches.has(b.id)) || b.steps.length || b.error === JobStepBatchError.None) {
         return false;
      }
      return true;
   }).reverse();

   batches.forEach(b => {

      if (batchFilter && b.id !== batchFilter) {
         return;
      }

      const pool = jobDetails.stream?.agentTypes[b?.agentType!];

      const nitem = {
         agentId: b.agentId,
         batch: b,
         agentRow: true,
         agentType: b?.agentType.toUpperCase(),
         agentPool: pool?.pool?.toUpperCase(),
      };

      const index = items.findIndex(item => item.batch?.groupIdx === b.groupIdx);

      if (index === -1) {
         items.push(nitem);
      } else {
         items.splice(index, 0, nitem);
      }

   })

   const sideRail = depStepId ? depSideRail : stepsSideRail;

   dataView.initialize(items.length ? [sideRail] : undefined);

   if (!items.length) {
      return null;
   }

   const stepList = <Stack>
      <DetailsList
         isHeaderVisible={false}
         indentWidth={0}
         compact={dashboard.compactViews}
         selectionMode={SelectionMode.none}
         items={items}
         columns={buildColumns()}
         layoutMode={DetailsListLayoutMode.fixedColumns}
         constrainMode={ConstrainMode.unconstrained}
         onRenderDetailsHeader={onRenderDetailsHeader}
         onRenderItemColumn={onRenderItemColumn}
         onRenderRow={onRenderRow}
         onShouldVirtualize={() => { return false; }} // <--- previous versions of Fluent were very slow if not virtualized, new version seems faster and not virtualizing makes the size estimations less janky, if there are performance issues, look here
      />
   </Stack>;


   if (singleStep) {
      return <div id={sideRail.url}>{stepList}<HistoryModal agentId={lastSelectedAgent} onDismiss={() => setLastSelectedAgent(undefined)} /></div>;
   }
   return <Stack>
      <Stack style={{ paddingBottom: 14 }}>
         {!depStepId && <LabelsPanelV2 jobDetails={jobDetails} dataView={dataView} />}
      </Stack>

      <Stack tokens={{ padding: 4 }}>
         {stepList}
      </Stack>
      <HistoryModal agentId={lastSelectedAgent} onDismiss={() => setLastSelectedAgent(undefined)} />
   </Stack >


});

export const StepsPanelV2: React.FC<{ jobDetails: JobDetailsV2, depStepId?: string }> = observer(({ jobDetails, depStepId }) => {

   const { hordeClasses } = getHordeStyling();

   jobDetails.subscribe();

   if (!jobDetails.jobData) {
      return null;
   }

   const sideRail = depStepId ? depSideRail : stepsSideRail;

   // do not use useQuery() hook as will negatively impact rendering   
   const query = new URLSearchParams(window.location.search);
   const batchFilter = query.get("batch");
   let stepsName = "Steps";
   if (batchFilter) {
      stepsName = `Steps (Batch ${batchFilter})`;
   }


   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack>
               <Stack horizontal verticalAlign="center">
                  <Stack>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{depStepId ? "Dependencies" : stepsName}</Text>
                  </Stack>
                  <Stack grow />
                  {!depStepId && <Stack verticalAlign="center">
                     <JobFilterBar jobDetails={jobDetails} />
                  </Stack>}
               </Stack>
               <StepsPanelInner jobDetails={jobDetails} depStepId={depStepId} />
            </Stack>
         </Stack>
      </Stack>

   </Stack>);

});