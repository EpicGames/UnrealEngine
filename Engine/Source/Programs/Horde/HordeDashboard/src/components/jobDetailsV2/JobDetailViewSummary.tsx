// Copyright Epic Games, Inc. All Rights Reserved.

import { Label, Separator, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useEffect } from 'react';
import { DashboardPreference, JobState, ReportPlacement } from '../../backend/Api';
import { Markdown } from '../../base/components/Markdown';
import { ISideRailLink } from '../../base/components/SideRail';
import { getNiceTime } from '../../base/utilities/timeUtils';
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { getHordeStyling } from '../../styles/Styles';
import dashboard from 'horde/backend/Dashboard';

const sideRail: ISideRailLink = { text: "Summary", url: "rail_summary" };

class SummaryDataView extends JobDataView {

   filterUpdated() {

   }

   clear() {
      super.clear();
   }


   detailsUpdated() {

      const details = this.details;

      if (!details) {
         return;
      }

      const jobData = details.jobData;

      if (!jobData) {
         return;
      }

      let dirty = true;

      if (dirty) {
         this.updateReady();
      }
   }

   order = -1;

}

JobDetailsV2.registerDataView("SummaryDataView", (details: JobDetailsV2) => new SummaryDataView(details));

export const SummaryPanel: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const { hordeClasses, modeColors } = getHordeStyling();

   if (jobDetails.updated) { }

   const summaryView = jobDetails.getDataView<SummaryDataView>("SummaryDataView");

   useEffect(() => {
      return () => {
         summaryView.clear();
      }
   }, [summaryView]);

   summaryView.subscribe();

   summaryView.initialize([sideRail]);

   const text: string[] = [];

   const jobData = jobDetails.jobData;

   /*
   if (!jobDetails?.viewsReady) {
      console.log("Summary Views Not Ready", jobDetails);
      jobDetails?.views.forEach(v => {
         console.log(v.name, v.initialized)
      })

   }*/

   if (jobDetails.jobError) {
      document.title = "Error Loading Job";
      let errorTextColor: string | undefined = dashboard.getPreference(DashboardPreference.ColorError) ?? "#c44525";

      return <Stack style={{ paddingTop: 64, paddingBottom: 82 }}>
         <Text variant="large" styles={{ root: { fontFamily: "Horde Open Sans SemiBold", padding: "25px 30px 25px", color: errorTextColor } }}>Error loading job, please check network connection.</Text>
         <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold", padding: "10px 30px 25px", color: errorTextColor } }}>{jobDetails.jobError}</Text>
      </Stack>
   }

   if (!jobData) {
      return null;
   }

   const price = jobDetails.jobPrice();

   const timeStr = getNiceTime(jobData.createTime);

   let jobText = `Job created ${timeStr} by ${jobData.startedByUserInfo ? jobData.startedByUserInfo.name : "scheduler"} and `;

   if (jobDetails.aborted) {
      jobText += "was canceled";
      if (jobData.abortedByUserInfo) {
         jobText += ` by ${jobData.abortedByUserInfo.name}.`;
      } else {
         jobText += ` by Horde.`;
      }
   } else {
      jobText += `${jobData.state === JobState.Complete ? `completed ${getNiceTime(jobData.updateTime, false)}.` : "is currently running."}`;
   }

   text.push(jobText);

   const summary = text.join(".  ");

   const reportData = jobDetails.getSummaryReport();

   const parentJobUrl = jobData.parentJobId && jobData.parentJobId ? `/job/${jobData.parentJobId}?step=${jobData.parentJobStepId}` : undefined;

   const desc = jobDetails.template?.description;

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 0, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised} >
         <Stack tokens={{ childrenGap: 12 }} grow>
            <Stack horizontal>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Summary</Text>
               </Stack>
            </Stack>
            <Stack >
               <Stack tokens={{ childrenGap: 12 }}>
                  <Text styles={{ root: { whiteSpace: "pre" } }}>{"" + summary}</Text>
                  {!!jobData.cancellationReason && <Stack style={{ color: modeColors.text }} tokens={{ childrenGap: 8 }}>
                     <Label>Cancellation Reason</Label>
                     <Markdown>{jobData.cancellationReason}</Markdown>
                  </Stack>
                  }
                  {!!parentJobUrl && <Stack><Markdown>{`This job was spawned by a [parent job](${parentJobUrl}).`}</Markdown></Stack>}
                  {!!reportData && <Stack> <Markdown>{reportData}</Markdown></Stack>}
                  {!!price && <Stack>
                     <Text>{`Estimated cost: $${price.toFixed(2)}`}</Text>
                  </Stack>}
                  {!!desc && <Stack> <Separator /> <Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold"} }}>Description</Text><Markdown>{desc}</Markdown></Stack>}
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});
