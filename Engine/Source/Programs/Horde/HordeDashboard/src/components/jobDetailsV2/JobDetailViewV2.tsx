// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyleSets, Pivot, PivotItem, Spinner, SpinnerSize, Stack } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { useEffect, useRef, useState } from "react";
import { useLocation, useNavigate, useParams } from "react-router-dom";
import { GetJobsTabResponse } from "../../backend/Api";
import { useWindowSize } from "../../base/utilities/hooks";
import { getHordeStyling } from "../../styles/Styles";
import { BreadcrumbItem, Breadcrumbs } from "../Breadcrumbs";
import { useQuery } from "horde/base/utilities/hooks";
import { getPreflightDisplay } from "../../base/utilities/commitUtils";
import { getStreamDisplayName, getStreamLink } from "../../base/utilities/streamUtils";
import { TopNav } from "../TopNav";
import { JobArtifactsPanel } from "./JobDetailArtifacts";
import { BisectionPanel } from "./JobDetailBisection";
import { HealthPanel } from "./JobDetailHealthV2";
import { PreflightPanel } from "./JobDetailPreflight";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { TimelinePanel } from "./JobDetailTimeline";
import { StepsPanelV2 } from "./JobDetailViewSteps";
import { SummaryPanel } from "./JobDetailViewSummary";
import { JobOperations } from "./JobOperationsBar";
import { StepDetailView } from "./StepDetailView";
import userInactivity from "horde/backend/UserInactivity";
import { ErrorView } from "horde/components/error/ErrorView.tsx";

class BreadcrumbDataView extends JobDataView {

   filterUpdated() {

   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();
   }
}

JobDetailsV2.registerDataView("BreadcrumbDataView", (details: JobDetailsV2) => new BreadcrumbDataView(details));

const JobBreadCrumbs: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const query = useQuery();
   const stepId = query.get("step");

   const data = jobDetails.getDataView<BreadcrumbDataView>("BreadcrumbDataView");

   data.subscribe();

   data.initialize();

   const jobData = jobDetails.jobData;
   if (!jobData) {
      if (!jobDetails.jobError) {
         return <Breadcrumbs items={[{ text: "Loading Job" }]} title={"Loading Job"} spinner={true} />
      } else {
         return null;
      }
   }

   if (jobDetails.jobError) {
      return null;
   }

   const crumbItems: BreadcrumbItem[] = [];

   const stream = jobDetails.stream;

   if (stream) {

      const tab = stream.tabs?.find((tab) => {
         const jtab = tab as GetJobsTabResponse;
         return !!jtab.templates?.find(t => t === jobData.templateId);
      });

      const aliasParam = query.get("alias");
      let streamLink = getStreamLink(stream.id, aliasParam);
      if (tab) {
         streamLink += aliasParam ? `&tab=${tab.title}` : `?tab=${tab.title}`;
      }

      crumbItems.push({ text: stream.project!.name, link: `/project/${stream.project!.id}` });
      crumbItems.push({ text: getStreamDisplayName(stream, aliasParam), link: streamLink });

   }

   crumbItems.push({
      text: jobData.name,
      link: `/job/${jobData.id}`
   });


   let clText = "";
   const preflightDisplay = getPreflightDisplay(jobData.preflightCommitId, jobData.preflightChange);
   if (preflightDisplay) {
      clText = `Preflight ${preflightDisplay} `;
      clText += ` - Base ${jobData.change ? jobData.change : "Latest"}`;

   } else {
      clText = `${jobData.change ? "CL " + jobData.change : "Latest CL"}`;
   }

   if (crumbItems.length) {
      crumbItems[crumbItems.length - 1].text += ` - ${clText}`;
   }

   let crumbTitle = `${clText}`;
   if (stream) {
      crumbTitle = `${stream.fullname} - ${jobData.name} - ${clText}`;
   }

   if (stepId) {
      let stepText = jobDetails.getStepName(stepId);
      crumbItems.push({
         text: stepText
      });
   }

   const label = jobDetails.labelByIndex(query.get("label") ?? undefined);
   if (label) {
      crumbItems.push({
         text: label.stateResponse.dashboardCategory ? `${label.stateResponse.dashboardCategory}: ${label.stateResponse.dashboardName!}` : label.stateResponse.dashboardName!
      });
   }

   return <Stack>
      <Breadcrumbs items={crumbItems} title={crumbTitle} />
   </Stack>

});

const pivotClasses = mergeStyleSets({
   pivot: {
      selectors: {
         ".ms-Pivot-link": {
            lineHeight: "36px",
            height: "36px",
            paddingTop: 0,
            paddingBottom: 0
         }
      }
   }
});

const visibility = new Map<string, number>();
let crender = 0;

const JobPanelPivot: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {
   useQuery();
   const navigate = useNavigate();
   const [state, setState] = useState<{ iobserver?: IntersectionObserver }>({});
   const [, setRender] = useState(0);

   useEffect(() => {
      return () => {
         visibility.clear();
         state.iobserver?.disconnect();
      };
   }, [state.iobserver]);

   jobDetails.subscribeToRoot();

   const rootArea = document.getElementById("hordeContentArea");
   const supported = typeof IntersectionObserver !== 'undefined';

   // install intersection observer
   if (supported && rootArea && !state.iobserver) {
      if (window.location.hash) {
         navigate(window.location.pathname + window.location.search, { replace: true });
      }
      setState({
         iobserver: new IntersectionObserver((entries: IntersectionObserverEntry[]) => {
            let nupdate = false;
            for (const entry of entries) {
               const current = visibility.get(entry.target.id);
               if (current !== entry.intersectionRatio) {
                  visibility.set(entry.target.id, entry.intersectionRatio);
                  nupdate = true;
               }
            }
            if (nupdate) {
               setRender(crender++);
            }
         }, {
            root: rootArea,
            threshold: [0.0, 0.2, 0.4, 0.6, 0.8, 1.0]
         })
      })
      return null;
   }

   if ((!jobDetails.jobData || !jobDetails.views.length || jobDetails.views.find(v => !v.initialized))) {
      return null;
   }

   const views = jobDetails.views.sort((a, b) => a.order - b.order);

   const links = views.map(v => v.railLinks).flat();

   state.iobserver?.disconnect();

   let currentHash: string | undefined = window.location.hash?.replace("#", "");
   currentHash = currentHash ? currentHash : undefined;

   let best = "";
   let bestv = 0.0;
   links.forEach(link => {

      const vis = visibility.get(link.url) ?? 0;
      if (vis && vis > bestv) {
         best = link.url;
         bestv = vis;
      }

      const element = document.getElementById(link.url);
      if (element) {
         state.iobserver?.observe(element);

      }
   });

   if (currentHash && visibility.get(currentHash)) {
      best = currentHash;
   }

   if (best && best !== currentHash) {
      currentHash = best;
   }

   if (!currentHash && links.length) {
      currentHash = links[0].url;
   }

   // selectedKey must be null and not undefined on first render, or pivot will be uncontrolled
   const selectedKey = currentHash ? `item_key_${currentHash}` : null;


   const viewLinks = links.map(link => {
      const key = `item_key_${link.url}`;
      return <PivotItem headerText={link.text} key={key} itemKey={key} />
   });

   return <Stack horizontal tokens={{ childrenGap: 18 }} verticalAlign="center" verticalFill={false}>
      <Pivot className={pivotClasses.pivot} selectedKey={selectedKey} linkSize="normal" linkFormat="links" onLinkClick={(item, ev) => {
         if (item) {

            const hashLink = item.props.itemKey!.replace("item_key_", "");
            const newUrl = window.location.pathname + window.location.search + "#" + hashLink;
            window.location.replace(newUrl);
            /*
            if (`#${hashLink}` === window.location.hash) {
               // clean and then set again to force jump, certainly all browsers wil observe this behavior
               //window.location.hash = "";
               //window.location.hash = hashLink;
            } else {
               window.location.hash = hashLink;
            }
            */
         }
      }}>
         {viewLinks}
      </Pivot>
   </Stack>;
});


const DetailsViewStep: React.FC<{ jobDetails: JobDetailsV2 }> = ({ jobDetails }) => {

   const query = useQuery();

   const stepId = query.get("step") ? query.get("step")! : undefined;

   if (!stepId) {
      return null;
   }

   const details = jobDetails;

   return <Stack style={{ width: "100%" }}>
      <StepDetailView jobDetails={details} stepId={stepId} />
   </Stack>
};

const DetailsViewOverview: React.FC<{ jobDetails: JobDetailsV2 }> = ({ jobDetails }) => {

   const query = useQuery();

   const stepId = query.get("step") ? query.get("step")! : undefined;

   const details = jobDetails;

   // Note the view clear has to be in here as happens before DetailsViewStep
   // this could probably be refactored
   if (stepId) {
      if (details.overview) {
         details.clearViews();
      }
      details.overview = false;
      return null;
   } else {
      if (!jobDetails.overview) {
         jobDetails.clearViews();
      }
   }

   details.overview = true;

   return <Stack style={{ width: "100%" }} >
      <SummaryPanel jobDetails={details} />
      <PreflightPanel jobDetails={details} />
      <JobArtifactsPanel jobDetails={details} />
      <StepsPanelV2 jobDetails={details} />
      <HealthPanel jobDetails={details} />
      <TimelinePanel jobDetails={details} />
      <BisectionPanel jobDetails={details} />
      {!jobDetails.viewsReady && <Stack style={{ paddingTop: 32 }}>
         <Spinner size={SpinnerSize.large} />
      </Stack>}

   </Stack>
};

const rootWidth = 1440;

const ScrollRestore: React.FC<{ jobDetails: JobDetailsV2, scrollRef: React.RefObject<HTMLDivElement> }> = ({ jobDetails, scrollRef }) => {

   const location = useLocation();

   if (!scrollRef?.current) {
      return null;
   }

   try {
      if (!location.hash) {
         scrollRef.current.scrollTop = 0;
      }
   } catch (message) {
      console.error(message);
   }

   return null;
}

const DetailsViewErrorView: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   jobDetails.subscribe();

   if (jobDetails.jobError) {
      return <ErrorView title="Unable to load job" description="Verify job and stream exists and potentially your internet connection." reason={jobDetails.jobError} />;
   }
})


/** This is a root view and should not be made an observer other than for a specific observable, ie. not JobDetails updates as these are very coarse and cause issues */
const DetailsView: React.FC<{ jobDetails: JobDetailsV2 }> = ({ jobDetails }) => {

   const windowSize = useWindowSize();
   const scrollRef = useRef<HTMLDivElement>(null);
   const { modeColors } = getHordeStyling();

   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const details = jobDetails;

   const centerAlign = vw / 2 - 720 /*890*/;
   const key = `windowsize_jobdetail_view_${windowSize.width}_${windowSize.height}`;

   return <Stack id="rail_job_overview">
      <DetailsViewErrorView jobDetails={jobDetails}/>
      <JobBreadCrumbs jobDetails={details} />
      <ScrollRestore jobDetails={details} scrollRef={scrollRef} />
      <Stack horizontal styles={{ root: { backgroundColor: modeColors.background } }}>
         <Stack styles={{ root: { width: "100%" } }}>
            <Stack horizontal>
               <Stack key={`${key}_1`} style={{ paddingLeft: centerAlign }} />
               <Stack horizontal style={{ width: rootWidth - 8, maxWidth: windowSize.width - 12, paddingLeft: 0, paddingTop: 24, paddingBottom: 24, paddingRight: 0 }} >
                  <Stack verticalAlign="center" style={{ height: 32 }}>
                     <JobPanelPivot jobDetails={details} />
                  </Stack>
                  <Stack grow />
                  <Stack verticalAlign="center" style={{ height: 32 }}>
                     <JobOperations jobDetails={details} /*stepId={stepId}*/ />
                  </Stack>
               </Stack>
            </Stack>
            <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
               <Stack horizontal>
                  <Stack /*className={classNames.pointerSuppress} */ style={{ position: "relative", width: "100%", height: 'calc(100vh - 228px)' }}>
                     <div id="hordeContentArea" ref={scrollRef} style={{ overflowX: "auto", overflowY: "visible" }}>
                        <Stack horizontal style={{ paddingBottom: 48 }}>
                           <Stack key={`${key}_2`} style={{ paddingLeft: centerAlign }} />
                           <Stack style={{ width: rootWidth }}>
                              <DetailsViewOverview jobDetails={details} />
                              <DetailsViewStep jobDetails={details} />
                           </Stack>
                        </Stack>
                     </div>
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

export const JobDetailViewV2: React.FC = () => {

   let { jobId } = useParams<{ jobId: string }>();
   const [state, setState] = useState<{ jobDetails?: JobDetailsV2 }>({});


   useEffect(() => {

      function reloadPage() {

         // If haven't touched page in 10 minutes reload it
         if (userInactivity.getSecondsSinceLastActivity() > 600) {
            location.reload();
         }

      }

      // reload the page once every 8 hours of inactivity
      const interval = setInterval(() => reloadPage(), 3600000 * 8);

      return () => {
         clearInterval(interval);
         state.jobDetails?.clear();
         state.jobDetails = undefined;
      };
   }, [state]);

   const { hordeClasses } = getHordeStyling();

   if (!jobId) {
      return null;
   }

   let stepId = new URLSearchParams(window.location.search).get("step") ?? "";

   let needNavigate = false;
   if (stepId && stepId.length !== 4) {
      needNavigate = true;
      stepId = stepId.slice(0, 4);
      if (stepId.length !== 4) {
         stepId = "";
      }
   }

   if (jobId.length !== 24) {
      needNavigate = true;
      jobId = jobId.slice(0, 24);
   }

   if (needNavigate) {
      if (jobId.length !== 24) {
         // navigate isn't working here
         window.location.assign("/");
      } else {
         let url = `/job/${jobId}`;

         if (stepId) {
            url += `?step=${stepId}`;
         }

         // navigate isn't working here
         window.location.assign(url);
      }
      return null;
   }

   if (!state.jobDetails || state.jobDetails.jobId !== jobId) {
      if (state.jobDetails) {
         state.jobDetails.clear();
      }
      const jobDetails = new JobDetailsV2(jobId);
      setState({ jobDetails: jobDetails });
      return null;
   }

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <DetailsView jobDetails={state.jobDetails} />
      </Stack>
   );
};

