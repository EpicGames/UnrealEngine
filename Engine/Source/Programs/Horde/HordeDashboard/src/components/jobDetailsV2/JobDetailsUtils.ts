import { GetJobStepRefResponse, GetReportResponse, ReportPlacement } from "horde/backend/Api";
import dashboard, { StatusColor } from "horde/backend/Dashboard";

export function getStepRefMessageInfo(ref: GetJobStepRefResponse | undefined): { message: string | undefined; color: string | undefined } {
    
    if (!ref?.metadata) {
        return { message: ref?.outcome ?? "", color: undefined };
    }

    const report: GetReportResponse = {
        placement: ReportPlacement.Summary,
        name: ref.stepName || "",
        message: ref.metadata.find(m => m.startsWith("ReportStepMessage="))?.split("=")[1]?.trim() ?? ref.outcome ?? "",
        color: ref.metadata.find(m => m.startsWith("ReportStepColor="))?.split("=")[1]?.trim(),
        severity: ref.metadata.find(m => m.startsWith("ReportStepSeverity="))?.split("=")[1]?.trim(),
    };

    return getStepMessageInfo(report);
}

export function getStepMessageInfo(report: GetReportResponse | undefined): { message: string | undefined; color: string | undefined } {
    
    if (!report?.message) {
        return { message: undefined, color: undefined };
    }

    const statusColors = dashboard.getStatusColors();
    const colorLower = report.color?.toLowerCase();
    const severityLower = report.severity?.toLowerCase();

    const userColors: Record<string, StatusColor> = { user1: StatusColor.User1, user2: StatusColor.User2, user3: StatusColor.User3 };
    const severityColors: Record<string, StatusColor> = { error: StatusColor.Failure, errors: StatusColor.Failure, warning: StatusColor.Warnings, warnings: StatusColor.Warnings };

    let color = colorLower && userColors[colorLower] ? statusColors.get(userColors[colorLower])
        : report.color?.match(/^#?([0-9a-fA-F]{6}|[0-9a-fA-F]{3})$/) ? (report.color.startsWith('#') ? report.color : `#${report.color}`)
            : severityLower && severityColors[severityLower] ? statusColors.get(severityColors[severityLower])
                : undefined;

    return { message: report.message, color };
}
