// Copyright Epic Games, Inc. All Rights Reserved.

import { Dropdown, FontIcon, IDropdownOption, IPickerItemProps, ISuggestionItemProps, ITag, PrimaryButton, Stack, TagPicker, Text, ValidationState } from "@fluentui/react";
import { action, makeObservable, observable } from "mobx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import React, { useState } from "react";
import { useLocation, useNavigate } from "react-router-dom";
import backend from "../../backend";
import { BatchData, GetArtifactResponse, GetJobTimingResponse, GetLabelStateResponse, GetReportResponse, GetTemplateRefResponse, JobData, JobLabel, JobState, LabelState, ReportPlacement, StepData, StreamData } from "../../backend/Api";
import { PollBase } from "../../backend/PollBase";
import { projectStore } from "../../backend/ProjectStore";
import TemplateCache from "../../backend/TemplateCache";
import { ISideRailLink } from "../../base/components/SideRail";
import { useQuery } from "horde/base/utilities/hooks";
import { LabelStatusIcon, StepStatusIcon } from "../StatusIcon";
import { getHordeStyling } from "../../styles/Styles";
import { errorBarStore } from "../error/ErrorStore";

export abstract class JobDataView {

   constructor(details: JobDetailsV2, initObservable?: boolean) {
      if (initObservable === undefined || initObservable) {
         makeObservable(this);
      }
      this.details = details;
   }

   @observable
   private updated = 0

   @action
   updateReady() {
      this.updated++;
   }

   subscribe() {
      if (this.updated) { }
      this.details?.subscribeToRoot();
   }

   subscribeToTick() {
      if (this.ticked) { }
   }

   @observable
   private ticked = 0

   @action
   tick() {
      this.ticked++;
   }


   abstract detailsUpdated(): void;
   abstract filterUpdated(): void;

   get railLinks(): ISideRailLink[] {
      return this._railLinks;
   }

   get initialized(): boolean {
      return this._initialized;
   }

   addRail(link: ISideRailLink) {

      const existing = this._railLinks.find(r => r.text === link.text);
      if (existing) {
         return;
      }

      this._railLinks.push(link);
      this.details?.externalUpdate();
      this.details?.setRootUpdated();
   }


   // may be called more than once with different parameters, though always initializes
   initialize(railLinks?: ISideRailLink[]) {

      if (this._initialized) {

         if (railLinks?.length === this._railLinks.length) {
            let i = 0;
            for (; i < this._railLinks.length; i++) {
               const rail1 = this._railLinks[i];
               const rail2 = railLinks[i];
               if (rail1.text !== rail2.text && rail1.url !== rail2.url) {
                  break;
               }
            }
            if (i === this._railLinks.length) {
               return;
            }
         }
      }

      const viewsReady = this.details?.viewsReady;

      this._initialized = true;
      this._railLinks = railLinks ?? [];
      this.details?.externalUpdate();

      if (viewsReady !== this.details?.viewsReady) {
         this.details?.setRootUpdated();
      }

   }

   clear() {
      if (!this.details) {
         return;
      }
      this.name = "Removed - " + this.name;
      const details = this.details;
      this.details = undefined;
      this.updateTime = undefined;
      this._initialized = false;
      this._railLinks = [];
      details?.removeDataView(this);
      console.log("Removed", this.name);
   }

   name: string = "Anonymous";

   order: number = 0;

   details?: JobDetailsV2;

   // job detail update time cache
   updateTime?: Date | string;

   private _initialized: boolean = false;
   private _railLinks: ISideRailLink[] = [];
}



const defaultUpdateMS = 5000;
export class JobDetailsV2 extends PollBase {

   constructor(jobId: string) {
      super(defaultUpdateMS)
      makeObservable(this);
      this.jobId = jobId;
      this.filter = new JobDetailFilters(this);
      this.start();
   }

   @action
   externalUpdate() {
      this.updated++;
   }

   // Be careful, this will generate a full tree render
   @action
   setRootUpdated() {
      this.rootUpdated++;
   }

   subscribeToRoot() {
      if (this.rootUpdated) { }
   }

   clearViews() {
      const views = [...this.views];
      views.forEach(v => v.clear());
   }

   clear() {
      super.stop();
      this.jobId = undefined;
      this.jobError = undefined;
      this.jobData = undefined;
      this.stream = undefined;
      this.template = undefined;
      this.labels = [];
      this.batches = [];
      this.views = [];
      this.timing = undefined;
      this.overview = undefined;
   }

   viewReady(order: number): boolean {

      const views = this.views.sort((a, b) => {
         return a.order - b.order;
      });

      for (let i = 0; i < views.length; i++) {
         if (!views[i].initialized && views[i].order < order) {
            return false;
         }
      }

      return true;
   }

   get viewsReady(): boolean {
      return !this.views.find(v => !v.initialized);
   }

   get targets(): string[] {

      const detailArgs = this.jobData?.arguments;

      if (!detailArgs) {
         return [];
      }

      const targets: string[] = [];

      if (detailArgs && detailArgs.length) {

         const targetArgs = detailArgs.map(arg => arg.trim()).filter(arg => arg.toLowerCase().startsWith("-target="));

         targetArgs.forEach(t => {

            const index = t.indexOf("=");
            if (index === -1) {
               return;
            }
            const target = t.slice(index + 1)?.trim();

            if (target) {
               targets.push(target);
            }
         });
      }

      return targets;

   }


   batchByStepId(stepId: string | undefined): BatchData | undefined {
      if (!stepId) {
         return undefined;
      }
      return this.batches.find(batch => {
         return batch.steps.find(step => step.id === stepId) ? true : false;
      });

   }

   getSteps(): StepData[] {
      return this.batches.map(b => b.steps).flat();
   }

   /**
    * Returns the first step that satisfies the predicate.
    * @param predicate The predicate is called for every step until it finds one where the predicate returns true.
    * @returns The first step that satisfies the predicate. If none do, findStep returns undefined.
    */
   private findStep(predicate: (s: StepData) => boolean): StepData | undefined {
      if(!this.jobId || !this.batches.length) return undefined;

      for (const batch of this.batches) {
         const step = batch.steps.find(predicate);
         if(step) return step;
      }
   }

   /**
    * Returns the first step with matching step id.
    * @param id The id of the step to get.
    * @returns The first step with matching step id. If none found, returns undefined.
    */
   stepById(id?: string): StepData | undefined {
      if(!id) return undefined;

      return this.findStep(s => s.id === id);
   }

   /**
    * Returns the furst step with matching log id.
    * @param logId The log id of the step to get.
    * @returns The first step with matching log id. If none found, returns undefined.
    */
   stepByLogId(logId: string): StepData | undefined {
      return this.findStep(s => s.logId === logId);
   }

   getStepName(stepId: string | undefined, includeRetry: boolean = true): string {

      if (!stepId) {
         return "";
      }

      const step = this.stepById(stepId);

      if (!step?.name) {
         return "";
      }

      const idx = this.getStepRetryNumber(stepId);
      if (!idx || !includeRetry) {
         return step.name;
      }

      return `${step.name} (${idx + 1})`;

   }



   stepRetries: Map<string, number> = new Map();

   getStepRetries(stepId: string): StepData[] {

      const step = this.stepById(stepId);
      if (!step) {
         return [];
      }


      let steps: StepData[] = [];

      this.batches.forEach(b => {
         b.steps.filter(s => s.name === step.name).forEach(s => {
            steps.push(s);
         });
      })

      return steps;

   }

   getStepRetryNumber(stepId: string): number {

      if (this.stepRetries.has(stepId)) {
         return this.stepRetries.get(stepId)!;
      }

      const steps = this.getStepRetries(stepId);

      if (!steps.length) {
         this.stepRetries.set(stepId, 0);
         return 0;
      }

      const step = this.stepById(stepId);

      if (!step) {
         this.stepRetries.set(stepId, 0);
         return 0;
      }

      const idx = steps.indexOf(step);

      const retry = idx === -1 ? 0 : idx;
      this.stepRetries.set(stepId, retry);

      return retry;

   }

   getRetryStepId(stepId: string): string | undefined {

      const steps = this.getStepRetries(stepId);

      if (steps.length <= 1) {
         return undefined;
      }

      const step = this.stepById(stepId);

      if (!step) {
         return undefined;
      }

      const idx = steps.indexOf(step);

      if (idx === -1 || idx === steps.length - 1) {
         return undefined;
      }

      return steps[idx + 1]?.id;

   }

   processGraph() {

      const jobData = this.jobData;

      if (!jobData) {
         return;
      }

      this.batches = jobData.batches ?? [];

      let labels = jobData.labels ?? [];
      this.labels = labels.map((label, index) => {
         return {
            default: false,
            stateResponse: label,
            timing: undefined
         }
      });

      const defaultLabel = jobData.defaultLabel;
      if (defaultLabel) {
         this.labels.push({
            stateResponse: defaultLabel,
            default: true,
            timing: undefined
         })
      }
   }

   labelByIndex(idxIn: number | string | undefined | null): JobLabel | undefined {

      if (idxIn === undefined || idxIn === null) {
         return undefined;
      }

      let idx = idxIn;

      if (typeof (idxIn) === "string") {
         idx = parseInt((idxIn ?? "").toString());
         if (isNaN(idx)) {
            return undefined;
         }
      }

      const labels = this.labels;

      if (!labels || (idx as number) >= labels.length) {
         return undefined;
      }

      return labels[idx as number];
   }

   labelIndex(name: string, category?: string): number {
      const label = this.findLabel(name, category);
      if (!label) {
         return -1;
      }
      return this.labels.indexOf(label);
   }

   findLabel(name: string, category?: string): JobLabel | undefined {

      const labels = this.labels.filter(label => label.stateResponse.state !== LabelState.Unspecified);

      const label = labels.find(label => {
         if (category) {
            return category === label.stateResponse.dashboardCategory && name === label.stateResponse.dashboardName;
         }
         return name === label.stateResponse.dashboardName;
      });

      return label;

   }


   stepByName(name: string): StepData | undefined {

      if (!this.jobId || !this.batches.length) {
         return undefined;
      }

      let step: StepData | undefined;

      this.batches.forEach(b => {

         if (step) {
            return;
         }

         step = b.steps.find(s => s.name === name);

      });

      return step;
   }

   stepPrice(stepId: string): number | undefined {

      const step = this.stepById(stepId);
      const batch = this.batchByStepId(stepId);

      if (!step || !batch || !batch.agentRate || !step.startTime || !step.finishTime) {
         return undefined;
      }

      const start = moment(step.startTime);
      const end = moment(step.finishTime);
      const hours = moment.duration(end.diff(start)).asHours();
      const price = hours * batch.agentRate;

      return price ? price : undefined;
   }

   jobPrice(): number | undefined {

      if (!this.batches || this.jobData?.state !== JobState.Complete) {
         return undefined;
      }

      let price = 0;

      this.batches.forEach(b => {

         if (b.agentRate && b.startTime && b.finishTime) {
            const start = moment(b.startTime);
            const end = moment(b.finishTime);
            const hours = moment.duration(end.diff(start)).asHours();
            price += hours * b.agentRate;

         }
      });

      return price ? price : undefined;
   }



   filterUpdated() {
      this.views.forEach(v => v.filterUpdated());
   }

   bisectionUpdated() {
      const v = this.views.find(v => v.name === "StepBisectionView") as any;
      if (v) {
         v.bisectionUpdated();
      }
   }


   getSummaryReport(stepId?: string): string | undefined {

      let reports: GetReportResponse[] = [];

      if (stepId) {

         const step = this.stepById(stepId);

         if (!step) {
            return undefined;
         }

         reports = step.reports?.filter(r => r.placement === ReportPlacement.Summary) ?? [];

      } else {

         reports = this.jobData?.reports?.filter(r => r.placement === ReportPlacement.Summary) ?? [];         
      }

      if (!reports.length) {
         return undefined;
      }      
      return reports.map(r => {

         if (r.content) {
            return `**${r.name}**:  \n` + `${r.content}  \n  \n`            
         }  else {
            return "";
         }

      }).filter(r => !!r).join("");
   }

   syncTiming() {
      if (this.timing) {
         this.getSteps().forEach(step => step.timing = this.timing!.steps[step.id] ? this.timing!.steps[step.id] : undefined);
      }
   }

   async poll() {

      try {

         if (!this.jobId || this.jobError) {
            return;
         }

         let requests: any[] = [];

         const initialRequest = !this.jobData;

         requests.push(backend.getJob(this.jobId, undefined, true));

         let results: any;

         await Promise.all(requests as any).then(r => results = r).catch(reason => {
            console.error(reason);
            this.jobError = reason;
         });

         if (!this.jobError && !results?.length) {
            this.jobError = "Not Found";
         }

         if (this.jobError) {
            errorBarStore.set({title: "Error Getting Job Data", message: this.jobError}, true);
            this.setRootUpdated();
            return;
         }

         const lastUpdateTime = this.jobData?.updateTime;

         const jobData = this.jobData = results[0] as JobData;

         if (!this.stream) {
            this.stream = projectStore.streamById(jobData.streamId);
         }

         if (!this.stream) {
            throw new Error(`Unable to get stream for job ${this.jobId}`)
         }

         let forceUpdate = false;
         if (initialRequest) {
            const templates = await TemplateCache.getStreamTemplates(this.stream);
            this.template = templates.find(t => t.id === jobData.templateId);
            if (!this.template) {
               throw new Error(`Error: Missing template ${this.stream?.fullname ?? jobData.streamId} - ${jobData.templateId}`)
            }
         } else {

            if (!this.timing && this.jobId) {
               forceUpdate = true;
               this.timing = await backend.getJobTiming(this.jobId);
               this.syncTiming();
            }
         }


         if (jobData.updateTime !== lastUpdateTime) {
            this.processGraph();
            this.syncTiming();
            this.setUpdated();
         }

         if (initialRequest) {
            this.filter.updateFilterItems();
            this.filterUpdated();

            // poll again for initial request, to save delay on 2nd pass
            clearTimeout(this.timeoutId);
            this.timeoutId = setTimeout(() => { this.update(); }, 500);

         }

         this.views.forEach(v => {
            v.tick();
         });

         this.views.forEach(v => {
            if (forceUpdate || (v.updateTime !== jobData.updateTime)) {
               v.updateTime = jobData.updateTime;
               v.detailsUpdated();
            }
         });
         
      } catch (reason) {
         console.error(reason);
         this.jobError = reason?.message ?? "Error loading job";
         this.setRootUpdated();
         
      } finally {

      }

   }

   get aborted(): boolean {

      return this.jobData?.abortedByUserInfo?.id ? true : false;
   }


   /** Subscribe to job data poll, use with caution as will force a component update upon polling
    *  In general, prefer subscribing to a finer gained data view or the filter changed observable
    */
   subscribe() {
      if (this.updated) { }
   }

   jobId?: string;
   jobError?: string;

   overview?: boolean;

   jobData?: JobData;
   stream?: StreamData;
   labels: JobLabel[] = [];
   batches: BatchData[] = [];
   template?: GetTemplateRefResponse;

   // stepId => artifacts
   stepArtifacts = new Map<string, GetArtifactResponse[]>();

   @observable
   private rootUpdated: number = 0;

   filter: JobDetailFilters;

   timing?: GetJobTimingResponse;

   removeDataView(view: JobDataView) {
      this.views = this.views.filter(v => v !== view);
   }

   getStepDependencies(stepId: string): StepData[] {

      const step = this.stepById(stepId);

      if (!step) {
         return [];
      }

      const steps: StepData[] = [];

      const getStepsRecursive = (stepId: string) => {

         const step = this.stepById(stepId);
         if (!step) {
            return;
         }

         [step.inputDependencies, step.optionalInputDependencies, step.orderDependencies].flat().forEach(id => {
            const s = this.stepById(id);
            if (s && !steps.find(s => s.id === id)) {
               steps.push(s);
               getStepsRecursive(s.id);
            }
         });
      };

      getStepsRecursive(stepId);

      steps.push(step);

      return steps;

   }

   getDataView<T extends JobDataView>(name: string): T {

      let view = this.views.find(v => v.name === name);
      if (view) {
         return view as T;
      }

      const func = JobDetailsV2.registeredViews.get(name);

      if (!func) {
         throw new Error(`Attempting to get unregistered view ${name}`);
      }

      view = func(this);
      view.name = name;

      console.log("Adding view ", name);

      this.views.push(view);

      return view as T;

   }

   static registerDataView(name: string, construct: (details: JobDetailsV2) => JobDataView) {
      this.registeredViews.set(name, construct);
   }

   views: JobDataView[] = [];

   private static registeredViews = new Map<string, (details: JobDetailsV2) => JobDataView>();

}

export type StateFilter = "All" | "Waiting" | "Ready" | "Skipped" | "Running" | "Completed" | "Aborted" | "Failure" | "Warnings";

// filter options for picker
type FilterPickerItem = ITag & {
   label?: GetLabelStateResponse;
   step?: StepData;
   searchItem?: boolean;
   searchInput?: string;
}


export class JobDetailFilters {

   constructor(details: JobDetailsV2) {
      makeObservable(this);
      this.details = details;
   }

   get label(): GetLabelStateResponse | undefined {
      return this._selected?.label;
   }

   get step(): StepData | undefined {
      return this._selected?.step;
   }

   get search(): string | undefined {
      return this._search;
   }

   get query(): string {
      return "";
   }

   get selected(): FilterPickerItem | undefined {
      return this._selected;
   }


   changed() {
      this.details.filterUpdated();
   }

   get filterItems(): FilterPickerItem[] {
      return this._filterItems;
   }

   get filterStates(): StateFilter[] {
      if (this._filterStates) {
         return this._filterStates;
      }
      return ["All"];
   }

   @observable
   searchInputUpdated = 0

   @action
   private _searchInputUpdated() {
      this.searchInputUpdated++;
   }


   @observable
   inputChanged = 0


   currentInput?: string;

   @action
   onInputChanged(input?: string) {
      this.currentInput = input;
      this.inputChanged++;
   }

   setSearchInput(search?: string) {

      const searchItem = this.filterItems.find(item => item.searchItem);
      if (!searchItem) {
         return;
      }

      let dirty = false;

      if (searchItem.searchInput !== search) {
         searchItem.searchInput = search;
         dirty = true;
      }

      if (dirty) {
         this._searchInputUpdated();
      }

   }

   clear() {
      //console.log("Clear selected")
      this.setSelected(undefined, undefined, undefined);
   }

   setSelected(step: StepData | undefined, label: GetLabelStateResponse | undefined, search?: string) {
      /*
      console.log("setSelected Filter", step, label, search, this.filterItems.length);
      console.log(console.trace(""));
      */

      this.currentInput = undefined;

      const selected = this.filterItems.find(item => {

         if (step && item.step?.id === step.id) {
            return true;
         }

         if (label && item.label?.dashboardCategory === label.dashboardCategory && item.label?.dashboardName === label.dashboardName) {
            return true;
         }

         if (search) {
            if (item.searchItem) {
               item.searchInput = search;
               item.name = `Filter: ${search}`;
               return true;
            }
         }

         return false;

      });

      this._search = search;

      if (this._selected !== selected) {
         this._selected = selected;
         this.changed();
      }
   }

   updateFilterItems() {

      if (this.filterItems.length) {
         console.error("Filter items are already added");
         return;
      }

      const details = this.details;

      const labels = details.labels.filter(label => label.stateResponse.state !== LabelState.Unspecified);

      type LabelItem = {
         category: string;
         labels: JobLabel[];
      };

      const categories: Set<string> = new Set();
      labels.forEach(label => { if (label.stateResponse.dashboardName) { categories.add(label.stateResponse.dashboardCategory!); } });


      let labelItems = Array.from(categories.values()).map(c => {
         return {
            category: c,
            labels: labels.filter(label => label.stateResponse.dashboardName && (label.stateResponse.dashboardCategory === c)).sort((a, b) => {
               return a.stateResponse.dashboardName! < b.stateResponse.dashboardName! ? -1 : 1;
            })
         } as LabelItem;
      }).filter(item => item.labels?.length).sort((a, b) => {
         return a.category < b.category ? -1 : 1;
      });


      const filterItems: FilterPickerItem[] = [];

      // generic search
      filterItems.push({
         name: `Filter: Default`,
         searchItem: true,
         key: `filter_item_search`
      })

      // step items
      details.getSteps().sort((a, b) => {
         return details.getStepName(a.id).localeCompare(details.getStepName(b.id));
      }).forEach(step => {
         const name = details.getStepName(step.id);
         if (!name) {
            return;
         }
         filterItems.push({
            key: `label_item_steo_${name}`,
            name: `${name}`,
            step: step
         })
      });

      const labelDupes = new Map<string, number>();

      labelItems.forEach(labelItem => {
         labelItem.labels.forEach(label => {
            const count = labelDupes.get(label.stateResponse.dashboardName ?? "");
            if (!count) {
               labelDupes.set(label.stateResponse.dashboardName ?? "", 1);
            } else {
               labelDupes.set(label.stateResponse.dashboardName ?? "", count + 1);
            }
         });
      });


      // label items
      labelItems.forEach(labelItem => {
         labelItem.labels.forEach(label => {

            const count = labelDupes.get(label.stateResponse.dashboardName ?? "");
            let name = `Label: ${label.stateResponse.dashboardName ?? ""}`;
            if (count! > 1) {
               name = `Label: ${labelItem.category} - ${label.stateResponse.dashboardName ?? ""}`
            }

            filterItems.push({
               key: `label_item_${labelItem.category}_${label.stateResponse.dashboardName ?? ""}`,
               name: name,
               label: label.stateResponse
            })
         });
      });


      // @todo: check if dirty, though right now we're only calling this on initial job response being available and shouldn't mutate
      this._filterItems = filterItems;
      this.changed();

   }

   setFilterStates(filters: StateFilter[] | undefined) {
      // @todo: possibly compare if slow
      this._filterStates = filters;
      this.changed();
   }

   private _selected?: FilterPickerItem
   private _search?: string;


   private _filterStates?: StateFilter[];
   private _filterItems: FilterPickerItem[] = [];

   details: JobDetailsV2;

}

export const JobFilterBar: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const location = useLocation();
   const query = useQuery();
   const filterPicker = React.useRef(null);
   const navigate = useNavigate();
   const [state, setState] = useState<{}>({});
   const { modeColors } = getHordeStyling();

   const jobFilter = jobDetails.filter;
   const stateFilter = jobFilter.filterStates;

   // subscribe
   if (jobFilter.searchInputUpdated) { }

   // filter picker
   const filterItems = jobFilter.filterItems;

   if (!filterItems.length) {
      jobDetails.subscribe();
      return null;
   }

   const dropDownStyle: any = () => {
      return {
         dropdown: {},
         callout: {
            selectors: {
               ".ms-Callout-main": {
                  padding: "4px 4px 12px 12px",
                  overflow: "hidden"
               }
            }
         },
         dropdownItemHeader: { fontSize: 12, color: "#FFFFFF" },
         dropdownOptionText: { fontSize: 12 },
         dropdownItem: {
            minHeight: 28, lineHeight: 28
         },
         dropdownItemSelected: {
            minHeight: 28, lineHeight: 28, backgroundColor: "inherit"
         }
      }
   }

   interface StateItem extends IDropdownOption {
      state: StateFilter
   }

   const stateItems: StateItem[] = ["All", "Failure", "Warnings", "Skipped", "Running", "Completed", "Aborted", "Waiting", "Ready"].map(state => {
      return {
         key: state,
         text: state,
         state: state as StateFilter
      };
   });

   stateItems.forEach(item => {
      if (item.state === "Aborted") {
         item.text = "Canceled";
      }
      else if (item.state === "Failure") {
         item.text = "Errors";
      }

   });

   // steps
   /*
   const step = selected?.step;
   const stepId = query.get("step") ? query.get("step")! : undefined

   if (stepId) {
      if (step?.id !== stepId) {
         const nstep = jobDetails.stepById(stepId);
         if (nstep) {
            jobFilter.setSelected(nstep, undefined);
         }
      }
   } else {

      if (step) {
         jobFilter.setSelected(undefined, undefined);
      }
   }
   */

   // labels
   const label = jobFilter.selected?.label;
   const labelIdx = query.get("label") ? query.get("label")! : undefined


   if (labelIdx) {
      const idx = parseInt(labelIdx);
      if (!isNaN(idx)) {
         const nlabel = jobDetails.labelByIndex(labelIdx);
         if (nlabel) {
            if (!label || (nlabel?.stateResponse.dashboardCategory !== label.dashboardCategory || nlabel?.stateResponse.dashboardName !== label.dashboardName)) {
               jobFilter.setSelected(undefined, nlabel.stateResponse);
            }
         }
      }

   } else {
      if (label) {
         jobFilter.setSelected(undefined, undefined);
      }
   }

   const selected = jobFilter.selected;


   return <Stack horizontal tokens={{ childrenGap: 16 }}>
      <Stack style={{ width: 320 }}>
         <TagPicker
            itemLimit={1}
            inputProps={{ placeholder: "Filter" }}
            /*pickerCalloutProps={{setInitialFocus: false}}*/

            componentRef={filterPicker}

            onInputChange={(input) => {
               let ninput = (input ? input : undefined);
               jobFilter.setSearchInput(ninput);
               jobFilter.onInputChanged(ninput);
               return input;
            }}

            onBlur={(ev) => {
               if (ev.target.value && ev.target.value.trim()) {
                  // buildParams.addTarget(ev.target.value.trim())
               }

               // This is using undocumented behavior to clear the input when you lose focus
               // It could very well break
               if (filterPicker.current) {
                  try {
                     if (jobFilter.currentInput) {
                        console.log("selecting current input", jobFilter.currentInput);
                        jobFilter.setSelected(undefined, undefined, jobFilter.currentInput)
                     }
                     //console.log((filterPicker.current as any).input.current);
                     (filterPicker.current as any).input.current._updateValue("");
                  } catch (reason) {
                     console.error("There was an error adding filter to list and clearing the input in process\n" + reason);
                  }
               }
            }}

            resolveDelay={undefined}

            onResolveSuggestions={(filter, selected) => {
               return filterItems.filter(item => {
                  filter = filter.toLowerCase();
                  if (item.searchItem && item.searchInput) {
                     return true;
                  }
                  const name = item.name.toLowerCase();
                  return name.indexOf(filter) !== -1;
               });
            }}

            onEmptyResolveSuggestions={(selected) => {
               return filterItems.filter(item => {
                  if (item.searchItem && !item.searchInput) {
                     return false;
                  }
                  return true;
               });
            }}

            onRenderItem={(props: IPickerItemProps<ITag>) => {
               return <Stack style={{ paddingLeft: 8 }} key={`picker_item_${props.item.name}`}>
                  <PrimaryButton
                     iconProps={{ iconName: "Cancel", styles: { root: { fontSize: 12, margin: 0, padding: 0 } } }}
                     style={{ padding: 2, paddingLeft: 8, paddingRight: 8, fontFamily: "Horde Open Sans SemiBold", fontSize: 12, height: "unset" }}
                     text={props.item.name}
                     onClick={() => {
                        navigate(location.pathname);
                        jobFilter.setSelected(undefined, undefined, undefined)
                     }} />
               </Stack>;
            }}

            onRenderSuggestionsItem={(props: ITag, itemProps: ISuggestionItemProps<ITag>) => {
               const item = props as FilterPickerItem;

               let inner: JSX.Element | undefined;
               if (item.label) {
                  inner = <Stack horizontal verticalAlign="center" verticalFill={true}>
                     <LabelStatusIcon label={item.label} />
                     <Text>{item.name}</Text>
                  </Stack>
               }
               if (item.step) {
                  inner = <Stack horizontal verticalAlign="center" verticalFill={true}>
                     <StepStatusIcon step={item.step} />
                     <Text>{item.name}</Text>
                  </Stack>
               }

               if (item.searchItem) {

                  if (!item.searchInput) {
                     return <div />
                  }


                  inner = <Stack horizontal verticalAlign="center" verticalFill={true}>
                     <FontIcon style={{ color: modeColors.text, fontSize: 13, paddingTop: 4, paddingRight: 8 }} iconName={"Filter"} />
                     <Text>{`Filter: ${item.searchInput}`}</Text>
                  </Stack>
               }

               if (!inner) {
                  inner = <Text>{item.name}</Text>
               }

               return <Stack style={{ margin: 6, marginTop: 5, marginBottom: 5 }}>{inner}</Stack>;
            }}

            onRemoveSuggestion={(item) => { console.log(item) }}

            /*
            createGenericItem={(input: string, ValidationState: ValidationState) => {

               const filterItem = filterItems.find(item => item.searchItem)!;
               filterItem.searchInput = input ? input : undefined;
               jobFilter.setSelected(undefined, undefined, filterItem.searchInput);
               return filterItem;

         }}
         */
            /*
            removeButtonIconProps={{ iconName: "Cancel", onClick: () => { jobFilter.setSelected(undefined, undefined, undefined) } }}
            */

            //onChange={(items) => { console.log(items) }}

            onValidateInput={(input?: string) => {
               const filterItem = filterItems.find(item => item.searchItem)!;
               filterItem.searchInput = input ? input : undefined;
               jobFilter.setSelected(undefined, undefined, filterItem.searchInput);
               return input ? ValidationState.valid : ValidationState.invalid
            }}

            selectedItems={selected ? [selected] : []}

            onItemSelected={(item) => {

               if (!item || !item.name?.trim()) {
                  return null;
               }

               const step = (item as FilterPickerItem).step;
               if (step) {
                  navigate(location.pathname + `?step=${step.id}`);
                  return null;
               }

               const label = (item as FilterPickerItem).label;
               if (label) {
                  const idx = jobDetails.labelIndex(label.dashboardName ?? "", label.dashboardCategory ?? "");
                  if (idx >= 0) {
                     navigate(location.pathname + `?label=${idx}`);
                  } else {
                     navigate(location.pathname, { replace: true });
                  }
               }

               jobFilter.setSelected((item as FilterPickerItem).step, (item as FilterPickerItem).label, (item as FilterPickerItem).searchInput);
               return null;
            }}
         />
      </Stack>
      <Stack >
         <Dropdown style={{ width: 140 }} styles={dropDownStyle} options={stateItems} multiSelect selectedKeys={stateFilter}
            onChange={(event, option, index) => {

               if (option) {

                  let filter = [...stateFilter];
                  if (!option.selected) {
                     filter = filter.filter(k => k !== option.key);
                  } else {
                     if (filter.indexOf(option.key as StateFilter) === -1) {
                        filter.push(option.key as StateFilter);
                     }
                  }

                  if (!filter.length || (option.selected && option.key === "All")) {
                     filter = ["All"];
                  }

                  if (filter.find(k => k === "All") && filter.length > 1) {
                     filter = filter.filter(k => k !== "All");
                  }
                  jobFilter.setFilterStates(filter);
                  // redraw
                  setState({ ...state });
               }
            }}
         />
      </Stack>
   </Stack>
});
