// Copyright Epic Games, Inc. All Rights Reserved.

import { CommandBar, CommandBarButton, ICommandBarItemProps, IContextualMenuItem, Stack } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useState } from 'react';
import { Link, useNavigate } from 'react-router-dom';
import { ArtifactContextType, JobState, JobStepOutcome, JobStepState, StepData } from '../../backend/Api';
import { getSiteConfig } from '../../backend/Config';
import dashboard from '../../backend/Dashboard';
import { getHordeStyling } from '../../styles/Styles';
import { EditJobModal } from '../EditJobModal';
import { useQuery } from "horde/base/utilities/hooks";
import { NewBuild } from '../NewBuild';
import { NotificationDropdown } from '../NotificationDropdown';
import { PauseStepModal } from '../StepPauseModal';
import { JobArtifactsModal } from '../artifacts/ArtifactsModal';
import { BisectionCreateModal } from '../bisection/CreateModal';
import { NewBuildV2 } from '../build/NewBuildV2';
import { AbortJobModal } from './AbortJobModal';
import { JobDetailsV2 } from './JobDetailsViewCommon';
import { RetryStepsModal, StepRetryModal, StepRetryType } from './StepRetryModal';
import { JobSearchSimpleModal } from '../JobSearchSimple';

enum ParameterState {
   Hidden,
   Parameters,
   Clone
}

export const JobOperations: React.FC<{ jobDetails: JobDetailsV2 }> = observer(({ jobDetails }) => {

   const query = useQuery();
   const navigate = useNavigate();
   const [abortShown, setAbortShown] = useState(false);
   const [editShown, setEditShown] = useState(false);
   const [retryStepsShown, setRetryStepsShown] = useState(false);

   const [findJobsShown, setFindJobsShown] = useState(query.has("search"));

   const [parametersState, setParametersState] = useState(query.get("newbuild") ? ParameterState.Clone : ParameterState.Hidden);

   const { hordeClasses } = getHordeStyling();

   const stepId = query.get("step") ? query.get("step")! : undefined;
   const batchFilter = query.get("batch");
   let newBuildVersion: string = query.get("newbuildversion") ? query.get("newbuildversion")! : "1";

   newBuildVersion = "2";

   // subscribe
   if (dashboard.updated) { }
   if (jobDetails.updated) { }

   const jobId = jobDetails.jobId;
   const jobData = jobDetails.jobData;

   if (!jobId || !jobData) {
      return null;
   }

   const abortDisabled = jobData.state === JobState.Complete || !jobDetails.template?.canRun;

   const seenSteps = new Set<string>();
   const runningSteps = new Set<string>();

   let failedSteps = jobDetails.getSteps().reverse().filter(s => {

      if (!s.allowRetry) {
         return false;
      }

      if (seenSteps.has(s.name) || runningSteps.has(s.name)) {
         return false;
      }

      seenSteps.add(s.name);

      if (s.state === JobStepState.Running || s.state === JobStepState.Waiting || s.state === JobStepState.Ready) { 
         runningSteps.add(s.name);
         return false;
      }
      
      if (s.state === JobStepState.Completed && (s.outcome === JobStepOutcome.Success || s.outcome === JobStepOutcome.Warnings))
      {
         return false;
      }

      return true;

   }).reverse();

   const allSteps = jobDetails.getSteps();
   failedSteps = failedSteps.sort((a, b) => {

      const idxA = allSteps.findIndex((c) => a.name === c.name);
      const idxB = allSteps.findIndex((c) => b.name === c.name);

      return idxA - idxB;
   })

   
   const canRun = jobDetails.template?.canRun ? true : false;
   const retryFailedStepsDisabled = !failedSteps.length || !canRun;

   const pinned = dashboard.jobPinned(jobId);

   const opsList: IContextualMenuItem[] = [];

   opsList.push({
      key: 'jobops_pin',
      text: pinned ? "Unpin" : "Pin",
      iconProps: { iconName: "Pin" },
      onClick: () => { pinned ? dashboard.unpinJob(jobId) : dashboard.pinJob(jobId); }
   });

   opsList.push({
      key: 'jobops_parameters',
      text: "Parameters",
      iconProps: { iconName: "Settings" },
      onClick: () => { setParametersState(ParameterState.Parameters); }
   });

   if (jobData.change) {
      opsList.push({
         key: 'jobops_relatedjobs',
         text: `Related Jobs`,
         iconProps: { iconName: "Search" },
         onClick: () => { setFindJobsShown(true) }
      });      
   }
   opsList.push({
      key: 'jobops_edit',
      text: "Edit...",
      disabled: !jobData.canUpdate,
      iconProps: { iconName: "Edit" },
      onClick: () => { setEditShown(true); }
   });

   //if (getSiteConfig().environment !== "production") {

   opsList.push({
      key: 'jobops_runfailedsteps',
      text: "Retry Steps",
      disabled: retryFailedStepsDisabled,
      iconProps: { iconName: "Repeat" },
      onClick: () => { setRetryStepsShown(true); }
   });
   //}

   opsList.push({
      key: 'jobops_abort',
      text: "Cancel Job...",
      disabled: abortDisabled,
      iconProps: { iconName: "Delete" },
      onClick: () => { setAbortShown(true); }
   });

   opsList.push({
      key: 'jobops_runagain',
      text: "Run Again...",
      iconProps: { iconName: "Duplicate" },
      disabled: !canRun,
      onClick: () => { setParametersState(ParameterState.Clone); }
   });

   opsList.push({
      key: 'jobops_audit',
      text: "Audit",
      iconProps: { iconName: "SearchTemplate" },
      onClick: () => { 
         const href = `/audit/job/${jobId}`;
         window.open(href, "_blank")
       }
   });

   const opsItems: ICommandBarItemProps[] = [
      {
         key: 'jobops_items',
         text: "Job",
         iconProps: { iconName: "Properties" },
         subMenuProps: {
            items: opsList
         }
      }
   ];

   let step: StepData | undefined;
   let viewLogDisabled = true;
   let logUrl = "";
   if (stepId) {
      step = jobDetails.stepById(stepId)
      if (step) {
         viewLogDisabled = !step.logId;
         logUrl = `/log/${step!.logId}`;
      }
   }

   const notifications = <NotificationDropdown jobDetails={jobDetails} />

   if (!jobDetails.stream) {
      return null;
   }

   let showNewBuildV1 = newBuildVersion !== "2" && (parametersState === ParameterState.Parameters || parametersState === ParameterState.Clone);
   let showNewBuildV2 = newBuildVersion === "2" && (parametersState === ParameterState.Parameters || parametersState === ParameterState.Clone);

   return <Stack>
      {retryStepsShown && <RetryStepsModal stepIds={failedSteps.map(s => s.id)} jobDetails={jobDetails} onClose={() => { setRetryStepsShown(false); }} />}
      <AbortJobModal jobDetails={jobDetails} show={abortShown} onClose={() => { setAbortShown(false); }} />
      <EditJobModal jobData={jobDetails.jobData} show={editShown} onClose={() => { setEditShown(false); }} />
      {showNewBuildV1 && <NewBuild streamId={jobDetails.stream!.id} jobDetails={jobDetails} readOnly={parametersState === ParameterState.Parameters}
         show={true}
         onClose={(newJobId) => {
            setParametersState(ParameterState.Hidden);
            if (newJobId) {
               navigate(`/job/${newJobId}`);
            } else {
               if (query.get("newbuild")) {
                  navigate(`/job/${jobId}`, { replace: true });
               }
            }
         }} />}

      {showNewBuildV2 && <NewBuildV2 streamId={jobDetails.stream!.id} jobDetails={jobDetails} readOnly={parametersState === ParameterState.Parameters}
         show={true}
         onClose={(newJobId) => {
            setParametersState(ParameterState.Hidden);
            if (newJobId) {
               navigate(`/job/${newJobId}`);
            } else {
               if (query.get("newbuild")) {
                  navigate(`/job/${jobId}`, { replace: true });
               }
            }
         }} />}

      <Stack horizontal>
         {findJobsShown && <JobSearchSimpleModal changeListIn={jobData.change?.toString()} onClose={() => { setFindJobsShown(false) }} streamIdIn={jobData.streamId} autoloadIn={true} />}
         <Stack grow />
         <Stack horizontal tokens={{ childrenGap: 8 }}>

            {(!!stepId || !!batchFilter) && <Link to={`/job/${jobId}`}> <Stack styles={{ root: { paddingTop: 4 } }}>
               <CommandBarButton className={hordeClasses.commandBarSmall} styles={{ root: { padding: "10px 8px 10px 8px" } }} iconProps={{ iconName: "Dashboard" }} text="Job Overview" />
            </Stack>
            </Link>}

            {!!stepId && <Stack styles={{ root: { paddingTop: 4 } }}>
               <StepArtifactsOperations jobDetails={jobDetails} stepId={stepId} />
            </Stack>}

            <Stack className={hordeClasses.commandBarSmall} styles={{ root: { paddingTop: 4 } }}>
               {notifications}
            </Stack>

            <Stack className={hordeClasses.commandBarSmall} styles={{ root: { paddingTop: 4 } }}>
               <CommandBar
                  items={opsItems}
                  onReduceData={() => undefined} />
            </Stack>

            {!!stepId && <Stack styles={{ root: { paddingTop: 4 } }}>
               <StepOperations jobDetails={jobDetails} stepId={stepId} />
            </Stack>}

            {!!step && !viewLogDisabled && <Link to={`${viewLogDisabled ? "" : logUrl}`}>
               <Stack styles={{ root: { paddingTop: 4 } }}>
                  <CommandBarButton disabled={viewLogDisabled} className={hordeClasses.commandBarSmall} styles={{ root: { padding: "10px 8px 10px 8px" } }} iconProps={{ iconName: "AlignLeft" }} text="View Log" />
               </Stack>
            </Link>}

         </Stack>
      </Stack>

   </Stack>

});

const StepArtifactsOperations: React.FC<{ jobDetails: JobDetailsV2, stepId: string }> = observer(({ jobDetails, stepId }) => {

   const { hordeClasses } = getHordeStyling();

   const navigate = useNavigate();

   const query = useQuery();
   let artifactContext = !!query.get("artifactContext") ? query.get("artifactContext")! as ArtifactContextType : undefined;
   const artifactPath = !!query.get("artifactPath") ? query.get("artifactPath")! : undefined;
   const artifactId = !!query.get("artifactId") ? query.get("artifactId")! : undefined;

   // subscribe
   if (dashboard.updated) { }
   if (jobDetails.updated) { }

   const stepArtifacts = jobDetails.stepArtifacts.get(stepId ?? "");

   const step = jobDetails.stepById(stepId)
   if (!step) {
      return null;
   }

   const atypes = new Map<ArtifactContextType, number>();
   const knownTypes = new Set<string>(["step-saved", "step-output", "step-trace"]);

   stepArtifacts?.forEach(a => {
      let c = atypes.get(a.type) ?? 0;
      c++;
      atypes.set(a.type, c);
   });

   let artifact = stepArtifacts?.find(a => a.id === artifactId);
   if (!artifactContext && artifact) {
      artifactContext = artifact.type;
   }
   
   const opsList: IContextualMenuItem[] = [];

   const baseUrl = window.location.pathname + window.location.search;


   opsList.push({
      key: 'stepops_artifacts_step',
      text: "Logs",
      iconProps: { iconName: "Folder" },
      disabled: !atypes.get("step-saved"),
      onClick: () => { navigate(`${baseUrl}&artifactContext=step-saved`, { replace: true }) }
   });

   opsList.push({
      key: 'stepops_artifacts_output',
      text: "Temp Storage",
      iconProps: { iconName: "MenuOpen" },
      disabled: !atypes.get("step-output"),
      onClick: () => { navigate(`${baseUrl}&artifactContext=step-output`, { replace: true }) }
   });

   opsList.push({
      key: 'stepops_artifacts_trace',
      text: "Traces",
      iconProps: { iconName: "SearchTemplate" },
      disabled: !atypes.get("step-trace"),
      onClick: () => { navigate(`${baseUrl}&artifactContext=step-trace`, { replace: true }) }
   });

   const custom = stepArtifacts?.filter(a => !knownTypes.has(a.type)).sort((a, b) => a.type.localeCompare(b.type));
   custom?.forEach(c => {
      opsList.push({
         key: `stepops_artifacts_${c.type}`,
         text: c.description ?? c.name,
         iconProps: { iconName: "Clean" },
         onClick: () => { navigate(`${baseUrl}&artifactContext=${c.type}`, { replace: true }) }
      });
   })

   const opsItems: ICommandBarItemProps[] = [
      {
         key: 'stepops_artifacts',
         text: "Artifacts",
         iconProps: { iconName: "CloudDownload" },
         disabled: !stepArtifacts?.length,
         subMenuProps: {
            items: opsList
         }
      }
   ];

   return <Stack>
      {!!artifactContext && <JobArtifactsModal stepId={step.id} jobId={jobDetails.jobId!} contextType={artifactContext} artifactPath={artifactPath} artifacts={stepArtifacts} onClose={() => { navigate(window.location.pathname + `?step=${stepId}`, { replace: true }) }} />}
      <Stack horizontal styles={{ root: { paddingLeft: 0 } }}>
         <Stack grow />
         <Stack horizontal>
            <Stack className={hordeClasses.commandBarSmall} style={{ paddingRight: 0 }}>
               <CommandBar
                  items={opsItems}
                  onReduceData={() => undefined} />
            </Stack>
         </Stack>
      </Stack>
   </Stack>


});

const StepOperations: React.FC<{ jobDetails: JobDetailsV2, stepId: string }> = observer(({ jobDetails, stepId }) => {

   const [shown, setShown] = useState<{ abortShown?: boolean, retryShown?: boolean, pauseShown?: boolean, bisectShown?: boolean }>({});
   const [runType, setRunType] = useState(StepRetryType.RunAgain);

   const { hordeClasses } = getHordeStyling();

   // subscribe
   if (dashboard.updated) { }
   if (jobDetails.updated) { }

   const jobId = jobDetails.jobId;
   const jobData = jobDetails.jobData;

   if (!jobId || !jobData) {
      return null;
   }

   const step = jobDetails.stepById(stepId);
   if (!step) {
      return null;
   }

   const stepOutputInfo = dashboard.getArtifactTypeInfo("step-output")

   let tempStorageExpired = false;
   if (stepOutputInfo && stepOutputInfo.keepDays && step.startTime) {

      const expireMs = stepOutputInfo.keepDays * 86400000;
      if ((Date.now() - new Date(step.startTime).getTime()) > expireMs)
      {         
         tempStorageExpired = true;
      }      
   }

   const canRunDisabled = !step?.allowRetry || !!step.retriedByUserInfo || !!jobData.abortedByUserInfo || !jobDetails.template?.canRun || tempStorageExpired;
   const canTryFix = jobDetails.template?.allowPreflights && !!jobDetails.template?.canRun;
   // Bisection is currently disabled via feature flag
   const canBisect = false;

   const opsList: IContextualMenuItem[] = [];

   opsList.push({
      key: 'stepops_runagain',
      text: "Run Again",
      iconProps: { iconName: "Redo" },
      disabled: canRunDisabled,
      onClick: () => { setShown({ retryShown: true }); setRunType(StepRetryType.RunAgain); }
   });


   opsList.push({
      key: 'stepops_abort',
      text: "Cancel Step...",
      iconProps: { iconName: "Cross" },
      disabled: !!step.finishTime || !jobData.canUpdate,
      onClick: () => { setShown({ abortShown: true }); }
   });

   opsList.push({
      key: 'stepops_preflight',
      text: "Preflight",
      iconProps: { iconName: "Locate" },
      disabled: !canTryFix,
      onClick: () => { setShown({ retryShown: true }); setRunType(StepRetryType.TestFix); }
   });

   opsList.push({
      key: 'stepops_bisect',
      text: "Bisect",
      iconProps: { iconName: "FlowReview" },
      disabled: !canBisect,
      onClick: () => { setShown({ bisectShown: true }); }
   });

   opsList.push({
      key: 'stepops_buildlocally',
      text: "Build Locally",
      iconProps: { iconName: "Build" },
      disabled: !canTryFix,
      href: buildUrl()
   });

   /*
   if (dashboard.hordeAdmin && jobDetails.template && node) {

      opsList.push({
         key: 'jobops_pause',
         text: "Pause",
         iconProps: { iconName: "Pause" },
         onClick: () => { setShown({ pauseShown: true }); }
      });
   }*/

   const opsItems: ICommandBarItemProps[] = [
      {
         key: 'stepops_items',
         text: "Step",
         iconProps: { iconName: "MenuOpen" },
         subMenuProps: {
            items: opsList
         }
      }
   ];

   function buildUrl(): string {

      // @todo: we should be able to get stream from details, even if deleted from Horde
      if (!jobDetails.stream?.project) {
         return "";
      }

      const stream = `//${jobDetails.stream!.project!.name}/${jobDetails.stream!.name}`;
      const changelist = jobData!.change;

      let args = "RunUAT.bat BuildGraph ";

      args += `-Target="${step!.name}" `;

      args += jobData!.arguments!.map(a => {

         if (a.indexOf("Setup Build") !== -1) {
            return "";
         }

         if (a.toLowerCase().indexOf("target=") !== -1) {
            return "";
         }

         return a.indexOf(" ") === -1 ? a : `"${a}"`;
      }
      ).join(" ");

      return `ugs://execute?stream=${encodeURIComponent(stream)}&changelist=${encodeURIComponent(changelist ?? "")}&command=${encodeURIComponent(args)}`;
   }

   return <Stack>
      {!!shown.pauseShown && <PauseStepModal streamId={jobDetails.stream!.id} stepName={step!.name} templateName={jobDetails.template!.name} onClose={() => setShown({ pauseShown: false })} />}
      {!!shown.retryShown && <StepRetryModal stepId={stepId} jobDetails={jobDetails} type={runType} show={true} onClose={() => { setShown({}); }} />}
      {!!shown.abortShown && <AbortJobModal stepId={stepId} jobDetails={jobDetails} show={true} onClose={() => { setShown({}); }} />}
      {!!shown.bisectShown && <BisectionCreateModal jobId={jobId} nodeName={step?.name ?? "Unknown Node"} onClose={(response) => {
         if (response) {
            jobDetails.bisectionUpdated();
         }

         setShown({});
      }} />}
      <Stack horizontal styles={{ root: { paddingLeft: 0 } }}>
         <Stack grow />
         <Stack horizontal>
            <Stack className={hordeClasses.commandBarSmall} style={{ paddingRight: 0 }}>
               <CommandBar
                  items={opsItems}
                  onReduceData={() => undefined} />
            </Stack>
         </Stack>
      </Stack>
   </Stack>

});