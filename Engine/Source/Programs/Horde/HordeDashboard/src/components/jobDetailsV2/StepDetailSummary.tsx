// Copyright Epic Games, Inc. All Rights Reserved.

import { FontIcon, Label, Stack, Text } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import React, { useEffect } from 'react';
import { GetReportResponse, JobStepError, JobStepOutcome, JobStepState } from '../../backend/Api';
import { Markdown } from '../../base/components/Markdown';
import { ISideRailLink } from '../../base/components/SideRail';
import { getNiceTime, getStepElapsed, getStepETA, getStepFinishTime, getStepTimingDelta } from '../../base/utilities/timeUtils';
import { getHordeStyling } from '../../styles/Styles';
import { getBatchText } from '../JobDetailCommon';
import { JobDataView, JobDetailsV2 } from './JobDetailsViewCommon';
import dashboard, { StatusColor } from 'horde/backend/Dashboard';
import { getStepMessageInfo } from './JobDetailsUtils';

const sideRail: ISideRailLink = { text: "Summary", url: "rail_step_summary" };

class StepSummaryView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   set() {

   }

   clear() {
      super.clear();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();

   }

   order = 0;

}

JobDetailsV2.registerDataView("StepSummaryView", (details: JobDetailsV2) => new StepSummaryView(details));


const getStepSummaryMarkdown = (jobDetails: JobDetailsV2, stepId: string): string => {

   const jobData = jobDetails.jobData;

   const step = jobDetails.stepById(stepId)!;
   const batch = jobDetails.batchByStepId(stepId);

   if (!step || !batch || !jobData) {
      return "";
   }

   const duration = getStepElapsed(step);
   let eta = getStepETA(step, jobData);

   const text: string[] = [];

   if (jobData) {
      text.push(`Job created by ${jobData.startedByUserInfo ? jobData.startedByUserInfo.name : "scheduler"}`);
   }

   let batchIncluded = false;
   const batchText = () => {

      if (!batch) {
         return undefined;
      }

      batchIncluded = true;

      const agentType = batch?.agentType;
      const agentPool = jobDetails.stream?.agentTypes[agentType!]?.pool;
      return getBatchText({ batch: batch, agentType: agentType, agentPool: agentPool });

   };

   const retries = jobDetails.getStepRetries(step.id);
   const idx = retries.findIndex(s => s.id === step.id);
   if (idx > 0) {

      const pstep = retries[idx - 1];

      let msg = `This is a retry of a [previous step](/job/${jobDetails.jobId!}?step=${pstep.id})`;
      if (pstep.retriedByUserInfo) {
         msg += ` started by ${pstep.retriedByUserInfo?.name}`;
      }

      text.push(msg);

   }

   if (step.retriedByUserInfo) {
      const retryId = jobDetails.getRetryStepId(step.id);
      if (retryId) {
         text.push(`[Step was retried by ${step.retriedByUserInfo.name}](/job/${jobDetails.jobId!}?step=${retryId})`);
      } else {
         text.push(`Step was retried by ${step.retriedByUserInfo.name}`);
      }
   }

   if (step.abortRequested || step.state === JobStepState.Aborted) {
      eta.display = eta.server = "";
      let aborted = "";
      if (step.abortedByUserInfo) {
         aborted = "This step was canceled";
         aborted += ` by ${step.abortedByUserInfo.name}.`;
      } else if (jobData?.abortedByUserInfo) {
         aborted = "The job was canceled";
         aborted += ` by ${jobData?.abortedByUserInfo.name}.`;
      } else {
         aborted = "This step was canceled by Horde";

         if (step.error === JobStepError.TimedOut) {
            aborted = "The step was canceled due to reaching the maximum run time limit";
         }
      }
      text.push(aborted);
   } else if (step.state === JobStepState.Skipped) {
      eta.display = eta.server = "";
      text.push("The step was skipped");
   } else if (step.state === JobStepState.Ready || step.state === JobStepState.Waiting) {

      text.push(batchText() ?? `The step is pending in ${step.state} state`);
   }

   if (batch?.agentId) {

      if (step.startTime) {
         const str = getNiceTime(step.startTime);
         text.push(`Step started on ${str}`);
      }

      let runningText = `${step.finishTime ? "Ran" : "Running"} on [${batch.agentId}](?step=${stepId}&agentId=${encodeURIComponent(batch.agentId)})`;

      if (duration) {
         runningText += ` for ${duration.trim()}`;
      }

      if (step.finishTime && (step.outcome === JobStepOutcome.Success || step.outcome === JobStepOutcome.Warnings)) {
         const delta = getStepTimingDelta(step);
         if (delta) {
            runningText += " " + delta;
         }
      }

      if (step.state === JobStepState.Skipped) {
         runningText = "";
      }

      if (runningText) {
         text.push(runningText);
      }

   } else {
      if (!step.abortRequested && !batchIncluded) {
         text.push(batchText() ?? "Step does not have a batch.");
      }
   }

   if (eta.display) {
      text.push(`Expected to finish around ${eta.display}.`);
   }

   const finish = getStepFinishTime(step).display;

   if (finish && step.state !== JobStepState.Aborted) {

      let outcome = "";
      if (step.outcome === JobStepOutcome.Success) {
         outcome += `Completed at ${finish}`;
      }
      if (step.outcome === JobStepOutcome.Failure)
         outcome += `Completed with errors at ${finish}.`;
      if (step.outcome === JobStepOutcome.Warnings)
         outcome += `Completed with warnings at ${finish}.`;

      if (outcome) {
         text.push(outcome);
      }
   }

   if (!text.length) {
      text.push(`Step is in ${step.state} state.`);
   }

   return text.join(".&nbsp;&nbsp;");

}

export const StepSummaryPanel: React.FC<{ jobDetails: JobDetailsV2; stepId: string }> = observer(({ jobDetails, stepId }) => {

   if (jobDetails.updated) { }

   const dataView = jobDetails.getDataView<StepSummaryView>("StepSummaryView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   const { hordeClasses, modeColors } = getHordeStyling();

   dataView.subscribe();

   dataView.initialize([sideRail]);

   const jobData = jobDetails.jobData;

   if (!jobData) {
      return null;
   }

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   const jobPrice = jobDetails.jobPrice();
   const stepPrice = jobDetails.stepPrice(stepId);

   let priceText = "";

   if (stepPrice) {
      priceText = `Estimated cost: $${stepPrice.toFixed(2)}`;
      if (jobPrice) {
         priceText = `Estimated cost: $${stepPrice.toFixed(2)} (of $${jobPrice.toFixed(2)})`;
      }
   }

   const step = jobDetails.stepById(stepId)!

   const reportData = jobDetails.getSummaryReport(stepId);

   const spawnedJobs = step.spawnedJobs ?? [];
   const spawnedJobUrl = spawnedJobs.length > 0 ? `/job/${spawnedJobs[0]}` : undefined;
   const parentJobUrl = jobData.parentJobId && jobData.parentJobId ? `/job/${jobData.parentJobId}?step=${jobData.parentJobStepId}` : undefined;

   const report = step?.reports?.find(r => !!r.message);
   const { message: stepMessage, color: stepMessageColor } = getStepMessageInfo(report);

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 0, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 18 }}>
            <Stack horizontal>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Summary</Text>
               </Stack>
            </Stack>
            <Stack >
               <Stack style={{ color: modeColors.text }}>
                  <Markdown>{getStepSummaryMarkdown(jobDetails, stepId)}</Markdown>
               </Stack>
               {!!step?.cancellationReason && <Stack style={{ color: modeColors.text, paddingTop: 12, paddingBottom: 12 }} tokens={{ childrenGap: 8 }}>
                  <Label>Cancellation Reason</Label>
                  <Markdown>{step.cancellationReason}</Markdown>
               </Stack>
               }
               {!!parentJobUrl && <Stack style={{ paddingTop: 8 }}><Markdown>{`This step was spawned by a [parent job](${parentJobUrl}).`}</Markdown></Stack>}
               {!!spawnedJobUrl && <Stack style={{ paddingTop: 8 }}><Markdown>{`This step spawned a [new job](${spawnedJobUrl}).`}</Markdown></Stack>}
               {!!stepMessage && <Stack verticalFill verticalAlign="center" style={{ paddingTop: 12 }} tokens={{ childrenGap: 6 }}>
                  <Text variant='medium' style={{ fontWeight: "bolder" }}>Step Message:</Text>
                  <Stack horizontal verticalFill verticalAlign="center" style={{ paddingLeft: 24, paddingTop: 2 }} tokens={{ childrenGap: 6 }}>
                     {!!stepMessageColor && <FontIcon style={{ fontSize: 13, color: stepMessageColor }} iconName="Square" />}
                     <Text>{stepMessage}</Text>
                  </Stack>
               </Stack>}
               {!!reportData && <Stack style={{ paddingTop: 8 }}> <Markdown>{reportData}</Markdown> </Stack>}
               {!!priceText && <Stack style={{ paddingTop: 8 }}>
                  <Text>{priceText}</Text>
               </Stack>}
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});

