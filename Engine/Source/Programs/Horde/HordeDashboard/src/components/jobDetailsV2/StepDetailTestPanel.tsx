// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from '@fluentui/react';
import { mergeStyleSets } from '@fluentui/react/lib/Styling';
import { observer } from 'mobx-react-lite';
import React, { useEffect } from 'react';
import { Link } from 'react-router-dom';
import backend from '../../backend';
import { TestData } from '../../backend/Api';
import { ISideRailLink } from '../../base/components/SideRail';
import { PluginMount } from  '../../legacyPlugins/pluginTypes';
import hordePlugins from  '../../legacyPlugins';
import { ComponentMount } from '../TestReportView';
import { JobDataView, JobDetailsV2 } from './JobDetailsViewCommon';
import { getHordeStyling } from '../../styles/Styles';
import { getHordeTheme } from '../../styles/theme';


const sideRail: ISideRailLink = { text: "Test Reports", url: "rail_step_tests" };


class StepTestDataView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   set(stepId?: string) {

      const details = this.details;
      if (!details) {
         return;
      }

      if (this.stepId === stepId || !details.jobId) {
         return;
      }

      this.stepId = stepId;

      if (!this.stepId) {
         return;
      }

      backend.getJobTestData(details.jobId!, this.stepId).then(response => {
         const testdata = response;

         this.items = [];
         testdata.forEach((test) => {
            let components: ComponentItem = { hasComponent: false };
            const splitKey = test.key.split('::', 2);
            const type = splitKey[0];
            const name = splitKey.length > 1 ? splitKey[1] : type;
            if (testReportComponentTypes.has(type)) {
               components = testReportComponentTypes.get(type)!;
            }
            else {
               let pluginComponents = hordePlugins.getComponents(PluginMount.TestReportPanel, type);
               components.hasComponent = pluginComponents.length > 0
               testReportComponentTypes.set(type, components);
               if (components.hasComponent) {
                  pluginComponents = hordePlugins.getComponents(PluginMount.TestReportLink, type);
                  if (pluginComponents.length > 0) {
                     components.linkComponent = pluginComponents[0].component;
                  }
               }
            }
            components.hasComponent && this.items.push({ ...components, item: test, name: name })
         });
      
         this.updateReady();
      }).finally(() => {
         this.initialize(this.items?.length ? [sideRail] : undefined);
      });

   }

   clear() {
      this.items = [];      
      this.stepId = undefined;
      super.clear();
   }

   detailsUpdated() {
      
      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();

   }

   items: TestDataItem[] = [];

   stepId?: string;

   order = 4;

}

JobDetailsV2.registerDataView("StepTestDataView", (details: JobDetailsV2) => new StepTestDataView(details));


let _styles: any;
const getStyles = () => {

   const theme = getHordeTheme();

   const styles = _styles ?? mergeStyleSets({
      item: {
         padding: 8,
         borderBottom: '1px solid ' + theme.palette.neutralLighter,
         selectors: {
            ':hover': { background: theme.palette.neutralLight }
         }
      }
   });

   _styles = styles;

   return styles;
   
}


type ComponentItem = {
   hasComponent: boolean;
   linkComponent?: React.FC<any>;
}

type TestDataItem = ComponentItem & {
   name: string;
   item: TestData;
}

const testReportComponentTypes: Map<string, ComponentItem> = new Map();

export const StepTestReportPanel: React.FC<{ jobDetails: JobDetailsV2, stepId?: string }> = observer(({ jobDetails, stepId }) => {

   const dataView = jobDetails.getDataView<StepTestDataView>("StepTestDataView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   const { hordeClasses } = getHordeStyling();
   const styles = getStyles();

   dataView.subscribe();

   dataView.set(stepId);

   const items = dataView.items;
   if (!items?.length) {
      return null;
   }   

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   const isPreflight = jobDetails.jobData?.preflightCommitId !== undefined;

   return (<Stack id={sideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Test Report</Text>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               {!isPreflight && <Link className={"view-log-link"} to={`/automation?stream=${jobDetails.stream?.id}`}>Open Automation Hub</Link>}
               {items.map((test) => <Stack key={test.item.id} className={styles.item} horizontal wrap tokens={{ childrenGap: 30 }}>
                  <Link className={"view-log-link"} to={`/testreport/${test.item.id}`}>{test.name}</Link>
                  <ComponentMount component={test.linkComponent} id={test.item.id} fullName={test.item.key} />
               </Stack>)}
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});

