// Copyright Epic Games, Inc. All Rights Reserved.

import { Spinner, SpinnerSize, Stack } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import { useEffect, useState } from "react";
import { useNavigate } from "react-router-dom";
import backend from "../../backend";
import { GetArtifactResponse } from "../../backend/Api";
import { getHordeStyling } from "../../styles/Styles";
import { HistoryModal } from "../agents/HistoryModal";
import { useQuery } from "horde/base/utilities/hooks";
import { BisectionPanel } from "./JobDetailBisection";
import { HealthPanel } from "./JobDetailHealthV2";
import { PreflightPanel } from "./JobDetailPreflight";
import { StepHistoryPanel } from "./JobDetailStepHistory";
import { StepTrendsPanelV2 } from "./JobDetailStepTrendsV2";
import { TimelinePanel } from "./JobDetailTimeline";
import { StepsPanelV2 } from "./JobDetailViewSteps";
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import { StepSummaryPanel } from "./StepDetailSummary";
import { StepTestReportPanel } from "./StepDetailTestPanel";
import { StepErrorPanel } from "./StepErrorPanel";

class StepDetailDataView extends JobDataView {

   filterUpdated() {
      this.updateReady();
   }

   detailsUpdated() {

      if (!this.details?.jobData) {
         return;
      }

      this.queryArtifacts();

      this.updateReady();

   }

   async queryArtifacts() {

      if (!this.details?.jobData) {
         return;
      }

      const stepId = this.stepId!;

      let artifacts = this.details.stepArtifacts.get(stepId);
      if (!artifacts) {
         const key = `job:${this.details!.jobData!.id}/step:${this.stepId}`;
         try {
            const v = await backend.getJobArtifactsV2(undefined, [key]);
            artifacts = v.artifacts;
            this.details.stepArtifacts.set(stepId, artifacts);

            // need to notify things like the operations bar
            this.details.externalUpdate();
         } catch (err) {
            console.error(err);
         } finally {
            artifacts = [];
         }
         this.artifacts = artifacts;
      }
   }

   async set(stepId: string) {

      if (this.stepId === stepId) {
         return;
      }

      this.stepId = stepId;

      this.queryArtifacts();

      this.updateReady();

   }

   clear() {
      this.stepId = undefined;
      this.artifacts = undefined;
      super.clear();
   }

   artifacts?: GetArtifactResponse[];

   stepId?: string;

}

JobDetailsV2.registerDataView("StepDetailDataView", (details: JobDetailsV2) => new StepDetailDataView(details));

const StepDetailViewInner: React.FC<{ jobDetails: JobDetailsV2, stepId: string }> = observer(({ jobDetails, stepId }) => {

   let [historyAgentId, setHistoryAgentId] = useState<string | undefined>(undefined);
   const navigate = useNavigate();
   const query = useQuery();

   const dataView = jobDetails.getDataView<StepDetailDataView>("StepDetailDataView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);


   dataView.subscribe();

   dataView.initialize();

   dataView.set(stepId);

   if (!historyAgentId && query.get("agentId")) {
      historyAgentId = query.get("agentId")!;
   }

   const jobData = jobDetails.jobData;

   if (!jobData) {
      return null;
   }

   const step = jobDetails.stepById(stepId);

   return <Stack>
      <HistoryModal agentId={historyAgentId} onDismiss={() => { navigate(`/job/${jobData.id}?step=${stepId}`, { replace: true }); setHistoryAgentId(undefined); }} />

      <Stack>
         <StepSummaryPanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>
      <Stack>
         <PreflightPanel jobDetails={jobDetails} />
      </Stack>
      <Stack>
         <HealthPanel jobDetails={jobDetails} />
      </Stack>
      <Stack>
         <StepErrorPanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>
      <Stack>
         <StepsPanelV2 jobDetails={jobDetails} depStepId={stepId} />
      </Stack>
      <Stack>
         <StepTestReportPanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>
      {<Stack>
         <StepHistoryPanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>}
      {!!step && <Stack>
         <TimelinePanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>}
      {!!step && <Stack>
         <StepTrendsPanelV2 jobDetails={jobDetails} stepId={stepId}/>
      </Stack>}
      {!!step && <Stack>
         <BisectionPanel jobDetails={jobDetails} stepId={stepId} />
      </Stack>}
      {!jobDetails.viewsReady && <Stack style={{paddingTop: 32}}>
         <Spinner size={SpinnerSize.large} />
      </Stack>}

   </Stack>
});


export const StepDetailView: React.FC<{ jobDetails: JobDetailsV2, stepId: string }> = ({ jobDetails, stepId }) => {
   const { hordeClasses } = getHordeStyling();
   return (
      <Stack className={hordeClasses.horde}>
         <StepDetailViewInner jobDetails={jobDetails} stepId={stepId} />
      </Stack>
   );
};

