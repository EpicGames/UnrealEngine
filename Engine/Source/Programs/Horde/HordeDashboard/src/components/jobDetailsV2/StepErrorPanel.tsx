// Copyright Epic Games, Inc. All Rights Reserved.

import { List, Pivot, PivotItem, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { getFocusStyle, mergeStyleSets } from '@fluentui/react/lib/Styling';
import { observer } from "mobx-react-lite";
import React, { useEffect } from 'react';
import { Link, useNavigate } from 'react-router-dom';
import backend from "../../backend";
import { DashboardPreference, EventData, EventSeverity, JobStepState } from '../../backend/Api';
import { ISideRailLink } from "../../base/components/SideRail";
import { getHordeStyling } from "../../styles/Styles";
import { getHordeTheme } from "../../styles/theme";
import { renderLine } from '../logs/LogRender';
import { JobDataView, JobDetailsV2 } from "./JobDetailsViewCommon";
import dashboard from "horde/backend/Dashboard";

const errorSideRail: ISideRailLink = { text: "Events", url: "rail_log_events" };

class StepSummaryErrorsView extends JobDataView {

   filterUpdated() {
      // this.updateReady();
   }

   get errors() {

      return this.events.filter(e => e.severity === EventSeverity.Error).sort((a, b) => {
         return a.lineIndex < b.lineIndex ? -1 : 1;
      });

   }

   get warnings() {

      return this.events.filter(e => e.severity === EventSeverity.Warning).sort((a, b) => {
         return a.lineIndex < b.lineIndex ? -1 : 1;
      });

   }

   async set(stepId: string) {

      const details = this.details;
      if (!details) {
         return;
      }

      if (this.stepId === stepId) {
         return;
      }

      this.stepId = stepId;

      if (!this.stepId) {
         this.initialize();
         this.events = [];
         return;
      }

      const step = details.stepById(stepId);

      if (!step || !step.logId) {
         this.initialize();
         this.events = [];
         return;
      }

      let index = 0;
      let count = 20;
      this.events = [];

      let init = false;

      try {

         while (true) {

            const events = await backend.getLogEvents(step.logId, { index: index, count: count });

            this.events.push(...events);

            this.updateReady();

            if (!init) {

               init = true;

               const rails: ISideRailLink[] = [];
               if (this.events.length) {
                  rails.push(errorSideRail);
               }

               this.initialize(rails);
            }

            if (!events.length) {
               break;
            }

            index += count;
         }


      } finally {

         this.loaded = true;
         this.updateReady();

         if (!init) {
            this.initialize([]);
         }

      }
   }

   clear() {
      this.stepId = "";
      this.events = [];
      this.loaded = false;
      super.clear();
   }

   detailsUpdated() {
      if (!this.details?.jobData) {
         return;
      }

      this.updateReady();
   }

   stepId: string = "";

   events: EventData[] = []

   loaded = false;

   order = 2;

}

JobDetailsV2.registerDataView("StepSummaryErrorsView", (details: JobDetailsV2) => new StepSummaryErrorsView(details));

let _styles: any;

const getStyles = () => {

   const theme = getHordeTheme();

   const warningColorPref = dashboard.getPreference(DashboardPreference.ColorWarning);
   const errorColorPref = dashboard.getPreference(DashboardPreference.ColorError);

   const styles = _styles ?? mergeStyleSets({
      gutter: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: errorColorPref ? errorColorPref : "#EC4C47",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      gutterWarning: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: warningColorPref ? warningColorPref : "rgb(247, 209, 84)",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 8,
            paddingBottom: 8,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0
         }
      ],
      itemCell: [
         getFocusStyle(theme, { inset: -1 }),
         {
            selectors: {
               '&:hover': { background: theme.palette.neutralLight }
            }
         }
      ]
   });

   _styles = styles;
   return styles;

}



const ErrorPane: React.FC<{ jobDetails: JobDetailsV2; view: StepSummaryErrorsView, stepId: string; showErrors?: boolean }> = ({ jobDetails, view, stepId, showErrors }) => {

   const navigate = useNavigate();
   const styles = getStyles();
   const { modeColors } = getHordeStyling();

   if (!stepId) {
      return (<div />);
   }

   const events = showErrors ? view.errors : view.warnings;

   const step = jobDetails.stepById(stepId);

   if (!step) {
      return null;
   }

   if (!events.length) {
      return null;
   }

   const onRenderCell = (item?: EventData, index?: number, isScrolling?: boolean): JSX.Element => {

      if (!item) {
         return <div>???</div>;
      }

      const url = `/log/${item.logId}?lineIndex=${item.lineIndex + 1}`;

      const lines = item.lines.filter(line => line.message?.trim().length).map(line => <Stack key={`steperrorpane_line_${item.lineIndex}`} styles={{ root: { paddingLeft: 8, paddingRight: 8, lineBreak: "anywhere", whiteSpace: "pre-wrap", lineHeight: 18, fontSize: 10, fontFamily: "Horde Cousine Regular, monospace, monospace" } }}> <Link style={{ color: modeColors.text }} to={url}>{renderLine(navigate, line, undefined, {})}</Link></Stack>);

      return (<Stack className={styles.itemCell} styles={{ root: { padding: 8, marginRight: 8 } }}><Stack className={item.severity === EventSeverity.Warning ? styles.gutterWarning : styles.gutter} styles={{ root: { padding: 0, margin: 0 } }}>
         <Stack styles={{ root: { paddingLeft: 14 } }}>
            {!!lines.length && lines}
            {!lines.length && <Text>{step.state === JobStepState.Running ? "Generating Log Data" : "Missing Log Data"}</Text>}
         </Stack>
      </Stack>
      </Stack>
      );
   };

   return (<Stack>
      <Stack tokens={{ padding: 4 }}>
         <Stack>
            <div style={{ overflowY: 'auto', overflowX: 'visible', maxHeight: "400px" }} data-is-scrollable={true}>
               <List
                  items={events}
                  data-is-focusable={false}
                  onRenderCell={onRenderCell} />
               <div style={{ padding: 8 }} />
            </div>
         </Stack>
      </Stack>
   </Stack>);
};

export const StepErrorPanel: React.FC<{ jobDetails: JobDetailsV2; stepId: string }> = observer(({ jobDetails, stepId }) => {

   const dataView = jobDetails.getDataView<StepSummaryErrorsView>("StepSummaryErrorsView");

   useEffect(() => {
      return () => {
         dataView?.clear();
      };
   }, [dataView]);

   const { hordeClasses } = getHordeStyling();

   dataView.subscribe();

   dataView.set(stepId);

   const events = dataView.events;

   if (!events.length) {
      return null;
   }

   if (!jobDetails.viewReady(dataView.order)) {
      return null;
   }

   const errors = dataView.errors;
   const warnings = dataView.warnings;


   return (<Stack id={errorSideRail.url} styles={{ root: { paddingTop: 18, paddingRight: 12 } }}>
      <Stack className={hordeClasses.raised}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal tokens={{ childrenGap: 18 }}>
               <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Events</Text>
               {!dataView.loaded && <Spinner size={SpinnerSize.medium} />}
            </Stack>
            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <Pivot>
                  {!!errors.length && <PivotItem headerText="Errors" itemCount={errors.length}>
                     <Stack style={{ paddingTop: 12 }}>
                        <ErrorPane view={dataView} stepId={stepId} jobDetails={jobDetails} showErrors={true} />
                     </Stack>
                  </PivotItem>}
                  {!!warnings.length && <PivotItem headerText="Warnings" itemCount={warnings.length}>
                     <Stack style={{ paddingTop: 12 }}>
                        <ErrorPane view={dataView} stepId={stepId} jobDetails={jobDetails} />
                     </Stack>
                  </PivotItem>}
               </Pivot>
            </Stack>
         </Stack>
      </Stack>
   </Stack>);
});

