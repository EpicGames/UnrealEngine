// Copyright Epic Games, Inc. All Rights Reserved.
import { Checkbox, DefaultButton, IconButton, MessageBar, Modal, PrimaryButton, ScrollablePane, ScrollbarVisibility, Spinner, SpinnerSize, Stack, Text, TextField } from '@fluentui/react';
import React, { useState } from 'react';
import backend from '../../backend';
import { CreateJobRequest } from '../../backend/Api';
import { Link, useNavigate } from 'react-router-dom';
import { JobDetailsV2 } from './JobDetailsViewCommon';
import { getHordeStyling } from '../../styles/Styles';
import { errorDialogStore } from '../error/ErrorStore';
import moment from 'moment';
import dashboard from 'horde/backend/Dashboard';

export enum StepRetryType {
   RunAgain,
   TestFix
}

type RunAgainResult = {
   stepId: string,
   newStepId?: string,
   error?: string
}

export const RetryStepsModal: React.FC<{ stepIds: string[]; jobDetails: JobDetailsV2; onClose: () => void }> = ({ stepIds, jobDetails, onClose }) => {

   const [submitting, setSubmitting] = useState(false);
   const [submitResults, setSubmitResults] = useState<RunAgainResult[] | undefined>(undefined);
   const [retrySteps, setRetrySteps] = useState(new Set<string>());

   const { hordeClasses } = getHordeStyling();

   if (submitting && !submitResults) {
      return <Modal className={hordeClasses.modal} isOpen={true} styles={{ main: { padding: 8, width: 800 } }} onDismiss={() => { onClose() }}>
         <Stack horizontal styles={{ root: { padding: 8, paddingBottom: 32 } }}>
            <Text variant="mediumPlus">Updating Steps</Text>
         </Stack>
         <Stack horizontalAlign="center">
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Modal>
   }

   if (submitResults) {

      const resultElements = submitResults.map(result => {
         const stepName = jobDetails.getStepName(result.stepId, true);
         let resultText = `${stepName} - `;
         if (result.newStepId) {
            const url = `/job/${jobDetails.jobId!}?step=${result.newStepId}`
            resultText += "Retried"
            return <Stack horizontal>
               <Link to={url}>{resultText}</Link>
            </Stack>
         } else if (result.error) {
            resultText += `Error: ${result.error}`;
         } else {
            resultText += "Step could not be retried"
         }
         return <Stack><Text>{resultText}</Text></Stack>
      })


      return <Modal className={hordeClasses.modal} isOpen={true} styles={{ main: { padding: 8, width: 800 } }} onDismiss={() => { onClose() }}>
         <Stack horizontal styles={{ root: { padding: 8 } }}>
            <Stack.Item grow={2}>
               <Text variant="mediumPlus">Step Retry Results</Text>
            </Stack.Item>
            <Stack.Item grow={0}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onClose(); }}
               />
            </Stack.Item>
         </Stack>
         <Stack styles={{ root: { paddingLeft: 32 } }} tokens={{ childrenGap: 8 }}>
            {resultElements}
         </Stack>
      </Modal>
   }


   const jobData = jobDetails.jobData;
   if (!jobData) {
      console.error("Attempting to display StepRetryModal without job data");
      return null;
   }

   const headerText = stepIds.length > 1 ? "Retry Steps?" : "Retry Step?";

   const onRetry = async () => {

      const results: RunAgainResult[] = [];

      const retryStepIds = Array.from(retrySteps).sort((a, b) => {
         const idxA = stepIds.indexOf(a);
         const idxB = stepIds.indexOf(b);

         return idxA - idxB;
      });

      for (let i = 0; i < retryStepIds.length; i++) {

         const stepId = retryStepIds[i];

         console.log(`Retrying: ${jobDetails.stepById(stepId)?.name}`);

         const batch = jobDetails.batchByStepId(stepId);
         if (!batch) {
            results.push({
               stepId: stepId,
               error: `Unable to get batch for step: ${stepId}`
            })
            continue;
         }

         try {
            
            const response = await backend.updateJobStep(jobData.id, batch.id, stepId, { retry: true })

            if (!response.stepId) {
               results.push({
                  stepId: stepId
               })
            } else {
               results.push({
                  stepId: stepId,
                  newStepId: response.stepId
               })
            }

         } catch (error) {

            results.push({ stepId: stepId, error: error?.toString() });

         }
      }

      setSubmitResults(results);

   }

   const stepLookup = new Map<string, string>();

   const stepNames: string[] = [];
   const disabledSteps = new Set<string>();
   const stepOutputInfo = dashboard.getArtifactTypeInfo("step-output")
   stepIds.forEach((stepId, index) => {
      const stepName = jobDetails.getStepName(stepId, true) ?? `Unknown Step ${index}`;
      stepLookup.set(stepName, stepId);
      stepNames.push(stepName);

      // Determine if temp storage for step is expired
      const step = jobDetails.stepById(stepId)!;
      let tempStorageExpired = false;
      if (stepOutputInfo && stepOutputInfo.keepDays && step.startTime) {

         const expireMs = stepOutputInfo.keepDays * 86400000;
         if ((Date.now() - new Date(step.startTime).getTime()) > expireMs)
         {
            tempStorageExpired = true;
            // Disable step if temp storage expired
            disabledSteps.add(stepId);
         }      
      } 

   });

   const stepElements: JSX.Element[] = [];

   if (stepIds.length > 1) {
      const allSelected = retrySteps.size === stepIds.length - disabledSteps.size;

      stepElements.push(
         <Checkbox
            key={"selAll"}
            label={allSelected ? "Deselect All" : "Select All"} 
            checked={allSelected} 
            onChange={(ev, checked) => {
               const newSteps = new Set<string>();
               if (checked) {
                  stepIds.forEach(s => {
                     if(!disabledSteps.has(s)) {
                        newSteps.add(s)
                     }
                  });
               }
               setRetrySteps(newSteps);
            }} 
         />
      )
   }

   stepElements.push(...stepNames.map(name => {
      const stepId = stepLookup.get(name)!;

      return (
         <Checkbox 
            label={name}
            key={stepId}
            disabled={disabledSteps.has(stepId)} 
            checked={retrySteps.has(stepId)} 
            onChange={(ev, checked) => {
               const newSteps = new Set(retrySteps);
               if (checked) {
                  newSteps.add(stepId);
               } else {
                  newSteps.delete(stepId);
               }
               setRetrySteps(newSteps);
            }} 
         />
      );
   }))

   let height = Math.max(stepElements.length * 32, 128);
   if (height > 432) {
      height = 432;
   }

   return <Modal className={hordeClasses.modal} isOpen={true} styles={{ main: { padding: 8, width: 800, maxHeight: 640 } }} onDismiss={() => { onClose() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2} style={{paddingLeft: 8}}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { onClose(); }}
            />
         </Stack.Item>
      </Stack>

      {disabledSteps.size > 0 && <MessageBar styles={{root: {marginLeft: 16, marginRight: 16, width: "fitContent"}}}>Some steps are unavailable for retry because required artifacts are missing or have expired.</MessageBar>}

      <Stack style={{marginRight: 18, marginTop: 18}}>
         <Stack styles={{ root: { position: "relative", height: height } }}>
            <ScrollablePane scrollbarVisibility={ScrollbarVisibility.auto}>
               <Stack style={{ paddingLeft: 24, paddingRight: 24, height: height - 12 }} tokens={{ childrenGap: 8 }}>
                  {stepElements}
               </Stack>
            </ScrollablePane>
         </Stack>
      </Stack>

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8, paddingRight: 12 } }}>
            <Stack grow />
            <PrimaryButton text="Retry" disabled={submitting || retrySteps.size === 0} onClick={() => { onRetry(); setSubmitting(true) }} />
            <DefaultButton text="Cancel" disabled={submitting} onClick={() => { onClose(); }} />
         </Stack>
      </Stack>
   </Modal>;
}

export const StepRetryModal: React.FC<{ stepId: string; jobDetails: JobDetailsV2; show: boolean; type: StepRetryType; onClose: () => void }> = ({ stepId, jobDetails, show, type, onClose }) => {

   const navigate = useNavigate();
   const { hordeClasses } = getHordeStyling();

   const jobData = jobDetails.jobData;
   if (!jobData) {
      console.error("Attempting to display StepRetryModal without job data");
      return null;
   }

   const step = jobDetails.stepById(stepId);

   const headerText = type === StepRetryType.RunAgain ? `Run "${step?.name}" Again?` : "Preflight Step";

   let fixCL = 0;

   const close = () => {
      onClose();
   };

   const onRunAgain = async () => {

      const batch = jobDetails.batchByStepId(stepId);
      if (!batch) {
         console.error("Unable to get batch for step");
         return;
      }

      await backend.updateJobStep(jobData.id, batch.id, stepId, {
         retry: true
      }).then(async (response) => {

         if (response.stepId) {
            if (jobDetails.jobData?.id) {
               // get new job data so we have the new batch/step
               jobDetails.jobData = await backend.getJob(jobDetails.jobData?.id);
               jobDetails.processGraph();
            }

            navigate(`/job/${jobData.id}?step=${response.stepId!}`)
         }

      }).catch((reason) => {
         // @todo: error ui
         console.error(reason);
      }).finally(() => {
         close();
      });

   };

   const onTryFix = async () => {

      if (!fixCL || isNaN(fixCL)) {
         console.error("Invalid Fix CL");
         return;
      }

      const job = jobData;

      const data: CreateJobRequest = {
         streamId: job.streamId,
         templateId: job.templateId!,
         parameters: job.parameters,
         change: job.change,
         targets: [step!.name],
         preflightChange: fixCL
      };

      console.log("Submitting job");
      console.log(data);

      backend.createJob(data).then(data => {
         navigate(`/job/${data.id}`);
      }).catch(reason => {

         errorDialogStore.set({

            reason: `${reason}`,
            title: `Error Creating Job`,
            message: `There was an issue creating the job.\n\nReason: ${reason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`

         }, true);

      });
   };

   const height = type === StepRetryType.TestFix ? 200 : 140;

   return <Modal className={hordeClasses.modal} isOpen={show} styles={{ main: { padding: 8, width: 540, height: height, minHeight: height } }} onDismiss={() => { close() }}>
      <Stack horizontal styles={{ root: { padding: 8 } }}>
         <Stack.Item grow={2}>
            <Text variant="mediumPlus">{headerText}</Text>
         </Stack.Item>
         <Stack.Item grow={0}>
            <IconButton
               iconProps={{ iconName: 'Cancel' }}
               ariaLabel="Close popup modal"
               onClick={() => { close(); }}
            />
         </Stack.Item>
      </Stack>

      <Stack styles={{ root: { paddingLeft: 8, width: 500 } }}>
         {type === StepRetryType.TestFix && <TextField label="Shelved Change" onChange={(ev, newValue) => { ev.preventDefault(); fixCL = parseInt(newValue ?? ""); }} />}
      </Stack>

      <Stack styles={{ root: { padding: 8 } }}>
         <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 12, paddingLeft: 8, paddingBottom: 8 } }}>
            <Stack grow />
            <PrimaryButton text="Run" disabled={false} onClick={() => { type === StepRetryType.TestFix ? onTryFix() : onRunAgain(); }} />
            <DefaultButton text="Cancel" disabled={false} onClick={() => { close(); }} />
         </Stack>
      </Stack>
   </Modal>;

};