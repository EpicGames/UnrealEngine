// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton } from '@fluentui/react';
import React from 'react';
import { Link } from 'react-router-dom';
import { GetLabelStateResponse, LabelOutcome } from '../../backend/Api';
import { getLabelColor } from '../../styles/colors';
import { getHordeStyling } from '../../styles/Styles';
import { LabelHoverCard } from './LabelHoverCard';

interface LabelBadgeProps {
    label: GetLabelStateResponse;
    jobId: string;
    labelIndex: number;
    streamId: string;
    templateId: string;
    templateName?: string;
    change?: number;
}

export const LabelBadge: React.FC<LabelBadgeProps> = ({ label, jobId, labelIndex, streamId, templateId, templateName, change }) => {
    const { hordeClasses } = getHordeStyling();
    const color = getLabelColor(label.state, label.outcome);
    const isFailure = label.outcome === LabelOutcome.Failure;
    const badgeClass = isFailure ? hordeClasses.badgeNoIcon : hordeClasses.badgeCompact;

    const url = labelIndex >= 0 ? `/job/${jobId}?label=${labelIndex}` : `/job/${jobId}`;

    const badge = (
        <div id={`label_${jobId}_${label.dashboardName}_${label.dashboardCategory}`.replace(/[^A-Za-z0-9]/g, "")} style={{ lineHeight: 0 }}>
            <Link to={url}><div className={badgeClass}>
                <DefaultButton key={label.dashboardName} style={{ backgroundColor: color.primaryColor }} text={label.dashboardName ?? 'Other'}
                    onMouseOver={(ev) => ev.stopPropagation()}
                    onMouseMove={(ev) => ev.stopPropagation()}>
                    {!!color.secondaryColor && <div style={{
                        borderLeft: "10px solid transparent",
                        borderRight: `10px solid ${color.secondaryColor}`,
                        borderBottom: "10px solid transparent",
                        height: 0,
                        width: 0,
                        position: "absolute",
                        right: 0,
                        top: 0,
                        zIndex: 1
                    }} />}
                </DefaultButton>
            </div></Link>
        </div>
    );

    // Only show hover card for actual defined labels (not the procedural defaultLabel "Other")
    if (labelIndex < 0) {
        return badge;
    }

    return (
        <LabelHoverCard
            jobId={jobId}
            labelIndex={labelIndex}
            streamId={streamId}
            templateId={templateId}
            templateName={templateName}
            dashboardName={label.dashboardName}
            dashboardCategory={label.dashboardCategory}
            state={label.state}
            outcome={label.outcome}
            change={change}
        >
            {badge}
        </LabelHoverCard>
    );
};

export default LabelBadge;
