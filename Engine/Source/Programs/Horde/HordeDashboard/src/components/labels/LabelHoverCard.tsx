// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, HoverCard, HoverCardType, mergeStyleSets, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import React, { useEffect, useState } from 'react';
import { Link } from 'react-router-dom';
import backend from '../../backend';
import { GetLabelDetailsResponse, GetLabelIssueResponse, GetLabelStepResponse, GetLogEventResponse, IssueSeverity, JobStepOutcome, JobStepState, LabelOutcome, LabelState } from '../../backend/Api';
import dashboard, { StatusColor } from '../../backend/Dashboard';
import { getLabelColor, getStepStatusColor } from '../../styles/colors';
import { ChangeButton } from '../ChangeButton';
import { IssueModalV2 } from '../IssueViewV2';

// Client-side cache for label details responses (avoids re-fetching on re-hover)
const labelDetailsCache = new Map<string, { data: GetLabelDetailsResponse; timestamp: number }>();
const cacheTtlMs = 30_000;

function getCachedDetails(jobId: string, labelIndex: number): GetLabelDetailsResponse | undefined {
    const key = `${jobId}:${labelIndex}`;
    const entry = labelDetailsCache.get(key);
    if (!entry) return undefined;
    if (Date.now() - entry.timestamp > cacheTtlMs) {
        labelDetailsCache.delete(key);
        return undefined;
    }
    return entry.data;
}

function setCachedDetails(jobId: string, labelIndex: number, data: GetLabelDetailsResponse): void {
    labelDetailsCache.set(`${jobId}:${labelIndex}`, { data, timestamp: Date.now() });
}

interface LabelHoverCardProps {
    jobId: string;
    labelIndex: number;
    streamId: string;
    templateId: string;
    templateName?: string;
    dashboardName?: string;
    dashboardCategory?: string;
    state?: LabelState;
    outcome?: LabelOutcome;
    change?: number;
    children: React.ReactNode;
}

export const LabelHoverCard: React.FC<LabelHoverCardProps> = (props) => {
    const { jobId, labelIndex, children } = props;

    const onRenderPlainCard = (): JSX.Element => {
        return <LabelHoverContent {...props} />;
    };

    return (
        <HoverCard
            cardOpenDelay={300}
            type={HoverCardType.plain}
            plainCardProps={{
                onRenderPlainCard,
                directionalHintFixed: true
            }}
            cardDismissDelay={200}
            key={`label-hover-${jobId}-${labelIndex}`}
        >
            {children}
        </HoverCard>
    );
};

const StatusSquare: React.FC<{ color: string; size?: number }> = ({ color: c, size }) => (
    <div style={{
        width: size ?? 11, height: size ?? 11, minWidth: size ?? 11,
        backgroundColor: c, borderRadius: 2
    }} />
);

const LabelHoverContent: React.FC<LabelHoverCardProps> = ({ jobId, labelIndex, streamId, templateName, dashboardName, dashboardCategory, state, outcome, change }) => {
    const [details, setDetails] = useState<GetLabelDetailsResponse | undefined>();
    const [loadError, setLoadError] = useState<string | undefined>();
    const [expandedLogs, setExpandedLogs] = useState<Set<string>>(new Set());
    const [selectedIssueId, setSelectedIssueId] = useState<string | undefined>();

    const colors = dashboard.getStatusColors();

    useEffect(() => {
        if (labelIndex < 0) {
            return;
        }
        const cached = getCachedDetails(jobId, labelIndex);
        if (cached) {
            setDetails(cached);
            return;
        }
        const abortController = new AbortController();
        backend.getLabelDetails(jobId, labelIndex, { signal: abortController.signal })
            .then(data => {
                setCachedDetails(jobId, labelIndex, data);
                setDetails(data);
            })
            .catch(err => {
                if (!abortController.signal.aborted) {
                    setLoadError(`Failed to load: ${err}`);
                }
            });
        return () => abortController.abort();
    }, [jobId, labelIndex]);

    const labelColor = getLabelColor(
        details?.state ?? state,
        details?.outcome ?? outcome
    );

    return (
        <Stack style={{ padding: 14, maxWidth: 744, minWidth: 464 }} tokens={{ childrenGap: 10 }}>
            {/* Header + Previous build */}
            <Stack tokens={{ childrenGap: 10 }}>
                <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
                    <StatusSquare color={labelColor.primaryColor} size={12} />
                    <Text style={{ fontFamily: "Horde Open Sans Bold", fontSize: 14 }}>
                        {dashboardCategory ? `${dashboardCategory} / ` : ''}{dashboardName ?? details?.dashboardName ?? ''}
                    </Text>
                    <ChangeButton job={{ id: jobId, streamId, change }} />
                </Stack>
                {details?.previousBuild && <PreviousBuildSection details={details} streamId={streamId} />}
            </Stack>

            {loadError && <Text style={{ color: colors.get(StatusColor.Failure), fontSize: 11 }}>{loadError}</Text>}

            {!details && !loadError && (
                <Stack horizontalAlign="center" style={{ padding: 8 }}>
                    <Spinner size={SpinnerSize.small} />
                </Stack>
            )}

            {/* Issues */}
            {details && details.issues.length > 0 && (
                <IssuesSection issues={details.issues} jobId={jobId} labelIndex={labelIndex} onIssueClick={(id) => setSelectedIssueId(id.toString())} />
            )}

            {/* Failing Steps */}
            {details && details.steps.length > 0 && (
                <StepsSection
                    steps={details.steps}
                    logEvents={details.logEvents}
                    totalStepCount={details.totalStepCount}
                    jobId={jobId}
                    expandedLogs={expandedLogs}
                    setExpandedLogs={setExpandedLogs}
                />
            )}

            {/* Cross-Stream Merge Flow */}
            {details?.crossStreamChain && details.crossStreamChain.entries.length > 0 && (
                <CrossStreamSection chain={details.crossStreamChain} templateName={templateName} />
            )}

            {/* Issue Modal */}
            {selectedIssueId && (
                <IssueModalV2
                    issueId={selectedIssueId}
                    popHistoryOnClose={false}
                    streamId={streamId}
                    onCloseExternal={() => setSelectedIssueId(undefined)}
                />
            )}
        </Stack>
    );
};

const PreviousBuildSection: React.FC<{ details: GetLabelDetailsResponse; streamId: string }> = ({ details, streamId }) => {
    const prev = details.previousBuild!;
    const prevColor = getLabelColor(prev.state, prev.outcome);

    return (
        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }}>
            <Link to={`/job/${prev.jobId}?label=${prev.labelIndex}`} style={{ textDecoration: 'none' }}>
                <div style={{
                    display: "inline-flex", alignItems: "center",
                    backgroundColor: prevColor.primaryColor, borderRadius: 2,
                    padding: "2px 6px", height: 15
                }}>
                    <Text style={{ fontSize: 10, fontFamily: "Horde Open Sans SemiBold", color: "#fff" }}>
                        {details.dashboardName ?? 'Label'}
                    </Text>
                </div>
            </Link>
            <ChangeButton job={{ id: prev.jobId, streamId, change: prev.change }} />
            <Text variant="small" style={{ opacity: 0.6 }}>Previous Job</Text>
        </Stack>
    );
};

const IssuesSection: React.FC<{ issues: GetLabelIssueResponse[]; jobId: string; labelIndex: number; onIssueClick?: (issueId: number) => void }> = ({ issues, jobId, labelIndex, onIssueClick }) => {
    const colors = dashboard.getStatusColors();
    const maxDisplay = 5;

    return (
        <Stack tokens={{ childrenGap: 4 }}>
            <Text variant="small" style={{ fontWeight: 600, opacity: 0.7 }}>Issues</Text>
            {issues.slice(0, maxDisplay).map(issue => (
                <Stack key={issue.id} tokens={{ childrenGap: 2 }} className={hoverRowStyles.row} style={{ padding: '2px 4px' }}>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                        <div style={{
                            width: 8, height: 8, flexShrink: 0,
                            backgroundColor: issue.severity === IssueSeverity.Warning
                                ? colors.get(StatusColor.Warnings)
                                : colors.get(StatusColor.Failure)
                        }} />
                        <span
                            style={{
                                fontSize: 12, textDecoration: issue.resolvedInStream ? 'line-through' : 'none',
                                color: 'inherit', overflow: 'hidden', textOverflow: 'ellipsis', whiteSpace: 'nowrap',
                                cursor: 'pointer'
                            }}
                            onClick={(ev) => { ev.stopPropagation(); ev.preventDefault(); onIssueClick?.(issue.id); }}>
                            #{issue.id}: {issue.summary}
                        </span>
                    </Stack>
                    <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0 }} style={{ paddingLeft: 12, flexWrap: 'wrap' }}>
                        {(() => {
                            const parts: JSX.Element[] = [];
                            const sep = <Text key={`sep-${parts.length}`} style={{ fontSize: 12, opacity: 0.5, padding: '0 4px' }}>/</Text>;

                            if (issue.workflowThreadUrl) {
                                parts.push(
                                    <a key="slack" href={issue.workflowThreadUrl}
                                        target="_blank" rel="noopener noreferrer"
                                        style={{ fontSize: 12, color: '#4A9FD9', textDecoration: 'none' }}
                                        onClick={(ev) => ev.stopPropagation()}>
                                        Slack
                                    </a>
                                );
                            }

                            if (issue.externalIssueKey && dashboard.externalIssueService?.url) {
                                if (parts.length > 0) parts.push(React.cloneElement(sep, { key: 'sep-jira' }));
                                parts.push(
                                    <a key="jira" href={`${dashboard.externalIssueService.url}/browse/${issue.externalIssueKey}`}
                                        target="_blank" rel="noopener noreferrer"
                                        style={{ fontSize: 12, color: '#4A9FD9', textDecoration: 'none' }}
                                        onClick={(ev) => ev.stopPropagation()}>
                                        {issue.externalIssueKey}
                                    </a>
                                );
                            }

                            if (parts.length > 0) parts.push(React.cloneElement(sep, { key: 'sep-owner' }));
                            parts.push(<Text key="owner" style={{ fontSize: 12 }}>{issue.owner?.name ?? 'Unassigned'}</Text>);

                            return parts;
                        })()}
                    </Stack>
                </Stack>
            ))}
            {issues.length > maxDisplay && (
                <Text style={{ fontSize: 10, opacity: 0.6 }}>and {issues.length - maxDisplay} more issues</Text>
            )}
        </Stack>
    );
};

const StepsSection: React.FC<{
    steps: GetLabelStepResponse[];
    logEvents: Record<string, { total: number; events: GetLogEventResponse[] }>;
    totalStepCount: number;
    jobId: string;
    expandedLogs: Set<string>;
    setExpandedLogs: (s: Set<string>) => void;
}> = ({ steps, logEvents, totalStepCount, jobId, expandedLogs, setExpandedLogs }) => {
    const maxDisplay = 8;
    const failingSteps = steps.filter(s => s.outcome === JobStepOutcome.Failure || s.outcome === JobStepOutcome.Warnings);
    const displaySteps = failingSteps.length > 0 ? failingSteps : steps;

    return (
        <Stack tokens={{ childrenGap: 4 }}>
            <Text variant="small" style={{ fontWeight: 600, opacity: 0.7 }}>
                {failingSteps.length > 0 ? 'Failing Steps' : 'Steps'}
            </Text>
            {displaySteps.slice(0, maxDisplay).map(step => {
                const stepColor = getStepStatusColor(step.state, step.outcome);
                const logEntry = step.logId ? logEvents[step.logId] : undefined;
                const isExpanded = step.logId ? expandedLogs.has(step.logId) : false;

                return (
                    <Stack key={step.id} tokens={{ childrenGap: 2 }} className={hoverRowStyles.row} style={{ padding: '2px 4px' }}>
                        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                            <div style={{ width: 8, height: 8, flexShrink: 0, backgroundColor: stepColor }} />
                            <Link to={`/job/${jobId}?step=${step.id}`}
                                style={{ fontSize: 11, color: 'inherit', overflow: 'hidden', textOverflow: 'ellipsis', whiteSpace: 'nowrap' }}>
                                {step.name}
                            </Link>
                            {logEntry && logEntry.total > 0 && (
                                <DefaultButton
                                    style={{
                                        minWidth: 0, height: 16, padding: '0 4px',
                                        fontSize: 9, backgroundColor: stepColor, color: '#fff',
                                        border: 'none', borderRadius: 2
                                    }}
                                    onClick={() => {
                                        const next = new Set(expandedLogs);
                                        if (isExpanded) { next.delete(step.logId!); } else { next.add(step.logId!); }
                                        setExpandedLogs(next);
                                    }}
                                >
                                    {logEntry.total} {step.outcome === JobStepOutcome.Warnings ? 'warning' : 'error'}{logEntry.total !== 1 ? 's' : ''}
                                </DefaultButton>
                            )}
                        </Stack>
                        {step.diagnosticMessage && (
                            <Text style={{ fontSize: 10, opacity: 0.7, paddingLeft: 12, fontStyle: 'italic' }}>
                                {step.diagnosticMessage}
                            </Text>
                        )}
                        {isExpanded && logEntry && (
                            <LogEventsInline events={logEntry.events} logId={step.logId!} total={logEntry.total} />
                        )}
                    </Stack>
                );
            })}
            {totalStepCount > maxDisplay && (
                <Text style={{ fontSize: 10, opacity: 0.6 }}>and {totalStepCount - maxDisplay} more steps</Text>
            )}
        </Stack>
    );
};

const hoverRowStyles = mergeStyleSets({
    row: {
        borderRadius: 2,
        transition: 'background-color 0.1s',
        selectors: {
            ':hover': {
                backgroundColor: dashboard.darktheme ? 'rgba(255,255,255,0.08)' : 'rgba(0,0,0,0.06)'
            }
        }
    }
});

const LogEventsInline: React.FC<{ events: GetLogEventResponse[]; logId: string; total: number }> = ({ events, logId, total }) => {
    const colors = dashboard.getStatusColors();
    const maxDisplay = 5;

    return (
        <Stack style={{ paddingLeft: 12, paddingTop: 2 }} tokens={{ childrenGap: 4 }}>
            {events.slice(0, maxDisplay).map((evt, idx) => {
                const borderColor = evt.severity === 'Error'
                    ? colors.get(StatusColor.Failure)
                    : colors.get(StatusColor.Warnings);
                const message = evt.lines?.map(line => {
                    try {
                        const parsed = typeof line === 'string' ? JSON.parse(line) : line;
                        const text = parsed?.message ?? parsed?.properties?.message;
                        if (typeof text === 'string') {
                            return text;
                        }
                        const str = String(line);
                        return str === '[object Object]' ? 'Log event' : str;
                    } catch {
                        return String(line);
                    }
                }).join('\n') ?? '';

                return (
                    <Link key={idx} to={`/log/${logId}?lineIndex=${evt.lineIndex}`}
                        style={{ textDecoration: 'none', color: 'inherit' }}>
                        <div className={hoverRowStyles.row} style={{
                            borderLeft: `4px solid ${borderColor}`,
                            paddingLeft: 8, paddingTop: 2, paddingBottom: 2,
                            fontSize: 10,
                            fontFamily: 'Horde Cousine Regular, monospace',
                            whiteSpace: 'pre-wrap', wordBreak: 'break-word',
                            maxWidth: 704
                        }}>
                            {message.substring(0, 400)}
                        </div>
                    </Link>
                );
            })}
            {total > maxDisplay && (
                <Link to={`/log/${logId}`} style={{ fontSize: 10, opacity: 0.7 }}>
                    View {total - maxDisplay} more in full log...
                </Link>
            )}
        </Stack>
    );
};

const CrossStreamSection: React.FC<{
    chain: { botName: string; entries: Array<{ streamId: string; branchName: string; jobId?: string; state?: LabelState; outcome?: LabelOutcome; hasApprovalGateBefore: boolean; isCurrent: boolean }> };
    templateName?: string;
}> = ({ chain, templateName }) => {
    const colors = dashboard.getStatusColors();

    const noDataColor = dashboard.darktheme ? '#555' : '#bbb';

    const getStreamColor = (state?: LabelState, outcome?: LabelOutcome): string => {
        if (!state || state === LabelState.Unspecified) return noDataColor;
        if (state === LabelState.Running) {
            if (outcome === LabelOutcome.Failure) return colors.get(StatusColor.Failure)!;
            if (outcome === LabelOutcome.Warnings) return colors.get(StatusColor.Warnings)!;
            return colors.get(StatusColor.Running)!;
        }
        if (outcome === LabelOutcome.Failure) return colors.get(StatusColor.Failure)!;
        if (outcome === LabelOutcome.Warnings) return colors.get(StatusColor.Warnings)!;
        if (outcome === LabelOutcome.Success) return colors.get(StatusColor.Success)!;
        return noDataColor;
    };

    return (
        <Stack tokens={{ childrenGap: 0 }}>
            <Text style={{ fontFamily: "Horde Open Sans SemiBold", fontSize: 11, opacity: 0.7, paddingBottom: 4 }}>
                Merge Flow ({templateName ?? 'template'})
            </Text>
            <div style={{ paddingTop: 4, paddingLeft: 2 }}>
                {chain.entries.map((entry, idx) => {
                    const isFirst = idx === 0;
                    const entryColor = getStreamColor(entry.state, entry.outcome);

                    const connector = !isFirst ? (
                        <div style={{ display: "flex", alignItems: "stretch", paddingLeft: 6 }}>
                            <div style={{ width: 2, backgroundColor: dashboard.darktheme ? "#444" : "#ccc", minHeight: entry.hasApprovalGateBefore ? 16 : 8 }} />
                            {entry.hasApprovalGateBefore && (
                                <Text style={{ fontSize: 9, color: '#DAA520', paddingLeft: 8, alignSelf: "center" }}>
                                    approval gate
                                </Text>
                            )}
                        </div>
                    ) : null;

                    if (entry.isCurrent) {
                        return (
                            <div key={entry.streamId}>
                                {connector}
                                <div style={{
                                    backgroundColor: dashboard.darktheme ? "#2d2d3d" : "#e8e8f0", borderRadius: 3,
                                    padding: "3px 6px 3px 0", margin: "1px -6px",
                                    display: "flex", alignItems: "center", gap: 6
                                }}>
                                    <StatusSquare color={entryColor} size={12} />
                                    <Text style={{ fontFamily: "Horde Open Sans Bold", fontSize: 13 }}>
                                        {entry.branchName}
                                    </Text>
                                    <Text variant="small" style={{ opacity: 0.6 }}>this stream</Text>
                                </div>
                            </div>
                        );
                    }

                    return (
                        <div key={entry.streamId}>
                            {connector}
                            <Link to={entry.jobId ? `/job/${entry.jobId}` : "#"}
                                style={{ pointerEvents: entry.jobId ? "auto" : "none", textDecoration: "none" }}>
                                <div style={{ display: "flex", alignItems: "center", gap: 6, padding: "1px 0" }}>
                                    <StatusSquare color={entryColor} />
                                    <Text variant="small" style={{
                                        fontFamily: "Horde Open Sans SemiBold",
                                        color: entry.jobId ? undefined : noDataColor
                                    }}>
                                        {entry.branchName}
                                    </Text>
                                </div>
                            </Link>
                        </div>
                    );
                })}
            </div>
        </Stack>
    );
};

export default LabelHoverCard;
