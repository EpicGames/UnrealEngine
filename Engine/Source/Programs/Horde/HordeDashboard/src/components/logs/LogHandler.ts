// Copyright Epic Games, Inc. All Rights Reserved.

import { action, makeObservable, observable } from "mobx";
import { getLogStyles, logMetricNormal, logMetricSmall } from "./LogStyle";
import { EventSeverity, EventData } from "horde/backend/Api";
import { LogSource } from "./LogSource";

/**
 * Handler class for Log View data like the log source, navigation state, and helper functions.
 */
export class LogHandler {
    private static instance?: LogHandler;
    
    logSource?: LogSource;
    currentLine?: number;
    trailing?: boolean;
    scroll?: number;
    initialRender = true;

    @observable
    updated = 0

    constructor() {
        makeObservable(this);
        LogHandler.instance = this;
    }

    async init(logId: string, startLine?: number) {
        this.logSource = await LogSource.create(logId, startLine);
        await this.logSource.init();

        if(!startLine && this.logSource.active) {
            this.trailing = true;
        }

        this.currentLine = startLine;
    }

    static clear() {
        this.instance?.logSource?.stop();
        this.instance = undefined;
    }

    @action
    externalUpdate() {
        this.updated++;
    }

    stopTrailing() {
        this.scroll = undefined;
        this.trailing = false;
    }

    getCurrentEvent() {
        return this.getLogEvent(this.currentLine)
    }

    getEventsOfSeverity(warnings?: boolean){
        return this.events.filter(e => {
            if (warnings && e.severity !== EventSeverity.Warning) {
                return false;
            }
            if (!warnings && e.severity !== EventSeverity.Error) {
                return false;
            }
            return true;
        });
    }

    getEventBlocks(events: EventData[]){
        return events.filter((e, i, events) => {
            // first event must be a block
            if (i == 0) {
                return true
            }
            // consider different issue a new block, regardless of spacing
            if (e.issueId != events[i-1].issueId) {
                return true
            }
            
            if (e.lineIndex == events[i-1].lineIndex + events[i-1].lineCount) {
                return false
            }
            return true
        });
    }

    getNextLogEvent(warnings?: boolean) {

        let currentLine = -1;
        if (this.currentLine !== undefined) {
            currentLine = this.currentLine - 1;
        }

        const startLine = currentLine + 1;

        const events = this.getEventsOfSeverity(warnings);

        let event = events.find(e => e.lineIndex >= startLine);

        if (!event && events.length) {
            event = events[0];
        }

        return event;
    }

    getNextLogEventBlock(warnings?: boolean) {
        
        let currentLine = -1;
        if (this.currentLine !== undefined) {
            currentLine = this.currentLine - 1;
        }

        const startLine = currentLine + 1;

        const events = this.getEventsOfSeverity(warnings);
        const eventBlocks = this.getEventBlocks(events)

        let event = eventBlocks.find(eb => eb.lineIndex >= startLine);

        if (!event && eventBlocks.length) {
            event = eventBlocks[0];
        }

        return event;
    }

    getPrevLogEvent(warnings?: boolean) {

        let currentLine = 0;
        if (this.currentLine !== undefined) {
            currentLine = this.currentLine - 1;
        }

        const startLine = currentLine - 1;

        const events = this.getEventsOfSeverity(warnings).reverse();

        if (startLine < 0) {
            if (events.length) {
                return events[0];
            }
            return undefined;
        }

        let event = events.find(e => e.lineIndex <= startLine)

        if (!event && events.length) {
            event = events[0];
        }

        return event;
    }

    getPrevLogEventBlock(warnings?: boolean) {
        
        let currentLine = -1;
        if (this.currentLine !== undefined) {
            currentLine = this.currentLine - 1;
        }

        const startLine = currentLine - 1;

        const events = this.getEventsOfSeverity(warnings);
        const eventBlocks = this.getEventBlocks(events).reverse();

        if (startLine < 0) {
            if (eventBlocks.length) {
                return eventBlocks[0];
            }
            return undefined;
        }

        let event = eventBlocks.find(eb => eb.lineIndex <= startLine);

        if (!event && eventBlocks.length) {
            event = eventBlocks[0];
        }

        return event;
    }

    getLogEvent(line: number | undefined) {

        if (line === undefined) {
            return undefined;
        }

        line--;

        return this.events.find(e => line >= e.lineIndex && line < (e.lineIndex + e.lineCount));
    }

    get events(): EventData[] {
        const events = this.logSource?.errors.map(e => e) ?? [];
        events.push(...(this.logSource?.warnings.map(e => e) ?? []))
        return events;
    }

    infoLine?: number;

    // could be a preference
    compact = false;

    get lineHeight(): number {

        if (!this.logSource?.logItems) {
            return logMetricNormal.lineHeight;
        }

        return !this.compact ? logMetricNormal.lineHeight : logMetricSmall.lineHeight;

    }

    get fontSize(): number {

        if (!this.logSource?.logItems) {
            return logMetricNormal.fontSize;
        }

        return !this.compact ? logMetricNormal.fontSize : logMetricSmall.fontSize;

    }

    get style(): any {

        const { logStyleNormal, logStyleSmall } = getLogStyles();

        if (!this.logSource?.logItems) {
            return logStyleNormal;
        }

        return !this.compact ? logStyleNormal : logStyleSmall;

    }

    get lineRenderStyle(): any {

        const { lineRenderStyleNormal, lineRenderStyleSmall } = getLogStyles();

        if (!this.logSource?.logItems) {
            return lineRenderStyleNormal;
        }

        return !this.compact ? lineRenderStyleNormal : lineRenderStyleSmall;

    }
}

