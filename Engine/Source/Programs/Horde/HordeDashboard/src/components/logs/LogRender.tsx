// Copyright Epic Games, Inc. All Rights Reserved.

import { Stack, Text } from '@fluentui/react';
import Highlight from 'react-highlighter';
import { IssueData, LogLine } from '../../backend/Api';
import backend from '../../backend';
import { NavigateFunction } from 'react-router-dom';

enum TagType {
   None,
   SourceFile,
   MSDNCode,
   Link,
   JobId,
   JobStepId,
   LogId,
   ArtifactId,
   AgentId,
   PoolId,
   LeaseId,
   ProjectId,
   StreamId,
   DeviceId,
   IgnoredLine,
}

export type LogItem = {
   line?: LogLine;
   lineNumber: number;
   requested: boolean;
   issueId?: number;
   issue?: IssueData;
}

// Provides the start of hrefs for each tag type with simple link construction behavior
const simpleTagLinks: Partial<Record<TagType, string>> = {
   [TagType.JobId]: "/job/",
   [TagType.LogId]: "/log/",
   [TagType.ProjectId]: "/project/",
   [TagType.StreamId]: "/stream/",
   [TagType.PoolId]: "/pools?poolid=",
   [TagType.DeviceId]: "/audit/device/"
}

const getLineNumber = (line: LogLine): number | undefined => {

   const properties = line.properties;
   if (!properties) {
      return undefined;
   }

   const lineProp = properties["line"];

   if (!lineProp || typeof (lineProp) !== "object") {
      return undefined;
   }

   return (lineProp as Record<string, number | undefined>).line;

}

const renderMessage = (line: LogLine, lineNumber: number | undefined, logStyle: any, search?: string) => {

   const message = line?.message ?? "";

   if (lineNumber === undefined || lineNumber === null) {
      lineNumber = getLineNumber(line);
   }

   const key = `log_line_${lineNumber}_message`;

   return <Highlight key={key} search={search ? search : ""} className={logStyle.logLine}>{message ?? ""}</Highlight >;

};


const renderTags = (navigate: NavigateFunction, line: LogLine, lineNumber: number | undefined, logStyle: any, tags: string[], search?: string) => {

   if (!line || !line.format || !line.properties) {
      return <Stack styles={{ root: { color: "#000000", paddingLeft: 8, whiteSpace: "pre", tabSize: "3" } }}>Internal log line format error</Stack>;
   }
   
   const properties = line.properties;
   const lineProp = properties["line"];

   if (lineNumber === undefined || lineNumber === null) {
      lineNumber = (lineProp as Record<string, number | undefined>)?.line ?? 0;
   }

   // render tags
   let renderedTags = tags.map((tag, index) => {

      let tagType = TagType.None;
      let property: Record<string, string | number> | string | undefined;
      let record: Record<string, string> | undefined;
      const key = `log_line_${lineNumber}_${index}`;

      let formatter = "";
      tag = tag.replace("{", "").replace("}", "");
      if (tag.indexOf(":") !== -1) {
         [tag, formatter] = tag.split(":");
      }

      property = properties[tag];

      // C# format specifiers, yes this could be more elegant
      if (formatter) {
         if (typeof (property) === "number") {

            let precision: number | undefined;

            if (formatter.startsWith("n") && formatter.length > 1) {
               precision = parseInt(formatter.slice(1));
            }

            if (formatter.startsWith("0.") && formatter.length > 2) {
               precision = 0;
               for (let i = 2; i < formatter.length; i++) {
                  if (formatter[i] === "0") {
                     precision++;
                  } else {
                     break;
                  }
               }
            }

            if (precision !== undefined && !isNaN(precision)) {
               property = (property as number).toFixed(precision);
            }

         }
      }

      let type = "";
      let text = "";

      if (property && typeof (property) !== "string" && typeof (property) !== "number" && typeof (property) !== "boolean" && property !== null) {
         record = property as Record<string, string>;
      } else {
         if (property === undefined) {
            return null;
         }
         if (property === null) {
            text = "null";
         } else {
            text = property.toString();
         }
      }

      if (record) {

         type = record.type ?? record["$type"] ?? "";
         text = record.text ?? record["$text"] ?? "";
         
         if (type === "ErrorCode" && text.startsWith("C")) {
            tagType = TagType.MSDNCode;
         }

         const enumCast = type as keyof typeof TagType;
         if(enumCast in TagType) {
            tagType = TagType[enumCast];
         }
      }

      let highlight = (
         <Highlight
            search={search ? search : ""} 
            className={logStyle.logLine} 
         >
            {text}
         </Highlight>
      )

      if(tagType === TagType.None || !record) {
         return (
            // Defined separately from highlight because of needed key prop
            <Highlight
               key={key}
               search={search ? search : ""} 
               className={logStyle.logLine} 
            >
               {text}
            </Highlight>
         )
      }

      let linkProps: React.AnchorHTMLAttributes<HTMLAnchorElement> | null = null;

      // Update link props object based on tag type, then return a link with those props
      // or a span
      switch(tagType) {
         // Cases with similar simple tag behaviors
         case TagType.JobId:
         case TagType.LogId:
         case TagType.ProjectId:
         case TagType.StreamId:
         case TagType.PoolId:
         case TagType.DeviceId:
            const url = simpleTagLinks[tagType] + text; // simpleTagLinks created for concision here
            linkProps = {
               href: url,
               onClick: (ev) => { ev.stopPropagation(); ev.preventDefault(); navigate(url) }
            }
            break;

         case TagType.JobStepId:            
            const jobId = (line as any)?.properties?.JobId?.$text
            if (jobId) {
               const stepUrl = `/job/${jobId}?step=${text}`;
               linkProps = {
                  href: stepUrl,
                  onClick: (ev) => { ev.stopPropagation(); ev.preventDefault(); navigate(stepUrl) }
               }
            }
            break;

         case TagType.MSDNCode:
            linkProps = {
               href: `https://msdn.microsoft.com/query/dev16.query?appId=Dev16IDEF1&l=EN-US&k=k(${text.toLowerCase()})&rd=true`,
               onClick: (ev) => ev.stopPropagation(),
               target: "_blank",
               rel: "noopener noreferrer"
            };
            break;

         case TagType.Link:
            linkProps = {
               href: record.target,
               onClick: (ev) => ev.stopPropagation(),
               rel: "noreferrer"
            }
            break;

         case TagType.AgentId:
            const search = new URLSearchParams(window.location.search);
            search.set("agentId", encodeURIComponent(text));
            // Update highlight because of new search in block scope
            highlight = (
               <Highlight 
                  search={search ? search : ""} 
                  className={logStyle.logLine}
               >
                  {text}
               </Highlight>
            )
            const url2 = `${window.location.pathname}?` + search.toString();
            linkProps = {
               href: url2,
               onClick: async (ev) => { ev.stopPropagation(); ev.preventDefault(); navigate(url2, { replace: true }) }
            }
            break;

         case TagType.ArtifactId:
            const artifactType = properties.ArtifactType ?? "_none";
            const search2 = new URLSearchParams(window.location.search);
            search2.set("artifactContext", encodeURIComponent(artifactType as string));
            if (properties.ArtifactId) {
               if (properties.ArtifactId["$text"]) {
                  search2.set("artifactId", encodeURIComponent(properties.ArtifactId["$text"] as string));
               }
            }
            // Update highlight because of new search in block scope
            highlight = (
               <Highlight 
                  search={search2 ? search2 : ""} 
                  className={logStyle.logLine}
               >
                  {text}
               </Highlight>
            )
            const url3 = `${window.location.pathname}?${search2.toString()}`;
            linkProps = {
               href: url3,
               onClick: async (ev) => { ev.stopPropagation(); ev.preventDefault(); navigate(url3, { replace: true }) }
            };
            break;

         case TagType.SourceFile:
            let depotPath = record.depotPath;
            if (!depotPath) break;
            if (depotPath.indexOf("@") !== -1) {
               depotPath = depotPath.slice(0, depotPath.indexOf("@"));
            }
            depotPath = encodeURIComponent(depotPath);
            // @todo: handle line, it isn't in the source file meta data, and matching isn't obvious (file vs note, etc)
            let tagLine = "";
            if (tagLine) {
               depotPath += `&line=${tagLine}`;
            }
            linkProps = {
               href: `ugs://timelapse?depotPath=${(depotPath)}`,
               onClick: (ev) => ev.stopPropagation()
            }
            break;

         case TagType.LeaseId:
            const navigateToLeaseLog = async (toplevel: boolean) => {
               const logData = await backend.getLease(text);
               const url = `/log/${logData?.logId}`;
               if (!toplevel) {
                  navigate(url)
               } else {
                  window.open(url, "_blank");
               }
            }
            linkProps = {
               href: "/",
               onClick: (ev) => {
                  ev.stopPropagation();
                  ev.preventDefault();
                  navigateToLeaseLog(!!ev?.ctrlKey || !!ev.metaKey)
               },
               onAuxClick: (ev) => {
                  ev.preventDefault();
                  ev.stopPropagation()
                  navigateToLeaseLog(true)
               }
            }
            break;

         case TagType.IgnoredLine:
            return (
               <span key={key}>
                  <span style={{ fontFamily: "Horde Open Sans SemiBold, sans-serif", fontWeight: 600 }}>IGNORED:</span> {highlight}
               </span>
            );
      }

      return linkProps ? <a key={key} {...linkProps}>{highlight}</a> : <span key={key}/>
   }).filter(t => !!t);

   let remaining = line.format;

   renderedTags = renderedTags.map((t, idx) => {

      let current = remaining;

      const tag = tags[idx];
      const index = remaining.indexOf(tag);

      remaining = remaining.slice(tag.length + (index > 0 ? index : 0));

      if (index < 0) {
         console.error("not able to find tag in format");
         return <Text>Error, unable to find tag</Text>;
      }

      if (index === 0) {
         return t;
      }

      const rtags: any = [];

      const key = `log_line_${lineNumber}_${idx}_${index}_fragment`;

      rtags.push(<Highlight key={key} search={search ? search : ""} className={logStyle.logLine}>{current.slice(0, index)}</Highlight>);
      rtags.push(t);

      return rtags;

   }).flat();

   if (remaining) {
      const key = `log_line_${lineNumber}_remaining_fragment`;
      renderedTags.push(<Highlight key={key} search={search ? search : ""} className={logStyle.logLine}>{remaining}</Highlight>)
   }

   return <div>
      {renderedTags}
   </div>;

};

export const renderLine = (navigate: NavigateFunction, line: LogLine | undefined, lineNumber: number | undefined, logStyle: any, search?: string) => {

   if (!line) {
      return null;
   }

   const tagRegex = /{[^{}]+}/g
   let format = line?.format;

   let tags: string[] = [];
   if (format) {

      // some automation logs output their own structured logging, which is marked by {{}}
      if (format.indexOf("{{") !== -1) {
         format = format.replaceAll("{", "");
         format = format.replaceAll("}", "");
      }

      const match = format.match(tagRegex);
      if (match?.length)
         tags = match;

      const properties = line.properties;

      // fix issue with tag span, we need to replace react-highlighter as it does not highlight across child nodes
      // should just be using a selector
      if (properties) {
         tags = tags.filter(t => {
            const pname = t.slice(1, -1);

            if (pname !== "WarningCode" && pname !== "WarningMessage") {
               return true;
            }

            const ptext = (properties[pname] as any);
            if (!ptext) {
               return true;
            }

            line.format = line.format?.replaceAll(t, ptext);
            return false;
         });
      }
   }

   // we don't support c# alignment, as this requireds read behinds, etc
   // so strip tags in this case and just output the line
   if (tags.find(t => t.indexOf(",") !== -1)) {
      tags = [];
   }

   if (tags.length && format && line.properties) {
      return renderTags(navigate, line, lineNumber, logStyle, tags, search);
   }

   return renderMessage(line, lineNumber, logStyle, search);

};


