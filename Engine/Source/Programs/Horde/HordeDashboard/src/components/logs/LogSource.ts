// Copyright Epic Games, Inc. All Rights Reserved.

import backend from "horde/backend";
import { LogData, EventData, IssueData, StepData, BatchData, ArtifactData, StreamData, EventSeverity, GetJobsTabResponse, IssueQuery, AgentData, LeaseData } from "horde/backend/Api";
import { PollBase } from "horde/backend/PollBase";
import { waitFor } from "horde/base/utilities/promiseUtils";
import { getStepPercent, getLeaseElapsed } from "horde/base/utilities/timeUtils";
import { observable, makeObservable, action, runInAction, computed } from "mobx";
import moment from "moment";
import { Moment } from "moment-timezone";
import { AgentTelemetryHandler } from "../agents/AgentTelemetrySparkline";
import { BreadcrumbItem } from "../Breadcrumbs";
import { errorBarStore } from "../error/ErrorStore";
import { JobDetailsV2 } from "../jobDetailsV2/JobDetailsViewCommon";
import { LogItem } from "./LogRender";
import { getStepSummaryMarkdown, getBatchSummaryMarkdown } from "horde/base/utilities/logUtils";

/**
 * Data handler parent class for job and lease logs.
 */
export abstract class LogSource extends PollBase {
    startTime: Moment;
    startLine?: number;
    agentTelemetry: AgentTelemetryHandler;

    logData: LogData;
    logId: string;
    @observable logItems: LogItem[];
    @observable private _itemsVersion = 0;
    @observable active: boolean;

    @observable fatalError?: string;

    /**
     * Factory method for creating Lease or Log Job Sources
     * @param logId The ID of the source's log
     * @returns A LogSource: JobLogSource or LeaseLogSource
    */
    static async create(logId: string, startLine?: number): Promise<LogSource> {
        let logData: LogData;
       
        try {
           logData = await backend.getLogData(logId);
        } catch (reason) {
            errorBarStore.set({title: "Error Loading Log Data", reason: reason, message: `Failed to load log data for ${logId}`});
            throw new Error(reason);
        }

        // JobId may come in as a string of 0s. This converts to boolean.
        const hasJobId = !!Number(`0x${logData.jobId}`);
        if(hasJobId) return new JobLogSource(logData, startLine);
        if(!logData.leaseId) throw new Error("Bad Lease Log");
        return new LeaseLogSource(logData, startLine);
    }

    //#region Abstract members
    
    /**
     * Downloads the log in either JSON or rest format.
     * @param json If true, downloads the log in .jsonl (JSON Lines). If false, downloads as text in .log format.
     */
    abstract download(json: boolean);

    abstract get summary(): string;

    abstract get agentId(): string | undefined;

    abstract get crumbs(): BreadcrumbItem[];

    //#endregion Abstract members

    //#region LeaseLogSource No-ops
    
    get events(): EventData[] { return [] };

    get warnings(): EventData[] { return [] };

    get errors(): EventData[] { return [] };

    get issues(): IssueData[] { return [] };

    get percentComplete(): number | undefined { return undefined };

    get crumbTitle(): string | undefined { return undefined };

    //#endregion LeaseLogSource No-ops

    @computed get itemsVersion(): number {
        return this._itemsVersion;
    }

    @action.bound
    setActive(active: boolean) {
        this.active = active;
        this.agentTelemetry.setActive(active);
    }

    @action.bound
    setFatalError(error: string) {
        this.fatalError = error;
    }
    
    constructor(logData: LogData, startLine?: number) {
        super();
        
        makeObservable(this);
        
        this.logData = logData;
        this.logId = logData.id;
        this.startLine = startLine;
        this.logItems = [];
        this.agentTelemetry = new AgentTelemetryHandler();
    }

    async init() {
        try {
            await this.updateLogData();
        } catch(reason) {
            errorBarStore.set({title: "Error Loading Log Data", reason: reason}, true)
        }
    }

    async updateLogData() {
        try {
            const newLogData = await backend.getLogData(this.logId);
            runInAction(() => {
                this.logData = newLogData;
                this.resize(this.logData.lineCount);
            })
        } catch(reason) {
            errorBarStore.set({title: "Error Updating Log Data", reason: reason}, true);
        }
    }

    /**
     * Adds a number of placeholder line items to the logItems array.
     * @param lineCount The desired total line count of the updated logItems array.
     */
    @action.bound
    resize(lineCount) {
        // LogData is updated before resizing, so use logItems.
        const count = lineCount - this.logItems.length;

        if(count <= 0) return;
        // Do not use spread operator to push placeholder items. Causes Range Error (overloads Call Stack).
        this.logItems = this.logItems.concat(this.getPlaceholderLogItems(count, this.logItems.length + 1));
        this._itemsVersion++;
    }
    
    /**
     * Creates an array of simple (placeholder/unrequested) LogItems.
     * @param count Number of placeholder items to generate.
     * @param start The starting index of the new placeholder LogItems.
     */
    getPlaceholderLogItems(count: number, start = 1): LogItem[] {
        return Array.from({length: count}, (_, i) => ({
            lineNumber: start + i,
            requested: false
        }))
    }

    /**
     * Loads log data to unrequested lines from the logItems array.
     * @param startIndex The index from which to load logItem data.
     * @param count The number of lines after startIndex to load/check.
     */
    async loadLines(startIndex: number, count: number) {
        try {
            if(!this.logData) throw new Error("Invalid/Missing Log Data");

            let anyLinesRequested = false;

            // Detect if any lines within requested range are actually unrequested.
            for (let i = 0; i < count; i++) {
                const offset = i + startIndex;
                
                if (offset >= this.logItems.length) break;

                const item = this.logItems![i + startIndex];
                if (!item.requested) {
                    item.requested = true;
                    anyLinesRequested = true;
                }
            }

            // If all specified lines are already requested, return early.
            if(!anyLinesRequested) return;

            const data = await backend.getLogLines(this.logId, startIndex, count);
            
            if(!this.startTime) throw new Error("Invalid log start time.");

            runInAction(() => {
                for(let i = 0; i < data.count; i++) {
                    const line = data.lines![i];
    
                    const offset = i + data.index;
                    if(offset >= this.logItems.length) break;
                    
                    // Whitespace adjustments
                    const stripNewlinesRegex = /(\r\n|\n|\r)/gm;
                    const fixWhitespace = (text: string | undefined): string | undefined => {
                      return text?.replace(stripNewlinesRegex, "").trimEnd();
                    };
                    line.message = fixWhitespace(line.message)!;
                    line.format = fixWhitespace(line.format);
                    
                    this.logItems[offset].line = line;
                }

                this._itemsVersion++;
            })
        } catch(reason) {
            errorBarStore.set({title: "Error Loading Line Data", reason: reason}, true)
        }
    }

    async poll() {
        if(!this.logData) return;

        if(!this.active) {
            this.stop();
        }

        try {
            await this.updateLogData();
        } catch(reason) {
            errorBarStore.set({title: "Error Polling Log Data", reason: reason}, true)
        }
    }
}

/**
 * Log source for job logs. Contains extra information like job details, events, etc.
 */
export class JobLogSource extends LogSource {
    jobDetails: JobDetailsV2;
    @observable _events: EventData[];
    _issues: IssueData[];
    jobName: string = "Unknown Job";
    projectName: string = "Unknown Project";
    change: string | number;
    step?: StepData;
    batch?: BatchData;
    artifactsV2?: ArtifactData[];
    stream?: StreamData;
    
    get events(): EventData[] {
        return this._events;
    };

    get issues(): IssueData[] {
        return this._issues;
    }

    @computed
    get warnings(): EventData[] {
        return this.events.filter(e => e.severity === EventSeverity.Warning);
    };

    @computed
    get errors(): EventData[] {
        return this.events.filter(e => e.severity === EventSeverity.Error);
    }

    get percentComplete(): number | undefined {
        if(!this.step) return undefined;

        return getStepPercent(this.step);
    }

    get summary(): string {
        if(this.step) return getStepSummaryMarkdown(this.jobDetails, this.step.id);
        if(this.batch) return getBatchSummaryMarkdown(this.jobDetails, this.batch.id);
        return "";
    }
    
    get agentId(): string | undefined {
        return this.batch?.agentId ?? this.jobDetails.batchByStepId(this.step?.id)?.agentId;
    }
    
    get crumbs(): BreadcrumbItem[] {
        if(!this.stream) return [];

        const data = this.jobDetails.jobData!;

        const tab = this.stream.tabs?.find(tab => {
            const jobTab = tab as GetJobsTabResponse;
            return !!jobTab.templates?.find(template => template === data.templateId);
        });

        let streamLink = `/stream/${this.stream.id}`;
        if(tab) streamLink += `?tab=${tab.title}`;

        return [
            {
                text: this.projectName ?? "Unknown Project",
                link: `/project/${this.stream?.project?.id}`
            },
            {
                text: this.stream.name,
                link: streamLink
            },
            {
                text: `${data?.name ?? ""} - ${this.clText}`,
                link: `/job/${data?.id}`
            },
            {
                text: this.jobName,
                link: this.step ? `/job/${data?.id}?step=${this.step?.id}` : `/job/${data?.id}?batch=${this.batch?.id}`
            },
            {
                text: this.jobName + " (Log)"
            },
        ];
    }

    get crumbTitle(): string | undefined {
        if(!this.stream) return undefined;
        
        return `Horde - ${this.clText}: ${this.jobDetails.jobData?.name ?? ""} - ${this.jobName}`;
    }

    get clText(): string {
        const data = this.jobDetails.jobData;

        if(!data) return "";

        let clText = "";
        if(data.preflightChange) {
            clText = `Preflight ${data.preflightChange} - Base ${data.change ? data.change : "Latest"}`;
        } else {
            clText = `${data.change ? "CL " + data.change : "Latest CL"}`;
        }

        return clText;
    }

    constructor(logData: LogData, startLine?: number) {
        super(logData, startLine);

        makeObservable(this);

        this._events = [];
        this._issues = [];
    }

    async init() {
        await super.init();
        this.jobDetails = new JobDetailsV2(this.logData.jobId);

        // JobDetailsV2 extends PollBase, so jobData is asynchronously fetched. Wait for that to finish.
        await waitFor(() => this.jobDetails.jobData);

        if(this.jobDetails.jobError) this.setFatalError(this.jobDetails.jobError);

        this.updateStepAndBatch();

        await this.updateArtifacts();

        this.startTime = moment.utc(this.step?.startTime ?? this.batch?.startTime);
        
        // Determines if this log source should continue polling
        this.setActive(this.getLogActive());

        this.stream = this.jobDetails.stream;

        // Determine project name
        this.projectName = this.stream?.project?.name ?? this.projectName;
        if(this.projectName === "Engine") this.projectName = "UE4";

        this.change = this.jobDetails.jobData?.change ?? "Latest";
        
        // Determine job name
        if(this.step) this.jobName = this.jobDetails.getStepName(this.step.id) ?? "Unknown Step Node";
        if(this.batch) this.jobName = `Batch-${this.batch.id}`;
        
        this.agentTelemetry.set(
            this.agentId ?? "", 
            this.startTime.toDate(), 
            this.step?.finishTime ? new Date(this.step.finishTime) : undefined
        );

        await this.updateEvents();
        await this.updateIssues();

        if(this.active) this.start();
    }

    async poll() {
        super.poll();

        await this.updateEvents();
        await this.updateIssues();
        
        this.setActive(this.getLogActive());
    }

    private async updateEvents() {
        try {
            const events = await backend.getLogEvents(this.logId);
            runInAction(() => {
                this._events = events;
            })
        } catch (reason) {
            errorBarStore.set({title: "Error Getting Events", reason: reason}, true);
        }
    }

    private async updateIssues() {
        try {
            // Update Issues
            const issueQuery: IssueQuery = { jobId: this.jobDetails.jobId, stepId: this.step?.id, label: undefined, count: 50};
            // Get both resolved and unresolved issues
            const unresolvedIssues = await backend.getIssues(issueQuery);
            const resolvedIssues = await backend.getIssues({...issueQuery, resolved: true});
            
            runInAction(() => {
                this._issues = [...unresolvedIssues, ...resolvedIssues];
            })
        } catch (reason) {
            errorBarStore.set({title: "Error Getting Issues", reason: reason}, true);
        }
    }

    private async updateArtifacts() {
        if (
            !! this.artifactsV2 
            || !this.logId
            || !this.step
            || this.getLogActive()
        ) {
            return;
        }

        const key = `job:${this.jobDetails.jobId!}/step:${this.step.id}`;
        try {
            const artifactResponse = await backend.getJobArtifactsV2(undefined, [key]);
            runInAction(() => {
                this.artifactsV2 = artifactResponse.artifacts;
            })
        } catch (reason) {
            errorBarStore.set({title: "Error Getting Artifacts", reason: reason}, true);
        }
    }

    download(json: boolean) {
        const name = `++${this.projectName}+${this.stream?.name}-CL ${this.change}-${this.jobName}.${json ? "jsonl" : "log"}`;
        backend.downloadLog(this.logId, name, json);
    }

    /**
     * Returns true if related batch or step do not have a finish time. False otherwise.
     */
    private getLogActive(): boolean {
        this.updateStepAndBatch();

        // If related batch has a finish time, log isn't active
        if(this.batch) return !this.batch.finishTime;

        // If step has a finish time, log isn't active
        if(this.step) return !this.step.finishTime;

        return false;
    }

    private updateStepAndBatch() {
        this.step = this.jobDetails.stepByLogId(this.logId);
        this.batch = this.jobDetails.batches.find(b => b.logId === this.logId);
    }
}

/**
 * Log source for lease logs. Contains extra information about leases.
 */
export class LeaseLogSource extends LogSource {
    leaseId: string;
    agent?: AgentData;
    lease?: LeaseData;

    get agentId(): string | undefined {
        return this.lease?.agentId;
    }

    get crumbs(): BreadcrumbItem[] {
        return [
            {
                text: `Agents`,
                link: '/agents'
            },
            {
                text: `${this.agent?.name}`,
                link: `/agents?agentId=${this.agent?.id}`
            },
            {
                text: `${this.lease?.name ?? this.leaseId}`,
                link: `/agents?agentId=${this.agent?.id}`
            }
        ];
    }
    
    get summary(): string {
        const duration = getLeaseElapsed(this.lease) 
        if(duration){
            return `${!!this.lease?.finishTime ? "Ran" : "Running"} on [${this.agent?.id}](?agentId=${encodeURIComponent(this.agent?.id ?? "")}) for ${duration}`;
        }

        return "";
    }

    constructor(logData: LogData, startLine?: number) {
        super(logData, startLine);

        if(!logData.leaseId) {
            errorBarStore.set({title: "Failed to Create LeaseLogSource", message: "Bad Lease ID"});
        } else {
            this.leaseId = logData.leaseId!;
        }
    }

    async init() {
        await super.init();

        this.lease = await backend.getLease(this.leaseId); 

        try {
            this.agent = await backend.getAgent(this.lease.agentId!);
        } catch (reason) {
            this.agent = {
                id: "missing-agent",
                name: "Missing Agent"
            } as AgentData;
        }

        this.leaseUpdated();

        if(this.active) this.start();
    }

    async poll() {
        if(!this.lease) {
            super.poll();
            return;
        }

        this.updateAgent();
        this.updateLease();
        
        this.leaseUpdated();

        super.poll();
    }

    async updateAgent() {
        try {
            const agent = await backend.getAgent(this.agentId!);
            runInAction(() => {
                this.agent = agent;
            })
        } catch (reason) {
            errorBarStore.set({title: "Error Updating Agent", reason: reason}, true);
        }
    }

    async updateLease() {
        try {
            const lease = await backend.getLease(this.leaseId);
            runInAction(() => {
                this.lease = lease;
            })
        } catch (reason) {
            errorBarStore.set({title: "Error Updating Lease", reason: reason}, true);
        }
    }

    /**
     * Secondary updates to run when the lease log source updates
     */
    leaseUpdated() {
        this.startTime = moment.utc(this.lease!.startTime);
        this.setActive(!this.lease!.finishTime);
        this.agentTelemetry.set(this.lease?.agentId ?? "", new Date(this.startTime as any), this.lease?.finishTime ? new Date(this.lease.finishTime) : undefined);
    }

    download(json: boolean) {
        const name = `${this.agent!.name}-Lease-${this.leaseId}.${json ? "jsonl" : "log"}`;
        backend.downloadLog(this.logId, name, json);
    }
}