// Copyright Epic Games, Inc. All Rights Reserved.

import { mergeStyleSets } from "@fluentui/react";
import dashboard from "../../backend/Dashboard";
import { DashboardPreference } from "horde/backend/Api";
import { hexToRgb, isBright } from "horde/base/utilities/colors";

export type LogMetricType = {
   lineHeight: number;
   fontSize: number;
}

export const logMetricNormal = {
   lineHeight: 24,
   fontSize: 11
}

const adjustForDisplayScale = /*(window as any).safari === undefined &&*/ window.devicePixelRatio > 1.25;

export const logMetricSmall = {
   // note, making this any larger limits log line range for overflow
   // make height even to avoid rounding issues
   lineHeight: adjustForDisplayScale ? 14 : 18,
   // don't go below 10pt for legibility
   fontSize: 10
}

let _lineRenderStyleNormal: any;
let _lineRenderStyleSmall: any;
let _logStyleBase: any;
let _logStyleNormal: any;
let _logStyleSmall: any;

export const getLogStyles = () => {

   const lineRenderStyleNormal = _lineRenderStyleNormal ?? mergeStyleSets({
      logLine: {
         padding: 0,
         height: logMetricNormal.lineHeight,
         tabSize: "3",
         fontFamily: "Horde Cousine Regular, monospace, monospace",
         fontSize: logMetricNormal.fontSize,
         whiteSpace: "pre-wrap"

      }
   });

   const lineRenderStyleSmall = _lineRenderStyleSmall ?? mergeStyleSets({
      logLine: {
         padding: 0,
         height: logMetricSmall.lineHeight,
         tabSize: "3",
         fontFamily: "Horde Cousine Regular, monospace, monospace",
         fontSize: logMetricSmall.fontSize,
         whiteSpace: "pre-wrap"

      }
   });

   const warningColorPref = dashboard.getPreference(DashboardPreference.ColorWarning);
   const errorColorPref = dashboard.getPreference(DashboardPreference.ColorError);
   const warningHighlight = warningColorPref ? isBright(warningColorPref) ? "brightness(0.9)" : "brightness(1.2)" : "";
   const errorHighlight = errorColorPref ? isBright(errorColorPref) ? "brightness(0.9)" : "brightness(1.2)" : "";

   const logStyleBase = _logStyleBase ?? mergeStyleSets({
      container: {
         overflow: 'auto',         
         marginTop: 8,
      },
      logLine: [
         {
            fontFamily: "Horde Cousine Regular, monospace, monospace"
         }
      ],
      logLineOuter: {
      },
      itemWarning: [
         {
            background: warningColorPref ? `rgb(${hexToRgb(warningColorPref)}, 0.2)` : dashboard.darktheme ? "#302402": "#FEF8E7"
         }
      ],
      errorButton: {
         backgroundColor: errorColorPref ? errorColorPref : dashboard.darktheme ? "#9D1410" : "#EC4C47",
         borderStyle: "hidden",
         color: "#FFFFFF",
         selectors: {
            ':active,:hover': {
               color: "#FFFFFF",
               backgroundColor: errorColorPref ? errorColorPref : "#DC3C37",
               filter: errorHighlight
            }
         }
      },
      errorButtonDisabled: {         
         color: dashboard.darktheme ? "#909398" : "#616E85 !important",
         backgroundColor: dashboard.darktheme ? "#1F2223" : "#f3f2f1"
      },
      warningButton: {
         backgroundColor: warningColorPref ? warningColorPref : dashboard.darktheme ? "#9D840E" : "#F7D154",
         borderStyle: "hidden",
         selectors: {
            ':active,:hover': {
               backgroundColor: warningColorPref ? warningColorPref : "#E7C144",
               filter: warningHighlight
            }
         }
      },
      warningButtonDisabled: {
         color: dashboard.darktheme ? "#909398" : undefined,
         backgroundColor: dashboard.darktheme ? "#1F2223" : "#f3f2f1"
      },
      gutter: [
         {
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 14,
            marginTop: 0,
            marginBottom: 0,
         }
      ],
      gutterError: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: errorColorPref ? errorColorPref : "#EC4C47",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0,
         }
      ],
      gutterWarning: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: warningColorPref ? warningColorPref : "#F7D154",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0,
         }
      ],
      itemError: [
         {
           background: errorColorPref ? `rgb(${hexToRgb(errorColorPref)}, 0.2)` : dashboard.darktheme ? "#330606" : "#FEF6F6",
         }
      ]
   });


   const logStyleNormal = _logStyleNormal ?? mergeStyleSets(logStyleBase, {

      container: {
         selectors: {
            '.ms-List-cell': {
               height: logMetricNormal.lineHeight,
               lineHeight: logMetricNormal.lineHeight
            }
         }
      },
      logLine: [
         {
            fontSize: logMetricNormal.fontSize,
            selectors: {
               "#infoview": {
                  opacity: 0
               },
               ":hover #infoview": {
                  opacity: 1
               },
            },
   
         }
      ],
      gutter: [
         {
            height: logMetricNormal.lineHeight
         }
      ],
      gutterError: [
         {
            height: logMetricNormal.lineHeight
         }
      ],
      gutterWarning: [
         {
            height: logMetricNormal.lineHeight
         }
      ]
   });

   const logStyleSmall = _logStyleSmall ?? mergeStyleSets(logStyleBase, {

      container: {
         selectors: {
            '.ms-List-cell': {
               height: logMetricSmall.lineHeight,
               lineHeight: logMetricSmall.lineHeight
            }
         }
      },
      logLine: [
         {
            fontSize: logMetricSmall.fontSize,
            selectors: {
               "#infoview": {
                  opacity: 0
               },
               ":hover #infoview": {
                  opacity: 1
               },
            },
   
         }
      ],
      gutter: [
         {
            height: logMetricSmall.lineHeight
         }
      ],
      gutterError: [
         {
            height: logMetricSmall.lineHeight
         }
      ],
      gutterWarning: [
         {
            height: logMetricSmall.lineHeight
         }
      ]
   });

   _lineRenderStyleNormal = lineRenderStyleNormal;
   _lineRenderStyleSmall = lineRenderStyleSmall;
   _logStyleBase = logStyleBase;
   _logStyleNormal = logStyleNormal;
   _logStyleSmall = logStyleSmall;

   return {
      lineRenderStyleNormal:lineRenderStyleNormal,
      lineRenderStyleSmall:lineRenderStyleSmall,
      logStyleBase:logStyleBase,
      logStyleNormal:logStyleNormal,
      logStyleSmall:logStyleSmall
   }
}


