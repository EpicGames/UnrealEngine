// Copyright Epic Games, Inc. All Rights Reserved.

import { Callout, DefaultButton, DetailsList, DetailsListLayoutMode, DetailsRow, DirectionalHint, Dropdown, FocusZone, FocusZoneDirection, FontIcon, IColumn, Icon, IconButton, IContextualMenuItem, IContextualMenuProps, IDetailsListProps, ITextField, List, mergeStyles, MessageBar, MessageBarType, Modal, PrimaryButton, ProgressIndicator, ScrollToMode, Selection, SelectionMode, SelectionZone, Separator, setMonth, Spinner, SpinnerSize, Stack, Text, TextField, TooltipHost } from '@fluentui/react';
import { observer } from 'mobx-react-lite';
import moment from 'moment-timezone';
import React, { useEffect, useId, useState } from 'react';
import { Link, useLocation, useNavigate, useParams } from 'react-router-dom';
import backend from '../../backend';
import { ArtifactContextType, DashboardPreference, EventSeverity, GetChangeSummaryResponse, GetJobStepRefResponse, GetLogEventResponse, LogLevel } from '../../backend/Api';
import { CommitCache } from '../../backend/CommitCache';
import dashboard from '../../backend/Dashboard';
import { Markdown } from '../../base/components/Markdown';
import { useWindowSize } from '../../base/utilities/hooks';
import { displayTimeZone, getElapsedString } from '../../base/utilities/timeUtils';
import { getHordeStyling } from '../../styles/Styles';
import { getHordeTheme } from '../../styles/theme';
import { AgentTelemetrySparkline } from "../agents/AgentTelemetrySparkline";
import { JobArtifactsModal } from '../artifacts/ArtifactsModal';
import { Breadcrumbs } from '../Breadcrumbs';
import { ChangeContextMenu, ChangeContextMenuTarget } from '../ChangeButton';
import { HistoryModal } from '../agents/HistoryModal';
import { IssueModalV2 } from '../IssueViewV2';
import { useQuery } from "horde/base/utilities/hooks";
import { LogItem, renderLine } from './LogRender';
import { StepRefStatusIcon } from '../StatusIcon';
import { TopNav } from '../TopNav';
import { isBright } from 'horde/base/utilities/colors';
import { JobLogSource, LogSource } from './LogSource';
import { JobDetailsV2 } from '../jobDetailsV2/JobDetailsViewCommon';
import { getStepMessageInfo } from '../jobDetailsV2/JobDetailsUtils';
import { LogHandler } from './LogHandler';
import { updateLineQuery, getQueryLine } from 'horde/base/utilities/logUtils';

let listRef: List | undefined;

const selection = new Selection({ selectionMode: SelectionMode.multiple });

const searchBox = React.createRef<ITextField>();

let curSearchIdx = 0;

let logListKey = 0;

// have to be available to a global keyboard handler
let globalHandler: LogHandler | undefined;
let globalSearchState: { search?: string, results?: number[], curRequest?: any, text?: string } | undefined;

const searchUp = (navigate: any) => {

   if (globalSearchState?.results?.length) {

      curSearchIdx--;
      if (curSearchIdx < 0) {
         curSearchIdx = globalSearchState.results.length - 1;
      }

      globalHandler?.stopTrailing();
      let lineIdx = globalSearchState.results[curSearchIdx];

      if (globalHandler) {
         globalHandler.currentLine = lineIdx + 1;
      }

      updateLineQuery(lineIdx + 1, navigate);

      lineIdx -= 10;
      if (lineIdx < 0) {
         lineIdx = 0;
      }

      if (globalHandler) {
         listRef?.scrollToIndex(lineIdx, (index) => { return globalHandler!.lineHeight; }, ScrollToMode.top);
      }


      globalHandler?.externalUpdate();

   }

}

const searchDown = (navigate: any) => {

   if (globalSearchState?.results?.length) {

      curSearchIdx++;
      if (curSearchIdx >= globalSearchState.results.length) {
         curSearchIdx = 0;
      }

      globalHandler?.stopTrailing();

      let lineIdx = globalSearchState.results[curSearchIdx];

      if (globalHandler) {
         globalHandler.currentLine = lineIdx + 1;
      }

      updateLineQuery(lineIdx + 1, navigate);

      lineIdx -= 10;
      if (lineIdx < 0) {
         lineIdx = 0;
      }

      if (globalHandler) {
         listRef?.scrollToIndex(lineIdx, (index) => { return globalHandler!.lineHeight; }, ScrollToMode.top);
      }

      globalHandler?.externalUpdate();

   }

}

const commitCache = new CommitCache();

const StepHistoryModal: React.FC<{ jobDetails: JobDetailsV2, stepId: string | undefined, onClose: () => void }> = observer(({ jobDetails, stepId, onClose }) => {

   const navigate = useNavigate();
   const location = useLocation();
   const [commitState, setCommitState] = useState<{ target?: ChangeContextMenuTarget, commit?: GetChangeSummaryResponse, rangeCL?: number }>({});
   const [stepHistory, setStepHistory] = useState<GetJobStepRefResponse[] | undefined>(undefined);

   const { hordeClasses } = getHordeStyling();
   const hordeTheme = getHordeTheme();

   const jobData = jobDetails.jobData;

   if (!jobData || !jobData.streamId || !jobData.templateId) {
      return null;
   }

   if (stepHistory === undefined) {
      backend.getJobStepHistory(jobData.streamId, jobDetails.getStepName(stepId, false), 1024, jobData!.templateId!).then(r => {
         setStepHistory(r);
      })

      return <Modal isOpen={true} styles={{ main: { padding: 8, width: 1084, height: '624px', backgroundColor: hordeTheme.horde.contentBackground } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
         <Stack style={{ paddingTop: 24 }} horizontalAlign='center' tokens={{ childrenGap: 18 }}>
            <Stack>
               <Text variant='mediumPlus'>Loading Step History</Text>
            </Stack>
            <Spinner size={SpinnerSize.large} />
         </Stack>
      </Modal>
   }

   // subscribe
   if (commitCache.updated) { }
   if (jobDetails.updated) { }

   if (!stepId || !stepHistory?.length) {
      return null;
   }

   const step = jobDetails.stepById(stepId);

   type HistoryItem = {
      ref: GetJobStepRefResponse;
   };

   const items: HistoryItem[] = stepHistory.map(h => {
      return { ref: h }
   });

   const columns = [
      { key: 'column1', name: 'Name', minWidth: 400, maxWidth: 400, isResizable: false },
      { key: 'column2', name: 'Change', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'column3', name: 'Started', minWidth: 180, maxWidth: 180, isResizable: false },
      { key: 'column4', name: 'Agent', minWidth: 100, maxWidth: 100, isResizable: false },
      { key: 'column5', name: 'Duration', minWidth: 110, maxWidth: 110, isResizable: false },
   ];

   const renderItem = (item: HistoryItem, index?: number, column?: IColumn) => {

      if (!column) {
         return <div />;
      }

      const ref = item.ref;


      if (column.name === "Name") {
         return <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }} >{<StepRefStatusIcon stepRef={ref} />}<Text>{step?.name}</Text></Stack>;
      }

      if (column.name === "Change") {

         if (item.ref.change && jobDetails.stream?.id) {

            return <Stack verticalAlign="center" horizontalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }} onClick={async (ev) => {
               ev?.stopPropagation();
               ev?.preventDefault();

               let commit = commitCache.getCommit(jobDetails.stream!.id, item.ref.change);

               if (!commit) {
                  await commitCache.set(jobDetails.stream!.id, [item.ref.change]);
               }

               commit = commitCache.getCommit(jobDetails.stream!.id, item.ref.change);

               if (commit) {
                  const index = stepHistory.indexOf(item.ref);
                  let rangeCL: number | undefined;
                  if (index < stepHistory.length - 1) {
                     rangeCL = stepHistory[index + 1].change;
                  }
                  setCommitState({ commit: commit, rangeCL: rangeCL, target: { point: { x: ev.clientX, y: ev.clientY } } })
               }
            }}>
               <Text style={{ color: "rgb(0, 120, 212)" }}>{item.ref.change}</Text>

            </Stack>
         }

         return null;
      }

      if (column.name === "Agent") {

         const agentId = item.ref.agentId;

         if (!agentId) {
            return null;
         }

         const url = `${location.pathname}?agentId=${agentId}`;

         return <Stack verticalAlign="center" horizontalAlign="center" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}>
            <a href={url} onClick={(ev) => { ev.preventDefault(); ev.stopPropagation(); navigate(url, { replace: true }); }}><Stack horizontal horizontalAlign={"end"} verticalFill={true} tokens={{ childrenGap: 0, padding: 0 }}><Text>{agentId}</Text></Stack></a>
         </Stack>
      }

      if (column.name === "Started") {

         if (ref.startTime) {

            const displayTime = moment(ref.startTime).tz(displayTimeZone());
            const format = dashboard.display24HourClock ? "HH:mm:ss z" : "LT z";

            let displayTimeStr = displayTime.format('MMM Do') + ` at ${displayTime.format(format)}`;


            return <Stack verticalAlign="center" horizontalAlign="start" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}>
               <Stack >{displayTimeStr}</Stack>
            </Stack>;

         } else {
            return "???";
         }
      }

      if (column.name === "Duration") {

         const start = moment(ref.startTime);
         let end = moment(Date.now());

         if (ref.finishTime) {
            end = moment(ref.finishTime);
         }
         if (item.ref.startTime) {
            const time = getElapsedString(start, end);
            return <Stack verticalAlign="center" horizontalAlign="end" tokens={{ childrenGap: 0, padding: 0 }} style={{ width: "100%", height: "100%" }}><Stack style={{ paddingRight: 8 }}><Text>{time}</Text></Stack></Stack>;
         } else {
            return "???";
         }
      }


      return <Stack />;
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as HistoryItem;
         const ref = item.ref;

         const url = `/log/${ref.logId}`;

         const commonSelectors = { ".ms-DetailsRow-cell": { "overflow": "visible", padding: 0 } };

         if (ref.stepId === stepId && ref.jobId === jobDetails.jobId) {
            props.styles = { ...props.styles, root: { background: `${hordeTheme.palette.neutralLight} !important`, selectors: { ...commonSelectors as any } } };
         } else {
            props.styles = { ...props.styles, root: { selectors: { ...commonSelectors as any } } };
         }

         return <Link to={url} onClick={(ev) => { if (!ev.ctrlKey) onClose() }}><div className="job-item"><DetailsRow {...props} /> </div></Link>;

      }
      return null;
   };

   return (<Modal isOpen={true} styles={{ main: { padding: 8, width: 1084, height: '624px', backgroundColor: hordeTheme.horde.contentBackground } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
      {commitState.target && <ChangeContextMenu target={commitState.target} job={jobDetails.jobData} commit={commitState.commit} rangeCL={commitState.rangeCL} onDismiss={() => setCommitState({})} />}
      <Stack styles={{ root: { paddingTop: 8, paddingLeft: 24, paddingRight: 12, paddingBottom: 8 } }}>
         <Stack tokens={{ childrenGap: 12 }}>
            <Stack horizontal styles={{ root: { padding: 8 } }}>
               <Stack style={{ paddingLeft: 8, paddingTop: 4 }} grow>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{"Step History"}</Text>
               </Stack>
               <Stack grow horizontalAlign="end">
                  <IconButton
                     iconProps={{ iconName: 'Cancel' }}
                     onClick={() => { onClose(); }}
                  />
               </Stack>
            </Stack>

            <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4 } }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', height: "504px" }} data-is-scrollable={true}>
                  <DetailsList
                     compact={true}
                     isHeaderVisible={false}
                     indentWidth={0}
                     items={items}
                     columns={columns}
                     setKey="set"
                     selectionMode={SelectionMode.none}
                     layoutMode={DetailsListLayoutMode.justified}
                     onRenderItemColumn={renderItem}
                     onRenderRow={renderRow}
                  />
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Modal>);
});

const LogProgressIndicator: React.FC<{ logSource: LogSource }> = observer(({ logSource }) => {

   // subscribe
   const details = (logSource as JobLogSource).jobDetails;
   if (details?.updated) { }

   const percentComplete = logSource.percentComplete;

   return <Stack>
      <ProgressIndicator percentComplete={percentComplete} barHeight={2} styles={{ root: { paddingLeft: 12, width: 300 } }} />
   </Stack>
});

const LogLineIndicator: React.FC<{ lineIndex: number }> = ({ lineIndex }) => {

   useQuery();

   if (!globalHandler) {
      return null;
   }

   let prefix = "";
   const event = globalHandler.getLogEvent(globalHandler.currentLine);

   const queryLine = getQueryLine();

   if (queryLine) {
      if (lineIndex === queryLine) {
         prefix = ">>> ";
      }
   } else if (event && lineIndex >= event.lineIndex && (lineIndex < event.lineIndex + event.lineCount)) {
      prefix = ">>> ";
   } else if (globalSearchState?.results?.length && curSearchIdx < globalSearchState.results.length) {
      if (lineIndex === globalSearchState.results[curSearchIdx]) {
         prefix = ">>> ";
      }
   }

   if (!prefix) {
      return null;
   }

   return <span>{">>>"}</span>
}

const LogList: React.FC<{ logId: string }> = observer(({ logId }) => {

   const windowSize = useWindowSize();
   const query = useQuery();
   const location = useLocation();
   const navigate = useNavigate();
   const [tsFormat, setTSFormat] = useState('relative');
   const [handler, setHandler] = useState<LogHandler | undefined>(undefined);
   const [searchState, setSearchState] = useState<{ search?: string, results?: number[], curRequest?: any, text?: string }>({});
   const [issueHistory, setIssueHistory] = useState(false);
   const [logHistory, setLogHistory] = useState(false);
   const [logTelemetry, setLogTelemetry] = useState(false);
   const [logError, setLogError] = useState<Error | undefined>(undefined);

   let [historyAgentId, setHistoryAgentId] = useState<string | undefined>(undefined);

   if (!historyAgentId && query.get("agentId")) {
      historyAgentId = query.get("agentId")!;
   }

   const artifactContext = !!query.get("artifactContext") ? query.get("artifactContext")! as ArtifactContextType : undefined;
   const artifactPath = !!query.get("artifactPath") ? query.get("artifactPath")! : undefined;
   const artifactId = !!query.get("artifactId") ? query.get("artifactId")! : undefined;

   globalHandler = handler;
   globalSearchState = searchState;
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   let inTimeout = false;

   function searchSelectedText() {
      const selection = window.getSelection();
      const selected_text = selection ? selection.toString() : '';

      if (selected_text) {
         setSearchState({ ...searchState, text: selected_text })
      }
   }

   useEffect(() => {

      const handler = (e: KeyboardEvent) => {


         if (e.keyCode === 114) {
            e.preventDefault();
            if (e.shiftKey) {
               searchUp(navigate);
            } else {
               searchDown(navigate);
            }

         }

         if ((e.ctrlKey || e.metaKey) && e.keyCode === 70) {
            e.preventDefault();

            searchSelectedText();

            if (searchState.text !== '') {
               searchBox.current?.setSelectionRange(0, searchBox.current.value!.length)
            }

            searchBox.current?.focus();
         }
      }

      window.addEventListener("keydown", handler);

      return () => {
         globalHandler = globalSearchState = undefined;
         window.removeEventListener("keydown", handler);
      };

   }, []);

   if (handler && handler.logSource && handler.logSource.logId !== logId) {
      selection.setItems([], true);
      LogHandler.clear();

      setHandler(undefined);
   }

   const hordeTheme = getHordeTheme();
   const { hordeClasses, modeColors } = getHordeStyling();

   const warningColorPref = dashboard.getPreference(DashboardPreference.ColorWarning);
   const errorColorPref = dashboard.getPreference(DashboardPreference.ColorError);
   const buttonNoneTextColor = dashboard.darktheme ? "#949898" : "#616e85";
   const warningButtonTextColor = warningColorPref ? isBright(warningColorPref) ? "#2d3f5f" : "#F9F9FB" : dashboard.darktheme ? modeColors.text : buttonNoneTextColor;
   const errorButtonTextColor = errorColorPref && isBright(errorColorPref) ? "#2d3f5f" : "#F9F9FB";

   if (!logId) {
      console.error("Bad log id settings up LogList");
      return <div>Error</div>;
   }

   if (handler?.logSource?.fatalError) {

      const error = `Error getting job data, please check that you are logged in and that the link is valid.\n\n${handler.logSource.fatalError}`;
      return (
         <Stack horizontal style={{ paddingTop: 48 }}>
            <div key={`windowsize_logview1_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - 720, flexShrink: 0, backgroundColor: hordeTheme.horde.contentBackground }} />
            <Stack horizontalAlign="center" style={{ width: 1440 }}>
               <div style={{ whiteSpace: "pre" }}> {error} </div>
            </Stack>
         </Stack>
      )

   }

   // Important: This should come before handler init below
   if (logError) {
      return (
         <Stack horizontalAlign='center' style={{ paddingTop: 24 }} >
            <Text variant='mediumPlus'>{`Unable to load log data - ${logError.message}`}</Text>
         </Stack>
      )
   }

   if (!handler) {
      let lineIndexQuery: number | undefined;
      if (query.get("lineIndex")) {
         lineIndexQuery = parseInt(query.get("lineIndex")!);
      } else if (query.get("lineindex")) {
         lineIndexQuery = parseInt(query.get("lineindex")!) + 1;
      }

      const handler = new LogHandler();
      handler.init(logId, lineIndexQuery ?? undefined).then(() => {
         setHandler(handler);
      }).catch(error => {
         // This should primarily catch errors bubbling up from logSource.create()
         setLogError(error);
      });

      return <Spinner size={SpinnerSize.large} />;
   }

   // subscribe
   if (handler.updated) { }

   const logSource = handler.logSource!;
   const logSourceIsJobLogSource = logSource instanceof JobLogSource;
   const warnings = logSource.warnings.sort((a, b) => a.lineIndex - b.lineIndex);
   const errors = logSource.errors.sort((a, b) => a.lineIndex - b.lineIndex);
   const issues = logSource.issues;

   handler.compact = !!(logSource.logItems?.length > 500000);

   const onRenderCell = (item?: LogItem, index?: number, isScrolling?: boolean): JSX.Element => {

      const LogCell: React.FC = observer(() => {

         if (handler.updated) { }

         if (!item) {
            return (<div style={{ height: handler.lineHeight }} />);
         }

         const styles = handler.style;

         if (!item.requested) {
            let index = item.lineNumber - 50;
            if (index < 0) {
               index = 0;
            }

            logSource.loadLines(index, 100);

            return <Stack key={`key_log_line_${item.lineNumber}`} style={{ width: "max-content", height: handler.lineHeight }}>
               <div style={{ position: "relative" }}>
                  <Stack className={styles.logLine} tokens={{ childrenGap: 8 }} horizontal disableShrink={true}>
                     <Stack styles={{ root: { color: "#c0c0c0", width: 48, textAlign: "right", userSelect: "none", fontSize: handler.fontSize } }}>{item.lineNumber}</Stack>
                     <Stack className={styles.logLine} horizontal disableShrink={true} >
                        <Stack className={styles.gutter}></Stack>
                        <Stack styles={{ root: { color: "#8a8a8a", width: 82, whiteSpace: "nowrap", fontSize: handler.fontSize, userSelect: "none" } }}> </Stack>
                        <div className={styles.logLineOuter}> <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8 } }}> </Stack></div>
                     </Stack>
                  </Stack>
               </div>
            </Stack>

         }

         const line = item.line;
         const warning = line?.level === LogLevel.Warning;
         const error = (line?.level === LogLevel.Error) || (line?.level === LogLevel.Critical);

         const ev = logSource.events.find(event => event.lineIndex === item.lineNumber - 1);

         if (ev && ev.issueId) {
            item.issueId = ev.issueId;
            item.issue = issues.find(issue => issue.id === ev.issueId);
         }

         const style = warning ? styles.itemWarning : error ? styles.itemError : styles.logLine;
         const gutterStyle = warning ? styles.gutterWarning : error ? styles.gutterError : styles.gutter;

         let timestamp = "";
         let tsWidth = 82;
         if (line && line.time && logSource.startTime) {

            if (tsFormat === 'relative') {
               const start = logSource.startTime;
               const end = moment.utc(line.time);
               if (end >= start) {
                  const duration = moment.duration(end.diff(start));
                  const hours = Math.floor(duration.asHours());
                  const minutes = duration.minutes();
                  const seconds = duration.seconds();
                  timestamp = `[${hours <= 9 ? "0" + hours : hours}:${minutes <= 9 ? "0" + minutes : minutes}:${seconds <= 9 ? "0" + seconds : seconds}]`;
               } else {
                  timestamp = "[00:00:00]";
               }
            }

            if (tsFormat === 'utc' || tsFormat === 'local') {
               tsWidth = 112;
               let tm = moment.utc(line.time);
               if (tsFormat === 'local') {
                  tm = tm.local();
               }
               timestamp = `[${tm.format("MMM DD HH:mm:ss")}]`;
            }
         }

         const IssueButton: React.FC<{ item: LogItem, event: GetLogEventResponse }> = ({ item, event }) => {

            const tooltipId = useId();

            const error = event.severity === EventSeverity.Error;

            const issueId = item!.issueId!.toString();

            const newQuery = query.getCopy();
            newQuery.set("issue", issueId);

            // this href is for when log is copy/pasted, note that we need the onClick handler when on site, so don't reload page
            const href = `${location.pathname}?${newQuery.toString()}`;

            const fontStyle = {
               fontFamily: "Horde Open Sans SemiBold, sans-serif, sans-serif", color: error ? errorButtonTextColor : warningButtonTextColor, fontSize: handler.fontSize, textDecoration: !item!.issue?.resolvedAt ? undefined : "line-through"
            }

            return <Stack style={{ width: 80 }}>
               <TooltipHost
                  content={timestamp}
                  id={tooltipId}
                  calloutProps={{ gapSpace: 0 }}
                  styles={{ root: { display: 'inline-block', width: 80 } }}>
                  <DefaultButton className={error ? styles.errorButton : styles.warningButton}
                     href={href}
                     style={{ padding: 0, margin: 0, minWidth: 65, width: 65, paddingLeft: 4, paddingRight: 6, height: "100%", fontWeight: "unset" }}
                     onClick={(ev) => {
                        ev.preventDefault();
                        ev.stopPropagation();
                        location.search = `?issue=${issueId}`
                        setIssueHistory(true);
                        navigate(location);
                     }}>
                     <Text variant="small" style={{ ...fontStyle }}>{`${issueId}`}</Text>
                  </DefaultButton>
               </TooltipHost>
            </Stack>
         }

         const eyeColor = modeColors.text + "44";

         return (
            <Stack key={`key_log_line_${item.lineNumber}`} style={{ width: "max-content", height: handler.lineHeight }}
               onMouseEnter={(ev) => {

                  if (item.line?.time) {
                     let time = item.line?.time;
                     if (!time.endsWith("Z")) {
                        time += "Z";
                     }
                     logSource?.agentTelemetry?.setCurrentTime(new Date(time));
                  }
               }}
               onClick={() => {
                  updateLineQuery(item.lineNumber, navigate);
                  handler.currentLine = item.lineNumber;
               }}>
               <div style={{ position: "relative" }}>
                  <Stack className={styles.logLine} style={{ position: "relative" }} tokens={{ childrenGap: 8 }} horizontal disableShrink={true}>
                     <Stack horizontal styles={{ root: { color: "#c0c0c0", width: 80, textAlign: "right", userSelect: "none", fontSize: handler.fontSize } }}>
                        <LogLineIndicator lineIndex={item.lineNumber} />
                        <Stack styles={{ root: { color: "#c0c0c0", width: 80, textAlign: "right", userSelect: "none", fontSize: handler.fontSize } }}>{item.lineNumber}</Stack>
                     </Stack>
                     <Stack className={style} horizontal disableShrink={true}>
                        <Stack className={gutterStyle}></Stack>
                        {(!item.issueId || !ev) && <Stack styles={{ root: { color: "#8a8a8a", width: tsWidth, whiteSpace: "nowrap", fontSize: handler.fontSize, userSelect: "none" } }}> {timestamp}</Stack>}
                        {!!item.issueId && !!ev && <IssueButton item={item} event={ev!} />}
                        <div className={styles.logLineOuter}> <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8, position: "relative", verticalAlign: "center" } }}> {renderLine(navigate, item.line, item.lineNumber, handler.lineRenderStyle, searchState.search)}
                           <Stack id={`callout_target_${item?.lineNumber}`} style={{ position: "absolute", cursor: "pointer", userSelect: "none", left: "-12px", top: "0px", zIndex: 100 }} onClick={() => {
                              handler.infoLine = item.lineNumber;
                              handler.externalUpdate();
                           }}><FontIcon id="infoview" style={{ fontSize: 14, color: eyeColor }} iconName="Eye" /></Stack>
                           {handler.infoLine === item.lineNumber && <Callout
                              styles={{ root: { padding: "32px 24px", maxWidth: 1300 } }}
                              role="dialog"
                              gapSpace={4}
                              target={`#callout_target_${item?.lineNumber}`}
                              isBeakVisible={true}
                              beakWidth={12}
                              onDismiss={() => {
                                 handler.infoLine = undefined;
                                 handler.externalUpdate();
                              }}
                              directionalHint={DirectionalHint.bottomCenter}
                              setInitialFocus>
                              <Stack style={{ maxWidth: 1140 }}>
                                 <Stack style={{ paddingBottom: 24 }}>
                                    <Text style={{ fontSize: 14, fontFamily: "Horde Open Sans SemiBold" }}>Structured Log Line</Text>
                                 </Stack>
                                 <Stack style={{ paddingLeft: 12 }}>
                                    <Text style={{ fontSize: 11, whiteSpace: "pre-wrap", fontFamily: "Horde Cousine Regular" }}>{JSON.stringify(item.line, undefined, 2).replaceAll("\\r", "").replaceAll("\\n", "\n")}</Text>
                                 </Stack>
                              </Stack>
                           </Callout>}

                        </Stack>
                        </div>
                     </Stack>
                  </Stack>
               </div>
            </Stack>

         );
      });

      return <LogCell />
   };

   if (handler.trailing) {
      listRef?.scrollToIndex(logSource.logData!.lineCount - 1, undefined, ScrollToMode.bottom);
   }

   const downloadProps: IContextualMenuProps = {
      items: [
         {
            key: 'downloadLog',
            text: 'Download Text',
            onClick: () => {
               logSource.download(false);
            }
         },
         {
            key: 'downloadJson',
            text: 'Download JSON',
            onClick: () => {
               logSource.download(true);
            }
         },
      ],
      directionalHint: DirectionalHint.bottomRightEdge
   };

   let summaryText = logSource.summary;

   let largeLogWarning = logSource.logItems?.length > 1500000;
   let largeLogWarningText = `Warning: large log file (${logSource.logItems?.length} lines) may not display entirely.  Please download to view full log and consider reducing verbosity.`;

   let baseUrl = location.pathname;

   /**
    * Executes a log search query and updates UI state accordingly.
    * 
    * @todo Improve logic, flow, and commenting for this function.
    */
   const doQuery = (newValue: string) => {

      let currentLine = 0
      if (handler.currentLine !== undefined) {
         currentLine = handler.currentLine
      }

      if (!newValue.trim()) {

         logListKey++;

         setSearchState({
            search: '',
            curRequest: undefined,
            results: undefined,
            text: ''
         });

         // Reset current line and scroll to value before search was cleared
         handler.stopTrailing();
         let lineIdx = currentLine - 10;
         if (lineIdx < 0) {
            lineIdx = 0;
         }

         updateLineQuery(currentLine, navigate);

         inTimeout = true;
         setTimeout(() => {
            inTimeout = false; listRef?.scrollToIndex(lineIdx, () => { return handler.lineHeight; }, ScrollToMode.top);
         }, 10)

      } else if (newValue !== searchState.search) {

         const request = backend.searchLog(logId, newValue, 0, 65535);
         let newState = {
            search: newValue,
            curRequest: request,
            text: newValue
         }

         request.then(logResponse => {

            if (newState.curRequest === request) {

               const lines = logResponse.lines;

               if (lines.length) {

                  logListKey++;

                  // Get next occurrence after current line
                  curSearchIdx = 0;
                  for (const [index, value] of lines.entries()) {
                     if (value + 1 >= currentLine) {
                        curSearchIdx = index
                        break
                     }
                  }

                  handler.currentLine = lines[curSearchIdx] + 1

                  setSearchState({
                     search: newValue,
                     curRequest: undefined,
                     results: lines,
                     text: newValue
                  });


                  handler.stopTrailing();
                  let lineIdx = lines[curSearchIdx] - 10;
                  if (lineIdx < 0) {
                     lineIdx = 0;
                  }

                  updateLineQuery(lines[curSearchIdx] + 1, navigate);

                  // oof
                  inTimeout = true;
                  setTimeout(() => {
                     inTimeout = false;
                     listRef?.scrollToIndex(lineIdx, () => { return handler.lineHeight; }, ScrollToMode.top);
                  }, 250)

               } else {
                  setSearchState({
                     search: newValue,
                     curRequest: undefined,
                     results: lines,
                     text: newValue
                  });

               }

            };
         });

         setSearchState(newState);

      } else {

         curSearchIdx = 0;

         if (searchState.results?.length) {

            updateLineQuery(searchState.results[curSearchIdx] + 1, navigate);

            handler.stopTrailing();
            let lineIdx = searchState.results[curSearchIdx] - 10;
            if (lineIdx < 0) {
               lineIdx = 0;
            }
            listRef?.scrollToIndex(lineIdx, (index) => { return handler.lineHeight; }, ScrollToMode.top);
         }

      }

   }

   //#region View Btn Menu Props

   const menuProps: IContextualMenuProps = { items: [] };

   if (logSourceIsJobLogSource) {
      const stepArtifacts = (logSource as JobLogSource).artifactsV2;

      const atypes = new Map<ArtifactContextType, number>();
      const knownTypes = new Set<string>(["step-saved", "step-output", "step-trace"]);

      stepArtifacts?.forEach(a => {
         let c = atypes.get(a.type) ?? 0;
         c++;
         atypes.set(a.type, c);
      });

      const opsList: IContextualMenuItem[] = [];

      const navigateToArtifacts = (context: string) => {
         const search = new URLSearchParams(window.location.search);
         search.set("artifactContext", encodeURIComponent(context));
         const url = `${window.location.pathname}?` + search.toString();
         navigate(url, { replace: true })
      }

      opsList.push({
         key: 'stepops_artifacts_step',
         text: "Logs",
         iconProps: { iconName: "Folder" },
         disabled: !atypes.get("step-saved"),
         onClick: () => {
            navigateToArtifacts("step-saved");
         }
      });

      opsList.push({
         key: 'stepops_artifacts_output',
         text: "Temp Storage",
         iconProps: { iconName: "MenuOpen" },
         disabled: !atypes.get("step-output"),
         onClick: () => {
            navigateToArtifacts("step-output");
         }
      });

      opsList.push({
         key: 'stepops_artifacts_trace',
         text: "Traces",
         iconProps: { iconName: "SearchTemplate" },
         disabled: !atypes.get("step-trace"),
         onClick: () => {
            navigateToArtifacts("step-trace");
         }
      });

      const custom = stepArtifacts?.filter(a => !knownTypes.has(a.type)).sort((a, b) => a.type.localeCompare(b.type));
      custom?.forEach(c => {
         opsList.push({
            key: `stepops_artifacts_${c.type}`,
            text: c.description ?? c.name,
            iconProps: { iconName: "Clean" },
            onClick: () => { navigateToArtifacts(c.type) }
         });
      })


      menuProps.items.push({
         key: 'jobstep_artifacts',
         text: `Artifacts`,
         subMenuProps: {
            items: opsList
         }
      })
   }

   menuProps.items.push({
      key: 'jobstep_history',
      text: 'Step History',
      onClick: () => setLogHistory(true)
   })

   if (logSource.agentTelemetry) {
      menuProps.items.push({
         key: 'jobstep_agent_telemetry',
         text: logTelemetry ? 'Hide Telemetry' : 'Show Telemetry',
         onClick: () => {
            logSource.agentTelemetry?.show(!logTelemetry);
            setLogTelemetry(!logTelemetry)
         }
      })
   }

   //#endregion View Btn Menu Props

   //#region Log Navigation Functions

   function updateEvent() {

      const event = handler?.getCurrentEvent();
      if (!handler || !event) {
         return;
      }

      handler.stopTrailing();

      updateLineQuery(event.lineIndex + 1, navigate);
      handler.externalUpdate();
      listRef?.scrollToIndex(event.lineIndex - 10 < 0 ? 0 : event.lineIndex - 10, () => handler.lineHeight, ScrollToMode.top);
   }

   function prevError() {

      if (!handler) {
         return;
      }

      const event = handler.getPrevLogEvent();

      if (event) {
         handler.currentLine = event.lineIndex + 1;
      }

      updateEvent();

   }

   function prevErrorBlock() {

      if (!handler) {
         return;
      }

      const eventBlock = handler.getPrevLogEventBlock();

      if (eventBlock) {
         handler.currentLine = eventBlock.lineIndex + 1;
      }

      updateEvent();
   }

   function nextError() {

      if (!handler) {
         return;
      }

      const event = handler.getNextLogEvent();

      if (event) {
         handler.currentLine = event.lineIndex + 1;
      }

      updateEvent();
   }

   function nextErrorBlock() {

      if (!handler) {
         return;
      }

      const eventBlock = handler.getNextLogEventBlock();

      if (eventBlock) {
         handler.currentLine = eventBlock.lineIndex + 1;
      }

      updateEvent();
   }

   function prevWarning() {

      if (!handler) {
         return;
      }

      const event = handler.getPrevLogEvent(true);

      if (event) {
         handler.currentLine = event.lineIndex + 1;
      }

      updateEvent();

   }

   function prevWarningBlock() {

      if (!handler) {
         return;
      }

      const eventBlock = handler.getPrevLogEventBlock(true);

      if (eventBlock) {
         handler.currentLine = eventBlock.lineIndex + 1;
      }

      updateEvent();

   }

   function nextWarning() {
      if (!handler) {
         return;
      }

      const event = handler.getNextLogEvent(true);

      if (event) {
         handler.currentLine = event.lineIndex + 1;
      }

      updateEvent();

   }

   function nextWarningBlock() {
      if (!handler) {
         return;
      }

      const eventBlock = handler.getNextLogEventBlock(true);

      if (eventBlock) {
         handler.currentLine = eventBlock.lineIndex + 1;
      }

      updateEvent();

   }

   //#endregion Log Navigation Functions

   let warningsText = "";
   let errorText = "";

   if (warnings?.length) {
      warningsText = warnings.length.toString();
      if (warnings.length >= 49) {
         warningsText += "+";
      }
   } else {
      warningsText = "No";
   }

   if (errors?.length) {
      errorText = errors.length.toString();
      if (errors.length >= 49) {
         errorText += "+";
      }
   } else {
      errorText = "No";
   }


   let heightAdjust = 285;
   if (logTelemetry) {
      // sparkline height
      heightAdjust += 238;
   }
   if (largeLogWarning) {
      heightAdjust += 55
   }

   const nextErrorEnabled: boolean = !!handler?.getNextLogEvent();
   const prevErrorEnabled: boolean = !!handler?.getPrevLogEvent();
   const nextErrorBlockEnabled: boolean = !!handler?.getNextLogEventBlock();
   const prevErrorBlockEnabled: boolean = !!handler?.getPrevLogEventBlock();
   const nextWarningEnabled: boolean = !!handler?.getNextLogEvent(true);
   const prevWarningEnabled: boolean = !!handler?.getPrevLogEvent(true);
   const nextWarningBlockEnabled: boolean = !!handler?.getNextLogEventBlock(true);
   const prevWarningBlockEnabled: boolean = !!handler?.getPrevLogEventBlock(true);

   let jobDetails: JobDetailsV2 | undefined;
   if (logSourceIsJobLogSource) {
      jobDetails = (logSource as JobLogSource).jobDetails;
   }

   const report = jobDetails?.stepByLogId(logId)?.reports?.find(r => !!r.message);
   const { message: stepMessage, color: stepMessageColor } = getStepMessageInfo(report);

   return <Stack>
      {logSourceIsJobLogSource && <IssueModalV2 issueId={query.get("issue")} popHistoryOnClose={issueHistory} />}

      {
         logSourceIsJobLogSource
         && logHistory
         &&
         <StepHistoryModal
            jobDetails={jobDetails!}
            stepId={jobDetails!.stepByLogId(logId)?.id}
            onClose={() => setLogHistory(false)}
         />
      }

      {
         logSourceIsJobLogSource
         && !!artifactContext
         && <JobArtifactsModal
            jobId={jobDetails!.jobData!.id}
            stepId={jobDetails!.stepByLogId(logId)?.id!}
            artifacts={(logSource as JobLogSource).artifactsV2}
            contextType={artifactContext} artifactPath={artifactPath}
            artifactId={artifactId}
            onClose={() => { navigate(window.location.pathname, { replace: true }) }}
         />
      }

      {!!historyAgentId && <HistoryModal agentId={historyAgentId} onDismiss={() => { navigate(baseUrl, { replace: true }); setHistoryAgentId(undefined) }} />}

      <Breadcrumbs items={logSource?.crumbs ?? []} title={logSource?.crumbTitle} />

      <Stack tokens={{ childrenGap: 0 }} style={{ backgroundColor: hordeTheme.horde.contentBackground, paddingTop: 12 }}>
         <Stack horizontal >
            <div key={`windowsize_logview1_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0 }} />
            <Stack tokens={{ childrenGap: 0, maxWidth: 1440 }} disableShrink={true} styles={{ root: { width: "100%", backgroundColor: hordeTheme.horde.contentBackground, paddingLeft: 4, paddingRight: 24, paddingTop: 12 } }}>
               <Stack horizontal style={{ paddingBottom: 4 }}>
                  <Stack className={hordeClasses.button} horizontal horizontalAlign={"start"} verticalAlign="center" tokens={{ childrenGap: 8 }}>
                     <Stack horizontal tokens={{ childrenGap: 2 }}>

                        <Stack horizontal verticalAlign='center'>
                           <Stack horizontal verticalAlign='center' style={{ position: 'relative' }}>
                              <TooltipHost content={nextErrorEnabled ? 'Navigate to next error' : ''}>
                                 <DefaultButton disabled={!nextErrorEnabled} className={nextErrorEnabled ? handler.style.errorButton : handler.style.errorButtonDisabled}
                                    text={`${errorText} ${errors.length === 1 ? "Error" : "Errors"}`}
                                    onClick={() => {
                                       nextError();
                                    }}
                                    style={{ color: errors.length ? errorButtonTextColor : buttonNoneTextColor, height: 30, fontSize: 19, paddingLeft: errors.length ? 12 : 8, borderRadius: errors.length ? '4px 0 0 4px' : '4px' }}>
                                    {!!errors.length && <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />}
                                 </DefaultButton>
                              </TooltipHost>
                              {!!errors.length && <div style={{ backgroundColor: errorButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />}
                           </Stack>
                           {!!errors.length && <TooltipHost content={'Navigate to previous error'}>
                              <IconButton disabled={!prevErrorEnabled} className={prevErrorEnabled ? handler.style.errorButton : handler.style.errorButtonDisabled} style={{ color: errorButtonTextColor, height: 30, fontSize: 19, padding: 2, borderRadius: '0 4px 4px 0' }} iconProps={{ iconName: 'ChevronUp' }} onClick={(event: any) => {
                                 event?.stopPropagation();
                                 prevError();
                              }} />
                           </TooltipHost>}
                        </Stack>

                        {!!errors.length && <Stack horizontal verticalAlign='center' style={{ paddingLeft: 1 }} >
                           <Stack horizontal verticalAlign='center' style={{ position: 'relative' }}>
                              <TooltipHost content={'Skip to next block of errors or issue'}>
                                 <IconButton disabled={!nextErrorBlockEnabled} className={nextErrorBlockEnabled ? handler.style.errorButton : handler.style.errorButtonDisabled}
                                    style={{ color: errorButtonTextColor, height: 30, fontSize: 19, padding: 9, borderRadius: '4px 0 0 4px' }}
                                    styles={{ rootPressed: { border: 'none', boxShadow: 'none' } }}
                                    iconProps={{ iconName: 'DoubleChevronDown' }}
                                    onClick={() => {
                                       nextErrorBlock();
                                    }} />
                              </TooltipHost>
                              <div style={{ backgroundColor: errorButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }}></div>
                           </Stack>
                           <TooltipHost content={'Skip to previous block of errors or issue'}>
                              <IconButton disabled={!prevErrorBlockEnabled} className={prevErrorBlockEnabled ? handler.style.errorButton : handler.style.errorButtonDisabled}
                                 style={{ color: errorButtonTextColor, height: 30, fontSize: 19, padding: 9, borderRadius: '0 4px 4px 0' }}
                                 iconProps={{ iconName: 'DoubleChevronUp' }}
                                 onClick={(event: any) => {
                                    event?.stopPropagation();
                                    prevErrorBlock();
                                 }} />
                           </TooltipHost>
                        </Stack>}


                        <Stack horizontal verticalAlign='center' style={{ paddingLeft: 12 }} >
                           <Stack horizontal verticalAlign='center' style={{ position: 'relative' }}>
                              <TooltipHost content={nextWarningEnabled ? 'Navigate to next warning' : ''}>
                                 <DefaultButton disabled={!nextWarningEnabled} className={nextWarningEnabled ? handler.style.warningButton : handler.style.warningButtonDisabled}
                                    text={`${warningsText} ${warnings.length === 1 ? "Warning" : "Warnings"}`}
                                    onClick={() => {
                                       nextWarning();
                                    }}
                                    style={{ color: warnings.length ? warningButtonTextColor : buttonNoneTextColor, height: 30, fontSize: 19, paddingLeft: warnings.length ? 12 : 8, borderRadius: warnings.length ? '4px 0 0 4px' : '4px' }} >
                                    {!!warnings.length && <Icon style={{ fontSize: 19, paddingLeft: 4, paddingRight: 0 }} iconName='ChevronDown' />}
                                 </DefaultButton>
                              </TooltipHost>
                              {!!warnings.length && <div style={{ backgroundColor: warningButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }} />}
                           </Stack>
                           {!!warnings.length && <Stack>
                              <TooltipHost content={'Navigate to previous warning'}>
                                 <IconButton disabled={!prevWarningEnabled} className={prevWarningEnabled ? handler.style.warningButton : handler.style.warningButtonDisabled} style={{ height: 30, fontSize: 19, padding: 2, color: warningButtonTextColor, borderRadius: '0 4px 4px 0' }} iconProps={{ iconName: 'ChevronUp' }} onClick={(event: any) => {
                                    event?.stopPropagation();
                                    prevWarning();
                                 }} />
                              </TooltipHost>
                           </Stack>}
                        </Stack>

                        {!!warnings.length && <Stack horizontal verticalAlign='center' style={{ paddingLeft: 1 }} >
                           <Stack horizontal verticalAlign='center' style={{ position: 'relative' }}>
                              <TooltipHost content={'Skip to next block of warnings or issue'}>
                                 <IconButton disabled={!nextWarningBlockEnabled} className={nextWarningBlockEnabled ? handler.style.warningButton : handler.style.warningButtonDisabled}
                                    style={{ color: warningButtonTextColor, height: 30, fontSize: 19, padding: 9, borderRadius: '4px 0 0 4px' }}
                                    styles={{ rootPressed: { border: 'none', boxShadow: 'none' } }}
                                    iconProps={{ iconName: 'DoubleChevronDown' }}
                                    onClick={() => {
                                       nextWarningBlock();
                                    }} />
                              </TooltipHost>
                              <div style={{ backgroundColor: warningButtonTextColor, height: 18, width: 1, position: 'absolute', right: -0.5, zIndex: 1 }}></div>
                           </Stack>
                           <TooltipHost content={'Skip to previous block of warnings or issue'}>
                              <IconButton disabled={!prevWarningBlockEnabled} className={prevWarningBlockEnabled ? handler.style.warningButton : handler.style.warningButtonDisabled}
                                 style={{ color: warningButtonTextColor, height: 30, fontSize: 19, padding: 9, borderRadius: '0 4px 4px 0' }}
                                 iconProps={{ iconName: 'DoubleChevronUp' }}
                                 onClick={(event: any) => {
                                    event?.stopPropagation();
                                    prevWarningBlock();
                                 }} />
                           </TooltipHost>
                        </Stack>}

                        {!!menuProps.items.find(i => !i.disabled) && <Stack style={{ paddingLeft: 18 }}><PrimaryButton disabled={!menuProps.items.find(i => !i.disabled)} text="View" menuProps={menuProps} style={{ fontFamily: "Horde Open Sans SemiBold", borderStyle: "hidden", padding: 15 }} /></Stack>}
                        {!menuProps.items.find(i => !i.disabled) && <Stack style={{ paddingLeft: 18 }}><DefaultButton disabled={true} className={handler.style.warningButtonDisabled} text="View" style={{ color: "rgb(97, 110, 133)", padding: 15 }} /> </Stack>}

                     </Stack>
                     <Stack>
                        {logSource.active && <LogProgressIndicator logSource={logSource} />}
                        {!!stepMessage && <Stack horizontalAlign='center'>
                           <Stack style={{ paddingLeft: 4 }} horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                              <Text style={{ fontSize: 11, fontFamily: "Horde Open Sans Semibold" }}>Step:</Text>
                              <Stack horizontal verticalFill verticalAlign="center" style={{ paddingTop: 2 }} tokens={{ childrenGap: 4 }}>
                                 {!!stepMessageColor && <FontIcon style={{ fontSize: 13, color: stepMessageColor, paddingTop: 1 }} iconName="Square" />}
                                 <Text style={{ maxWidth: 190, fontSize: 11, fontFamily: "Horde Open Sans Semibold", overflow: "hidden", textOverflow: "ellipsis" }}>{stepMessage}</Text>
                              </Stack>
                           </Stack>
                        </Stack>}

                     </Stack>
                  </Stack>

                  <Stack grow />
                  <Stack horizontalAlign={"end"}>
                     <Stack>
                        <Stack verticalAlign="center" horizontal tokens={{ childrenGap: 24 }} styles={{ root: { paddingRight: 4, paddingTop: 4 } }}>
                           <Stack horizontal tokens={{ childrenGap: 24 }}>
                              <Stack horizontal tokens={{ childrenGap: 0 }}>
                                 {!!searchState.curRequest && <Spinner style={{ paddingRight: 8 }} />}

                                 <TextField
                                    spellCheck={false}
                                    autoComplete="off"
                                    deferredValidationTime={1500}
                                    validateOnLoad={false}
                                    componentRef={searchBox}
                                    value={searchState.text ? searchState.text : ''}
                                    onChange={(ev, value) => { setSearchState({ ...searchState, text: value }); }}

                                    onRenderSuffix={
                                       !!searchState.text ? () => (
                                          <Stack verticalAlign="center" horizontalAlign="center" horizontal tokens={{ childrenGap: 2 }} styles={{ root: { padding: 0 } }}>
                                             <IconButton iconProps={{ iconName: 'Cancel' }} styles={{ root: { padding: 12, height: 20, width: 20 }, icon: { fontSize: 15 } }}
                                                onClick={() => { doQuery('') }}
                                             />
                                             <span className={mergeStyles({ paddingLeft: 1, paddingRight: 7, paddingBottom: 1 })}>
                                                {searchState?.results ? searchState?.results?.length ? `${curSearchIdx + 1}/${searchState?.results?.length}` : '0/0' : undefined}
                                             </span>
                                          </Stack>
                                       ) : undefined
                                    }

                                    styles={{
                                       root: { width: 280, fontSize: 12 },
                                       fieldGroup: { borderWidth: 1 },
                                       suffix: { padding: '0 2px', height: 30 }
                                    }}
                                    placeholder="Search"

                                    onKeyDown={(ev) => {

                                       if (ev.key === "Enter" && !searchState.curRequest && !inTimeout) {
                                          if (searchState.text === searchState.search) {
                                             searchDown(navigate);
                                          } else {
                                             doQuery(searchState.text ?? "");
                                          }
                                       }

                                    }}

                                    onGetErrorMessage={(newValue) => {
                                       if (searchState.text !== searchState.search) {
                                          doQuery(newValue);
                                       }
                                       return undefined;
                                    }}

                                 />
                                 <Stack horizontal style={{ borderWidth: 1, borderStyle: "solid", borderColor: dashboard.darktheme ? "#3F3F3F" : "rgb(96, 94, 92)", height: 32, borderLeft: 0 }}>
                                    <IconButton style={{ height: 30 }} iconProps={{ iconName: 'ChevronUp' }} onClick={(event: any) => {
                                       searchUp(navigate);
                                    }} />
                                    <IconButton style={{ height: 30 }} iconProps={{ iconName: 'ChevronDown' }} onClick={(event: any) => {
                                       searchDown(navigate);
                                    }} />

                                 </Stack>
                              </Stack>

                              <Dropdown
                                 styles={{ root: { width: 92 } }}
                                 options={[{ key: 'relative', text: 'Relative' }, { key: 'local', text: 'Local' }, { key: 'utc', text: 'UTC' }]}
                                 defaultSelectedKey={tsFormat}
                                 onChanged={(value) => {
                                    setTSFormat(value.key as string);
                                    listRef?.forceUpdate();
                                 }}
                              />

                              <PrimaryButton
                                 text="Download"
                                 split
                                 onClick={() => logSource.download(false)}
                                 menuProps={downloadProps}
                                 style={{ fontFamily: "Horde Open Sans SemiBold" }}
                              />
                           </Stack>
                        </Stack>
                     </Stack>
                  </Stack>

               </Stack>
               <Stack>
                  <Stack style={{ paddingTop: 12 }}>
                     <Separator styles={{ root: { fontSize: 0, width: "100%", padding: 0, selectors: { '::before': { background: dashboard.darktheme ? '#313638' : '#D3D2D1' } } } }} />
                  </Stack>

                  <Stack style={{ paddingBottom: 12, paddingTop: 12, fontSize: 12, fontFamily: "Horde Open Sans Regular" }}>
                     <Stack style={{ paddingLeft: 4 }} horizontal verticalAlign="center" tokens={{ childrenGap: 12 }}>
                        <Text style={{ fontSize: 12, fontFamily: "Horde Open Sans Semibold" }}>Summary:</Text>
                        <Markdown>{summaryText}</Markdown>
                     </Stack>
                  </Stack>
                  {logTelemetry && !!logSource.agentTelemetry && <AgentTelemetrySparkline handler={logSource.agentTelemetry} />}
                  {largeLogWarning && <Stack style={{ paddingBottom: 12, paddingTop: 12, margin: 'auto' }}>
                     <MessageBar
                        messageBarType={MessageBarType.warning} isMultiline={false} >
                        <Markdown styles={{ root: { fontSize: 12, fontFamily: "Horde Open Sans Bold" } }}>{largeLogWarningText}</Markdown>
                     </MessageBar>
                  </Stack>}
                  <Stack horizontalAlign="center" style={{ paddingBottom: 12 }}>
                     <Separator styles={{ root: { fontSize: 0, width: "100%", padding: 0, selectors: { '::before': { background: dashboard.darktheme ? '#313638' : '#D3D2D1' } } } }} />
                  </Stack>
               </Stack>

            </Stack>
         </Stack>


         <Stack style={{ backgroundColor: hordeTheme.horde.contentBackground, paddingLeft: "24px", paddingRight: "24px" }}>
            <Stack tokens={{ childrenGap: 0 }}>
               <FocusZone direction={FocusZoneDirection.vertical} isInnerZoneKeystroke={() => { return true; }} defaultActiveElement="#LogList" style={{ padding: 0, margin: 0 }} >
                  <div className={handler.style.container} data-is-scrollable={true} style={{ height: `calc(100vh - ${heightAdjust}px)` }}
                     onScroll={(ev) => {

                        const element: any = ev.target;

                        const scroll = Math.ceil(element.scrollHeight - element.scrollTop);

                        if (scroll < (element.clientHeight + 1)) {
                           handler.scroll = undefined;
                           handler.trailing = true;
                        } else {

                           if (handler.scroll === undefined) {
                              handler.scroll = scroll;
                           } else if (handler.scroll !== undefined && (handler.scroll < scroll || scroll > (element.clientHeight + 1))) {
                              handler.stopTrailing();
                           }
                        }


                     }}>
                     <Stack horizontal>
                        {!dashboard.leftAlignLog && <div key={`windowsize_logview2_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2) - 48, flexShrink: 0 }} />}
                        <Stack styles={{ root: { "backgroundColor": hordeTheme.horde.contentBackground, paddingLeft: "0px", paddingRight: "0px" } }}>
                           <SelectionZone selection={selection} selectionMode={SelectionMode.multiple}>
                              <List key={`log_list_key_${logListKey}`} version={logSource.itemsVersion} id="LogList" ref={(list: List) => { listRef = list; }}
                                 items={logSource.logItems}
                                 // NOTE: getPageSpecification breaks initial scrollToIndex when query contains lineIndex!
                                 getPageHeight={() => 10 * (handler.lineHeight)}
                                 onShouldVirtualize={() => { return true; }}
                                 onRenderCell={onRenderCell}
                                 onPagesUpdated={() => {
                                    if (handler.initialRender && listRef) {
                                       handler.initialRender = false;
                                       if (logSource?.startLine !== undefined) {

                                          listRef?.scrollToIndex(logSource?.startLine - 10 < 0 ? 0 : logSource?.startLine - 10, () => handler.lineHeight, ScrollToMode.top);
                                       } else if (handler.trailing) {
                                          listRef.scrollToIndex(logSource.logData!.lineCount - 1, undefined, ScrollToMode.bottom);
                                       }
                                    }
                                 }}

                                 data-is-focusable={true} />
                           </SelectionZone>
                        </Stack>
                     </Stack>
                  </div>
               </FocusZone>
            </Stack>
         </Stack>

      </Stack>
   </Stack>;

});

export const LogView: React.FC = () => {

   const { logId } = useParams<{ logId: string }>();

   useEffect(() => {
      return () => {
         selection.setItems([], true);
         LogHandler.clear();
      };
   }, []);

   const { hordeClasses } = getHordeStyling();


   if (!logId) {
      console.error("Bad log id to LogView");
   }

   return (
      <Stack className={hordeClasses.horde}>
         <TopNav />
         <LogList logId={logId!} />
      </Stack>
   );
};
