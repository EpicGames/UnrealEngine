// Copyright Epic Games, Inc. All Rights Reserved.

import { ComboBox, FontIcon, IComboBoxOption, IconButton, MaskedTextField, Modal, PrimaryButton, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { useState } from "react";
import { useNavigate } from "react-router-dom";
import backend from "../../backend";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { getHordeStyling } from "../../styles/Styles";

class PerforceServerHandler {

   async load(): Promise<void> {

      try {

         this.loading = true;
         const servers = await backend.getPerforceServerStatus();
         const unique = new Set(servers.map(s => s.cluster));
         this.options = [];
         unique.forEach(c => {
            this.options?.push({
               key: `${c}`,
               text: c
            })
         })           
         
         this.options = this.options?.sort((a, b) => {
            return a.text.localeCompare(b.text);
         })

      } catch (err) {
         console.error(err);
         this.options = [];
      } finally {
         this.loading = false;
      }
   }

   loading = false;
   options?: IComboBoxOption[];
}

const handler = new PerforceServerHandler();


const PreflightConfigPanel: React.FC = () => {

   const search = new URL(window.location.toString()).searchParams;
   const shelvedChange = search.get("preflightconfig") ? search.get("preflightconfig")! : undefined;
   const queryCluster = search.get("cluster") ? search.get("cluster")! : undefined;

   const navigate = useNavigate();
   const [state, setState] = useState<{ initialCL?: string, submitting?: boolean, success?: boolean, message?: string, warnings?: string[], cluster?: string }>({ initialCL: shelvedChange, cluster: queryCluster === undefined ? "Default" : queryCluster });
   const [servers, setServers] = useState(handler.options);

   if (servers === undefined) {
      if (!handler.loading) {
         handler.load().then(() => setServers(handler.options))
      }
      return null;
   }

   const maskFormat: { [key: string]: RegExp } = {
      '*': /[0-9]/,
   };

   const checkPreflight = async (preflightCL?: string) => {

      if (!preflightCL) {

         if (!shelvedChange) {
            return;
         }

         preflightCL = shelvedChange;
      }

      setState({ submitting: true, cluster: state.cluster });

      try {
         const response = await backend.checkPreflightConfig(parseInt(preflightCL), state.cluster === "Default" ? undefined : state.cluster);
         setState({ submitting: false, success: response.result, message: response.message, warnings: response.warnings });
      } catch (error) {
         setState({ submitting: false, success: false, message: error as string });
      }

   }

   if (state.initialCL && (state.cluster === "Default" || !!queryCluster)) {
      checkPreflight(state.initialCL);
      return null;
   }

   return (<Stack style={{ flex: 1, minHeight: 0 }}>
      <Stack styles={{ root: { paddingTop: 18, paddingLeft: 0, paddingRight: 0, width: "100%", flex: 1, minHeight: 0 } }} >
         <Stack tokens={{ childrenGap: 12 }} style={{ flex: 1, minHeight: 0, display: "flex" }}>
            <Stack style={{ width: 800, paddingLeft: 12, flex: 1, minHeight: 0, display: "flex", flexDirection: "column" }}>
               <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 24 }}>
                  <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 12 }}>
                     <Text styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Shelved Change</Text>
                     <MaskedTextField placeholder="Shelved Change" mask="***********" maskFormat={maskFormat} maskChar="" value={shelvedChange} onChange={(ev, newValue) => {
                        ev.preventDefault();

                        if (!newValue) {
                           navigate("/index?preflightconfig", { replace: true });
                        }

                        if (!isNaN(parseInt(newValue!))) {
                           navigate(`/index?preflightconfig=${newValue}`, { replace: true });
                        }

                     }} />
                  </Stack>

                  {(handler.options!.length > 1) && <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 12 }}>
                     <Text styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Cluster</Text>
                     <ComboBox styles={{ root: { width: 180 } }} disabled={state.submitting} selectedKey={state.cluster} options={handler.options!} onChange={(ev, option, index, value) => {
                        setState({ ...state, cluster: option?.text })
                     }} />
                  </Stack>}

                  <Stack grow />
                  {!!state.submitting && <Stack>
                     <Spinner size={SpinnerSize.large} />
                  </Stack>}

                  {state.success === true && <Stack>
                     <FontIcon style={{ color: dashboard.getStatusColors().get(StatusColor.Success)!, fontSize: 24 }} iconName="Tick" />
                  </Stack>}

                  {state.success === false && <Stack>
                     <FontIcon style={{ color: dashboard.getStatusColors().get(StatusColor.Failure)!, fontSize: 24 }} iconName="Cross" />

                  </Stack>}

                  <Stack>
                     <PrimaryButton disabled={!shelvedChange || state.submitting} styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }} onClick={async () => {
                        checkPreflight(shelvedChange);
                     }}>Check</PrimaryButton>
                  </Stack>
               </Stack>
               {(!!state.message || !!state.warnings?.length) && <Stack style={{ paddingTop: 18, paddingRight: 2, paddingBottom: 18, flex: 1, minHeight: 0 }}>
                  <Stack style={{ paddingBottom: 12 }}>
                     <Text styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{state.success === false ? "Error Message" : "Result"}</Text>
                  </Stack>
                  <Stack style={{ border: "1px solid #CDCBC9", flex: 1, minHeight: 0, overflowY: "auto", padding: 18 }} tokens={{ childrenGap: 18 }}>
                     {!!state.message && <Text style={{ whiteSpace: "pre-wrap" }}>{state.message}</Text>}
                     {!!state.warnings?.length && (() => {
                        const warningColor = dashboard.getStatusColors().get(StatusColor.Warnings)!;
                        return <Stack tokens={{ childrenGap: 8 }}>
                           <Text styles={{ root: { fontFamily: "Horde Open Sans SemiBold", color: warningColor } }}>Warnings</Text>
                           <Stack style={{ border: `1px solid ${warningColor}`, backgroundColor: warningColor + "1A", padding: 12 }} tokens={{ childrenGap: 8 }}>
                              {state.warnings.map((warning, index) => (
                                 <Text key={index} style={{ whiteSpace: "pre-wrap" }}>{warning}</Text>
                              ))}
                           </Stack>
                        </Stack>;
                     })()}
                  </Stack>
               </Stack>}
            </Stack>
         </Stack>
      </Stack>
   </Stack>)
}


export const PreflightConfigModal: React.FC<{ onClose: () => void }> = ({ onClose }) => {

   const { hordeClasses } = getHordeStyling();

   return <Stack>
      <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 880, height: 720, hasBeenOpened: false, top: "80px", position: "absolute" } }} onDismiss={() => onClose()} className={hordeClasses.modal}>
         <Stack className="horde-no-darktheme" styles={{ root: { paddingTop: 10, paddingRight: 0 } }}>
            <Stack style={{ paddingLeft: 24, paddingRight: 12, flex: 1, minHeight: 0 }}>
               <Stack tokens={{ childrenGap: 12 }} style={{ height: 700, display: "flex", flexDirection: "column" }}>
                  <Stack horizontal verticalAlign="start">
                     <Stack style={{ paddingTop: 3 }}>
                        <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Preflight Configuration</Text>
                     </Stack>
                     <Stack grow />
                     <Stack horizontalAlign="end" >
                        <IconButton
                           iconProps={{ iconName: 'Cancel' }}
                           onClick={() => { onClose() }}
                        />
                     </Stack>
                  </Stack>
                  <Stack styles={{ root: { paddingLeft: 4, paddingRight: 0, paddingTop: 8, paddingBottom: 4, flex: 1, minHeight: 0 } }}>
                     <PreflightConfigPanel />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Modal>
   </Stack>;
};
