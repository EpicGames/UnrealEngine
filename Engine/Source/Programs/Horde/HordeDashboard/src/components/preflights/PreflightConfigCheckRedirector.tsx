// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react';
import { Navigate } from 'react-router-dom';
import { useQuery } from "horde/base/utilities/hooks";
import { errorDialogStore } from '../error/ErrorStore';

function setError(message: string) {

   console.error(message);

   message = `${message}\n\nReferring URL: ${encodeURI(window.location.href)}`

   errorDialogStore.set({

      title: `Error handling Preflight Configuration`,
      message: message

   }, true);

}

// redirect from external source, where horde stream id, etc are not known by that application
export const PreflightConfigRedirector: React.FC = () => {

   const query = useQuery();

   const shelvedCL = !query.get("shelvedchange") ? "" : query.get("shelvedchange")!;
   
   if (!shelvedCL) { 
      setError("No shelved change specified");
   } else {
      return <Navigate to={`/index?preflightconfig=${shelvedCL}`} replace={true} />
   }

   return null;

}