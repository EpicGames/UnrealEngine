// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, IContextualMenuItem, IContextualMenuProps, IRefObject, Stack } from "@fluentui/react"
import { projectStore } from "../../backend/ProjectStore"
import { useImperativeHandle, useState } from "react";
import React from "react";


export interface IStreamChooser {
   streamId?: string;
}

export interface IStreamChooserProps {
   defaultStreamId?: string;
   allowAll?: boolean;
   width?: number;
   onChange?: (streamId: string | undefined) => void;

}

export const streamIdAll = "stream_id_all";

export const StreamChooser = React.forwardRef<IStreamChooser, IStreamChooserProps>((props, ref) => {

   const [chosen, setChosen] = useState(props?.defaultStreamId ?? streamIdAll);

   useImperativeHandle(ref, () => ({
      streamId: chosen,
   }));

   const projectIds = projectStore.projects.sort((a, b) => a.name.localeCompare(b.name)).map(p => p.id);

   let projectOptions: IContextualMenuItem[] = [];

   projectIds.forEach(pid => {

      const project = projectStore.projects.find(p => p.id === pid)!

      const streams = project.streams?.sort((a, b) => {
         return a.name.localeCompare(b.name);
      })

      const catItems: IContextualMenuItem[] = project.categories?.map(c => {
         return {
            key: c.name,
            text: c.name,
            data: c,
            subMenuProps: { items: [] }
         }
      }) ?? [];

      const streamItems: IContextualMenuItem[] = [];

      streams?.forEach(s => {

         const streamItem = {
            key: s.id,
            text: s.fullname ?? s.name,
            onClick: () => {
               setChosen(s.id);
               if (props.onChange) {
                  props.onChange(s.id);
               }
            }
         };

         const c = catItems.find(c => c.data.streams.find(sid => sid === s.id));
         if (c) {
            c.subMenuProps!.items.push(streamItem);
         } else {
            streamItems.push(streamItem)
         }
      });

      projectOptions.push({
         key: `project_key_${pid}`,
         text: project.name,
         subMenuProps: { items: catItems.length ? catItems : streamItems }
      })

   });

   if (props.allowAll) {
      projectOptions.unshift({
         key: `project_key_all`,
         text: "All Streams",
         onClick: () => {
            setChosen(streamIdAll);
            if (props.onChange) {
               props.onChange(streamIdAll);
            }
         }
      })
   }

   const templateMenuProps: IContextualMenuProps = {
      shouldFocusOnMount: true,
      subMenuHoverDelay: 0,
      items: projectOptions,
   };

   let text = "Choose Stream";

   if (chosen) {
      if (chosen === streamIdAll) {
         text = "All Streams";
      } else {
         const stream = projectStore.streamById(chosen);
         if (stream) {
            text = stream.fullname ?? stream.name;
         }
      }
   }

   return <div>
      <DefaultButton style={{ width: props.width ?? 320, textAlign: "left" }} menuProps={templateMenuProps} text={text} />
   </div>
})