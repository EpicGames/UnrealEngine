// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, FontIcon, IColumn, SelectionMode, Stack, Text, TooltipHost } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect } from "react";
import backend from "../../backend";
import { ServerStatusResponse, ServerStatusResult, ServerStatusSubsystem } from "../../backend/Api";
import { PollBase } from "../../backend/PollBase";
import { useWindowSize } from "../../base/utilities/hooks";
import { getShortNiceTime } from "../../base/utilities/timeUtils";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import dashboard, { StatusColor } from "../../backend/Dashboard";
import { Link } from 'react-router-dom';

class ServerHandler extends PollBase {

   constructor(pollTime = 5000) {

      super(pollTime);

   }

   clear() {
      this.status = undefined;
      super.stop();
   }

   async poll(): Promise<void> {

      try {

         this.status = await backend.getServerStatus();

         this.setUpdated();

      } catch (err) {

      }

   }

   status?: ServerStatusResponse;
}

const handler = new ServerHandler();

const ServerPanel: React.FC = observer(() => {

   useEffect(() => {

      handler.start();

      return () => {
         handler.clear();
      };

   }, []);

   const { modeColors, hordeClasses } = getHordeStyling();

   // subscribe
   if (handler.updated) { };

   if (!handler.status) {
      return null;
   }

   const statusColors = dashboard.getStatusColors();
   const perforceServerUrl = "/perforce/servers"

   const columns = [
      {
         key: 'column_service', name: 'Service', fieldName: 'name', minWidth: 180, maxWidth: 180, isResizable: false, onRender: (status: ServerStatusSubsystem) => {

            let color = statusColors.get(StatusColor.Skipped)!;

            const update = status.updates.length > 0 ? status.updates[0] : undefined;
            if (update) {
               if (update.result === ServerStatusResult.Healthy) {
                  color = statusColors.get(StatusColor.Success)!;
               }
               if (update.result === ServerStatusResult.Unhealthy) {
                  color = statusColors.get(StatusColor.Failure)!;
               }
               if (update.result === ServerStatusResult.Degraded) {
                  color = statusColors.get(StatusColor.Warnings)!;
               }
            }

            return <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 6 }} verticalFill>
               <FontIcon style={{ color: color, fontSize: 13 }} iconName="Square" />
               {status.name === "Perforce" ? <Link to={perforceServerUrl}><Text variant="small">{status.name}</Text></Link> : <Text variant="small">{status.name}</Text>}
            </Stack>

         }
      },
      {
         key: 'column_status', name: 'Status', minWidth: 180, maxWidth: 180, isResizable: false, onRender: (status: ServerStatusSubsystem) => {

            const update = status.updates.length > 0 ? status.updates[0] : undefined;
            if (!update) {
               return <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
                  <Text>No Status</Text>
               </Stack>;
            }
            return <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
               <Text>{update.result}</Text>
            </Stack>;
         }
      },
      {
         key: 'column_time', name: 'Time', minWidth: 240, maxWidth: 240, isResizable: false, onRender: (status: ServerStatusSubsystem) => {
            const update = status.updates.length > 0 ? status.updates[0] : undefined;
            if (!update) {
               return null;
            }

            return <Stack horizontalAlign="start" verticalAlign="center" verticalFill>
               <Text>{getShortNiceTime(update.updatedAt, true, true, true)}</Text>
            </Stack>;
         }
      }, {
         key: 'column_message', name: 'Message', minWidth: 120, isResizable: false, isMultline: true, onRender: (status: ServerStatusSubsystem) => {
            const update = status.updates.length > 0 ? status.updates[0] : undefined;
            if (!update) {
               return null;
            }

            if (!update.message?.length) {
               return null;
            }

            return <TooltipHost content={update.message} styles={{ root: { display: 'inline-block' } }} ><Stack horizontalAlign="start" verticalAlign="center" verticalFill>
               {status.name === "Perforce" ? <Text>{update.message.replace(/\.$/, ",") ?? ""} see <Link to={perforceServerUrl}>server status</Link>.</Text> : <Text>{update.message ?? ""}</Text>}
            </Stack></TooltipHost>;
         }
      }
   ];

   let status = handler.status?.statuses;

   status = status.sort((a, b) => {
      return a.name.localeCompare(b.name)
   });

   const renderItem = (item: any, index?: number, column?: IColumn) => {
      if (!column?.fieldName) {
         return null;
      }
      return <Text style={{ color: modeColors.text }}>{item[column?.fieldName]}</Text>
   };

   return (<Stack className={hordeClasses.raised} >
      <Stack styles={{ root: { paddingLeft: 12, paddingRight: 12, paddingBottom: 12, width: "100%" } }} >
         <Stack>
            <DetailsList
               items={status}
               columns={columns}
               selectionMode={SelectionMode.none}
               layoutMode={DetailsListLayoutMode.justified}
               compact={true}
               onRenderItemColumn={renderItem}
            />
         </Stack>
      </Stack>
   </Stack>);
});


export const ServerStatusView: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
   const centerAlign = vw / 2 - 720;

   const { hordeClasses, modeColors } = getHordeStyling();

   const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Status' }]} />
      <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
         <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
            <Stack style={{ position: "relative", width: "100%", height: 'calc(100vh - 148px)' }}>
               <div style={{ overflowX: "auto", overflowY: "visible" }}>
                  <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                     <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                     <Stack style={{ width: 1440 }}>
                        <ServerPanel />
                     </Stack>
                  </Stack>
               </div>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

