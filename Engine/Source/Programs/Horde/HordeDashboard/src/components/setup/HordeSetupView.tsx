// Copyright Epic Games, Inc. All Rights Reserved.

import { ThemeProvider, Stack, Image, TextField, PrimaryButton, MessageBar, MessageBarType } from "@fluentui/react";
import dashboard from "../../backend/Dashboard";
import { darkTheme } from "../../styles/darkTheme";
import { lightTheme } from "../../styles/lightTheme";
import { useState } from "react";


export const HordeSetupView: React.FC = () => {

   const [state, setState] = useState<{ error?: string, submitting?: boolean }>({});
   const [secrets, setSecrets] = useState<{ newPassword?: string, confirmPassword?: string }>({});
   
   const error = state.error;

   const username = "Admin";

   document.body.setAttribute('style', `background: ${dashboard.darktheme ? "#0F0F0F" : "#FFFFFFF"}`)   

   const onLoginAdmin = async () => {

      const password = secrets.newPassword?.trim();

      try {

         const request: RequestInit = {
            method: "POST",
            mode: "cors",
            cache: "no-cache",
            credentials: "same-origin",
            headers: { "Content-Type": "application/json" },
            redirect: "follow",
            body: JSON.stringify({
               username: username,
               password: password,
            })
         };

         const response = await fetch("/account/login/dashboard", request);

         if (!response.ok || response.status !== 200 || !response.redirected) {

            let error = `Problem logging in: ${response.status}`;

            if (response.status === 403) {
               error = "Invalid username or password";
            } 

            setState({ ...state, error: error, submitting: false });
            return;
         }

         window.location.assign("/docs/Landing.md");

      } catch (error) {
         setState({ ...state, error: `Problem logging in ${error}`, submitting: false });
      }
   }

   const onSetPassword = async () => {

      const password = secrets.newPassword?.trim() ?? "";
      const cpassword = secrets.confirmPassword?.trim() ?? "";

      if (cpassword !== password) {
         setState({ ...state, error: "Passwords don't match, please correct and try again"});
         return;
      }

      try {

         const request: RequestInit = {
            method: "POST",
            mode: "cors",
            cache: "no-cache",
            credentials: "same-origin",
            headers: { "Content-Type": "application/json" },
            redirect: "follow",
            body: JSON.stringify({
               password: password,
            })
         };

         const response = await fetch("/api/v1/accounts/admin/create", request);

         if (!response.ok || response.status !== 200) {

            setState({ ...state, error: `Problem setting admin account password: ${response.status}`, submitting: false });
            return;
         }

         onLoginAdmin();


      } catch (error) {
         setState({ ...state, error: `Problem setting admin account password: ${error}`, submitting: false });
      }

   }


   return (<ThemeProvider applyTo='element' theme={dashboard.darktheme ? darkTheme : lightTheme}>
      <div style={{ position: 'absolute', left: '50%', top: '50%', transform: 'translate(-50%, -50%)' }}>
         <Stack>
            <Stack horizontalAlign="center">
               <Stack styles={{ root: { paddingTop: 2 } }}>
                  <Image shouldFadeIn={false} shouldStartVisible={true} src={`/images/${dashboard.darktheme ? "unreal_horde_logo_dark_mode.png" : "unreal_horde_logo.png"}`} />
               </Stack>

               <Stack style={{ width: 420 }} tokens={{ childrenGap: 12 }}>
                  {!!error && <Stack>
                     <MessageBar key={`validation_error`} messageBarType={MessageBarType.error} isMultiline={false}>{error}</MessageBar>
                  </Stack>}

                  <Stack style={{ padding: 8 }}>
                     <TextField label={"Username"} value="Admin" disabled/>
                  </Stack>

                  <Stack style={{ padding: 8 }}>
                     <TextField label={"Set Password"} autoComplete="new-password" placeholder="Optional" spellCheck={false} type="password" defaultValue={secrets.newPassword} canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, newPassword: value ?? "" }) }} />
                  </Stack>

                  <Stack style={{ padding: 8 }}>
                     <TextField label={"Confirm Password"} autoComplete="new-password" spellCheck={false} placeholder="" type="password" defaultValue={secrets.confirmPassword} canRevealPassword onChange={(ev, value) => { setSecrets({ ...secrets, confirmPassword: value ?? "" }) }} />
                  </Stack>

                  <Stack style={{ padding: 8 }}>
                     <PrimaryButton disabled={state.submitting} text="Login" onClick={() => {
                        setState({ ...state, error: undefined, submitting: true });
                        onSetPassword();
                     }} />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </div>
   </ThemeProvider>);

}