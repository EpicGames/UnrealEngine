// Copyright Epic Games, Inc. All Rights Reserved.

import { ComboBox, DetailsList, DetailsListLayoutMode, FontIcon, IColumn, IComboBoxOption, IconButton, MessageBar, MessageBarType, Modal, PrimaryButton, SelectionMode, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import Markdown from "markdown-to-jsx";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import { Sparklines, SparklinesLine, SparklinesReferenceLine } from "react-sparklines";
import backend from "../../backend";
import { GetStorageDashboardResponse, StorageBackendInfo, StorageDashboardStatus, StorageGcMetricEntry, StorageNamespaceStatus, CollectionStatsSnapshot, CollectionStat } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { PollBase } from "../../backend/PollBase";
import { useWindowSize } from "../../base/utilities/hooks";
import { getShortNiceTime } from "../../base/utilities/timeUtils";
import { formatBytes } from "../../base/utilities/stringUtills";
import { getHordeStyling } from "../../styles/Styles";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";

// ─── Time Range ───

type TimeSelection = {
    text: string;
    key: string;
    minutes: number;
}

const timeSelections: TimeSelection[] = [
    { text: "Last 1 Hour", key: "time_1_hour", minutes: 60 },
    { text: "Last 4 Hours", key: "time_4_hours", minutes: 60 * 4 },
    { text: "Last 24 Hours", key: "time_24_hours", minutes: 60 * 24 },
    { text: "Last 7 Days", key: "time_7_days", minutes: 60 * 24 * 7 },
    { text: "Last 30 Days", key: "time_30_days", minutes: 60 * 24 * 30 },
];

function getTimeRange(selection: TimeSelection): { minTime: Date; maxTime: Date } {
    const now = new Date();
    return { minTime: new Date(now.getTime() - selection.minutes * 60 * 1000), maxTime: now };
}

function getTrendCount(selection: TimeSelection): number {
    if (selection.minutes <= 60) return 60;
    if (selection.minutes <= 240) return 100;
    return 500;
}

// ─── Handler ───

class StorageHandler extends PollBase {
    data?: GetStorageDashboardResponse;
    lastUpdated?: Date;
    error?: string;
    timeSelection: TimeSelection = timeSelections[2]; // default: 24h

    constructor() {
        super(30000); // 30s poll
    }

    clear() {
        this.data = undefined;
        this.error = undefined;
        super.stop();
    }

    setTimeSelection(selection: TimeSelection) {
        this.timeSelection = selection;
        this.data = undefined;
        this.forceUpdate();
    }

    async poll(): Promise<void> {
        try {
            const range = getTimeRange(this.timeSelection);
            const result = await backend.getStorageDashboard(range.minTime, range.maxTime, getTrendCount(this.timeSelection));
            this.data = result;
            this.lastUpdated = new Date();
            this.error = undefined;
            this.setUpdated();
        } catch (err) {
            this.error = err instanceof Error ? err.message : "Failed to load storage dashboard";
            console.error("Storage dashboard poll failed:", err);
            this.setUpdated();
        }
    }
}

const handler = new StorageHandler();

// ─── Helpers ───

function formatCount(n: number): string {
    if (n >= 1_000_000_000) return `${(n / 1_000_000_000).toFixed(1)}B`;
    if (n >= 1_000_000) return `${(n / 1_000_000).toFixed(1)}M`;
    if (n >= 1_000) return `${(n / 1_000).toFixed(1)}K`;
    return n.toLocaleString();
}

function formatNumber(n: number): string {
    return n.toLocaleString();
}

function queueDepthColor(depth: number): string {
    if (depth > 10000) return "#d13438";
    if (depth > 1000) return "#fce100";
    return "#107c10";
}

function fragmentationColor(ratio: number): string {
    if (ratio > 2.0) return "#d13438";
    if (ratio > 1.5) return "#fce100";
    return "#107c10";
}

// ─── GC Config Panel ───

const GcConfigPanel: React.FC<{ status: StorageDashboardStatus }> = ({ status }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    const boolIcon = (value: boolean) => (
        <FontIcon iconName="CircleFill" style={{ color: value ? "#107c10" : "#d13438", fontSize: 10, marginRight: 2 }} />
    );

    const configItem = (label: string, value: React.ReactNode, hint: string) => (
        <Stack style={{ marginBottom: 12, minWidth: 280 }}>
            <Stack horizontal verticalAlign="center">
                <Text variant="mediumPlus" style={{ fontWeight: 600, color: modeColors.text }}>{label}:&nbsp;</Text>
                <Text variant="mediumPlus" style={{ color: modeColors.text, marginLeft: 4 }}>{value}</Text>
            </Stack>
            <Text variant="small" style={{ color: modeColors.text, opacity: 0.6, marginTop: 2 }}>{hint}</Text>
        </Stack>
    );

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>GC Configuration</Text>
            <Stack horizontal wrap tokens={{ childrenGap: 24 }}>
                <Stack style={{ flex: 1, minWidth: 280 }}>
                    {configItem("Enable GC", <>{boolIcon(status.enableGc)} {status.enableGc ? "ON" : "OFF"}</>, "Primary switch for blob deletion. When OFF, no blobs are ever deleted by GC.")}
                    {configItem("Verification Mode", <>{boolIcon(status.enableGcVerification)} {status.enableGcVerification ? "ON" : "OFF"}</>, "Verification mode: logs access to deleted blobs but never actually deletes.")}
                    {configItem("GC Workers", formatNumber(status.gcWorkerCount), "Concurrent MongoDB reachability-check workers.")}
                    {configItem("Queue Limit", formatNumber(status.gcQueueLimit), "Max entries in namespace's Redis GC queue. Blob ingestion pauses when hit.")}
                </Stack>
                <Stack style={{ flex: 1, minWidth: 280 }}>
                    {configItem("Blob Ingestion Batch", formatNumber(status.gcBlobIngestionBatchSize), "Blobs scanned per TickBlobsAsync tick (every 5 min).")}
                    {configItem("GC Batch Size", formatNumber(status.gcBatchSize), "Blob IDs batched into single MongoDB $in query. Recommended: 50-200.")}
                    {configItem("Reliable Enqueue", <>{boolIcon(status.enableReliableGcEnqueue)} {status.enableReliableGcEnqueue ? "ON" : "OFF"}</>, "OFF=fire-and-forget Redis; ON=awaited writes (slower but reliable).")}
                    {configItem("Namespace Isolation", <>{boolIcon(status.enablePerNamespaceQueueIsolation)} {status.enablePerNamespaceQueueIsolation ? "ON" : "OFF"}</>, "OFF=global queue pause; ON=per-namespace limits.")}
                </Stack>
            </Stack>
            {(status.bundleCacheDir || status.bundleCacheSize) &&
                <Stack horizontal tokens={{ childrenGap: 24 }} style={{ marginTop: 8, borderTop: "1px solid rgba(128,128,128,0.2)", paddingTop: 8 }}>
                    <Text style={{ color: modeColors.text, opacity: 0.8 }}>Bundle Cache Dir: {status.bundleCacheDir || "(default)"}</Text>
                    <Text style={{ color: modeColors.text, opacity: 0.8 }}>Cache Size: {status.bundleCacheSize}</Text>
                </Stack>
            }
        </Stack>
    );
};

// ─── Namespace Status Table ───

const NamespaceStatusPanel: React.FC<{ status: StorageDashboardStatus }> = ({ status }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    const stateLabel = (ns: StorageNamespaceStatus) => {
        if (!status.enableGc) return <Text style={{ color: "gray" }}>OFF</Text>;
        if (ns.isPaused) return <Text style={{ color: "#d13438", fontWeight: 600 }}>PAUSED</Text>;
        return <Text style={{ color: "#107c10" }}>OK</Text>;
    };

    const columns: IColumn[] = [
        { key: "ns_id", name: "Namespace", fieldName: "id", minWidth: 120, maxWidth: 180, isResizable: true },
        { key: "ns_backend", name: "Backend", fieldName: "backend", minWidth: 100, maxWidth: 140, isResizable: true },
        {
            key: "ns_queue", name: "Queue Depth", minWidth: 100, maxWidth: 120, isResizable: true,
            onRender: (item: StorageNamespaceStatus) => (
                <Text style={{ color: queueDepthColor(item.gcQueueDepth), fontWeight: 600 }}>{formatCount(item.gcQueueDepth)}</Text>
            )
        },
        {
            key: "ns_lastgc", name: "Last GC", minWidth: 100, maxWidth: 140, isResizable: true,
            onRender: (item: StorageNamespaceStatus) => (
                <Text style={{ color: modeColors.text }}>{item.lastGcTime ? getShortNiceTime(item.lastGcTime, true) : "Never"}</Text>
            )
        },
        {
            key: "ns_freq", name: "Frequency", minWidth: 70, maxWidth: 90, isResizable: true,
            onRender: (item: StorageNamespaceStatus) => <Text style={{ color: modeColors.text }}>{item.gcFrequencyHrs < 1 ? `${Math.round(item.gcFrequencyHrs * 60)}min` : `${item.gcFrequencyHrs}hr`}</Text>
        },
        {
            key: "ns_delay", name: "Delay", minWidth: 60, maxWidth: 80, isResizable: true,
            onRender: (item: StorageNamespaceStatus) => <Text style={{ color: modeColors.text }}>{item.gcDelayHrs}hr</Text>
        },
        { key: "ns_prefix", name: "Prefix", fieldName: "prefix", minWidth: 60, maxWidth: 100, isResizable: true },
        {
            key: "ns_state", name: "State", minWidth: 70, maxWidth: 90, isResizable: true,
            onRender: (item: StorageNamespaceStatus) => stateLabel(item)
        }
    ];

    const sorted = [...status.namespaces].sort((a, b) => b.gcQueueDepth - a.gcQueueDepth);

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>Namespace Status</Text>
            <DetailsList
                items={sorted}
                columns={columns}
                selectionMode={SelectionMode.none}
                layoutMode={DetailsListLayoutMode.justified}
                compact={true}
                onRenderItemColumn={(item: StorageNamespaceStatus, _index?: number, column?: IColumn) => {
                    if (!column?.fieldName) return null;
                    return <Text style={{ color: modeColors.text }}>{(item as Record<string, unknown>)[column.fieldName] as string}</Text>;
                }}
            />
        </Stack>
    );
};

// ─── Backend Configuration Table ───

const BackendsPanel: React.FC<{ backends: StorageBackendInfo[] }> = ({ backends }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    const detailsString = (b: StorageBackendInfo): string => {
        const parts: string[] = [];
        if (b.bucketName) parts.push(`Bucket: ${b.bucketName}`);
        if (b.bucketPath) parts.push(`Path: ${b.bucketPath}`);
        if (b.region) parts.push(`Region: ${b.region}`);
        if (b.containerName) parts.push(`Container: ${b.containerName}`);
        if (b.gcsBucketName) parts.push(`GCS Bucket: ${b.gcsBucketName}`);
        if (b.gcsBucketPath) parts.push(`GCS Path: ${b.gcsBucketPath}`);
        if (b.baseDir) parts.push(`Dir: ${b.baseDir}`);
        if (b.secondaryBackend) parts.push(`Secondary: ${b.secondaryBackend}`);
        return parts.join("  |  ");
    };

    const columns: IColumn[] = [
        { key: "be_id", name: "Backend ID", fieldName: "id", minWidth: 120, maxWidth: 180, isResizable: true },
        { key: "be_type", name: "Type", fieldName: "type", minWidth: 80, maxWidth: 120, isResizable: true },
        {
            key: "be_details", name: "Details", minWidth: 300, isResizable: true,
            onRender: (item: StorageBackendInfo) => <Text style={{ color: modeColors.text }}>{detailsString(item)}</Text>
        }
    ];

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>Storage Backends</Text>
            <DetailsList
                items={backends}
                columns={columns}
                selectionMode={SelectionMode.none}
                layoutMode={DetailsListLayoutMode.justified}
                compact={true}
                onRenderItemColumn={(item: StorageBackendInfo, _index?: number, column?: IColumn) => {
                    if (!column?.fieldName) return null;
                    return <Text style={{ color: modeColors.text }}>{(item as Record<string, unknown>)[column.fieldName] as string}</Text>;
                }}
            />
        </Stack>
    );
};

// ─── GC Health Summary ───

const GcHealthPanel: React.FC<{ entries: StorageGcMetricEntry[]; status: StorageDashboardStatus }> = ({ entries, status }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    if (entries.length === 0) {
        return (
            <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
                <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>GC Health Summary</Text>
                <Text style={{ color: modeColors.text, opacity: 0.6 }}>No GC metric data yet. Metrics are recorded per GC tick (~6 minutes per namespace).</Text>
            </Stack>
        );
    }

    const totalIngested = entries.reduce((sum, e) => sum + e.blobsIngested, 0);
    const totalDeleted = entries.reduce((sum, e) => sum + e.blobsDeleted, 0);
    const totalBytesFreed = entries.reduce((sum, e) => sum + e.bytesFreed, 0);
    const totalEnqueueFails = entries.reduce((sum, e) => sum + e.enqueueFailures, 0);
    const totalThrottle = entries.reduce((sum, e) => sum + e.throttleEvents, 0);
    const totalRefsExpired = entries.reduce((sum, e) => sum + e.refsExpired, 0);
    const pausedCount = status.namespaces.filter(ns => ns.isPaused).length;

    // Compute hourly rates
    const times = entries.map(e => new Date(e.time).getTime());
    const rangeHours = times.length > 1 ? (Math.max(...times) - Math.min(...times)) / 3600000 : 1;
    const ingestionRate = rangeHours > 0 ? Math.round(totalIngested / rangeHours) : 0;
    const deletionRate = rangeHours > 0 ? Math.round(totalDeleted / rangeHours) : 0;

    const netRate = deletionRate - ingestionRate;
    let balanceColor = "#107c10";
    let balanceText = `+${formatCount(netRate)}/hr net deletion`;
    if (ingestionRate > deletionRate * 1.5) {
        balanceColor = "#d13438";
        balanceText = `${formatCount(netRate)}/hr — queue growing fast`;
    } else if (ingestionRate > deletionRate) {
        balanceColor = "#fce100";
        balanceText = `${formatCount(netRate)}/hr — queue growing slowly`;
    }

    // Bucket time-series for chart
    const sorted = [...entries].sort((a, b) => new Date(a.time).getTime() - new Date(b.time).getTime());
    const bucketCount = Math.min(sorted.length, 80);
    const bucketSize = Math.max(1, Math.floor(sorted.length / bucketCount));

    const bucketedDeleted: number[] = [];
    const bucketedIngested: number[] = [];
    const bucketedThrottles: number[] = [];
    const bucketedPaused: boolean[] = [];

    for (let i = 0; i < sorted.length; i += bucketSize) {
        const slice = sorted.slice(i, i + bucketSize);
        bucketedDeleted.push(slice.reduce((s, e) => s + e.blobsDeleted, 0));
        bucketedIngested.push(slice.reduce((s, e) => s + e.blobsIngested, 0));
        bucketedThrottles.push(slice.reduce((s, e) => s + e.throttleEvents, 0));
        bucketedPaused.push(slice.some(e => e.wasPaused));
    }

    const chartW = 1100;
    const chartH = 160;
    const chartBg = dashboard.darktheme ? "#060709" : "#F3F2F1";
    const chartBorder = dashboard.darktheme ? "solid 1px #181A1B" : "solid 1px #E1DFDD";

    // Compute SVG overlay rects for paused/throttle regions
    const overlayRects: React.ReactNode[] = [];
    const barW = chartW / bucketedPaused.length;
    for (let i = 0; i < bucketedPaused.length; i++) {
        if (bucketedPaused[i]) {
            overlayRects.push(<rect key={`p${i}`} x={i * barW} y={0} width={barW} height={chartH} fill="#d13438" fillOpacity={0.12} />);
        } else if (bucketedThrottles[i] > 0) {
            overlayRects.push(<rect key={`t${i}`} x={i * barW} y={0} width={barW} height={chartH} fill="#fce100" fillOpacity={0.08} />);
        }
    }

    // Time axis for the chart
    const formatTimeLabel = (iso: string): string => {
        const d = new Date(iso);
        return d.toLocaleString(undefined, { month: "short", day: "numeric", hour: "2-digit", minute: "2-digit" });
    };
    const firstTime = sorted.length > 0 ? formatTimeLabel(sorted[0].time) : "";
    const lastTime = sorted.length > 0 ? formatTimeLabel(sorted[sorted.length - 1].time) : "";
    const midTime = sorted.length > 1 ? formatTimeLabel(sorted[Math.floor(sorted.length / 2)].time) : "";

    // Max across both series for normalization
    const allMax = Math.max(...bucketedDeleted, ...bucketedIngested, 1);

    // Normalize both series to same scale for overlay
    const normalizedDeleted = bucketedDeleted.map(v => (v / allMax) * 100);
    const normalizedIngested = bucketedIngested.map(v => (v / allMax) * 100);

    const statItem = (label: string, value: string, color?: string, width = 160) => (
        <Stack style={{ width, marginBottom: 4 }}>
            <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.5, textTransform: "uppercase", letterSpacing: 0.5 }}>{label}</Text>
            <Text variant="mediumPlus" style={{ fontWeight: 600, color: color ?? modeColors.text }}>{value}</Text>
        </Stack>
    );

    const hasChartData = bucketedDeleted.length >= 2;

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            {/* Header + Balance banner */}
            <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }} style={{ marginBottom: 12 }}>
                <Text variant="xLarge" style={{ fontWeight: 600, color: modeColors.text }}>GC Health Summary</Text>
                <Stack horizontal verticalAlign="center" style={{ padding: "4px 12px", borderRadius: 3, backgroundColor: balanceColor + "18", border: `1px solid ${balanceColor}44` }}>
                    <FontIcon iconName="CircleFill" style={{ color: balanceColor, fontSize: 8, marginRight: 8 }} />
                    <Text variant="medium" style={{ fontWeight: 600, color: balanceColor }}>{balanceText}</Text>
                </Stack>
            </Stack>
            {/* Stats row */}
            <Stack horizontal tokens={{ childrenGap: 8 }} style={{ marginBottom: 12 }}>
                {statItem("Ingestion", `~${formatCount(ingestionRate)}/hr`, undefined, 130)}
                {statItem("Deletion", `~${formatCount(deletionRate)}/hr`, undefined, 130)}
                {statItem("Ingested", `${formatCount(totalIngested)} blobs`, "#ca5010", 130)}
                {statItem("Deleted", `${formatCount(totalDeleted)} blobs`, "#107c10", 130)}
                {statItem("Freed", formatBytes(totalBytesFreed), "#107c10", 110)}
                {statItem("Paused NS", `${pausedCount} / ${status.namespaces.length}`, pausedCount > 0 ? "#d13438" : undefined, 100)}
                {statItem("Enqueue Fail", formatCount(totalEnqueueFails), totalEnqueueFails > 0 ? "#d13438" : undefined, 120)}
                {statItem("Throttles", formatCount(totalThrottle), totalThrottle > 0 ? "#fce100" : undefined, 100)}
                {statItem("Refs Expired", formatCount(totalRefsExpired), undefined, 110)}
            </Stack>
            {/* Chart */}
            {hasChartData && (
                <Stack>
                    <Stack horizontal tokens={{ childrenGap: 16 }} style={{ marginBottom: 4 }}>
                        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                            <div style={{ width: 16, height: 3, backgroundColor: "#107c10" }} />
                            <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>Deleted</Text>
                        </Stack>
                        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                            <div style={{ width: 16, height: 3, backgroundColor: "#ca5010" }} />
                            <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>Ingested</Text>
                        </Stack>
                        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                            <div style={{ width: 12, height: 12, backgroundColor: "#d13438", opacity: 0.15 }} />
                            <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>Paused</Text>
                        </Stack>
                        <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 4 }}>
                            <div style={{ width: 12, height: 12, backgroundColor: "#fce100", opacity: 0.15 }} />
                            <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>Throttled</Text>
                        </Stack>
                    </Stack>
                    <div style={{ backgroundColor: chartBg, padding: 8, border: chartBorder, position: "relative" }}>
                        <svg style={{ position: "absolute", top: 8, left: 8, pointerEvents: "none" }} width={chartW} height={chartH} viewBox={`0 0 ${chartW} ${chartH}`} preserveAspectRatio="none">
                            {overlayRects}
                        </svg>
                        <div style={{ position: "relative" }}>
                            <Sparklines data={normalizedDeleted} width={chartW} height={chartH} svgWidth={chartW} svgHeight={chartH} min={0} max={100}>
                                <SparklinesLine color="#107c10" style={{ strokeWidth: 2, fill: "none" }} />
                                <SparklinesReferenceLine type="mean" style={{ stroke: "#107c10", strokeOpacity: 0.2, strokeDasharray: "4 4" }} />
                            </Sparklines>
                        </div>
                        <div style={{ position: "absolute", top: 8, left: 8, pointerEvents: "none" }}>
                            <Sparklines data={normalizedIngested} width={chartW} height={chartH} svgWidth={chartW} svgHeight={chartH} min={0} max={100}>
                                <SparklinesLine color="#ca5010" style={{ strokeWidth: 1.5, fill: "none", opacity: 0.7 }} />
                            </Sparklines>
                        </div>
                    </div>
                    <Stack horizontal style={{ justifyContent: "space-between", marginTop: 2, opacity: 0.4 }}>
                        <Text variant="tiny" style={{ color: modeColors.text }}>{firstTime}</Text>
                        <Text variant="tiny" style={{ color: modeColors.text }}>{midTime}</Text>
                        <Text variant="tiny" style={{ color: modeColors.text }}>{lastTime}</Text>
                    </Stack>
                </Stack>
            )}
        </Stack>
    );
};

// ─── MongoDB Collection Health ───

const CollectionHealthPanel: React.FC<{ snapshots: CollectionStatsSnapshot[] }> = ({ snapshots }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    if (snapshots.length === 0) {
        return (
            <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
                <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>MongoDB Collection Health</Text>
                <Text style={{ color: modeColors.text, opacity: 0.6 }}>No collection stats snapshots yet. Snapshots are recorded hourly.</Text>
            </Stack>
        );
    }

    // Use latest snapshot for the table
    const latest = snapshots[0];

    const columns: IColumn[] = [
        { key: "col_name", name: "Collection", minWidth: 140, maxWidth: 180, isResizable: true, onRender: (item: CollectionStat) => <Text style={{ color: modeColors.text, fontWeight: 600 }}>{item.name}</Text> },
        {
            key: "col_docs", name: "Documents", minWidth: 90, maxWidth: 110, isResizable: true,
            onRender: (item: CollectionStat) => <Text style={{ color: modeColors.text }}>{formatCount(item.documentCount)}</Text>
        },
        {
            key: "col_data", name: "Data Size", minWidth: 90, maxWidth: 110, isResizable: true,
            onRender: (item: CollectionStat) => <Text style={{ color: modeColors.text }}>{formatBytes(item.dataSizeBytes)}</Text>
        },
        {
            key: "col_idx", name: "Index Size", minWidth: 90, maxWidth: 110, isResizable: true,
            onRender: (item: CollectionStat) => <Text style={{ color: modeColors.text }}>{formatBytes(item.indexSizeBytes)}</Text>
        },
        {
            key: "col_storage", name: "Storage Size", minWidth: 90, maxWidth: 110, isResizable: true,
            onRender: (item: CollectionStat) => <Text style={{ color: modeColors.text }}>{formatBytes(item.storageSizeBytes)}</Text>
        },
        {
            key: "col_frag", name: "Fragmentation", minWidth: 100, maxWidth: 120, isResizable: true,
            onRender: (item: CollectionStat) => (
                <Text style={{ color: fragmentationColor(item.fragmentationRatio), fontWeight: 600 }}>
                    {item.fragmentationRatio.toFixed(2)}x
                </Text>
            )
        }
    ];

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>MongoDB Collection Health</Text>
            <DetailsList
                items={latest.collections}
                columns={columns}
                selectionMode={SelectionMode.none}
                layoutMode={DetailsListLayoutMode.justified}
                compact={true}
            />
        </Stack>
    );
};

// ─── GC Rate Sparklines (per namespace) ───

const GcRatePanel: React.FC<{ entries: StorageGcMetricEntry[] }> = ({ entries }) => {
    const { hordeClasses, modeColors } = getHordeStyling();

    if (entries.length === 0) {
        return (
            <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
                <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>GC Rate Trends</Text>
                <Text style={{ color: modeColors.text, opacity: 0.6 }}>No GC metric data yet. Metrics are recorded per GC tick (~6 minutes per namespace).</Text>
            </Stack>
        );
    }

    // Group by namespace
    const byNamespace: Record<string, StorageGcMetricEntry[]> = {};
    for (const entry of entries) {
        if (!byNamespace[entry.namespaceId]) byNamespace[entry.namespaceId] = [];
        byNamespace[entry.namespaceId].push(entry);
    }

    // Sort entries chronologically within each namespace
    for (const ns of Object.keys(byNamespace)) {
        byNamespace[ns].sort((a, b) => new Date(a.time).getTime() - new Date(b.time).getTime());
    }

    const sparkWidth = 280;
    const sparkHeight = 48;
    const chartBg = dashboard.darktheme ? "#060709" : "#F3F2F1";
    const chartBorder = dashboard.darktheme ? "solid 1px #181A1B" : "solid 1px #E1DFDD";

    const formatTimeLabel = (iso: string): string => {
        const d = new Date(iso);
        return d.toLocaleString(undefined, { month: "short", day: "numeric", hour: "2-digit", minute: "2-digit" });
    };

    type HealthLevel = "good" | "warning" | "critical";

    const healthColor = (level: HealthLevel): string => {
        if (level === "critical") return "#d13438";
        if (level === "warning") return "#fce100";
        return "#107c10";
    };

    const healthBorderColor = (level: HealthLevel): string => {
        if (level === "critical") return dashboard.darktheme ? "solid 1px #5c1517" : "solid 1px #d1343844";
        if (level === "warning") return dashboard.darktheme ? "solid 1px #5c5200" : "solid 1px #fce10044";
        return chartBorder;
    };

    const healthBgColor = (level: HealthLevel): string => {
        if (level === "critical") return dashboard.darktheme ? "#1a0506" : "#fde7e9";
        if (level === "warning") return dashboard.darktheme ? "#141200" : "#fff8e1";
        return chartBg;
    };

    const chartPanel = (data: number[], label: string, health: HealthLevel, formatFn?: (n: number) => string) => {
        if (data.length < 2) return null;
        const latest = data[data.length - 1];
        const avg = data.reduce((s, v) => s + v, 0) / data.length;
        const dataMax = Math.max(...data);
        const fmt = formatFn ?? formatCount;
        const hc = healthColor(health);

        return (
            <Stack style={{ marginBottom: 8 }}>
                <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 6 }} style={{ marginBottom: 2 }}>
                    <FontIcon iconName="CircleFill" style={{ color: hc, fontSize: 6 }} />
                    <Text variant="small" style={{ fontWeight: 600, color: modeColors.text }}>{label}</Text>
                    <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.5 }}>{fmt(latest)} (avg {fmt(Math.round(avg))})</Text>
                </Stack>
                <div style={{ backgroundColor: healthBgColor(health), padding: 4, border: healthBorderColor(health) }}>
                    <Sparklines data={data} width={sparkWidth} height={sparkHeight} svgWidth={sparkWidth} svgHeight={sparkHeight} min={0} max={Math.max(dataMax, 1)}>
                        <SparklinesLine color={hc} style={{ strokeWidth: 1.5, fill: "none" }} />
                        <SparklinesReferenceLine type="mean" style={{ stroke: hc, strokeOpacity: 0.3, strokeDasharray: "4 4" }} />
                    </Sparklines>
                </div>
            </Stack>
        );
    };

    const deletionHealth = (nsEntries: StorageGcMetricEntry[]): HealthLevel => {
        const latest = nsEntries[nsEntries.length - 1];
        if (latest.wasPaused) return "critical";
        const recentDeleted = nsEntries.slice(-10).reduce((s, e) => s + e.blobsDeleted, 0);
        const recentIngested = nsEntries.slice(-10).reduce((s, e) => s + e.blobsIngested, 0);
        if (recentIngested > recentDeleted * 2) return "critical";
        if (recentIngested > recentDeleted) return "warning";
        return "good";
    };

    const queueHealth = (nsEntries: StorageGcMetricEntry[]): HealthLevel => {
        const latest = nsEntries[nsEntries.length - 1];
        if (latest.queueDepth > 10000) return "critical";
        if (latest.queueDepth > 1000) return "warning";
        return "good";
    };

    const enqueueHealth = (nsEntries: StorageGcMetricEntry[]): HealthLevel => {
        const recentFails = nsEntries.slice(-10).reduce((s, e) => s + e.enqueueFailures, 0);
        if (recentFails > 10) return "critical";
        if (recentFails > 0) return "warning";
        return "good";
    };

    const sweepHealth = (nsEntries: StorageGcMetricEntry[]): HealthLevel => {
        const recentAvg = nsEntries.slice(-10).reduce((s, e) => s + e.sweepDurationMs, 0) / Math.min(nsEntries.length, 10);
        if (recentAvg > 60000) return "critical";
        if (recentAvg > 30000) return "warning";
        return "good";
    };

    const renderNamespaceCard = (nsId: string, nsEntries: StorageGcMetricEntry[]) => {
        const totalChecked = nsEntries.reduce((s, e) => s + e.blobsChecked, 0);
        const totalDeleted = nsEntries.reduce((s, e) => s + e.blobsDeleted, 0);
        const totalBytesFreed = nsEntries.reduce((s, e) => s + e.bytesFreed, 0);
        const totalEnqueueFails = nsEntries.reduce((s, e) => s + e.enqueueFailures, 0);
        const totalThrottles = nsEntries.reduce((s, e) => s + e.throttleEvents, 0);
        const totalRefsExpired = nsEntries.reduce((s, e) => s + e.refsExpired, 0);
        const pausedTicks = nsEntries.filter(e => e.wasPaused).length;
        const efficiency = totalChecked > 0 ? Math.round((totalDeleted / totalChecked) * 100) : 0;
        const latestEntry = nsEntries[nsEntries.length - 1];
        const isActive = !latestEntry.wasPaused;
        const firstTime = formatTimeLabel(nsEntries[0].time);
        const lastTime = formatTimeLabel(nsEntries[nsEntries.length - 1].time);

        return (
            <Stack key={nsId} style={{ padding: 16, border: dashboard.darktheme ? "1px solid #2d2d2d" : "1px solid #e1dfdd", borderRadius: 4, backgroundColor: dashboard.darktheme ? "#1b1b1f" : "#faf9f8", boxSizing: "border-box" }}>
                <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 8 }} style={{ marginBottom: 8 }}>
                    <FontIcon iconName="CircleFill" style={{ color: isActive ? "#107c10" : "#d13438", fontSize: 10 }} />
                    <Text variant="mediumPlus" style={{ fontWeight: 600, color: modeColors.text }}>{nsId}</Text>
                    <Text variant="small" style={{ color: isActive ? "#107c10" : "#d13438", fontWeight: 600 }}>{isActive ? "Active" : "Paused"}</Text>
                </Stack>
                <Stack style={{ marginBottom: 8, padding: "4px 0", borderBottom: "1px solid rgba(128,128,128,0.1)" }}>
                    <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>
                        {nsEntries.length} ticks &middot; {firstTime} — {lastTime}
                    </Text>
                    <Text variant="tiny" style={{ color: modeColors.text, opacity: 0.6 }}>
                        Eff: {efficiency}% &middot; Del: {formatCount(totalDeleted)} &middot; Freed: {formatBytes(totalBytesFreed)} &middot; Refs: {formatCount(totalRefsExpired)}
                    </Text>
                    {(totalEnqueueFails > 0 || totalThrottles > 0 || pausedTicks > 0) && (
                        <Text variant="tiny" style={{ color: "#d13438" }}>
                            {totalEnqueueFails > 0 ? `Enq Fail: ${formatCount(totalEnqueueFails)} ` : ""}
                            {totalThrottles > 0 ? `Throttles: ${formatCount(totalThrottles)} ` : ""}
                            {pausedTicks > 0 ? `Paused: ${pausedTicks}/${nsEntries.length}` : ""}
                        </Text>
                    )}
                </Stack>
                <Stack horizontal tokens={{ childrenGap: 12 }}>
                    <Stack style={{ flex: 1 }}>
                        {chartPanel(nsEntries.map(e => e.blobsDeleted), "Deleted", deletionHealth(nsEntries))}
                        {chartPanel(nsEntries.map(e => e.bytesFreed), "Freed", "good", formatBytes)}
                        {chartPanel(nsEntries.map(e => e.queueDepth), "Queue", queueHealth(nsEntries))}
                    </Stack>
                    <Stack style={{ flex: 1 }}>
                        {chartPanel(nsEntries.map(e => e.blobsChecked), "Checked", deletionHealth(nsEntries))}
                        {chartPanel(nsEntries.map(e => e.blobsIngested), "Ingested", enqueueHealth(nsEntries))}
                        {chartPanel(nsEntries.map(e => e.sweepDurationMs), "Sweep", sweepHealth(nsEntries), (n: number) => `${formatNumber(n)}ms`)}
                    </Stack>
                </Stack>
            </Stack>
        );
    };

    const nsKeys = Object.keys(byNamespace);

    return (
        <Stack className={hordeClasses.raised} style={{ padding: 16 }}>
            <Text variant="xLarge" style={{ fontWeight: 600, marginBottom: 12, color: modeColors.text }}>GC Rate Trends</Text>
            <div style={{ display: "grid", gridTemplateColumns: "1fr 1fr", gap: 16, alignItems: "stretch" }}>
                {nsKeys.map(nsId => renderNamespaceCard(nsId, byNamespace[nsId]))}
            </div>
        </Stack>
    );
};

// ─── Help Content ───

const storageHelpMarkdown = `
## GC Health Summary

The top-level overview of garbage collection across all storage namespaces.

- **Ingestion** — Rate at which new blobs are being added to storage (blobs/hr)
- **Deletion** — Rate at which GC is removing unreferenced blobs (blobs/hr)
- **Balance** — Net rate (deletion minus ingestion). Positive = queue shrinking, negative = queue growing
- **Freed** — Total bytes reclaimed by GC in the selected time window
- **Paused NS** — How many namespaces have GC paused (queue full or manual pause)
- **Enqueue Fail** — Failed attempts to add blobs to the GC queue (Redis write failures)
- **Throttles** — Times GC was throttled due to queue limits being hit
- **Refs Expired** — Reference entries that expired and were cleaned up

The chart overlays **deleted** (red) and **ingested** (purple) blob counts on the same scale. Red-tinted regions indicate paused periods; yellow-tinted regions indicate throttle events.

---

## MongoDB Collection Health

Hourly snapshots of MongoDB collection statistics via the \`collStats\` command.

- **Documents** — Number of documents in the collection
- **Data Size** — Logical size of all documents
- **Index Size** — Total size of all indexes
- **Storage Size** — Disk space allocated by MongoDB (includes pre-allocated and freed space)
- **Fragmentation** — Ratio of storage size to data size. **1.0x** = perfectly compact. **>1.5x** = moderate fragmentation (yellow). **>2.0x** = significant bloat (red), may benefit from a \`compact\` command

The **Collection Size Trends** sparklines show data and index size over time for collections larger than 1 MB.

---

## GC Rate Trends

Per-namespace GC metrics displayed as paired cards. Each card contains six mini-charts:

- **Deleted** — Blobs deleted per GC tick (~6 min). Health based on deletion vs ingestion balance
- **Checked** — Blobs checked for reachability per tick. Same health as Deleted
- **Freed** — Bytes freed per tick. Always green (informational)
- **Ingested** — Blobs ingested into the GC queue per tick. Health based on enqueue failure rate
- **Queue** — Current GC queue depth. **>1K** = yellow, **>10K** = red
- **Sweep** — Duration of each GC sweep in milliseconds. **>30s** = yellow, **>60s** = red

Card header shows: active/paused status, tick count, time range, GC efficiency (% of checked blobs that were deleted), total deleted, total freed, and refs expired. Error indicators appear when enqueue failures, throttles, or paused ticks are detected.

---

## GC Configuration

Current server-side GC settings:

- **Enable GC** — Primary switch. When OFF, no blobs are ever deleted
- **Verification Mode** — When ON, GC logs what it would delete but does not actually remove blobs
- **GC Workers** — Number of concurrent MongoDB reachability-check workers
- **Queue Limit** — Max entries in a namespace's Redis GC queue. Blob ingestion pauses when this limit is hit
- **Blob Ingestion Batch** — Number of blobs scanned per TickBlobsAsync cycle (every 5 minutes)
- **GC Batch Size** — Number of blob IDs batched into a single MongoDB \`$in\` query (recommended: 50-200)
- **Reliable Enqueue** — OFF = fire-and-forget Redis writes (fast). ON = awaited writes (slower but guaranteed)
- **Namespace Isolation** — OFF = one global queue pause. ON = per-namespace queue limits

---

## Namespace Status

Table of all storage namespaces with their current GC state:

- **Backend** — Which storage backend (S3, Azure, filesystem, etc.) the namespace uses
- **Queue Depth** — Current number of blobs waiting for GC. Color-coded: green <1K, yellow 1K-10K, red >10K
- **Last GC** — When GC last ran for this namespace
- **Frequency** — How often GC runs (e.g., every 6 minutes)
- **Delay** — Minimum age before a blob becomes eligible for GC
- **State** — OK (running), PAUSED (queue full or manual), or OFF (GC disabled globally)

---

## Storage Backends

Configuration details for each registered storage backend, including bucket names, paths, regions, and secondary/fallback backends.
`;

// ─── Main Inner Panel ───

const StoragePanel: React.FC = observer(() => {
    useEffect(() => {
        handler.start();
        return () => { handler.clear(); };
    }, []);

    const { modeColors } = getHordeStyling();
    const [timeKey, setTimeKey] = useState(handler.timeSelection.key);
    const [showHelp, setShowHelp] = useState(false);

    // subscribe
    if (handler.updated) { }

    const timeOptions: IComboBoxOption[] = timeSelections.map(t => ({ key: t.key, text: t.text }));

    const onTimeChange = (_ev: unknown, option?: IComboBoxOption) => {
        if (option) {
            const selection = timeSelections.find(t => t.key === option.key);
            if (selection) {
                setTimeKey(selection.key);
                handler.setTimeSelection(selection);
            }
        }
    };

    return (
        <>
        <Modal isOpen={showHelp} onDismiss={() => setShowHelp(false)} isBlocking={false} styles={{ main: { maxWidth: 900, width: "90%", maxHeight: "80vh" } }}>
            <Stack style={{ padding: 24 }}>
                <Stack horizontal horizontalAlign="space-between" verticalAlign="center" style={{ marginBottom: 16 }}>
                    <Text variant="xLarge" style={{ fontWeight: 600 }}>Storage Dashboard Guide</Text>
                    <IconButton iconProps={{ iconName: "Cancel" }} onClick={() => setShowHelp(false)} />
                </Stack>
                <div style={{ overflowY: "auto", maxHeight: "calc(80vh - 100px)" }}>
                    <Markdown options={{ forceBlock: true }}>{storageHelpMarkdown}</Markdown>
                </div>
            </Stack>
        </Modal>
        <Stack tokens={{ childrenGap: 16 }}>
            {/* Time range + last updated + help */}
            <Stack horizontal verticalAlign="center" tokens={{ childrenGap: 16 }}>
                <ComboBox
                    selectedKey={timeKey}
                    options={timeOptions}
                    onChange={onTimeChange}
                    styles={{ root: { width: 200 } }}
                />
                {handler.lastUpdated && (
                    <Text variant="small" style={{ color: modeColors.text, opacity: 0.5 }}>
                        Updated {getShortNiceTime(handler.lastUpdated, true)}
                    </Text>
                )}
                <Stack grow horizontalAlign="end">
                    <PrimaryButton text="Help" onClick={() => setShowHelp(true)} />
                </Stack>
            </Stack>

            {/* Error banner */}
            {handler.error && (
                <MessageBar messageBarType={MessageBarType.error} isMultiline={false} onDismiss={() => { handler.error = undefined; handler.setUpdated(); }}>
                    {handler.error}
                </MessageBar>
            )}

            {/* Loading state */}
            {!handler.data && !handler.error && (
                <Stack horizontalAlign="center" style={{ padding: 48 }}>
                    <Spinner size={SpinnerSize.large} label="Loading storage dashboard..." />
                </Stack>
            )}

            {/* Panels */}
            {handler.data && (
                <>
                    <GcHealthPanel entries={handler.data.gcMetrics.entries} status={handler.data.status} />
                    <CollectionHealthPanel snapshots={handler.data.collectionStats.entries} />
                    <GcRatePanel entries={handler.data.gcMetrics.entries} />
                    <GcConfigPanel status={handler.data.status} />
                    <NamespaceStatusPanel status={handler.data.status} />
                    <BackendsPanel backends={handler.data.status.backends} />
                </>
            )}
        </Stack>
        </>
    );
});

// ─── Outer View ───

export const StorageView: React.FC = () => {
    const windowSize = useWindowSize();
    const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);
    const centerAlign = vw / 2 - 720;

    const { hordeClasses, modeColors } = getHordeStyling();

    const key = `windowsize_view_${windowSize.width}_${windowSize.height}`;

    return (
        <Stack className={hordeClasses.horde}>
            <TopNav />
            <Breadcrumbs items={[{ text: "Storage" }]} />
            <Stack styles={{ root: { width: "100%", backgroundColor: modeColors.background } }}>
                <Stack style={{ width: "100%", backgroundColor: modeColors.background }}>
                    <Stack style={{ position: "relative", width: "100%", height: "calc(100vh - 148px)" }}>
                        <div style={{ overflowX: "auto", overflowY: "auto" }}>
                            <Stack horizontal style={{ paddingTop: 30, paddingBottom: 48 }}>
                                <Stack key={`${key}`} style={{ paddingLeft: centerAlign }} />
                                <Stack style={{ width: 1440 }}>
                                    <StoragePanel />
                                </Stack>
                            </Stack>
                        </div>
                    </Stack>
                </Stack>
            </Stack>
        </Stack>
    );
};
