

// Copyright Epic Games, Inc. All Rights Reserved.

import { DetailsList, DetailsListLayoutMode, IColumn, PrimaryButton, SelectionMode, Slider, Spinner, SpinnerSize, Stack, Text, TextField } from "@fluentui/react";
import { observer } from "mobx-react-lite";
import React, { useEffect, useState } from "react";
import backend from "../../backend";
import { IssueData, JobData, JobStepOutcome } from "../../backend/Api";
import { useWindowSize } from "../../base/utilities/hooks";
import { Breadcrumbs } from "../Breadcrumbs";
import { TopNav } from "../TopNav";
import { action, makeObservable, observable } from "mobx";
import moment from "moment";
import { getNiceTime } from "../../base/utilities/timeUtils";
import { IssueModalV2 } from "../IssueViewV2";
import { getHordeStyling } from "../../styles/Styles";

class ToolHandler {

   constructor() {
      makeObservable(this);

   }

   clear() {
      this.loading = false;
      this.jobId = "";
      this.stepId = "";
      this.months = -1;
      this.count = 0;
      this.total = 0;
      this.stepName = "";
      this.issues.clear();
   }

   async update(jobId: string, stepId: string, months: number) {

      if (!jobId || !stepId) {
         return;
      }

      if (jobId === this.jobId && stepId === this.stepId && this.months === months) {
         return;
      }
      this.issues.clear();
      this.loading = true;
      this.setUpdated();

      this.jobId = jobId;
      this.stepId = stepId;
      this.months = months;
      this.count = 0;
      this.total = 0;

      let job: JobData | undefined;

      try {
         job = await backend.getJob(jobId);
      } catch {
         console.error("Unable to get job");
         this.loading = false;
         this.setUpdated();
         return;
      }
      
      const batch = job.batches?.find(b => b.steps.find(s => s.id === stepId));
      if (!batch) {
         console.error("Batch not found for step id");
         this.loading = false;
         this.setUpdated();
         return;
      }


      const stepName = batch.steps.find(s => s.id === stepId)?.name;
      this.stepName = stepName ?? "";

      if (!stepName) {
         console.error("Bad step name");
         this.loading = false;
         this.setUpdated();
         return;
      }

      let history = await backend.getJobStepHistory(job.streamId, stepName, 65535 * 4, job.templateId!);

      const cutoff = moment().subtract(months, 'months').toDate();

      history = history.filter(item => {
         if (new Date(item.startTime) < cutoff) {
            return false;
         }
         if (item.outcome !== JobStepOutcome.Failure) {
            return false;
         }
         return true;
      })

      this.total = history.length;

      let batches = [...history];

      // get current issues
      const unresolved = await backend.getIssues({ jobId: jobId, stepId: stepId, resolved: false });

      unresolved.forEach(r => {
         this.issues.set(r.id, r);
      });

      while (batches.length) {

         const batch = batches.slice(0, 10).map(h => {
            return backend.getIssues({ jobId: h.jobId, stepId: h.stepId, resolved: true })
         });

         try {
            const result = await Promise.all(batch);

            result.forEach((r) => {
               r.forEach(i => {
                  if (!this.issues.has(i.id)) {
                     this.issues.set(i.id, i);
                  }
               });
               this.count++;
            });

         } catch (error) {
            console.log(error);
         } finally {
            batches = batches.slice(10);
            this.setUpdated();
         }
      }

      this.loading = false;
      this.setUpdated();
   }

   @action
   setUpdated() {
      this.updated++;
   }

   count = 0;
   total = 0;

   @observable
   updated = 0;

   loading = false;

   jobId = "";
   stepId = "";
   months = -1;
   stepName = "";

   issues: Map<number, IssueData> = new Map();

}

const handler = new ToolHandler();

const IssuePanel: React.FC = observer(() => {

   const [jobId, setJobId] = useState("");
   const [stepId, setStepId] = useState("");
   const [months, setMonths] = useState(1);
   const [selectedIssue, setSelectedIssue] = useState<number | undefined>();

   useEffect(() => {

      return () => {
         handler.clear();
      };

   }, []);

   const { modeColors } = getHordeStyling();

   // subscribe
   if (handler.updated) { };

   const issues = Array.from((handler.issues.values())).sort((a, b) => b.id - a.id);


   const columns: IColumn[] = [
      { key: 'column_id', name: 'Id', minWidth: 120, maxWidth: 120, isResizable: false },
      { key: 'column_desc', name: 'Description', minWidth: 780, maxWidth: 780, isResizable: false, isMultiline: true },
      { key: 'column_date', name: 'Date', minWidth: 200, maxWidth: 200, isResizable: false, isMultiline: true },
   ];

   const renderItem = (item: IssueData, index?: number, column?: IColumn) => {

      if (!column) {
         return null;
      }

      if (column.key === 'column_id') {
         return <Stack verticalAlign="center" verticalFill={true} style={{ cursor: "pointer" }} onClick={() => setSelectedIssue(item.id)}>
            <Text style={{ fontFamily: "Horde Open Sans SemiBold", color: modeColors.text }}>Issue {item.id}</Text>
         </Stack>
      }

      if (column.key === 'column_desc') {
         return <Stack verticalAlign="center" verticalFill={true}>
            <Text style={{ color: modeColors.text }}>{item.summary ?? "No Summary"}</Text>
         </Stack>
      }

      if (column.key === 'column_date') {
         return <Stack horizontalAlign="end" verticalAlign="center" verticalFill={true}>
            <Text style={{ color: modeColors.text }}>{getNiceTime(item.createdAt)}</Text>
         </Stack>
      }

   };

   return (<Stack>
      {!!selectedIssue && <IssueModalV2 issueId={selectedIssue.toString()} popHistoryOnClose={false} onCloseExternal={() => setSelectedIssue(undefined)} />}
      <Stack styles={{ root: { paddingTop: 0, paddingLeft: 12, paddingRight: 12, width: "100%" } }} >
         <Stack horizontal verticalAlign="end" tokens={{ childrenGap: 24 }}>
            <TextField autoComplete="off" disabled={handler.loading} style={{ width: 200 }} label="job Id" value={jobId} onChange={(ev, newValue) => {
               setJobId(newValue || "");
            }} />
            <TextField autoComplete="off" disabled={handler.loading} style={{ width: 54 }} label="Step Id" value={stepId} onChange={(ev, newValue) => {
               setStepId(newValue || "");
            }} />
            <Stack style={{ width: 180 }}>
               <Slider
                  disabled={handler.loading}
                  label="Months"
                  min={1}
                  max={6}
                  value={months}
                  showValue
                  // eslint-disable-next-line react/jsx-no-bind
                  onChange={(newValue) => {
                     setMonths(newValue || 1)
                  }}
               />
            </Stack>
            <Stack style={{ paddingLeft: 32 }}>
               <PrimaryButton disabled={handler.loading || !jobId || !stepId} text="Go" onClick={(ev) => {
                  handler.update(jobId, stepId, months);
               }} />
            </Stack>

         </Stack>
         <Stack style={{ paddingTop: 32 }}>
            {handler.loading && <Stack horizontalAlign="center" tokens={{ childrenGap: 24 }}>
               {!!handler.total && <Text variant="mediumPlus">{`Loaded ${handler.count} of ${handler.total} steps (This is a prototype and queries are very slow)`}</Text>}
               <Spinner size={SpinnerSize.large} />
            </Stack>
            }
            {!handler.loading && handler.stepName && <Stack style={{paddingBottom: 24}}>
               <Text variant="mediumPlus">{`${handler.stepName} - Issues (${issues.length})`}</Text>
            </Stack>
            }
            {!handler.loading && handler.stepName && <Stack tokens={{ childrenGap: 32 }}>
               <div style={{ overflowY: 'auto', overflowX: 'hidden', maxHeight: "calc(100vh - 362px)" }} data-is-scrollable={true}>
                  <Stack>
                     <DetailsList
                        isHeaderVisible={false}
                        items={issues}
                        columns={columns}
                        selectionMode={SelectionMode.none}
                        layoutMode={DetailsListLayoutMode.justified}
                        compact={false}
                        onRenderItemColumn={renderItem}

                     />
                  </Stack>
               </div>
            </Stack>}
         </Stack>
      </Stack>
   </Stack>);
});


export const StepIssueReportTest: React.FC = () => {

   const windowSize = useWindowSize();
   const vw = Math.max(document.documentElement.clientWidth, window.innerWidth || 0);

   const { hordeClasses, modeColors } = getHordeStyling();

   return <Stack className={hordeClasses.horde}>
      <TopNav />
      <Breadcrumbs items={[{ text: 'Step Issue Report Prototype' }]} />
      <Stack horizontal>
         <div key={`windowsize_streamview_${windowSize.width}_${windowSize.height}`} style={{ width: vw / 2 - (1440 / 2), flexShrink: 0, backgroundColor: modeColors.background }} />
         <Stack tokens={{ childrenGap: 0 }} styles={{ root: { backgroundColor: modeColors.background, width: "100%" } }}>
            <Stack style={{ maxWidth: 1440, paddingTop: 6, marginLeft: 4, height: 'calc(100vh - 8px)' }}>
               <Stack horizontal className={hordeClasses.raised}>
                  <Stack style={{ width: "100%", height: 'calc(100vh - 228px)' }} tokens={{ childrenGap: 18 }}>
                     <IssuePanel />
                  </Stack>
               </Stack>
            </Stack>
         </Stack>
      </Stack>
   </Stack>
};

