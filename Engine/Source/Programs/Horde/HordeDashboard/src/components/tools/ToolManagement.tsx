// Copyright Epic Games, Inc. All Rights Reserved.

import { DefaultButton, DetailsList, DetailsListLayoutMode, IContextualMenuProps, IDetailsListProps, IconButton, Modal, PrimaryButton, ProgressIndicator, SelectionMode, Spinner, SpinnerSize, Stack, Text } from "@fluentui/react";
import { useConst } from '@fluentui/react-hooks';
import Markdown from "markdown-to-jsx";
import { observer } from "mobx-react-lite";
import moment from "moment";
import { useState } from "react";
import backend from "../../backend";
import { GetToolDeploymentResponse, GetToolResponse, GetToolSummaryResponse, ToolDeploymentState, UpdateDeploymentRequest } from "../../backend/Api";
import dashboard from "../../backend/Dashboard";
import { PollBase } from "../../backend/PollBase";
import { getHordeStyling } from "../../styles/Styles";
import { errorDialogStore } from "../error/ErrorStore";

class ToolHandler extends PollBase {

   constructor(toolId: string, pollTime = 10000) {
      super(pollTime);
      this.toolId = toolId;

      this.start();
   }

   async poll(): Promise<void> {

      const requests = [backend.getTools(), backend.getTool(this.toolId)]

      const results = await Promise.all(requests);

      const tools = results[0] as GetToolSummaryResponse[];

      this.data = results[1] as GetToolResponse;

      this.currentDeployment = undefined;

      const tool = tools.find(t => t.id === this.data?.id);

      if (tool) {
         this.currentDeployment = this.data!.deployments?.find(d => d.id === tool.deploymentId);
      }

      this.setUpdated();
   }

   data?: GetToolResponse
   currentDeployment?: GetToolDeploymentResponse;

   toolId: string;
}

type DeploymentItem = {
   deploy: GetToolDeploymentResponse;
}

const ActionConfirmation: React.FC<{ handler: ToolHandler, request: UpdateDeploymentRequest, text: string, actionName: string, deployment: GetToolDeploymentResponse, onClose: () => void }> = observer(({ handler, request, text, actionName, deployment, onClose }) => {

   const [submitting, setSubmitting] = useState(false);

   const { hordeClasses, modeColors } = getHordeStyling();
   if (submitting) {
      return <Modal isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 800, backgroundColor: modeColors.background, height: 180, hasBeenOpened: false, top: "24px", position: "absolute" } }} className={hordeClasses.modal}>
         <Stack tokens={{ childrenGap: 40 }} styles={{ root: { padding: 8, paddingBottom: 24 } }}>
            <Stack grow verticalAlign="center" horizontalAlign="center" style={{ paddingTop: 12 }}>
               <Text variant="mediumPlus" styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold" } }}>{`Updating Deployment, One Moment Please...`}</Text>
            </Stack>
            <Stack verticalAlign="center">
               <Spinner size={SpinnerSize.large} />
            </Stack>
         </Stack>
      </Modal>
   }

   const height = 120;

   const onSubmit = async () => {

      setSubmitting(true);

      let errorReason: any = undefined;

      try {

         await backend.updateDeployment(handler.data!.id, deployment.id, request);


      } catch (reason) {

         errorReason = reason ? reason : "Unknown Error";

      } finally {

         if (errorReason) {

            errorDialogStore.set({

               reason: `${errorReason}`,
               title: `Error Updating Deployment`,
               message: `There was an issue updating the tool deployment.\n\nReason: ${errorReason}\n\nTime: ${moment.utc().format("MMM Do, HH:mm z")}`

            }, true);
         }
         else {
            handler.poll();
         }

         setSubmitting(false);
         onClose();

      }
   }

   return <Stack>
      <Modal isModeless={false} key="action_confirm_key" isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 480, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { }}>
         <Stack horizontal styles={{ root: { padding: 8 } }}>
            <Stack.Item grow={2} style={{ paddingLeft: 8 }}>
               <Text variant="mediumPlus">{`Update ${handler.data?.name}`}</Text>
            </Stack.Item>
            <Stack.Item grow={0}>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onClose(); }}
               />
            </Stack.Item>
         </Stack>

         <Stack style={{ marginRight: 18, marginTop: 18 }}>
            <Stack styles={{ root: { height: height } }}>
               <Stack style={{ paddingLeft: 24, paddingRight: 24, height: height - 12 }} tokens={{ childrenGap: 8 }}>
                  <Markdown>{text}</Markdown>
               </Stack>
            </Stack>
         </Stack>

         <Stack styles={{ root: { padding: 8 } }}>
            <Stack horizontal tokens={{ childrenGap: 16 }} styles={{ root: { paddingTop: 24, paddingLeft: 8, paddingBottom: 8, paddingRight: 12 } }}>
               <Stack grow />
               <PrimaryButton text={actionName} disabled={submitting} onClick={() => { onSubmit() }} />
               <DefaultButton text="Cancel" disabled={submitting} onClick={() => { onClose(); }} />
            </Stack>
         </Stack>
      </Modal>
   </Stack>;
});


const ToolManagementInner: React.FC<{ handler: ToolHandler }> = observer(({ handler }) => {

   const [actionState, setActionState] = useState<{ update?: UpdateDeploymentRequest, text?: string, actionName?: string, deployment?: GetToolDeploymentResponse }>({});

   handler.subscribe();

   if (!handler.data) {
      return null;
   }

   const items: DeploymentItem[] = handler.data.deployments.map(d => {
      return {
         deploy: d
      };
   }).sort((a, b) => {
      if (a.deploy.id === handler.currentDeployment?.id) {
         return -1;
      }

      if (b.deploy.id === handler.currentDeployment?.id) {
         return 1;
      }

      return -a.deploy.version.localeCompare(b.deploy.version)
   })

   const renderProgress = (item: DeploymentItem) => {

      const active = item.deploy.state === ToolDeploymentState.Active;
      if (active || item.deploy.progress >= 1) {
         return null;
      }

      return <Stack>
         <ProgressIndicator percentComplete={item.deploy.progress} barHeight={3} styles={{ root: { paddingLeft: 12, width: 208 } }} />
      </Stack>

   }

   const renderActions = (item: DeploymentItem) => {

      const current = item.deploy.id === handler.currentDeployment?.id;
      const active = item.deploy.state === ToolDeploymentState.Active;
      const disabled = !dashboard.development;

      const actionProps: IContextualMenuProps = {
         items: [
            {
               key: 'manage_current',
               text: 'Set Current',
               disabled: current || active || disabled,
               onClick: () => {
                  setActionState({ update: { state: ToolDeploymentState.Active }, text: `Set **${item.deploy.version}** Current?`, actionName: "Set Current", deployment: item.deploy })
               }
            },
            {
               key: 'manage_pause',
               text: 'Pause',
               disabled: !active || disabled,
               onClick: () => {
                  setActionState({ update: { state: ToolDeploymentState.Paused }, text: `Pause **${item.deploy.version}** Deployment?`, actionName: "Pause", deployment: item.deploy })
               }
            },
            {
               key: 'manage_delete',
               text: 'Delete',
               disabled: current || disabled,
               onClick: () => {
                  setActionState({ update: { state: ToolDeploymentState.Cancelled }, text: `Delete **${item.deploy.version}** Deployment?`, actionName: "Delete", deployment: item.deploy })
               }
            },
         ]
      };


      const r = item.deploy;

      return <Stack>
         <DefaultButton text="Actions" menuProps={actionProps} />
      </Stack>

   }

   const renderState = (item: DeploymentItem) => {
      let state: string = item.deploy.state;
      if (item.deploy.id === handler.currentDeployment?.id) {
         state = "Current";
      }

      if (item.deploy.progress < 1.0) {
         state = "Deploying"
      }

      return <Stack style={{ width: 96 }} horizontalAlign="center"><Text style={{ fontWeight: state === "Current" ? 600 : undefined }}>{state}</Text></Stack>
   }

   const renderRow: IDetailsListProps['onRenderRow'] = (props) => {

      if (props) {

         const item = props!.item as DeploymentItem;

         let background: string | undefined;
         if (props.itemIndex % 2 === 0) {
            //background = dashboard.darktheme ? "#1D2021" : "#FAF9F9";
         }

         return <Stack horizontal verticalAlign="center" verticalFill tokens={{ childrenGap: 24 }} styles={{ root: { backgroundColor: background, paddingLeft: 12, paddingRight: 12, paddingTop: 8, paddingBottom: 8 } }}>
            <Stack><Text style={{ fontWeight: 600 }}>{item.deploy.version}</Text></Stack>
            {renderState(item)}
            {renderProgress(item)}
            <Stack grow />
            {renderActions(item)}
         </Stack>
      }
      return null;
   };


   return <Stack style={{ padding: 24 }}>
      {!!actionState.update && <ActionConfirmation handler={handler} request={actionState.update} text={actionState.text ?? ""} actionName={actionState.actionName ?? ""} deployment={actionState.deployment!} onClose={() => setActionState({})} />}
      <DetailsList
         isHeaderVisible={false}
         styles={{
            headerWrapper: {
               paddingTop: 0
            }
         }}
         items={items}
         selectionMode={SelectionMode.single}
         layoutMode={DetailsListLayoutMode.justified}
         compact={false}
         onRenderRow={renderRow}
      />

   </Stack>

})

export const ToolManagementModal: React.FC<{ toolId: string; toolName?: string, onClose: () => void }> = ({ toolId, toolName, onClose }) => {

   const handler = useConst(new ToolHandler(toolId))

   const { hordeClasses } = getHordeStyling();

   let headerText = toolName ?? "Tool Management";


   return <Stack>
      <Modal isModeless={false} key="tool_management_key" isOpen={true} isBlocking={true} topOffsetFixed={true} styles={{ main: { padding: 8, width: 700, hasBeenOpened: false, top: "80px", position: "absolute" } }} className={hordeClasses.modal} onDismiss={() => { onClose() }}>
         <Stack horizontal styles={{ root: { padding: 8 } }}>
            <Stack>
               <Text variant="mediumPlus">{headerText}</Text>
            </Stack>
            <Stack grow />
            <Stack>
               <IconButton
                  iconProps={{ iconName: 'Cancel' }}
                  ariaLabel="Close popup modal"
                  onClick={() => { onClose(); }}
               />
            </Stack>
         </Stack>
         <ToolManagementInner handler={handler} />
      </Modal>
   </Stack>
}

