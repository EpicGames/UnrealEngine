/* Copyright Epic Games, Inc. All Rights Reserved. */

body {
  margin: 0;
  font-family: "Horde Open Sans Regular";
  -webkit-font-smoothing: antialiased;
  -moz-osx-font-smoothing: grayscale;
  overflow: hidden;  
}

*::-webkit-scrollbar {
  width: 12px;
}

a,
a:link,
a:visited,
a:focus,
a:hover,
a:active {  
  text-decoration: none;   
}

:focus {
  outline: none;
}


code {
  font-family: source-code-pro, Menlo, Monaco, Consolas, 'Courier New',
    monospace;
}

.log-link,
.log-link:visited,
.log-link:hover {
  text-decoration: none;
}

.horde-link,
.horde-link:visited {
  font-family: "Horde Open Sans Regular";
  text-decoration: none;
}

.horde-link:hover {
  font-family: "Horde Open Sans Regular";
  text-decoration: underline;
}

.view-log-link,
.view-log-link:visited,
.view-log-link:hover {
  text-decoration: none;
  user-select: none;
  cursor: pointer;
}

.view-log-link-disabled,
.view-log-link-disabled:visited,
.view-log-link-disabled:hover {
  text-decoration: none;
  user-select: none;
}

.cl-callout-button {
  text-decoration: none;
  white-space: nowrap;
  user-select: none;
  font-family: "Horde Open Sans SemiBold";
  font-size: 10px;
  border-width: 0;
}

.cl-callout-button:hover, .cl-callout-button-user:hover {
  filter: brightness(1.2);
}

.cl-callout-button-user {
  text-decoration: none;
  white-space: nowrap;
  user-select: none;
  font-family: "Horde Open Sans SemiBold";
  font-size: 10px;
  border-width: 0;  
}

@keyframes red-pulse {
  0% {
    background-color: #CC0000;
    border: 1px solid #CC0000;
  }

  50% {
    background-color: #FF0000;
    border: 1px solid #FF0000;
  }

  100% {
    background-color: #CC0000;
    border: 1px solid #CC0000;
  }

}

@keyframes hordeFadeIn {
  from {
    opacity: 0;
  }

  to {
    opacity: 1;
  }
}