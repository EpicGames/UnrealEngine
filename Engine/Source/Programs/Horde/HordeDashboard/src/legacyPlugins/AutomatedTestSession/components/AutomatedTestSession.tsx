// Copyright Epic Games, Inc. All Rights Reserved.

import { ActionButton, Checkbox, DefaultButton, DirectionalHint, Dropdown, HoverCard, HoverCardType, IContextualMenuProps, IDropdownOption, IPlainCardProps, ITextFieldProps, IconButton, Image, Modal, Pivot, PivotItem, ProgressIndicator, Spinner, SpinnerSize, Stack, Text, TextField } from '@fluentui/react';
import { FontIcon } from '@fluentui/react/lib/Icon';
import { mergeStyleSets, mergeStyles } from '@fluentui/react/lib/Styling';
import React, { useEffect, useMemo, useReducer, useRef, useState } from 'react';
import { generatePath, useNavigate, useParams } from 'react-router';
import { Link } from 'react-router-dom';
import dashboard, { StatusColor } from "../../../backend/Dashboard";
import { msecToElapsed } from '../../../base/utilities/timeUtils';
import { BreadcrumbItem, Breadcrumbs } from '../../../components/Breadcrumbs';
import { useQuery } from "horde/base/utilities/hooks";
import { getProjectName, getStreamData, testDataHandler } from '../../../components/TestReportView';
import { TopNav } from '../../../components/TopNav';
import { getHordeStyling } from '../../../styles/Styles';
import { getHordeTheme } from '../../../styles/theme';
import { ArtifactType, EventType, FilterType, Loader, MetaFilterTools, MetaWrapper, Metadata, MetadataFilter, Section, SectionCollection, SectionFilter, TestCase, TestDevice, TestEvent, TestResult, TestSessionCollection, TestSessionWrapper, TestState, TestStats } from '../models/AutomatedTestSession';

const gutterClass = mergeStyles({
   borderLeftWidth: 6,
   padding: 0,
   margin: 0,
   paddingTop: 0,
   paddingBottom: 0,
   paddingRight: 8,
   marginTop: 0,
   marginBottom: 0,
   height: 20
});

let _styles: any;
let _stateColorStyles: any;

const getStyling = () => {

   const theme = getHordeTheme();
   const colors = dashboard.getStatusColors();

   const styles = _styles ?? mergeStyleSets({
      container: {
         overflow: 'auto'
      },
      event: [
         {
            fontSize: "11px",
            fontFamily: "Horde Cousine Regular"
         }
      ],
      gutter: [
         {
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 14,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      gutterError: [
         {
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
            borderLeftStyle: 'solid',
            borderLeftColor: colors.get(StatusColor.Failure)!
         }, gutterClass
      ],
      gutterWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",            
            borderLeftStyle: 'solid',
            borderLeftColor: colors.get(StatusColor.Warnings)!
         }, gutterClass
      ],
      gutterSuccess: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: colors.get(StatusColor.Success)!
         }, gutterClass
      ],
      eventWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",
         }
      ],
      eventError: [
         {
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
         }
      ],
      itemHover: {
         cursor: "pointer",
         selectors: {
            ':hover': {
               background: theme.palette.neutralLight
            }
         }
      },
      icon: {
         userSelect: 'none',
         fontSize: '16px' //'1.2vw'
      },
      sectionItem: {
         zIndex: 1,
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid",
         background: dashboard.darktheme ? "#111" : "#FFF",
         padding: "6px",
         overflow: "auto"
      },
      plainCard: {
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid",
         background: dashboard.darktheme ? "#111" : "#FFF",
         padding: "6px",
         display: 'flex',
      },
      filterCard: {
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid",
         background: dashboard.darktheme ? "#111" : "#FFF",
         padding: "6px",
         display: 'flex',
      },
      bottomLoadMore: {
         borderWidth: "1px",
         borderColor: "#CCC",
         borderStyle: "solid",
         padding: 3,
         background: dashboard.darktheme ? "#111" : "#FFF",
         fontSize: 11
      },
      compactItem: {
         paddingLeft: 6
      },
      compactItemRight: {
         paddingRight: 6
      },
      reportLeftPanel: {
         borderWidth: "1px",
         borderColor: "#CCC",
         borderStyle: "solid",
         borderRadius: "5px",
         width: "220px",
         padding: 4,
         background: dashboard.darktheme ? "#111" : "#FFF",
      },
      reportPivotItem: {
         paddingTop: 8
      },
      bottomArrow: {
         fontSize: 0,
         position: "relative",
         selectors: {
            ':before': {
               content: "''",
               display: 'inline-block',
               width: 0,
               height: 0,
               border: '8px solid transparent',
               verticalAlign: 'middle',
               borderBottomColor: '#666',
               position: "absolute",
               transform: 'translateY(9px)',
            }
         }
      },
      rotateText: {
         transform: 'translateX(15px) translateY(15px) rotate(90deg)',
         transformOrigin: '0 0',
         position: 'absolute'
      },
      borderSolid: {
         border: '1px solid transparent',
      },
      sideFilterButton: {
         padding: 4,
         minWidth: 0,
         marginLeft: '4px !important'
      },
      stripes: {
         backgroundImage: 'repeating-linear-gradient(-45deg, rgba(255, 255, 255, .2) 25%, transparent 25%, transparent 50%, rgba(255, 255, 255, .2) 50%, rgba(255, 255, 255, .2) 75%, transparent 75%, transparent)',
      }
   });

   const stateColorStyles = _stateColorStyles ?? new Map<string, string>([
      [TestState.Success, colors.get(StatusColor.Success)!],
      [TestState.InProcess, colors.get(StatusColor.Running)!],
      [TestState.NotRun, colors.get(StatusColor.Waiting)!],
      [TestState.SuccessWithWarnings, colors.get(StatusColor.Warnings)!],
      [TestState.Failed, colors.get(StatusColor.Failure)!],
      [TestState.Unknown, "#000000"],
   ]);

   _styles = styles;
   _stateColorStyles = stateColorStyles;

   return {
      theme: theme,
      styles: _styles,
      stateColorStyles: _stateColorStyles
   }
}


const getIconTestStateStyles = (state: string): string => {

   const { styles, stateColorStyles } = getStyling();

   let color: string = "#000000";
   if (!stateColorStyles.has(state)) {
      color = stateColorStyles.get(TestState.Unknown)!;
   } else {
      color = stateColorStyles.get(state)!;
   }

   return mergeStyles({ color: color, cursor: "pointer" }, styles.icon);
}

const stateIconName = new Map<string, string>([
   [TestState.Success, "BoxCheckmarkSolid"],
   [TestState.InProcess, "BoxPlaySolid"],
   [TestState.NotRun, "CheckboxFill"],
   [TestState.SuccessWithWarnings, "BoxCheckmarkSolid"],
   [TestState.Failed, "BoxMultiplySolid"],
   [TestState.Skipped, 'BorderDash'],
   [TestState.Unknown, "SquareShape"],
]);
const getStateIconName = (state: string): string => {
   if (!stateIconName.has(state)) {
      return stateIconName.get(TestState.Unknown)!;
   }
   return stateIconName.get(state)!;
}

const stateLabels = new Map<string, string>([
   [TestState.Success, 'Passed'],
   [TestState.InProcess, 'Incomplete'],
   [TestState.NotRun, 'Unexecuted'],
   [TestState.SuccessWithWarnings, 'Passed'],
   [TestState.Failed, 'Failed'],
   [TestState.Skipped, 'Skipped'],
   [TestState.Unknown, 'Unknown'],
]);
const getStateLabel = (state: string): string | undefined => { return stateLabels.has(state) ? stateLabels.get(state) : state }

const copyToClipboard = (value: string | undefined) => {

   if (!value) {
      return;
   }

   const el = document.createElement('textarea');
   el.value = value;
   document.body.appendChild(el);
   el.select();
   document.execCommand('copy');
   document.body.removeChild(el);
}

const missingImage = "/images/missing-image.png";
const MissingImageLabel = (): JSX.Element => { return <span style={{ fontWeight: 'bold' }}> [missing image]</span> }
type imageData = { link?: string, ref?: string }
type ImageLinks = { [key: string]: imageData; }

const Stats = (stats: TestStats): JSX.Element => {
   const { stateColorStyles } = getStyling();
   return (
      <Stack horizontal verticalAlign="center">
         <Text styles={{ root: { fontWeight: 'bold' } }}>
            <span>[ </span>
            {stats.Failed > 0 && <span>{stats.Failed} <span style={{ fontSize: 12 }} className={mergeStyles({ color: stateColorStyles.get(TestState.Failed) })}>{stateLabels.get(TestState.Failed)?.toLowerCase()}</span>. </span>}
            {stats.Incomplete > 0 && <span>{stats.Incomplete} <span style={{ fontSize: 12 }} className={mergeStyles({ color: stateColorStyles.get(TestState.InProcess) })}>{stateLabels.get(TestState.InProcess)?.toLowerCase()}</span>. </span>}
            {stats.Unexecuted > 0 && <span>{stats.Unexecuted} <span style={{ fontSize: 12 }} className={mergeStyles({ color: stateColorStyles.get(TestState.NotRun) })}>{stateLabels.get(TestState.NotRun)?.toLowerCase()}</span>. </span>}
            {stats.Passed > 0 && <span>{stats.Passed} <span style={{ fontSize: 12 }} className={mergeStyles({ color: stateColorStyles.get(TestState.Success) })}>{stateLabels.get(TestState.Success)?.toLowerCase()}</span>. </span>}
            {stats.Skipped > 0 && <span>{stats.Skipped} <span style={{ fontSize: 12 }} className={mergeStyles({ color: stateColorStyles.get(TestState.Skipped) })}>{stateLabels.get(TestState.Skipped)?.toLowerCase()}</span>. </span>}
            <span> ]</span>
         </Text>
      </Stack>
   );
}

const ItemMeta = (item: TestResult, metaMask: string[]): JSX.Element => {
   const { styles } = getStyling();
   const meta = item.Session.MetaHandler;
   const metaInfo: string[] | undefined = meta?.maskByKeys(metaMask);
   return (
      <Link to={onClickTestResult(item.TestUID, item.Session.MetaHandler?.hash)} key={item.Session?.MetaHandler?.hash}>
         <Stack styles={{ root: { fontSize: 0 } }} disableShrink horizontal>
            <Stack className={styles.compactItem}><FontIcon title={getStateLabel(item.State)} className={getIconTestStateStyles(item.State)} iconName={getStateIconName(item.State)} /></Stack>
            {metaInfo && metaInfo.length > 0 && <Text nowrap>{metaInfo.map((info) => <span className={styles.compactItem} key={info}>{info}</span>)}</Text>}
            <Text><span className={styles.compactItem}>#{item.Session.Testdata.change}</span></Text>
         </Stack>
      </Link>
   );
}

const OnIconCard = (item: TestCase): JSX.Element => {
   const { styles } = getStyling();
   const metas = item.getResults(true);
   const metaMask = metas.length > 1 ? GetCommonMetaKeys(item.getResults().map((item) => item.Session.Metadata)) : [];
   return (
      <Stack className={styles.plainCard}>
         <Text>{item.Name}</Text>
         {metas.map((meta) => ItemMeta(meta, metaMask))}
      </Stack>
   );
};

const OnTestCard = (item: TestResult): JSX.Element => {
   const { styles } = getStyling();
   const meta = item.Session.MetaHandler;
   return <Stack className={styles.plainCard}>
      {TestInfoItemRow('Changelist', item.Session.Testdata.change)}
      {meta?.map((key) => TestInfoItemRow(key, meta.get(key)))}
   </Stack>;
};

const OnDeviceCard = (device: TestDevice): JSX.Element => {
   const { styles } = getStyling();
   const meta = device.Metadata && new MetaWrapper(device.Metadata);
   return <Stack className={styles.plainCard}>
      {meta && meta.map((key, value) => TestInfoItemRow(key, value))}
      {!meta && <Text>no metadata</Text>}
   </Stack>;
};

const LoadingSpinner = (text: string, overlay: boolean = false): JSX.Element => {
   return (
      <Stack>
         <Stack.Item align="center" styles={{ root: [{ padding: 10 }, overlay ? { position: 'fixed', bottom: '20px', left: '20px' } : undefined] }}>
            <Text variant="mediumPlus">{text}</Text>
            <Spinner styles={{ root: { padding: 10 } }} size={SpinnerSize.large}></Spinner>
         </Stack.Item>
      </Stack>
   );
};

const setQuery = (query: URLSearchParams, type: FilterType, value?: string) => {
   if (query.has(type)) {
      if (value) {
         query.set(type, value);
      } else {
         query.delete(type);
      }
   } else if (value) {
      query.append(type, value);
   }
}

const urlFromFilterType = (type: FilterType, target?: string) => {
   const searchQueries = new URLSearchParams(window.location.search);
   setQuery(searchQueries, type, target);
   return `${window.location.pathname}?${searchQueries.toString()}`;
}

const urlFromFilter = (filter: Metadata) => {
   const searchQueries = new URLSearchParams(window.location.search);
   const keys = Object.keys(filter);
   keys.forEach((key) => {
      setQuery(searchQueries, key as FilterType, filter[key]);
   });
   return `${window.location.pathname}?${searchQueries.toString()}`;
}

const urlFromMetaFilterType = (key: string, value?: string[]) => {
   const searchQueries = new URLSearchParams(window.location.search);
   if (searchQueries.has(FilterType.meta)) {
      let filters = searchQueries.getAll(FilterType.meta);
      searchQueries.delete(FilterType.meta);
      filters = filters.filter((meta) => !MetaFilterTools.Match(key, meta));
      filters.forEach((meta) => searchQueries.append(FilterType.meta, meta));
   }
   if (value && value.length > 0) {
      searchQueries.append(FilterType.meta, MetaFilterTools.Encode(key, value));
   }
   return `${window.location.pathname}?${searchQueries.toString()}`;
}

const onClickTestResult = (testuid?: string, metauid?: string, change?: number, view?: string) => {
   return urlFromFilter({ testuid: testuid, metauid: metauid, testchange: change?.toString(), view: view });
}

const IconItem = (item: TestCase): JSX.Element => {
   const plainCardProps: IPlainCardProps = {
      renderData: item,
      onRenderPlainCard: OnIconCard,
      gapSpace: 8,
   };
   return (
      <Stack.Item styles={{ root: { fontSize: 0 } }} key={item.TestUID}>
         <HoverCard plainCardProps={plainCardProps} type={HoverCardType.plain} cardOpenDelay={200}>
            <Link to={onClickTestResult(item.TestUID, item.StateMetaUID)}><FontIcon title={getStateLabel(item.State)} className={getIconTestStateStyles(item.State)} iconName={getStateIconName(item.State)} /></Link>
         </HoverCard>
      </Stack.Item>
   );
}

const CompactItem = (item: TestResult, metaMask: string[], highlighted: boolean = false): JSX.Element => {

   const { styles, stateColorStyles } = getStyling();

   const plainCardProps: IPlainCardProps = {
      renderData: item,
      onRenderPlainCard: OnTestCard,
      directionalHint: DirectionalHint.rightTopEdge,
      gapSpace: 8,
   };
   const meta = item.Session.MetaHandler;
   const metaInfo: string[] | undefined = meta?.maskByKeys(metaMask);

   return (
      <Stack.Item styles={{ root: { fontSize: 0 } }} key={item.TestUID + item.Session?.MetaHandler?.hash} disableShrink style={{ background: highlighted ? 'rgba(0,0,0,0.1)' : undefined }}>
         <HoverCard plainCardProps={plainCardProps} type={HoverCardType.plain} cardOpenDelay={200}>
            <Link to={onClickTestResult(item.TestUID, item.Session.MetaHandler?.hash)}>
               <Stack horizontal>
                  <Stack className={styles.compactItem}><FontIcon title={getStateLabel(item.State)} className={getIconTestStateStyles(item.State)} iconName={getStateIconName(item.State)} /></Stack>
                  {metaInfo && metaInfo.length > 0 && <Text nowrap>{metaInfo.map((info) => <span className={styles.compactItem} key={info}>{info}</span>)}</Text>}
                  {item.ErrorCount > 0 && <Text className={styles.compactItem} nowrap>({item.ErrorCount} <span style={{ color: stateColorStyles.get(TestState.Failed), fontWeight: 'bold' }}>errors</span>)</Text>}
                  {item.WarningCount > 0 && <Text className={styles.compactItem} nowrap>({item.WarningCount} <span style={{ color: stateColorStyles.get(TestState.SuccessWithWarnings), fontWeight: 'bold' }}>warnings</span>)</Text>}
               </Stack>
            </Link>
         </HoverCard>
      </Stack.Item>
   );
}

const TestDetailItemRow = (label: string, element?: JSX.Element): JSX.Element => {
   return (
      <Stack horizontal key={label}>
         <Stack.Item styles={{ root: { width: '90px' } }}><Text style={{ fontWeight: 'bold' }} nowrap>{label}:</Text></Stack.Item>
         {element && <Stack.Item>{element}</Stack.Item>}
      </Stack>
   );
}

const TestInfoItemRow = (label: string, value?: any, color?: string): JSX.Element => {
   return TestDetailItemRow(label, value && <Text styles={{ root: { color: color } }}>{value.toString()}</Text>);
}

const TestResultModal: React.FC<{ testuid: string, collection: TestSessionCollection, onLoadMore?: () => void }> = (props) => {
   const { testuid, collection, onLoadMore } = props;
   const [loading, setLoading] = useState(true);
   const [loadingTest, setLoadingTest] = useState(true);
   const testRef = useRef<TestResult | undefined>(undefined);
   const testcaseRef = useRef<TestCase | undefined>(undefined);
   const navigate = useNavigate();
   const onClickCancel = () => {
      navigate(onClickTestResult());
   }
   const onClickPivot = (item?: PivotItem) => {
      navigate(urlFromFilterType(FilterType.view, item?.props.itemKey), { replace: true });
   }

   const query = useQuery();
   const metauid = query.get(FilterType.metauid) ?? undefined;
   const change = parseInt(query.get(FilterType.testchange) ?? "");
   const view = query.get(FilterType.view) ?? undefined;

   useEffect(() => {
      const cancel = { current: false };
      const assignTestFromCache = async () => {
         if (!testcaseRef.current) {
            setLoadingTest(false);
            return;
         }
         setLoadingTest(true);
         try {
            const testcase = testcaseRef.current;
            if (testcase && metauid) {
               const testFromQuery = await collection.getTestResultByQuery(testuid, metauid, change);
               if (testFromQuery && !cancel.current) {
                  testRef.current = testFromQuery;
               }
            }
            if (!cancel.current && !testRef.current) {
               const first_metauid = testcase.getResults(true)[0].Session.MetaHandler!.hash;
               testRef.current = await collection.getTestResultByQuery(testuid, first_metauid);
            }
         } catch (reason) {
            console.error(reason);
         } finally { !cancel.current && setLoadingTest(false) }
      }
      const getTestResults = async () => {
         setLoading(true);
         try {
            const testcase = await collection.getTestCaseByUID(testuid);
            if (testcase && !cancel.current) {
               testcaseRef.current = testcase;
            }
         } catch (reason) {
            console.error(reason);
         } finally { if (!cancel.current) { setLoading(false); assignTestFromCache() } }
      }

      const test = testRef.current;
      if (!test || testuid !== test.TestUID) {

         testRef.current = undefined;
         testcaseRef.current = undefined;
         getTestResults();

      } else if (testcaseRef.current
         && (!test || metauid !== test.Session.MetaHandler?.hash || change !== test.Session.Testdata.change)) {

         testRef.current = undefined;
         assignTestFromCache();
      }
      return function cleanup() { cancel.current = true }
   }, [testuid, metauid, change, collection])

   const test = testRef.current;
   const meta = test?.Session.MetaHandler;
   const alternates = testcaseRef.current?.getResults(true);
   const metaMask: string[] = alternates && !testDataHandler.cursor ? GetCommonMetaKeys(alternates.map((item) => item.Session.Metadata)) : [];
   const selectedChange = change ? change : (test?.Session.Testdata.change ?? 0);
   let testName: string | undefined = undefined;
   if (testcaseRef.current) {
      testName = testcaseRef.current.Name;
   }

   const { hordeClasses, modeColors } = getHordeStyling();
   const { styles } = getStyling();

   return (
      <Modal isOpen styles={{ main: { padding: 8, width: '100vw', height: 'calc(100vh - 24px)', background: modeColors.background} }} className={hordeClasses.modal} onDismiss={onClickCancel}>
         <Stack>
            <Stack horizontal styles={{ root: { padding: 8 } }}>
               <Stack verticalAlign="center" styles={{ root: { overflow: 'hidden' } }}>
                  <Text styles={{ root: { fontWeight: "unset", fontFamily: "Horde Open Sans SemiBold", fontSize: "16px", color: "#087BC4", textOverflow: 'ellipsis' } }} nowrap block>{testName ?? (loading ? "Loading..." : "Unknown test")}</Text>
               </Stack>
               <Stack verticalAlign="center" grow>
                  <div><FontIcon iconName="AddToShoppingList" title="Copy Name to Clipboard" onClick={(ev) => { copyToClipboard(testName); }} style={{ padding: 6, fontSize: 18 }} className={styles.itemHover} /></div>
               </Stack>
               <Stack.Item align="end" disableShrink>
                  <IconButton iconProps={{ iconName: 'Cancel' }} ariaLabel="Close test result" onClick={onClickCancel} />
               </Stack.Item>
            </Stack>
            {!loading && alternates &&
               <Stack horizontal styles={{ root: { paddingLeft: 8, paddingRight: 8 } }} tokens={{ childrenGap: 16 }} verticalFill disableShrink>
                  <Stack.Item className={styles.reportLeftPanel} grow>
                     <Pivot selectedKey={view} onLinkClick={onClickPivot}>
                        <PivotItem headerText="General" className={styles.reportPivotItem}>
                           {!loadingTest && test?.Details !== undefined && TestDetails(test, alternates, metaMask)}
                           {loadingTest && LoadingSpinner('Loading Details')}
                        </PivotItem>
                        <PivotItem headerText="History" className={styles.reportPivotItem} itemKey="history">
                           {testcaseRef.current && <TestHistory collection={collection} testcase={testcaseRef.current} metaMask={metaMask} metauid={metauid ?? meta?.hash} change={selectedChange} onLoadMore={onLoadMore} />}
                        </PivotItem>
                        <PivotItem headerText="Artifacts" className={styles.reportPivotItem} itemKey="artifacts">
                           Available soon
                        </PivotItem>
                     </Pivot>
                  </Stack.Item>
                  <Stack.Item grow style={{ width: '100%' }} className={styles.container}>
                     {!loadingTest && test?.Details !== undefined && <TestEvents test={test} />}
                     {loadingTest && LoadingSpinner('Loading Details')}
                  </Stack.Item>
               </Stack>
            }
            {!loading && !loadingTest && !test && <Stack styles={{ root: { paddingLeft: 8, paddingRight: 8 } }}><Text>Could not find Test {testuid} for meta {metauid}</Text></Stack>}
            {loading && LoadingSpinner('Loading Test Result')}
         </Stack>
      </Modal>
   );
}

const TestDetails = (test: TestResult, alternates: TestResult[], metaMask: string[]): JSX.Element => {
   const { stateColorStyles } = getStyling();
   const meta = test.Session.MetaHandler;
   const devices = test.Session.getDevices(test.DeviceAppInstanceName);

   return (
      <Stack>
         {TestDetailItemRow('State',
            <Stack horizontal wrap tokens={{ childrenGap: 6 }}>
               <FontIcon className={getIconTestStateStyles(test.State)} iconName={getStateIconName(test.State)} />
               <Text style={{ color: stateColorStyles.get(test.State) }}>{getStateLabel(test.State)}</Text>
            </Stack>)
         }
         {TestInfoItemRow('Errors', test.ErrorCount ?? 0, stateColorStyles.get(TestState.Failed))}
         {TestInfoItemRow('Warnings', test.WarningCount ?? 0, stateColorStyles.get(TestState.SuccessWithWarnings))}
         <Stack>
            {TestInfoItemRow('Conditions')}
            <Stack.Item styles={{ root: { marginLeft: 30 } }}>
               {TestInfoItemRow('Changelist', test.Session.Testdata.change)}
               {meta?.map((key, value) => TestInfoItemRow(key, value))}
            </Stack.Item>
         </Stack>
         {test.DateTime && TestInfoItemRow('Run at', test.DateTime)}
         {test.TimeElapseSec && TestInfoItemRow('For', msecToElapsed(test.TimeElapseSec * 1000))}
         {devices && devices.length > 0 &&
            TestDetailItemRow('Devices', <Stack>{devices.map((device) => TestDeviceItem(device))}</Stack>)
         }
         {TestDetailItemRow('Link to', <Link to={test.Session.Testdata.getJobStepLink()}>Job Step</Link>)}
         {alternates.length > 0 &&
            <Stack>
               {TestInfoItemRow(testDataHandler.cursor ? `Result on ${alternates[0].Session.Testdata.change}` : 'Latest Alternates')}
               <Stack>
                  {alternates.map((item) => CompactItem(item, metaMask, item.Session.MetaHandler!.hash === meta!.hash))}
               </Stack>
            </Stack>
         }
      </Stack>
   );
}

const TestDeviceItem = (device: TestDevice): JSX.Element => {
   const deviceCardProps: IPlainCardProps = {
      renderData: device,
      onRenderPlainCard: OnDeviceCard,
      directionalHint: DirectionalHint.rightTopEdge,
      gapSpace: 8,
   };

   return (
      <HoverCard plainCardProps={deviceCardProps} type={HoverCardType.plain} cardOpenDelay={20} key={device.AppInstanceName}>
         <Text style={{ cursor: 'pointer' }}>{device.Name}</Text>
      </HoverCard>
   );

}

const TestHistory: React.FC<{ collection: TestSessionCollection, testcase: TestCase, metaMask: string[], metauid?: string, change: number, onLoadMore?: () => void }> = (props) => {
   const { collection, testcase, metaMask, metauid, change, onLoadMore } = props;
   const [loading, setLoading] = useState(true);
   const [history, updateHistory] = useState(testcase.history);
   const [needFetching, forceFetch] = useReducer(x => x + 1, 1);
   const testuid = testcase.TestUID;

   useEffect(() => {
      const cancel = { current: false };
      const historyChangeDisposer = testcase.reactOnHistoryChange((version) => updateHistory(version));
      const FetchTestResultHistory = async () => {
         setLoading(true);
         try {
            await collection.fetchSessionHistory(testcase.SessionName, testcase.TestUID);
         } catch (reason) {
            console.error(reason);
         } finally { !cancel.current && setLoading(false) }
      }
      needFetching && FetchTestResultHistory();
      return function cleanup() { cancel.current = true; historyChangeDisposer() }
   }, [testcase, collection, needFetching])

   const { styles } = getStyling();

   let metasRef: MetaWrapper[] | undefined = undefined;
   let historyMetaMask: string[] = metaMask;
   const changesByMeta = collection.getLoadedTestHistoryByMeta(testcase.TestUID);
   if (changesByMeta && changesByMeta.size > 0) {
      // get the sorted list of meta uids
      const metauids = testDataHandler.cursor ?
         Array.from(changesByMeta.keys()).sort()
         : testcase.getResults(true).map((test) => test.Session.MetaHandler!.hash);
      if (testDataHandler.cursor) {
         historyMetaMask = GetCommonMetaKeys(metauids.map((key) => changesByMeta?.get(key)![0].Session.Metadata));
      }
      metasRef = metauids.map((key) => changesByMeta?.get(key)![0].Session.MetaHandler!);
   }

   return (
      <Stack style={{ height: 'calc(100vh - 160px)' }} className={styles.container}>
         {loading && <LoaderProgressBar loader={testcase} target={testcase.historyTarget} />}
         {!loading && onLoadMore &&
            <Stack className={styles.bottomLoadMore} style={{ fontSize: 11 }} horizontal horizontalAlign="center">
               <Stack style={{ color: "rgb(0, 120, 212)", cursor: 'pointer' }} onClick={() => { onLoadMore(); forceFetch() }}>Load More History...</Stack>
            </Stack>
         }
         {history > -1 && metasRef &&
            <Stack tokens={{ childrenGap: 6 }} className={styles.container} verticalFill>
               {metasRef.map((meta) => TestHistoryMeta(
                  meta, historyMetaMask, changesByMeta?.get(meta.hash) ?? [], testuid, meta.hash === metauid, change)
               )
               }
            </Stack>
         }
      </Stack>
   );
}

const TestHistoryMeta = (meta: MetaWrapper, metaMask: string[], tests: TestResult[], testuid: string, selected: boolean, change: number): JSX.Element => {

   const { styles } = getStyling();

   let metaInfo: string[] = meta.maskByKeys(metaMask);
   if (metaInfo.length === 0) {
      metaInfo = meta.map((key, value) => value);
   }
   return (
      <Stack key={meta.hash}>
         <Text variant="smallPlus">{metaInfo.map((info) => <span className={styles.compactItemRight} key={info}>{info}</span>)}</Text>
         <Stack horizontal style={{ paddingLeft: 6 }}>
            {tests.map((test) => <TestHistoryItem key={test.Session.Testdata.change} test={test} selected={selected && test.Session.Testdata.change === change} />)}
         </Stack>
      </Stack>
   );
}

const TestHistoryItem: React.FC<{ test: TestResult, selected: boolean }> = (props) => {
   const { test, selected } = props;
   const navigate = useNavigate();
   const { styles } = getStyling();

   const hoverCardProps: IPlainCardProps = {
      renderData: test,
      onRenderPlainCard: TestHistoryItemCard,
      directionalHint: DirectionalHint.topCenter,
      gapSpace: 8,
      calloutProps: { isBeakVisible: true }
   };

   const onClickIcon = () => navigate(
      urlFromFilter({
         metauid: test.Session.MetaHandler?.hash,
         testchange: test.Session.Testdata.change?.toString(),
         view: 'history'
      })
   );

   const scrollToSelected = (ref: any) => {
      if (ref && selected) {
         ref.scrollIntoView({ behavior: 'smooth', block: 'start' });
      }
   }

   return (
      <HoverCard plainCardProps={hoverCardProps} type={HoverCardType.plain} cardOpenDelay={20}>
         <div className={selected ? styles.bottomArrow : undefined} ref={selected ? scrollToSelected : undefined}>
            {test && <FontIcon title={getStateLabel(test.State)} className={getIconTestStateStyles(test.State)} iconName={getStateIconName(test.State)} onClick={onClickIcon} />}
         </div>
      </HoverCard>
   );
}

const TestHistoryItemCard = (test: TestResult): JSX.Element => {
   const { styles } = getStyling();
   return <Stack className={styles.plainCard}>
      {TestInfoItemRow('Changelist', test.Session.Testdata.change)}
   </Stack>;
};

const TestEvents: React.FC<{ test: TestResult }> = (props) => {
   const { test } = props;
   const [filterError, setFilterError] = useState(true);
   const [filterWarning, setFilterWarning] = useState(true);
   const [filterInfo, setFilterInfo] = useState(test.State !== TestState.Failed);
   const { styles } = getStyling();

   function onChangeFilterError(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterError(checked);
   }
   function onChangeFilterWarning(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterWarning(checked);
   }
   function onChangeFilterInfo(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterInfo(checked);
   }

   function isEntryNeedDisplay(entry: TestEvent, index?: number, array?: TestEvent[]): boolean {
      const entryEventType = entry.Type;
      if (filterError && entryEventType === EventType.Error) {
         return true;
      }
      if (filterWarning && entryEventType === EventType.Warning) {
         return true;
      }
      if (filterInfo && entryEventType === EventType.Info) {
         return true;
      }

      return false;
   }

   return (
      <Stack styles={{ root: { width: '100%' } }} className={styles.container}>
         <Stack horizontal tokens={{ childrenGap: 10 }} styles={{ root: { padding: 5 } }}>
            <FontIcon iconName="PageListFilter" title="Filters" className={styles.icon} />
            <Checkbox label="Error" defaultChecked={filterError} onChange={(onChangeFilterError)} />
            <Checkbox label="Warning" defaultChecked={filterWarning} onChange={onChangeFilterWarning} />
            <Checkbox label="Info" defaultChecked={filterInfo} onChange={onChangeFilterInfo} />
         </Stack>
         <Stack styles={{ root: { height: 'calc(100vh - 125px)', width: '100%', overflow: 'auto' } }}>
            {test.Details.Events.filter(isEntryNeedDisplay).map((value, index) => <EventPane key={index} entry={value} test={test} />)}
            {test.Details.Events.length === 0 && <Text styles={{ root: { padding: 8, fontWeight: 'bold' } }}>No event for this test.</Text>}
         </Stack>
      </Stack>
   );
}

const EventPane: React.FC<{ entry: TestEvent, test: TestResult }> = (props) => {

   const { styles } = getStyling();

   const { entry, test } = props;
   const eventType = entry.Type;
   const is_error = eventType === EventType.Error;
   const is_warning = eventType === EventType.Warning;
   const style = is_error ? styles.eventError : (is_warning ? styles.eventWarning : styles.event);
   const gutterStyle = is_error ? styles.gutterError : (is_warning ? styles.gutterWarning : styles.gutter);

   const [imageInfo, setImageInfo] = useState<ImageLinks>({});
   const need_image_comparison = entry.Artifacts && entry.Artifacts.length > 0 && entry.Tag === ArtifactType.ImageCompare;
   const need_image_comparisonRef = useRef(need_image_comparison);


   useEffect(() => {
      const cancel = { current: false };
      if (need_image_comparisonRef.current) {
         const findLinks = async () => {
            const imageInfo: ImageLinks = {};
            for (const key in entry.Artifacts) {

               const item = entry.Artifacts[key];
               const url = await test.Session.Testdata?.getArtifactImageLink(item.ReferencePath);
               imageInfo[item.Tag] = { link: url, ref: item.ReferencePath };
            }
            !cancel.current && setImageInfo(imageInfo);
         }
         findLinks();
      }
      return function cleanup() { cancel.current = true }
   }, [entry.Artifacts, test]);

   return (
      <Stack className={styles.event} disableShrink={true} styles={{ root: { paddingLeft: 8, width: "100%" } }}>
         <Stack horizontal>
            <Stack className={gutterStyle}></Stack>
            <Stack.Item className={style} align="center" styles={{ root: { paddingLeft: 8, width: "100%" } }}><pre style={{ margin: 0, whiteSpace: "pre-wrap" }}>[{entry.DateTime}] {eventType}: {entry.Message}</pre></Stack.Item>
         </Stack>
         {need_image_comparison &&
            <Stack styles={{ root: { paddingLeft: 16, width: '100%' } }}>
               <Stack><Text variant="medium"><span style={{ fontWeight: "bold" }}>Image comparison: </span>{entry.Context}</Text></Stack>
               <Stack horizontal>
                  {(is_error || is_warning) &&
                     <Stack styles={{ root: { padding: 5 } }}>
                        <a href={imageInfo[ArtifactType.Approved]?.link}><Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo[ArtifactType.Approved]?.link || missingImage} alt={imageInfo[ArtifactType.Approved]?.ref} /></a>
                        <Stack.Item align="center">Reference{!imageInfo[ArtifactType.Approved] && MissingImageLabel()}</Stack.Item>
                     </Stack>
                  }
                  {(is_error || is_warning) &&
                     <Stack styles={{ root: { padding: 5 } }}>
                        <a href={imageInfo[ArtifactType.Difference]?.link}><Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo[ArtifactType.Difference]?.link || missingImage} alt={imageInfo[ArtifactType.Difference]?.ref} /></a>
                        <Stack.Item align="center">Difference{!imageInfo[ArtifactType.Difference] && MissingImageLabel()}</Stack.Item>
                     </Stack>
                  }
                  <Stack styles={{ root: { padding: 5 } }}>
                     <a href={imageInfo[ArtifactType.Unapproved]?.link}><Image width="100%" style={{ minWidth: 200, maxWidth: 400, minHeight: 120 }} src={imageInfo[ArtifactType.Unapproved]?.link || missingImage} alt={imageInfo[ArtifactType.Unapproved]?.ref} /></a>
                     <Stack.Item align="center">Produced{!imageInfo[ArtifactType.Unapproved] && MissingImageLabel()}</Stack.Item>
                  </Stack>
               </Stack>
            </Stack>
         }
      </Stack>
   );
}

type ChartStack = {
   value: number,
   title?: string,
   color?: string,
   onClick?: () => void,
   stripes?: boolean,

}

const Chart = (stack: ChartStack[], width: number, height: number, basecolor?: string, style?: any): JSX.Element => {

   const { styles } = getStyling();

   const mainTitle = stack.map((item) => `${item.value}% ${item.title}`).join(' ');

   return (
      <div className={mergeStyles({ backgroundColor: basecolor, width: width, height: height, verticalAlign: 'middle' }, style)} title={mainTitle}>
         {stack.map((item) => <span key={item.title!}
            onClick={item.onClick}
            className={item.stripes ? styles.stripes : undefined}
            style={{
               width: `${item.value}%`, height: '100%',
               backgroundColor: item.color,
               display: 'inline-block',
               cursor: item.onClick ? 'pointer' : 'inherit',
               backgroundSize: `${height * 2}px ${height * 2}px`
            }} />)}
      </div>
   );
}

type BreadCrumbFill = {
   title: string,
   linkTo: any,
};

const SectionBreadCrumb = (filling: BreadCrumbFill[], stack?: ChartStack[], total?: number): JSX.Element => {

   const { stateColorStyles } = getStyling();

   return (
      <Stack grow>
         <Stack horizontal wrap>
            {filling.slice(0, -1).map((crumb, index) => <Stack styles={{ root: { padding: 3 } }} key={index}><Link to={crumb.linkTo}><Text>{crumb.title} &gt;</Text></Link></Stack>)}
         </Stack>
         <Stack horizontal verticalAlign="center" styles={{ root: { padding: 6 } }}>
            <Text variant="mediumPlus">{filling[filling.length - 1].title}</Text>
            {total && stack &&
               Chart(stack, 200, 20, stateColorStyles.get(TestState.Success), { margin: '3px !important' })
            }
            {total !== undefined && <Text style={{ fontWeight: 'bold' }}>[ {total} results ]</Text>}
         </Stack>
      </Stack>
   );
}

const sectionCrumbLink = (type: string, target: string) => {
   const searchQueries = new URLSearchParams(window.location.search);
   searchQueries.set(type, target);
   if (type === FilterType.suite) {
      searchQueries.delete(FilterType.name);
   }

   return `${window.location.pathname}?${searchQueries.toString()}`;
}

const noSectionCrumbLink = () => {
   const searchQueries = new URLSearchParams(window.location.search);
   if (searchQueries.has(FilterType.suite)) {
      searchQueries.delete(FilterType.suite);
   }
   if (searchQueries.has(FilterType.name)) {
      searchQueries.delete(FilterType.name);
   }

   return `${window.location.pathname}?${searchQueries.toString()}`;
}

const ResultsFolder: React.FC<{ items: TestCase[], name: string, expanded?: boolean }> = (props) => {
   const { items, name, expanded } = props;
   const [isExpanded, setExpanded] = useState(expanded ?? false);
   const { styles } = getStyling();
   return (
      <Stack styles={{ root: { padding: 5, userSelect: "none" } }}>
         <Stack verticalAlign="center" horizontal styles={{ root: { cursor: 'pointer' } }} onClick={() => setExpanded(!isExpanded)}>
            <FontIcon className={styles.icon} iconName={isExpanded ? "CaretSolidDown" : "CaretSolidRight"} style={{ fontSize: 11 }} />
            <Text variant="smallPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold", paddingLeft: 3 } }}>{name} ({items.length})</Text>
         </Stack>
         {isExpanded &&
            <Stack horizontal wrap tokens={{ childrenGap: 2 }} styles={{ root: { paddingTop: 4 } }}>
               {items.map((testcase) => IconItem(testcase))}
            </Stack>
         }
      </Stack>
   );
}

const ResultsSelector: React.FC<{ items: TestCase[], stats: TestStats }> = (props) => {
   const { items, stats } = props;
   const [isFailureExpanded, setFailureExpanded] = useState(false);
   const [isOtherExpanded, setOtherExpanded] = useState(false);


   const failures = useMemo(() => {
      return items.filter((testcase) => testcase.State !== TestState.Success && testcase.State !== TestState.Skipped)
   }, [items]);

   const others = useMemo(() => {
      const othersFilter: TestCase[] = [];
      items.forEach((testcase) => {
         const filteredCase = testcase.getFilteredByState(TestState.SuccessWithWarnings);
         if (filteredCase) {
            othersFilter.push(filteredCase);
         }
      });
      SectionCollection.sortTestCases(othersFilter);
      return othersFilter;
   }, [items]);

   const failedFactor = Math.ceil((stats.Failed + stats.Incomplete) / (stats.TotalRun || 1) * 20) / 20;
   const { stateColorStyles } = getStyling();

   const stack: ChartStack[] = [
      {
         value: failedFactor * 100,
         title: "Failure",
         color: stateColorStyles.get(TestState.Failed),
         onClick: () => { setFailureExpanded(!isFailureExpanded); setOtherExpanded(false) },
         stripes: true
      },
      {
         value: (1 - failedFactor) * 100,
         title: "Passed",
         color: 'transparent',
         onClick: () => { setFailureExpanded(false); setOtherExpanded(!isOtherExpanded) }
      }
   ]

   return (
      <Stack styles={{ root: { padding: 5, userSelect: "none" } }}>
         <Stack horizontal verticalAlign="center">
            {Chart(stack, 500, 30, stateColorStyles.get(TestState.Success), { margin: '3px !important' })}
         </Stack>
         {isFailureExpanded &&
            <Stack horizontal wrap tokens={{ childrenGap: 2 }} styles={{ root: { paddingTop: 4 } }}>
               {failures.map((testcase) => IconItem(testcase))}
            </Stack>
         }
         {isOtherExpanded &&
            <Stack horizontal wrap tokens={{ childrenGap: 2 }} styles={{ root: { paddingTop: 4 } }}>
               {others.map((testcase) => IconItem(testcase))}
            </Stack>
         }
      </Stack>
   );
}

const GetCommonMetaKeys = (items: Metadata[]) => {
   const first = items[0];
   const commonKeys = new Set(Object.keys(first));
   if (items.length > 1) {
      items.slice(1).forEach((item) => {
         Array.from(commonKeys.values()).forEach((key) => {
            if (first[key] !== item[key]) {
               commonKeys.delete(key);
            }
         });
      });
   }
   return Array.from(commonKeys);
}

const SectionItem = (item: Section): JSX.Element => {
   const { styles } = getStyling();
   const key = item.Type + item.FullName;

   if (item.isOneTest) {
      const testcase = item.TestCases[0];

      return (
         <Stack className={styles.sectionItem} styles={{ root: { background: item.Stats.Failed > 0 ? (dashboard.darktheme ? "#111" : "#FFF") : undefined } }} grow key={key}>
            <Stack verticalAlign="center" horizontal styles={{ root: { paddingBottom: 4 } }}>
               {IconItem(testcase)}
               <Link to={onClickTestResult(testcase.TestUID, testcase.StateMetaUID)}>
                  <Text variant="smallPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold", paddingLeft: 3 } }}>{item.Name}</Text>
               </Link>
            </Stack>
            {testcase.Metas.size > 1 && Stats(testcase.Stats)}
         </Stack>
      );

   } else {
      const onClickSection = () => {
         const searchQueries = new URLSearchParams(window.location.search);
         searchQueries.set(item.Type, item.FullName);
         return `${window.location.pathname}?${searchQueries.toString()}`;
      }

      return (
         <Stack className={styles.sectionItem} styles={{ root: { background: item.Stats.Failed > 0 ? (dashboard.darktheme ? "#111" : "#FFF") : undefined } }} grow key={key}>
            <Link to={onClickSection()} style={{ fontSize: 0, color: 'inherit' }}>
               <Stack horizontal styles={{ root: { paddingBottom: 4 } }}>
                  <FontIcon className={styles.icon} iconName="FolderList" />
                  <Text variant="smallPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold", paddingLeft: 3 } }}>{item.Name}</Text>
               </Stack>
            </Link>
            {item.Stats.TotalRun > 0 && <ResultsSelector items={item.TestCases} stats={item.Stats} />}
            {item.Stats.TotalRun === 0 && <ResultsFolder items={item.TestCases} name="Unexecuted" />}
            {Stats(item.Stats)}
         </Stack>
      );

   }
}

const TestMetaFilter: React.FC<{ metaOptions: Map<string, Set<string>>, filter?: MetadataFilter }> = (props) => {
   const { filter, metaOptions } = props;
   const navigate = useNavigate();
   const newFilter = useRef<string | undefined>(undefined);
   const [metaFilters, setMetaFilters] = useState<{ key: string, values: string[] }[]>(
      (filter ? Array.from(Object.entries(filter)) : []).map((meta) => {
         return { key: meta[0], values: meta[1] }
      }).filter((item) => metaOptions.has(item.key))
   );
   const selectedMeta = new Set(metaFilters.map((item) => item.key));
   const metaCanAddKeys: IDropdownOption[] = Array.from(metaOptions.entries()).map((data) => {
      return { key: data[0], text: data[0] }
   }).filter((item) => !selectedMeta.has(item.key));

   const { styles } = getStyling();

   return (
      <Stack tokens={{ childrenGap: 4 }}>
         {metaFilters.length > 0 &&
            metaFilters.map((entry) => {
               const possibilities = metaOptions.get(entry.key)!;
               const selected = entry.values.filter((item) => possibilities.has(item));
               const options: IDropdownOption[] = Array.from(possibilities.values()).map((item) => { return { key: item, text: item } });
               return (
                  <Stack key={entry.key} horizontal horizontalAlign="end" verticalAlign="center">
                     <Text style={{ fontWeight: 'bold', paddingRight: 4 }}>{entry.key}</Text>
                     <Dropdown
                        placeholder="select options"
                        styles={{ dropdown: { width: 150 } }}
                        onChange={(ev, option) => {
                           const isSelected = option!.selected;
                           const key = option!.key as string;
                           const selectedIndex = selected.findIndex((item) => item === key);
                           if (isSelected) {
                              selectedIndex === -1 && selected.push(key);
                           } else {
                              selectedIndex > -1 && selected.splice(selectedIndex, 1);
                           }
                           navigate(urlFromMetaFilterType(entry.key, selected));
                           setMetaFilters(metaFilters.map((item) => item.key !== entry.key ? item : { key: entry.key, values: selected }));
                        }}
                        multiSelect
                        options={options}
                        defaultSelectedKeys={selected} />
                     <DefaultButton
                        onClick={() => {
                           setMetaFilters(metaFilters.filter((item) => item.key !== entry.key));
                           selected.length > 0 && navigate(urlFromMetaFilterType(entry.key, undefined));
                        }}
                        title="Update filtered results"
                        className={styles.sideFilterButton}>
                        <FontIcon className={styles.icon} iconName="CalculatorMultiply" style={{ fontSize: 11 }} />
                     </DefaultButton>
                  </Stack>
               );
            })
         }
         {metaCanAddKeys.length > 0 &&
            <Stack horizontal horizontalAlign="end" verticalAlign="center">
               <Dropdown
                  placeholder="select filter to Add"
                  key={`add ${metaCanAddKeys.length}`}
                  styles={{ dropdown: { width: 150 } }}
                  options={metaCanAddKeys}
                  onChange={(ev, item) => newFilter.current = item?.key as string} />
               <DefaultButton
                  onClick={() => {
                     newFilter.current && setMetaFilters([...metaFilters, { key: newFilter.current, values: [] }]);
                     newFilter.current = undefined
                  }}
                  title="Add filter"
                  className={styles.sideFilterButton}>
                  <FontIcon className={styles.icon} iconName="CalculatorAddition" style={{ fontSize: 11 }} />
               </DefaultButton>
            </Stack>
         }
      </Stack>
   );
}

const TestNameFilter: React.FC<{ filter: SectionFilter, onChange?: (value: string) => void, onValidate?: (value?: string) => void, dismissMenu?: (ev?: any, dismissAll?: boolean) => void }> = (props) => {
   const { filter, onChange, onValidate, dismissMenu } = props;
   const [name, setName] = useState<string>(filter.namecontains ?? '');

   const { styles } = getStyling();

   return (
      <Stack horizontal horizontalAlign="end">
         <TextField
            deferredValidationTime={750}
            validateOnLoad={false}
            spellCheck={false}
            placeholder="search test name"
            value={name}
            onChange={(ev, value) => setName(value ?? '')}
            onGetErrorMessage={onChange && ((value) => { onChange(value); return undefined })}
            onKeyDown={onValidate && ((ev) => {
               switch (ev.key) {
                  case "Enter":
                     onValidate((ev.target as ITextFieldProps).value);
                     dismissMenu && dismissMenu();
                     break;
                  case "Escape":
                     onValidate(filter.namecontains);
                     break;
               }
            })}
            onBlur={onValidate && ((ev) => onValidate(ev.target.value))}
         />
         <DefaultButton
            onClick={() => { onValidate && onValidate(undefined); setName(''); !filter.meta && dismissMenu && dismissMenu(); }}
            title="Clear filter"
            className={styles.sideFilterButton}>
            <FontIcon className={styles.icon} iconName="CalculatorMultiply" style={{ fontSize: 11 }} />
         </DefaultButton>
      </Stack>
   );
}

const TestFilterCard = (filter: SectionFilter, metaOptions: Map<string, Set<string>>, onChange?: (value: string) => void, onValidate?: (value?: string) => void, dismissMenu?: (ev?: any, dismissAll?: boolean) => void): JSX.Element => {
   return (
      <Stack style={{ padding: 8, paddingTop: 16, paddingBottom: 12 }} tokens={{ childrenGap: 4 }}>
         <TestNameFilter filter={filter} onChange={onChange} onValidate={onValidate} dismissMenu={dismissMenu} />
         <TestMetaFilter filter={filter.meta} metaOptions={metaOptions} />
      </Stack>
   );
}

const TestCollection: React.FC<{ collection: TestSessionCollection, onLoadMore?: () => void }> = (props) => {

   const { modeColors } = getHordeStyling();
   const { styles, stateColorStyles } = getStyling();
   const colors = dashboard.getStatusColors();

   const { collection, onLoadMore } = props;
   const sectionsHandler = useRef<SectionCollection | undefined>(undefined);
   const navigate = useNavigate();
   if (!sectionsHandler.current) {
      sectionsHandler.current = new SectionCollection();
   }
   const [onTheFlyTestNameFilter, setOnTheFlyTestNameFilter] = useState<string | undefined>(undefined);
   const query = useQuery();

   const filter: SectionFilter = {
      suite: query.get(FilterType.suite) ?? undefined,
      name: query.get(FilterType.name) ?? undefined,
      project: query.get(FilterType.project) ?? undefined,
      namecontains: query.get(FilterType.namecontains) ?? undefined,
      meta: MetaFilterTools.Decode(query.getAll(FilterType.meta)),
   };
   const testuid = query.get(FilterType.testuid) ?? undefined;

   const data = useMemo(() => testDataHandler.cursor ? collection.iterSessions().next().value : undefined, [collection]); // for single view mode

   const sections = !onTheFlyTestNameFilter ?
      sectionsHandler.current.extractSections(filter, collection) :
      sectionsHandler.current.reExtractSectionOnTheFly(filter, onTheFlyTestNameFilter, collection);

   const sectionBreadCrumbs: BreadCrumbFill[] = [{ title: 'All', linkTo: noSectionCrumbLink }];
   const totalTests = sections.reduce((a, b) => a + b.Stats.TotalRun, 0);
   const stackFailure: ChartStack = {
      value: Math.ceil(sections.reduce((a, b) => a + b.Stats.Failed + b.Stats.Incomplete, 0) / (totalTests || 1) * 20) * 5,
      title: "Failure",
      color: stateColorStyles.get(TestState.Failed),
      stripes: true
   };
   if (filter.suite) {
      const item: BreadCrumbFill = { title: `${filter.suite} [${FilterType.suite}]`, linkTo: sectionCrumbLink(FilterType.suite, filter.suite!) };
      sectionBreadCrumbs.push(item);
   }
   if (filter.name) {
      const firstSection = sections.length > 0 ? sections[0] : undefined;
      const commonSection = firstSection ? firstSection.FullName.slice(0, firstSection.FullName.length - firstSection.Name.length - 1) : filter.name;
      commonSection.split('.').forEach((part, i, allParts) => {
         const item: BreadCrumbFill = { title: part, linkTo: sectionCrumbLink(FilterType.name, allParts.slice(0, i + 1).join('.')) };
         sectionBreadCrumbs.push(item);
      });
   }

   const metaOptions = data ? new Map() : collection.getMetaValueByKey();

   const addFilters: IContextualMenuProps = {
      items: [
         {
            key: 'search_name',
            onRender: (item, dismissMenu) => TestFilterCard(
               filter,
               metaOptions,
               (value) => setOnTheFlyTestNameFilter(value),
               (value) => {
                  if (onTheFlyTestNameFilter && onTheFlyTestNameFilter === value) {
                     sectionsHandler.current?.promoteOnTheFlyCache();
                  }
                  setOnTheFlyTestNameFilter(undefined);
                  if (filter.namecontains !== value) {
                     navigate(urlFromFilterType(FilterType.namecontains, value?.toLowerCase()));
                  }
               },
               dismissMenu
            )
         }
      ]
   };

   return <Stack className={styles.container} styles={{ root: { paddingTop: 8 } }} grow>
      {!!data &&
         <Stack styles={{ root: { paddingTop: 4, paddingLeft: 20, paddingBottom: 8, paddingRight: 8 } }}>
            <Text>This test pass ran on <span style={{ fontWeight: 'bold' }}>{data.TestSessionInfo.DateTime}</span> for a duration of <span style={{ fontWeight: 'bold' }}>{msecToElapsed(data.TestSessionInfo.TimeElapseSec * 1000)}</span> for <span style={{ fontWeight: 'bold' }}>{data.MetaHandler?.map((key, value) => <span key={key}>{value} </span>)}</span></Text>
         </Stack>
      }

      <Stack styles={{ root: { background: modeColors.crumbs, padding: 4, paddingLeft: 10 } }} horizontal>
         {SectionBreadCrumb(sectionBreadCrumbs, [stackFailure], totalTests)}
         <Stack horizontalAlign="end" disableShrink>
            <ActionButton menuProps={addFilters}>
               <FontIcon
                  iconName={filter.namecontains || filter.meta ? "Warning" : "CirclePlus"}
                  style={{ padding: 3, color: (filter.namecontains || filter.meta) ? colors.get(StatusColor.Warnings) : undefined }} /> Filters
            </ActionButton>
         </Stack>
      </Stack>

      <Stack className={styles.container} grow>
         <Stack tokens={{ childrenGap: 12 }} styles={{ root: { paddingLeft: 20, paddingRight: 10, paddingTop: 6, paddingBottom: 6 } }}>
            {sections.length > 0 && sections.map((item: Section) => SectionItem(item))}
            {sections.length === 0 && <Stack><Stack.Item align="center" style={{ fontSize: 16 }}>No test found</Stack.Item></Stack>}
         </Stack>
      </Stack>

      {testuid && <TestResultModal testuid={testuid} collection={collection} onLoadMore={onLoadMore} />}
   </Stack>
}

export const TestSessionView: React.FC<any> = (props) => {
   const sessions = useRef<TestSessionCollection | undefined>(undefined);
   const item = testDataHandler.cursor;
   let sessionHandler = item?.getDataHandler() as TestSessionWrapper;
   if (item && !sessionHandler) {
      sessionHandler = new TestSessionWrapper(item);
      item.setDataHandler(sessionHandler);
   }
   if (!sessions.current) {
      sessions.current = new TestSessionCollection(testDataHandler);
   }
   const session_uid = sessionHandler.getUID();
   if (!sessions.current.has(session_uid)) {
      sessions.current.clear() //Keep only one session when in single session view
      sessions.current.set(session_uid, sessionHandler);
   }
   return <TestCollection collection={sessions.current} />
}

const getStreamBreadcrumbProps = (streamId: string, key: string) => {
   const stream = getStreamData(streamId);
   const projectName = getProjectName(streamId);
   const streamName = stream?.name ?? "Unknown Stream";

   const crumbItems: BreadcrumbItem[] = [
      {
         text: projectName,
         link: `/project/${stream?.project?.id}`
      },
      {
         text: streamName,
         link: `/stream/${stream?.id}`
      },
      {
         text: key
      }
   ];

   return {
      items: crumbItems,
      title: `Horde - ${key}: ${projectName}/${streamName}`
   }
}

export const TestSessionViewAll: React.FC = () => {
   const [loading, setLoading] = useState(false);
   const [maxCount, setMaxCount] = useState(500);
   const [version, updateVersion] = useState(0);
   const [crumbProps, setCrumbProps] = useState<{ items: BreadcrumbItem[], title: string }>({ items: [], title: "Test Report" });
   const sessions = useRef<TestSessionCollection | undefined>(undefined);

   const { streamId } = useParams<{ streamId: string }>();

   const { hordeClasses } = getHordeStyling();

   const count = testDataHandler.items?.size ?? 0;
   const changeRange = !loading && count >= maxCount ? testDataHandler.getChanges() : undefined;

   const loadMore = () => setMaxCount(maxCount + 500);

   useEffect(() => {
      if (!sessions.current || testDataHandler.streamId !== streamId) {
         sessions.current = new TestSessionCollection(testDataHandler);
      }
      const versionChangeDisposer = sessions.current.reactOnVersionChange((version) => updateVersion(version))
      const fetchData = async () => {
         setCrumbProps(getStreamBreadcrumbProps(streamId!, "Automated Test Session"));
         setLoading(true);
         try {
            await sessions.current?.getSessionsForStream(streamId!, maxCount);
            if (sessions.current?.version === 0 && testDataHandler.activated) {
               // No data was fetched, but we still want to mount the TestCollection view
               sessions.current.updateVersion();
            }
         } finally { testDataHandler.activated && setLoading(false); }
      }
      fetchData();

      return function cleanup() { versionChangeDisposer(); testDataHandler.desactivate(); }
   }, [maxCount, streamId]);

   const { styles } = getStyling();

   return (
      <Stack className={hordeClasses.horde} style={{ height: '100vh' }}>
         <TopNav />
         <Breadcrumbs items={crumbProps.items.concat()} title={crumbProps.title} />
         <Stack className={styles.container} grow>
            {loading && sessions.current && <LoaderProgressBar loader={sessions.current} target={maxCount} />}
            {changeRange &&
               <Stack className={styles.bottomLoadMore} tokens={{ childrenGap: 8 }} horizontal horizontalAlign="center">
                  <Stack>Changelist range [#{changeRange[0]} - #{changeRange[changeRange.length - 1]}] -</Stack>
                  <Stack style={{ color: "rgb(0, 120, 212)", cursor: 'pointer' }} onClick={loadMore}>Load more data...</Stack>
               </Stack>
            }
            {version > 0 && sessions.current && <TestCollection collection={sessions.current} onLoadMore={loadMore} />}
         </Stack>
      </Stack>
   );
}

const LoaderProgressBar: React.FC<{ loader: Loader, target: number }> = ({ loader, target }) => {
   const [progress, setProgress] = useState(loader.loadingProgress);
   const whenUpdate = useRef<Promise<void> & { cancel: () => void } | undefined>(undefined);

   whenUpdate.current = loader.whenLoadUpdate();
   whenUpdate.current.then(
      () => testDataHandler.activated && setProgress(loader.loadingProgress)
   ).catch((cancel) => null /*ignore cancel event*/);

   useEffect(() => {
      return function cleanup() { whenUpdate.current?.cancel(); }
   }, []);

   return (
      <Stack>
         <ProgressIndicator percentComplete={target > 1 && target !== progress ? progress / target : undefined} barHeight={7} />
      </Stack>
   );
};

export const routePath: string = "/automatedtestsession/:streamId";

export const BuildHealthTestSessionView: React.FC<{ streamId: string }> = (props) => {
   const { streamId } = props;

   const theme = getHordeTheme();

   return (
      <DefaultButton href={generatePath(routePath, { streamId: streamId })} text="Test Results" style={{ color: theme.semanticColors.bodyText }} />
   )
}
