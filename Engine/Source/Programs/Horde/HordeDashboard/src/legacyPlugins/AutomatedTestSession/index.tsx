// Copyright Epic Games, Inc. All Rights Reserved.

import { Plugin, PluginMount } from "../pluginTypes";
import { TestSessionView, TestSessionViewAll, BuildHealthTestSessionView, routePath } from "./components/AutomatedTestSession";

class AutomatedTestSessionPlugin implements Plugin {

    name = "Automated Test Session";
    components = [
        {
            mount: PluginMount.TestReportPanel,
            component: TestSessionView,
            id: "Automated Test Session"
        },
        {
            mount: PluginMount.BuildHealthSummary,
            component: BuildHealthTestSessionView,
        }
    ];
    routes = [
        {
            path: routePath,
            component: TestSessionViewAll
        }
    ];
}

const automatedTestSessionPlugin = new AutomatedTestSessionPlugin();
export default automatedTestSessionPlugin;

