// Copyright Epic Games, Inc. All Rights Reserved.

import { observable, action, when, reaction, makeObservable } from "mobx";
import { DataWrapper, TestDataWrapper, TestDataCollection } from '../../../backend/TestDataHandler'

export enum TestState {
    Failed = "Fail",
    Success = "Success",
    SuccessWithWarnings = "SuccessWithWarnings",
    NotRun = "NotRun",
    InProcess = "InProcess",
    Skipped = "Skipped",
    Unknown = "Unknown",
}

const TestStateRank: { [Key in string]: number } = {
    "Fail": 0,
    "InProcess": 1,
    "NotRun": 2,
    "SuccessWithWarnings": 3,
    "Success": 4,
    "Skipped": 5,
    "Unknown": 6,
};

export enum EventType {
    Info = "Info",
    Error = "Error",
    Warning = "Warning",
}

export enum ArtifactType {
    ImageCompare = "image comparison",
    Approved = "approved",
    Unapproved = "unapproved",
    Difference = "difference",
}

export type TestArtifact = {
    Tag: string;
    ReferencePath: string;
}

export type TestEvent = {
    Message: string;
    Context: string;
    Type: string;
    Tag: string;
    Hash: string;
    DateTime: string;
    Artifacts: TestArtifact[];
}

export type TestResult = {
    Name: string;
    TestUID: string;
    Suite: string;
    State: string;
    DeviceAppInstanceName: string | string[];
    ErrorCount: number;
    WarningCount: number;
    ErrorHashAggregate: string;
    DateTime: string;
    TimeElapseSec: number;
    // references that need to be initiated post fetch
    Details: TestResultDetails;
    Session: TestSessionWrapper;
}

export type TestResultDetails = {
    Events: TestEvent[];
}

export type TestDevice = MetadataHolder & {
    Name: string;
    AppInstanceName: string;
    AppInstanceLog: string;
}

export type TestSessionInfo = {
    DateTime: string;
    TimeElapseSec: number;
    Tests: TestResultsMap;
    TestResultsTestDataUID: string;
}

export type IndexedError = {
    Message: string;
    Tag: string;
    TestUIDs: string[];
}

export type Metadata = { [Key in string]: string | undefined }

type MetadataHolder = {
    Metadata?: Metadata
}

export type Loader = {
    loadingProgress: number;
    whenLoadUpdate: () => Promise<void> & { cancel: () => void };
}

const generatetHashFromString = (aggregate: string) => {
    let hash = 0;
    for (let i = 0; i < aggregate.length; i++) {
        const char = aggregate.charCodeAt(i);
        hash = ((hash << 5) - hash) + char;
        hash = hash & hash;
    }
    return hash.toString(16);
}

export const generateHashFromMetadata = (obj: Metadata) => {
    const keys = Object.keys(obj).sort();
    const aggregate = keys.map((key) => key + (obj[key] ?? '')).join();
    return generatetHashFromString(aggregate);
}

export class MetaWrapper {
    private collection?: Metadata
    private _hash: string

    constructor(collection?: Metadata) {
        this.collection = collection;
        this._hash = "";
    }

    get(key: string): string | undefined {
        if (this.collection) {
            return this.collection[key];
        }

        return undefined;
    }

    map(mapFunc: (key: string, value?: string) => any): any[] {
        if (this.collection) {
            return Object.keys(this.collection).map((key) => mapFunc(key, this.collection![key]));
        }
        return [];
    }

    forEach(predicate: (key: string, value?: string) => void) {
        if (this.collection) {
            Object.keys(this.collection).forEach((key) => predicate(key, this.collection![key]));
        }
    }

    maskByKeys(keyMask: string[]) {
        return this.map((key, value) => keyMask.includes(key) ? undefined : value).filter((info) => info)
    }

    private generateHashCode(): string {
        if (this.collection) {
            return generateHashFromMetadata(this.collection);
        }
        return "";
    }

    get hash(): string {
        if (!this._hash) {
            this._hash = this.generateHashCode();
        }
        return this._hash;
    }
}

export type IndexedErrorMap = { [Key in string]: IndexedError }

export type TestResultDetailsMap = { [Key in string]: TestResultDetails }

export type TestResultsMap = { [Key in string]: TestResult }

export type TestSession = MetadataHolder & {
    Name: string;
    Type: string;
    PreFlightChange: string;
    TestSessionInfo: TestSessionInfo;
    IndexedErrors: IndexedErrorMap;
    Devices: TestDevice[];
}

type CheckList = { [Key in string]: boolean }

export enum SessionFields {
    initiate = "data.Metadata,data.Name,data.PreFlightChange",
    remaining = "data.TestSessionInfo,data.IndexedErrors,data.Devices,data.Type",
}

export class TestSessionWrapper implements TestSession, DataWrapper {
    get Name(): string { return this.getOwnProperty('Name'); }
    get Type(): string { return this.getOwnProperty('Type'); }
    get PreFlightChange(): string { return this.getOwnProperty('PreFlightChange'); }
    get TestSessionInfo(): TestSessionInfo { return this.getOwnProperty('TestSessionInfo'); }
    get IndexedErrors(): IndexedErrorMap { return this.getOwnProperty('IndexedErrors'); }
    get Devices(): TestDevice[] { return this.getOwnProperty('Devices'); }
    get Metadata(): Metadata { return this.getOwnProperty('Metadata'); }

    Testdata: TestDataWrapper;
    MetaHandler?: MetaWrapper;

    @observable
    propertiesLoaded: CheckList = {
        TestSessionInfo: false
    };
    @action
    setPropertiesLoaded(...properties: string[]) {
        properties.forEach((key) => {
            this.propertiesLoaded[key] = true;
        })
    }
    isPropertyLoaded(key: string) {
        return this.propertiesLoaded[key];
    }
    isAllPropertiesLoaded() {
        return this.propertiesLoaded['TestSessionInfo']; // check TestSessionInfo only for now
    }
    getMissingProperties() {
        return SessionFields.remaining.split(',').map((item) => item.slice(5));
    }
    whenPropertyLoaded(key: string) {
        return when(() => this.propertiesLoaded[key]);
    }

    private _testsession: TestSession;

    constructor(testdata: TestDataWrapper) {
        makeObservable(this);
        this._testsession = testdata.data;
        this.Testdata = testdata;
        // first initialization, we should call the post Properties Update
        this.postPropertiesUpdate();
    }

    getOwnProperty<T>(key: string): T {
        return Object.getOwnPropertyDescriptor(this._testsession, key)?.value as T;
    }

    getUID(): string {
        return this.Name + (this.MetaHandler?.hash ?? '');
    }

    postPropertiesUpdate() {
        this.capitalizeKeys()
        const data = this._testsession;
        if (!this.MetaHandler && data.Metadata) {
            this.MetaHandler = new MetaWrapper(data.Metadata);
        }
        if (data.TestSessionInfo && !this.isPropertyLoaded('TestSessionInfo')) {
            for (const key in data.TestSessionInfo.Tests) {
                const test = data.TestSessionInfo.Tests[key];
                test.Session = this;
            }
            this.setPropertiesLoaded('TestSessionInfo');
        }
    }

    getTestResultByUID(uid: string) {
        if (this.TestSessionInfo) {
            return this.TestSessionInfo.Tests[uid];
        }

        return undefined
    }

    getDevices(AppInstanceName: string | string[]) {
        if (this.Devices) {
            if (typeof (AppInstanceName) === 'string') {
                const asString = AppInstanceName as string;
                const device = this.Devices.find((item) => item.AppInstanceName === asString);
                if (device) {
                    return [device];
                }
            }
            else {
                const asArray = AppInstanceName as string[];
                return this.Devices.filter((item) => asArray.includes(item.AppInstanceName))
            }
        }

        return undefined
    }

    private capitalizeKeys() {
        // Fix inconsistances in object keys fetched from Horde backend.
        const data = this._testsession as any;
        for (var key in data) {
            const first = key.charAt(0);
            const firstUpper = first.toUpperCase();
            if (first !== firstUpper) {
                const value = data[key];
                delete data[key];
                data[firstUpper + key.slice(1)] = value;
            }
        }
    }
}

export class TestStats {
    Failed: number = 0;
    Passed: number = 0;
    Skipped: number = 0;
    Unexecuted: number = 0;
    Incomplete: number = 0;
    TotalRun: number = 0;

    update(state: string, increment: boolean = true) {
        switch (state) {
            case TestState.Failed:
                increment ? this.Failed++ : this.Failed--;
                increment ? this.TotalRun++ : this.TotalRun--
                break;
            case TestState.NotRun:
                increment ? this.Unexecuted++ : this.Unexecuted--;
                break;
            case TestState.Success:
            case TestState.SuccessWithWarnings:
                increment ? this.Passed++ : this.Passed--;
                increment ? this.TotalRun++ : this.TotalRun--
                break;
            case TestState.InProcess:
                increment ? this.Incomplete++ : this.Incomplete--;
                increment ? this.TotalRun++ : this.TotalRun--
                break;
            case TestState.Skipped:
                increment ? this.Skipped++ : this.Skipped--;
                break;
        }
    }

    updateFromStats(stats: TestStats) {
        this.Failed += stats.Failed;
        this.Passed += stats.Passed;
        this.Skipped += stats.Skipped;
        this.Unexecuted += stats.Unexecuted;
        this.Incomplete += stats.Incomplete;
        this.TotalRun += stats.TotalRun;
    }
}

export class TestCase implements Loader {
    Name: string;
    DisplayName: string;
    TestUID: string;
    SessionName: string;
    Suite: string;
    State: string;
    StateMetaUID: string | undefined;
    Metas: Map<string, TestResult>;
    Stats: TestStats;

    @observable
    version: number = 0;
    @action
    updateVersion() {
        this.version++;
    }
    whenVersionChange() {
        const start = this.version;
        return when(() => this.version !== start);
    }

    @observable
    history: number = 0;
    @action
    updateHistory() {
        this.history++;
    }
    reactOnHistoryChange(callback: (version: number) => void) {
        return reaction(
            () => this.history,
            (version) => callback(version)
        );
    }

    @observable
    historyTarget: number = 0;
    @action
    incrementHistoryTarget() {
        this.historyTarget++;
    }
    get loadingProgress() { return this.history }
    whenLoadUpdate() {
        const previousVersion = this.history;
        return when(() => this.version !== previousVersion);
    }

    constructor(name: string, testuid: string, suite: string, sessionName: string, displayName?: string) {
        makeObservable(this);
        this.Name = name;
        this.DisplayName = displayName ?? this.toDislayName(name);
        this.TestUID = testuid;
        this.State = TestState.Unknown;
        this.StateMetaUID = undefined;
        this.Suite = suite;
        this.SessionName = sessionName;
        this.Metas = new Map();
        this.Stats = new TestStats();
    }

    removeMeta(meta: MetaWrapper) {
        const metauid = meta.hash;
        const test = this.Metas.get(metauid);
        if (!test) {
            return false;
        }
        this.Metas.delete(metauid);
        this.Stats.update(test.State, false);
        this.updateVersion();
        return true;
    }

    isEmpty() {
        return this.Metas.size === 0;
    }

    addMeta(item: TestResult) {
        const metauid = item.Session.MetaHandler!.hash;
        if (this.Metas.has(metauid)) {
            return false;
        }
        this.Metas.set(metauid, item);
        let state = item.State;
        if (state === TestState.Success && item.WarningCount > 0) {
            state = TestState.SuccessWithWarnings;
            item.State = state;
        }
        if (this.setState(state)) {
            this.StateMetaUID = metauid;
        }
        this.Stats.update(state);
        this.updateVersion();
        return true;
    }

    forEachMeta(predicate: (test: TestResult) => void) {
        this.Metas.forEach(predicate);
    }

    getResults(sorted: boolean = false) {
        const results = Array.from(this.Metas.values());
        if (sorted) {
            results.sort(this.sortResults);
        }
        return results;
    }

    getFilteredByMeta(metafilter: MetadataFilter) {
        const metas = Array.from(this.Metas.values()).filter(
            (test) => Object.keys(metafilter).every((key) => {
                const sessionKey = test.Session.Metadata[key];
                return sessionKey && metafilter[key].includes(sessionKey);
            })
        );
        if (metas.length === 0) {
            return undefined;
        }
        const testcase = new TestCase(this.Name, this.TestUID, this.Suite, this.SessionName, this.DisplayName);
        metas.forEach((test) => testcase.addMeta(test));
        return testcase;
    }

    getFilteredByState(state: TestState) {
        const targetRank = TestStateRank[state];
        const metas = Array.from(this.Metas.values()).filter(
            (test) => targetRank <= TestStateRank[test.State]
        );
        if (metas.length === 0) {
            return undefined;
        }
        const testcase = new TestCase(this.Name, this.TestUID, this.Suite, this.SessionName, this.DisplayName);
        metas.forEach((test) => testcase.addMeta(test));
        return testcase;
    }

    private toDislayName(name: string): string {
        // It is necessary to sanitize the test names that contains / instead of . syntax
        if (name.indexOf('/') > -1) {
            return name.replaceAll(/\.*\//g, ".");
        }
        return name;
    }

    private setState(state: string) {
        if (this.State !== state) {
            const currentRank = TestStateRank[this.State];
            const newRank = TestStateRank[state];
            if (newRank < currentRank) {
                this.State = state;
                return true;
            }
        }
        return false;
    }

    private sortResults(a: TestResult, b: TestResult) {
        const aRank = TestStateRank[a.State];
        const bRank = TestStateRank[b.State];
        if (aRank > bRank) {
            return 1;
        } else if (aRank < bRank) {
            return -1;
        }
        return 0;
    }
}

export class Section {
    Name: string;
    FullName: string;
    Type: FilterType;
    Stats: TestStats;
    TestCases: TestCase[];
    TestNames: Set<string>;

    constructor(name: string, fullName: string, type: FilterType) {
        this.Name = name;
        this.FullName = fullName;
        this.Type = type;
        this.Stats = new TestStats();
        this.TestCases = [];
        this.TestNames = new Set();
    }

    addTestCase(testcase: TestCase, testName?: string) {
        this.TestCases.push(testcase);
        this.TestNames.add(testName ?? testcase.DisplayName);
        this.Stats.updateFromStats(testcase.Stats);
    }

    get isOneTest(): boolean {
        return this.TestNames.size === 1 && this.TestNames.has(this.FullName)
    }
}

export enum FilterType {
    suite = 'suite',
    name = 'name',
    project = 'project',
    testuid = 'testuid',
    metauid = 'metauid',
    testchange = 'testchange',
    view = 'view',
    namecontains = 'namecontains',
    meta = 'meta',
}

export type MetadataFilter = { [Key in string]: string[] }
export class MetaFilterTools {
    static Match = (key: string, target: string): boolean => target.startsWith(`${key}.`)
    static Encode = (key: string, items: string[]): string => `${key}.${items.join('~')}`
    static Decode = (items: string[]): MetadataFilter | undefined => {
        return items.length === 0 ? undefined :
            Object.fromEntries(items.map((item) => {
                const [key, value] = item.split('.', 2);
                return [key, value.split('~')]
            }))
    }
}

export type SectionFilter = {
    suite?: string;
    name?: string;
    project?: string;
    namecontains?: string;
    meta?: MetadataFilter;
}

export type ReportFilter = {
    testuid?: string;
    metauid?: string;
    testchange?: number;
    view?: string;
}

export const generateHashFromSectionFilter = (obj: SectionFilter) => {
    const baseFilter: Metadata = Object.fromEntries(
        Object.keys(obj).filter((key) => key !== 'meta').map((key) => [
            key, Object.getOwnPropertyDescriptor(obj, key)?.value
        ])
    );
    let aggregate = Object.keys(baseFilter).sort().map((key) => key + (baseFilter[key] ?? '')).join();
    const subObj = obj.meta;
    if (subObj !== undefined) {
        aggregate += 'meta' + Object.keys(subObj).sort().map((key) => key + (subObj[key]?.join('~') ?? '')).join();
    }
    return generatetHashFromString(aggregate);
}

type GetKeyFunction = (testcase: TestCase) => [string, string];
const regexNameFilter = /"([^"]+)"|([^\s"]+)/g;

export class SectionCollection {

    private filterCache: Map<string, { hash: string, sections: Section[] }>;
    private onTheFlyCache: { filter: string, hash: string, sections: Section[] } | undefined;

    constructor() {
        this.filterCache = new Map();
    }

    extractSections(filter: SectionFilter, sessions: TestSessionCollection) {
        // Get the sections, first look for in the cache, otherwise generate it
        const filterCacheKey = generateHashFromSectionFilter(filter);
        const cache = this.filterCache.get(filterCacheKey);
        const isSameHash = cache && cache.hash === sessions.hash;
        const sections = cache && isSameHash ? cache.sections : this.filterSections(filter, sessions.getTestCases());
        if (!this.filterCache.has(filterCacheKey) || !isSameHash) {
            this.filterCache.set(filterCacheKey, { hash: sessions.hash, sections: sections })
        }
        return sections;
    }

    reExtractSectionOnTheFly(filter: SectionFilter, testNameFilter: string, sessions: TestSessionCollection) {
        // to use only when iterating through cached data
        let sections = this.extractSections({ ...filter, namecontains: undefined }, sessions);
        const onTheFlyFilter = { ...filter, namecontains: testNameFilter.toLowerCase() }
        sections = this.filterSections(onTheFlyFilter, sections.flatMap((section) => section.TestCases));
        this.onTheFlyCache = { filter: generateHashFromSectionFilter(onTheFlyFilter), hash: sessions.hash, sections };
        return sections;
    }

    promoteOnTheFlyCache() {
        if (this.onTheFlyCache) {
            this.filterCache.set(this.onTheFlyCache.filter, this.onTheFlyCache);
            this.onTheFlyCache = undefined;
        }
    }

    private filterSections(filter: SectionFilter, tests: TestCase[]) {
        const filterFunc = (test: TestCase) => {
            if (filter.suite) {
                if (test.Suite !== filter.suite) {
                    return false;
                }
            }
            if (filter.namecontains) {
                const substrings = Array.from(filter.namecontains.matchAll(regexNameFilter), match => match[1] || match[2]);
                const testName = test.Name.toLowerCase();
                if (!substrings.some((value) => testName.indexOf(value) >= 0)) {
                    return false;
                }
            }
            if (filter.name) {
                if (test.DisplayName.indexOf(filter.name + '.') !== 0) {
                    return false;
                }
            }
            return true;
        }
        const filteredTests = tests.reduce((filtered: TestCase[], item: TestCase) => {
            if (filterFunc(item)) {
                if (filter.meta === undefined) {
                    filtered.push(item);
                } else {
                    const filteredCase = item.getFilteredByMeta(filter.meta);
                    if (filteredCase) {
                        filtered.push(filteredCase);
                    }
                }
            }
            return filtered
        }, []);

        const commonSection = filter.name ? this.getCommonSection(filteredTests.map((item) => item.DisplayName)) : "";
        const sectionKeyFunc = (key: FilterType): GetKeyFunction | undefined => {
            switch (key) {
                case FilterType.suite:
                    return (testcase: TestCase) => {
                        return [testcase.Suite, testcase.Suite];
                    }
                case FilterType.name:
                    return (testcase: TestCase) => {
                        const suite = testcase.DisplayName.slice(commonSection.length).split('.', 1)[0];
                        return [suite, commonSection + suite];
                    }
            }
        };
        const sectionFocus = filter.suite || filter.name ? FilterType.name : FilterType.suite;
        const sections = this.buildSections(filteredTests, sectionFocus, sectionKeyFunc(sectionFocus)!);
        // sorting
        sections.forEach((section) => this.sortSectionResults(section));
        this.sortSections(sections);
        return sections;
    }

    private buildSections(testcases: TestCase[], type: FilterType, keyFunc: GetKeyFunction): Section[] {
        const sections: Section[] = [];
        const sectionsMap = new Map<string, Section>();

        testcases.forEach((test) => {
            const [key, fullKey] = keyFunc(test);
            let targetSection = sectionsMap.get(key);
            if (!targetSection) {
                targetSection = new Section(key, fullKey, type);
                sections.push(targetSection);
                sectionsMap.set(key, targetSection);
            }
            targetSection.addTestCase(test, test.DisplayName);
        });

        return sections;
    }

    private getCommonString(list: string[]): string {
        if (list.length === 0) {
            return "";
        }
        let shortest: string = list[0];
        let longest: string = "";
        list.forEach((item) => {
            if (item > longest) {
                longest = item;
                return;
            }
            if (item < shortest) {
                shortest = item;
            }
        });
        while (longest.indexOf(shortest) !== 0) {
            shortest = shortest.slice(0, -1)
        }

        return shortest;
    }

    private getCommonSection(list: string[]): string {
        let commonKey = this.getCommonString(list);
        if (commonKey[commonKey.length - 1] !== '.' && commonKey.indexOf('.') >= 0) {
            commonKey = commonKey.slice(0, commonKey.lastIndexOf('.') + 1);
        }

        return commonKey;
    }

    // sort comparing the number of Failed, Incomplete, Unexecuted and Passed
    private sortSections(sections: Section[]) {
        sections.sort((a, b) => {
            const aStats = a.isOneTest ? a.TestCases[0].Stats : a.Stats;
            const bStats = b.isOneTest ? b.TestCases[0].Stats : b.Stats;
            if (aStats.Failed > bStats.Failed) {
                return -1;
            } else if (aStats.Failed < bStats.Failed) {
                return 1;
            } else if (aStats.Incomplete > bStats.Incomplete) {
                return -1;
            } else if (aStats.Incomplete < bStats.Incomplete) {
                return 1;
            } else if (aStats.Unexecuted > bStats.Unexecuted) {
                return -1;
            } else if (aStats.Unexecuted < bStats.Unexecuted) {
                return 1;
            } else if (aStats.Passed > bStats.Passed) {
                return -1;
            } else if (aStats.Passed < bStats.Passed) {
                return 1;
            } else if (a.isOneTest && !b.isOneTest) {
                return 1;
            } else if (b.isOneTest && !a.isOneTest) {
                return -1;
            }
            return 0;
        });
    }

    // sort in that order Failed, Incomplete, Unexecuted, Passed, Skipped
    private sortSectionResults(section: Section) {
        SectionCollection.sortTestCases(section.TestCases);
    }

    static sortTestCases(testcases: TestCase[]) {
        testcases.sort((a, b) => {
            const aRank = TestStateRank[a.State];
            const bRank = TestStateRank[b.State];
            if (aRank > bRank) {
                return 1;
            } else if (aRank < bRank) {
                return -1;
            }

            if (a.Name > b.Name) {
                return 1;
            } else if (a.Name < b.Name) {
                return -1;
            }

            return 0;
        })
    }
}

type SessionReferences = {
    SessionId: string;
    ResultsId: string;
}

export class TestSessionCollection implements Loader {

    @observable
    version: number = 0;
    @action
    updateVersion() {
        this.version++;
    }
    reactOnVersionChange(callback: (version: number) => void) {
        return reaction(
            () => this.version,
            (version) => callback(version)
        );
    }

    @observable
    loadingCount: number = 0;
    @action
    incrementLoading() {
        this.loadingCount++;
    }
    @action
    decrementLoading() {
        this.loadingCount--;
    }
    whenLoadUpdate() {
        const previousCount = this.loadingCount;
        return when(() => this.loadingCount !== previousCount)
    }
    get loadingProgress(): number { return this._datahandler.items?.size ?? 0 }

    get size(): number { return this._collection.size; }
    get hash(): string {
        let aggregate = "";
        this.forEachSession( // only include fully loaded session
            (session) => aggregate += session.isAllPropertiesLoaded() ? session.getUID() : ""
        );
        return generatetHashFromString(aggregate);
    }

    private _collection: Map<string, TestSessionWrapper>;
    private _testcases: Map<string, TestCase>;
    private _datahandler: TestDataCollection;
    private _includePreflight: boolean = false;
    private _cachedMetaValueByKey: { hash: string, value: Map<string, Set<string>> } | undefined;

    constructor(handler: TestDataCollection, includePreflight: boolean = false) {
        makeObservable(this);
        this._collection = new Map();
        this._testcases = new Map();
        this._datahandler = handler;
        this._includePreflight = includePreflight;
    }

    set(uid: string, item: TestSessionWrapper) {
        this._collection.set(uid, item);
        if (item.isAllPropertiesLoaded()) {
            this.indexTestsFromSession(item);
            this.updateVersion();
        }
    }

    get(uid: string) {
        return this._collection.get(uid);
    }

    has(uid: string) {
        return this._collection.has(uid);
    }

    delete(uid: string) {
        const session = this._collection.get(uid);
        this._collection.delete(uid);
        // clean up testuids map
        if (session && session.TestSessionInfo) {
            Object.keys(session.TestSessionInfo.Tests).forEach((key) => {
                const testcase = this._testcases.get(key)!;
                if (testcase.removeMeta(session.MetaHandler!)) {
                    if (testcase.isEmpty()) {
                        this._testcases.delete(key)
                    }
                }
            });
        }
        this.updateVersion();
    }

    clear() {
        this._collection.clear()
        this._testcases.clear()
        this.updateVersion();
    }

    forEachSession(func: (item: TestSessionWrapper) => void) {
        this._collection.forEach(func);
    }

    getMetaValueByKey() {
        if (this._cachedMetaValueByKey !== undefined) {
            if (this._cachedMetaValueByKey.hash === this.hash) {
                return this._cachedMetaValueByKey.value;
            }
        }
        const keys: Map<string, Set<string>> = new Map();
        this.forEachSession((item) => {
            const meta = item.MetaHandler;
            if (meta) {
                meta.forEach((key, value) => {
                    if (value) {
                        let valueSet = keys.get(key);
                        if (!valueSet) {
                            valueSet = new Set();
                            keys.set(key, valueSet);
                        }
                        valueSet.add(value);
                    }
                });
            }
        });
        this._cachedMetaValueByKey = { hash: this.hash, value: keys };
        return keys;
    }

    *iterSessions() {
        const iterator = this._collection.values();
        let result = iterator.next();
        while (!result.done) {
            yield result.value;
            result = iterator.next();
        }
    }

    *iterTests() {
        const iterSession = this._collection.values();
        let resultSession = iterSession.next();
        while (!resultSession.done) {
            const sessionInfo = resultSession.value.TestSessionInfo;
            if (sessionInfo) {
                for (const key in sessionInfo.Tests) {
                    yield sessionInfo.Tests[key];
                }
            }
            resultSession = iterSession.next();
        }
    }

    async getSessionsForStream(streamId: string, maxCount: number = 1000) {
        this.incrementLoading();
        const onFetch = (items: (TestDataWrapper | undefined)[]) => this.onInitialFetch(items);
        try {
            if (streamId === this._datahandler.streamId
                && this._datahandler.items
                && maxCount > this._datahandler.items.size) {
                // Max count resize
                this._datahandler.setActive(true);
                await this._datahandler.queueFetchItems(onFetch, SessionFields.initiate, maxCount);
            } else {
                // Initial fetch or reload
                await this._datahandler.setFromStream(
                    streamId, "Automated Test Session", onFetch, SessionFields.initiate, maxCount
                );
            }
        } finally { this.decrementLoading() }
        await when(() => this.loadingCount === 0);
    }

    private onInitialFetch(items: (TestDataWrapper | undefined)[]) {
        const sessionsToFetch: string[] = [];
        items.forEach((item) => {
            if (item) {
                let data = item.getDataHandler() as TestSessionWrapper;
                if (!data) {
                    data = new TestSessionWrapper(item);
                    item.setDataHandler(data);
                }
                if (!this._includePreflight && data.PreFlightChange) {
                    return;
                }
                const session_uid = data.getUID();
                if (!this.has(session_uid)) {
                    this.set(session_uid, data);
                    if (!data.isAllPropertiesLoaded()) {
                        sessionsToFetch.push(item.id);
                    }
                }
            }
        });
        if (sessionsToFetch.length > 0) {
            this.incrementLoading();
            this._datahandler.fetchUpdateItems(
                sessionsToFetch, (items) => this.onRemainingFetch(items), SessionFields.remaining
            ).finally(() => this.decrementLoading());
        }
    }

    private onRemainingFetch(items: (TestDataWrapper | undefined)[]) {
        let hasUpdated = false;
        items.forEach((item) => {
            if (item) {
                const data = item.getDataHandler() as TestSessionWrapper;
                if (data && data.isAllPropertiesLoaded()) {
                    this.indexTestsFromSession(data);
                    hasUpdated = true;
                }
            }
        });
        if (hasUpdated) { this.updateVersion() }
    }

    private indexTestsFromSession(session: TestSessionWrapper) {
        const tests = Object.values(session.TestSessionInfo!.Tests);
        if (tests.length === 0) {
            return;
        }
        const first = this._testcases.get(tests[0].TestUID);
        if (first && first.SessionName !== session.Name) {
            return; // Skip conflicting session name, most likely session was renamed.
        }
        tests.forEach((item) => {
            const testuid = item.TestUID;
            let testcase = this._testcases.get(testuid);
            if (!testcase) {
                testcase = new TestCase(item.Name, testuid, item.Suite, session.Name);
                this._testcases.set(testuid, testcase);
            }
            else if (testcase.SessionName !== session.Name) {
                console.warn(`'${testcase.DisplayName}' is in another session. Appears in '${testcase.SessionName}' and '${session.Name}'.`);
            }
            testcase.addMeta(item);
        });
    }

    async getTestCaseByUID(uid: string): Promise<TestCase | undefined> {
        let testcase: TestCase | undefined = undefined;
        const toFetch: TestSessionWrapper[] = [];
        const testcases = this._testcases;

        function getTestCase() {
            testcase = testcases.get(uid);
            if (testcase) {
                testcase.forEachMeta((test) => {
                    if (test.Details === undefined) {
                        toFetch.push(test.Session);
                    }
                });
            }
        }
        getTestCase();

        while (!testcase && toFetch.length === 0 && this.loadingCount !== 0) {
            // Give a chance to the collection to load a bit more if nothing was found
            // It is possible the test is run infrequently and is deep in the result collection
            const previousVersion = this.version;
            await when(() => this.version !== previousVersion || this.loadingCount === 0);
            // then retry
            getTestCase();
        }

        if (toFetch.length > 0) {
            // Fetch missing data from session test results
            this.fetchSessionResults(
                toFetch.map((item) => {
                    return { SessionId: item.Testdata.id, ResultsId: item.TestSessionInfo.TestResultsTestDataUID }
                }), 'data.' + uid
            );
        }

        return testcase;
    }

    async getTestResultByQuery(uid: string, metauid: string, change?: number): Promise<TestResult | undefined> {
        const testcase = await this.getTestCaseByUID(uid);
        let test: TestResult | undefined = undefined;

        if (testcase) {
            test = testcase.Metas.get(metauid);
            while (!test && this.loadingCount !== 0) {
                const previousVersion = testcase.version;
                await when(() => testcase.version !== previousVersion || this.loadingCount === 0);
                test = testcase.Metas.get(metauid);
            }
            if (change && test?.Session.Testdata.change !== change) {
                // Need to get the target change from the history
                test = this.getLoadedTestHistoryByMeta(uid)?.get(metauid)?.find(
                    (item) => item.Session.Testdata.change === change
                );
                if (!test) {
                    // trigger fetch history in background
                    this.fetchSessionHistory(testcase.SessionName, uid);
                }
                while (!test && this.loadingCount !== 0) {
                    const previousVersion = this.version;
                    await when(() => this.version !== previousVersion || this.loadingCount === 0);
                    test = this.getLoadedTestHistoryByMeta(uid)?.get(metauid)?.find(
                        (item) => item.Session.Testdata.change === change
                    );
                }

            }
        }

        if (test && test.Details === undefined) {
            // Fetch missing data from the session test result
            await this.fetchSessionResults(
                [{ SessionId: test.Session.Testdata.id, ResultsId: test.Session.TestSessionInfo.TestResultsTestDataUID }]
                , 'data.' + uid
            );
            if (test.Details === undefined) {
                // if nothing was fetched or failed to fetch, return nothing
                return undefined;
            }
        }

        return test;
    }

    private async fetchSessionResults(items: SessionReferences[], filter?: string, parallelCount: number = 30) {
        let count = 0;
        for (let i = 0; i < items.length; i += parallelCount) {
            if (!this._datahandler.activated) {
                break;
            }
            const chunk = items.slice(i, i + parallelCount);
            count += (
                await Promise.all(chunk.map((item) => this.fetchResultItemFromSession(item, filter)))
            ).reduce((a, b) => a + b);
        }
        return count;
    }

    private async fetchResultItemFromSession(item: SessionReferences, filter?: string): Promise<number> {
        const update = await this._datahandler.fetchItemsFromAlternateKey(`${this._datahandler.key} Result Details::${item.ResultsId}`, undefined, 1, 0, filter);
        // We only expect one returned item
        if (update.length > 0) {
            const results = update[0].data as TestResultDetailsMap;
            const session = this._datahandler.items?.get(item.SessionId)?.getDataHandler() as TestSessionWrapper;
            if (session) {
                for (const uid in results) {
                    const test = session.getTestResultByUID(uid);
                    if (test) {
                        test.Details = results[uid];
                    }
                }
                return 1;
            }
        }

        return 0;
    }

    getTests() {
        const tests: TestResult[] = [];
        const iterator = this._collection.values();
        let result = iterator.next();
        while (!result.done) {
            const sessionInfo = result.value.TestSessionInfo;
            if (sessionInfo) {
                tests.push(...Object.values(result.value.TestSessionInfo.Tests) as TestResult[]);
            }
            result = iterator.next();
        }
        return tests;
    }

    getTestCases() {
        return Array.from(this._testcases.values());
    }

    getLoadedTestHistoryByMeta(uid: string) {
        const testcase = this._testcases.get(uid);
        if (!testcase) {
            return undefined;
        }

        const testresultsByMeta: Map<string, TestResult[]> = new Map();
        this.forEachSessionHistoryByChange(testcase.SessionName,
            (session) => {
                if (session.isAllPropertiesLoaded()) {
                    const test = session.getTestResultByUID(uid);
                    if (test) {
                        const metauid = session.MetaHandler!.hash;
                        const meta = testresultsByMeta.get(metauid);
                        if (meta) {
                            const change = session.Testdata.change;
                            if (!meta.some((item) => item.Session.Testdata.change === change)) {
                                // Keep only one result per change
                                meta.push(test);
                            }
                        } else {
                            testresultsByMeta.set(metauid, [test]);
                        }
                    }
                }
            }
        );

        return testresultsByMeta;
    }

    async fetchSessionHistory(sessionName: string, testuid: string, maxCount: number = 1000) {
        if (this._datahandler.cursor) {
            // for single view mode we need to fetch the whole pool for context
            const onFetch = (items: (TestDataWrapper | undefined)[]) => this.onInitialHistoryFetch(items, sessionName, testuid)
            await this._datahandler.fetchCursorHistory(onFetch, SessionFields.initiate, maxCount);
            return;
        }

        await this.fetchIncompleteSessionsForHistory(sessionName, testuid);
        while (this.loadingCount !== 0) {
            // wait for the collection to be fully loaded, in the meanwhile load the missing session history
            const previousVersion = this.version;
            await when(() => this.version !== previousVersion || this.loadingCount === 0);
            await this.fetchIncompleteSessionsForHistory(sessionName, testuid);
        }
    }

    private async fetchIncompleteSessionsForHistory(sessionName: string, testuid: string) {
        const missingSessions: string[] = [];
        this.forEachSessionHistoryByChange(sessionName,
            (session) => {
                if (!session.isAllPropertiesLoaded()) {
                    missingSessions.push(session.Testdata.id);
                    this._testcases.get(testuid)?.incrementHistoryTarget();
                }
            }
        );

        if (missingSessions.length > 0) {
            await this._datahandler.fetchUpdateItems(
                missingSessions, (item) => this.onFetchTestCaseHistory(item, testuid), SessionFields.remaining
            )
        }
        return missingSessions.length;
    }

    private forEachSessionHistoryByChange(sessionName: string, callback: (session: TestSessionWrapper) => void) {
        const iterator = this._datahandler.iterItemsByChange();
        let change = iterator.next();
        while (!change.done) {
            (change.value as TestDataWrapper[]).forEach((item) => {
                const session = item.getDataHandler() as TestSessionWrapper;
                if (session.Name === sessionName) {
                    if (!this._includePreflight && session.PreFlightChange) {
                        return;
                    }
                    callback(session);
                }
            });
            change = iterator.next();
        }
    }

    private onInitialHistoryFetch(items: (TestDataWrapper | undefined)[], sessionName: string, testuid: string) {
        const toFetch: TestDataWrapper[] = [];
        items.forEach((item) => {
            if (item) {
                let data = item.getDataHandler() as TestSessionWrapper;
                if (!data) {
                    data = new TestSessionWrapper(item);
                    item.setDataHandler(data);
                    if (data.Name === sessionName) {
                        toFetch.push(item);
                        this._testcases.get(testuid)?.incrementHistoryTarget();
                    }
                }
            }
        });
        if (toFetch.length > 0) {
            this._datahandler.fetchUpdateItems(
                toFetch.map((item) => item.id), (items) => this.onFetchTestCaseHistory(items, testuid), SessionFields.remaining
            )
        }
    }

    private onFetchTestCaseHistory(items: (TestDataWrapper | undefined)[], testuid: string) {
        items.forEach((item) => {
            if (item) {
                this._testcases.get(testuid)?.updateHistory();
            }
        });
    }

}
