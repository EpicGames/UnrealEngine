// Copyright Epic Games, Inc. All Rights Reserved.

import { Icon, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import { mergeStyleSets, mergeStyles } from '@fluentui/react/lib/Styling';
import React, { useEffect, useRef, useState } from 'react';
import { Link } from 'react-router-dom';

import backend from '../../../backend';
import { TestData } from '../../../backend/Api';
import dashboard from '../../../backend/Dashboard';
import { msecToElapsed } from '../../../base/utilities/timeUtils';
import { testDataHandler } from '../../../components/TestReportView';
import { getHordeStyling } from '../../../styles/Styles';
import { getHordeTheme } from '../../../styles/theme';
import { HistoryItem, LogLevel, SimpleReport } from '../models/SimpleReport';

let _styles: any;
let _stateStyles: any;

const getStyling = () => {

   const theme = getHordeTheme();

   const styles = _styles ?? mergeStyleSets({
      container: {
         overflow: 'auto',
         height: 'calc(100vh - 165px)',
         marginTop: 8
      },
      item: [
         {
            fontSize: "11px",
            fontFamily: "Horde Cousine Regular"
         }
      ],
      gutter: [
         {
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 14,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      gutterError: [
         {
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
            borderLeftStyle: 'solid',
            borderLeftColor: "#EC4C47",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      gutterWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",
            borderLeftStyle: 'solid',
            borderLeftColor: "#F7D154",
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      gutterSuccess: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: theme.palette.green,
            borderLeftWidth: 6,
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 8,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      itemWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" :"#FEF8E7"
         }
      ],
      itemError: [
         {
            background: dashboard.darktheme ? "#330606" :"#FEF6F6"
         }
      ],
      itemHover: {
         selectors: {
            ':hover': {
               background: theme.palette.neutralLight
            }
         }
      },
      itemHighlighted: {
         background: theme.palette.neutralLighter
      },
      historyList: {
         zIndex: 1,
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid",
         backgroundColor: dashboard.darktheme ? "#111" : "#FFF",
         padding: "3px",
         overflow: "auto"
      },
      jobDetailsItem: {
         padding: 8,
         borderBottom: '1px solid ' + theme.palette.neutralLighter,
         selectors: {
            ':hover': { background: theme.palette.neutralLight }
         }
      }
   });


   const stateStyles = _stateStyles ?? mergeStyleSets({
      success: [{ color: theme.palette.green, userSelect: "none" }, iconClass],
      warnings: [{ color: "#F7D154", userSelect: "none" }, iconClass],
      failure: [{ color: "#EC4C47", userSelect: "none" }, iconClass],
   });

   _stateStyles = stateStyles;
   _styles = styles;

   return {
      theme: theme,
      styles: styles,
      stateStyles: stateStyles
   };
}

const iconClass = mergeStyles({
   fontSize: 13
});

const wait = (ms: number) => new Promise(resolve => setTimeout(resolve, ms));
const secToElapsed = (sec: number) => msecToElapsed(sec * 1000);

const markdownURL = /^\[([^\]]+)\]\((.+)\)$/;
function getUrlFromLink(link: string): { text: string, url: string } {
   let url = link;
   let text = link;
   const match = markdownURL.exec(link);
   if (match !== null) {
      text = match[1];
      url = match[2];
   }
   return { text: text, url: url };
}

const URLLink: React.FC<{ link: string }> = (props) => {

   const { styles } = getStyling();

   const { link } = props;

   const linkInfo = getUrlFromLink(link);

   return (<Stack disableShrink={true}><Text className={styles.itemHover}><a href={linkInfo.url}>{linkInfo.text}</a></Text></Stack>);
}

const LogItem: React.FC<{ line: string, level: string }> = (props) => {

   const { styles } = getStyling();

   const { line, level } = props;
   const is_error = level === LogLevel.Error;
   const is_warning = level === LogLevel.Warning;
   const style = is_error ? styles.itemError : (is_warning ? styles.itemWarning : styles.item);
   const gutterStyle = is_error ? styles.gutterError : (is_warning ? styles.gutterWarning : styles.gutter);

   return (
      <Stack className={styles.item} disableShrink={true} styles={{ root: { paddingLeft: 8, width: "100%" } }}>
         <Stack horizontal>
            <Stack className={gutterStyle}></Stack>
            <Stack.Item className={style} align="center" styles={{ root: { paddingLeft: 8, width: "100%" } }}><pre style={{ margin: 0, whiteSpace: "pre-wrap" }}>{line}</pre></Stack.Item>
         </Stack>
      </Stack>
   );
}

const LogLink: React.FC<{ to: string }> = (props) => {

   const { styles } = getStyling();

   const { to } = props;
   //const [link, setLink] = useState<string | undefined>(undefined)

   // this view is deprecated and these links were nonfunctional in tests I checked,  would also need to be updated to artifacts v2
   /*
    useEffect(() => {
       const getLink = async () => {
          
           
            const artifact = await testDataHandler.cursor?.findArtifactData(to);
            if (artifact !== undefined) {
                return `${backend.serverUrl}/api/v1/artifacts/${artifact.id}/download?Code=${artifact.code}`
            }
           
            return undefined;
        }
        getLink().then((item) => setLink(item));
    });
    */

   // <a href={link}>{to}</a>

   return (
      <Stack disableShrink={true} styles={{ root: { paddingLeft: 8 } }}><Text className={styles.itemHover}>{to}</Text></Stack>
   );
}

const HistoryItemPane: React.FC<{ item: HistoryItem, selected: boolean }> = (props) => {
   const { styles } = getStyling();
   const { item, selected } = props;
   const gutterStyle = item.HasSucceeded ? styles.gutterSuccess : styles.gutterError;
   const state = item.HasSucceeded ? "Success" : "Failed";
   const errorCount = item.HasSucceeded ? "" : ` (${item.ErrorCount} error${item.ErrorCount > 1 ? "s" : ""})`;

   return (
      <Stack horizontal className={`${styles.itemHover} ${selected ? styles.itemHighlighted : ""}`}>
         <Stack className={gutterStyle}></Stack>
         <Stack horizontal horizontalAlign="space-between" grow tokens={{ childrenGap: 11 }}>
            <Stack.Item styles={{ root: { whiteSpace: "nowrap" } }}>
               {item.Url.indexOf('http') < 0 ?
                  <Link to={item.Url}>
                     <Text variant="smallPlus">{state} on {item.Change}</Text>
                     {selected && <Text styles={{ root: { fontWeight: 'bold' } }}> &lt;</Text>}
                  </Link> :
                  <a href={item.Url}>
                     <Text variant="smallPlus">{state} on {item.Change}</Text>
                     {selected && <Text styles={{ root: { fontWeight: 'bold' } }}> &lt;</Text>}
                  </a>
               }
            </Stack.Item>
            <Stack.Item styles={{ root: { width: '22ch' } }}><Text>{item.Date}</Text></Stack.Item>
            <Stack.Item styles={{ root: { width: '8ch' } }}>
               <Text styles={{ root: { color: "#0089DD" } }}>{secToElapsed(item.TotalDurationSeconds)}</Text>
            </Stack.Item>
            <Stack.Item styles={{ root: { width: '12ch', textAlign: 'right' } }}><Text>{errorCount}</Text></Stack.Item>
         </Stack>
      </Stack>
   );
}

const HistoryPanel: React.FC = () => {
   const [historyLoaded, setHistoryLoaded] = useState(false);
   const historyItemsRef = useRef<HistoryItem[]>([]);
   const { hordeClasses } = getHordeStyling();
   const historyItems = historyItemsRef.current;

   useEffect(() => {
      testDataHandler.getCursorHistory(undefined, undefined, 100).then(
         (items) => {
            if (items.length > 0) {
               items.forEach((item) => {
                  const testdata = item.data as SimpleReport;
                  const testHistoryItem: HistoryItem = {
                     Change: item.change,
                     TestdataId: item.id,
                     Url: `/testreport/${item.id}`,
                     HasSucceeded: testdata.HasSucceeded,
                     Date: testdata.ReportCreatedOn,
                     TotalDurationSeconds: testdata.TotalDurationSeconds,
                     ErrorCount: testdata.Errors.length,
                  };
                  historyItemsRef.current.push(testHistoryItem);
               });
            }
         }
      ).catch(
         (reason) => console.error(reason)
      ).finally(
         () => { setHistoryLoaded(true) }
      );
   }, []);

   return (
      <Stack styles={{ root: { paddingTop: 18, paddingRight: 0 } }}>
         <Stack className={hordeClasses.raised}>
            <Stack tokens={{ childrenGap: 9 }} grow>
               <Stack horizontal tokens={{ childrenGap: 12 }}>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>History</Text>
               </Stack>
               {!historyLoaded && <Stack><Spinner styles={{ root: { padding: 10 } }} size={SpinnerSize.medium}></Spinner></Stack>}
               {(historyLoaded || historyItems.length > 0) &&
                  <Stack styles={{ root: { width: 500 } }}>
                     {historyItems.map((item) => <HistoryItemPane key={item.TestdataId} item={item} selected={testDataHandler.cursor?.id === item.TestdataId} />)}
                  </Stack>
               }
               {(historyLoaded && historyItems.length === 0) && <Stack><Text> No history </Text></Stack>}
            </Stack>
         </Stack>
      </Stack>
   );
}

export const TestReportView: React.FC<{ data: SimpleReport, query: URLSearchParams }> = (props) => {

   const { hordeClasses, modeColors } = getHordeStyling();
   const { styles, stateStyles } = getStyling();

   const { data } = props;

   const style = data.HasSucceeded ? stateStyles.success : stateStyles.failure;

   return (
      <Stack className={styles.container} styles={{ root: { backgroundColor: modeColors.backend, paddingLeft: 24, paddingTop: 12, paddingRight: 12 } }}>

         <Stack styles={{ root: { paddingTop: 18, paddingRight: 0 } }}>
            <Stack className={hordeClasses.raised}>
               <Stack tokens={{ childrenGap: 12 }} grow>
                  <Stack>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Summary</Text>
                  </Stack>
                  <Stack styles={{ root: { paddingLeft: 8 } }}>
                     <Text>This test pass run on <span style={{ fontWeight: 'bold' }}>{data.ReportCreatedOn}</span> for a duration of <span style={{ fontWeight: 'bold' }}>{secToElapsed(data.TotalDurationSeconds)}</span> on <span style={{ fontWeight: 'bold' }}>{data.Description}</span></Text>
                     <Stack horizontal>
                        <Stack.Item><Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Status:&nbsp;</Text></Stack.Item>
                        <Stack.Item><Icon className={style} iconName="Square" /></Stack.Item>
                        <Stack.Item align="center" styles={{ root: { paddingLeft: 8, width: "100%", fontWeight: "bold" } }}>{data.Status}</Stack.Item>
                     </Stack>
                     <Text>
                        {data.Errors.length > 0 && <span style={{ fontWeight: 'bold' }}>{data.Errors.length} error messages. </span>}
                        {data.Warnings.length > 0 && <span>{data.Warnings.length} warning messages. </span>}
                     </Text>
                     {data.URLLink &&
                        <Stack horizontal><Text variant="medium" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Link to:&nbsp;</Text><URLLink link={data.URLLink} /></Stack>
                     }
                  </Stack>
               </Stack>
            </Stack>
         </Stack>

         <Stack styles={{ root: { paddingTop: 18, paddingRight: 0 } }}>
            <Stack className={hordeClasses.raised}>
               <Stack tokens={{ childrenGap: 9 }} grow>
                  <Stack>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Messages</Text>
                  </Stack>
                  {(data.Errors.length > 0 || data.Warnings.length > 0) &&
                     <Stack>
                        {data.Errors.map((line, index) => <LogItem key={`E${index}`} line={line} level={LogLevel.Error} />)}
                        {data.Warnings.map((line, index) => <LogItem key={`W${index}`} line={line} level={LogLevel.Warning} />)}
                     </Stack>
                  }
                  {(data.Errors.length === 0 && data.Warnings.length === 0) && <Stack><Text> No log event reported </Text></Stack>}
                  {data.Logs.length > 0 && <Stack><Text styles={{ root: { fontWeight: "bold" } }}>Logs: </Text>{data.Logs.map((item) => <LogLink key={item} to={item} />)}</Stack>}
               </Stack>
            </Stack>
         </Stack>

         <HistoryPanel />

      </Stack>
   );
}

export const ReportLinkItemPane: React.FC<{ id: string, fullName: string }> = (props) => {
   const { id, fullName } = props;
   const testdata = useRef<TestData>();
   const result = useRef<SimpleReport>();
   const history = useRef<HistoryItem[]>();
   const [testdataLoading, setTestdataLoading] = useState(false);
   const [historyLoaded, setHistoryLoaded] = useState(false);
   const [historyVisible, setHistoryVisible] = useState(false);

   const { styles, stateStyles } = getStyling();

   async function getTestData() {
      if (testdata.current !== undefined || testdataLoading) {
         return;
      }
      setTestdataLoading(true);
      try {
         testdata.current = await backend.getTestData(id);
         result.current = testdata.current.data as SimpleReport;
      } catch (reason) {
         console.error(`Failed to load test data id: ${id}`);
         throw reason;
      } finally {
         setTestdataLoading(false);
      }
   }

   async function getTestHistory() {
      if (history.current !== undefined || historyLoaded) {
         return;
      }
      try {
         if (testdataLoading) {
            // we wait while the test data is loading.
            do { await wait(3 * 1000); } while (testdataLoading);
         } else {
            // Otherwise we fetch it
            await getTestData();
         }
         if (testdata.current !== undefined && history.current === undefined) {
            const rawHistoryItems = await backend.getTestDataHistory(testdata.current.streamId, fullName, testdata.current.change);
            const historyItems: HistoryItem[] = [];
            rawHistoryItems.forEach((item) => {
               if (historyItems.length > 0 && item.change === historyItems[historyItems.length - 1].Change) return; // keep only item per changelist
               const testdata = item.data as SimpleReport;
               const testHistoryItem: HistoryItem = {
                  Change: item.change,
                  TestdataId: item.id,
                  Url: !testdata.URLLink ? `/testreport/${item.id}` : getUrlFromLink(testdata.URLLink).url,
                  HasSucceeded: testdata.HasSucceeded,
                  Date: testdata.ReportCreatedOn,
                  TotalDurationSeconds: testdata.TotalDurationSeconds,
                  ErrorCount: testdata.Errors.length,
               };
               historyItems.push(testHistoryItem);
            });
            history.current = historyItems;
         }
      } catch (reason) {
         console.error(reason);
      } finally {
         setHistoryLoaded(true);
      }
   }

   function onClickHistory() {
      const needHistoryVisible = !historyVisible;
      setHistoryVisible(needHistoryVisible);
      if (needHistoryVisible) {
         getTestHistory();
      }
   }
   function onHistoryBlur(ev: any) {
      if (!ev.currentTarget.contains(ev.relatedTarget as Node)) setHistoryVisible(false);
   }

   if (testdata.current === undefined) {
      getTestData();
   }

   return (
      <Stack horizontal wrap tokens={{ childrenGap: 30 }}>
         {result.current !== undefined &&
            <Stack horizontal>
               <Stack.Item><Icon className={result.current.HasSucceeded ? stateStyles.success : stateStyles.failure} iconName="Square" /></Stack.Item>
               <Stack.Item align="center" styles={{ root: { paddingLeft: 8, width: "100%", fontWeight: "bold" } }}>{result.current.Status}</Stack.Item>
            </Stack>
         }
         {result.current !== undefined && result.current.URLLink &&
            <a href={getUrlFromLink(result.current.URLLink).url}>Report Link</a>
         }
         <Stack onClick={(ev) => { onClickHistory() }} tabIndex={0} onBlur={onHistoryBlur} className={styles.itemHover} style={{ position: "relative", cursor: "pointer" }}>
            <Text styles={{ root: { fontWeight: 'bold' } }}>History</Text>
            {historyVisible &&
               <div style={{ position: "absolute", width: 500, top: "100%", maxHeight: 300, cursor: 'auto' }} className={styles.historyList} onClick={(ev) => { ev.stopPropagation() }}>
                  {!historyLoaded && <Spinner styles={{ root: { padding: 10 } }} size={SpinnerSize.small}></Spinner>}
                  {historyLoaded && history.current !== undefined && history.current.length > 0 &&
                     history.current.map((historyItem) => <HistoryItemPane key={historyItem.Change} item={historyItem} selected={id === historyItem.TestdataId} />)
                  }
                  {historyLoaded && history.current !== undefined && history.current.length === 0 &&
                     <div><Text variant="smallPlus">no history found</Text></div>
                  }
               </div>
            }
         </Stack>
      </Stack>
   );
}
