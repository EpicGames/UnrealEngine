// Copyright Epic Games, Inc. All Rights Reserved.

import { Plugin, PluginMount } from "../pluginTypes";
import { TestReportView, ReportLinkItemPane } from "./components/SimpleReport";

class SimpleTestReportPlugin implements Plugin {

    name = "Simple Report";
    components = [
        {
            mount: PluginMount.TestReportPanel,
            component: TestReportView,
            id: "Simple Report"
        },
        {
            mount: PluginMount.TestReportLink,
            component: ReportLinkItemPane,
            id: "Simple Report"
        }
    ];
}

const simpleTestReportPlugin = new SimpleTestReportPlugin();
export default simpleTestReportPlugin; 
