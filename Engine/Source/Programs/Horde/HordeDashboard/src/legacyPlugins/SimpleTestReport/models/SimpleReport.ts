// Copyright Epic Games, Inc. All Rights Reserved.

export enum LogLevel {
    Info = "Info",
    Error = "Error",
    Warning = "Warning",
}

export type SimpleReport = {
    Description: string;
    HasSucceeded: boolean;
    Status: string;
    ReportCreatedOn: string;
    TotalDurationSeconds: number;
    URLLink: string;
    Logs: string[];
    Errors: string[];
    Warnings: string[];
}

export type HistoryItem = {
    TestdataId: string;
    Url: string;
    Change: number;
    HasSucceeded: boolean;
    Date: string;
    TotalDurationSeconds: number;
    ErrorCount: number;
}