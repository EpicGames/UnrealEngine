// Copyright Epic Games, Inc. All Rights Reserved.

import { Checkbox, Icon, Image, Spinner, SpinnerSize, Stack, Text } from '@fluentui/react';
import { mergeStyleSets, mergeStyles } from '@fluentui/react/lib/Styling';
import React, { useEffect, useRef, useState } from 'react';
import { useNavigate } from 'react-router';
import { Link } from 'react-router-dom';
import dashboard from "../../../backend/Dashboard";
import { projectStore } from "../../../backend/ProjectStore";
import { TestDataWrapper } from '../../../backend/TestDataHandler';
import { msecToElapsed } from '../../../base/utilities/timeUtils';
import { testDataHandler } from '../../../components/TestReportView';
import { getHordeStyling } from '../../../styles/Styles';
import { getHordeTheme } from '../../../styles/theme';
import { EventType, Metadata, TestDetails, TestEntry, TestEntryArtifact, TestPassSummary, TestResult, TestState, TestStateHistoryItem } from '../models/UnrealAutomatedTests';

let _styles: any;
let _gutterHystoryStyles: any;
let _stateStyles: any;

const getStyling = () => {
   const theme = getHordeTheme();

   const styles = _styles ?? mergeStyleSets({
      container: {
         overflow: 'auto',
         height: 'calc(100vh - 165px)',
         marginTop: 8
      },
      item: [
         {
            fontSize: "11px",
            fontFamily: "Horde Cousine Regular"
         }
      ],
      gutter: [
         {
            padding: 0,
            margin: 0,
            paddingTop: 0,
            paddingBottom: 0,
            paddingRight: 14,
            marginTop: 0,
            marginBottom: 0,
            height: 20
         }
      ],
      gutterError: [
         {
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
            borderLeftStyle: 'solid',
            borderLeftColor: "#EC4C47"
         }, gutterClass
      ],
      gutterWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",
            borderLeftStyle: 'solid',
            borderLeftColor: "#F7D154"
         }, gutterClass
      ],
      gutterSuccess: [
         {
            borderLeftStyle: 'solid',
            borderLeftColor: theme.palette.green
         }, gutterClass
      ],
      itemWarning: [
         {
            background: dashboard.darktheme ? "#1E1817" : "#FEF8E7",
         }
      ],
      itemError: [
         {
            background: dashboard.darktheme ? "#330606" : "#FEF6F6",
         }
      ],
      itemHover: {
         cursor: "pointer",
         selectors: {
            ':hover': {
               background: theme.palette.neutralLight
            }
         }
      },
      itemHighlighted: {
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid"
      },
      historyList: {
         zIndex: 1,
         borderWidth: "1px",
         borderColor: "#888",
         borderStyle: "solid",
         backgroundColor: theme.palette.white,
         padding: "3px",
         overflow: "auto"
      }
   });

   const gutterHystoryStyles = _gutterHystoryStyles ?? new Map<string, string>([
      [TestState.Success, styles.gutterSuccess],
      [TestState.InProcess, mergeStyles({
         background: dashboard.darktheme ? "#141115" : "#E4F1F5",
         borderLeftStyle: 'solid',
         borderLeftColor: "#01BCF2"
      }, gutterClass)],
      [TestState.NotRun, mergeStyles({
         borderLeftStyle: 'solid',
         borderLeftColor: "#A19F9D"
      }, gutterClass)],
      [TestState.SuccessWithWarnings, styles.gutterWarning],
      [TestState.Failed, styles.gutterError],
   ]);

   const stateStyles = _stateStyles ?? new Map<string, string>([
      [TestState.Success, mergeStyles({ color: "#9BF95C", userSelect: "none" }, iconClass)],
      [TestState.InProcess, mergeStyles({ color: "#01BCF2", userSelect: "none" }, iconClass)],
      [TestState.NotRun, mergeStyles({ color: "#A19F9D", userSelect: "none" }, iconClass)],
      [TestState.SuccessWithWarnings, mergeStyles({ color: "#F7D154", userSelect: "none" }, iconClass)],
      [TestState.Failed, mergeStyles({ color: "#EC4C47", userSelect: "none" }, iconClass)],
      [TestState.Unknown, mergeStyles({ color: dashboard.darktheme ? "#666666" : "#000000", userSelect: "none" }, iconClass)],
   ]);

   _styles = styles;
   _gutterHystoryStyles = gutterHystoryStyles;
   _stateStyles = stateStyles;

   return {
      theme: theme,
      styles: styles,
      stateStyles: stateStyles,
      gutterHystoryStyles: gutterHystoryStyles
   }
}

const gutterClass = mergeStyles({
   borderLeftWidth: 6,
   padding: 0,
   margin: 0,
   paddingTop: 0,
   paddingBottom: 0,
   paddingRight: 8,
   marginTop: 0,
   marginBottom: 0,
   height: 20
});

const iconClass = mergeStyles({
   fontSize: 12
});

const getTestStateStyles = (test: TestResult): string | undefined => {
   const { stateStyles } = getStyling();
   if (!stateStyles.has(test.State)) {
      return stateStyles.get(TestState.Unknown);
   }
   return stateStyles.get(test.Warnings > 0 && test.State === TestState.Success ? TestState.SuccessWithWarnings : test.State)
}

const shadowBoxSlyle = {
   boxShadow: '0 4px 8px 0 rgba(0, 0, 0, 0.2), 0 6px 20px 0 rgba(0, 0, 0, 0.19)',
   margin: 5,
   paddingRight: 5
};

const stateLabels = new Map<string, string>([
   [TestState.Success, 'Passed'],
   [TestState.InProcess, 'Incomplete'],
   [TestState.NotRun, 'Unexecuted'],
   [TestState.SuccessWithWarnings, 'Passed'],
   [TestState.Failed, 'Failed'],
   [TestState.Skipped, 'Skipped'],
   [TestState.Unknown, 'Unknown'],
]);
const getStateLabel = (state: string): string | undefined => { return stateLabels.has(state) ? stateLabels.get(state) : state }

function getMetadata(collection: Metadata, key: string): string | undefined {
   if (collection) {
      return collection[key];
   }

   return undefined;
}

const copyToClipboard = (value: string | undefined) => {

   if (!value) {
      return;
   }

   const el = document.createElement('textarea');
   el.value = value;
   document.body.appendChild(el);
   el.select();
   document.execCommand('copy');
   document.body.removeChild(el);
}

const missingImage = "/images/missing-image.png";
const MissingImageLabel = (): JSX.Element => { return <span style={{ fontWeight: 'bold' }}> [missing image]</span> }
type ImageLinks = { approved?: string, unapproved?: string, difference?: string }


const EntryPane: React.FC<{ entry: TestEntry, testArtifacts: TestEntryArtifact[] }> = (props) => {
   const { entry, testArtifacts } = props;
   const [imageLinks, setImageLinks] = useState<ImageLinks>({});


   const { styles } = getStyling();

   const eventType = entry.Event.Type;
   const is_error = eventType === EventType.Error;
   const is_warning = eventType === EventType.Warning;
   const style = is_error ? styles.itemError : (is_warning ? styles.itemWarning : styles.item);
   const gutterStyle = is_error ? styles.gutterError : (is_warning ? styles.gutterWarning : styles.gutter);

   const artifact = testArtifacts.find((value) => value.Id === entry.Event.Artifact);
   const artifactRef = useRef(artifact);

   const need_image_comparison = artifact !== undefined && artifact.Type === "Comparison";
   useEffect(() => {
      const artifact = artifactRef.current;
      if (artifact !== undefined) {
         const findLinks = async () => {
            const imageLinks: ImageLinks = {};
            // Approved            
            imageLinks.approved = artifact.Files.Approved && await testDataHandler.cursor?.getArtifactImageLink(artifact.Files.Approved);
            // Unapproved            
            imageLinks.unapproved = artifact.Files.Unapproved && await testDataHandler.cursor?.getArtifactImageLink(artifact.Files.Unapproved);
            // Difference
            imageLinks.difference = artifact.Files.Difference && await testDataHandler.cursor?.getArtifactImageLink(artifact.Files.Difference);

            setImageLinks(imageLinks);
         }
         findLinks();
      }
   }, []);

   return (
      <Stack className={styles.item} disableShrink={true} styles={{ root: { paddingLeft: 8, width: "100%" } }}>
         <Stack horizontal>
            <Stack className={gutterStyle}></Stack>
            <Stack.Item className={style} align="center" styles={{ root: { paddingLeft: 8, width: "100%" } }}><pre style={{ margin: 0, whiteSpace: "pre-wrap" }}>[{entry.Timestamp}] {eventType}: {entry.Event.Message}</pre></Stack.Item>
         </Stack>
         {need_image_comparison &&
            <Stack styles={{ root: { paddingLeft: 16 } }}>
               <Stack><Text variant="medium" styles={{ root: { fontWeight: "bold" } }}>Image comparison: {artifact?.Name}</Text></Stack>
               <Stack horizontal>
                  {(is_error || is_warning) &&
                     <Stack styles={{ root: { padding: 5 } }}>
                        <a href={imageLinks.approved}><Image width={400} src={imageLinks.approved || missingImage} alt={artifact?.Files.Approved} /></a>
                        <Stack.Item align="center">Approved{!imageLinks.approved && MissingImageLabel()}</Stack.Item>
                     </Stack>
                  }
                  {(is_error || is_warning) &&
                     <Stack styles={{ root: { padding: 5 } }}>
                        <a href={imageLinks.difference}><Image width={400} src={imageLinks.difference || missingImage} alt={artifact?.Files.Difference} /></a>
                        <Stack.Item align="center">Difference{!imageLinks.difference && MissingImageLabel()}</Stack.Item>
                     </Stack>
                  }
                  <Stack styles={{ root: { padding: 5 } }}>
                     <a href={imageLinks.unapproved}><Image width={400} src={imageLinks.unapproved || missingImage} alt={artifact?.Files.Unapproved} /></a>
                     <Stack.Item align="center">Unapproved{!imageLinks.unapproved && MissingImageLabel()}</Stack.Item>
                  </Stack>
               </Stack>
            </Stack>
         }
      </Stack>
   );
}

const HistoryItem: React.FC<{ item: TestStateHistoryItem, testName: string, selected: boolean }> = (props) => {
   const { item, testName, selected } = props;

   const {theme, gutterHystoryStyles, styles} = getStyling();

   const gutterStyle = gutterHystoryStyles.get(item.State);

   return (
      <Stack horizontal className={`${selected ? styles.itemHighlighted : ""}`}>
         <Stack className={gutterStyle}></Stack>
         <Stack.Item>
            <Link to={`/testreport/${item.TestdataId}?test=${testName}`} className={`${styles.itemHover}`}>
               <Text variant="smallPlus">{getStateLabel(item.State)} on {item.Change}</Text>
            </Link>
            {item.RangeUrl &&
               <a href={item.RangeUrl} target="blank" className={`${styles.itemHover}`}>
                  <Text variant="smallPlus" styles={{ root: { color: theme.palette.neutralDark, paddingLeft: 4, paddingRight: 4 } }}>[ Swarm Range ]</Text>
               </a>}
         </Stack.Item>
      </Stack>
   );
}

const IconItem: React.FC<{ item: TestResult }> = (props) => {
   const { item } = props;
   const navigate = useNavigate();
   const style = getTestStateStyles(item);
   function onClickTest() {
      navigate(`${window.location.pathname}?test=${item.FullTestPath}`, { replace: true });
   }
   return (
      <Icon styles={{ root: { cursor: "pointer" } }} className={style} iconName="Square" title={item.FullTestPath} onClick={(ev) => { onClickTest() }} />
   );
}

const TestResultPane: React.FC<{ test: TestResult, selected: boolean }> = (props) => {
   const { test, selected } = props;
   const [testDetails, setTestDetails] = useState<TestDetails | undefined>(undefined);
   const [loading, setLoading] = useState(false);
   const [visible, setVisible] = useState(selected);
   const [filterError, setFilterError] = useState(true);
   const [filterWarning, setFilterWarning] = useState(true);
   const [filterInfo, setFilterInfo] = useState(test.State !== TestState.Failed);
   const [historyLoaded, setHistoryLoaded] = useState(false);
   const [historyVisible, setHistoryVisible] = useState(false);
   const historyRef = useRef<TestStateHistoryItem[]>([]);

   const navigate = useNavigate();

   const { styles } = getStyling();

   function onClickTest() {
      if (selected) {
         if (visible) {
            // remove test name selection
            navigate(window.location.pathname, { replace: true });
            return;
         }
      }
      // add test name selection
      navigate(`${window.location.pathname}?test=${test.FullTestPath}`, { replace: true });
   }

   function onClickTestName() {
      if (testDetails === undefined) {
         getTestDetails();
         setVisible(true);
         return;
      }

      setVisible(!visible);
   }

   async function getTestDetails() {
      if (testDetails !== undefined) {
         return;
      }

      if (loading) {
         return;
      }
      testDataHandler.cursor?.findArtifactData(test.ArtifactName).then((value) => {
         setTestDetails(value as TestDetails);
      }).catch((reason) => {
         console.error(reason);
      }).finally(() => { setLoading(false) });

      setLoading(true);
   }

   function onChangeFilterError(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterError(checked);
   }
   function onChangeFilterWarning(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterWarning(checked);
   }
   function onChangeFilterInfo(ev?: React.FormEvent<HTMLElement | HTMLInputElement>, checked?: boolean) {
      checked !== undefined && setFilterInfo(checked);
   }

   function isEntryNeedDisplay(entry: TestEntry, index?: number, array?: TestEntry[]): boolean {
      const entryEventType = entry.Event.Type;
      if (filterError && entryEventType === EventType.Error) {
         return true;
      }
      if (filterWarning && entryEventType === EventType.Warning) {
         return true;
      }
      if (filterInfo && entryEventType === EventType.Info) {
         return true;
      }

      return false;
   }

   function getTestHistory(testFullName: string, testdataItems: TestDataWrapper[]): TestStateHistoryItem[] {
      const testResults: TestStateHistoryItem[] = [];

      let previousChange: number = 0;
      const stream = projectStore.streamById(testDataHandler.streamId)!;
      const project = projectStore.byId(stream?.projectId)!;
      const name = project?.name === "Engine" ? "UE4" : project.name;

      testdataItems.reverse().forEach((item) => {
         const testdata = item.data as TestPassSummary;
         const foundTest = testdata.Tests.find(testItem => testItem.FullTestPath === testFullName);

         if (foundTest !== undefined) {
            const testHistoryItem: TestStateHistoryItem = {
               Change: item.change,
               RangeUrl: previousChange > 0 ? `${dashboard.swarmUrl}/files/${name}/${stream.name}?range=@${previousChange},@${item.change}#commits` : undefined,
               TestdataId: item.id,
               State: foundTest.Warnings > 0 && foundTest.State === TestState.Success ? TestState.SuccessWithWarnings : foundTest.State,
            };
            testResults.push(testHistoryItem);
         }
         previousChange = item.change;
      });

      return testResults.reverse();
   }

   function onClickHistory() {
      const needHistoryVisible = !historyVisible;
      setHistoryVisible(needHistoryVisible);
      if (needHistoryVisible && !historyLoaded) {
         testDataHandler.getCursorHistory(undefined, undefined, 100).then(
            (items) => historyRef.current = getTestHistory(test.FullTestPath, items)
         ).catch(
            (reason) => console.error(reason)
         ).finally(
            () => { setHistoryLoaded(true) }
         );
      }
   }
   function onHistoryBlur(ev: any) {
      // onBlur is used on the History widget in order to make the widget hide when user click outside of it (it loses the focus).
      // However, item links clicked in the widget make the parent widget lose its focus, so we check if the focus is lost in favor
      // of one of its children; in that case the widget is kept visible.
      if (!ev.currentTarget.contains(ev.relatedTarget as Node)) setHistoryVisible(false);
   }

   const style = getTestStateStyles(test);
   const history = historyRef.current;

   if (selected) {
      getTestDetails();
      if (!visible) {
         setVisible(true); // Set the visibility when using the top icon for focus
      }
   }

   const scrollToSelected = (ref: any) => {
      if (ref && selected && visible) {
         ref.scrollIntoView({ behavior: 'smooth', block: 'start' });
      }
   }

   return (
      <Stack styles={{ root: { paddingBottom: 6 } }} horizontalAlign="start" onClick={(ev) => { onClickTest() }}>
         <div onClick={(ev) => { onClickTestName() }} className={styles.itemHover} style={{ width: "100%" }} ref={scrollToSelected}>
            <Stack horizontal>
               <Icon styles={{ root: { paddingTop: 7, paddingRight: 8 } }} className={style} iconName="Square" />
               <Stack.Item disableShrink><Text variant="mediumPlus" >{test.TestDisplayName}</Text></Stack.Item>
            </Stack>
         </div>
         {loading && <Spinner styles={{ root: { padding: 10 } }} size={SpinnerSize.medium}></Spinner>}
         {testDetails && visible &&
            <div style={shadowBoxSlyle} onClick={(ev) => { if (selected) ev.stopPropagation() }}>
               <Stack horizontal tokens={{ childrenGap: 10 }} styles={{ root: { padding: 5 } }}>
                  <Text styles={{ root: { fontWeight: 'bold' } }}>Filters:</Text>
                  <Checkbox label="Error" defaultChecked={filterError} onChange={onChangeFilterError} />
                  <Checkbox label="Warning" defaultChecked={filterWarning} onChange={onChangeFilterWarning} />
                  <Checkbox label="Info" defaultChecked={filterInfo} onChange={onChangeFilterInfo} />
                  <Stack onClick={(ev) => { onClickHistory() }} tabIndex={0} onBlur={onHistoryBlur} className={styles.itemHover} style={{ position: "relative" }}>
                     <Text styles={{ root: { fontWeight: 'bold', paddingLeft: 4, paddingRight: 4 } }}>History</Text>
                     {historyVisible &&
                        <div style={{ position: "absolute", width: 250, top: "100%", maxHeight: 300 }} className={styles.historyList} onClick={(ev) => { ev.stopPropagation() }}>
                           {!historyLoaded && <Spinner styles={{ root: { padding: 10 } }} size={SpinnerSize.small}></Spinner>}
                           {historyLoaded && history.length > 0 &&
                              history.map((historyItem) => <HistoryItem key={historyItem.Change} item={historyItem} testName={test.FullTestPath} selected={historyItem.Change === testDataHandler.cursor?.change} />)
                           }
                           {historyLoaded && history.length === 0 && <div><Text variant="smallPlus">no history found</Text></div>}
                        </div>
                     }
                  </Stack>
                  <Text onClick={(ev) => { copyToClipboard(test.FullTestPath); }} styles={{ root: { fontWeight: 'bold', paddingLeft: 4, paddingRight: 4 } }} className={styles.itemHover}>To Clipboard</Text>
               </Stack>
               {!!testDetails.Entries && testDetails.Entries.filter(isEntryNeedDisplay).map((value, index) => <EntryPane key={index} entry={value} testArtifacts={testDetails.Artifacts ?? []} />)}
               {(!testDetails.Entries || testDetails.Entries.length === 0) && <Text styles={{ root: { padding: 8, fontWeight: 'bold' } }}>No event for this test.</Text>}
            </div>
         }
      </Stack>
   );
}

const TestResultPanel: React.FC<{ tests: TestResult[], title?: string, selected?: string }> = (props) => {

   const { hordeClasses } = getHordeStyling();
   const { theme } = getStyling();

   const { tests, title, selected } = props;

   const suites = new Map<string, TestResult[]>();
   tests.forEach((test) => {
      const testSuite = test.FullTestPath.replace('.' + test.TestDisplayName, '');
      if (!suites.has(testSuite)) {
         suites.set(testSuite, [test]);
      } else {
         suites.get(testSuite)?.push(test);
      }
   });
   const suiteKeys = Array.from(suites.keys());

   return (
      <Stack styles={{ root: { paddingTop: 18, paddingRight: 0 } }}>
         <Stack className={hordeClasses.raised}>
            <Stack tokens={{ childrenGap: 9 }} grow>
               <Stack>
                  <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>{title} tests ({tests.length})</Text>
               </Stack>
               {suiteKeys.map((suite) => {
                  return (<Stack styles={{ root: { paddingLeft: 8 } }} tokens={{ childrenGap: 5 }} grow key={`${title}-${suite}`}>
                     <Text variant="mediumPlus" styles={{ root: { color: theme.palette.neutralDark } }}>{suite}</Text>
                     <Stack styles={{ root: { paddingLeft: 8 } }}>
                        {suites.get(suite)?.map((test: TestResult) => <TestResultPane test={test} selected={selected === test.FullTestPath} key={test.ArtifactName} />)}
                     </Stack>
                  </Stack>)
               })}
            </Stack>
         </Stack>
      </Stack>
   );
}

export const TestPassSummaryView: React.FC<{ data: TestPassSummary, query: URLSearchParams }> = (props) => {

   const { hordeClasses, modeColors } = getHordeStyling();   
   const { stateStyles, styles } = getStyling();

   const { data, query } = props;
   const failedTests: TestResult[] = [];
   const notrunTests: TestResult[] = [];
   const inprocessTests: TestResult[] = [];
   const passedTests: TestResult[] = [];
   const skippedTests: TestResult[] = [];
   const TestsMap = new Map<string, TestResult[]>([
      [TestState.Failed, failedTests],
      [TestState.NotRun, notrunTests],
      [TestState.InProcess, inprocessTests],
      [TestState.Success, passedTests],
      [TestState.Skipped, skippedTests],
   ]);
   data.Tests.forEach((test) => TestsMap.get(test.State)?.push(test));

   const selectedTest = query.get('test') ? query.get('test')! : undefined;

   return (
      <Stack className={styles.container} styles={{ root: { backgroundColor: modeColors.background, paddingLeft: 24, paddingTop: 12, paddingRight: 12 } }}>

         <Stack styles={{ root: { paddingTop: 18, paddingRight: 0 } }}>
            <Stack className={hordeClasses.raised}>
               <Stack tokens={{ childrenGap: 12 }} grow>
                  <Stack>
                     <Text variant="mediumPlus" styles={{ root: { fontFamily: "Horde Open Sans SemiBold" } }}>Summary</Text>
                  </Stack>
                  <Stack styles={{ root: { paddingLeft: 8 } }}>
                     <Text>This test pass ran on <span style={{ fontWeight: 'bold' }}>{data.ReportCreatedOn}</span> for a duration of <span style={{ fontWeight: 'bold' }}>{msecToElapsed(data.TotalDurationSeconds * 1000)}</span> on <span style={{ fontWeight: 'bold' }}>{getMetadata(data.Metadata, 'Platform') ?? "Unknown"}</span></Text>
                     <Text>
                        {data.FailedCount > 0 && <span><span style={{ fontWeight: 'bold' }}>{data.FailedCount}</span> tests <span style={{ fontWeight: 'bold' }} className={stateStyles.get('Fail')}>{stateLabels.get('Fail')?.toLowerCase()}</span>. </span>}
                        {data.InProcessCount > 0 && <span><span style={{ fontWeight: 'bold' }}>{data.InProcessCount}</span> tests <span style={{ fontWeight: 'bold' }} className={stateStyles.get('InProcess')}>{stateLabels.get('InProcess')?.toLowerCase()}</span>. </span>}
                        {data.NotRunCount > 0 && <span><span style={{ fontWeight: 'bold' }}>{data.NotRunCount}</span> tests <span style={{ fontWeight: 'bold' }} className={stateStyles.get('NotRun')}>{stateLabels.get('NotRun')?.toLowerCase()}</span>. </span>}
                        {data.SucceededCount > 0 &&
                           <span>
                              <span style={{ fontWeight: 'bold' }}>{(data.SucceededCount + data.SucceededWithWarningsCount)}</span> tests <span style={{ fontWeight: 'bold' }} className={stateStyles.get('Success')}>{stateLabels.get('Success')?.toLowerCase()}</span>
                              {data.SucceededWithWarningsCount > 0 &&
                                 <span>, including <span style={{ fontWeight: 'bold' }}>{data.SucceededWithWarningsCount}</span> with <span style={{ fontWeight: 'bold' }} className={stateStyles.get('SuccessWithWarnings')}>warnings</span></span>
                              }.
                           </span>
                        }
                     </Text>
                     <Stack horizontal wrap>
                        {data.Tests.map((item: TestResult) => <IconItem item={item} key={item.ArtifactName} />)}
                     </Stack>
                     {data.ReportURL && <Text>Link to: <a href={data.ReportURL}>External report</a></Text>}
                  </Stack>
               </Stack>
            </Stack>
         </Stack>

         {failedTests.length > 0 &&
            <TestResultPanel tests={failedTests} title={stateLabels.get('Fail')} selected={selectedTest} />
         }
         {inprocessTests.length > 0 &&
            <TestResultPanel tests={inprocessTests} title={stateLabels.get('InProcess')} selected={selectedTest} />
         }
         {notrunTests.length > 0 &&
            <TestResultPanel tests={notrunTests} title={stateLabels.get('NotRun')} selected={selectedTest} />
         }
         {skippedTests.length > 0 &&
            <TestResultPanel tests={skippedTests} title={stateLabels.get('Skipped')} selected={selectedTest} />
         }
         {passedTests.length > 0 &&
            <TestResultPanel tests={passedTests} title={stateLabels.get('Success')} selected={selectedTest} />
         }
      </Stack>
   );
}
