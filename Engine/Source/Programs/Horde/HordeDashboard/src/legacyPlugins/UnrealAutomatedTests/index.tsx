// Copyright Epic Games, Inc. All Rights Reserved.

import { Plugin, PluginMount } from "../pluginTypes";
import { TestPassSummaryView } from "./components/UnrealAutomatedTests";

class UnrealAutomatedTestsPlugin implements Plugin {

    name = "Unreal Automated Tests";
    components = [{
        mount: PluginMount.TestReportPanel,
        component: TestPassSummaryView,
        id: "Unreal Automated Tests"
    }];

}

const unrealAutomatedTestsPlugin = new UnrealAutomatedTestsPlugin();
export default unrealAutomatedTestsPlugin;

