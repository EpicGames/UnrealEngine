// Copyright Epic Games, Inc. All Rights Reserved.

export enum TestState {
    Failed = "Fail",
    Success = "Success",
    SuccessWithWarnings = "SuccessWithWarnings",
    NotRun = "NotRun",
    InProcess = "InProcess",
    Skipped = "Skipped",
    Unknown = "Unknown",
}

export enum EventType {
    Info = "Info",
    Error = "Error",
    Warning = "Warning",
}

export type TestEntryArtifact = {
    Id: string;
    Name: string;
    Type: string;
    Files: { Difference: string, Approved: string, Unapproved: string };
}

export type TestEntry = {
    Filename: string;
    LineNumber: number;
    Timestamp: string;
    Event: { Type: string, Message: string, Context: string, Artifact: string };
}

export type TestDetails = {
    TestDisplayName: string;
    FullTestPath: string;
    State: string;
    DeviceInstance: string;
    Errors: number;
    Warnings: number;
    Entries?: TestEntry[];
    Artifacts?: TestEntryArtifact[];
}

export type TestResult = {
    TestDisplayName: string;
    FullTestPath: string;
    State: string;
    DeviceInstance: string;
    ArtifactName: string;
    Errors: number;
    Warnings: number;
}

export type TestDevice = {
    DeviceName: string;
    Instance: string;
    Platform: string;
    OSVersion: string;
    Model: string;
    GPU: string;
    CPUModel: string;
    RAMInGB: number;
    RenderMode: string;
    RHI: string;
}

export type Metadata = {[Key in string]: string}

export type TestPassSummary = {
    ReportURL: string;
    FailedCount: number;
    NotRunCount: number;
    InProcessCount: number;
    ReportCreatedOn: string;
    SucceededCount: number;
    SucceededWithWarningsCount: number;
    TotalDurationSeconds: number;
    Tests: TestResult[];
    Devices: TestDevice[];
    Metadata: Metadata;
}

export type TestStateHistoryItem = {
    TestdataId: string;
    Change: number;
    RangeUrl?: string;
    State: string;
}
