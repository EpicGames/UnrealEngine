// Copyright Epic Games, Inc. All Rights Reserved.

import { Plugin, PluginMount, PluginRoute, PluginComponent } from "./pluginTypes";
import SimpleTestReportPlugin from "./SimpleTestReport"
import AutomatedTestSessionPlugin from "./AutomatedTestSession"
import UnrealAutomatedTestsPlugin from "./UnrealAutomatedTests"

export class Plugins {

   get routes(): PluginRoute[] {
      return this.plugins.map(p => p.routes ?? []).flat();
   }

   getComponents(mount: PluginMount, id?: string): PluginComponent[] {
      return this.plugins.map(p => {
         return p.components?.filter(c => c.mount === mount && c.id === id) ?? [];
      }).flat();
   }

   loadPlugins(pluginList: string[] | undefined): void {

      console.log("Loading Legacy Plugins");

      this.plugins.push(...[AutomatedTestSessionPlugin, SimpleTestReportPlugin, UnrealAutomatedTestsPlugin]);

      /*
      return new Promise<void>((resolve) => {

         if (!pluginList || !pluginList.length || this.plugins.length) {
            return resolve();
         }

         Promise.all(pluginList.map(async (plugin: any) => {
            await import(`./${plugin}/index.js`).then((m: any) => {
               this.plugins.push(m.default);
               console.log(`loaded plugin: ${plugin}`);
            }).catch((reason: any) => {
               console.error(`unable to load plugin: ${plugin}, ${reason}`);
            });
         })).finally(() => {
            resolve();
         });
      });
      */
   }

   private plugins: Plugin[] = [];

}

const plugins = new Plugins();

export default plugins;