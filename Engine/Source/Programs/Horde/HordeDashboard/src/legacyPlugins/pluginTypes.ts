
// render components with top level routes
export type PluginRoute = {
    path: string;
    component: React.FC;
 }
 
 // render components with specific mount points
 export enum PluginMount {
    JobDetailPanel,
    TestReportPanel,
    TestReportLink,
    BuildHealthSummary,
 }
 
 export type PluginComponent = {
    mount: PluginMount;
    component: React.FC<any>;
    // optional unique identifier for the component
    id?: string;
 }
 
 export interface Plugin {
    name: string;
    routes?: PluginRoute[];
    components?: PluginComponent[];
 }
 