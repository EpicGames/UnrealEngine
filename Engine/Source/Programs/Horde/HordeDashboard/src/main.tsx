// Copyright Epic Games, Inc. All Rights Reserved.

import React from 'react'
import ReactDOM from 'react-dom/client'
import App from './App.tsx'
import './index.css'
import { configure } from 'mobx';

// configure mobx
configure({
   reactionRequiresObservable: true,
   enforceActions: "observed"
});

ReactDOM.createRoot(document.getElementById('root')!).render(<App />);

