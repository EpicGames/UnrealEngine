// Copyright Epic Games, Inc. All Rights Reserved.

import { IconContents } from '@blueprintjs/icons';
import { fontFace, IFontWeight } from '@fluentui/merge-styles';
import { registerIcons } from '@fluentui/style-utilities';
import { mergeStyleSets } from '@fluentui/react';
import { initializeIcons } from '@fluentui/font-icons-mdl2';
import { getHordeTheme } from './theme';
import dashboard from '../backend/Dashboard';

// THIS MODULE MUST BE IMPORTED FIRST AS IT SETS UP THEME AND STYLING!
// If styling isn't working, make sure index.tsx is importing this before any other module

export const preloadFonts = ["Horde Open Sans Regular", "Horde Open Sans Light", "Horde Open Sans Bold", "Horde Open Sans SemiBold",
   "Horde Raleway Regular", "Horde Raleway Bold", "Horde Cousine Regular", "Horde Cousine Bold",
   "Icons16", "FARegular400", "FASolid900"];

// register fonts
registerFontFace("Horde Open Sans Regular", `url('/fonts/OpenSans-Regular.ttf') format('truetype')`, 'normal');

// use semi-bold for these weights, note that bold/bolder font weights do not work and override normal
registerFontFace("Horde Open Sans Regular", `url('/fonts/OpenSans-SemiBold.ttf') format('truetype')`, '600');
registerFontFace("Horde Open Sans Regular", `url('/fonts/OpenSans-SemiBold.ttf') format('truetype')`, '700');
registerFontFace("Horde Open Sans Regular", `url('/fonts/OpenSans-SemiBold.ttf') format('truetype')`, '800');
registerFontFace("Horde Open Sans Regular", `url('/fonts/OpenSans-SemiBold.ttf') format('truetype')`, '900');

registerFontFace("Horde Open Sans Light", `url('/fonts/OpenSans-Light.ttf') format('truetype')`, 'lighter');
registerFontFace("Horde Open Sans Bold", `url('/fonts/OpenSans-Bold.ttf') format('truetype')`, 'bold');
registerFontFace("Horde Open Sans SemiBold", `url('/fonts/OpenSans-SemiBold.ttf') format('truetype')`, 'bold');
registerFontFace("Horde Raleway Regular", `url('/fonts/Raleway-Regular.ttf') format('truetype')`, 'normal');
registerFontFace("Horde Raleway Bold", `url('/fonts/Raleway-Bold.ttf') format('truetype')`, 'bold');
registerFontFace("Horde Cousine Regular", `url('/fonts/Cousine-Regular.ttf') format('truetype')`, 'normal');
registerFontFace("Horde Cousine Bold", `url('/fonts/Cousine-Bold.ttf') format('truetype')`, 'bold');

registerIcons({
   fontFace: {
      fontFamily: 'Icons16',
      src: `url('/fonts/icons-16.woff') format('woff')`
   },
   icons: {
      'Workflow': IconContents.COG,
      'AlignLeft': IconContents.ALIGN_LEFT,
      'Add': IconContents.ADD,
      'Filter': IconContents.FILTER,
      'Download': IconContents.DOWNLOAD,
      "Circle": IconContents.CIRCLE,
      "FullCircle": IconContents.FULL_CIRCLE,
      "Square": IconContents.SYMBOL_SQUARE,
      "Redo": IconContents.REDO,
      "Locate": IconContents.LOCATE,
      "Tick": IconContents.TICK,
      "Delete": IconContents.DELETE,
      "Build": IconContents.BUILD,
      "Edit": IconContents.EDIT,
      "Settings": IconContents.SETTINGS,
      "Duplicate": IconContents.DUPLICATE,
      "User": IconContents.USER,
      "Person": IconContents.PERSON,
      "Endorsed": IconContents.ENDORSED,
      "Cross": IconContents.CROSS,
      "ChevronUp": IconContents.CHEVRON_UP,
      "ChevronDown": IconContents.CHEVRON_DOWN,
      "ChevronLeft": IconContents.CHEVRON_LEFT,
      "ChevronRight": IconContents.CHEVRON_RIGHT,
      "DoubleChevronUp": IconContents.DOUBLE_CHEVRON_UP,
      "DoubleChevronDown": IconContents.DOUBLE_CHEVRON_DOWN,
      "Refresh": IconContents.REFRESH,
      "Warning": IconContents.WARNING_SIGN,
      "LayoutCircle": IconContents.LAYOUT_CIRCLE,
      "Error": IconContents.ERROR,
      "TickCircle": IconContents.TICK_CIRCLE,
      "Stop": IconContents.STOP,
      "KeyDelete": IconContents.KEY_DELETE,
      "Issue": IconContents.ISSUE,
      "Dot": IconContents.DOT,
      "Pin": IconContents.PIN,
      "Unpin": IconContents.UNPIN,
      "Search": IconContents.SEARCH,
      "SearchTemplate": IconContents.SEARCH_TEMPLATE,
      "History": IconContents.HISTORY,
      "Import": IconContents.IMPORT,
      "Pulse": IconContents.PULSE,
      "Commit": IconContents.GIT_COMMIT,
      "Properties": IconContents.PROPERTIES,
      "Clip": IconContents.CLIPBOARD,
      "Pause": IconContents.PAUSE,
      "Flash": IconContents.FLASH,
      "MenuOpen": IconContents.MENU_OPEN,
      "IssueClosed": IconContents.ISSUE_CLOSED,
      "IssueNew": IconContents.ISSUE_NEW,
      "Link": IconContents.LINK,
      "Tag": IconContents.TAG,
      "More": IconContents.MORE,
      "Share": IconContents.SHARE,
      "Wrench": IconContents.WRENCH,
      "Dashboard": IconContents.DASHBOARD,
      "Document": IconContents.DOCUMENT,
      "Folder": IconContents.FOLDER_CLOSE,
      "CloudDownload": IconContents.CLOUD_DOWNLOAD,
      "Eye": IconContents.EYE_OPEN,
      "ArrowLeft": IconContents.ARROW_LEFT,
      "ArrowRight": IconContents.ARROW_RIGHT,
      "ArrowUp": IconContents.ARROW_UP,
      "Repeat": IconContents.REPEAT,
      "FlowReview": IconContents.FLOW_REVIEW,
      "FastForward": IconContents.FAST_FORWARD,
      "FilterList": IconContents.FILTER_LIST,
      "Play": IconContents.PLAY,
      "Maximize" : IconContents.MAXIMIZE,
      "Star": IconContents.STAR,
      "Clean": IconContents.CLEAN,
      "Info" : IconContents.INFO_SIGN,
      "Form": IconContents.FORM,
      "Grouped": IconContents.GROUP_OBJECTS,
      "UnGrouped": IconContents.UNGROUP_OBJECTS,
      "PredictiveAnalysis": IconContents.PREDICTIVE_ANALYSIS
   }
});

// font awesome regular
registerIcons({
   fontFace: {
      fontFamily: 'FARegular400',
      src: `url('/fonts/fa-regular-400.woff') format('woff')`
   },
   icons: {
      'FAWindowCloseRegular': "\uF410",
   }
});

// font awesome solid
registerIcons({
   fontFace: {
      fontFamily: 'FASolid900',
      src: `url('/fonts/fa-solid-900.woff') format('woff')`
   },
   icons: {
      'FAWindowCloseSolid': "\uF410",
      'FATimesSolid': "\uF00D",
   }
});


function registerFontFace(fontFamily: string, url: string, fontWeight?: IFontWeight): void {

   fontFamily = `'${fontFamily}'`;

   fontFace({
      fontFamily,
      src: url,
      fontWeight,
      fontStyle: 'normal'
   });
}


initializeIcons();

const colorSteps = [
   { step: 0, color: "#00BCF2" },
   { step: 150, color: "#5AC95A" },
   { step: 300, color: "#FF6600" },
   { step: 450, color: "#DF8BE5" },
   { step: 600, color: "#00BCF2" }
];

export function hexToRGB(hex: string) {
   const rgb = /^#?([a-f\d]{2})([a-f\d]{2})([a-f\d]{2})$/i.exec(hex)!;
   return {
      r: parseInt(rgb[1], 16),
      g: parseInt(rgb[2], 16),
      b: parseInt(rgb[3], 16)
   };
}

export function channelToHex(color: number) {
   let hex = color.toString(16);
   if (hex.length < 2) {
      hex = "0" + hex;
   }
   return hex;
}

export function linearInterpolate(colorStepValue: string) {
   let step = parseInt(colorStepValue);
   if (isNaN(step)) {
      step = 0;
   }

   const length = 150;
   const modStep = step % length;
   let hexA = "";
   let hexB = "";
   for (let idx = 0; idx < colorSteps.length - 1; idx++) {
      if (step >= colorSteps[idx].step && step < colorSteps[idx + 1].step) {
         hexA = colorSteps[idx].color;
         hexB = colorSteps[idx + 1].color;
         break;
      }
   }

   const channelsA = hexToRGB(hexA);
   const channelsB = hexToRGB(hexB);

   const result = { r: 0, g: 0, b: 0 };
   result.r = Math.trunc(channelsA.r + modStep * (channelsB.r - channelsA.r) / length);
   result.g = Math.trunc(channelsA.g + modStep * (channelsB.g - channelsA.g) / length);
   result.b = Math.trunc(channelsA.b + modStep * (channelsB.b - channelsA.b) / length);

   //console.log("step: " + step + ", modStep: " + modStep + ", A: " + hexA + ", B: " + hexB + ", rR: " + result.r + ", rG: " + result.g + ", rB: " + result.b);

   return "#" + channelToHex(result.r) + channelToHex(result.g) + channelToHex(result.b);
}

// Pre-Fluent theming support, customizations (legacy, need to be factored into theme compomonents)

export const MAX_WIDTH = '1440px';
export const ROW_HEIGHT = 42; // from DEFAULT_ROW_HEIGHTS in DetailsRow.styles.ts
export const GROUP_HEADER_AND_FOOTER_SPACING = 4;
export const GROUP_HEADER_AND_FOOTER_BORDER_WIDTH = 1;
export const GROUP_HEADER_HEIGHT = 95;
export const GROUP_FOOTER_HEIGHT: number = GROUP_HEADER_AND_FOOTER_SPACING * 4 + GROUP_HEADER_AND_FOOTER_BORDER_WIDTH * 2;



// must not be called in module scope

let hordeStyles: any = undefined;
export const getHordeStyling = () => {

   if (hordeStyles !== undefined) {
      return hordeStyles;
   }

   if (!dashboard.available) {
      console.error("getHordeStyling must not be called before dashboard is available");
   }

   const theme = getHordeTheme();

   const modeColors = {
      header: undefined,
      crumbs: undefined,
      content: theme.horde.contentBackground,
      background: theme.horde.neutralBackground,
      text: theme.semanticColors.bodyText
   }


   const hordeClasses = mergeStyleSets({
      horde: {
         selectors: {
            "a": {
               color: theme.semanticColors.link
            },
            "a:hover": {
               color: theme.semanticColors.linkHovered
            }
         }
      },
      raised: {
         boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)",
         padding: "25px 30px 25px 30px",
         backgroundColor: theme.horde.contentBackground
      },
      projectLogoCard: {
         width: 613,
         height: 300
      },
      projectLogoCardDropShadow: {
         width: 560,
         height: 280,
         margin: '10px 20px',
         boxShadow: "0 1.6px 3.6px 0 rgba(0,0,0,0.132), 0 0.3px 0.9px 0 rgba(0,0,0,0.108)"
      },
      relativeModalSmall: {
         minHeight: '400px',
         height: '500px',
         maxHeight: '700px',
         position: 'relative'
      },
      relativeWrapper: {
         height: '75vh',
         position: 'relative'
      },
      relativeWrapperFullHeight: {
         height: '92vh',
         position: 'relative'
      },
      button: {
         selectors: {
            '.ms-Button': {
               height: 22,
               borderStyle: 'hidden',
               minWidth: 0,
               padding: 8
            },
            '.ms-Button-label': {
               fontSize: '13px',
               fontFamily: "Horde Open Sans SemiBold !important"
            }
         }
      },
      badge: {
         selectors: {
            '.ms-Button': {
               borderStyle: 'hidden',
               borderRadius: 'unset',
               minWidth: 0,
               padding: 3,
               height: 18,
               color: "#FFFFFF"
            },
            '.ms-Button:hover': {
               color: "#FFFFFF",
               filter: "brightness(1.15)"
            },
            '.ms-Button-label': {
               fontSize: '10px',
               fontFamily: "Horde Open Sans SemiBold",
               whiteSpace: "nowrap"
            }
         }
      },
      badgeNoIcon: {
         selectors: {
            '.ms-Button': {
               borderStyle: 'hidden',
               borderRadius: '2px',
               minWidth: 0,
               paddingTop: 6,
               paddingBottom: 6,
               paddingLeft: 1,
               paddingRight: 1,
               height: 18,
               color: "#FFFFFF"
            },
            '.ms-Button:hover': {
               color: "#FFFFFF",
               filter: "brightness(1.15)"
            },
            '.ms-Button-label': {
               fontSize: '9px !important',
               fontFamily: "Horde Open Sans SemiBold",
               whiteSpace: "nowrap"
            }
         }
      },
      badgeCompact: {
         selectors: {
            '.ms-Button': {
               borderStyle: 'hidden',
               borderRadius: '2px',
               minWidth: 0,
               paddingTop: 3,
               paddingBottom: 3,
               paddingLeft: 3,
               paddingRight: 3,
               height: 15,
               color: "#FFFFFF"
            },
            '.ms-Button:hover': {
               color: "#FFFFFF",
               filter: "brightness(1.15)"
            },
            '.ms-Button-label': {
               fontSize: '9px !important',
               fontFamily: "Horde Open Sans SemiBold",
               whiteSpace: "nowrap"
            }
         }
      },
      labelCategoryRow: {
         selectors: {
            ':hover': {
               backgroundColor: dashboard.darktheme ? 'rgba(255,255,255,0.08) !important' : 'rgba(0,0,0,0.04) !important'
            }
         }
      },
      commandBar: {
         selectors: {
            '.ms-Button--commandBar': {
               fontFamily: 'Horde Open Sans SemiBold',
            }
         }
      },
      commandBarSmall: {      
         backgroundColor: modeColors.background,      
         selectors: {         
            '.ms-CommandBar': {
               backgroundColor: modeColors.background,
               paddingLeft: "0px !important",
               paddingRight: "0px !important",
            },
            '.ms-Button--commandBar': {
               backgroundColor: modeColors.background,
               height: 36
            },
            '.ms-Button--commandBar:hover': {
               backgroundColor: modeColors.crumbs
            },
            '.ms-Button-label': {
               fontSize: '12px',
               fontWeight: 400,
               fontFamily: 'Horde Open Sans Regular'
   
            },
            '.ms-Icon': {
               fontSize: '12px'
            },
            '.ms-Icon.is-expanded': {
               backgroundColor: "#00000000 !important"
            },
            '.ms-Button-menuIcon': {
               backgroundColor: "#00000000 !important"
            }
         }
      },
   
      iconSmall: {
         selectors: {
            '.ms-Icon': {
               fontSize: '13px'
            }
         }
      },
      iconBlue: {
         color: 'rgb(0, 120, 212)'
      },
      iconDisabled: {
         userSelect: 'none'
      },
      pivot: {
         selectors: {
            '.ms-Button': {
               fontFamily: 'Horde Open Sans SemiBold',
            },
            '.ms-Button.is-selected': {
               fontFamily: 'Horde Open Sans Bold'
            }
         }
      },
      colorPreview: {
         width: 32,
         height: 32,
         border: '1px solid #8f8f8f',
         borderRadius: 2,
         marginTop: 28,
         marginRight: 10
      },
      modal: {
         selectors: {
            ".ms-Label": {
               fontWeight: "unset",
               fontFamily: "Horde Open Sans SemiBold",
               fontSize: "12px",
               paddingTop: 0
            },
            ".ms-Button-label": {
               fontWeight: "unset",
               fontFamily: "Horde Open Sans SemiBold",
               fontSize: "12px",
               paddingTop: 0,
            },
            ".ms-TextField-fieldGroup,.ms-TextField-field,.ms-TextField-field::placeholder": {
               fontSize: "12px",
               fontFamily: "Horde Open Sans Regular",
            },
            ".ms-Dropdown-title,.ms-Dropdown-titleIsPlaceHolder": {
               fontSize: "12px",
            },
            ".ms-ComboBox": {
               fontSize: "12px",
            },
            ".ms-Checkbox": {
               marginTop: 0,
               paddingTop: 3
            },
            ".ms-Checkbox-text, .ms-Checkbox-label": {
               height: 14,
               lineHeight: 14,
            },
            ".ms-Checkbox-checkbox": {
               paddingTop: 0,
               height: "13px",
               width: "13px",
            },
            ".ms-Modal-scrollableContent": {
               overflowX: "hidden",
               overflowY: "hidden",
               maxHeight: "95vh"
            },
            ".ms-Dialog-main": {
               overflow: "hidden"
            }
         }
      }

   });
   //


   const detailClasses = mergeStyleSets({
      headerAndFooter: {
         borderTop: `${GROUP_HEADER_AND_FOOTER_BORDER_WIDTH}px solid ${theme.palette.neutralQuaternary}`,
         borderBottom: `${GROUP_HEADER_AND_FOOTER_BORDER_WIDTH}px solid ${theme.palette.neutralQuaternary}`,
         padding: GROUP_HEADER_AND_FOOTER_SPACING,
         margin: `${GROUP_HEADER_AND_FOOTER_SPACING}px 0`,
         // Overlay the sizer bars
         position: 'relative',
         overflow: 'hidden',
         backgroundColor: theme.palette.neutralLighter
      },
      headerTitle: [
         {
            fontSize: "16px",
            padding: '8px 8px',
            userSelect: 'none'

         }
      ],
      headerLinkSet: {
         margin: '4px -8px'
      },
      headerLink: {
         margin: '0 8px'
      },
      detailsRow: {
         selectors: {
            '.ms-DetailsRow': {
               borderBottom: '0px'
            },
            '.ms-DetailsRow-cell': {
               position: 'relative',
               textAlign: "center",
               padding: 0,
               overflow: "visible",
               whiteSpace: "nowrap"
            },
         }
      },
      detailsHeader: {
         selectors: {
            '.ms-DetailsHeader-cellTitle': {
               justifyContent: 'center',
               padding: 0
            },
            '.ms-DetailsHeader-cellName': {
               fontFamily: "Horde Open Sans SemiBold"
            }


         }
      },
      container: {
         overflowX: 'hidden',
         marginTop: 8

      }
   });

   hordeStyles = {
      modeColors: modeColors,
      hordeClasses: hordeClasses,
      detailClasses: detailClasses
   }

   return hordeStyles;
}


