// Copyright Epic Games, Inc. All Rights Reserved.

import { JobState, JobStepOutcome, JobStepState, LabelOutcome, LabelState } from "../backend/Api";
import dashboard, { StatusColor } from "../backend/Dashboard";

export const getJobStateColor = (state: JobState): string => {

    const colors = dashboard.getStatusColors();

    if (state === JobState.Waiting) {
        return colors.get(StatusColor.Waiting)!;
    }

    if (state === JobState.Running) {
        return colors.get(StatusColor.Running)!;
    }


    return colors.get(StatusColor.Unspecified)!;

}

export const getStepStatusColor = (state: JobStepState | undefined, outcome: JobStepOutcome | undefined): string => {

    const colors = dashboard.getStatusColors();

    let color: string | undefined;

    if (state === JobStepState.Running) {

        color = colors.get(StatusColor.Running);

        if (outcome === JobStepOutcome.Warnings) {
            color = colors.get(StatusColor.Warnings);
        }

        if (outcome === JobStepOutcome.Failure) {
            color = colors.get(StatusColor.Failure);
        }
    }

    if (state === JobStepState.Waiting) {
        color = colors.get(StatusColor.Waiting);
    }

    if (state === JobStepState.Ready) {
        color = colors.get(StatusColor.Ready);

    }

    if (state === JobStepState.Skipped) {
        color = colors.get(StatusColor.Skipped);

    }

    if (state === JobStepState.Aborted) {
        color = colors.get(StatusColor.Aborted);

    }

    if (state === JobStepState.Completed) {

        if (outcome === JobStepOutcome.Success) {
            color = colors.get(StatusColor.Success);
        } else if (outcome === JobStepOutcome.Unspecified) {
            color = colors.get(StatusColor.Skipped);
        } else if (outcome === JobStepOutcome.Warnings) {
            color = colors.get(StatusColor.Warnings);
        } else {

            color = colors.get(StatusColor.Failure);
        }
    }

    return color ?? "#AAAAAA";
}

export const getLabelColor = (state: LabelState | undefined, outcome: LabelOutcome | undefined): { primaryColor: string, secondaryColor?: string } => {

    if (!state || !outcome) {
        return {
            primaryColor: "#000000"
        }
    }

    const colors = dashboard.getStatusColors();

    if (state === LabelState.Complete) {
        switch (outcome!) {
            case LabelOutcome.Failure:
                return { primaryColor: colors.get(StatusColor.Failure)! };
            case LabelOutcome.Success:
                return { primaryColor: colors.get(StatusColor.Success)! };
            case LabelOutcome.Warnings:
                return { primaryColor: colors.get(StatusColor.Warnings)! };
            default:
                return { primaryColor: colors.get(StatusColor.Unspecified)! };
        }
    }

    if (outcome === LabelOutcome.Failure) {
        return { primaryColor: colors.get(StatusColor.Running)!, secondaryColor: colors.get(StatusColor.Failure)! };
    }

    if (outcome === LabelOutcome.Warnings) {
        return { primaryColor: colors.get(StatusColor.Running)!, secondaryColor: colors.get(StatusColor.Warnings)! };
    }

    return {
        primaryColor: colors.get(StatusColor.Running)!
    }
};

