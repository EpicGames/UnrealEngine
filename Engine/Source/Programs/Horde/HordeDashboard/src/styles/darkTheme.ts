// Copyright Epic Games, Inc. All Rights Reserved.

import { createTheme } from '@fluentui/react';
import { HordeTheme, HordeThemeExtensions } from './themeTypes';

// Primary white 
const baseWhite = "#e0e0e0"

// Primary black (backgrounds)
const baseBlack = "#0f0f0f"

// Neutral color for panels
const baseNeutral = "#181A1B"

const neutralLight = "#3F3F3F"


// Text in drop downs, etc
const neutralPrimary = "#c0c0c0"

// Lighter neutral color, disabled text
const neutralDisabled = "#868686"

// Primary color, for main action buttons, command bar icons
const basePrimary = "#0078D4"

// Widget (input) border colors
const borderColor = neutralLight
const borderHoverColor = "#4f4f4f"

// Checkbox border colors (brighter for visibility in dark mode)
const checkboxBorderColor = "#6F6F6F"
const checkboxBorderHoverColor = "#8F8F8F"
const checkboxBorderDisabledColor = "#4F4F4F"

// Sub-menu divider color
const menuHeader = "#2b94fe";

// background highlight color 
const backgroundHoverColor = borderHoverColor;

// text colors
const textColor = baseWhite;
const textHoverColor = "#FFFFFF";
const linkColor = "#2b94fe";
const linkColorHovered = "#50a4f9";

// Horde specific color extensions
const hordeDarkTheme: HordeThemeExtensions = {
    darkTheme: true,
    // Top nav background color
    topNavBackground: "#242729",
    // Breadcrumb area background color
    breadCrumbsBackground: "#1B1D1E",
    // Panel and modal content background color
    contentBackground: baseNeutral,
    // Neutral background color, used for site margins, and hinting such as in detail list rows
    neutralBackground: baseBlack,
    // divides sections
    dividerColor: "#25282A",
    // Scrollbar theme colors
    scrollbarThumbColor: "#606060",
    scrollbarTrackColor: "#3F3F3F"

}

// Fluent theme
export const darkTheme = createTheme({
    isInverted: true,
    palette: {
        // primary theme color, for primary actions, commandbar icons, etc
        themePrimary: basePrimary,
        // used on spinner
        themeLight: "#7F7F7F",

        neutralPrimary: neutralPrimary,

        // highlights in dropdowns, etc
        neutralLight: borderHoverColor,

        // a bit misleading, highlight on hover for combobox for example 
        neutralDark: "#FFFFFF",

        // disabled border on text field, empty area of slifer
        neutralLighter: neutralLight,

        // This is used by top nav ContextualMenu, chevron focus
        neutralQuaternaryAlt: baseNeutral,

        // used for down chevron
        neutralSecondary: textColor,

        // Disabled menu item
        neutralTertiary: neutralDisabled,
        neutralTertiaryAlt: "#444444",

        neutralQuaternary: "#121212",

        black: baseWhite,
        white: baseBlack,

        // debugging
        /*
        themeDarker: "#FF0000",
        themeDark: "#FF0000",
        themeDarkAlt: "#FF0000",        
        themeSecondary: "#FF0000",
        themeTertiary: "#FF0000",    
        themeLighter: "#FF0000",
        themeLighterAlt: "#FF0000"
        */

    },
    semanticColors: {
        // body
        bodyBackground: baseBlack,
        bodyText: textColor,

        // buttons
        buttonText: textColor,
        buttonTextHovered: textHoverColor,
        buttonTextDisabled: "#949898",
        primaryButtonText: "#F8FBFE",
        primaryButtonTextPressed: textHoverColor,
        primaryButtonTextHovered: textHoverColor,
        primaryButtonTextDisabled: "#949898",

        // links
        actionLink: textColor,
        link: linkColor,
        linkHovered: linkColorHovered,

        // menus
        menuHeader: menuHeader,
        menuItemText: textColor,
        menuItemTextHovered: textHoverColor,
        menuItemBackgroundHovered: backgroundHoverColor,

        // list 
        listText: textColor,

        // input
        inputBackground: baseBlack,
        inputText: textColor,
        inputIcon: textColor,
        inputIconHovered: textColor,
        inputPlaceholderText: "#7F7F7F",
        inputBorder: borderColor,
        inputBorderHovered: borderHoverColor,
        inputFocusBorderAlt: basePrimary,
        smallInputBorder: borderHoverColor
    },
    components: {
        "DefaultButton": {
            styles: {
                root: {
                    border: "1px solid #121212",
                    backgroundColor: neutralLight
                },
                rootHovered: {                    
                    backgroundColor: backgroundHoverColor
            },
                splitButtonDivider: {
                    backgroundColor: "#121212",
                },
                splitButtonMenuButton: {
                    backgroundColor: neutralLight,
                    border: "1px solid #121212"
                }
            }
        },
        "PrimaryButton": {
            styles: {
                root: {                    
                    backgroundColor: basePrimary,
                },
                rootHovered: {                    
                    backgroundColor: "#0089E5"
                },
                splitButtonDivider: {
                    backgroundColor: "#F8FBFE",
                },
                splitButtonMenuButton: {
                    backgroundColor: basePrimary,
                    border: "1px solid #121212"
                }
            }
        },
        "ScrollablePane": {
            styles: {
                root: {
                    selectors: {
                        '.ms-DetailsHeader': { // this is for stickys
                            background: hordeDarkTheme.contentBackground,
                            borderBottomColor: "#363A3C"
                        }
                    }
                }
            }
        },
        "DetailsList": {
            styles: {
                root: {                    
                    selectors: {
                        '.ms-DetailsHeader': {
                            background: baseNeutral,
                            borderBottomColor: baseNeutral

                        },
                        '.ms-DetailsRow': {
                            color: textColor,
                            backgroundColor: "unset",
                            background: "#181A1B",
                            borderBottom: "1px solid #212425"
                        },
                        '.ms-DetailsRow:hover': {
                            color: textColor,
                            backgroundColor: "unset",
                            background: "#2F2F2F"
                        }
                    }
                }
            }
        },
        "Modal": {
            styles: {
                main: {
                    background: hordeDarkTheme.contentBackground,
                }
            }
        },
        "Stack": {
            styles: {
                root: {
                    selectors: {
                        'a': {
                            color: linkColor
                        },
                        'a:hover': {
                            color: linkColorHovered
                        },
                        "*::-webkit-scrollbar-track": {
                            background: hordeDarkTheme.scrollbarTrackColor
                        },
                        "*::-webkit-scrollbar-thumb": {
                            background: hordeDarkTheme.scrollbarThumbColor
                        },
                        "*::-webkit-scrollbar-corner": {
                            background: hordeDarkTheme.scrollbarTrackColor
                        }

                    }
                }
            }
        },
        "Checkbox": {
            styles: {
                checkmark: {
                    color: "#FFFFFF"
                },
                checkbox: {
                    borderColor: checkboxBorderColor,
                },
                root: {
                    selectors: {
                        "&:not(.is-checked):not(.is-disabled):hover .ms-Checkbox-checkbox": {
                            borderColor: `${checkboxBorderHoverColor} !important`,
                        },
                        "&.is-disabled .ms-Checkbox-checkbox": {
                            borderColor: `${checkboxBorderDisabledColor} !important`,
                        }
                    }
                }
            }
        },
        "Dropdown": {
            styles: {
                dropdown: {
                    ":focus::after": {
                        borderColor: `${borderHoverColor} !important`
                    }
                },
                title: {
                    ":hover": {
                        borderColor: `${borderHoverColor} !important`
                    }
                },
                dropdownItemHeader: {
                    color: menuHeader
                }
            }
        },
        "ComboBox": {
            styles: {
                root: {
                    backgroundColor: baseBlack,                    
                    selectors: {
                        "button": {
                            backgroundColor: neutralLight
                        },
                        "button:hover": {
                            backgroundColor: backgroundHoverColor
                        },
                        ".ms-Icon": {
                            color: textColor
                        }
                    }
                },
                input: {
                    borderBottomColor: borderHoverColor,
                    borderTopColor: borderHoverColor,
                    borderLeftColor: borderHoverColor,
                    borderRightColor: borderHoverColor
                }
            }
        },
        "SearchBox": {
            styles: {
                iconContainer: {
                    color: "#bfbfbf",
                    borderBottomColor: borderHoverColor,
                    borderTopColor: borderHoverColor,
                    borderLeftColor: borderHoverColor,
                    borderRightColor: borderHoverColor
                },
                field: {
                    backgroundColor: baseBlack,
                    borderBottomColor: borderHoverColor,
                    borderTopColor: borderHoverColor,
                    borderLeftColor: borderHoverColor,
                    borderRightColor: borderHoverColor
                }
            }
        },
        "Toggle": {
            styles: {
                root: {
                    selectors: {
                        '.ms-Toggle-thumb': {
                            background: "#0070e0 !important"
                        }
                    }
                },
                pill: {
                    backgroundColor: baseBlack
                }
            }
        },
        "DatePicker": {
            styles: {
                callout: {
                    selectors: {
                        'button': {
                            color: "#FFFFFF !important",
                        },
                        'button:disabled': {
                            color: `#5F5F5F !important`,
                        },
                    }
                }
            }
        },
        "TextField": {
            styles: {
                root: {
                    selectors: {
                        'input:disabled': {
                            backgroundColor: "#2f2f2f"

                        },
                    },


                }
            }
        },
        "CommandBar": {
            styles: {
                root: {
                    ".icon": {
                        color: "#FF0000 !important"
                    }
                }
            }
        },
        "TagPicker": {
            styles: {
                root: {
                    selectors: {
                        '.ms-BasePicker-input': {
                            background: `${baseBlack} !important`
                        }
                    }
                }
            }
        },
        "Separator": {
            styles: {
                root: {
                    "::before": {
                        backgroundColor: "#2F2F2F"
                    }
                }
            }
        }
    },
    defaultFontStyle: {
        fontFamily: 'Horde Open Sans Regular'
    },
    fonts: {
        small: { fontSize: 12 },
        medium: { fontSize: 13 },
        mediumPlus: { fontSize: 18 },
        large: { fontSize: 32 }
    }
}) as HordeTheme;

darkTheme.horde = hordeDarkTheme;