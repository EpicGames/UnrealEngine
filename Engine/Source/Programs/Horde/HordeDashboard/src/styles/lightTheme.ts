// Copyright Epic Games, Inc. All Rights Reserved.

import { createTheme } from '@fluentui/react';
import { HordeTheme } from './themeTypes';


const bodyText = "#2d3f5f";
const linkColor = "#0078D4";
const linkColorHovered = "#0078D4";


// Horde specific color extensions
const hordeLightTheme = {
    darkTheme: false,
    topNavBackground: "#FFFFFF",
    breadCrumbsBackground: "#F3F2F1",
    contentBackground: "#FFFFFF",
    neutralBackground: "#FAF9F9",
    scrollbarThumbColor: "#C1C1C1",
    scrollbarTrackColor: "#F1F1F1",
    dividerColor: "#E9E8E7"
}


export const lightTheme = createTheme({
    components: {
        "CommandBarButton": {
            styles: {
                label: {
                    color: bodyText
                }
            }
        },
        "DefaultButton": {
            styles: {
                splitButtonDivider: {
                    backgroundColor: bodyText
                },

                splitButtonMenuButton: {
                    backgroundColor: "#FFFFFF"
                }
            }
        },
        "PrimaryButton": {
            styles: {
                splitButtonDivider: {
                    backgroundColor: bodyText
                },

                splitButtonMenuButton: {
                    backgroundColor: "#0078D4"
                }
            }
        },
        "Stack": {
            styles: {
                root: {
                    selectors: {
                        'a': {
                            color: linkColor
                        },
                        'a:hover': {
                            color: linkColorHovered
                        },
                        "*::-webkit-scrollbar-track": {
                            background: hordeLightTheme.scrollbarTrackColor
                        },
                        "*::-webkit-scrollbar-thumb": {
                            background: hordeLightTheme.scrollbarThumbColor
                        }
                    }
                }
            }
        }
    },
    palette: {
        themePrimary: '#0078d4',
        themeLighterAlt: '#eff6fc',
        themeLighter: '#deecf9',
        themeLight: '#c7e0f4',
        themeTertiary: '#71afe5',
        themeSecondary: '#2b88d8',
        themeDarkAlt: '#106ebe',
        themeDark: '#005a9e',
        themeDarker: '#004578',
        neutralLighterAlt: '#faf9f8',
        neutralLighter: '#f3f2f1',
        neutralLight: '#edebe9',
        neutralQuaternaryAlt: '#e1dfdd',
        neutralQuaternary: '#d0d0d0',
        neutralTertiaryAlt: '#c8c6c4',
        neutralTertiary: '#a19f9d',
        neutralSecondary: '#605e5c',
        neutralPrimaryAlt: '#3b3a39',
        neutralPrimary: '#323130',
        neutralDark: '#201f1e',
        black: '#000000',
        white: '#ffffff',
    },
    semanticColors: {
        bodyBackground: "#FFFFFF",
        bodyText: bodyText,
        buttonText: "#000000",
        buttonTextDisabled: "#949898",
        actionLink: "#000000",
        actionLinkHovered: "#000000",
        link: linkColor,
        linkHovered: linkColorHovered,
        listText: "#000000",
        listBackground: "#FF0000",
        listItemBackgroundHovered: "#f3f2f1",
        primaryButtonText: "#FFFFFF",
        primaryButtonTextDisabled: "#949898"


    },
    defaultFontStyle: {
        fontFamily: 'Horde Open Sans Regular'
    },
    fonts: {
        small: { fontSize: 12 },
        medium: { fontSize: 13 },
        mediumPlus: { fontSize: 18 },
        large: { fontSize: 32 }
    }
}) as HordeTheme;

lightTheme.horde = hordeLightTheme;