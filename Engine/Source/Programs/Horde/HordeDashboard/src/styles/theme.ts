// Copyright Epic Games, Inc. All Rights Reserved.

import dashboard from "../backend/Dashboard";
import { darkTheme } from "./darkTheme";
import { lightTheme } from "./lightTheme";

/** Please note, backend must be initialized before calling this method, so don't use in module scope */
export function getHordeTheme() {  
    
    if (!dashboard.available) {
        throw new Error("getHordeStyling must not be called before dashboard is available");
     }
      
    return dashboard.darktheme ? darkTheme : lightTheme;
}
