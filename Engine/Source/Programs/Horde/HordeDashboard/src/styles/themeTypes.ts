// Copyright Epic Games, Inc. All Rights Reserved.

import { Theme } from "@fluentui/react";

export type HordeThemeExtensions = {

    // Whether using the dark theme
    darkTheme: boolean;

    // background color of the top navigation bar
    topNavBackground: string;

    // background color of the top bread crumbs area
    breadCrumbsBackground: string;

    // background of neutral area
    neutralBackground: string;

    // background of neutral area
    dividerColor: string;

    // background of content areas
    contentBackground: string;

    // background color for scrollbar track
    scrollbarTrackColor: string;

    // background color for scrollbar thumb
    scrollbarThumbColor: string;

}

export interface HordeTheme extends Theme {
    horde: HordeThemeExtensions;
}
