// Copyright Epic Games, Inc. All Rights Reserved.

/// <reference types="vite/client" />
