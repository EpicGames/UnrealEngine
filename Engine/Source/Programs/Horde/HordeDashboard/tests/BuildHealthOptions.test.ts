import { parseBuildHealthQueryParams, upgradeBuildHealthQueryParams } from 'horde/components/buildhealth/BuildHealthOptions';
import { describe, it, expect } from 'vitest';
import { Location } from 'react-router-dom';
import LZString from "lz-string"

describe('parseBuildHealthQueryParams', () => {
	it('should correctly parse query parameters (schema v1)', () => {
		// Arrange
		const mockParams = new URLSearchParams();

		// Use LZString compression for encoded URL params
		const compressedProjects = LZString.compressToEncodedURIComponent('projA,projB');
		const compressedStreams = LZString.compressToEncodedURIComponent('stream1,stream2');
		const compressedTemplates = LZString.compressToEncodedURIComponent('stream1::template1,stream2::template2');
		const compressedSteps = LZString.compressToEncodedURIComponent('stream1::template1::step1,stream2::template2::step2');
		const compressedTimeSpan = LZString.compressToEncodedURIComponent('time_1_week');
		const compressedPreflight = 'true';

		// Do *NOT* use the param const strings; we want to catch those contract failures as well.
		mockParams.set('projects', compressedProjects);
		mockParams.set('streams', compressedStreams);
		mockParams.set('templates', compressedTemplates);
		mockParams.set('stepNames', compressedSteps);
		mockParams.set('lastTimeRange', compressedTimeSpan);
		mockParams.set('includePreflight', compressedPreflight);
		mockParams.set('schema', '1');

		// Act
		const result = parseBuildHealthQueryParams(new URLSearchParams(mockParams.toString()));

		// Assert
		expect(result.buildHealthQueryParams.projectIds).toEqual(['projA', 'projB']);
		expect(result.buildHealthQueryParams.streamIds).toEqual(['stream1', 'stream2']);
		expect(result.buildHealthQueryParams.templateIds).toEqual(['stream1::template1', 'stream2::template2']);
		expect(result.buildHealthQueryParams.stepNames).toEqual(['stream1::template1::step1', 'stream2::template2::step2']);
		expect(result.buildHealthQueryParams.timeSpanKey).toBe('time_1_week');
		expect(result.buildHealthQueryParams.includePreflight).toBe(true);
		expect(result.buildHealthQueryParams.querySchemaVersion).toBe(1);
	});
});

describe('upgradeBuildHealthQueryParams', () => {
	it('should correctly upgrade query parameters (schema v1 to v2)', () => {
		// Arrange
		const mockParams = new URLSearchParams();

		// Use LZString compression for encoded URL params
		const compressedProjects = LZString.compressToEncodedURIComponent('projA,projB');
		const compressedStreams = LZString.compressToEncodedURIComponent('stream1,stream2');
		const compressedTemplates = LZString.compressToEncodedURIComponent('stream1::template1,stream2::template2,stream2::template3');
		const compressedSteps = LZString.compressToEncodedURIComponent('stream1::template1::step1,stream2::template2::step2');
		const compressedTimeSpan = LZString.compressToEncodedURIComponent('time_1_week');
		const compressedPreflight = 'true';

		// Do *NOT* use the param const strings; we want to catch those contract failures as well.
		mockParams.set('projects', compressedProjects);
		mockParams.set('streams', compressedStreams);
		mockParams.set('templates', compressedTemplates);
		mockParams.set('stepNames', compressedSteps);
		mockParams.set('lastTimeRange', compressedTimeSpan);
		mockParams.set('includePreflight', compressedPreflight);
		mockParams.set('schema', '1');

		// Act
		const result = parseBuildHealthQueryParams(new URLSearchParams(mockParams.toString()));
		const upgradedParams = upgradeBuildHealthQueryParams(result.buildHealthQueryParams, 2);

		// Assert
		expect(upgradedParams.projectIds).toEqual(['projA', 'projB']);
		expect(upgradedParams.streamIds).toEqual(['stream1', 'stream2']);
		expect(upgradedParams.templateIds).toEqual(['stream1::template1', 'stream2::template2', 'stream2::template3']);
		expect(upgradedParams.stepNames).toEqual(['stream1::template1::step1', 'stream2::template2::step2', 'stream2::template3::stepNames_AllKnownValues']);
		expect(upgradedParams.timeSpanKey).toBe('time_1_week');
		expect(upgradedParams.includePreflight).toBe(true);
		expect(upgradedParams.querySchemaVersion).toBe(1);
	});
});