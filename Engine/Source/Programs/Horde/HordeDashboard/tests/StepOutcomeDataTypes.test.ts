import './helpers/FoundationMocks.test.setup';
import { describe, it, expect} from 'vitest';
import { encodeStepName } from '../src/components/buildhealth/stepoutcome/StepOutcomeDataTypes';

// Mock object type
type StepOutcomeTableEntry = {
    streamId: string;
    name: string;
};

describe('encodeStepName', () => {
    it('Should encode the step name correctly', () => {
        const mockEntry : StepOutcomeTableEntry = {
            streamId: 'STREAM1',
            name: 'StepA',
        };

        const result = encodeStepName(mockEntry as any);

        expect(result).toBe('stream1::stepa'); // streamId + KEY_SEPARATOR + stepResponse.name, lowercased
    });
});