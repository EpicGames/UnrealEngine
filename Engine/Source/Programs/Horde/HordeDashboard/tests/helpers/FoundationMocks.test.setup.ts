import { vi } from 'vitest';

// Mock UserInactivity BEFORE anything imports it
vi.mock('../src/backend/UserInactivity', () => {
    return {
        UserInactivity: class {
            start = vi.fn();
            stop = vi.fn();
            eventHandler = vi.fn();
        },
    };
});

// Mock Dashboard BEFORE importing anything that uses it
vi.mock('../src/backend/Dashboard', () => {
    return {
        default: {
            get darktheme() { return false; },
            set darktheme(value: boolean) { },
            getStatusColors: vi.fn(() => new Map([
                ["Success", "#00FF00"],
                ["Warning", "#FFFF00"],
                ["Failure", "#FF0000"],
                ["Skipped", "#AAAAAA"],
                ["Running", "#0000FF"],
            ])),
        },
        StatusColor: {
            Success: "Success",
            Warning: "Warning",
            Failure: "Failure",
            Skipped: "Skipped",
            Running: "Running",
        },
    };
});

// Mock global browser APIs immediately
globalThis.window = {
    addEventListener: vi.fn(),
    removeEventListener: vi.fn(),
} as any;

globalThis.localStorage = {
    getItem: vi.fn(),
    setItem: vi.fn(),
    removeItem: vi.fn(),
    clear: vi.fn(),
} as any;