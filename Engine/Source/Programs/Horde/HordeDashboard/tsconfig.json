{
  "compilerOptions": {
    "target": "ES2020",
    "useDefineForClassFields": true,
    "lib": [
      "dom",
      "dom.iterable",
      "esnext"
    ],
    "module": "ESNext",
    "skipLibCheck": true,
    "experimentalDecorators": true,

    /* Bundler mode */
    "moduleResolution": "bundler",
    "allowImportingTsExtensions": true,
    "resolveJsonModule": true,
    "isolatedModules": true,
    "noEmit": true,
    "jsx": "react-jsx",
    
    "paths": {
      "horde/*" : ["./src/*"],
      "hordePlugins": ["./plugins"],
      "hordePlugins/*": ["./plugins/*"]
    },

    "preserveSymlinks": true,

    /* Linting */
    "strict": false,
    "noFallthroughCasesInSwitch": true,
    "strictNullChecks": true
    /*
    "noUnusedLocals": true,
    "noUnusedParameters": true,    
    */
  },
  "include": ["src", "plugins", "tests"],
  "references": [{ "path": "./tsconfig.node.json" }]
}
