// Copyright Epic Games, Inc. All Rights Reserved.

declare module 'react-flame-graph'