// Copyright Epic Games, Inc. All Rights Reserved.

declare module 'react-text-marquee'