import { defineConfig } from 'vite'
import tsconfigPaths from 'vite-tsconfig-paths'
import react from '@vitejs/plugin-react'

const proxyTarget = "http://127.0.0.1:13340"
const debug = false;

const cacheBreak = Date.now();

// https://vitejs.dev/config/
export default defineConfig({
  plugins: [react({
    babel: {
      parserOpts: {
        plugins: ['decorators-legacy'],
      },
    },
  }),
  tsconfigPaths()],
  build: {
    chunkSizeWarningLimit: 8192,
    rollupOptions: {
      output: {
        entryFileNames: `[name].${cacheBreak}.js`,
        chunkFileNames: `[name].${cacheBreak}.js`,
        assetFileNames: `[name].${cacheBreak}.[ext]`
      }
    }
  },
  server: {
    proxy: {
      '/api': {
        target: proxyTarget,
        changeOrigin: true
      },
    }
  },
  resolve: {
    preserveSymlinks: true
  },
  preview: {
    proxy: {
      '/api': {
        target: proxyTarget,
        changeOrigin: true,
        secure: false,
        ws: true,
        configure: (proxy, _options) => {
          proxy.on('error', (err, _req, _res) => {
            if (debug)
              console.log('proxy error', err);
          });
          proxy.on('proxyReq', (proxyReq, req, _res) => {
            if (debug)
              console.log('Sending Request to the Target:', req.method, req.url);
          });
          proxy.on('proxyRes', (proxyRes, req, _res) => {
            if (debug)
              console.log('Received Response from the Target:', proxyRes.statusCode, req.url);
          });
        },
      }
    }
  }
})
