// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Reflection;
using System.Security.Claims;
using System.Text.Json.Serialization;
using EpicGames.Horde.Acls;
using HordeServer.Utilities;

#pragma warning disable CA2227 // Change 'X' to be read-only by removing the property setter

namespace HordeServer.Acls
{
	/// <summary>
	/// Parameters to update an ACL
	/// </summary>
	public class AclConfig
	{
		/// <summary>
		/// The parent scope object
		/// </summary>
		[JsonIgnore]
		public AclConfig? Parent { get; set; }

		/// <summary>
		/// Name of this scope
		/// </summary>
		[JsonIgnore]
		public AclScopeName ScopeName { get; set; }

		/// <summary>
		/// Legacy aliases for this scope
		/// </summary>
		[JsonIgnore]
		public AclScopeName[]? LegacyScopeNames { get; set; }
		
		/// <summary>
		/// ACL actions associated with this config (for debugging purposes)
		/// </summary>
		[JsonIgnore]
		public AclAction[]? Actions { get; set; }

		/// <summary>
		/// ACLs which are parented to this
		/// </summary>
		[JsonIgnore]
		public List<AclConfig>? Children { get; set; }

		/// <summary>
		/// Entries to replace the existing ACL
		/// </summary>
		public List<AclEntryConfig>? Entries { get; set; }

		/// <summary>
		/// Defines profiles which allow grouping sets of actions into named collections
		/// </summary>
		public List<AclProfileConfig>? Profiles { get; set; }

		/// <summary>
		/// Whether to inherit permissions from the parent ACL
		/// </summary>
		public bool? Inherit { get; set; }

		/// <summary>
		/// List of exceptions to the inherited setting
		/// </summary>
		public List<AclAction>? Exceptions { get; set; }

		IReadOnlyDictionary<AclProfileId, AclProfileConfig> _profileLookup = null!;

		/// <summary>
		/// Tests whether a user is authorized to perform the given actions
		/// </summary>
		/// <param name="action">Action that is being performed. This should be a single flag.</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True/false if the action is allowed or denied, null if there is no specific setting for this user</returns>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> TryAuthorize(action, user) ?? false;

		/// <summary>
		/// Tests whether a user is authorized to perform the given actions
		/// </summary>
		/// <param name="action">Action that is being performed. This should be a single flag.</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True/false if the action is allowed or denied, null if there is no specific setting for this user</returns>
		public bool? TryAuthorize(AclAction action, ClaimsPrincipal user)
		{
			if (user.HasAdminClaim())
			{
				return true;
			}

			for (AclConfig? next = this; next != null; next = next.Parent)
			{
				bool? result = next.AuthorizeSingleScope(action, user);
				if (result.HasValue)
				{
					return result.Value;
				}
			}

			return null;
		}

		/// <summary>
		/// Tests whether a user is authorized to perform the given actions in this specific scope
		/// </summary>
		/// <param name="action">Action that is being performed. This should be a single flag.</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True/false if the action is allowed or denied, null if there is no specific setting for this user</returns>
		public bool? AuthorizeSingleScope(AclAction action, ClaimsPrincipal user)
		{
			// Check if there's a specific entry for this action
			if (Entries != null)
			{
				foreach (AclEntryConfig entry in Entries)
				{
					if (user == null)
					{
						throw new NullReferenceException("User is null");
					}
					if (entry == null)
					{
						throw new NullReferenceException("Entry is null");
					}
					if (entry.ComputedActions == null)
					{
						throw new NullReferenceException("ComputedActions is null");
					}
					if (entry.Claim.Type == null)
					{
						throw new NullReferenceException("Claim.Type is null");
					}
					if (entry.Claim.Value == null)
					{
						throw new NullReferenceException("Claim.Value is null");
					}

					if (entry.ComputedActions.Contains(action) && user.HasClaim(entry.Claim.Type, entry.Claim.Value))
					{
						return true;
					}
				}
			}

			// Otherwise check if we're prevented from inheriting permissions
			if (Inherit ?? true)
			{
				if (Exceptions != null && Exceptions.Contains(action))
				{
					return false;
				}
			}
			else
			{
				if (Exceptions == null || !Exceptions.Contains(action))
				{
					return false;
				}
			}

			// Otherwise allow to propagate up the hierarchy
			return null;
		}

		/// <summary>
		/// Called after the config file has been read
		/// </summary>
		public void PostLoad(AclConfig parentScope, string scopeNameSuffix, AclAction[] actions)
		{
			PostLoad(parentScope, parentScope.ScopeName.Append(scopeNameSuffix), actions);
		}

		/// <summary>
		/// Called after the config file has been read
		/// </summary>
		public void PostLoad(AclConfig? parentScope, AclScopeName scopeName, AclAction[] actions)
		{
			Parent = parentScope;
			ScopeName = scopeName;
			Children = null;
			Actions = actions;

			if (parentScope == null)
			{
				_profileLookup = new Dictionary<AclProfileId, AclProfileConfig>();
			}
			else
			{
				parentScope.Children ??= new List<AclConfig>();
				parentScope.Children.Add(this);

				_profileLookup = parentScope._profileLookup;
			}

			if (Profiles != null && Profiles.Count > 0)
			{
				Dictionary<AclProfileId, AclProfileConfig> newProfileLookup = new Dictionary<AclProfileId, AclProfileConfig>(_profileLookup);

				foreach (AclProfileConfig profileConfig in Profiles)
				{
					newProfileLookup.Add(profileConfig.Id, profileConfig);
				}
				_profileLookup = newProfileLookup;

				HashSet<AclProfileId> visited = new HashSet<AclProfileId>();
				foreach (AclProfileConfig profileConfig in Profiles)
				{
					profileConfig.PostLoad(ScopeName, newProfileLookup, visited);
				}
			}

			if (Entries != null)
			{
				foreach (AclEntryConfig entryConfig in Entries)
				{
					entryConfig.PostLoad(ScopeName, _profileLookup);
				}
			}
		}

		/// <summary>
		/// Find all entitlements for a user
		/// </summary>
		public Dictionary<AclScopeName, HashSet<AclAction>> FindEntitlements(Predicate<AclClaimConfig> predicate)
		{
			Dictionary<AclScopeName, HashSet<AclAction>> scopeToActions = new Dictionary<AclScopeName, HashSet<AclAction>>();
			FindEntitlements(predicate, scopeToActions);
			return scopeToActions;
		}

		/// <summary>
		/// Find all entitlements for a user
		/// </summary>
		public void FindEntitlements(Predicate<AclClaimConfig> predicate, Dictionary<AclScopeName, HashSet<AclAction>> scopeToActions)
		{
			if (Entries != null)
			{
				foreach (AclEntryConfig entry in Entries)
				{
					if (predicate(entry.Claim))
					{
						HashSet<AclAction>? actions;
						if (!scopeToActions.TryGetValue(ScopeName, out actions))
						{
							actions = new HashSet<AclAction>();
							scopeToActions.Add(ScopeName, actions);
						}
						actions.UnionWith(entry.ComputedActions);
					}
				}
			}
			if (Children != null)
			{
				foreach (AclConfig childAclConfig in Children)
				{
					childAclConfig.FindEntitlements(predicate, scopeToActions);
				}
			}
		}
		
		/// <summary>
		/// Get all AclActions declared for the types specified
		/// </summary>
		/// <param name="type">Array of struct/class types</param>
		/// <returns>List of AclAction</returns>
		public static AclAction[] GetActions(Type[] type)
		{
			return type
				.SelectMany(x => x.GetProperties(BindingFlags.Static | BindingFlags.Public))
				.Where(x => x.PropertyType == typeof(AclAction))
				.Select(x => (AclAction)x.GetValue(null)!)
				.ToArray();
		}
	}

	/// <summary>
	/// Individual entry in an ACL
	/// </summary>
	public class AclEntryConfig
	{
		/// <summary>
		/// Name of the user or group
		/// </summary>
		[Required]
		public AclClaimConfig Claim { get; set; }

		/// <summary>
		/// Array of actions to allow
		/// </summary>
		public List<AclAction>? Actions { get; set; }

		/// <summary>
		/// List of profiles to grant
		/// </summary>
		public List<AclProfileId>? Profiles { get; set; }

		/// <summary>
		/// List of all actions inherited from all profiles
		/// </summary>
		[JsonIgnore]
		public HashSet<AclAction> ComputedActions { get; set; } = null!;

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		public AclEntryConfig()
		{
			Claim = new AclClaimConfig();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="claim">The claim this entry applies to</param>
		/// <param name="actions">List of allowed operations</param>
		/// <param name="profiles">Profiles to inherit</param>
		public AclEntryConfig(AclClaimConfig claim, IEnumerable<AclAction>? actions, IEnumerable<AclProfileId>? profiles = null)
		{
			Claim = claim;
			if (actions != null)
			{
				Actions = new List<AclAction>(actions);
			}
			if (profiles != null)
			{
				Profiles = new List<AclProfileId>(profiles);
			}
		}

		internal void PostLoad(AclScopeName scopeName, IReadOnlyDictionary<AclProfileId, AclProfileConfig> profileLookup)
		{
			HashSet<AclAction> computedActions = new HashSet<AclAction>();
			if (Actions != null)
			{
				computedActions.UnionWith(Actions);
			}
			if (Profiles != null)
			{
				computedActions.UnionWith(Profiles.SelectMany(profileId => GetActionsForProfile(scopeName, profileId, profileLookup)));
			}
			ComputedActions = computedActions;
		}

		static IEnumerable<AclAction> GetActionsForProfile(AclScopeName scopeName, AclProfileId profileId, IReadOnlyDictionary<AclProfileId, AclProfileConfig> profileLookup)
		{
			AclProfileConfig? profileConfig;
			if (!profileLookup.TryGetValue(profileId, out profileConfig))
			{
				throw new Exception($"Undefined profile '{profileId}' referenced from {scopeName}");
			}
			return profileConfig.ComputedActions;
		}
	}

	/// <summary>
	/// Configuration for an ACL profile. This defines a preset group of actions which can be given to a user via an ACL entry.
	/// </summary>
	public class AclProfileConfig
	{
		/// <summary>
		/// Identifier for this profile
		/// </summary>
		public AclProfileId Id { get; set; }

		/// <summary>
		/// Actions to include
		/// </summary>
		public List<AclAction>? Actions { get; set; }

		/// <summary>
		/// Actions to exclude from the inherited actions
		/// </summary>
		public List<AclAction>? ExcludeActions { get; set; }

		/// <summary>
		/// Other profiles to extend from
		/// </summary>
		public List<AclProfileId>? Extends { get; set; }

		/// <summary>
		/// Computed list of actions after considering base profiles etc... Fixed up by calling PostLoad().
		/// </summary>
		[JsonIgnore]
		internal HashSet<AclAction> ComputedActions { get; set; } = null!;

		internal void PostLoad(AclScopeName scopeName, Dictionary<AclProfileId, AclProfileConfig> profileLookup, HashSet<AclProfileId> visited)
		{
			if (ComputedActions == null)
			{
				if (!visited.Add(Id))
				{
					throw new Exception($"Recursive profile definition for '{Id}' in {scopeName}");
				}

				HashSet<AclAction> computedActions = new HashSet<AclAction>();
				if (Extends != null)
				{
					computedActions.UnionWith(Extends.SelectMany(profileId => GetActionsForProfile(scopeName, profileId, profileLookup, visited)));
				}
				if (Actions != null)
				{
					computedActions.UnionWith(Actions);
				}
				if (ExcludeActions != null)
				{
					computedActions.ExceptWith(ExcludeActions);
				}
				ComputedActions = computedActions;
			}
		}

		static IEnumerable<AclAction> GetActionsForProfile(AclScopeName scopeName, AclProfileId profileId, Dictionary<AclProfileId, AclProfileConfig> profileLookup, HashSet<AclProfileId> visited)
		{
			AclProfileConfig? profileConfig;
			if (!profileLookup.TryGetValue(profileId, out profileConfig))
			{
				throw new Exception($"Undefined profile '{profileId}'");
			}

			profileConfig.PostLoad(scopeName, profileLookup, visited);
			return profileConfig.ComputedActions!;
		}
	}

	/// <summary>
	/// New claim to create
	/// </summary>
	public class AclClaimConfig : IAclClaim
	{
		/// <summary>
		/// The claim type
		/// </summary>
		[Required]
		public string Type { get; set; } = null!;

		/// <summary>
		/// The claim value
		/// </summary>
		[Required]
		public string Value { get; set; } = null!;

		/// <summary>
		/// Constructor
		/// </summary>
		public AclClaimConfig()
		{
			Type = String.Empty;
			Value = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="claim">The claim object</param>
		public AclClaimConfig(Claim claim)
			: this(claim.Type, claim.Value)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The claim type</param>
		/// <param name="value">The claim value</param>
		public AclClaimConfig(string type, string value)
		{
			Type = type;
			Value = value;
		}

		/// <summary>
		/// Converts this object to a regular <see cref="Claim"/> object.
		/// </summary>
		public Claim ToClaim() => new Claim(Type, Value);
	}
}
