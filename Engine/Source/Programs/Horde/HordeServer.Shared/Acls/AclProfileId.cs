// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;

namespace HordeServer.Acls
{
	/// <summary>
	/// Identifier for a profile; a group of actions which can be assigned to a user as a whole.
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<AclProfileId, AclProfileIdConverter>))]
	[StringIdConverter(typeof(AclProfileIdConverter))]
	public record struct AclProfileId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public AclProfileId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc cref="StringId.IsEmpty"/>
		public bool IsEmpty => Id.IsEmpty;

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class AclProfileIdConverter : StringIdConverter<AclProfileId>
	{
		/// <inheritdoc/>
		public override AclProfileId FromStringId(StringId id) => new AclProfileId(id);

		/// <inheritdoc/>
		public override StringId ToStringId(AclProfileId value) => value.Id;
	}
}
