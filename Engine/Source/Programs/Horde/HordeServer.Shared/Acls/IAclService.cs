// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using EpicGames.Horde.Acls;

namespace HordeServer.Acls
{
	/// <summary>
	/// Functionality for manipulating ACLs
	/// </summary>
	public interface IAclService
	{
		/// <summary>
		/// Issues a bearer token with the given roles
		/// </summary>
		/// <param name="claims">List of claims to include</param>
		/// <param name="expiry">Time that the token expires</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>JWT security token with a claim for creating new agents</returns>
		ValueTask<string> IssueBearerTokenAsync(IEnumerable<Claim> claims, TimeSpan? expiry, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds an ACL scope by name
		/// </summary>
		/// <param name="scopeName">Name of the scope to auth against</param>
		/// <param name="scopeConfig">Configuration for the scope</param>
		bool TryGetAclScope(AclScopeName scopeName, [NotNullWhen(true)] out AclConfig? scopeConfig);
	}

	/// <summary>
	/// Extension methods for <see cref="IAclService"/>
	/// </summary>
	public static class AclServiceExtensions
	{
		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="aclService">Instance of the ACL service</param>
		/// <param name="scopeName">Name of the scope to auth against</param>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to validate</param>
		public static bool Authorize(this IAclService aclService, AclScopeName scopeName, AclAction action, ClaimsPrincipal user)
			=> aclService.TryGetAclScope(scopeName, out AclConfig? scopeConfig) && scopeConfig.Authorize(action, user);

		/// <summary>
		/// Issues a bearer token with the given roles
		/// </summary>
		/// <param name="aclService">Instance of the ACL service</param>
		/// <param name="claims">List of claims to include</param>
		/// <param name="expiry">Time that the token expires</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>JWT security token with a claim for creating new agents</returns>
		public static async ValueTask<string> IssueBearerTokenAsync(this IAclService aclService, IEnumerable<AclClaimConfig> claims, TimeSpan? expiry, CancellationToken cancellationToken = default)
			=> await aclService.IssueBearerTokenAsync(claims.Select(x => new Claim(x.Type, x.Value)), expiry, cancellationToken);
	}
}
