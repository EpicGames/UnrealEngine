// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Utilities;

namespace HordeServer.Acls
{
	/// <summary>
	/// Interface which can be implemented by classes that mutate the default ACL
	/// </summary>
	public interface IDefaultAclModifier
	{
		/// <summary>
		/// Modifies the default ACL
		/// </summary>
		void Apply(DefaultAclBuilder builder);
	}
}
