// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO.Pipelines;
using System.Text;
using Microsoft.Extensions.Logging;

namespace HordeServer.Auditing
{
	/// <summary>
	/// Message from an audit log
	/// </summary>
	public interface IAuditLogMessage
	{
		/// <summary>
		/// Timestamp for the event
		/// </summary>
		public DateTime TimeUtc { get; }

		/// <summary>
		/// Severity of the message
		/// </summary>
		public LogLevel Level { get; }

		/// <summary>
		/// The message payload. Should be an encoded JSON object, with format/properties fields.
		/// </summary>
		public string Data { get; }
	}

	/// <summary>
	/// Channel for a particular entity
	/// </summary>
	public interface IAuditLogChannel : ILogger
	{
		/// <summary>
		/// Finds messages matching certain criteria
		/// </summary>
		/// <param name="minTime"></param>
		/// <param name="maxTime"></param>
		/// <param name="index"></param>
		/// <param name="count"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		IAsyncEnumerable<IAuditLogMessage> FindAsync(DateTime? minTime = null, DateTime? maxTime = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes messages between a given time range for a particular object
		/// </summary>
		/// <param name="minTime">Minimum time to remove</param>
		/// <param name="maxTime">Maximum time to remove</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task<long> DeleteAsync(DateTime? minTime = null, DateTime? maxTime = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Flush all writes to this log
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task FlushAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Channel for a particular entity
	/// </summary>
	public interface IAuditLogChannel<TSubject> : IAuditLogChannel
	{
		/// <summary>
		/// Identifier for the subject of this channel
		/// </summary>
		TSubject Subject { get; }
	}

	/// <summary>
	/// Message in an audit log
	/// </summary>
	/// <typeparam name="TTargetId">Type of entity that the log is for</typeparam>
	public interface IAuditLogMessage<TTargetId> : IAuditLogMessage
	{
		/// <summary>
		/// Unique id for the entity
		/// </summary>
		public TTargetId Subject { get; }
	}

	/// <summary>
	/// Interface for a collection of log messages for a particular document type
	/// </summary>
	public interface IAuditLog<TSubject>
	{
		/// <summary>
		/// Get the channel for a particular subject
		/// </summary>
		/// <param name="subject"></param>
		/// <returns></returns>
		IAuditLogChannel<TSubject> this[TSubject subject] { get; }
	}

	/// <summary>
	/// Factory for instantiating audit log instances
	/// </summary>
	/// <typeparam name="TSubject"></typeparam>
	public interface IAuditLogFactory<TSubject>
	{
		/// <summary>
		/// Create a new audit log instance, with the given database name
		/// </summary>
		/// <param name="collectionName"></param>
		/// <param name="subjectProperty"></param>
		/// <returns></returns>
		IAuditLog<TSubject> Create(string collectionName, string subjectProperty);
	}

	/// <summary>
	/// Extension methods for audit log channels
	/// </summary>
	public static class AuditLogExtensions
	{
		/// <summary>
		/// Retrieve historical information about a specific agent
		/// </summary>
		/// <param name="channel">Channel to query</param>
		/// <param name="bodyWriter">Writer for Json data</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested agent</returns>
		public static async Task FindAsync(this IAuditLogChannel channel, PipeWriter bodyWriter, DateTime? minTime = null, DateTime? maxTime = null, int index = 0, int count = 50, CancellationToken cancellationToken = default)
		{
			string prefix = "{\n\t\"entries\":\n\t[";
			await bodyWriter.WriteAsync(Encoding.UTF8.GetBytes(prefix), cancellationToken);

			string separator = "";
			await foreach (IAuditLogMessage message in channel.FindAsync(minTime, maxTime, index, count, cancellationToken))
			{
				string line = $"{separator}\n\t\t{message.Data}";
				await bodyWriter.WriteAsync(Encoding.UTF8.GetBytes(line), cancellationToken);
				separator = ",";
			}

			string suffix = "\n\t]\n}";
			await bodyWriter.WriteAsync(Encoding.UTF8.GetBytes(suffix), cancellationToken);
		}
	}
}
