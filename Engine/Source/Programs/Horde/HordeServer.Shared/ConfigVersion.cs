// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer
{
#pragma warning disable CA1027 // Mark enums with FlagsAttribute
#pragma warning disable CA1069 // Enum member 'Latest' has same value as ...
	/// <summary>
	/// Global version number for running the server. As new features are introduced that require data migrations, this version number indicates the backwards compatibility functionality that must be enabled.
	/// When adding a new version here, also add a message to ConfigService.CreateSnapshotAsync describing the steps that need to be taken to upgrade the deployment.
	/// </summary>
	public enum ConfigVersion
	{
		/// <summary>
		/// Not specified
		/// </summary>
		None,

		/// <summary>
		/// Initial version number
		/// </summary>
		Initial,

		/// <summary>
		/// All pools are managed through config files. Database pool reads are disabled.
		/// </summary>
		PoolsInConfigFiles,

		/// <summary>
		/// One after the last defined version number
		/// </summary>
		LatestPlusOne,

		/// <summary>
		/// Latest version number
		/// </summary>
		Latest = LatestPlusOne - 1,
	}
#pragma warning restore CA1069
#pragma warning restore CA1027
}
