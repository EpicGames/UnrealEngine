// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace HordeServer.Configuration
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Context for reading a tree of config files
	/// </summary>
	[DebuggerDisplay("{CurrentFile}")]
	public class ConfigContext
	{
		/// <summary>
		/// Options for serializing config files
		/// </summary>
		public JsonSerializerOptions JsonOptions { get; }

		/// <summary>
		/// Stack of included files
		/// </summary>
		public Stack<IConfigFile> IncludeStack { get; } = new Stack<IConfigFile>();

		/// <summary>
		/// Stack of properties
		/// </summary>
		public Stack<string> ScopeStack { get; } = new Stack<string>();

		/// <summary>
		/// Map of property path to the file declaring a value for it
		/// </summary>
		public Dictionary<string, Uri> PropertyPathToFile { get; } = new Dictionary<string, Uri>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Sources to read config files from
		/// </summary>
		public IReadOnlyDictionary<string, IConfigSource> Sources { get; }

		/// <summary>
		/// Map of macro name to value 
		/// </summary>
		public List<Dictionary<string, string>> MacroScopes { get; } = new List<Dictionary<string, string>>();

		/// <summary>
		/// Tracks files read as part of the configuration
		/// </summary>
		public Dictionary<Uri, IConfigFile> Files { get; } = new Dictionary<Uri, IConfigFile>();

		/// <summary>
		/// Tracks files already processed as includes within the current include root scope,
		/// preventing diamond dependencies from causing the same file to be merged multiple times.
		/// Maps each included URI to the URI of the file that first included it.
		/// Each [ConfigIncludeRoot] entered from property/array traversal saves and restores this
		/// so sibling roots (e.g. two StreamConfigs in the same project) get independent scopes.
		/// Roots entered from include processing share the parent's scope for diamond detection.
		/// </summary>
		public Dictionary<Uri, Uri?> ProcessedIncludes { get; set; } = new Dictionary<Uri, Uri?>();

		/// <summary>
		/// Depth counter for nested [ConfigIncludeRoot] processing.
		/// </summary>
		public int IncludeRootDepth { get; set; }

		/// <summary>
		/// When true, the current PreprocessAsync call is processing an included file (recursive
		/// from the include loop). [ConfigIncludeRoot] entries should share the parent's diamond
		/// dedup scope rather than creating a fresh one, so that diamonds within the same include
		/// tree are still detected.
		/// </summary>
		public bool InIncludeScope { get; set; }

		/// <summary>
		/// Warnings generated during config processing (e.g. diamond include dependencies)
		/// </summary>
		public List<string> Warnings { get; } = new List<string>();

		/// <summary>
		/// Logger for config messages
		/// </summary>
		public ILogger Logger { get; }

		/// <summary>
		/// Uri of the current file
		/// </summary>
		public Uri CurrentFile => (IncludeStack.Count > 0) ? IncludeStack.Peek().Uri : null!;

		/// <summary>
		/// Current property scope
		/// </summary>
		public string CurrentScope => ScopeStack.Peek();

		readonly Func<string, string?> _getMacroValue;

		/// <summary>
		/// Constructor
		/// </summary>
		public ConfigContext(JsonSerializerOptions jsonOptions, IReadOnlyDictionary<string, IConfigSource> sources, ILogger logger)
		{
			JsonOptions = jsonOptions;
			Sources = sources;
			ScopeStack.Push("$");
			Logger = logger;

			_getMacroValue = GetMacroValue;
		}

		/// <summary>
		/// Marks a property as defined in the current file
		/// </summary>
		/// <param name="name"></param>
		public void AddProperty(string name)
		{
			if (!TryAddProperty(name, out Uri? otherFile))
			{
				throw new ConfigException(this, $"Property {CurrentScope}.{name} was already defined in {otherFile}.");
			}
		}

		/// <summary>
		/// Marks a property as defined in the current file
		/// </summary>
		/// <param name="name">Name of the property within the current scope</param>
		/// <param name="otherFile">If the property is not added, the file that previously defined it</param>
		public bool TryAddProperty(string name, [NotNullWhen(false)] out Uri? otherFile)
		{
			string propertyPath = $"{CurrentScope}.{name}";

			Uri currentFile = CurrentFile;
			if (PropertyPathToFile.TryAdd(propertyPath, currentFile))
			{
				otherFile = null;
				return true;
			}
			else
			{
				otherFile = PropertyPathToFile[propertyPath];
				return false;
			}
		}

		/// <summary>
		/// Pushes a scope to the property stack
		/// </summary>
		/// <param name="name"></param>
		public void EnterScope(string name)
		{
			ScopeStack.Push($"{CurrentScope}.{name}");
		}

		/// <summary>
		/// Pops a scope from the property stack
		/// </summary>
		public void LeaveScope()
		{
			ScopeStack.Pop();
		}

		/// <summary>
		/// Expand macros in a string property
		/// </summary>
		public string ExpandMacros(string text) => StringUtils.ExpandProperties(text, _getMacroValue);

		/// <summary>
		/// Gets the value of a named macro
		/// </summary>
		string? GetMacroValue(string name)
		{
			for (int idx = MacroScopes.Count - 1; idx >= 0; idx--)
			{
				if (MacroScopes[idx].TryGetValue(name, out string? value))
				{
					return value;
				}
			}
			return null;
		}

		/// <summary>
		/// Reads the contents of a file using the appropriate source
		/// </summary>
		/// <param name="uri">Uri of the file to read. The scheme indicates the source to read from.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the file</returns>
		public async ValueTask<IConfigFile> ReadFileAsync(Uri uri, CancellationToken cancellationToken)
		{
			IConfigSource? source = Sources[uri.Scheme];
			if (source == null)
			{
				throw new ConfigException(this, $"Invalid/unknown scheme for config file {uri}");
			}

			IConfigFile? file;
			if (!Files.TryGetValue(uri, out file))
			{
				file = await source.GetAsync(uri, cancellationToken);
				Files.Add(uri, file);
			}

			return file;
		}

		/// <summary>
		/// Parses a config file as a json object
		/// </summary>
		/// <param name="file">File to parse</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The parsed file</returns>
		public async ValueTask<JsonObject> ParseFileAsync(IConfigFile file, CancellationToken cancellationToken = default)
		{
			ReadOnlyMemory<byte> data = await file.ReadAsync(cancellationToken);

			JsonObject? obj = JsonSerializer.Deserialize<JsonObject>(data.Span, JsonOptions);
			if (obj == null)
			{
				throw new ConfigException(this, $"Config file {file.Uri} contains a null object.");
			}

			return obj;
		}

		/// <summary>
		/// Reads an object from a particular URL
		/// </summary>
		/// <param name="uri">Location of the file to read</param>
		/// <param name="rootNode">Root node describing the preprocessor fields</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<JsonObject> PreprocessFileAsync(Uri uri, ObjectConfigNode rootNode, CancellationToken cancellationToken)
		{
			IConfigFile file = await ReadFileAsync(uri, cancellationToken);
			JsonObject obj = await ParseFileAsync(file, cancellationToken);

			IncludeStack.Push(file);
			obj = await rootNode.PreprocessAsync(obj, this, cancellationToken);
			IncludeStack.Pop();

			return obj;
		}

		/// <summary>
		/// Reads an object from a particular URL
		/// </summary>
		/// <typeparam name="T">Type of object to read</typeparam>
		/// <param name="uri">Location of the file to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<T> ReadAsync<T>(Uri uri, CancellationToken cancellationToken) where T : class, new()
		{
			ObjectConfigNode type = new ObjectConfigNode(typeof(T));
			JsonObject obj = await PreprocessFileAsync(uri, type, cancellationToken);
			return JsonSerializer.Deserialize<T>(obj, JsonOptions) ?? new T();
		}
	}
}
