// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Configuration
{
	/// <summary>
	/// Exception thrown when reading config files
	/// </summary>
	public sealed class ConfigException : Exception
	{
		readonly ConfigContext _context;

		/// <summary>
		/// Stack of properties
		/// </summary>
		public IEnumerable<string> ScopeStack => _context.ScopeStack;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="context">Current parse context for the error</param>
		/// <param name="message">Description of the error</param>
		/// <param name="innerException">Inner exception details</param>
		public ConfigException(ConfigContext context, string message, Exception? innerException = null)
			: base(message, innerException)
		{
			_context = context;
		}

		/// <summary>
		/// Gets the parser context when this exception was thrown. This is not exposed as a public property to avoid serializing the whole thing to Serilog.
		/// </summary>
		public ConfigContext GetContext() => _context;
	}
}
