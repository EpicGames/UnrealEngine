// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Directive to merge config data from another source
	/// </summary>
	public class ConfigInclude
	{
		/// <summary>
		/// Path to the config data to be included. May be relative to the including file's location.
		/// </summary>
		[Required, ConfigInclude, ConfigRelativePath]
		public string Path { get; set; } = null!;
	}
}
