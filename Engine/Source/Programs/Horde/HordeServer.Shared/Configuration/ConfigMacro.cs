// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Configuration
{
	/// <summary>
	/// Declares a config macro
	/// </summary>
	public class ConfigMacro
	{
		/// <summary>
		/// Name of the macro property
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Value for the macro property
		/// </summary>
		public string Value { get; set; } = String.Empty;
	}
}
