// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Nodes;
using System.Text.Json.Serialization;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Attribute used to mark <see cref="Uri"/> properties that include other config files
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class ConfigIncludeAttribute : Attribute
	{
	}

	/// <summary>
	/// Specifies that a class is the root for including other files
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class ConfigIncludeRootAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute used to mark <see cref="Uri"/> properties that are relative to their containing file
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class ConfigRelativePathAttribute : Attribute
	{
	}

	/// <summary>
	/// Attribute used to mark <see cref="Uri"/> properties that are relative to their containing file
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class ConfigMacroScopeAttribute : Attribute
	{
	}

	/// <summary>
	/// Node in the preprocessor parse tree
	/// </summary>
	public abstract class ConfigNode
	{
		/// <summary>
		/// Default options for new json node objects
		/// </summary>
		protected static JsonNodeOptions DefaultJsonNodeOptions { get; } = new JsonNodeOptions { PropertyNameCaseInsensitive = true };

		/// <summary>
		/// Parses macro definitions from this object
		/// </summary>
		/// <param name="node">Node to parse macros from</param>
		/// <param name="context">Context for the preprocessor</param>
		/// <param name="macros">Macros parsed from the boject</param>
		public abstract void ParseMacros(JsonNode? node, ConfigContext context, Dictionary<string, string> macros);

		/// <summary>
		/// Preprocesses a json document
		/// </summary>
		/// <param name="node">Node to preprocess</param>
		/// <param name="context">Context for the preprocessor</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Preprocessed node</returns>
		public Task<JsonNode?> PreprocessAsync(JsonNode? node, ConfigContext context, CancellationToken cancellationToken)
			=> PreprocessAsync(node, null, context, cancellationToken);

		/// <summary>
		/// Preprocesses a json document (possibly merging with an existing property)
		/// </summary>
		/// <param name="node">Node to preprocess</param>
		/// <param name="existingNode">Optional existing node to merge with. Can be modified.</param>
		/// <param name="context">Context for the preprocessor</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The preprocessed node. May be existingNode if changes are merged with it.</returns>
		public abstract Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken);

		/// <summary>
		/// Traverse the property tree starting with 'node' and process any include directives, merging the results into the given target object.
		/// </summary>
		/// <param name="node"></param>
		/// <param name="includes">Receives the included files</param>
		/// <param name="context"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public abstract Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken);

		/// <summary>
		/// Combine a relative path with a base URI to produce a new URI
		/// </summary>
		/// <param name="baseUri">Base uri to rebase relative to</param>
		/// <param name="path">Relative path</param>
		/// <returns>Absolute URI</returns>
		public static Uri CombinePaths(Uri baseUri, string path)
		{
			if (path.StartsWith("//", StringComparison.Ordinal))
			{
				const string PerforceScheme = "perforce";
				if (baseUri.Scheme == PerforceScheme)
				{
					return new Uri($"{PerforceScheme}://{baseUri.Host}{path}");
				}
				else
				{
					return new Uri($"{PerforceScheme}://default{path}");
				}
			}
			return new Uri(baseUri, path);
		}

		/// <summary>
		/// Helper method to expand all macros within a node without performing any other processing on it
		/// </summary>
		protected static JsonNode? ExpandMacros(JsonNode? node, ConfigContext context)
		{
			if (node == null)
			{
				return node;
			}
			else if (node is JsonObject obj)
			{
				JsonObject result = new JsonObject(DefaultJsonNodeOptions);
				foreach ((string propertyName, JsonNode? propertyNode) in obj)
				{
					result[propertyName] = ExpandMacros(propertyNode, context);
				}
				return result;
			}
			else if (node is JsonArray arr)
			{
				JsonArray result = new JsonArray();
				foreach (JsonNode? elementNode in arr)
				{
					result.Add(ExpandMacros(elementNode, context));
				}
				return result;
			}
			else if (node is JsonValue val && val.GetValueKind() == JsonValueKind.String)
			{
				string strValue = ((string?)val)!;
				string expandedStrValue = context.ExpandMacros(strValue);
				return JsonValue.Create(expandedStrValue);
			}
			else
			{
				return node.DeepClone();
			}
		}
	}

	/// <summary>
	/// Implementation of <see cref="ConfigNode"/> for scalar types
	/// </summary>
	public class ScalarConfigNode : ConfigNode
	{
		/// <summary>
		/// Whether this node represents an include directive
		/// </summary>
		public bool Include { get; set; }

		/// <summary>
		/// Specifies that this node is a string which should be made into an absolute path using the path of the current file.
		/// </summary>
		public bool RelativeToContainingFile { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ScalarConfigNode(bool include = false, bool relativePath = false)
		{
			Include = include;
			RelativeToContainingFile = relativePath;
		}

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? jsonNode, ConfigContext context, Dictionary<string, string> macros)
		{ }

		/// <inheritdoc/>
		public override Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
		{
			JsonNode? result;
			if (node == null)
			{
				result = null;
			}
			else if (RelativeToContainingFile && node is JsonValue value && value.GetValueKind() == JsonValueKind.String)
			{
				result = JsonValue.Create(CombinePaths(context.CurrentFile, value.ToString()).AbsoluteUri);
			}
			else
			{
				result = ExpandMacros(node, context);
			}

			return Task.FromResult(result);
		}

		/// <inheritdoc/>
		public override async Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
		{
			if (Include)
			{
				string? path = (string?)node;

				Uri uri = CombinePaths(context.CurrentFile, context.ExpandMacros(path!));

				IConfigFile file = await context.ReadFileAsync(uri, cancellationToken);
				includes.Add(file);
			}
		}
	}

	/// <summary>
	/// Property containing a binary resource
	/// </summary>
	public class ResourceConfigNode : ConfigNode
	{
		/// <inheritdoc/>
		public override Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? node, ConfigContext context, Dictionary<string, string> macros)
		{ }

		/// <inheritdoc/>
		public override async Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
		{
			Uri uri = CombinePaths(context.CurrentFile, JsonSerializer.Deserialize<string>(node, context.JsonOptions) ?? String.Empty);
			IConfigFile file = await context.ReadFileAsync(uri, cancellationToken);

			ConfigResource resource = new ConfigResource();
			resource.Path = uri.AbsoluteUri;
			resource.Data = await file.ReadAsync(cancellationToken);

			return JsonSerializer.SerializeToNode(resource, context.JsonOptions);
		}
	}

	/// <summary>
	/// Array of Json values
	/// </summary>
	public class ArrayConfigNode : ConfigNode
	{
		/// <summary>
		/// Type of each element of the array
		/// </summary>
		public ConfigNode ElementType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ArrayConfigNode(ConfigNode elementType)
		{
			ElementType = elementType;
		}

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? node, ConfigContext context, Dictionary<string, string> macros)
		{
			if (node is JsonArray arrayNode)
			{
				foreach (JsonNode? elementNode in arrayNode)
				{
					ElementType.ParseMacros(elementNode, context, macros);
				}
			}
		}

		/// <inheritdoc/>
		public override async Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
		{
			JsonArray targetArray = ((JsonArray?)existingNode) ?? new JsonArray();
			foreach (JsonNode? element in (JsonArray)node!)
			{
				context.EnterScope($"[{targetArray.Count}]");

				JsonNode? elementValue = await ElementType.PreprocessAsync(element, context, cancellationToken);
				targetArray.Add(elementValue);

				context.LeaveScope();
			}
			return targetArray;
		}

		/// <inheritdoc/>
		public override async Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
		{
			if (node is JsonArray arrayNode)
			{
				foreach (JsonObject? element in arrayNode)
				{
					await ElementType.ParseIncludesAsync(element, includes, context, cancellationToken);
				}
			}
		}
	}

	/// <summary>
	/// Arbitary mapping of string values to keys
	/// </summary>
	public class DictionaryConfigNode : ConfigNode
	{
		/// <summary>
		/// Type of values in the dictionary
		/// </summary>
		public ConfigNode ValueType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public DictionaryConfigNode(ConfigNode valueType)
			=> ValueType = valueType;

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? jsonNode, ConfigContext context, Dictionary<string, string> macros)
		{ }

		/// <inheritdoc/>
		public override async Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
		{
			JsonObject? targetObject = ((JsonObject?)existingNode) ?? new JsonObject(DefaultJsonNodeOptions);
			foreach ((string key, JsonNode? element) in (JsonObject)node!)
			{
				context.EnterScope($"[{key}]");

				JsonNode? elementValue = await ValueType.PreprocessAsync(element, context, cancellationToken);
				targetObject[key] = elementValue;

				context.LeaveScope();
			}
			return targetObject;
		}

		/// <inheritdoc/>
		public override async Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
		{
			if (node is JsonObject obj && ValueType is ObjectConfigNode classElementType)
			{
				foreach ((_, JsonNode? value) in obj)
				{
					await classElementType.ParseIncludesAsync(value, includes, context, cancellationToken);
				}
			}
		}
	}

	/// <summary>
	/// Handles macro objects
	/// </summary>
	public class MacroConfigNode : ConfigNode
	{
		/// <inheritdoc/>
		public override Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? node, ConfigContext context, Dictionary<string, string> macros)
		{
			ConfigMacro? macro = JsonSerializer.Deserialize<ConfigMacro>(node, context.JsonOptions);
			if (macro != null)
			{
				macros.Add(macro.Name, macro.Value);
			}
		}

		/// <inheritdoc/>
		public override Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
			=> Task.FromResult<JsonNode?>(node?.DeepClone());
	}

	/// <summary>
	/// Implementation of <see cref="ConfigNode"/> to handle class types
	/// </summary>
	public class ObjectConfigNode : ConfigNode
	{
		/// <summary>
		/// Whether this object should be treated as a root for nested include directies
		/// </summary>
		public bool IncludeRoot { get; set; }

		/// <summary>
		/// Declares a new macro scope
		/// </summary>
		public bool MacroScope { get; set; }

		/// <summary>
		/// Properties within this object
		/// </summary>
		public Dictionary<string, ConfigNode> Properties { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectConfigNode(bool isIncludeRoot, bool isMacroScope)
			: this(isIncludeRoot, isMacroScope, Array.Empty<KeyValuePair<string, ConfigNode>>())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectConfigNode(bool isIncludeRoot, bool isMacroScope, IEnumerable<KeyValuePair<string, ConfigNode>> properties)
		{
			IncludeRoot = isIncludeRoot;
			MacroScope = isMacroScope;
			Properties = new Dictionary<string, ConfigNode>(properties, StringComparer.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">Type to construct from</param>
		public ObjectConfigNode(Type type)
			: this(type, new Dictionary<Type, ConfigNode>())
		{ }

		ObjectConfigNode(Type type, Dictionary<Type, ConfigNode> recursiveTypes)
		{
			recursiveTypes.Add(type, this);

			IncludeRoot = type.GetCustomAttribute<ConfigIncludeRootAttribute>() != null;
			MacroScope = type.GetCustomAttribute<ConfigMacroScopeAttribute>() != null;
			Properties = new Dictionary<string, ConfigNode>(StringComparer.OrdinalIgnoreCase);

			// Find all the direct include properties
			PropertyInfo[] propertyInfos = type.GetProperties(BindingFlags.Instance | BindingFlags.Public | BindingFlags.GetProperty);
			foreach (PropertyInfo propertyInfo in propertyInfos)
			{
				if (propertyInfo.GetCustomAttribute<JsonIgnoreAttribute>() == null)
				{
					ConfigNode? propertyType = CreateTypeForProperty(propertyInfo, recursiveTypes);
					if (propertyType != null)
					{
						string name = propertyInfo.GetCustomAttribute<JsonPropertyNameAttribute>()?.Name ?? propertyInfo.Name;
						Properties.Add(name, propertyType);
					}
				}
			}
		}

		bool IsDefault()
			=> !IncludeRoot && !MacroScope && Properties.Count == 0;

		static ConfigNode? CreateTypeForProperty(PropertyInfo propertyInfo, Dictionary<Type, ConfigNode> recursiveTypes)
		{
			Type propertyType = propertyInfo.PropertyType;
			if (!propertyType.IsClass || propertyType == typeof(string))
			{
				bool include = propertyInfo.GetCustomAttribute<ConfigIncludeAttribute>() != null;
				bool relativePath = propertyInfo.GetCustomAttribute<ConfigRelativePathAttribute>() != null;
				if (include || relativePath)
				{
					return new ScalarConfigNode(include, relativePath);
				}
				else
				{
					return null;
				}
			}
			return CreateType(propertyType, recursiveTypes);
		}

		static ConfigNode? CreateType(Type type, Dictionary<Type, ConfigNode> recursiveTypes)
		{
			ConfigNode? value;
			if (!type.IsClass || type == typeof(string) || type == typeof(JsonNode))
			{
				value = null;
			}
			else if (type == typeof(ConfigMacro))
			{
				value = new MacroConfigNode();
			}
			else if (type.IsAssignableTo(typeof(ConfigResource)))
			{
				value = new ResourceConfigNode();
			}
			else if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(List<>))
			{
				ConfigNode? elementType = CreateType(type.GetGenericArguments()[0], recursiveTypes);
				value = (elementType == null) ? null : new ArrayConfigNode(elementType);
			}
			else if (type.IsGenericType && type.GetGenericTypeDefinition() == typeof(Dictionary<,>))
			{
				ConfigNode? elementType = CreateType(type.GetGenericArguments()[1], recursiveTypes);
				value = (elementType == null) ? null : new DictionaryConfigNode(elementType);
			}
			else if (!recursiveTypes.TryGetValue(type, out value))
			{
				ObjectConfigNode objValue = new ObjectConfigNode(type);
				value = objValue.IsDefault() ? null : objValue;
			}
			return value;
		}

		/// <summary>
		/// Preprocesses an object
		/// </summary>
		/// <param name="obj">Object to process</param>
		/// <param name="context">Preprocessor context</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<JsonObject> PreprocessAsync(JsonObject obj, ConfigContext context, CancellationToken cancellationToken)
			=> (JsonObject)(await PreprocessAsync(obj, null, context, cancellationToken))!;

		/// <inheritdoc/>
		public override void ParseMacros(JsonNode? jsonNode, ConfigContext context, Dictionary<string, string> macros)
		{
			if (jsonNode is JsonObject jsonObject)
			{
				foreach ((string name, JsonNode? node) in jsonObject)
				{
					if (node != null && Properties.TryGetValue(name, out ConfigNode? property))
					{
						if (property is not ObjectConfigNode propertyObj || !propertyObj.MacroScope)
						{
							property.ParseMacros(node, context, macros);
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public override async Task<JsonNode?> PreprocessAsync(JsonNode? node, JsonNode? existingNode, ConfigContext context, CancellationToken cancellationToken)
		{
			JsonObject? obj = (JsonObject?)node;
			JsonObject? target = (JsonObject?)existingNode;
			if (obj == null)
			{
				return target;
			}

			// Before parsing properties into this object, read all the includes recursively
			if (IncludeRoot)
			{
				// When entering from property/array processing (not from an include), create a
				// fresh diamond dedup scope so sibling roots (e.g. two StreamConfigs in the same
				// project) can independently include the same file.
				// When entering from include processing (InIncludeScope is true), share the
				// parent's scope so diamonds within the same include tree are still detected.
				Dictionary<Uri, Uri?>? savedProcessedIncludes = null;
				if (!context.InIncludeScope)
				{
					savedProcessedIncludes = context.ProcessedIncludes;
					context.ProcessedIncludes = new Dictionary<Uri, Uri?>();
				}
				bool savedInIncludeScope = context.InIncludeScope;
				context.IncludeRootDepth++;

				try
				{
					// Find all the includes
					List<IConfigFile> includes = new List<IConfigFile>();
					await ParseIncludesInternalAsync(obj, includes, context, cancellationToken);

					// Find all the files, merge them into target. Skip files that have already been
					// processed to handle diamond dependencies (e.g. A includes B and C, both B and C
					// include D - D should only be merged once).
					string currentFile = context.CurrentFile?.ToString() ?? "(unknown)";
					context.InIncludeScope = true;
					foreach (IConfigFile include in includes)
					{
						if (context.ProcessedIncludes.TryGetValue(include.Uri, out Uri? firstIncluder))
						{
							context.Warnings.Add($"Diamond include: '{include.Uri}' was already included by '{firstIncluder}' (skipped duplicate from '{currentFile}'). Consider moving this include to a common parent file to avoid redundant includes.");
							continue;
						}
						context.ProcessedIncludes.Add(include.Uri, context.CurrentFile);

						context.IncludeStack.Push(include);

						JsonObject includedJsonObject = await context.ParseFileAsync(include, cancellationToken);
						target = (JsonObject?)await PreprocessAsync(includedJsonObject, target, context, cancellationToken);

						context.IncludeStack.Pop();
					}
				}
				finally
				{
					context.InIncludeScope = savedInIncludeScope;
					context.IncludeRootDepth--;
					if (savedProcessedIncludes != null)
					{
						context.ProcessedIncludes = savedProcessedIncludes;
					}
				}
			}

			// Ensure that the target object is valid so we can write properties into it
			target ??= new JsonObject(DefaultJsonNodeOptions);

			// Parse all the macros for this scope
			if (MacroScope)
			{
				Dictionary<string, string> macros = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);
				ParseMacros(obj, context, macros);
				context.MacroScopes.Add(macros);
			}

			// Parse all the properties into this object
			foreach ((string name, JsonNode? newNode) in obj)
			{
				if (newNode is JsonValue)
				{
					context.AddProperty(name);
				}

				if (Properties.TryGetValue(name, out ConfigNode? property))
				{
					context.EnterScope(name);
					target[name] = await property.PreprocessAsync(newNode, target[name], context, cancellationToken);
					context.LeaveScope();
				}
				else
				{
					target[name] = Merge(ExpandMacros(newNode, context), target[name]);
				}
			}

			// Parse all the macros for this scope
			if (MacroScope)
			{
				context.MacroScopes.RemoveAt(context.MacroScopes.Count - 1);
			}

			return target;
		}

		static JsonNode? Merge(JsonNode? source, JsonNode? target)
		{
			if (source is JsonObject sourceObj && target is JsonObject targetObj)
			{
				JsonObject result = new JsonObject();
				foreach ((string name, JsonNode? targetNode) in targetObj)
				{
					JsonNode? sourceNode = sourceObj[name];
					if (sourceNode == null)
					{
						result[name] = targetNode?.DeepClone();
					}
					else
					{
						result[name] = Merge(sourceNode, targetNode);
					}
				}
				foreach ((string name, JsonNode? sourceNode) in sourceObj)
				{
					if (!result.ContainsKey(name))
					{
						result[name] = sourceNode?.DeepClone();
					}
				}
				return result;
			}
			else if (source is JsonArray sourceArr && target is JsonArray targetArr)
			{
				JsonArray result = (JsonArray)targetArr.DeepClone();
				foreach (JsonNode? node in sourceArr)
				{
					result.Add(node?.DeepClone());
				}
				return result;
			}
			return source?.DeepClone();
		}

		/// <inheritdoc/>
		public override async Task ParseIncludesAsync(JsonNode? node, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
		{
			if (!IncludeRoot && node is JsonObject obj)
			{
				await ParseIncludesInternalAsync(obj, includes, context, cancellationToken);
			}
		}

		async Task ParseIncludesInternalAsync(JsonObject obj, List<IConfigFile> includes, ConfigContext context, CancellationToken cancellationToken)
		{
			foreach ((string name, JsonNode? propertyNode) in obj)
			{
				if (Properties.TryGetValue(name, out ConfigNode? property))
				{
					await property.ParseIncludesAsync(propertyNode, includes, context, cancellationToken);
				}
			}
		}

		/// <summary>
		/// Adds a new node at a given path
		/// </summary>
		/// <param name="path">Path to the node to add</param>
		/// <param name="node">The node to add the requested path</param>
		public void AddChildNode(string path, ConfigNode node)
		{
			int nextDotIdx = path.IndexOf('.', StringComparison.Ordinal);
			if (nextDotIdx == -1)
			{
				Properties[path] = node;
			}
			else
			{
				string nextName = path.Substring(0, nextDotIdx);

				ConfigNode? nextNode;
				if (!Properties.TryGetValue(nextName, out nextNode))
				{
					nextNode = new ObjectConfigNode(false, false);
					Properties.Add(nextName, nextNode);
				}

				ObjectConfigNode nextObj = (ObjectConfigNode)nextNode;
				nextObj.AddChildNode(path.Substring(nextDotIdx + 1), node);
			}
		}

		/// <summary>
		/// Tries to get a node by path
		/// </summary>
		/// <param name="path">Path to the node to find</param>
		/// <param name="node">The node at the requested path</param>
		public bool TryGetChildNode(string path, [NotNullWhen(true)] out ConfigNode? node)
		{
			int nextDotIdx = path.IndexOf('.', StringComparison.Ordinal);
			if (nextDotIdx == -1)
			{
				return Properties.TryGetValue(path, out node);
			}

			if (Properties.TryGetValue(path.Substring(0, nextDotIdx), out ConfigNode? nextNode) && nextNode is ObjectConfigNode nextObj)
			{
				return nextObj.TryGetChildNode(path.Substring(nextDotIdx + 1), out node);
			}

			node = null;
			return false;
		}
	}
}
