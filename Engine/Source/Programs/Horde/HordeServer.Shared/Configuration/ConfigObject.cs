// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections;
using System.Reflection;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Possible methods for merging config values
	/// </summary>
	public enum ConfigMergeStrategy
	{
		/// <summary>
		/// Default strategy; replace with the base value if the current value is null
		/// </summary>
		Default,

		/// <summary>
		/// Append the contents of this list to the base list
		/// </summary>
		Append,

		/// <summary>
		/// Recursively merge object properties
		/// </summary>
		Recursive,
	}

	/// <summary>
	/// Attribute used to mark object properties whose child properties should be merged individually
	/// </summary>
	[AttributeUsage(AttributeTargets.Property)]
	public sealed class ConfigMergeStrategyAttribute : Attribute
	{
		/// <summary>
		/// Strategy for merging this property
		/// </summary>
		public ConfigMergeStrategy Strategy { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ConfigMergeStrategyAttribute(ConfigMergeStrategy strategy) => Strategy = strategy;
	}

	/// <summary>
	/// Helper methods for merging default values
	/// </summary>
	public static class ConfigObject
	{
		/// <summary>
		/// Merges any unassigned properties from the source object to the target
		/// </summary>
		/// <param name="objects">Set of objects to use for merging</param>
		/// <param name="logger">Optional logger. When provided, invalid bases and circular dependencies are logged as warnings instead of throwing exceptions.</param>
		public static void MergeDefaults<TKey, TValue>(IEnumerable<(TKey Key, TKey? BaseKey, TValue Value)> objects, ILogger? logger = null)
			where TKey : notnull
			where TValue : class
		{
			// Convert the objects to a dictionary
			List<(TKey Key, TKey BaseKey, TValue Value)> remainingObjects = new List<(TKey Key, TKey BaseKey, TValue Value)>();

			// Find all the objects with no base
			Dictionary<TKey, TValue> handledValues = new Dictionary<TKey, TValue>();
			foreach ((TKey key, TKey? baseKey, TValue value) in objects)
			{
#pragma warning disable CA1508 // Avoid dead conditional code (false positive due to generics)
				if (baseKey == null || Equals(baseKey, default(TKey)))
				{
					handledValues.Add(key, value);
				}
				else
				{
					remainingObjects.Add((key, baseKey, value));
				}
#pragma warning restore CA1508 // Avoid dead conditional code
			}

			// Iteratively merge objects with their base
			for (int lastRemainingObjectCount = 0; remainingObjects.Count != lastRemainingObjectCount;)
			{
				lastRemainingObjectCount = remainingObjects.Count;
				for (int idx = remainingObjects.Count - 1; idx >= 0; idx--)
				{
					(TKey key, TKey baseKey, TValue value) = remainingObjects[idx];
					if (handledValues.TryGetValue(baseKey, out TValue? baseValue))
					{
						MergeDefaults(value, baseValue);
						handledValues.Add(key, value);
						remainingObjects.RemoveAt(idx);
					}
				}
			}

			// Check we were able to merge everything
			if (remainingObjects.Count > 0)
			{
				HashSet<TKey> validKeys = new HashSet<TKey>(objects.Select(x => x.Key));
				foreach ((TKey key, TKey baseKey, TValue value) in remainingObjects)
				{
					if (!validKeys.Contains(baseKey!))
					{
						if (logger != null)
						{
							logger.LogWarning("{Key} has invalid/missing base {BaseKey}; skipping merge", key, baseKey);
							handledValues.Add(key, value);
						}
						else
						{
							throw new Exception($"{key} has invalid/missing base {baseKey}");
						}
					}
				}

				// Remove items that were handled above (invalid base, logged as warning)
				remainingObjects.RemoveAll(x => handledValues.ContainsKey(x.Key));

				if (remainingObjects.Count > 0)
				{
					List<TKey> circularKeys = remainingObjects.Select(x => x.Key).ToList();
					if (logger != null)
					{
						if (circularKeys.Count == 1)
						{
							logger.LogWarning("{Key} has a circular dependency on itself; skipping merge", circularKeys[0]);
						}
						else
						{
							logger.LogWarning("{Keys} have circular dependencies; skipping merge", StringUtils.FormatList(circularKeys.Select(x => x.ToString() ?? "(unknown)")));
						}
					}
					else
					{
						if (circularKeys.Count == 1)
						{
							throw new Exception($"{circularKeys[0]} has a circular dependency on itself");
						}
						else
						{
							throw new Exception($"{StringUtils.FormatList(circularKeys.Select(x => x.ToString() ?? "(unknown)"))} have circular dependencies.");
						}
					}
				}
			}
		}

		/// <summary>
		/// Merges any unassigned properties from the source object to the target
		/// </summary>
		/// <param name="target">Target object to merge into</param>
		/// <param name="source">Source object to merge from</param>
		public static void MergeDefaults<T>(T target, T source) where T : class
		{
			MergeDefaults(typeof(T), target, source);
		}

		/// <summary>
		/// Merges any unassigned properties from the source object to the target
		/// </summary>
		/// <param name="type">Type of the referenced object</param>
		/// <param name="target">Target object to merge into</param>
		/// <param name="source">Source object to merge from</param>
		public static void MergeDefaults(Type type, object target, object source)
		{
			foreach (PropertyInfo propertyInfo in type.GetProperties(BindingFlags.Instance | BindingFlags.Public))
			{
				object? targetValue = propertyInfo.GetValue(target);
				object? sourceValue = propertyInfo.GetValue(source);

				ConfigMergeStrategy strategy = propertyInfo.GetCustomAttribute<ConfigMergeStrategyAttribute>()?.Strategy ?? ConfigMergeStrategy.Default;
				switch (strategy)
				{
					case ConfigMergeStrategy.Default:
						if (targetValue == null)
						{
							propertyInfo.SetValue(target, sourceValue);
						}
						break;
					case ConfigMergeStrategy.Append:
						if (targetValue == null)
						{
							propertyInfo.SetValue(target, sourceValue);
						}
						else if (sourceValue != null && !ReferenceEquals(sourceValue, targetValue))
						{
							IList sourceList = (IList)sourceValue;
							IList targetList = (IList)targetValue;
							for (int idx = 0; idx < sourceList.Count; idx++)
							{
								targetList.Insert(idx, sourceList[idx]);
							}
						}
						break;
					case ConfigMergeStrategy.Recursive:
						if (targetValue == null)
						{
							propertyInfo.SetValue(target, sourceValue);
						}
						else if (sourceValue != null)
						{
							MergeDefaults(propertyInfo.PropertyType, targetValue, sourceValue);
						}
						break;
				}
			}
		}
	}
}
