// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Special resource type in configuration files which stores data from an external source
	/// </summary>
	[JsonSchemaString]
	[JsonConverter(typeof(ConfigResourceSerializer))]
	public sealed class ConfigResource
	{
		/// <summary>
		/// Path to the resource
		/// </summary>
		[ConfigRelativePath]
		public string? Path { get; set; }

		/// <summary>
		/// Data for the resource. This will be stored externally to the config data.
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; set; } = ReadOnlyMemory<byte>.Empty;
	}

	/// <summary>
	/// Serializer for <see cref="ConfigResource"/> objects
	/// </summary>
	public class ConfigResourceSerializer : JsonConverter<ConfigResource>
	{
		readonly bool _withData;

		/// <summary>
		/// Explicit default constructor (cannot use default argument with JsonSerializer)
		/// </summary>
		public ConfigResourceSerializer()
			: this(true) { }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="withData">Whether to include data in serialized resource objects</param>
		public ConfigResourceSerializer(bool withData)
			=> _withData = withData;

		/// <inheritdoc/>
		public override ConfigResource Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			ConfigResource resource = new ConfigResource();
			if (reader.TokenType == JsonTokenType.String)
			{
				resource.Path = reader.GetString();
			}
			else if (reader.TokenType == JsonTokenType.StartObject)
			{
				while (reader.Read() && reader.TokenType != JsonTokenType.EndObject)
				{
					if (reader.TokenType == JsonTokenType.PropertyName)
					{
						if (reader.ValueTextEquals(nameof(ConfigResource.Path)))
						{
							reader.Read();
							resource.Path = reader.GetString();
						}
						else if (reader.ValueTextEquals(nameof(ConfigResource.Data)) && _withData)
						{
							reader.Read();
							resource.Data = reader.GetBytesFromBase64();
						}
					}
				}
			}
			else
			{
				throw new InvalidOperationException();
			}
			return resource;
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ConfigResource resource, JsonSerializerOptions options)
		{
			if (resource.Data.Length == 0 || !_withData)
			{
				writer.WriteStringValue(resource.Path);
			}
			else
			{
				writer.WriteStartObject();
				writer.WriteString(nameof(ConfigResource.Path), resource.Path);
				writer.WriteBase64String(nameof(ConfigResource.Data), resource.Data.Span);
				writer.WriteEndObject();
			}
		}
	}
}
