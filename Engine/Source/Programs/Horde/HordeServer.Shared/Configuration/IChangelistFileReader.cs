// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Configuration
{
	/// <summary>
	/// Reads files from a shelved Perforce changelist for config validation and diffing.
	/// </summary>
	public interface IChangelistFileReader
	{
		/// <summary>
		/// Reads all config-like files from a shelved changelist.
		/// </summary>
		/// <param name="changelist">The changelist number (must be shelved)</param>
		/// <param name="cluster">Perforce cluster name (null for default)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Dictionary mapping config URI to file contents</returns>
		Task<Dictionary<Uri, byte[]>> ReadChangelistFilesAsync(int changelist, string? cluster, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Exception thrown when a changelist cannot be found or described.
	/// </summary>
	public class ChangelistNotFoundException : Exception
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ChangelistNotFoundException(string message) : base(message)
		{
		}
	}
}
