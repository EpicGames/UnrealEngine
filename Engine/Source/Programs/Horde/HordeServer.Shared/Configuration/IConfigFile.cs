// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Users;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Accessor for a specific revision of a config file. Provides metadata about the current revision, and allows reading its data.
	/// </summary>
	public interface IConfigFile
	{
		/// <summary>
		/// URI of the config file
		/// </summary>
		Uri Uri { get; }

		/// <summary>
		/// String used to identify a specific version of the config data. Used for ordinal comparisons, otherwise opaque.
		/// </summary>
		string Revision { get; }

		/// <summary>
		/// Author of this revision, if known
		/// </summary>
		IUser? Author { get; }

		/// <summary>
		/// Reads data for the config file
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>UTF-8 encoded data for the config file</returns>
		ValueTask<ReadOnlyMemory<byte>> ReadAsync(CancellationToken cancellationToken);
	}

	/// <summary>
	/// Extension methods for config files
	/// </summary>
	public static class ConfigFileExtensions
	{
		/// <summary>
		/// Formats a the URI for a config file in a format that is more readable for users
		/// </summary>
		/// <param name="file">Config file to get a path for</param>
		public static string GetUserFormattedPath(this IConfigFile file)
		{
			const string DefaultPerforcePrefix = "perforce://default//";

			string path = file.Uri.ToString();
			if (path.StartsWith(DefaultPerforcePrefix, StringComparison.OrdinalIgnoreCase))
			{
				path = path.Substring(DefaultPerforcePrefix.Length - 2);
			}
			if (!String.IsNullOrEmpty(file.Revision))
			{
				path = $"{path}@{file.Revision}";
			}
			return path;
		}
	}
}
