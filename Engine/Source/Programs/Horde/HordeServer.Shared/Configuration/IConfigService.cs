// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Interface for the config service
	/// </summary>
	public interface IConfigService
	{
		/// <summary>
		/// Event for notifications that the config has been updated
		/// </summary>
		event Action<ConfigUpdateInfo>? OnConfigUpdate;

		/// <summary>
		/// Validate a new set of config files. Parses and runs PostLoad methods on them.
		/// </summary>
		Task<ConfigValidationResult> ValidateAsync(Dictionary<Uri, byte[]> files, CancellationToken cancellationToken);

		/// <summary>
		/// Resolve a set of config file overrides through the full pipeline (includes, macros, inheritance)
		/// and return the resulting GlobalConfig. Unread files are returned as warnings, not errors.
		/// </summary>
		/// <param name="files">Config file overrides (URI to contents)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Resolved config, error message if parsing failed, list of unread file URIs</returns>
		Task<ConfigResolveResult> ResolveWithOverridesAsync(Dictionary<Uri, byte[]> files, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Result from validating config files
	/// </summary>
	public class ConfigValidationResult
	{
		/// <summary>
		/// Whether validation succeeded
		/// </summary>
		public bool Success => Error == null;

		/// <summary>
		/// Error message if validation failed (null on success)
		/// </summary>
		public string? Error { get; set; }

		/// <summary>
		/// Warnings generated during config processing (e.g. diamond include dependencies)
		/// </summary>
		public List<string> Warnings { get; set; } = new List<string>();
	}

	/// <summary>
	/// Result from resolving config with overrides
	/// </summary>
	public class ConfigResolveResult
	{
		/// <summary>
		/// The resolved global config (null if validation failed)
		/// </summary>
		public object? Config { get; set; }

		/// <summary>
		/// The resolved plugin configs (null if validation failed).
		/// Plugins can use their GetXxxConfig() extension methods on this.
		/// </summary>
		public IDictionary<PluginName, IPluginConfig>? Plugins { get; set; }

		/// <summary>
		/// Error message if config parsing failed (null on success)
		/// </summary>
		public string? Error { get; set; }

		/// <summary>
		/// Config file URIs from the overrides that were not consumed by the pipeline
		/// </summary>
		public List<Uri> UnreadFiles { get; set; } = new List<Uri>();

		/// <summary>
		/// Warnings generated during config processing (e.g. diamond include dependencies)
		/// </summary>
		public List<string> Warnings { get; set; } = new List<string>();
	}
}
