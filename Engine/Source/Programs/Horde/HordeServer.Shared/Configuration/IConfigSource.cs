// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using EpicGames.Core;
using EpicGames.Horde.Users;
using HordeServer.Users;

namespace HordeServer.Configuration
{
	/// <summary>
	/// Information about the updated config
	/// </summary>
	public record class ConfigUpdateInfo(List<string> Status, HashSet<UserId> Authors, Exception? Exception);

	/// <summary>
	/// Source for reading config files
	/// </summary>
	public interface IConfigSource
	{
		/// <summary>
		/// URI scheme for this config source
		/// </summary>
		string Scheme { get; }

		/// <summary>
		/// Update interval for this source
		/// </summary>
		TimeSpan UpdateInterval { get; }

		/// <summary>
		/// Reads a config file from this source
		/// </summary>
		/// <param name="uris">Locations of the config files to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Config file data</returns>
		Task<IConfigFile[]> GetFilesAsync(Uri[] uris, CancellationToken cancellationToken);

		/// <summary>
		/// Gets summary infomration for sending notifications
		/// </summary>
		/// <param name="files">New files for the configuration</param>
		/// <param name="prevFiles">Previous set of files for the configuration</param>
		/// <param name="updateInfo">Information about the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task GetUpdateInfoAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, ConfigUpdateInfo updateInfo, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Extension methods for <see cref="IConfigSource"/>
	/// </summary>
	public static class ConfigSource
	{
		/// <summary>
		/// Gets a single config file from a source
		/// </summary>
		/// <param name="source">Source to query</param>
		/// <param name="uri">Location of the config file to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Config file data</returns>
		public static async Task<IConfigFile> GetAsync(this IConfigSource source, Uri uri, CancellationToken cancellationToken)
		{
			IConfigFile[] result = await source.GetFilesAsync(new[] { uri }, cancellationToken);
			return result[0];
		}
	}

	/// <summary>
	/// In-memory config file source
	/// </summary>
	public sealed class InMemoryConfigSource : IConfigSource
	{
		class ConfigFileRevisionImpl : IConfigFile
		{
			public Uri Uri { get; }
			public string Revision { get; }
			public ReadOnlyMemory<byte> Data { get; }
			public IUser? Author => null;

			public ConfigFileRevisionImpl(Uri uri, string version, ReadOnlyMemory<byte> data)
			{
				Uri = uri;
				Revision = version;
				Data = data;
			}

			public ValueTask<ReadOnlyMemory<byte>> ReadAsync(CancellationToken cancellationToken) => new ValueTask<ReadOnlyMemory<byte>>(Data);
		}

		readonly Dictionary<Uri, ConfigFileRevisionImpl> _files = new Dictionary<Uri, ConfigFileRevisionImpl>();

		/// <summary>
		/// Name of the scheme for this source
		/// </summary>
		public const string Scheme = "memory";

		/// <inheritdoc/>
		string IConfigSource.Scheme => Scheme;

		/// <inheritdoc/>
		public TimeSpan UpdateInterval => TimeSpan.FromSeconds(1.0);

		/// <summary>
		/// Manually adds a new config file
		/// </summary>
		/// <param name="path">Path to the config file</param>
		/// <param name="data">Config file data</param>
		public void Add(Uri path, ReadOnlyMemory<byte> data)
		{
			_files.Add(path, new ConfigFileRevisionImpl(path, "v1", data));
		}

		/// <inheritdoc/>
		public Task<IConfigFile[]> GetFilesAsync(Uri[] uris, CancellationToken cancellationToken)
		{
			IConfigFile[] result = new IConfigFile[uris.Length];
			for (int idx = 0; idx < uris.Length; idx++)
			{
				ConfigFileRevisionImpl? configFile;
				if (!_files.TryGetValue(uris[idx], out configFile))
				{
					throw new FileNotFoundException($"Config file {uris[idx]} not found.");
				}
				result[idx] = configFile;
			}
			return Task.FromResult(result);
		}

		/// <inheritdoc/>
		public Task GetUpdateInfoAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, ConfigUpdateInfo updateInfo, CancellationToken cancellationToken)
			=> Task.CompletedTask;
	}

	/// <summary>
	/// Config file source which reads from the filesystem
	/// </summary>
	public sealed class FileConfigSource : IConfigSource
	{
		class ConfigFileImpl : IConfigFile
		{
			public Uri Uri { get; }
			public string Revision { get; }
			public DateTime LastWriteTimeUtc { get; }
			public ReadOnlyMemory<byte> Data { get; }
			public IUser? Author => null;

			public ConfigFileImpl(Uri uri, DateTime lastWriteTimeUtc, ReadOnlyMemory<byte> data)
			{
				Uri = uri;
				Revision = $"timestamp={lastWriteTimeUtc.Ticks}";
				LastWriteTimeUtc = lastWriteTimeUtc;
				Data = data;
			}

			public ValueTask<ReadOnlyMemory<byte>> ReadAsync(CancellationToken cancellationToken) => new ValueTask<ReadOnlyMemory<byte>>(Data);
		}

		/// <summary>
		/// Name of the scheme for this source
		/// </summary>
		public const string Scheme = "file";

		/// <inheritdoc/>
		string IConfigSource.Scheme => Scheme;

		/// <inheritdoc/>
		public TimeSpan UpdateInterval => TimeSpan.FromSeconds(5.0);

		readonly DirectoryReference _baseDir;
		readonly ConcurrentDictionary<FileReference, ConfigFileImpl> _files = new ConcurrentDictionary<FileReference, ConfigFileImpl>();

		/// <summary>
		/// Constructor
		/// </summary>
		public FileConfigSource()
			: this(DirectoryReference.GetCurrentDirectory())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="baseDir">Base directory for resolving relative paths</param>
		public FileConfigSource(DirectoryReference baseDir)
		{
			_baseDir = baseDir;
		}

		/// <inheritdoc/>
		public async Task<IConfigFile[]> GetFilesAsync(Uri[] uris, CancellationToken cancellationToken)
		{
			IConfigFile[] files = new IConfigFile[uris.Length];
			for (int idx = 0; idx < uris.Length; idx++)
			{
				Uri uri = uris[idx];
				FileReference localPath = FileReference.Combine(_baseDir, uri.LocalPath);

				ConfigFileImpl? file;
				for (; ; )
				{
					if (_files.TryGetValue(localPath, out file))
					{
						if (FileReference.GetLastWriteTimeUtc(localPath) == file.LastWriteTimeUtc)
						{
							break;
						}
						else
						{
							_files.TryRemove(new KeyValuePair<FileReference, ConfigFileImpl>(localPath, file));
						}
					}

					using (FileStream stream = FileReference.Open(localPath, FileMode.Open, FileAccess.Read, FileShare.Read))
					{
						using MemoryStream memoryStream = new MemoryStream();
						await stream.CopyToAsync(memoryStream, cancellationToken);
						DateTime lastWriteTime = FileReference.GetLastWriteTimeUtc(localPath);
						file = new ConfigFileImpl(uri, lastWriteTime, memoryStream.ToArray());
					}

					if (_files.TryAdd(localPath, file))
					{
						break;
					}
				}

				files[idx] = file;
			}
			return files;
		}

		/// <inheritdoc/>
		public Task GetUpdateInfoAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, ConfigUpdateInfo updateInfo, CancellationToken cancellationToken)
			=> Task.CompletedTask;
	}
}
