// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Configuration;

/// <summary>
/// Convert a string representation of a secret to a concrete value during config updates
/// </summary>
[AttributeUsage(AttributeTargets.Property)]
public sealed class ResolveSecretAttribute : Attribute;
