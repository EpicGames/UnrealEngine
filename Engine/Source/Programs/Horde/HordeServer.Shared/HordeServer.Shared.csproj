<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <FrameworkReference Include="Microsoft.AspNetCore.App" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="MongoDB.Driver" Version="2.22.0" />
    <PackageReference Include="OpenTelemetry" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.Console" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.OpenTelemetryProtocol" Version="1.8.1" />
  </ItemGroup>

</Project>
