// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Configuration;

namespace HordeServer
{
	/// <summary>
	/// Type of run mode this process should use. Each carry different types of workloads. 
	/// More than one mode can be active. But not all modes are not guaranteed to be compatible with each other and will
	/// raise an error if combined in such a way.
	/// </summary>
	public enum RunMode
	{
		/// <summary>
		/// Default no-op value (ASP.NET config will default to this for enums that cannot be parsed)
		/// </summary> 
		None,

		/// <summary>
		/// Handle and respond to incoming external requests, such as HTTP REST and gRPC calls.
		/// These requests are time-sensitive and short-lived, typically less than 5 secs.
		/// If processes handling requests are unavailable, it will be very visible for users.
		/// </summary>
		Server,

		/// <summary>
		/// Run non-request facing workloads. Such as background services, processing queues, running work
		/// based on timers etc. Short periods of downtime or high CPU usage due to bursts are fine for this mode.
		/// No user requests will be impacted directly. If auto-scaling is used, a much more aggressive policy can be
		/// applied (tighter process packing, higher avg CPU usage).
		/// </summary>
		Worker
	}

	/// <summary>
	/// Provides access to server deployment information
	/// </summary>
	public interface IServerInfo
	{
		/// <summary>
		/// Current version of the server
		/// </summary>
		SemVer Version { get; }

		/// <summary>
		/// Environment that the server is deployed to
		/// </summary>
		string Environment { get; }

		/// <summary>
		/// Unique session id
		/// </summary>
		string SessionId { get; }

		/// <summary>
		/// Directory containing the server executable
		/// </summary>
		DirectoryReference AppDir { get; }

		/// <summary>
		/// Directory to store server data files
		/// </summary>
		DirectoryReference DataDir { get; }

		/// <summary>
		/// Global configuration settings
		/// </summary>
		IConfiguration Configuration { get; }

		/// <summary>
		/// Whether the server is running in read-only mode
		/// </summary>
		bool ReadOnlyMode { get; }

		/// <summary>
		/// Whether to enable endpoints which are for debugging purposes
		/// </summary>
		bool EnableDebugEndpoints { get; }

		/// <summary>
		/// Url to use for generating links back to the server
		/// </summary>
		Uri ServerUrl { get; }

		/// <summary>
		/// Url to use for generating links back to the dashboard.
		/// </summary>
		Uri DashboardUrl { get; }

		/// <summary>
		/// Helper method to check if this process has activated the given mode
		/// </summary>
		/// <param name="mode">Run mode</param>
		/// <returns>True if mode is active</returns>
		bool IsRunModeActive(RunMode mode);
	}
}
