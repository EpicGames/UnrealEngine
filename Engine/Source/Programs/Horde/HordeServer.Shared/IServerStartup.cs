// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Interface for the main server startup class. Can be used by commands that want to instantiate server services.
	/// </summary>
	public interface IServerStartup
	{
		/// <summary>
		/// Configure services for the server
		/// </summary>
		void ConfigureServices(IServiceCollection services);
	}
}
