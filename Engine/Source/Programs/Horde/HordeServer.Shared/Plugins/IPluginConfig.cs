// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Acls;
using Microsoft.Extensions.Logging;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface for plugin extensions to the global config object
	/// </summary>
	public interface IPluginConfig
	{
		/// <summary>
		/// Called to fixup a plugin's configuration after deserialization
		/// </summary>
		/// <param name="configOptions">Options for configuring the plugin</param>
		void PostLoad(PluginConfigOptions configOptions);
	}

	/// <summary>
	/// Options passed to <see cref="IPluginConfig.PostLoad(PluginConfigOptions)"/>
	/// </summary>
	public record class PluginConfigOptions(ConfigVersion Version, IEnumerable<IPluginConfig> Plugins, AclConfig ParentAcl, ILogger? Logger = null);

	/// <summary>
	/// Empty implementation of <see cref="IPluginConfig"/>
	/// </summary>
	public sealed class EmptyPluginConfig : IPluginConfig
	{
		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{ }
	}
}
