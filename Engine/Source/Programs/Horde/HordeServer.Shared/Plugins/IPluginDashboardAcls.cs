// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface for plugins to declare ACL actions relevant to dashboard visibility.
	/// Plugins implementing this interface can specify what ACL actions control access
	/// to their dashboard features, and the server will return which actions the current
	/// user has when returning plugin information.
	/// </summary>
	public interface IPluginDashboardAcls
	{
		/// <summary>
		/// Gets the ACL actions that are relevant for dashboard visibility for this plugin.
		/// These actions will be checked against the current user and returned in the
		/// server info response so the dashboard can filter plugins and mount points.
		/// </summary>
		/// <returns>Array of ACL actions to check for dashboard visibility</returns>
		AclAction[] GetDashboardAclActions();

		/// <summary>
		/// Checks if the given user has the specified ACL action for this plugin
		/// </summary>
		/// <param name="action">The action to check</param>
		/// <param name="user">The user to check authorization for</param>
		/// <returns>True if the user has the action, false otherwise</returns>
		bool AuthorizeDashboardAction(AclAction action, ClaimsPrincipal user);
	}
}
