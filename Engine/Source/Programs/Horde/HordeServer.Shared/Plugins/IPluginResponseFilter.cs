// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.AspNetCore.Http;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface allowing plugins to modify API responses
	/// </summary>
	public interface IPluginResponseFilter
	{
		/// <summary>
		/// Apply any changes to the given response
		/// </summary>
		public void Apply(HttpContext context, object response);
	}
}
