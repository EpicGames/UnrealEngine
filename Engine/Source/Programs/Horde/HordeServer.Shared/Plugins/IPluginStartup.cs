// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface for modules that can extend the services known by Horde
	/// </summary>
	public interface IPluginStartup
	{
		/// <summary>
		/// Configure the application
		/// </summary>
		/// <param name="app">Application builder instance</param>
		void Configure(IApplicationBuilder app);

		/// <summary>
		/// Configure the services provided by the plugin
		/// </summary>
		/// <param name="serviceCollection">Collection of services to add to</param>
		void ConfigureServices(IServiceCollection serviceCollection);
	}
}
