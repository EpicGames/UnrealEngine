// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Plugins
{
	/// <summary>
	/// Attribute used to identify types that should be constructed for the given plugin name. Should be attached to a public class implementing
	/// <see cref="IPluginStartup"/>.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class PluginAttribute : Attribute
	{
		/// <summary>
		/// Name of the plugin
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Whether the plugin should be enabled by default
		/// </summary>
		public bool EnabledByDefault { get; set; } = true;

		/// <summary>
		/// Other plugins that this plugin depends on
		/// </summary>
		public string[] DependsOn { get; set; } = [];

		/// <summary>
		/// Type containing server settings for this plugin. Will be injected into a <see cref="Microsoft.Extensions.Options.IOptions{TOptions}"/>
		/// object in the service container, as well as (optionally) the constructor for the the concrete plugin startup object.
		/// </summary>
		public Type? ServerConfigType { get; set; }

		/// <summary>
		/// Type containing global settings for this plugin. Must implement the <see cref="IPluginConfig"/> interface.
		/// 
		/// Standard <see cref="Microsoft.Extensions.Options.IOptions{TOptions}"/> and 
		/// <see cref="Microsoft.Extensions.Options.IOptionsMonitor{TOptions}"/> objects will be registered for this type 
		/// in the service container.
		/// </summary>
		public Type? GlobalConfigType { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public PluginAttribute(string name)
			=> Name = name;
	}
}
