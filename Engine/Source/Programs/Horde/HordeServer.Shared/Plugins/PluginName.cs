// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Name of a plugin
	/// </summary>
	/// <param name="Id"></param>
	[LogValueType]
	[JsonSchemaString]
	[TypeConverter(typeof(StringIdTypeConverter<PluginName, PluginNameStringIdConverter>))]
	[StringIdConverter(typeof(PluginNameStringIdConverter))]
	public record struct PluginName(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name"></param>
		public PluginName(string name)
			: this(new StringId(name))
		{ }

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter from PluginName objects to regular StringId types
	/// </summary>
	class PluginNameStringIdConverter : StringIdConverter<PluginName>
	{
		/// <inheritdoc/>
		public override PluginName FromStringId(StringId id)
			=> new PluginName(id);

		/// <inheritdoc/>
		public override StringId ToStringId(PluginName value)
			=> value.Id;
	}
}
