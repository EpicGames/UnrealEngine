// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Plugins
{
	/// <summary>
	/// Base class for plugin server config objects
	/// </summary>
	public class PluginServerConfig
	{
		/// <summary>
		/// Whether the plugin should be enabled or not
		/// </summary>
		public bool? Enabled { get; set; }
	}
}
