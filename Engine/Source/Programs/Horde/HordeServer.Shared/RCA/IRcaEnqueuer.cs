// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Issues;
using EpicGames.Horde.Streams;

namespace HordeServer.RCA
{
	/// <summary>
	/// Interface for enqueuing RCA processing requests.
	/// Implementations persist the request and process it asynchronously.
	/// </summary>
	public interface IRcaEnqueuer
	{
		/// <summary>
		/// Enqueue an RCA processing request for the given issue.
		/// </summary>
		/// <param name="issueId">The issue ID to analyze</param>
		/// <param name="streamId">The stream the issue belongs to</param>
		/// <param name="workflowId">The workflow ID (null if not in a workflow)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		Task EnqueueAsync(int issueId, StreamId streamId, WorkflowId? workflowId, CancellationToken cancellationToken);
	}
}
