// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;

namespace HordeServer.RCA
{
	/// <summary>
	/// A runner-up suspect from an RCA analysis
	/// </summary>
	public record RcaRunnerUp(CommitId Commit, string? BriefReason);

	/// <summary>
	/// Result of an auto-RCA analysis
	/// </summary>
	public record RcaResult(CommitId? CulpritCommitId, string? Reason, double Confidence, string? Summary, string? Category, IReadOnlyList<RcaRunnerUp>? RunnerUps);

	/// <summary>
	/// Interface for notifying about completed RCA results (e.g., posting to Slack threads).
	/// </summary>
	public interface IRcaNotifier
	{
		/// <summary>
		/// Notify that RCA analysis has completed for an issue
		/// </summary>
		/// <param name="issueId">The issue ID that was analyzed</param>
		/// <param name="result">The RCA result to notify about</param>
		/// <param name="cancellationToken">Cancellation token</param>
		Task NotifyRcaCompleteAsync(int issueId, RcaResult result, CancellationToken cancellationToken);
	}
}
