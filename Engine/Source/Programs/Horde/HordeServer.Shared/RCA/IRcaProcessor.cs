// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Issues;
using EpicGames.Horde.Streams;

namespace HordeServer.RCA
{
	/// <summary>
	/// Interface for processing Root Cause Analysis on issues.
	/// Implementations may use LLMs, ML models, or other techniques.
	/// </summary>
	public interface IRcaProcessor
	{
		/// <summary>
		/// Process RCA for the given issue if eligible. Implementations should
		/// check workflow eligibility internally and return early if not eligible.
		/// </summary>
		/// <param name="issueId">The issue ID to analyze</param>
		/// <param name="streamId">The stream the issue belongs to</param>
		/// <param name="workflowId">The workflow ID (null if not in a workflow)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		Task ProcessRcaAsync(int issueId, StreamId streamId, WorkflowId? workflowId, CancellationToken cancellationToken);
	}
}
