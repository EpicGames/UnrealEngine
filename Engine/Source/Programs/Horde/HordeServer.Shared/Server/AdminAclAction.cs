// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Server
{
	/// <summary>
	/// ACL actions for admin operations
	/// </summary>
	public static class AdminAclAction
	{
		/// <summary>
		/// Ability to read any data from the server. Always inherited.
		/// </summary>
		public static AclAction AdminRead { get; } = new AclAction("AdminRead");

		/// <summary>
		/// Ability to write any data to the server.
		/// </summary>
		public static AclAction AdminWrite { get; } = new AclAction("AdminWrite");
	}
}
