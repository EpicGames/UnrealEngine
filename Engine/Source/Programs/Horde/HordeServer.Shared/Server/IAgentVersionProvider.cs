// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Server
{
	/// <summary>
	/// Allows subsystems to advertise the version of bundled tools via the /server/info endpoint
	/// </summary>
	public interface IAgentVersionProvider
	{
		/// <summary>
		/// Returns the version number of the Horde agent
		/// </summary>
		Task<string?> GetAsync(CancellationToken cancellationToken = default);
	}
}

