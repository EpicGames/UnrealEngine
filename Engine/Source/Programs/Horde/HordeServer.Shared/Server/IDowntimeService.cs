// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Server
{
	/// <summary>
	/// Interface for a service which keeps track of whether we're during downtime
	/// </summary>
	public interface IDowntimeService
	{
		/// <summary>
		/// Returns true if downtime is currently active
		/// </summary>
		public bool IsDowntimeActive
		{
			get;
		}
	}
}
