// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Diagnostics.HealthChecks;

namespace HordeServer.Server
{
	/// <summary>
	/// Allows reporting the health of a subsystem
	/// </summary>
	public interface IHealthMonitor
	{
		/// <summary>
		/// Sets the name used for reporting status of this service
		/// </summary>
		void SetName(string name);

		/// <summary>
		/// Updates the current health of a system
		/// </summary>
		Task UpdateAsync(HealthStatus result, string? message = null, DateTimeOffset? timestamp = null);
	}

	/// <summary>
	/// Typed implementation of <see cref="IHealthMonitor"/>
	/// </summary>
	/// <typeparam name="T">Type of the subsystem</typeparam>
	public interface IHealthMonitor<T> : IHealthMonitor
	{
	}
}
