// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Server
{
	/// <summary>
	/// Interface for the server lifetime service
	/// </summary>
	public interface ILifetimeService
	{
		/// <summary>
		/// Returns true if the server is stopping
		/// </summary>
		bool IsStopping { get; }

		/// <summary>
		/// Gets an awaitable task for the server stopping
		/// </summary>
		Task StoppingTask { get; }
	}
}
