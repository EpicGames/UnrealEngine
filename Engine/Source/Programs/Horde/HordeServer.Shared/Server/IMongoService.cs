// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Utilities;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using MongoDB.Driver;

namespace HordeServer.Server
{
	/// <summary>
	/// Interface for the Horde MongoDB instance
	/// </summary>
	public interface IMongoService : IHealthCheck
	{
		/// <summary>
		/// The database instance
		/// </summary>
		IMongoDatabase Database { get; }

		/// <summary>
		/// Access the database in a read-only mode (don't create indices or modify content)
		/// </summary>
		public bool ReadOnlyMode { get; }

		/// <summary>
		/// Get the MongoDB client
		/// </summary>
		/// <returns>A MongoDB client instance</returns>
		MongoClient GetClient();

		/// <summary>
		/// Get a MongoDB collection from database
		/// </summary>
		/// <param name="name">Name of collection</param>
		/// <typeparam name="T">A MongoDB document</typeparam>
		/// <returns></returns>
		IMongoCollection<T> GetCollection<T>(string name);

		/// <summary>
		/// Get a MongoDB collection from database with a single index
		/// </summary>
		/// <param name="name">Name of collection</param>
		/// <param name="keysFunc">Method to configure keys for the collection</param>
		/// <param name="unique">Whether a unique index is required</param>
		/// <typeparam name="T">A MongoDB document</typeparam>
		/// <returns></returns>
		IMongoCollection<T> GetCollection<T>(string name, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keysFunc, bool unique = false);

		/// <summary>
		/// Get a MongoDB collection from database
		/// </summary>
		/// <param name="name">Name of collection</param>
		/// <param name="indexes">Indexes for the collection</param>
		/// <typeparam name="T">A MongoDB document</typeparam>
		/// <returns></returns>
		IMongoCollection<T> GetCollection<T>(string name, IEnumerable<MongoIndex<T>> indexes);

		/// <summary>
		/// Gets a singleton document by id
		/// </summary>
		/// <returns>The document</returns>
		Task<T> GetSingletonAsync<T>(CancellationToken cancellationToken) where T : SingletonBase, new();

		/// <summary>
		/// Gets a singleton document by id
		/// </summary>
		/// <param name="constructor">Method to use to construct a new object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The document</returns>
		Task<T> GetSingletonAsync<T>(Func<T> constructor, CancellationToken cancellationToken) where T : SingletonBase, new();

		/// <summary>
		/// Updates a singleton
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="updater"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task UpdateSingletonAsync<T>(Action<T> updater, CancellationToken cancellationToken) where T : SingletonBase, new();

		/// <summary>
		/// Updates a singleton
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="updater"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task UpdateSingletonAsync<T>(Func<T, bool> updater, CancellationToken cancellationToken) where T : SingletonBase, new();

		/// <summary>
		/// Attempts to update a singleton object
		/// </summary>
		/// <param name="singletonObject">The singleton object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the singleton document was updated</returns>
		Task<bool> TryUpdateSingletonAsync<T>(T singletonObject, CancellationToken cancellationToken) where T : SingletonBase;
	}
}
