// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Redis;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using StackExchange.Redis;

namespace HordeServer.Server
{
	/// <summary>
	/// Provides access to a Redis database
	/// </summary>
	public interface IRedisService : IHealthCheck
	{
		/// <summary>
		/// Flag for whether the connection is read-only
		/// </summary>
		bool ReadOnlyMode { get; }

		/// <summary>
		/// Connection pool
		/// </summary>
		public RedisConnectionPool ConnectionPool { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IRedisService"/>
	/// </summary>
	public static class RedisServiceExtensions
	{
		/// <summary>
		/// Get the least-loaded Redis connection from the pool
		/// Don't store the returned object and try to resolve this as late as possible to ensure load is balanced.
		/// </summary>
		/// <returns>A Redis connection multiplexer</returns>
		public static IConnectionMultiplexer GetConnection(this IRedisService redisService)
			=> redisService.ConnectionPool.GetConnection();

		/// <summary>
		/// Get the least-loaded Redis database from the connection pool
		/// Don't store the returned object and try to resolve this as late as possible to ensure load is balanced.
		/// </summary>
		/// <returns>A Redis database</returns>
		public static IDatabase GetDatabase(this IRedisService redisService)
			=> redisService.ConnectionPool.GetDatabase();

		/// <summary>
		/// Publish a message to a channel
		/// </summary>
		/// <param name="redisService">The redis service instance</param>
		/// <param name="channel">Channel to post to</param>
		/// <param name="message">Message to post to the channel</param>
		/// <param name="flags">Flags for the request</param>
		public static Task PublishAsync(this IRedisService redisService, RedisChannel channel, RedisValue message, CommandFlags flags = CommandFlags.None)
			=> redisService.GetDatabase().PublishAsync(channel, message, flags);

		/// <summary>
		/// Publish a message to a channel
		/// </summary>
		/// <typeparam name="T">Type of elements sent over the channel</typeparam>
		/// <param name="redisService">The redis service instance</param>
		/// <param name="channel">Channel to post to</param>
		/// <param name="message">Message to post to the channel</param>
		/// <param name="flags">Flags for the request</param>
		public static Task PublishAsync<T>(this IRedisService redisService, RedisChannel<T> channel, T message, CommandFlags flags = CommandFlags.None)
			=> redisService.GetDatabase().PublishAsync(channel, message, flags);

		/// <inheritdoc cref="SubscribeAsync{T}(IRedisService, RedisChannel{T}, Action{RedisChannel{T}, T})"/>
		public static Task<RedisSubscription> SubscribeAsync(this IRedisService redisService, RedisChannel channel, Action<RedisValue> callback)
			=> SubscribeAsync(redisService, channel, (ch, x) => callback(x));

		/// <inheritdoc cref="SubscribeAsync{T}(IRedisService, RedisChannel{T}, Action{RedisChannel{T}, T})"/>
		public static Task<RedisSubscription> SubscribeAsync<T>(this IRedisService redisService, RedisChannel<T> channel, Action<T> callback)
			=> SubscribeAsync(redisService, channel, (ch, x) => callback(x));

		/// <summary>
		/// Subscribe to notifications on a channel
		/// </summary>
		/// <param name="redisService">The redis service instance</param>
		/// <param name="channel">Channel to monitor</param>
		/// <param name="callback">Callback for new events</param>
		/// <returns>Subscription object</returns>
		public static async Task<RedisSubscription> SubscribeAsync(this IRedisService redisService, RedisChannel channel, Action<RedisChannel, RedisValue> callback)
		{
			IConnectionMultiplexer connection = redisService.GetConnection();
			return await connection.SubscribeAsync(channel, callback);
		}

		/// <summary>
		/// Subscribe to notifications on a channel
		/// </summary>
		/// <typeparam name="T">Type of elements sent over the channel</typeparam>
		/// <param name="redisService">The redis service instance</param>
		/// <param name="channel">Channel to monitor</param>
		/// <param name="callback">Callback for new events</param>
		/// <returns>Subscription object</returns>
		public static async Task<RedisSubscription> SubscribeAsync<T>(this IRedisService redisService, RedisChannel<T> channel, Action<RedisChannel<T>, T> callback)
		{
			IConnectionMultiplexer connection = redisService.GetConnection();
			return await connection.SubscribeAsync(channel, callback);
		}
	}
}
