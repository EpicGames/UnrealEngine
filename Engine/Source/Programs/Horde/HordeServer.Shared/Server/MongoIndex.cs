// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Server
{
	/// <summary>
	/// Constrained set of parameters for building a mongo index
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class MongoIndex : IEquatable<MongoIndex>
	{
		/// <summary>
		/// Name of the index
		/// </summary>
		[BsonElement("name"), BsonRequired]
		public string Name { get; private set; }

		/// <summary>
		/// Keys for the index
		/// </summary>
		[BsonElement("key"), BsonRequired]
		public BsonDocument KeysDocument { get; private set; }

		/// <summary>
		/// The index should be unique
		/// </summary>
		[BsonElement("unique")]
		public bool Unique { get; private set; }

		/// <summary>
		/// Whether to create a sparse index
		/// </summary>
		[BsonElement("sparse")]
		public bool Sparse { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		[BsonConstructor]
		private MongoIndex()
		{
			Name = String.Empty;
			KeysDocument = new BsonDocument();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the index</param>
		/// <param name="keysDocument">Keys for the index</param>
		/// <param name="unique">Whether the index should be unique</param>
		/// <param name="sparse">Whether to create a sparse index</param>
		public MongoIndex(string? name, BsonDocument keysDocument, bool unique, bool sparse)
		{
			Name = name ?? GetDefaultName(keysDocument);
			KeysDocument = keysDocument;
			Unique = unique;
			Sparse = sparse;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="keysFunc">Callback to configure keys for this document</param>
		/// <param name="unique">Whether the index should be unique</param>
		/// <param name="sparse">Whether to create a sparse index</param>
		public static MongoIndex<T> Create<T>(Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keysFunc, bool unique = false, bool sparse = false)
		{
			return new MongoIndex<T>(null, keysFunc(Builders<T>.IndexKeys), unique, sparse);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the index</param>
		/// <param name="keysFunc">Callback to configure keys for this document</param>
		/// <param name="unique">Whether the index should be unique</param>
		/// <param name="sparse">Whether to create a sparse index</param>
		public static MongoIndex<T> Create<T>(string name, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keysFunc, bool unique = false, bool sparse = false)
		{
			return new MongoIndex<T>(name, keysFunc(Builders<T>.IndexKeys), unique, sparse);
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj) => obj is MongoIndex other && Equals(other);

		/// <inheritdoc/>
		public override int GetHashCode() => Name.GetHashCode(StringComparison.Ordinal);

		/// <inheritdoc/>
		public bool Equals(MongoIndex? other) => other != null && Name.Equals(other.Name, StringComparison.Ordinal) && KeysDocument.Equals(other.KeysDocument) && Sparse == other.Sparse && Unique == other.Unique;

		/// <summary>
		/// Gets the default name for an index based on its keys
		/// </summary>
		/// <param name="keys">Keys for the index</param>
		/// <returns>Name of the index</returns>
		protected static string GetDefaultName(BsonDocument keys)
		{
			StringBuilder name = new StringBuilder();
			foreach (BsonElement element in keys.Elements)
			{
				if (name.Length > 0)
				{
					name.Append('_');
				}
				name.Append(element.Name);
				name.Append('_');
				name.Append(element.Value.ToString());
			}
			return name.ToString();
		}
	}

	/// <summary>
	/// Strongly typed index document
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class MongoIndex<T> : MongoIndex
	{
		/// <summary>
		/// Keys for the index
		/// </summary>
		[BsonIgnore]
		public IndexKeysDefinition<T> Keys { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the index</param>
		/// <param name="keys">Keys for the index</param>
		/// <param name="unique">Whether the index should be unique</param>
		/// <param name="sparse">Whether to create a sparse index</param>
		public MongoIndex(string? name, IndexKeysDefinition<T> keys, bool unique, bool sparse)
			: base(name, keys.Render(BsonSerializer.LookupSerializer<T>(), BsonSerializer.SerializerRegistry), unique, sparse)
		{
			Keys = keys;
		}
	}

	/// <summary>
	/// Extension methods for <see cref="MongoIndex{T}"/>
	/// </summary>
	public static class MongoIndexExtensions
	{
		/// <summary>
		/// Adds a new index to the collection
		/// </summary>
		public static void Add<T>(this List<MongoIndex<T>> list, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keyFunc, bool unique = false, bool sparse = false)
		{
			list.Add(MongoIndex.Create(keyFunc, unique, sparse));
		}

		/// <summary>
		/// Adds a new index to the collection
		/// </summary>
		public static void Add<T>(this List<MongoIndex<T>> list, string name, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keyFunc, bool unique = false, bool sparse = false)
		{
			list.Add(MongoIndex.Create(name, keyFunc, unique, sparse));
		}
	}
}
