// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;
using MongoDB.Driver;

namespace HordeServer.Server;

/// <summary>
/// Attribute to decorate MongoDB migration classes with metadata
/// </summary>
[AttributeUsage(AttributeTargets.Class)]
public sealed class MongoMigrationAttribute(string plugin, int version, string name, bool autoLoad = true) : Attribute
{
	/// <summary>
	/// Plugin identifier for this migration.
	/// </summary>
	public string Plugin { get; } = plugin;
	
	/// <summary>
	/// Gets the version number for this migration and plugin
	/// Must be monotonically increasing and unique within in plugin namespace
	/// </summary>
	public int Version { get; } = version;
	
	/// <summary>
	/// Descriptive name of the change this migration performs
	/// </summary>
	public string Name { get; } = name;
	
	/// <summary>
	/// Whether migration should be detected and loaded during full reflection scans of available migrations
	/// If set to false, the migration must manually be registered with a migrator
	/// </summary>
	public bool AutoLoad { get; } = autoLoad;
}

/// <summary>
/// Provides context and services needed for executing migrations
/// </summary>
public record MigrationContext(IMongoService MongoService, ILogger Logger)
{
	/// <summary>
	/// Shorthand for accessing the MongoDB database object
	/// </summary>
	public IMongoDatabase Database => MongoService.Database;
}

/// <summary>
/// Defines operations for upgrading and downgrading database schemas
/// Implemented by classes providing a migration
/// </summary>
public interface IMongoMigration
{
	/// <summary>
	/// Upgrade the database schema version
	/// </summary>
	public Task UpAsync(MigrationContext context, CancellationToken token);
	
	/// <summary>
	/// Downgrade the database schema version
	/// </summary>
	public Task DownAsync(MigrationContext context, CancellationToken token);
}