// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Server
{
	/// <summary>
	/// General server ACL actions
	/// </summary>
	public static class ServerAclAction
	{
		/// <summary>
		/// Access to the debug endpoints
		/// </summary>
		public static AclAction Debug { get; } = new AclAction("Debug");

		/// <summary>
		/// Ability to impersonate another user
		/// </summary>
		public static AclAction Impersonate { get; } = new AclAction("Impersonate");

		/// <summary>
		/// View estimated costs for particular operations
		/// </summary>
		public static AclAction ViewCosts { get; } = new AclAction("ViewCosts");

		/// <summary>
		/// Issue bearer token for the current user
		/// </summary>
		public static AclAction IssueBearerToken { get; } = new AclAction("IssueBearerToken");

		/// <summary>
		/// Issue bearer token for the current user
		/// </summary>
		public static AclAction IssueExtendedBearerToken { get; } = new AclAction("IssueExtendedBearerToken");

	}
}
