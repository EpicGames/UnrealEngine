// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Telemetry;

namespace HordeServer.Telemetry
{
	/// <summary>
	/// Interface for writing telemetry events from the server
	/// </summary>
	public interface ITelemetryWriter
	{
		/// <summary>
		/// Whether the writer is enabled. Can be used to shortcut construction of event data.
		/// </summary>
		bool Enabled { get; }

		/// <summary>
		/// Writes a telemetry event using the server's metadata
		/// </summary>
		/// <param name="telemetryStoreId">The telemetry store to write to</param>
		/// <param name="payload">Event data to write</param>
		void WriteEvent(TelemetryStoreId telemetryStoreId, object payload);

		/// <summary>
		/// Writes a telemetry event
		/// </summary>
		/// <param name="telemetryStoreId">The telemetry store to write to</param>
		/// <param name="metadata">Metadata for the event</param>
		/// <param name="payload">Event data to write</param>
		void WriteEvent(TelemetryStoreId telemetryStoreId, TelemetryRecordMeta metadata, object payload);
	}

	/// <summary>
	/// Additional metadata associated with a telemetry event
	/// </summary>
	/// <param name="AppId">Identifier of the application sending the event</param>
	/// <param name="AppVersion">Version number of the application</param>
	/// <param name="AppEnvironment">Name of the environment that the sending application is running in</param>
	/// <param name="SessionId">Unique identifier for the current session</param>
	public record class TelemetryRecordMeta(string? AppId = null, string? AppVersion = null, string? AppEnvironment = null, string? SessionId = null)
	{
		/// <summary>
		/// App id for events originating from Horde itself
		/// </summary>
		public const string HordeAppId = "Horde";
	}

	/// <summary>
	/// Inactive implementation of <see cref="ITelemetryWriter"/>
	/// </summary>
	public class NullTelemetryWriter : ITelemetryWriter
	{
		/// <inheritdoc/>
		public bool Enabled { get; } = false;

		/// <inheritdoc/>
		public void WriteEvent(TelemetryStoreId telemetryStoreId, object payload)
		{ }

		/// <inheritdoc/>
		public void WriteEvent(TelemetryStoreId telemetryStoreId, TelemetryRecordMeta metadata, object payload)
		{ }
	}
}
