// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Telemetry
{
	/// <summary>
	/// ACL actions for telemetry operations
	/// </summary>
	public static class TelemetryAclAction
	{
		/// <summary>
		/// Ability to search for various metrics
		/// </summary>
		public static AclAction QueryMetrics { get; } = new AclAction("QueryMetrics");
	}
}
