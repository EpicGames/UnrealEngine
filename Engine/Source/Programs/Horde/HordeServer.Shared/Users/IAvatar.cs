// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Users
{
	/// <summary>
	/// Avatar for the user
	/// </summary>
	public interface IAvatar
	{
		/// <summary>
		/// Url for a 24x24 pixel image
		/// </summary>
		public string? Image24 { get; set; }

		/// <summary>
		/// Url for a 32x32 pixel image
		/// </summary>
		public string? Image32 { get; set; }

		/// <summary>
		/// Url for a 48x48 pixel image
		/// </summary>
		public string? Image48 { get; set; }

		/// <summary>
		/// Url for a 72x72 pixel image
		/// </summary>
		public string? Image72 { get; set; }
	}
}
