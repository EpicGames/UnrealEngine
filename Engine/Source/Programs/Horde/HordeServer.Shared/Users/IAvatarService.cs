// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Users
{
	/// <summary>
	/// Provides avatar images
	/// </summary>
	public interface IAvatarService
	{
		/// <summary>
		/// Gets a users's avatar
		/// </summary>
		/// <param name="user"></param>
		/// <param name="cancellationToken">CAncellation token for the operation</param>
		/// <returns></returns>
		Task<IAvatar?> GetAvatarAsync(IUser user, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Default implementation of <see cref="IAvatarService"/>
	/// </summary>
	public class NullAvatarService : IAvatarService
	{
		/// <inheritdoc/>
		public Task<IAvatar?> GetAvatarAsync(IUser user, CancellationToken cancellationToken) => Task.FromResult<IAvatar?>(null);
	}
}

