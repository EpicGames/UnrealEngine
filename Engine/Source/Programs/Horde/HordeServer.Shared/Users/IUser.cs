// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Users
{
	/// <summary>
	/// Known user ids
	/// </summary>
	public static class KnownUsers
	{
		/// <summary>
		/// The system user. Used for automated processes.
		/// </summary>
		public static UserId System { get; } = UserId.Parse("6170b423b94a2c7c2d6b6f87");
	}

	/// <summary>
	/// Document which collates information about a user, and their personal settings
	/// </summary>
	public interface IUser
	{
		/// <summary>
		/// The user id
		/// </summary>
		public UserId Id { get; }

		/// <summary>
		/// Full name of the user
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// The user's login id
		/// </summary>
		public string Login { get; }

		/// <summary>
		/// The user's email
		/// </summary>
		public string? Email { get; }
	}

	/// <summary>
	/// Claims for a particular user
	/// </summary>
	public interface IUserClaims
	{
		/// <summary>
		/// The user id
		/// </summary>
		public UserId UserId { get; }

		/// <summary>
		/// Claims for this user (on last login)
		/// </summary>
		public IReadOnlyList<IUserClaim> Claims { get; }
	}

	/// <summary>
	/// 
	/// </summary>
	public interface IUserJobTemplateSettings
	{
		/// <summary>
		/// The stream the job was run in
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// The template id of the job
		/// </summary>
		public TemplateId TemplateId { get; }

		/// <summary>
		/// The hash of the template definition
		/// </summary>
		public string TemplateHash { get; }

		/// <summary>
		/// The arguments defined when creating the job
		/// </summary>
		public IReadOnlyList<string> Arguments { get; }

		/// <summary>
		/// The last time the job template was used
		/// </summary>
		public DateTime UpdateTimeUtc { get; }
	}

	/// <summary>
	/// Settings for updating job template user preference
	/// </summary>
	public class UpdateUserJobTemplateOptions
	{
		/// <summary>
		/// The stream the job was run in
		/// </summary>
		public StreamId StreamId { get; set; }

		/// <summary>
		/// The template id of the job
		/// </summary>
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// The hash of the template definition
		/// </summary>
		public string TemplateHash { get; set; } = String.Empty;

		/// <summary>
		/// The arguments defined when creating the job
		/// </summary>
		public IReadOnlyList<string> Arguments { get; set; } = new List<string>();
	}

	/// <summary>
	/// User settings document
	/// </summary>
	public interface IUserSettings
	{
		/// <summary>
		/// The user id
		/// </summary>
		public UserId UserId { get; }

		/// <summary>
		/// Whether to enable experimental features
		/// </summary>
		public bool EnableExperimentalFeatures { get; }

		/// <summary>
		/// Whether to always tag CL descriptions for preflights
		/// </summary>
		public bool AlwaysTagPreflightCL { get; }

		/// <summary>
		/// Opaque settings dictionary for the dashboard
		/// </summary>
		public BsonValue DashboardSettings { get; }

		/// <summary>
		/// List of pinned jobs
		/// </summary>
		public IReadOnlyList<JobId> PinnedJobIds { get; }

		/// <summary>
		/// List of pinned bisection tasks
		/// </summary>
		public IReadOnlyList<BisectTaskId> PinnedBisectTaskIds { get; }

		/// <summary>
		/// List of job template preferences
		/// </summary>
		public IReadOnlyList<IUserJobTemplateSettings>? JobTemplateSettings { get; }
	}

	/// <summary>
	/// Extension methods
	/// </summary>
	public static class UserExtensions
	{
		/// <summary>
		/// Creates an API response object
		/// </summary>
		public static GetUserResponse ToApiResponse(this IUser user, IAvatar? avatar, IUserClaims? claims, IUserSettings? settings)
		{
			GetUserResponse response = new GetUserResponse(user.Id, user.Name);
			response.Email = user.Email;

			response.Image24 = avatar?.Image24;
			response.Image32 = avatar?.Image32;
			response.Image48 = avatar?.Image48;
			response.Image72 = avatar?.Image72;

			response.Claims = claims?.Claims.Select(x => new GetUserClaimResponse(x.Type, x.Value)).ToList();

			if (settings != null)
			{
				response.EnableExperimentalFeatures = settings.EnableExperimentalFeatures;
				response.AlwaysTagPreflightCL = settings.AlwaysTagPreflightCL;

				response.DashboardSettings = BsonTypeMapper.MapToDotNetValue(settings.DashboardSettings);
				response.PinnedJobIds = settings.PinnedJobIds.ConvertAll(x => x.ToString());
				response.PinnedBisectTaskIds = settings.PinnedBisectTaskIds.ConvertAll(x => x.ToString());

				if (settings.JobTemplateSettings != null && settings.JobTemplateSettings.Count > 0)
				{
					response.JobTemplateSettings = new List<GetJobTemplateSettingsResponse>();
					for (int i = 0; i < settings.JobTemplateSettings.Count; i++)
					{
						response.JobTemplateSettings.Add(settings.JobTemplateSettings[i].ToApiResponse());
					}
				}
			}
			return response;
		}

		/// <summary>
		/// Creates an API thin user response
		/// </summary>
		public static GetThinUserInfoResponse ToThinApiResponse(this IUser user)
		{
			return new GetThinUserInfoResponse(user.Id, user.Name, user.Email, user.Login);
		}

		/// <summary>
		/// Creates an API settings response
		/// </summary>
		public static GetJobTemplateSettingsResponse ToApiResponse(this IUserJobTemplateSettings settings)
		{
			return new GetJobTemplateSettingsResponse(settings.StreamId.ToString(), settings.TemplateId.ToString(), settings.TemplateHash.ToString(), settings.Arguments.ToList(), settings.UpdateTimeUtc);
		}
	}
}
