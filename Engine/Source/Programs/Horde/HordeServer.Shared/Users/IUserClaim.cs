// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;

namespace HordeServer.Users
{
	/// <summary>
	/// Claim for a user
	/// </summary>
	public interface IUserClaim
	{
		/// <summary>
		/// Name of the claim
		/// </summary>
		public string Type { get; }

		/// <summary>
		/// Value of the claim
		/// </summary>
		public string Value { get; }
	}

	/// <summary>
	/// New claim document
	/// </summary>
	public class UserClaim : IUserClaim
	{
		/// <inheritdoc/>
		public string Type { get; set; }

		/// <inheritdoc/>
		public string Value { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type"></param>
		/// <param name="value"></param>
		public UserClaim(string type, string value)
		{
			Type = type;
			Value = value;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="other">Claim to construct from</param>
		public UserClaim(IUserClaim other)
			: this(other.Type, other.Value)
		{
		}

		/// <summary>
		/// Constructs a UserClaim from a Claim object
		/// </summary>
		/// <param name="claim">Claim object</param>
		/// <returns></returns>
		public static UserClaim FromClaim(Claim claim)
		{
			return new UserClaim(claim.Type, claim.Value);
		}

		/// <summary>
		/// Conversion operator from NET claims
		/// </summary>
		/// <param name="claim"></param>
		public static implicit operator UserClaim(Claim claim) => FromClaim(claim);
	}
}
