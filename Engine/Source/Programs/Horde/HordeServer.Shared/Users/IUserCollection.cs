// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Users;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Users
{
	/// <summary>
	/// Manages user documents
	/// </summary>
	public interface IUserCollection
	{
		/// <summary>
		/// Gets a user by unique id
		/// </summary>
		/// <param name="id">Id of the user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user information</returns>
		Task<IUser?> GetUserAsync(UserId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a cached user by unique id
		/// </summary>
		/// <param name="id">Id of the user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user information</returns>
		ValueTask<IUser?> GetCachedUserAsync(UserId? id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a user by unique id
		/// </summary>
		/// <param name="ids">Ids of the users</param>
		/// <param name="nameRegex">Name regex to match for the users</param>
		/// <param name="index">Maximum number of results</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user information</returns>
		Task<IReadOnlyList<IUser>> FindUsersAsync(IEnumerable<UserId>? ids = null, string? nameRegex = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a user by login
		/// </summary>
		/// <param name="login">Login for the user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user information</returns>
		Task<IUser?> FindUserByLoginAsync(string login, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a user by email address
		/// </summary>
		/// <param name="email">Email for the user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user information</returns>
		Task<IUser?> FindUserByEmailAsync(string email, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find or add a user with the given claims. Claims will be updated if the user exists.
		/// </summary>
		/// <param name="login">Login id of the user</param>
		/// <param name="name">Full name of the user</param>
		/// <param name="email">Email address of the user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The user document</returns>
		Task<IUser> FindOrAddUserByLoginAsync(string login, string? name = null, string? email = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the claims for a user
		/// </summary>
		/// <param name="userId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IUserClaims> GetClaimsAsync(UserId userId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update the claims for a user
		/// </summary>
		/// <param name="userId"></param>
		/// <param name="claims"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task UpdateClaimsAsync(UserId userId, IEnumerable<IUserClaim> claims, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get settings for a user
		/// </summary>
		/// <param name="userId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IUserSettings> GetSettingsAsync(UserId userId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update a user
		/// </summary>
		/// <param name="userId">The user to update</param>
		/// <param name="enableExperimentalFeatures"></param>
		/// <param name="alwaysTagPreflightCL"></param>
		/// <param name="dashboardSettings">Opaque settings object for the dashboard</param>
		/// <param name="addPinnedJobIds"></param>
		/// <param name="removePinnedJobIds"></param>
		/// <param name="templateOptions"></param>
		/// <param name="addBisectTaskIds"></param>
		/// <param name="removeBisectTaskIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated user object</returns>
		Task UpdateSettingsAsync(UserId userId, bool? enableExperimentalFeatures = null, bool? alwaysTagPreflightCL = null, BsonValue? dashboardSettings = null, IEnumerable<JobId>? addPinnedJobIds = null, IEnumerable<JobId>? removePinnedJobIds = null, UpdateUserJobTemplateOptions? templateOptions = null, IEnumerable<BisectTaskId>? addBisectTaskIds = null, IEnumerable<BisectTaskId>? removeBisectTaskIds = null, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Extension methods for the user collect
	/// </summary>
	public static class UserCollectionExtensions
	{
		/// <summary>
		/// Gets a particular user info from the collection
		/// </summary>
		/// <param name="userCollection"></param>
		/// <param name="principal"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static Task<IUser?> GetUserAsync(this IUserCollection userCollection, ClaimsPrincipal principal, CancellationToken cancellationToken = default)
		{
			UserId? userId = principal.GetUserId();
			if (userId != null)
			{
				return userCollection.GetUserAsync(userId.Value, cancellationToken);
			}
			else
			{
				return Task.FromResult<IUser?>(null);
			}
		}
	}
}
