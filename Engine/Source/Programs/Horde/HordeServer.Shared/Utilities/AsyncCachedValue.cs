// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Caches a value and asynchronously updates it after a period of time
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public sealed class AsyncCachedValue<T> : IAsyncDisposable
	{
		class State
		{
			public readonly T Value;
			readonly Stopwatch _timer;
			public Task<State>? _next;

			public TimeSpan Elapsed => _timer.Elapsed;

			public State(T value)
			{
				Value = value;
				_timer = Stopwatch.StartNew();
			}
		}

		CancellationTokenSource _cancellationTokenSource = new CancellationTokenSource();

		/// <summary>
		/// The current state
		/// </summary>
		Task<State>? _current = null;

		/// <summary>
		/// Generator for the new value
		/// </summary>
		readonly Func<CancellationToken, Task<T>> _generator;

		/// <summary>
		/// Time at which to start to refresh the value
		/// </summary>
		readonly TimeSpan _minRefreshTime;

		/// <summary>
		/// Time at which to wait for the value to refresh
		/// </summary>
		readonly TimeSpan _maxRefreshTime;

		/// <summary>
		/// Default constructor
		/// </summary>
		public AsyncCachedValue(Func<CancellationToken, Task<T>> generator, TimeSpan refreshTime)
			: this(generator, refreshTime * 0.75, refreshTime)
		{
		}

		/// <summary>
		/// Default constructor
		/// </summary>
		public AsyncCachedValue(Func<CancellationToken, Task<T>> generator, TimeSpan minRefreshTime, TimeSpan maxRefreshTime)
		{
			_generator = generator;
			_minRefreshTime = minRefreshTime;
			_maxRefreshTime = maxRefreshTime;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_current != null)
			{
				await _cancellationTokenSource.CancelAsync();
				await _current.IgnoreCanceledExceptionsAsync();
				_current = null;
			}

			if (_cancellationTokenSource != null)
			{
				_cancellationTokenSource.Dispose();
				_cancellationTokenSource = null!;
			}
		}

		/// <summary>
		/// Invalidates the current value
		/// </summary>
		public void Invalidate()
		{
			_current = null;
		}

		/// <summary>
		/// Tries to get the current value
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>The cached value, if valid</returns>
		public Task<T> GetAsync(CancellationToken cancellationToken = default)
		{
			return GetAsync(_maxRefreshTime, cancellationToken);
		}

		/// <summary>
		/// Tries to get the current value
		/// </summary>
		/// <returns>The cached value, if valid</returns>
		public Task<T> GetAsync(TimeSpan maxAge, CancellationToken cancellationToken = default)
		{
			Task<T> task = GetInternalAsync(maxAge);
			if (cancellationToken.CanBeCanceled)
			{
				// The returned task object is shared, so we don't want to cancel computation of it; just the wait for a result.
				task = WrapCancellationAsync(task, cancellationToken);
			}
			return task;
		}

		static async Task<T> WrapCancellationAsync(Task<T> task, CancellationToken cancellationToken)
		{
			await Task.WhenAny(task, Task.Delay(-1, cancellationToken));
			cancellationToken.ThrowIfCancellationRequested();
			return await task;
		}

		async Task<T> GetInternalAsync(TimeSpan maxAge)
		{
			Task<State> stateTask = CreateOrGetStateTaskAsync(ref _current);

			State state = await stateTask;
			if (state._next != null && state._next.IsCompleted)
			{
				_ = Interlocked.CompareExchange(ref _current, state._next, stateTask);
				state = await state._next;
			}
			if (state.Elapsed > maxAge)
			{
				state = await CreateOrGetStateTaskAsync(ref state._next);
			}
			if (state.Elapsed > _minRefreshTime)
			{
				_ = CreateOrGetStateTaskAsync(ref state._next);
			}

			return state.Value;
		}

		Task<State> CreateOrGetStateTaskAsync(ref Task<State>? stateTask)
		{
			for (; ; )
			{
				Task<State>? currentStateTask = stateTask;
				if (currentStateTask != null)
				{
					return currentStateTask;
				}

				Task<Task<State>> innerNewStateTask = new Task<Task<State>>(() => CreateStateAsync());
				if (Interlocked.CompareExchange(ref stateTask, innerNewStateTask.Unwrap(), null) == null)
				{
					innerNewStateTask.Start();
				}
			}
		}

		async Task<State> CreateStateAsync()
		{
			return new State(await _generator(_cancellationTokenSource.Token));
		}
	}
}
