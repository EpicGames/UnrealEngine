// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Allows spawning async tasks that will run sequentially, and allows waiting for them to complete
	/// </summary>
	public sealed class AsyncTaskQueue : IAsyncDisposable
	{
		readonly object _lockObject = new object();
		Task _task;
		readonly ILogger _logger;
		CancellationTokenSource _cancellationSource;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">Logger for any exceptions thrown by tasks</param>
		public AsyncTaskQueue(ILogger logger)
		{
			_task = Task.CompletedTask;
			_logger = logger;
			_cancellationSource = new CancellationTokenSource();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_cancellationSource != null)
			{
				await _cancellationSource.CancelAsync();
				await _task;

				_cancellationSource.Dispose();
				_cancellationSource = null!;
			}
		}

		/// <summary>
		/// Adds a new task to the queue
		/// </summary>
		/// <param name="func">Method to create the new task</param>
		public void Enqueue(Func<CancellationToken, Task> func)
		{
			lock (_lockObject)
			{
				Task prevTask = _task;
				_task = Task.Run(() => RunAsync(prevTask, func));
			}
		}

		/// <summary>
		/// Waits for any queued tasks to complete
		/// </summary>
		public Task FlushAsync(CancellationToken cancellationToken = default)
			=> _task.WaitAsync(cancellationToken);

		async Task RunAsync(Task prevTask, Func<CancellationToken, Task> func)
		{
			try
			{
				await prevTask;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unhandled exception while running task in AsyncTaskQueue: {Message}", ex.Message);
			}

			await func(_cancellationSource.Token);
		}
	}
}
