// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Helper functions for interacting with Amazon Web Services (AWS) 
	/// </summary>
	public static class AwsHelper
	{
		/// <summary>
		/// Reads AWS credentials from 'credentials' file
		///
		/// The AWS SDK provided functionality was proven too much work, so this is a simplified version.
		/// </summary>
		/// <param name="profileName">Name of the AWS profile</param>
		/// <param name="credentialsFilePath">Override for the credentials file path, set to null for default path</param>
		/// <returns>Credentials as a tuple</returns>
		/// <exception cref="Exception"></exception>
		public static (string AccessKey, string SecretAccessKey, string SecretToken) ReadAwsCredentials(string profileName, string? credentialsFilePath = null)
		{
			credentialsFilePath ??= Path.Join(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".aws", "credentials");
			string[] lines = File.ReadAllLines(credentialsFilePath);

			static string ReadValue(string line, string expectedKey)
			{
				if (line.StartsWith(expectedKey, StringComparison.Ordinal))
				{
					return line.Split("=")[1].Trim();
				}
				throw new Exception($"Unable to read key/value on line {line} for key {expectedKey}");
			}

			for (int i = 0; i < lines.Length; i++)
			{
				string line = lines[i];
				if (line == $"[{profileName}]")
				{
					string accessKey = ReadValue(lines[i + 1], "aws_access_key_id");
					string secretAccessKey = ReadValue(lines[i + 2], "aws_secret_access_key");
					string sessionToken = ReadValue(lines[i + 3], "aws_session_token");

					return (accessKey, secretAccessKey, sessionToken);
				}
			}

			throw new Exception($"Unable to find profile {profileName} in file {credentialsFilePath}");
		}
	}
}