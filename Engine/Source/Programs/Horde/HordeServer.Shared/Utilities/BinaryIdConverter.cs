// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Horde;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Serializers;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class BinaryIdBsonSerializer<TValue, TConverter> : SerializerBase<TValue> where TValue : struct where TConverter : BinaryIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args) => _converter.FromBinaryId(BinaryIdUtils.FromObjectId(context.Reader.ReadObjectId()));

		/// <inheritdoc/>
		public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, TValue value) => context.Writer.WriteObjectId(BinaryIdUtils.ToObjectId(_converter.ToBinaryId(value)));
	}

	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class BinaryIdBsonSerializationProvider : BsonSerializationProviderBase
	{
		/// <inheritdoc/>
		public override IBsonSerializer? GetSerializer(Type type, IBsonSerializerRegistry serializerRegistry)
		{
			BinaryIdConverterAttribute? attribute = type.GetCustomAttribute<BinaryIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (IBsonSerializer?)Activator.CreateInstance(typeof(BinaryIdBsonSerializer<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}
}
