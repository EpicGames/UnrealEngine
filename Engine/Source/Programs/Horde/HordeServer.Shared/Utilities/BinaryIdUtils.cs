// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde;
using MongoDB.Bson;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Utility methods for <see cref="BinaryId"/>
	/// </summary>
	public static class BinaryIdUtils
	{
		/// <summary>
		/// Creates a new BinaryId
		/// </summary>
		public static BinaryId CreateNew()
		{
			ObjectId objectId = ObjectId.GenerateNewId();
			return new BinaryId(objectId.ToByteArray());
		}

		/// <summary>
		/// Creates a BinaryId from an ObjectId
		/// </summary>
		public static BinaryId FromObjectId(ObjectId objectId)
		{
			return new BinaryId(objectId.ToByteArray());
		}

		/// <summary>
		/// Creates an ObjectId from a BinaryId
		/// </summary>
		public static ObjectId ToObjectId(BinaryId binaryId)
		{
			return new ObjectId(binaryId.ToByteArray());
		}
	}
}
