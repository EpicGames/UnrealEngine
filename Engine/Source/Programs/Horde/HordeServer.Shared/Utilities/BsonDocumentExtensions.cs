// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using MongoDB.Bson;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for BsonDocument
	/// </summary>
	public static class BsonDocumentExtensions
	{
		/// <summary>
		/// Gets a property value from a document or subdocument, indicated with dotted notation
		/// </summary>
		/// <param name="document">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="type">Expected type of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property exists and was of the correct type</returns>
		public static bool TryGetPropertyValue(this BsonDocument document, string name, BsonType type, [NotNullWhen(true)] out BsonValue? outValue)
		{
			int dotIdx = name.IndexOf('.', StringComparison.Ordinal);
			if (dotIdx == -1)
			{
				return TryGetDirectPropertyValue(document, name, type, out outValue);
			}

			BsonValue? docValue;
			if (TryGetDirectPropertyValue(document, name.Substring(0, dotIdx), BsonType.Document, out docValue))
			{
				return TryGetPropertyValue(docValue.AsBsonDocument, name.Substring(dotIdx + 1), type, out outValue);
			}

			outValue = null;
			return false;
		}

		/// <summary>
		/// Gets a property value that's an immediate child of the document
		/// </summary>
		/// <param name="document">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="type">Expected type of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property exists and was of the correct type</returns>
		private static bool TryGetDirectPropertyValue(this BsonDocument document, string name, BsonType type, [NotNullWhen(true)] out BsonValue? outValue)
		{
			BsonValue value;
			if (document.TryGetValue(name, out value) && value.BsonType == type)
			{
				outValue = value;
				return true;
			}
			else
			{
				outValue = null;
				return false;
			}
		}

		/// <summary>
		/// Gets an int32 value from the document
		/// </summary>
		/// <param name="document">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property was retrieved</returns>
		public static bool TryGetInt32(this BsonDocument document, string name, out int outValue)
		{
			BsonValue? value;
			if (document.TryGetPropertyValue(name, BsonType.Int32, out value))
			{
				outValue = value.AsInt32;
				return true;
			}
			else
			{
				outValue = 0;
				return false;
			}
		}

		/// <summary>
		/// Gets a string value from the document
		/// </summary>
		/// <param name="document">Document to get a property for</param>
		/// <param name="name">Name of the property</param>
		/// <param name="outValue">Receives the property value</param>
		/// <returns>True if the property was retrieved</returns>
		public static bool TryGetString(this BsonDocument document, string name, [NotNullWhen(true)] out string? outValue)
		{
			BsonValue? value;
			if (document.TryGetPropertyValue(name, BsonType.String, out value))
			{
				outValue = value.AsString;
				return true;
			}
			else
			{
				outValue = null;
				return false;
			}
		}
	}
}
