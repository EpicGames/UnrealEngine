// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Case insensitive set of strings
	/// </summary>
	public class CaseInsensitiveDictionary<TValue> : Dictionary<string, TValue>
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public CaseInsensitiveDictionary()
			: base(StringComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public CaseInsensitiveDictionary(IDictionary<string, TValue> items)
			: base(items, StringComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public CaseInsensitiveDictionary(IEnumerable<KeyValuePair<string, TValue>> items)
			: base(items, StringComparer.OrdinalIgnoreCase)
		{
		}
	}
}
