// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Case insensitive set of strings
	/// </summary>
	public class CaseInsensitiveStringSet : HashSet<string>
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public CaseInsensitiveStringSet()
			: base(StringComparer.OrdinalIgnoreCase)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public CaseInsensitiveStringSet(IEnumerable<string> items)
			: base(items, StringComparer.OrdinalIgnoreCase)
		{
		}
	}
}
