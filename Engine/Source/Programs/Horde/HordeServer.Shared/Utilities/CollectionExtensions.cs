// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for collections
	/// </summary>
	public static class CollectionExtensions
	{
		/// <summary>
		/// Adds an arbitrary sequence of items to a protobuf map field
		/// </summary>
		/// <typeparam name="TKey">The key type</typeparam>
		/// <typeparam name="TValue">The value type</typeparam>
		/// <param name="map">The map to update</param>
		/// <param name="sequence">Sequence of items to add</param>
		public static void Add<TKey, TValue>(this Google.Protobuf.Collections.MapField<TKey, TValue> map, IEnumerable<KeyValuePair<TKey, TValue>> sequence)
		{
			foreach (KeyValuePair<TKey, TValue> pair in sequence)
			{
				map.Add(pair.Key, pair.Value);
			}
		}
	}
}
