// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Indicates that requests and responses should be included in the exported C++ API
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class CppApiAttribute : Attribute
	{
	}
}
