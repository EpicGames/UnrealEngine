// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Mime;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Infrastructure;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Class deriving from a FileResult that allows custom file types (used for zip file creation)
	/// </summary>
	public class CustomFileCallbackResult : FileResult
	{
		private readonly Func<Stream, ActionContext, Task> _callback;
		readonly string _fileName;
		readonly bool _inline;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileName">Default filename for the downloaded file</param>
		/// <param name="mimeType">Content type for the file</param>
		/// <param name="inline">Whether to display the file inline in the browser</param>
		/// <param name="callback">Callback used to write the data</param>
		public CustomFileCallbackResult(string fileName, string mimeType, bool inline, Func<Stream, ActionContext, Task> callback)
			: base(mimeType)
		{
			_fileName = fileName;
			_inline = inline;
			_callback = callback;
		}

		/// <summary>
		/// Executes the action result
		/// </summary>
		/// <param name="context">The controller context</param>
		/// <returns></returns>
		public override Task ExecuteResultAsync(ActionContext context)
		{
			ContentDisposition contentDisposition = new ContentDisposition();
			contentDisposition.Inline = _inline;
			contentDisposition.FileName = _fileName;
			context.HttpContext.Response.Headers["Content-Disposition"] = contentDisposition.ToString();

			CustomFileCallbackResultExecutor executor = new CustomFileCallbackResultExecutor(context.HttpContext.RequestServices.GetRequiredService<ILoggerFactory>());
			return executor.ExecuteAsync(context, this);
		}

		/// <summary>
		/// Exectutor for the custom FileResult
		/// </summary>
		private sealed class CustomFileCallbackResultExecutor : FileResultExecutorBase
		{
			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="loggerFactory">The logger</param>
			public CustomFileCallbackResultExecutor(ILoggerFactory loggerFactory)
				: base(CreateLogger<CustomFileCallbackResultExecutor>(loggerFactory))
			{
			}

			/// <summary>
			/// Executes a CustomFileResult callback
			/// </summary>
			/// <param name="context">The controller context</param>
			/// <param name="result">The custom file result</param>
			/// <returns></returns>
			public Task ExecuteAsync(ActionContext context, CustomFileCallbackResult result)
			{
				SetHeadersAndLog(context, result, null, false);
				return result._callback(context.HttpContext.Response.Body, context);
			}
		}
	}

	/// <summary>
	/// Stream overriding CanSeek to false so the zip file plays nice with it.
	/// </summary>
	public class CustomBufferStream : MemoryStream
	{
		/// <summary>
		/// Always report unseekable.
		/// </summary>
		public override bool CanSeek => false;
	}
}
