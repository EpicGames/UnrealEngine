// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Converter for <see cref="DateTime"/> values to JSON
	/// </summary>
	public class DateTimeJsonConverter : JsonConverter<DateTime>
	{
		/// <inheritdoc/>
		public override DateTime Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			Debug.Assert(typeToConvert == typeof(DateTime));

			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse DateTime");
			}
			return DateTime.Parse(str, CultureInfo.CurrentCulture);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, DateTime dateTime, JsonSerializerOptions options)
		{
			writer.WriteStringValue(dateTime.ToUniversalTime().ToString("yyyy'-'MM'-'dd'T'HH':'mm':'ssZ", CultureInfo.CurrentCulture));
		}
	}
}
