// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Converter for <see cref="DateTimeOffset"/> values to JSON
	/// </summary>
	public class DateTimeOffsetJsonConverter : JsonConverter<DateTimeOffset>
	{
		/// <inheritdoc/>
		public override DateTimeOffset Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			Debug.Assert(typeToConvert == typeof(DateTimeOffset));

			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse DateTimeOffset");
			}
			return DateTimeOffset.Parse(str, CultureInfo.InvariantCulture);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, DateTimeOffset dateTimeOffset, JsonSerializerOptions options)
		{
			writer.WriteStringValue(dateTimeOffset.ToString("o", CultureInfo.InvariantCulture));
		}
	}
}
