// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Filters;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Only requests to attached controller to pass if debug endpoint is enabled in settings
	/// Adds extra security for not enabling these admin endpoints by accident.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class DebugEndpointAttribute : Attribute, IActionFilter
	{
		/// <inheritdoc />
		public void OnActionExecuting(ActionExecutingContext context)
		{
			IServerInfo serverInfo = context.HttpContext.RequestServices.GetRequiredService<IServerInfo>();
			if (!serverInfo.EnableDebugEndpoints)
			{
				context.Result = new ForbidResult();
			}
		}

		/// <inheritdoc />
		public void OnActionExecuted(ActionExecutedContext context)
		{
		}
	}
}
