// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;
using HordeServer.Acls;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Utility class for allowing plugins to modify the default ACL used by the server
	/// </summary>
	public class DefaultAclBuilder
	{
		readonly List<AclEntryConfig> _roles = new List<AclEntryConfig>();
		readonly HashSet<AclAction> _readActions = new HashSet<AclAction>();
		readonly HashSet<AclAction> _writeActions = new HashSet<AclAction>();

		/// <summary>
		/// Adds a custom role with a certain set of entitlements
		/// </summary>
		/// <param name="claim">Claim to identify users that should be granted the entitlements</param>
		/// <param name="actions">Actions to allow the user to perform</param>
		public void AddCustomRole(AclClaimConfig claim, AclAction[] actions)
			=> _roles.Add(new AclEntryConfig(claim, actions));

		/// <summary>
		/// Adds a default read operation that users can perform
		/// </summary>
		public void AddDefaultReadAction(AclAction action)
			=> _readActions.Add(action);

		/// <summary>
		/// Adds a default write operation that users can perform
		/// </summary>
		public void AddDefaultWriteAction(AclAction action)
			=> _writeActions.Add(action);

		/// <summary>
		/// Create the new acl config
		/// </summary>
		public AclConfig Build()
		{
			AclConfig config = new AclConfig();
			config.Entries = _roles.ToList();

			config.Profiles = new List<AclProfileConfig>();
			config.Profiles.Add(new AclProfileConfig
			{
				Id = new AclProfileId("default-read"),
				Actions = _readActions.ToList()
			});

			config.Profiles.Add(new AclProfileConfig
			{
				Id = new AclProfileId("default-run"),
				Extends = new List<AclProfileId>
				{
					new AclProfileId("default-read")
				},
				Actions = _writeActions.ToList()
			});
			return config;
		}
	}
}
