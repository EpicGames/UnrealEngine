// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Users;
using HordeServer.Acls;
using HordeServer.Server;
using HordeServer.Users;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Claim types that are specific to horde
	/// </summary>
	public static class HordeClaimTypes
	{
		/// <summary>
		/// Version number for auth claims. Can be updated to force a re-login.
		/// </summary>
		public const string CurrentVersion = "1";

		/// <summary>
		/// Base URI for all Horde claims.
		/// </summary>
		const string Prefix = "http://epicgames.com/ue/horde/";

		/// <summary>
		/// Version number for the auth header
		/// </summary>
		public const string Version = Prefix + "version";

		/// <summary>
		/// Claim for a particular role. This is reserved for well-known roles defined by Horde itself.
		/// </summary>
		public const string Role = Prefix + "role";

		/// <summary>
		/// Claim type reserved for a particular group within the Horde account system. Values are user defined.
		/// </summary>
		public const string Group = Prefix + "group";

		/// <summary>
		/// Claim for a particular agent
		/// </summary>
		public const string Agent = Prefix + "agent";

		/// <summary>
		/// Claim for an agent's enrollment key
		/// </summary>
		public const string AgentEnrollmentKey = Prefix + "agent-enrollment-key";

		/// <summary>
		/// Claim for a particular account id
		/// </summary>
		public const string AccountId = Prefix + "account-id";

		/// <summary>
		/// Claim for a particular session
		/// </summary>
		public const string AgentSessionId = Prefix + "session";

		/// <summary>
		/// User name for horde
		/// </summary>
		public const string User = Prefix + "user";

		/// <summary>
		/// Unique id of the horde user (see <see cref="IUserCollection"/>)
		/// </summary>
		public const string UserId = Prefix + "user-id-v3";

		/// <summary>
		/// Claim for downloading artifacts from a particular job
		/// </summary>
		public const string JobArtifacts = Prefix + "job-artifacts";

		/// <summary>
		/// Claim for the Perforce username
		/// </summary>
		public const string PerforceUser = Prefix + "perforce-user";

		/// <summary>
		/// Claim for the external issue username
		/// </summary>
		public const string ExternalIssueUser = Prefix + "external-issue-user";

		/// <summary>
		/// Claim identifying an agent as executing a particular lease
		/// </summary>
		public const string Lease = Prefix + "lease";

		/// <summary>
		/// Claim identifying an agent as executing a lease in a particular stream
		/// </summary>
		public const string LeaseStream = Prefix + "lease-stream";

		/// <summary>
		/// Claim identifying an agent as executing a lease in a particular project
		/// </summary>
		public const string LeaseProject = Prefix + "lease-project";

		/// <summary>
		/// Claim identifying an agent as executing a lease in a particular job template
		/// </summary>
		public const string LeaseTemplate = Prefix + "lease-template";

		/// <summary>
		/// Claim allowing a certain namespace and/or path to be read from. See <see cref="WriteNamespace"/>.
		/// </summary>
		public const string ReadNamespace = Prefix + "read-namespace";

		/// <summary>
		/// Claim allowing a certain namespace and/or path to be written to. Value may be a namespace name, or a subpath (eg. ns:a/b will only allow blobs and refs with an a/b prefix to be written).
		/// </summary>
		public const string WriteNamespace = Prefix + "write-namespace";
	}

	/// <summary>
	/// Extension methods for getting Horde claims from a principal
	/// </summary>
	public static class HordeClaimExtensions
	{
		/// <summary>
		/// Gets the Horde account id from a principal
		/// </summary>
		/// <param name="principal"></param>
		/// <returns></returns>
		public static AccountId? GetAccountId(this ClaimsPrincipal principal)
		{
			string? idValue = principal.FindFirstValue(HordeClaimTypes.AccountId);
			return idValue == null ? null : AccountId.Parse(idValue);
		}

		/// <summary>
		/// Gets the Horde user ID from a principal
		/// </summary>
		/// <param name="principal"></param>
		/// <returns></returns>
		public static UserId? GetUserId(this ClaimsPrincipal principal)
		{
			string? idValue = principal.FindFirstValue(HordeClaimTypes.UserId);
			return idValue == null ? null : UserId.Parse(idValue);
		}

		/// <summary>
		/// Determines whether the given user can masquerade as a given user
		/// </summary>
		public static bool AuthorizeAsUser(this AclConfig aclConfig, ClaimsPrincipal user, UserId userId)
		{
			UserId? currentUserId = user.GetUserId();
			if (currentUserId != null && currentUserId.Value == userId)
			{
				return true;
			}
			else
			{
				return aclConfig.Authorize(ServerAclAction.Impersonate, user);
			}
		}

		/// <summary>
		/// Gets the Horde username from a principal
		/// </summary>
		/// <param name="principal"></param>
		/// <returns></returns>
		public static string? GetUser(this ClaimsPrincipal principal)
		{
			return principal.FindFirstValue(HordeClaimTypes.User);
		}

		/// <summary>
		/// Gets the external issue username for the given principal
		/// </summary>
		/// <param name="user">The principal to get the external issue user for</param>
		/// <returns>Jira user name</returns>
		public static string? GetExternalIssueUser(this ClaimsPrincipal user)
		{
			Claim? claim = user.FindFirst(HordeClaimTypes.ExternalIssueUser);
			return claim?.Value;
		}

		/// <summary>
		/// Determine if a user has the administrator claim
		/// </summary>
		/// <param name="user">The principal to get the external issue user for</param>
		public static bool HasAdminClaim(this ClaimsPrincipal user)
		{
			return user.HasClaim(HordeClaims.AdminClaim.Type, HordeClaims.AdminClaim.Value);
		}
	}
}
