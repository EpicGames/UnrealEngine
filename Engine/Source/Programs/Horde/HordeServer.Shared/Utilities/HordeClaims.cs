// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Sessions;
using HordeServer.Acls;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Predetermined claim values
	/// </summary>
	public static class HordeClaims
	{
		/// <summary>
		/// Claim which gives unfettered access to all Horde endpoints
		/// </summary>
		public static AclClaimConfig AdminClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "admin");

		/// <summary>
		/// Name of the role that can be used to administer agents
		/// </summary>
		public static AclClaimConfig AgentRegistrationClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "agent-registration");

		/// <summary>
		/// Name of the role used to upload software
		/// </summary>
		public static AclClaimConfig DownloadSoftwareClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "download-software");

		/// <summary>
		/// Name of the role used to upload tools
		/// </summary>
		public static AclClaimConfig UploadToolsClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "upload-software");

		/// <summary>
		/// Name of the role used to upload software
		/// </summary>
		public static AclClaimConfig ConfigureProjectsClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "configure-projects");

		/// <summary>
		/// Name of the role used to upload software
		/// </summary>
		public static AclClaimConfig StartChainedJobClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "start-chained-job");

		/// <summary>
		/// Role for dedicated agents
		/// </summary>
		public static AclClaimConfig AgentDedicatedRoleClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "agent");
		
		/// <summary>
		/// Role for workstation agents
		/// </summary>
		public static AclClaimConfig AgentWorkstationRoleClaim { get; } = new AclClaimConfig(HordeClaimTypes.Role, "agent-workstation");

		/// <summary>
		/// Gets the role for a specific agent
		/// </summary>
		/// <param name="agentId">The session id</param>
		/// <returns>New claim instance</returns>
		public static AclClaimConfig GetAgentClaim(AgentId agentId)
		{
			return new AclClaimConfig(HordeClaimTypes.Agent, agentId.ToString());
		}

		/// <summary>
		/// Gets the role for a specific agent session
		/// </summary>
		/// <param name="sessionId">The session id</param>
		/// <returns>New claim instance</returns>
		public static AclClaimConfig GetSessionClaim(SessionId sessionId)
		{
			return new AclClaimConfig(HordeClaimTypes.AgentSessionId, sessionId.ToString());
		}
	}
}
