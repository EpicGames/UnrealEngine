// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Symbols;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Tools;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Base class for Horde controllers
	/// </summary>
	public abstract class HordeControllerBase : ControllerBase
	{
		/// <summary>
		/// Create a response indicating a problem with the request
		/// </summary>
		/// <param name="message">Standard structured-logging format string</param>
		/// <param name="args">Arguments for the format string</param>
		[NonAction]
		protected ActionResult BadRequest(string message, params object[] args)
		{
			return BadRequest(LogEvent.Create(LogLevel.Error, message, args));
		}

		/// <summary>
		/// Create a response indicating a problem with the request
		/// </summary>
		/// <param name="eventId">Id identifying the error</param>
		/// <param name="message">Standard structured-logging format string</param>
		/// <param name="args">Arguments for the format string</param>
		[NonAction]
		protected ActionResult BadRequest(EventId eventId, string message, params object[] args)
		{
			return BadRequest(LogEvent.Create(LogLevel.Error, eventId, message, args));
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action)
		{
			return Forbid("User does not have {Action} permission", action);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, AgentId agentId)
		{
			return Forbid(action, "agent {AgentId}", agentId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, AclScopeName scopeName)
		{
			return Forbid(action, "scope {ScopeName}", scopeName);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, ClusterId clusterId)
		{
			return Forbid(action, "cluster {ClusterId}", clusterId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, JobId jobId)
		{
			return Forbid(action, "job {JobId}", jobId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, NamespaceId namespaceId)
		{
			return Forbid(action, "namespace {NamespaceId}", namespaceId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, ProjectId projectId)
		{
			return Forbid(action, "project {ProjectId}", projectId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, SecretId secretId)
		{
			return Forbid(action, "secret {SecretId}", secretId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, StreamId streamId)
		{
			return Forbid(action, "stream {StreamId}", streamId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, SymbolStoreId storeId)
		{
			return Forbid(action, "symbol store {StoreId}", storeId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, ToolId toolId)
		{
			return Forbid(action, "tool {ToolId}", toolId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(AclAction action, TemplateId templateId)
		{
			return Forbid(action, "template {TemplateId}", templateId);
		}

		/// <summary>
		/// Returns a 403 (forbidden) response with the given action and object
		/// </summary>
		[NonAction]
		private ActionResult Forbid(AclAction action, string objectMessage, object obj)
		{
			return Forbid($"User does not have {{Action}} permission for {objectMessage}", action, obj);
		}

		/// <summary>
		/// Returns a 403 response with the given log event
		/// </summary>
		[NonAction]
		protected ActionResult Forbid(string message, params object[] args)
		{
			return StatusCode(StatusCodes.Status403Forbidden, LogEvent.Create(LogLevel.Error, message, args));
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(AccountId accountId)
		{
			return NotFound("Account {AccountId} not found", accountId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(AgentId agentId)
		{
			return NotFound("Agent {AgentId} not found", agentId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(AgentId agentId, LeaseId leaseId)
		{
			return NotFound("Lease {LeaseId} not found for agent {AgentId}", leaseId, agentId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(ArtifactId artifactId)
		{
			return NotFound("Artifact {ArtifactId} not found", artifactId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(ClusterId clusterId)
		{
			return NotFound("Cluster {ClusterId} not found", clusterId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(JobId jobId)
		{
			return NotFound("Job {JobId} not found", jobId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(JobId jobId, int groupIdx)
		{
			return NotFound("Group {GroupIdx} on job {JobId} not found", groupIdx, jobId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(JobId jobId, int groupIdx, int nodeIdx)
		{
			return NotFound("Node {NodeIdx} not found on job {JobId} group {GroupIdx}", nodeIdx, jobId, groupIdx);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(JobId jobId, JobStepBatchId batchId)
		{
			return NotFound("Batch {BatchId} not found on job {JobId}", batchId, jobId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(JobId jobId, JobStepBatchId batchId, JobStepId stepId)
		{
			return NotFound("Step {StepId} not found on job {JobId} batch {BatchId}", stepId, jobId, batchId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(NamespaceId namespaceId)
		{
			return NotFound("Namespace {NamespaceId} not found", namespaceId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(NamespaceId namespaceId, RefName refName)
		{
			return NotFound("Ref {RefName} in namespace {NamespaceId} not found", refName, namespaceId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(BisectTaskId bisectTaskId)
		{
			return NotFound("Bisect task {BisectTaskId} not found", bisectTaskId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(SecretId secretId)
		{
			return NotFound("Secret {SecretId} not found", secretId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(StreamId streamId)
		{
			return NotFound("Stream {StreamId} not found", streamId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(StreamId streamId, TemplateId templateId)
		{
			return NotFound("Template {TemplateId} not found on stream {StreamId}", templateId, streamId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(TelemetryStoreId telemetryStoreId)
		{
			return NotFound("Telemetry store {TelemetryStoreId} not found", telemetryStoreId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(ToolId toolId)
		{
			return NotFound("Tool {ToolId} not found", toolId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(ToolId toolId, ToolDeploymentId deploymentId)
		{
			return NotFound("Deployment {DeploymentId} not found on tool {ToolId}", deploymentId, toolId);
		}

		/// <summary>
		/// Returns a 404 response for the given object
		/// </summary>
		[NonAction]
		protected ActionResult NotFound(string message, params object[] args)
		{
			return NotFound(LogEvent.Create(LogLevel.Error, message, args));
		}
	}
}
