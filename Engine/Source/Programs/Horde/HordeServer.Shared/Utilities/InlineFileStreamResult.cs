// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Mime;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Class to return a file stream without the "content-disposition: attachment" header
	/// </summary>
	public class InlineFileStreamResult : FileStreamResult
	{
		/// <summary>
		/// The suggested download filename
		/// </summary>
		readonly string _fileName;

		/// <summary>
		/// Constructor
		/// </summary>
		public InlineFileStreamResult(System.IO.Stream stream, string mimeType, string fileName)
			: base(stream, mimeType)
		{
			_fileName = fileName;
		}

		/// <inheritdoc/>
		public override Task ExecuteResultAsync(ActionContext context)
		{
			ContentDisposition contentDisposition = new ContentDisposition();
			contentDisposition.Inline = true;
			contentDisposition.FileName = _fileName;
			context.HttpContext.Response.Headers["Content-Disposition"] = contentDisposition.ToString();

			return base.ExecuteResultAsync(context);
		}
	}
}
