// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Core;

namespace HordeServer.Utilities
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Allows manipulating JSON config files without fully deserializing them
	/// </summary>
	public class JsonConfigFile
	{
		/// <summary>
		/// The root of the config file
		/// </summary>
		public JsonObject Root { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonConfigFile(JsonObject root)
		{
			Root = root;
		}

		/// <summary>
		/// Reads a json config file from a file on disk
		/// </summary>
		public static async Task<JsonConfigFile> ReadAsync(FileReference file, CancellationToken cancellationToken = default)
		{
			byte[] data = await FileReference.ReadAllBytesAsync(file, cancellationToken);
			JsonObject? obj = JsonNode.Parse(data, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip }) as JsonObject;
			return new JsonConfigFile(obj ?? new JsonObject());
		}

		/// <summary>
		/// Writes the config file back out to a file
		/// </summary>
		public async Task WriteAsync(FileReference file, CancellationToken cancellationToken = default)
		{
			ArrayBufferWriter<byte> buffer = new ArrayBufferWriter<byte>();
			using (Utf8JsonWriter writer = new Utf8JsonWriter(buffer, new JsonWriterOptions { Indented = true }))
			{
				Root.WriteTo(writer);
			}
			await FileReference.WriteAllBytesAsync(file, buffer.WrittenMemory.ToArray(), cancellationToken);
		}

		/// <summary>
		/// Adds a node with the given name to an object
		/// </summary>
		public static T FindOrAddNode<T>(JsonObject obj, string name, Func<T> factory) where T : JsonNode
		{
			JsonNode? node = obj[name];
			if (node != null)
			{
				if (node is T existingTypedNode)
				{
					return existingTypedNode;
				}
				else
				{
					obj.Remove(name);
				}
			}

			T newTypedNode = factory();
			obj.Add(name, newTypedNode);
			return newTypedNode;
		}

		/// <summary>
		/// Finds an existing element, or adds a new element to an array, with the given name.
		/// </summary>
		public static JsonObject FindOrAddElementByKey(JsonArray array, string key, string name)
		{
			foreach (JsonNode? element in array)
			{
				if (element is JsonObject obj)
				{
					JsonNode? node = obj[key];
					if (node != null && (string?)node.AsValue() == name)
					{
						return obj;
					}
				}
			}

			JsonObject newObj = new JsonObject();
			newObj[key] = name;
			array.Add(newObj);
			return newObj;
		}
	}
}
