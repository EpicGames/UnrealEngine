// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Jobs;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Helper methods for json serialization
	/// </summary>
	public static class JsonUtils
	{
		/// <summary>
		/// Default JSON serialization options
		/// </summary>
		public static JsonSerializerOptions DefaultSerializerOptions { get; } = CreateDefaultJsonSerializerOptions();

		static JsonSerializerOptions CreateDefaultJsonSerializerOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(options);
			options.MakeReadOnly(true);
			return options;
		}

		/// <summary>
		/// Configure a json serializer with all standard Horde converters
		/// </summary>
		public static void ConfigureJsonSerializer(JsonSerializerOptions options)
		{
			HordeHttpClient.ConfigureJsonSerializer(options);
			options.Converters.Add(new ObjectIdJsonConverter());
			options.Converters.Add(new ObjectIdJsonConverterFactory());
			options.Converters.Add(new JsonKnownTypesConverterFactory());
			options.Converters.Add(new SubResourceIdJsonConverterFactory());
			options.Converters.Add(new DateTimeJsonConverter());
			options.Converters.Add(new TimeSpanJsonConverter());
			options.Converters.Add(new DateTimeOffsetJsonConverter());
		}
	}
}
