// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using MongoDB.Driver;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Utility class for buffering document writes to a Mongo collection
	/// </summary>
	public sealed class MongoBufferedWriter<TDocument> : IAsyncDisposable
		where TDocument : class
	{
		readonly IMongoCollection<TDocument> _collection;
		readonly int _flushCount;
		readonly TimeSpan _flushTime;
		readonly ConcurrentQueue<TDocument> _queue = new ConcurrentQueue<TDocument>();
		readonly BackgroundTask _backgroundTask;
		readonly AsyncEvent _newDataEvent = new AsyncEvent();
		readonly AsyncEvent _flushEvent = new AsyncEvent();
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public MongoBufferedWriter(IMongoCollection<TDocument> collection, ILogger logger)
			: this(collection, 50, TimeSpan.FromSeconds(5.0), logger)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public MongoBufferedWriter(IMongoCollection<TDocument> collection, int flushCount, TimeSpan flushTime, ILogger logger)
		{
			_collection = collection;
			_flushCount = flushCount;
			_flushTime = flushTime;
			_backgroundTask = new BackgroundTask(BackgroundFlushAsync);
			_logger = logger;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
		}

		/// <summary>
		/// Start the background task to periodically flush data to the DB
		/// </summary>
		public ValueTask StartAsync()
		{
			_backgroundTask.Start();
			return new ValueTask();
		}

		/// <summary>
		/// Stops the background task
		/// </summary>
		public async ValueTask StopAsync(CancellationToken cancellationToken)
		{
			await _backgroundTask.StopAsync(cancellationToken);
			await FlushAsync(cancellationToken);
		}

		// Flushes the sink in the background
		async Task BackgroundFlushAsync(CancellationToken cancellationToken)
		{
			Task newDataTask = _newDataEvent.Task;
			Task flushTask = _flushEvent.Task;

			while (!cancellationToken.IsCancellationRequested)
			{
				try
				{
					await newDataTask.WaitAsync(cancellationToken);
					await Task.WhenAny(flushTask, Task.Delay(_flushTime, cancellationToken));

					newDataTask = _newDataEvent.Task;
					flushTask = _flushEvent.Task;

					await FlushAsync(cancellationToken);
				}
				catch (OperationCanceledException)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception in buffered writer: {Message}", ex.Message);
					await Task.Delay(TimeSpan.FromSeconds(30.0), cancellationToken);
				}
			}
		}

		/// <inheritdoc/>
		public async ValueTask FlushAsync(CancellationToken cancellationToken)
		{
			// Copy all the event documents from the queue
			List<TDocument> documents = new List<TDocument>(_queue.Count);
			while (_queue.TryDequeue(out TDocument? document))
			{
				documents.Add(document);
			}

			// Insert them into the database
			if (documents.Count > 0)
			{
				_logger.LogDebug("Writing {NumEvents} new telemetry events to {CollectionName}.", documents.Count, _collection.CollectionNamespace.CollectionName);
				await _collection.InsertManyAsync(documents, cancellationToken: cancellationToken);
			}
		}

		/// <inheritdoc/>
		public void Write(TDocument document)
		{
			_queue.Enqueue(document);
			_newDataEvent.Set();

			if (_queue.Count > _flushCount)
			{
				_flushEvent.Set();
			}
		}
	}
}
