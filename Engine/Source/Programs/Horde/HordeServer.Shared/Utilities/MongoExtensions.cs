// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq.Expressions;
using System.Runtime.CompilerServices;
using System.Text.Json;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Driver;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for MongoDB
	/// </summary>
	public static class MongoExtensions
	{
		/// <summary>
		/// Maps a constructor for a type into a classmap
		/// </summary>
		public static void MapConstructor<T>(this BsonClassMap classMap, Expression<Func<T>> generator, params string[] argumentNames)
		{
			NewExpression newExpr = (NewExpression)generator.Body;
			classMap.MapConstructor(newExpr.Constructor, argumentNames);
		}

		/// <summary>
		/// Rounds a time value to its BSON equivalent (ie. milliseconds since Unix Epoch).
		/// </summary>
		/// <param name="time"></param>
		/// <returns></returns>
		public static DateTime RoundToBsonDateTime(DateTime time)
		{
			return BsonUtils.ToDateTimeFromMillisecondsSinceEpoch(BsonUtils.ToMillisecondsSinceEpoch(time));
		}

		/// <summary>
		/// Renders a filter definition to a document using the default serializer registry
		/// </summary>
		public static BsonDocument Render<T>(this FilterDefinition<T> filter)
		{
			IBsonSerializerRegistry serializerRegistry = BsonSerializer.SerializerRegistry;
			IBsonSerializer<T> documentSerializer = serializerRegistry.GetSerializer<T>();
			return filter.Render(documentSerializer, serializerRegistry, MongoDB.Driver.Linq.LinqProvider.V2);
		}

		/// <summary>
		/// Renders a filter definition to a document using the default serializer registry
		/// </summary>
		public static BsonValue Render<T>(this UpdateDefinition<T> update)
		{
			IBsonSerializerRegistry serializerRegistry = BsonSerializer.SerializerRegistry;
			IBsonSerializer<T> documentSerializer = serializerRegistry.GetSerializer<T>();
			return update.Render(documentSerializer, serializerRegistry, MongoDB.Driver.Linq.LinqProvider.V2);
		}

		/// <summary>
		/// Executes a query with a particular index hint
		/// </summary>
		/// <typeparam name="TDoc"></typeparam>
		/// <typeparam name="TResult"></typeparam>
		/// <param name="collection"></param>
		/// <param name="filter"></param>
		/// <param name="indexHint"></param>
		/// <param name="processAsync"></param>
		/// <returns></returns>
		public static async Task<TResult> FindWithHintAsync<TDoc, TResult>(this IMongoCollection<TDoc> collection, FilterDefinition<TDoc> filter, string? indexHint, Func<IFindFluent<TDoc, TDoc>, Task<TResult>> processAsync)
		{
			FindOptions? findOptions = null;
			if (indexHint != null)
			{
				findOptions = new FindOptions { Hint = new BsonString(indexHint) };
			}
			try
			{
				return await processAsync(collection.Find(filter, findOptions));
			}
			catch (MongoCommandException ex) when (indexHint != null && ex.Code == 2)
			{
				return await processAsync(collection.Find(filter));
			}
		}

		/// <summary>
		/// Filters the documents returned from a search
		/// </summary>
		/// <param name="query">The query to filter</param>
		/// <param name="index">Index of the first document to return</param>
		/// <param name="count">Number of documents to return</param>
		/// <returns>New query</returns>
		public static IFindFluent<TDocument, TProjection> Range<TDocument, TProjection>(this IFindFluent<TDocument, TProjection> query, int? index, int? count)
		{
			if (index != null && index.Value != 0)
			{
				query = query.Skip(index.Value);
			}
			if (count != null)
			{
				query = query.Limit(count.Value);
			}
			return query;
		}

		/// <summary>
		/// Filters the documents returned from a search
		/// </summary>
		/// <param name="query">The query to filter</param>
		/// <param name="index">Index of the first document to return</param>
		/// <param name="count">Number of documents to return</param>
		/// <returns>New query</returns>
		public static IAggregateFluent<TDocument> Range<TDocument>(this IAggregateFluent<TDocument> query, int? index, int? count)
		{
			if (index != null && index.Value != 0)
			{
				query = query.Skip(index.Value);
			}
			if (count != null)
			{
				query = query.Limit(count.Value);
			}
			return query;
		}

		/// <summary>
		/// Attempts to insert a document into a collection, handling the error case that a document with the given key already exists
		/// </summary>
		/// <typeparam name="TDocument"></typeparam>
		/// <param name="collection">Collection to insert into</param>
		/// <param name="newDocument">The document to insert</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the document was inserted, false if it already exists</returns>
		public static async Task<bool> InsertOneIgnoreDuplicatesAsync<TDocument>(this IMongoCollection<TDocument> collection, TDocument newDocument, CancellationToken cancellationToken = default)
		{
			try
			{
				await collection.InsertOneAsync(newDocument, cancellationToken: cancellationToken);
				return true;
			}
			catch (MongoWriteException ex)
			{
				if (ex.WriteError.Category == ServerErrorCategory.DuplicateKey)
				{
					return false;
				}
				else
				{
					throw;
				}
			}
		}

		/// <summary>
		/// Attempts to insert a document into a collection, handling the error case that a document with the given key already exists
		/// </summary>
		/// <typeparam name="TDocument"></typeparam>
		/// <param name="collection">Collection to insert into</param>
		/// <param name="newDocuments">The document to insert</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public static async Task InsertManyIgnoreDuplicatesAsync<TDocument>(this IMongoCollection<TDocument> collection, List<TDocument> newDocuments, CancellationToken cancellationToken = default)
		{
			try
			{
				if (newDocuments.Count > 0)
				{
					await collection.InsertManyAsync(newDocuments, new InsertManyOptions { IsOrdered = false }, cancellationToken);
				}
			}
			catch (MongoWriteException ex)
			{
				if (ex.WriteError.Category != ServerErrorCategory.DuplicateKey)
				{
					throw;
				}
			}
			catch (MongoBulkWriteException ex)
			{
				if (ex.WriteErrors.Any(x => x.Category != ServerErrorCategory.DuplicateKey))
				{
					throw;
				}
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <typeparam name="TField">Type of the field to set</typeparam>
		/// <param name="updateBuilder">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetNull<TDocument, TField>(this UpdateDefinitionBuilder<TDocument> updateBuilder, Expression<Func<TDocument, TField?>> field, TField? value) where TField : struct
		{
			if (value.HasValue)
			{
				return updateBuilder.Set(field, value.Value);
			}
			else
			{
				return updateBuilder.Unset(new ExpressionFieldDefinition<TDocument, TField?>(field));
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <typeparam name="TField">Type of the field to set</typeparam>
		/// <param name="update">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetNull<TDocument, TField>(this UpdateDefinition<TDocument> update, Expression<Func<TDocument, TField?>> field, TField? value) where TField : struct
		{
			if (value.HasValue)
			{
				return update.Set(field, value.Value);
			}
			else
			{
				return update.Unset(new ExpressionFieldDefinition<TDocument, TField?>(field));
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <param name="update">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetBool<TDocument>(this UpdateDefinition<TDocument> update, Expression<Func<TDocument, bool>> field, bool value)
		{
			if (value)
			{
				return update.Set(field, value);
			}
			else
			{
				return update.Unset(new ExpressionFieldDefinition<TDocument>(field));
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <param name="updateBuilder">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetBool<TDocument>(this UpdateDefinitionBuilder<TDocument> updateBuilder, Expression<Func<TDocument, bool>> field, bool value)
		{
			if (value)
			{
				return updateBuilder.Set(field, value);
			}
			else
			{
				return updateBuilder.Unset(new ExpressionFieldDefinition<TDocument>(field));
			}
		}

		/// <summary>
		/// Sets a field to a value, or unsets it if the value is null
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <typeparam name="TField">Type of the field to set</typeparam>
		/// <param name="updateBuilder">Update builder</param>
		/// <param name="field">Expression for the field to set</param>
		/// <param name="value">New value to set</param>
		/// <returns>Update defintiion</returns>
		public static UpdateDefinition<TDocument> SetOrUnsetNullRef<TDocument, TField>(this UpdateDefinitionBuilder<TDocument> updateBuilder, Expression<Func<TDocument, TField?>> field, TField? value) where TField : class
		{
			if (value != null)
			{
				return updateBuilder.Set(field, value);
			}
			else
			{
				return updateBuilder.Unset(new ExpressionFieldDefinition<TDocument, TField?>(field));
			}
		}

		/// <summary>
		/// Creates a filter definition from a linq expression. This is not generally explicitly castable, so expose it as a FilterDefinitionBuilder method.
		/// </summary>
		/// <typeparam name="TDocument">The document type</typeparam>
		/// <param name="filter">The filter builder</param>
		/// <param name="expression">Expression to parse</param>
		/// <returns>New filter definition</returns>
#pragma warning disable IDE0060
		public static FilterDefinition<TDocument> Expr<TDocument>(this FilterDefinitionBuilder<TDocument> filter, Expression<Func<TDocument, bool>> expression)
		{
			return expression;
		}
#pragma warning restore IDE0060

		/// <summary>
		/// Creates an async enumerator from the given query
		/// </summary>
		/// <typeparam name="T">Document type</typeparam>
		/// <param name="source">Query source</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
#pragma warning disable IDE1006 // Naming Styles
#pragma warning disable VSTHRD200 // Use Async suffix
		public static async IAsyncEnumerable<T> ToAsyncEnumerable<T>(this IAsyncCursorSource<T> source, [EnumeratorCancellation] CancellationToken cancellationToken)
#pragma warning restore VSTHRD200 // Use Async suffix
#pragma warning restore IDE1006 // Naming Styles
		{
			using (IAsyncCursor<T> cursor = await source.ToCursorAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (T value in cursor.Current)
					{
						yield return value;
						cancellationToken.ThrowIfCancellationRequested();
					}
				}
			}
		}

		/// <summary>
		/// Converts a JsonElement to a BsonValue
		/// </summary>
		/// <param name="element">The element to convert</param>
		/// <returns>Bson value</returns>
		public static BsonValue ToBsonValue(this JsonElement element)
		{
			switch (element.ValueKind)
			{
				case JsonValueKind.True:
					return true;
				case JsonValueKind.False:
					return false;
				case JsonValueKind.String:
					return element.GetString();
				case JsonValueKind.Array:
					return new BsonArray(element.EnumerateArray().Select(x => x.ToBsonValue()));
				case JsonValueKind.Object:
					return new BsonDocument(element.EnumerateObject().Select(x => new BsonElement(x.Name, x.Value.ToBsonValue())));
				default:
					return BsonNull.Value;
			}
		}
	}
}
