// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Globalization;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Serializers;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Base class for converting to and from types containing an object id. Useful pattern for reducing boilerplate with strongly typed records.
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public abstract class ObjectIdConverter<T> where T : struct
	{
		/// <summary>
		/// Converts a type to a <see cref="ObjectId"/>
		/// </summary>
		public abstract ObjectId ToObjectId(T value);

		/// <summary>
		/// Constructs a type from an object id
		/// </summary>
		public abstract T FromObjectId(ObjectId id);
	}

	/// <summary>
	/// Attribute declaring an object id converter for a particular type
	/// </summary>
	[AttributeUsage(AttributeTargets.Struct)]
	public sealed class ObjectIdConverterAttribute : Attribute
	{
		/// <summary>
		/// The converter type
		/// </summary>
		public Type ConverterType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ObjectIdConverterAttribute(Type converterType) => ConverterType = converterType;
	}

	/// <summary>
	/// Class which serializes types with a <see cref="ObjectIdConverter{T}"/> to Json
	/// </summary>
	public sealed class ObjectIdTypeConverter<TValue, TConverter> : TypeConverter where TValue : struct where TConverter : ObjectIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object? ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			if (value is string str)
			{
				return _converter.FromObjectId(ObjectId.Parse(str));
			}
			return null;
		}

		/// <inheritdoc/>
		public override bool CanConvertTo(ITypeDescriptorContext? context, Type? destinationType) => destinationType == typeof(string);

		/// <inheritdoc/>
		public override object? ConvertTo(ITypeDescriptorContext? context, CultureInfo? culture, object? value, Type destinationType)
		{
			if (destinationType == typeof(string))
			{
				return _converter.ToObjectId((TValue)value!).ToString();
			}
			return null;
		}
	}

	/// <summary>
	/// Class which serializes ObjectId types to Json
	/// </summary>
	public class ObjectIdJsonConverter : JsonConverter<ObjectId>
	{
		/// <inheritdoc/>
		public override ObjectId Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse object id");
			}
			if (str.Length == 0)
			{
				return ObjectId.Empty;
			}
			return ObjectId.Parse(str);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, ObjectId objectId, JsonSerializerOptions options)
		{
			writer.WriteStringValue(objectId.ToString());
		}
	}

	/// <summary>
	/// Class which serializes types with a <see cref="ObjectIdConverter{T}"/> to Json
	/// </summary>
	public sealed class ObjectIdJsonConverter<TValue, TConverter> : JsonConverter<TValue> where TValue : struct where TConverter : ObjectIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			=> _converter.FromObjectId(ObjectId.Parse(reader.GetString()));

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TValue value, JsonSerializerOptions options)
			=> writer.WriteStringValue(_converter.ToObjectId(value).ToString());
	}

	/// <summary>
	/// Creates constructors for types with a <see cref="ObjectIdConverter{T}"/> to Json
	/// </summary>
	public sealed class ObjectIdJsonConverterFactory : JsonConverterFactory
	{
		/// <inheritdoc/>
		public override bool CanConvert(Type typeToConvert) => typeToConvert.GetCustomAttribute<ObjectIdConverterAttribute>() != null;

		/// <inheritdoc/>
		public override JsonConverter? CreateConverter(Type type, JsonSerializerOptions options)
		{
			ObjectIdConverterAttribute? attribute = type.GetCustomAttribute<ObjectIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (JsonConverter?)Activator.CreateInstance(typeof(ObjectIdJsonConverter<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}

	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class ObjectIdBsonSerializer<TValue, TConverter> : SerializerBase<TValue> where TValue : struct where TConverter : ObjectIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			=> _converter.FromObjectId(context.Reader.ReadObjectId());

		/// <inheritdoc/>
		public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, TValue value) => context.Writer.WriteObjectId(_converter.ToObjectId(value));
	}

	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class ObjectIdBsonSerializationProvider : BsonSerializationProviderBase
	{
		/// <inheritdoc/>
		public override IBsonSerializer? GetSerializer(Type type, IBsonSerializerRegistry serializerRegistry)
		{
			ObjectIdConverterAttribute? attribute = type.GetCustomAttribute<ObjectIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (IBsonSerializer?)Activator.CreateInstance(typeof(ObjectIdBsonSerializer<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}
}
