// Copyright Epic Games, Inc. All Rights Reserved.

using OpenTelemetry.Trace;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extensions to handle Horde specific data types in the OpenTelemetry library
	/// </summary>
	public static class OpenTelemetryExtensions
	{
		/// <summary>Set a key:value tag on the span</summary>
		/// <returns>This span instance, for chaining</returns>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, int? value)
		{
			if (value != null)
			{
				span.SetAttribute(key, value.Value);
			}
			return span;
		}

		/// <summary>Set a key:value tag on the span</summary>
		/// <returns>This span instance, for chaining</returns>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, DateTimeOffset? value)
		{
			if (value != null)
			{
				span.SetAttribute(key, value.ToString());
			}
			return span;
		}
	}
}
