// Copyright Epic Games, Inc. All Rights Reserved.

using OpenTelemetry.Trace;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Static initialization of all available OpenTelemetry tracers
	/// </summary>
	public static class OpenTelemetryTracers
	{
		/// <summary>
		/// Name of resource name attribute in Datadog
		/// Some traces use this for prettier display inside their UI
		/// </summary>
		public const string DatadogResourceAttribute = "resource.name";

		/// <summary>
		/// Name of default Horde tracer (aka activity source)
		/// </summary>
		public const string HordeName = "Horde";

		/// <summary>
		/// Name of MongoDB tracer (aka activity source)
		/// </summary>
		public const string MongoDbName = "MongoDB";
		
		/// <summary>
		/// Name of Perforce client tracer (aka activity source)
		/// By default, tracer is declared inside EpicGames.Perforce.
		/// </summary>
		public const string PerforceName = "Perforce";

		/// <summary>
		/// List of all source names configured in this class.
		/// They are needed at startup when initializing OpenTelemetry
		/// </summary>
		public static string[] SourceNames => [HordeName, MongoDbName, PerforceName];

		/// <summary>
		/// Default tracer used in Horde
		/// Prefer dependency-injected tracer over this static member.
		/// </summary>
		public static readonly Tracer Horde = TracerProvider.Default.GetTracer(HordeName);

		/// <summary>
		/// Tracer specific to MongoDB
		/// Prefer StartMongoDbSpan static extension for Tracer.
		/// </summary>
		public static readonly Tracer MongoDb = TracerProvider.Default.GetTracer(MongoDbName);
	}
}
