// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections;
using System.ComponentModel;
using System.Globalization;
using System.Reflection;
using System.Text.Json.Nodes;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Utility class to filter a response to include a subset of fields
	/// </summary>
	[TypeConverter(typeof(PropertyFilterTypeConverter))]
	public class PropertyFilter
	{
		/// <summary>
		/// Specific filters for properties of this object. A key of '*' will match any property not matched by anything else, and a value of null will include the whole object.
		/// </summary>
		readonly Dictionary<string, PropertyFilter?> _nameToFilter = new Dictionary<string, PropertyFilter?>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Parse a list of fields as a property filter
		/// </summary>
		/// <param name="fields">List of fields to parse, separated by commas</param>
		/// <returns>New property filter</returns>
		public static PropertyFilter Parse(string? fields)
		{
			PropertyFilter rootFilter = new PropertyFilter();
			if (fields != null)
			{
				foreach (string field in fields.Split(','))
				{
					// Split the field name into segments
					string[] segments = field.Split('.');

					// Create the tree of segments
					Dictionary<string, PropertyFilter?> nameToFilter = rootFilter._nameToFilter;
					for (int idx = 0; idx < segments.Length - 1; idx++)
					{
						PropertyFilter? nextFilter;
						if (!nameToFilter.TryGetValue(segments[idx], out nextFilter))
						{
							nextFilter = new PropertyFilter();
							nameToFilter[segments[idx]] = nextFilter;
						}
						else if (nextFilter == null)
						{
							break;
						}
						nameToFilter = nextFilter._nameToFilter;
					}
					nameToFilter[segments[^1]] = null;
				}
			}
			return rootFilter;
		}

		/// <summary>
		/// Checks whether the filter contains the given field
		/// </summary>
		/// <param name="filter">The filter to check</param>
		/// <param name="field">Name of the field to check for</param>
		/// <returns>True if the filter includes the given field</returns>
		public static bool Includes(PropertyFilter? filter, string field)
		{
			return filter == null || filter.Includes(field);
		}

		/// <summary>
		/// Attempts to apply a property filter to a response object, where the property filter may be null.
		/// </summary>
		/// <param name="response">Response object</param>
		/// <param name="filter">The filter to apply</param>
		/// <returns>Filtered object</returns>
		public static object Apply(object response, PropertyFilter? filter)
		{
			return (filter == null) ? response : filter.ApplyTo(response);
		}

		/// <summary>
		/// Checks whether the filter contains the given field
		/// </summary>
		/// <param name="field">Name of the field to check for</param>
		/// <returns>True if the filter includes the given field</returns>
		public bool Includes(string field)
		{
			return _nameToFilter.Count == 0 || _nameToFilter.ContainsKey(field) || _nameToFilter.ContainsKey("*");
		}

		/// <summary>
		/// Extract the given fields from the response
		/// </summary>
		/// <param name="response">The response to filter</param>
		/// <returns>Filtered response</returns>
		public object ApplyTo(object response)
		{
			// If there are no filters at this depth, return the whole object
			if (_nameToFilter.Count == 0)
			{
				return response;
			}

			// Check if this object is a list. If it is, we'll apply the filter to each element in the list.
			IList? list = response as IList;
			if (list != null)
			{
				List<object?> newResponse = new List<object?>();
				for (int index = 0; index < list.Count; index++)
				{
					object? value = list[index];
					if (value == null)
					{
						newResponse.Add(value);
					}
					else
					{
						newResponse.Add(ApplyTo(value));
					}
				}
				return newResponse;
			}
			else
			{
				Dictionary<string, object?> newResponse = new Dictionary<string, object?>(StringComparer.Ordinal);

				// Check if this object is a dictionary.
				IDictionary? dictionary = response as IDictionary;
				if (dictionary != null)
				{
					foreach (DictionaryEntry? entry in dictionary)
					{
						if (entry.HasValue)
						{
							string? key = entry.Value.Key.ToString();
							if (key != null)
							{
								key = ConvertPascalToCamelCase(key);

								PropertyFilter? filter;
								if (_nameToFilter.TryGetValue(key, out filter))
								{
									object? value = entry.Value.Value;
									if (filter == null)
									{
										newResponse[key] = value;
									}
									else if (value != null)
									{
										newResponse[key] = filter.ApplyTo(value);
									}
								}
							}
						}
					}
				}
				else if (response is JsonObject jsonObject)
				{
					// Iterate the keys in this object
					foreach ((string name, JsonNode? node) in jsonObject)
					{
						PropertyFilter? filter;
						if (_nameToFilter.TryGetValue(name, out filter))
						{
							string key = ConvertPascalToCamelCase(name);
							if (filter == null)
							{
								newResponse[key] = node;
							}
							else if (node != null)
							{
								newResponse[key] = filter.ApplyTo(node);
							}
						}
					}
				}
				else
				{
					// Otherwise try to get the properties of this object
					foreach (PropertyInfo property in response.GetType().GetProperties(BindingFlags.Public | BindingFlags.GetProperty | BindingFlags.Instance))
					{
						PropertyFilter? filter;
						if (_nameToFilter.TryGetValue(property.Name, out filter))
						{
							string key = ConvertPascalToCamelCase(property.Name);
							object? value = property.GetValue(response);
							if (filter == null)
							{
								newResponse[key] = value;
							}
							else if (value != null)
							{
								newResponse[key] = filter.ApplyTo(value);
							}
						}
					}
				}
				return newResponse;
			}
		}

		/// <summary>
		/// Convert a property name to camel case
		/// </summary>
		/// <param name="name">The name to convert to camelCase</param>
		/// <returns>Camelcase version of the name</returns>
		static string ConvertPascalToCamelCase(string name)
		{
			if (name[0] >= 'A' && name[0] <= 'Z')
			{
				return (char)(name[0] - 'A' + 'a') + name.Substring(1);
			}
			else
			{
				return name;
			}
		}
	}

	/// <summary>
	/// Type converter from strings to PropertyFilter objects
	/// </summary>
	public class PropertyFilterTypeConverter : TypeConverter
	{
		/// <inheritdoc/>
		public override bool CanConvertFrom(ITypeDescriptorContext? context, Type sourceType)
		{
			return sourceType == typeof(string);
		}

		/// <inheritdoc/>
		public override object ConvertFrom(ITypeDescriptorContext? context, CultureInfo? culture, object value)
		{
			return PropertyFilter.Parse((string)value);
		}
	}

	/// <summary>
	/// Extension methods for property filters
	/// </summary>
	public static class PropertyFilterExtensions
	{
		/// <summary>
		/// Apply a filter to a response object
		/// </summary>
		/// <param name="obj">The object to filter</param>
		/// <param name="filter">The filter to apply</param>
		/// <returns>The filtered object</returns>
		public static object ApplyFilter(this object obj, PropertyFilter? filter)
		{
			if (filter == null)
			{
				return obj;
			}
			else
			{
				return filter.ApplyTo(obj);
			}
		}
	}
}
