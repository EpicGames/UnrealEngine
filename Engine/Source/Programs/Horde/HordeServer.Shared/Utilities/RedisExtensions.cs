// Copyright Epic Games, Inc. All Rights Reserved.

using StackExchange.Redis;

namespace HordeServer.Utilities;

/// <summary>
/// Extensions for Redis classes
/// </summary>
public static class RedisExtensions
{
	/// <summary>
	/// Await given tasks and swallow any task cancellation exceptions.
	/// Useful to do after (failing) transaction to ensure no trailing tasks are left unawaited
	/// </summary>
	/// <param name="transaction"></param>
	/// <param name="tasks">Tasks to await</param>
	public static async Task WaitAndIgnoreCancellationsAsync(this ITransaction transaction, params Task[] tasks)
	{
		_ = transaction;
		try
		{
			await Task.WhenAll(tasks);
		}
		catch (TaskCanceledException)
		{
			/* Ignore */
		}
	}
}