// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde;
using HordeServer.Server;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[StringIdConverter(typeof(SingletonIdConverter))]
	public record struct SingletonId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public SingletonId(string id) : this(new StringId(id)) { }

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class SingletonIdConverter : StringIdConverter<SingletonId>
	{
		/// <inheritdoc/>
		public override SingletonId FromStringId(StringId id) => new SingletonId(id);

		/// <inheritdoc/>
		public override StringId ToStringId(SingletonId value) => value.Id;
	}

	/// <summary>
	/// Base class for singleton documents
	/// </summary>
	public abstract class SingletonBase
	{
		/// <summary>
		/// Unique id for this singleton
		/// </summary>
		[BsonId]
		public SingletonId Id { get; set; }

		/// <summary>
		/// The revision index of this document
		/// </summary>
		public int Revision { get; set; }

		/// <summary>
		/// Callback to allow the singleton to fix up itself after being read
		/// </summary>
		public virtual void PostLoad()
		{
		}
	}

	/// <summary>
	/// Attribute specifying the unique id for a singleton document
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class SingletonDocumentAttribute : Attribute
	{
		/// <summary>
		/// Unique id for the singleton document
		/// </summary>
		public string Id { get; }

		/// <summary>
		/// ObjectId for the legacy document
		/// </summary>
		public string? LegacyId { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Unique id for the singleton document. Should be a valid StringId.</param>
		/// <param name="legacyId">Legacy identifier for the document</param>
		public SingletonDocumentAttribute(string id, string? legacyId = null)
		{
			Id = id;
			LegacyId = legacyId;
		}
	}

	/// <summary>
	/// Interface for the getting and setting the singleton
	/// </summary>
	/// <typeparam name="T">Type of document</typeparam>
	public interface ISingletonDocument<T>
	{
		/// <summary>
		/// Gets the current document
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The current document</returns>
		Task<T> GetAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Attempts to update the document
		/// </summary>
		/// <param name="value">New state of the document</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the document was updated, false otherwise</returns>
		Task<bool> TryUpdateAsync(T value, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Concrete implementation of <see cref="ISingletonDocument{T}"/>
	/// </summary>
	/// <typeparam name="T">The document type</typeparam>
	public class SingletonDocument<T> : ISingletonDocument<T> where T : SingletonBase, new()
	{
		/// <summary>
		/// The database service instance
		/// </summary>
		readonly IMongoService _mongoService;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service instance</param>
		public SingletonDocument(IMongoService mongoService)
		{
			_mongoService = mongoService;
		}

		/// <inheritdoc/>
		public async Task<T> GetAsync(CancellationToken cancellationToken)
		{
			return await _mongoService.GetSingletonAsync<T>(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<bool> TryUpdateAsync(T value, CancellationToken cancellationToken)
		{
			return _mongoService.TryUpdateSingletonAsync<T>(value, cancellationToken);
		}
	}

	/// <summary>
	/// Extension methods for singletons
	/// </summary>
	public static class SingletonDocumentExtensions
	{
		/// <summary>
		/// Update a singleton
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="singleton"></param>
		/// <param name="updateAction"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<T> UpdateAsync<T>(this ISingletonDocument<T> singleton, Action<T> updateAction, CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			for (; ; )
			{
				T value = await singleton.GetAsync(cancellationToken);
				updateAction(value);

				if (await singleton.TryUpdateAsync(value, cancellationToken))
				{
					return value;
				}
			}
		}
	}
}
