// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Serializers;
using ProtoBuf;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class StringIdBsonSerializer<TValue, TConverter> : SerializerBase<TValue> where TValue : struct where TConverter : StringIdConverter<TValue>, new()
	{
		readonly TConverter _converter = new TConverter();

		/// <inheritdoc/>
		public override TValue Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args) => _converter.FromStringId(new StringId(new Utf8String(context.Reader.ReadString()), StringId.Validate.None));

		/// <inheritdoc/>
		public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, TValue value) => context.Writer.WriteString(_converter.ToStringId(value).ToString());
	}

	/// <summary>
	/// Class which serializes object id types to BSON
	/// </summary>
	public sealed class StringIdBsonSerializationProvider : BsonSerializationProviderBase
	{
		/// <inheritdoc/>
		public override IBsonSerializer? GetSerializer(Type type, IBsonSerializerRegistry serializerRegistry)
		{
			StringIdConverterAttribute? attribute = type.GetCustomAttribute<StringIdConverterAttribute>();
			if (attribute == null)
			{
				return null;
			}
			return (IBsonSerializer?)Activator.CreateInstance(typeof(StringIdBsonSerializer<,>).MakeGenericType(type, attribute.ConverterType));
		}
	}

#pragma warning disable CS1591
	/// <summary>
	/// Surrogate type for serializing StringId types to ProtoBuf
	/// </summary>
	[ProtoContract]
	public struct StringIdProto<TValue, TConverter> where TValue : struct where TConverter : StringIdConverter<TValue>, new()
	{
		static readonly TConverter s_converter = new TConverter();

		[ProtoMember(1)]
		public string? Id { get; set; }

		public static implicit operator TValue(StringIdProto<TValue, TConverter> source) => s_converter.FromStringId(new StringId(source.Id!));
		public static implicit operator StringIdProto<TValue, TConverter>(TValue source) => new StringIdProto<TValue, TConverter> { Id = s_converter.ToStringId(source).ToString() };
	}
#pragma warning restore CS1591
}
