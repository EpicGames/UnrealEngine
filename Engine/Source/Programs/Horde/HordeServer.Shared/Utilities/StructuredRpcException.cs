// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Text;
using Grpc.Core;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Exception class designed to allow logging structured log messages
	/// </summary>
	public class StructuredRpcException : RpcException
	{
		/// <summary>
		/// The format string with named parameters
		/// </summary>
		public string Format { get; }

		/// <summary>
		/// The argument list
		/// </summary>
		public object[] Args { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="statusCode">Status code to return</param>
		/// <param name="format">The format string</param>
		/// <param name="args">Arguments for the format string</param>
		public StructuredRpcException(StatusCode statusCode, string format, params object[] args)
			: base(new Status(statusCode, FormatMessage(format, args)))
		{
			Format = format;
			Args = args;
		}

		/// <summary>
		/// Replace named arguments in the format message with their values
		/// </summary>
		/// <param name="format"></param>
		/// <param name="args"></param>
		/// <returns></returns>
		static string FormatMessage(string format, params object[] args)
		{
			string newFormat = ConvertToFormatString(format);
			return String.Format(CultureInfo.CurrentCulture, newFormat, args);
		}

		/// <summary>
		/// Converts a named parameter format string to a String.Format style string
		/// </summary>
		/// <param name="format"></param>
		/// <returns></returns>
		static string ConvertToFormatString(string format)
		{
			int argIdx = 0;

			StringBuilder newFormat = new StringBuilder();
			for (int idx = 0; idx < format.Length; idx++)
			{
				char character = format[idx];
				newFormat.Append(character);

				if (character == '{' && idx + 1 < format.Length)
				{
					char nextCharacter = format[idx + 1];
					if ((nextCharacter >= 'a' && nextCharacter <= 'z') || (nextCharacter >= 'A' && nextCharacter <= 'Z') || nextCharacter == '_')
					{
						for (int endIdx = idx + 2; endIdx < format.Length; endIdx++)
						{
							if (format[endIdx] == ':' || format[endIdx] == '}')
							{
								newFormat.Append(argIdx++);
								idx = endIdx - 1;
								break;
							}
						}
					}
				}
			}

			return newFormat.ToString();
		}
	}
}
