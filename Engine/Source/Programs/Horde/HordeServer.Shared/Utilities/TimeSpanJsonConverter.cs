// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Converter for <see cref="TimeSpan"/> types to Json
	/// </summary>
	public class TimeSpanJsonConverter : JsonConverter<TimeSpan>
	{
		/// <inheritdoc/>
		public override TimeSpan Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			string? str = reader.GetString();
			if (str == null)
			{
				throw new InvalidDataException("Unable to parse TimeSpan");
			}
			return TimeSpan.Parse(str, CultureInfo.CurrentCulture);
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, TimeSpan timeSpan, JsonSerializerOptions options)
		{
			writer.WriteStringValue(timeSpan.ToString("c"));
		}
	}
}
