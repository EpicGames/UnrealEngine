// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for <see cref="TimeZoneInfo"/>
	/// </summary>
	public static class TimeZoneInfoExtensions
	{
		/// <summary>
		/// Gets the start of the day for the given datetime in UTC, respecting the configured timezone.
		/// </summary>
		/// <param name="timeZone">Time zone to adjust for</param>
		/// <param name="time">Time to convert</param>
		/// <returns>UTC datetime for the start of the day</returns>
		public static DateTime GetStartOfDayUtc(this TimeZoneInfo timeZone, DateTime time)
		{
			DateTime currentTimeLocal = TimeZoneInfo.ConvertTime(time, timeZone);
			DateTime startOfDayLocal = currentTimeLocal - currentTimeLocal.TimeOfDay;
			return TimeZoneInfo.ConvertTime(startOfDayLocal, timeZone, TimeZoneInfo.Utc);
		}
	}
}
