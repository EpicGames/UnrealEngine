// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Authorization.Policy;
using Microsoft.AspNetCore.Mvc.Filters;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Attempts to authorize the user, but does not challenge or forbid access if authorization fails.
	/// </summary>
	[AttributeUsage(AttributeTargets.Method | AttributeTargets.Class)]
	public sealed class TryAuthorizeAttribute : Attribute, IAsyncAuthorizationFilter
	{
		/// <inheritdoc/>
		public async Task OnAuthorizationAsync(AuthorizationFilterContext context)
		{
			IAuthorizationPolicyProvider policyProvider = context.HttpContext.RequestServices.GetRequiredService<IAuthorizationPolicyProvider>();
			AuthorizationPolicy effectivePolicy = await policyProvider.GetDefaultPolicyAsync();

			IPolicyEvaluator policyEvaluator = context.HttpContext.RequestServices.GetRequiredService<IPolicyEvaluator>();
			await policyEvaluator.AuthenticateAsync(effectivePolicy, context.HttpContext);
		}
	}
}
