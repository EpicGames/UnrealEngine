// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Hosting;

namespace HordeServer.Tests
{
	public class AppLifetimeStub : IHostApplicationLifetime
	{
		public CancellationToken ApplicationStarted { get; }
		public CancellationToken ApplicationStopping { get; }
		public CancellationToken ApplicationStopped { get; }

		public AppLifetimeStub()
		{
			ApplicationStarted = new CancellationToken();
			ApplicationStopping = new CancellationToken();
			ApplicationStopped = new CancellationToken();
		}

		public void StopApplication()
		{
			throw new NotImplementedException();
		}
	}
}
