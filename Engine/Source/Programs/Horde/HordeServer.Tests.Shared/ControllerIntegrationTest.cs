// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.CloudWatch;
using HordeServer.Server;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.Mvc.Testing;
using Microsoft.AspNetCore.TestHost;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Moq;
using Serilog;

namespace HordeServer.Tests;

static class SerilogExtensions
{
	public static LoggerConfiguration Override<T>(this Serilog.Configuration.LoggerMinimumLevelConfiguration configuration, Serilog.Events.LogEventLevel minimumLevel) where T : class
	{
		return configuration.Override(typeof(T).FullName!, minimumLevel);
	}
}

public class TestWebApplicationFactory<TStartup> : WebApplicationFactory<TStartup> where TStartup : class
{
	private readonly MongoInstance _mongoInstance;
	private readonly RedisInstance _redisInstance;
	private readonly Action<IServiceCollection>? _configureServices;
	private readonly Dictionary<string, string>? _extraSettings;

	public TestWebApplicationFactory(MongoInstance mongoInstance, RedisInstance redisInstance, Action<IServiceCollection>? configureServices, Dictionary<string, string>? extraSettings = null)
	{
		_mongoInstance = mongoInstance;
		_redisInstance = redisInstance;
		_configureServices = configureServices;
		_extraSettings = extraSettings;

		Environment.SetEnvironmentVariable("ASPNETCORE_ENVIRONMENT", "Testing");
		Serilog.Log.Logger = new LoggerConfiguration()
			.Enrich.FromLogContext()
			.WriteTo.Console()
			.MinimumLevel.Information()
			.MinimumLevel.Override<MongoService>(Serilog.Events.LogEventLevel.Warning)
			.MinimumLevel.Override("Redis", Serilog.Events.LogEventLevel.Warning)
			.CreateLogger();
	}

	protected override void ConfigureWebHost(IWebHostBuilder builder)
	{
		Dictionary<string, string?> dict = new()
		{
			{ "Horde:DatabaseConnectionString", _mongoInstance.ConnectionString },
			{ "Horde:DatabaseName", _mongoInstance.DatabaseName },
			{ "Horde:LogServiceWriteCacheType", "inmemory" },
			{ "Horde:AuthMethod", "Anonymous" },
			{ "Horde:OidcAuthority", null },
			{ "Horde:OidcClientId", null },

			{ "Horde:RedisConnectionConfig", _redisInstance.ConnectionString },
		};

		if (_extraSettings != null)
		{
			foreach ((string key, string value) in _extraSettings)
			{
				dict[key] = value;
			}
		}

		Mock<IAmazonCloudWatch> cloudWatchMock = new(MockBehavior.Strict);
		builder.ConfigureAppConfiguration((hostingContext, config) => { config.AddInMemoryCollection(dict); });
		if (_configureServices != null)
		{
			builder.ConfigureTestServices(_configureServices);
		}
		/*		
				=>
				{
					collection.AddSingleton<IPerforceService, PerforceServiceStub>();
					collection.AddSingleton<IAmazonCloudWatch>(x => cloudWatchMock.Object);
				});*/
	}
}

public class FakeHordeWebApp : IAsyncDisposable
{
	public MongoInstance MongoInstance { get; }
	public RedisInstance RedisInstance { get; }
	public IServiceProvider ServiceProvider => Factory.Services;
	private TestWebApplicationFactory<Startup> Factory { get; }
	private readonly DatabaseIntegrationTest _dbIntegrationTest;

	public FakeHordeWebApp(Action<IServiceCollection>? configureServices = null, Dictionary<string, string>? settings = null)
	{
		ServerApp.InitializePluginsForTests();
		MongoInstance = new MongoInstance();
		RedisInstance = new RedisInstance();
		_dbIntegrationTest = new DatabaseIntegrationTest();

		void ConfigureServices(IServiceCollection services)
		{
			configureServices?.Invoke(services);
			services.AddSingleton<IMongoService>(_dbIntegrationTest.GetMongoServiceSingleton());
			services.AddSingleton<IRedisService>(_dbIntegrationTest.GetRedisServiceSingleton());
		}
		
		Factory = new TestWebApplicationFactory<Startup>(MongoInstance, RedisInstance, ConfigureServices, settings);
		
	}

	public HttpClient CreateHttpClient(bool allowAutoRedirect = true)
	{
		WebApplicationFactoryClientOptions opts = new() { AllowAutoRedirect = allowAutoRedirect };
		return Factory.CreateClient(opts);
	}

	public async ValueTask DisposeAsync()
	{
		try
		{
			await Factory.DisposeAsync();
			await _dbIntegrationTest.DisposeAsync();
			MongoInstance.Dispose();
			RedisInstance.Dispose();
		}
		catch (Exception ex)
		{
			Console.WriteLine($"Exception running cleanup: {ex}");
			throw;
		}

		GC.SuppressFinalize(this);
	}
}

public class ControllerIntegrationTest : IAsyncDisposable
{
	protected HttpClient Client { get; }
	protected IServiceProvider ServiceProvider => _app.ServiceProvider;
	private readonly FakeHordeWebApp _app;

	public ControllerIntegrationTest()
	{
		_app = new FakeHordeWebApp(ConfigureServices);
		Client = _app.CreateHttpClient();
	}

	protected virtual void ConfigureServices(IServiceCollection services)
	{
	}

	public virtual async ValueTask DisposeAsync()
	{
		await _app.DisposeAsync();
		GC.SuppressFinalize(this);
	}

	public HttpClient CreateHttpClient(bool allowAutoRedirect = true)
		=> _app.CreateHttpClient(allowAutoRedirect);
}

/*
	public Task<Fixture> GetFixtureAsync()
	{
		return _fixture.Value;
	}

	private async Task<Fixture> CreateFixtureAsync()
	{
		IServiceProvider services = _app.ServiceProvider;
		ConfigService configService = services.GetRequiredService<ConfigService>();
		ITemplateCollection templateService = services.GetRequiredService<ITemplateCollection>();
		JobService jobService = services.GetRequiredService<JobService>();
		AgentService agentService = services.GetRequiredService<AgentService>();
		IGraphCollection graphCollection = services.GetRequiredService<IGraphCollection>();
		IPluginCollection pluginCollection = services.GetRequiredService<IPluginCollection>();
		IOptions<ServerSettings> serverSettings = services.GetRequiredService<IOptions<ServerSettings>>();

		return await Fixture.CreateAsync(configService, graphCollection, templateService, jobService, agentService, pluginCollection, serverSettings.Value);
	}
*/