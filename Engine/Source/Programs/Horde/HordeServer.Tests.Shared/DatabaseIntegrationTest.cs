// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Net;
using System.Runtime.InteropServices;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;
using MongoDB.Driver;
using StackExchange.Redis;

namespace HordeServer.Tests
{
	// Stub for fulfilling IOptions interface during testing
	public sealed class TestOptions<T> : IOptions<T> where T : class
	{
		public TestOptions(T options) => Value = options;
		public T Value { get; }
	}

	// Stub for fulfilling IOptionsMonitor interface during testing
	public sealed class TestOptionsMonitor<T> : IOptionsMonitor<T>
		where T : class, new()
	{
		sealed class Disposable : IDisposable
		{
			public void Dispose() { }
		}

		public TestOptionsMonitor(T currentValue)
		{
			CurrentValue = currentValue;
		}

		public T Get(string? name)
		{
			return CurrentValue;
		}

		public IDisposable OnChange(Action<T, string> listener)
		{
			return new Disposable();
		}

		public T CurrentValue { get; }
	}

	public sealed class MongoInstance : IDisposable
	{
		public string DatabaseName { get; }
		public string ConnectionString { get; }
		MongoClient Client { get; }

		private static readonly object s_lockObject = new object();
		private static MongoDbRunnerLocal? s_mongoDbRunner;
		private static int s_nextDatabaseIndex = 1;
		public const string MongoDbDatabaseNamePrefix = "HordeServerTest_";

		public MongoInstance()
		{
			int databaseIndex;
			lock (s_lockObject)
			{
				if (s_mongoDbRunner == null)
				{
					// One-time setup per test run to avoid overhead of starting the external MongoDB process
					Startup.ConfigureMongoDbClient();
					s_mongoDbRunner = new MongoDbRunnerLocal();
					s_mongoDbRunner.Start();

					// Drop all the previous databases
					MongoClientSettings mongoSettings = MongoClientSettings.FromConnectionString(s_mongoDbRunner.GetConnectionString());
					MongoClient client = new MongoClient(mongoSettings);

					List<string> dropDatabaseNames = client.ListDatabaseNames().ToList();
					foreach (string dropDatabaseName in dropDatabaseNames)
					{
						if (dropDatabaseName.StartsWith(MongoDbDatabaseNamePrefix, StringComparison.Ordinal))
						{
							client.DropDatabase(dropDatabaseName);
						}
					}
				}
				databaseIndex = s_nextDatabaseIndex++;
			}

			DatabaseName = $"{MongoDbDatabaseNamePrefix}{databaseIndex}";
			ConnectionString = $"{s_mongoDbRunner.GetConnectionString()}/{DatabaseName}";
			Client = new MongoClient(MongoClientSettings.FromConnectionString(ConnectionString));
		}

		public void Dispose()
		{

			IMongoClient strictClient = Client.WithWriteConcern(new WriteConcern(journal: true));

			for (int i = 0; i < 5; i++)
			{
				strictClient.DropDatabase(DatabaseName);
				List<string> dbNames = strictClient.ListDatabaseNames().ToList();
				if (!dbNames.Contains(DatabaseName))
				{
					return;
				}
				Thread.Sleep(300);
			}

			throw new Exception($"Unable to drop MongoDB database {DatabaseName}");
		}

		public static void FullDispose()
		{
			lock (s_lockObject)
			{
				s_mongoDbRunner?.Dispose();
				s_mongoDbRunner = null;
			}
		}
	}

	public sealed class RedisInstance : IDisposable
	{
		static readonly object s_lockObject = new object();

		private static (int min, int max) DatabaseNumbers { get; } = (1000, 2000);
		private static readonly BlockingCollection<int> s_availableRedisDbs = new ();
		private static string? s_redisConnectionString;
		private static RedisProcess? s_redisProcess;

		public string ConnectionString { get; }
		public int DatabaseNumber { get; }

		private const bool UseExistingRedisInstance = true;
		private const int RedisPort = 6379;
		
		static RedisInstance()
		{
			for (int i = DatabaseNumbers.min; i < DatabaseNumbers.max; i++)
			{
				s_availableRedisDbs.Add(i);
			}
		}
		
		public RedisInstance()
		{
			lock (s_lockObject)
			{
				if (s_redisConnectionString == null)
				{
					int port = GetRedisPortInternal();
					s_redisConnectionString = $"127.0.0.1:{port},allowAdmin=true";
				}

				ConnectionString = s_redisConnectionString;
				DatabaseNumber = CheckoutDatabase();
			}
		}

		public void Dispose()
		{
			ReturnDatabase(DatabaseNumber);
		}
		
		private static int CheckoutDatabase()
		{
			TimeSpan timeoutToUse = TimeSpan.FromMinutes(1);
			if (s_availableRedisDbs.TryTake(out int dbNumber, timeoutToUse))
			{
				return dbNumber;
			}
        
			throw new TimeoutException($"Failed to checkout database within {timeoutToUse}");
		}

		private static void ReturnDatabase(int dbNumber)
		{
			s_availableRedisDbs.Add(dbNumber);
		}

		static int GetRedisPortInternal()
		{
			if (UseExistingRedisInstance && !DatabaseRunner.IsPortAvailable(RedisPort))
			{
				Console.WriteLine("Using existing Redis instance on port {0}", RedisPort);
				return RedisPort;
			}

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				s_redisProcess = new RedisProcess(NullLogger.Instance);
				s_redisProcess.Start("");

				return s_redisProcess.Port;
			}

			throw new Exception("Unable to connect to Redis");
		}
	}

	public class DatabaseIntegrationTest : ServiceTest
	{
		private static readonly object s_lockObject = new object();
		private static MongoService? s_globalMongoService;

		private MongoInstance? _mongoInstance;
		private IMongoService? _mongoService;
		private readonly LoggerFactory _loggerFactory = new LoggerFactory();

		private RedisInstance? _redisInstance;
		private RedisService? _redisService;

		public DatabaseIntegrationTest()
		{
		}

		/// <summary>
		/// Explicitly calls a full dispose of the underyling <see cref="MongoInstance"/>.
		/// </summary>
		/// <remarks>This should be invoked in the <see cref="AssemblyCleanupAttribute"/> of test suite's assembly.</remarks>
		public static void Cleanup()
		{
			MongoInstance.FullDispose();
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			services.AddSingleton<IMongoService>(GetMongoServiceSingleton());
			services.AddSingleton<IRedisService>(GetRedisServiceSingleton());
		}

		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			GC.SuppressFinalize(this);

			if (_mongoService is IAsyncDisposable ad)
			{
				await ad.DisposeAsync();
			}
			_mongoInstance?.Dispose();

			if (_redisService != null)
			{
				await _redisService.DisposeAsync();
			}
			_redisInstance?.Dispose();

			_loggerFactory.Dispose();
		}

		public IMongoService GetMongoServiceSingleton()
		{
			lock (s_lockObject)
			{
				if (s_globalMongoService == null)
				{
					IRedisService redisService = GetRedisServiceSingleton();
					_mongoInstance = new MongoInstance();
					
					ServerSettings ss = new()
					{
						MongoDatabaseName = _mongoInstance.DatabaseName,
						MongoConnectionString = _mongoInstance.ConnectionString
					};
					
					MongoCommandTracer mongoTracer = new(OpenTelemetryTracers.MongoDb, _loggerFactory.CreateLogger<MongoCommandTracer>());
					s_globalMongoService = new MongoService(Options.Create(ss), redisService, mongoTracer, OpenTelemetryTracers.Horde, _loggerFactory.CreateLogger<MongoService>(), _loggerFactory);
				}

				if (_mongoService == null)
				{
					_mongoService = new TestScopedMongoService(s_globalMongoService.GetClient());
				}
			}
			return _mongoService;
		}

		public IRedisService GetRedisServiceSingleton()
		{
			if (_redisService == null)
			{
				_redisInstance = new RedisInstance();
				_redisService = new RedisService(_redisInstance.ConnectionString, _redisInstance.DatabaseNumber, _loggerFactory.CreateLogger<RedisService>());

				IConnectionMultiplexer cm = _redisService.ConnectionPool.GetConnection();
				foreach (EndPoint endpoint in cm.GetEndPoints())
				{
					cm.GetServer(endpoint).FlushDatabase(_redisInstance.DatabaseNumber);
				}
			}
			return _redisService;
		}

		public static T Deref<T>(T? item)
		{
			Assert.IsNotNull(item);
			return item!;
		}

		public static T Deref<T>(ActionResult<T>? item) where T : class
		{
			Assert.IsNotNull(item?.Value);
			return item!.Value!;
		}
	}
}