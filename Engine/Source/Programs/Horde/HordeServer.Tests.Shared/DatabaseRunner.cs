// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Net;
using System.Net.NetworkInformation;
using System.Net.Sockets;
using System.Reflection;
using System.Runtime.InteropServices;
using EpicGames.Core;

namespace HordeServer.Tests;

public abstract class DatabaseRunner : IDisposable
{
	private readonly string _binName;
	private readonly int _defaultPort;
	private readonly string _name;
	private readonly bool _reuseProcess;
	protected string TempDir { get; }

	private ManagedProcessGroup? _processGroup;
	private ManagedProcess? _process;

	protected DatabaseRunner(string name, string binName, int defaultPort, bool reuseProcess)
	{
		_name = name;
		_binName = binName;
		_defaultPort = defaultPort;
		_reuseProcess = reuseProcess;
		TempDir = GetTemporaryDirectory();
	}

	/// <inheritdoc/>
	public void Dispose()
	{
		Dispose(true);
		GC.SuppressFinalize(this);
	}

	protected virtual void Dispose(bool disposing)
	{
		Stop();
	}

	protected int Port { get; private set; } = -1;

	protected abstract string GetArguments();

	public void Start()
	{
		if (_reuseProcess && !IsPortAvailable(_defaultPort))
		{
			Console.WriteLine($"Re-using already running {_name} process!");
			Port = _defaultPort;
			return;
		}

		if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
		{
			Console.WriteLine($"Unable to find a running {_name} process to use during testing!");
			Console.WriteLine("This is required on any non-Windows as the runner can only start Windows binaries!");
			Console.WriteLine($"Please ensure {_binName} is running on the default port {_defaultPort}.");
			throw new Exception("Failed finding process to re-use! See stdout for info.");
		}

		if (_process != null)
		{
			return;
		}

		Port = GetAvailablePort();

		_processGroup = new ManagedProcessGroup();
		_process = new ManagedProcess(_processGroup, GetBinaryPath(), GetArguments(), TempDir, null, ProcessPriorityClass.Normal);
		Task.Run(() => RelayOutputAsync(_process));

		// Try detect when main .NET process exits and kill the runner
		AppDomain.CurrentDomain.ProcessExit += (sender, eventArgs) =>
		{
			Console.WriteLine("Main process exiting!");
			Stop();
		};
	}

	static async Task RelayOutputAsync(ManagedProcess process)
	{
		for (; ; )
		{
			string? line = await process.ReadLineAsync();
			if (line == null)
			{
				break;
			}
			//			Console.WriteLine("{0} output: {1}", _name, line);
		}
	}

	public void Stop()
	{
		if (_process != null)
		{
			_process.Dispose();
			_process = null;
		}
		if (_processGroup != null)
		{
			_processGroup.Dispose();
			_processGroup = null;
		}
		DeleteDirectory(TempDir);
	}

	public (string Host, int Port) GetListenAddress()
	{
		return ("localhost", Port);
	}

	private string GetBinaryPath()
	{
		FileReference file = new(new Uri(Assembly.GetExecutingAssembly().Location).LocalPath);
		FileReference binPath = FileReference.Combine(file.Directory, _binName);
		return binPath.FullName;
	}

	private string GetTemporaryDirectory()
	{
		string temp = Path.Join(Path.GetTempPath(), $"horde-{_name}-" + Path.GetRandomFileName());
		Directory.CreateDirectory(temp);
		return temp;
	}

	private static int GetAvailablePort(int timeoutMs = 5000)
	{
		using TcpListener listener = new(IPAddress.Loopback, 0);
		listener.Server.ReceiveTimeout = timeoutMs;
		listener.Server.SendTimeout = timeoutMs;
		listener.Start();
		int port = ((IPEndPoint)listener.LocalEndpoint).Port;
		listener.Stop();
		return port;
	}

	public static bool IsPortAvailable(int port, int timeoutMs = 5000)
	{
		if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
		{
			IPGlobalProperties ipGlobalProperties = IPGlobalProperties.GetIPGlobalProperties();

			IPEndPoint[] listeners = ipGlobalProperties.GetActiveTcpListeners();
			if (listeners.Any(x => x.Port == port))
			{
				return false;
			}

			return true;
		}

		try
		{
			using TcpListener listenerAny = new (IPAddress.Loopback, port);
			listenerAny.Server.ReceiveTimeout = timeoutMs;
			listenerAny.Server.SendTimeout = timeoutMs;
			listenerAny.Start();
			using TcpListener listenerLoopback = new (IPAddress.Any, port);
			listenerLoopback.Server.ReceiveTimeout = timeoutMs;
			listenerLoopback.Server.SendTimeout = timeoutMs;
			listenerLoopback.Start();
			return true;
		}
		catch (SocketException)
		{
		}

		return false;
	}

	private static void DeleteDirectory(string path)
	{
		DirectoryInfo dir = new DirectoryInfo(path);
		if (dir.Exists)
		{
			dir.Attributes = FileAttributes.Normal;

			foreach (FileSystemInfo info in dir.GetFileSystemInfos("*", SearchOption.AllDirectories))
			{
				info.Attributes = FileAttributes.Normal;
			}

			dir.Delete(true);
		}
	}
}

public class MongoDbRunnerLocal : DatabaseRunner
{
	public MongoDbRunnerLocal() : base("mongodb", "ThirdParty/Mongo/mongod.exe", 27017, true)
	{
	}

	protected override string GetArguments()
	{
		return $"--dbpath {TempDir} --noauth --quiet --port {Port}";
	}

	public string GetConnectionString()
	{
		(string host, int listenPort) = GetListenAddress();
		return $"mongodb://{host}:{listenPort}";
	}
}
