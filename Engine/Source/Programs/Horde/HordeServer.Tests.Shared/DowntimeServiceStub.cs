// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Server;

namespace HordeServer.Tests
{
	public class DowntimeServiceStub : IDowntimeService
	{
		public DowntimeServiceStub(bool isDowntimeActive = false)
		{
			IsDowntimeActive = isDowntimeActive;
		}

		public bool IsDowntimeActive { get; set; }
	}
}
