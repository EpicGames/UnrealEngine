<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer.Tests</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\HordeServer\HordeServer.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Testing" Version="10.0.6" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>

</Project>
