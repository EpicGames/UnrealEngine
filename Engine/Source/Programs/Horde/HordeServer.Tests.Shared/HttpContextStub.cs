// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Security.Claims;
using System.Security.Cryptography.X509Certificates;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Features;

namespace HordeServer.Tests
{
	using ISession = Microsoft.AspNetCore.Http.ISession;

	/// <summary>
	/// Stub implementation of <see cref="ConnectionInfo"/> for testing
	/// </summary>
	public sealed class ConnectionInfoStub : ConnectionInfo
	{
		/// <inheritdoc/>
		public override string Id { get; set; } = "test-connection";

		/// <inheritdoc/>
		public override IPAddress? RemoteIpAddress { get; set; }

		/// <inheritdoc/>
		public override int RemotePort { get; set; }

		/// <inheritdoc/>
		public override IPAddress? LocalIpAddress { get; set; }

		/// <inheritdoc/>
		public override int LocalPort { get; set; }

		/// <inheritdoc/>
		public override X509Certificate2? ClientCertificate { get; set; }

		/// <inheritdoc/>
		public override Task<X509Certificate2?> GetClientCertificateAsync(CancellationToken cancellationToken = default)
		{
			return Task.FromResult(ClientCertificate);
		}
	}

	sealed class HttpContextStub : HttpContext
	{
		public override ConnectionInfo Connection { get; }
		public override IFeatureCollection Features { get; } = null!;
		public override IDictionary<object, object?> Items { get; set; } = null!;
		public override HttpRequest Request { get; } = null!;
		public override CancellationToken RequestAborted { get; set; }
		public override IServiceProvider RequestServices { get; set; } = null!;
		public override HttpResponse Response { get; } = null!;
		public override ISession Session { get; set; } = null!;
		public override string TraceIdentifier { get; set; } = null!;
		public override ClaimsPrincipal User { get; set; }
		public override WebSocketManager WebSockets { get; } = null!;

		public HttpContextStub(Claim? roleClaimType, IPAddress? remoteIpAddress = null)
		{
			Connection = new ConnectionInfoStub { RemoteIpAddress = remoteIpAddress };
			User = new ClaimsPrincipal(new ClaimsIdentity(roleClaimType != null ? [roleClaimType] : [], "TestAuthType"));
		}

		public HttpContextStub(ClaimsPrincipal user, IPAddress? remoteIpAddress = null)
		{
			Connection = new ConnectionInfoStub { RemoteIpAddress = remoteIpAddress };
			User = user;
		}

		public override void Abort()
		{
			throw new NotImplementedException();
		}
	}
}
