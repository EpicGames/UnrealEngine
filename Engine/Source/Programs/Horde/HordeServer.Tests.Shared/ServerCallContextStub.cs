// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Security.Claims;
using Grpc.Core;
using HordeServer.Utilities;

namespace HordeServer.Tests
{
	public class ServerCallContextStub : ServerCallContext
	{
		// Copied from ServerCallContextExtensions.cs in Grpc.Core
		const string HttpContextKey = "__HttpContext";

		protected override string MethodCore { get; } = null!;
		protected override string HostCore { get; } = null!;
		protected override string PeerCore { get; } = null!;
		protected override DateTime DeadlineCore { get; } = DateTime.Now.AddHours(24);
		protected override Metadata RequestHeadersCore { get; } = null!;
		protected override CancellationToken CancellationTokenCore => _cancellationToken;
		protected override Metadata ResponseTrailersCore { get; } = null!;
		protected override Status StatusCore { get; set; }
		protected override WriteOptions? WriteOptionsCore { get; set; } = null!;
		protected override AuthContext AuthContextCore { get; } = null!;

		private CancellationToken _cancellationToken;

		public static ServerCallContext ForAdminWithAgentSessionId(string agentSessionId)
		{
			return new ServerCallContextStub(new ClaimsPrincipal(new ClaimsIdentity(new List<Claim>
			{
				HordeClaims.AdminClaim.ToClaim(),
				new Claim(HordeClaimTypes.AgentSessionId, agentSessionId),
			}, "TestAuthType")));
		}

		public static ServerCallContext ForAdmin(IPAddress? remoteIpAddress = null)
		{
			return new ServerCallContextStub(new ClaimsPrincipal(new ClaimsIdentity(new List<Claim>
			{
				HordeClaims.AdminClaim.ToClaim()
			}, "TestAuthType")), remoteIpAddress);
		}

		public ServerCallContextStub(Claim? roleClaimType, IPAddress? remoteIpAddress = null)
		{
			// The GetHttpContext extension falls back to getting the HttpContext from UserState
			// We can piggyback on that behavior during tests
			UserState[HttpContextKey] = new HttpContextStub(roleClaimType, remoteIpAddress);
		}

		public ServerCallContextStub(ClaimsPrincipal user, IPAddress? remoteIpAddress = null)
		{
			// The GetHttpContext extension falls back to getting the HttpContext from UserState
			// We can piggyback on that behavior during tests
			UserState[HttpContextKey] = new HttpContextStub(user, remoteIpAddress);
		}

		public void SetCancellationToken(CancellationToken cancellationToken)
		{
			_cancellationToken = cancellationToken;
		}

		protected override Task WriteResponseHeadersAsyncCore(Metadata responseHeaders)
		{
			throw new NotImplementedException();
		}

		protected override ContextPropagationToken CreatePropagationTokenCore(ContextPropagationOptions? options)
		{
			throw new NotImplementedException();
		}
	}
}
