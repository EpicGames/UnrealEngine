// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using EpicGames.Core;
using EpicGames.Horde.Tools;
using HordeCommon;
using HordeServer.Accounts;
using HordeServer.Acls;
using HordeServer.Auditing;
using HordeServer.Configuration;
using HordeServer.Dashboard;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Tools;
using HordeServer.Users;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using Serilog;
using Serilog.Exceptions;
using Serilog.Exceptions.Core;
using Serilog.Exceptions.Grpc.Destructurers;

namespace HordeServer.Tests
{
	public class ServerTestSetup : DatabaseIntegrationTest
	{
		public FakeClock Clock => ServiceProvider.GetRequiredService<FakeClock>();
		public IMemoryCache Cache => ServiceProvider.GetRequiredService<IMemoryCache>();

		public IAclService AclService => ServiceProvider.GetRequiredService<IAclService>();
		public IMongoService MongoService => ServiceProvider.GetRequiredService<IMongoService>();
		public IDowntimeService DowntimeService => ServiceProvider.GetRequiredService<IDowntimeService>();
		public LifetimeService LifetimeService => ServiceProvider.GetRequiredService<LifetimeService>();
		public ServerStatusService ServerStatusService => ServiceProvider.GetRequiredService<ServerStatusService>();

		public ServerSettings ServerSettings => ServiceProvider.GetRequiredService<IOptions<ServerSettings>>().Value;
		public IOptionsMonitor<ServerSettings> ServerSettingsMon => ServiceProvider.GetRequiredService<IOptionsMonitor<ServerSettings>>();

		public OpenTelemetry.Trace.Tracer Tracer => ServiceProvider.GetRequiredService<OpenTelemetry.Trace.Tracer>();
		public Meter Meter => ServiceProvider.GetRequiredService<Meter>();
		public ConfigService ConfigService => ServiceProvider.GetRequiredService<ConfigService>();
		public ILogger<ConfigService> ConfigServiceLogger => ServiceProvider.GetRequiredService<ILogger<ConfigService>>();
		public IOptionsMonitor<GlobalConfig> GlobalConfig => ServiceProvider.GetRequiredService<IOptionsMonitor<GlobalConfig>>();
		public IOptionsSnapshot<GlobalConfig> GlobalConfigSnapshot => ServiceProvider.GetRequiredService<IOptionsSnapshot<GlobalConfig>>();

		public IPluginCollection PluginCollection => ServiceProvider.GetRequiredService<IPluginCollection>();
		public IEnumerable<IDefaultAclModifier> DefaultAclModifiers => ServiceProvider.GetRequiredService<IEnumerable<IDefaultAclModifier>>();

		private static bool s_datadogWriterPatched;

		readonly PluginCollection _pluginCollection;

		public ServerTestSetup()
		{
			_pluginCollection = new PluginCollection();

			PatchDatadogWriter();
		}

		static ServerTestSetup()
		{
			Serilog.Log.Logger = new LoggerConfiguration()
				.Enrich.FromLogContext()
				.Enrich.WithExceptionDetails(new DestructuringOptionsBuilder()
					.WithDefaultDestructurers()
					.WithDestructurers([new RpcExceptionDestructurer()])
					.WithDestructuringDepth(3))
				.WriteTo.Console(outputTemplate: "[{Timestamp:HH:mm:ss} {Level:w3}] {Indent}{Message:l}{NewLine}{Exception}")
				.CreateLogger();
		}

		protected async Task SetConfigAsync(GlobalConfig globalConfig)
		{
			await ConfigService.WaitForInitialConfigAsync();
			globalConfig.PostLoad(ServerSettings, _pluginCollection.LoadedPlugins, DefaultAclModifiers, ConfigServiceLogger);
			await ConfigService.ResolveConfigSecretsAsync(globalConfig, CancellationToken.None);
			ConfigService.OverrideConfig(globalConfig);
		}

		protected async Task UpdateConfigAsync(Action<GlobalConfig> action)
		{
			await ConfigService.WaitForInitialConfigAsync();
			GlobalConfig globalConfig = GlobalConfig.CurrentValue;
			action(globalConfig);
			globalConfig.PostLoad(ServerSettings, _pluginCollection.LoadedPlugins, DefaultAclModifiers, ConfigServiceLogger);
			await ConfigService.ResolveConfigSecretsAsync(globalConfig, CancellationToken.None);
			ConfigService.OverrideConfig(globalConfig);
		}

		protected override void ConfigureSettings(ServerSettings settings)
		{
			DirectoryReference baseDir = DirectoryReference.Combine(ServerApp.DataDir, "Tests");
			try
			{
				FileUtils.ForceDeleteDirectoryContents(baseDir);
			}
			catch
			{
			}

			settings.ForceConfigUpdateOnStartup = true;
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			IConfiguration configuration = new ConfigurationBuilder().Build();

			ServerSettings settings = new ServerSettings();
			ConfigureSettings(settings);

			services.Configure<ComputeServerConfig>(x => x.WithAws = true);

			ServerInfo serverInfo = new ServerInfo(configuration, Options.Create(settings));
			services.AddSingleton<IServerInfo>(serverInfo);
			services.AddSingleton<IPluginCollection>(_pluginCollection);

			services.AddSingleton<MemoryMappedFileCache>();

			services.AddSingleton<IAccountCollection, AccountCollection>();
			services.AddSingleton<IToolCollection, ToolCollection>();

			services.AddLogging(builder => builder.AddSerilog());
			services.AddSingleton<IMemoryCache>(sp => new MemoryCache(new MemoryCacheOptions { }));

			services.AddSingleton<OpenTelemetry.Trace.Tracer>(sp => TracerProvider.Default.GetTracer("TestTracer"));
			services.AddSingleton(sp => new Meter("TestMeter"));

			services.AddSingleton(typeof(IAuditLogFactory<>), typeof(AuditLogFactory<>));
			
			services.AddSingleton<GlobalsService>();
			services.AddSingleton<ConfigService>();
			services.AddSingleton<IConfigService>(sp => sp.GetRequiredService<ConfigService>());
			services.AddSingleton<IOptionsFactory<GlobalConfig>>(sp => sp.GetRequiredService<ConfigService>());
			services.AddSingleton<IOptionsChangeTokenSource<GlobalConfig>>(sp => sp.GetRequiredService<ConfigService>());

			services.AddSingleton<IConfigSource, FileConfigSource>();

			services.AddSingleton<IUserCollection, UserCollectionV2>();
			services.AddSingleton<IDashboardPreviewCollection, DashboardPreviewCollection>();

			services.AddSingleton<FakeClock>();
			services.AddSingleton<IClock>(sp => sp.GetRequiredService<FakeClock>());
			services.AddSingleton<IHostApplicationLifetime, AppLifetimeStub>();
			services.AddSingleton<IHostEnvironment, WebHostEnvironmentStub>();

			services.AddSingleton<IAclService, AclService>();
			services.AddSingleton<IDowntimeService, DowntimeServiceStub>();
			services.AddSingleton<LifetimeService>();
			services.AddSingleton<ILifetimeService>(sp => sp.GetRequiredService<LifetimeService>());

			services.AddSingleton(typeof(IHealthMonitor<>), typeof(HealthMonitor<>));
			services.AddSingleton<ServerStatusService>();

			foreach (ILoadedPlugin plugin in _pluginCollection.LoadedPlugins)
			{
				plugin.ConfigureServices(configuration, serverInfo, services);
			}
		}

		protected void AddPlugin<T>() where T : class, IPluginStartup
		{
			_pluginCollection.Add<T>();
		}

		/// <summary>
		/// Create a console logger for tests
		/// </summary>
		/// <typeparam name="T">Type to instantiate</typeparam>
		/// <returns>A logger</returns>
		public static ILogger<T> CreateConsoleLogger<T>()
		{
			using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
			{
				builder.SetMinimumLevel(LogLevel.Debug);
				builder.AddSimpleConsole(options => { options.SingleLine = true; });
			});

			return loggerFactory.CreateLogger<T>();
		}

		/// <summary>
		/// Hack the Datadog tracing library to not block during shutdown of tests.
		/// Without this fix, the lib will try to send traces to a host that isn't running and block for +20 secs
		///
		/// Since so many of the interfaces and classes in the lib are internal it was difficult to replace Tracer.Instance
		/// </summary>
		private static void PatchDatadogWriter()
		{
			if (s_datadogWriterPatched)
			{
				return;
			}

			s_datadogWriterPatched = true;

			/*			string msg = "Unable to patch Datadog agent writer! Tests will still work, but shutdown will block for +20 seconds.";

						FieldInfo? agentWriterField = Datadog.Trace.Tracer.Instance.GetType().GetField("_agentWriter", BindingFlags.NonPublic | BindingFlags.Instance);
						if (agentWriterField == null)
						{
							Console.Error.WriteLine(msg);
							return;
						}

						object? agentWriterInstance = agentWriterField.GetValue(Datadog.Trace.Tracer.Instance);
						if (agentWriterInstance == null)
						{
							Console.Error.WriteLine(msg);
							return;
						}

						FieldInfo? processExitField = agentWriterInstance.GetType().GetField("_processExit", BindingFlags.NonPublic | BindingFlags.Instance);
						if (processExitField == null)
						{
							Console.Error.WriteLine(msg);
							return;
						}

						TaskCompletionSource<bool>? processExitInstance = (TaskCompletionSource<bool>?)processExitField.GetValue(agentWriterInstance);
						if (processExitInstance == null)
						{
							Console.Error.WriteLine(msg);
							return;
						}

						processExitInstance.TrySetResult(true);
			*/
		}
	}
}
