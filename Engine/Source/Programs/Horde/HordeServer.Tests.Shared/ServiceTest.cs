// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests
{
	/// <summary>
	/// Implements a test framework which manages a collection of services
	/// </summary>
	public class ServiceTest : IAsyncDisposable
	{
		private ServiceProvider? _serviceProvider = null;

		public IServiceProvider ServiceProvider
		{
			get
			{
				if (_serviceProvider == null)
				{
					IServiceCollection services = new ServiceCollection();
					ConfigureServices(services);

					_serviceProvider = services.BuildServiceProvider();
				}
				return _serviceProvider;
			}
		}

		public virtual async ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);

			if (_serviceProvider != null)
			{
				await _serviceProvider.DisposeAsync();
				_serviceProvider = null;
			}
		}

		protected virtual void ConfigureSettings(ServerSettings settings)
		{
		}

		protected virtual void ConfigureServices(IServiceCollection services)
		{
		}
	}
}
