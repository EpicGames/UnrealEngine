// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using MongoDB.Bson;
using MongoDB.Driver;

namespace HordeServer.Tests;

/// <summary>
/// Implementation instantiated per test with a unique database name to sandbox and isolate reads/writes during a test
/// </summary>
public sealed class TestScopedMongoService : IMongoService, IDisposable
{
	public IMongoDatabase Database { get; }
	public bool ReadOnlyMode { get; } = false;

	private readonly string _dbName = $"HordeTest-{Guid.NewGuid()}";
	private readonly MongoClient _mongoClient;
	private readonly IMongoCollection<BsonDocument> _singletonsV1;
	private readonly IMongoCollection<BsonDocument> _singletonsV2;
	private readonly ISet<string> _indexesCreated = new HashSet<string>();
	private readonly object _indexLock = new();
	
	public TestScopedMongoService(MongoClient mongoClient)
	{
		_mongoClient = mongoClient;
		Database = _mongoClient.GetDatabase(_dbName);
		_singletonsV1 = GetCollection<BsonDocument>("Singletons");
		_singletonsV2 = GetCollection<BsonDocument>("SingletonsV2");
	}
	
	/// <inheritdoc/>
	public void Dispose()
	{
		_mongoClient.DropDatabase(_dbName);
	}

	/// <inheritdoc/>
	public MongoClient GetClient()
	{
		return _mongoClient;
	}

	/// <inheritdoc/>
	public IMongoCollection<T> GetCollection<T>(string name)
	{
		return Database.GetCollection<T>(name);
	}

	/// <inheritdoc/>
	public IMongoCollection<T> GetCollection<T>(string name, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keysFunc, bool unique = false)
	{
		return GetCollection(name, [MongoIndex.Create<T>(keysFunc, unique)]);
	}

	/// <inheritdoc/>
	public IMongoCollection<T> GetCollection<T>(string name, IEnumerable<MongoIndex<T>> indexes)
	{
		IMongoCollection<T> collection = GetCollection<T>(name);
		AddIndexes(name, collection, indexes.ToArray());
		return collection;
	}

	private void AddIndexes<T>(string collectionName, IMongoCollection<T> collection, MongoIndex<T>[] indexes)
	{
		lock (_indexLock)
		{
			if (_indexesCreated.Contains(collectionName))
			{
				return;
			}
		
			foreach (MongoIndex<T> index in indexes)
			{
				CreateIndexOptions<T> options = new CreateIndexOptions<T>
				{
					Name = index.Name,
					Unique = index.Unique,
					Sparse = index.Sparse,
					Background = true
				};
				collection.Indexes.CreateOne(new CreateIndexModel<T>(index.Keys, options));
			}

			_indexesCreated.Add(collectionName);
		}
	}

	/// <inheritdoc/>
	public Task<T> GetSingletonAsync<T>(CancellationToken cancellationToken) where T : SingletonBase, new()
	{
		return GetSingletonAsync(() => new T(), cancellationToken);
	}

	/// <inheritdoc/>
	public Task<T> GetSingletonAsync<T>(Func<T> constructor, CancellationToken cancellationToken) where T : SingletonBase, new()
	{
		return MongoService.GetSingletonStaticAsync(_singletonsV1, _singletonsV2, constructor, cancellationToken);
	}

	/// <inheritdoc/>
	public Task UpdateSingletonAsync<T>(Action<T> updater, CancellationToken cancellationToken) where T : SingletonBase, new()
	{
		bool Update(T instance)
		{
			updater(instance);
			return true;
		}
		return UpdateSingletonAsync<T>(Update, cancellationToken);
	}

	/// <inheritdoc/>
    public async Task UpdateSingletonAsync<T>(Func<T, bool> updater, CancellationToken cancellationToken) where T : SingletonBase, new()
	{
		for (; ; )
		{
			T document = await GetSingletonAsync(() => new T(), cancellationToken);
			if (!updater(document))
			{
				break;
			}
			if (await TryUpdateSingletonAsync(document, cancellationToken))
			{
				break;
			}
		}
	}

	/// <inheritdoc/>
    public Task<bool> TryUpdateSingletonAsync<T>(T singletonObject, CancellationToken cancellationToken) where T : SingletonBase
    {
	    return MongoService.TryUpdateSingletonStaticAsync(_singletonsV2, singletonObject, cancellationToken);
	}
	
	/// <inheritdoc/>
	public Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = new CancellationToken())
	{
		return Task.FromResult(HealthCheckResult.Healthy());
	}
}