// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.FileProviders;
using Microsoft.Extensions.Hosting;

namespace HordeServer.Tests
{
	public class WebHostEnvironmentStub : IHostEnvironment
	{
		public string ApplicationName { get; set; } = "HordeTest";
		public IFileProvider ContentRootFileProvider { get; set; }
		public string ContentRootPath { get; set; }
		public string EnvironmentName { get; set; } = "Testing";

		public WebHostEnvironmentStub()
		{
			ContentRootPath = Directory.CreateTempSubdirectory("HordeTest").FullName;
			ContentRootFileProvider = new PhysicalFileProvider(ContentRootPath);
		}
	}
}
