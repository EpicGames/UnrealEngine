// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;
using HordeServer.Accounts;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Accounts
{
	[TestClass]
	public class AccountCollectionTests : ServerTestSetup
	{
		[TestMethod]
		public async Task CaseInsensitiveAsync()
		{
			IAccountCollection accountCollection = ServiceProvider.GetRequiredService<IAccountCollection>();

			IAccount? account = await accountCollection.FindByLoginAsync("bob");
			Assert.IsNull(account);

			await accountCollection.CreateAdminAccountAsync("", CancellationToken.None);

			account = await accountCollection.FindByLoginAsync("admin");
			Assert.IsNotNull(account);

			IAccount? account2 = await accountCollection.FindByLoginAsync("Admin");
			Assert.IsNotNull(account2);
			Assert.AreEqual(account.Id, account2.Id);

			account = await accountCollection.CreateAsync(new CreateAccountOptions("Robert", "bob"));
			Assert.IsNotNull(account);

			account2 = await accountCollection.FindByLoginAsync("Bob");
			Assert.IsNotNull(account2);
			Assert.AreEqual(account.Id, account2.Id);

			account2 = await account2.UpdateAsync(new UpdateAccountOptions(Name: "ROBERT", Login: "BOB"));
			Assert.IsNotNull(account2);

			account2 = await accountCollection.FindByLoginAsync("bob");
			Assert.IsNotNull(account2);
			Assert.AreEqual(account.Id, account2.Id);

			await Assert.ThrowsExactlyAsync<LoginAlreadyTakenException>(() => accountCollection.CreateAsync(new CreateAccountOptions("Bill", "bob")));
			await Assert.ThrowsExactlyAsync<LoginAlreadyTakenException>(() => account2.UpdateAsync(new UpdateAccountOptions { Login = "admin" }));
		}
	}
}
