// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using System.Net.Http;
using System.Text;
using System.Threading.Tasks;
using HordeServer.Accounts;
using HordeServer.Users;
using Microsoft.AspNetCore.WebUtilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA2234 // Pass system uri objects instead of strings
#pragma warning disable CA1307 // Specify StringComparison for clarity

namespace HordeServer.Tests.Accounts;

[TestClass]
public class AccountControllerTest : IAsyncDisposable
{
	private readonly FakeHordeWebApp _app;
	private readonly IAccountCollection _accountCollection;
	private IAccount _account1 = null!;

	public AccountControllerTest()
	{
		Dictionary<string, string> settings = new()
		{
			{ "Horde:AuthMethod", "Horde" },
		};
		_app = new FakeHordeWebApp(settings: settings);
		_accountCollection = _app.ServiceProvider.GetRequiredService<IAccountCollection>();
	}

	public async ValueTask DisposeAsync()
	{
		await _app.DisposeAsync();
		GC.SuppressFinalize(this);
	}

	[TestInitialize]
	public async Task InitAsync()
	{
		List<IUserClaim> claims = new()
		{
			new UserClaim("myClaimType1", "myClaimValue1"),
			new UserClaim("myClaimType2", "myClaimValue2")
		};
		_account1 = await _accountCollection.CreateAsync(new CreateAccountOptions("name1", "login1", claims, Email: "foo@horde", Description: "desc1", Password: "pass1"));
	}

	[TestMethod]
	public async Task NotLoggedIn_AccountPageShowsLoginLinkAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		HttpResponseMessage res = await httpClient.GetAsync("account");
		Assert.AreEqual(HttpStatusCode.OK, res.StatusCode);
		Assert.IsTrue((await res.Content.ReadAsStringAsync()).Contains("<b>Login</b>"));
	}

	[TestMethod]
	public async Task NotLoggedIn_DefaultLoginRedirectsToUserPassFormAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		HttpResponseMessage res = await httpClient.GetAsync("account/login");
		Assert.AreEqual(HttpStatusCode.Found, res.StatusCode);
		Assert.AreEqual("/account/login/horde?ReturnUrl=%2Faccount", res.Headers.Location!.PathAndQuery);
	}

	[TestMethod]
	public async Task NotLoggedIn_UserPassFormAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		HttpResponseMessage res = await httpClient.GetAsync("account/login/horde");
		Assert.AreEqual(HttpStatusCode.OK, res.StatusCode);
		Assert.IsTrue((await res.Content.ReadAsStringAsync()).Contains("<form action"));
	}

	[TestMethod]
	public async Task NotLoggedIn_BadCredentialsShowsErrorAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		{
			HttpResponseMessage res = await LoginAsync(httpClient, "does-not-exist", "foo");
			Assert.AreEqual(HttpStatusCode.BadRequest, res.StatusCode);
			Assert.IsTrue((await res.Content.ReadAsStringAsync()).Contains("Invalid username or password"));
		}

		{
			HttpResponseMessage res = await LoginAsync(httpClient, _account1.Login, "wrong-password");
			Assert.AreEqual(HttpStatusCode.BadRequest, res.StatusCode);
			Assert.IsTrue((await res.Content.ReadAsStringAsync()).Contains("Invalid username or password"));
		}
	}

	[TestMethod]
	public async Task NotLoggedIn_CorrectCredentialsLogsInAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		HttpResponseMessage res = await LoginAsync(httpClient, _account1.Login, "pass1");
		Assert.AreEqual(HttpStatusCode.Redirect, res.StatusCode);
		Assert.AreEqual("/", res.Headers.Location!.ToString());
	}

	[TestMethod]
	public async Task LoggedIn_ShowsCredentialsAsync()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		await LoginAsync(httpClient, _account1.Login, "pass1");
		HttpResponseMessage res2 = await httpClient.GetAsync("account");
		Assert.AreEqual(HttpStatusCode.OK, res2.StatusCode);

		string content = await res2.Content.ReadAsStringAsync();
		Assert.IsTrue(content.Contains("myClaimType1") && content.Contains("myClaimValue1"));
		Assert.IsTrue(content.Contains("myClaimType2") && content.Contains("myClaimValue2"));
	}

	private static async Task<HttpResponseMessage> LoginAsync(HttpClient httpClient, string username, string password, string? redirectUrl = null)
	{
		using StringContent sc = new($"username={username}&password={password}", Encoding.UTF8, "application/x-www-form-urlencoded");
		string url = "account/login/horde";
		if (redirectUrl != null)
		{
			url = QueryHelpers.AddQueryString("account/login/horde", "ReturnUrl", redirectUrl);
		}
		HttpResponseMessage response = await httpClient.PostAsync(url, sc);
		return response;
	}
}