// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Collections.Specialized;
using System.IdentityModel.Tokens.Jwt;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Json;
using System.Threading.Tasks;
using System.Web;
using HordeServer.Accounts;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA2234 // Pass system uri objects instead of strings
#pragma warning disable CA1307 // Specify StringComparison for clarity

namespace HordeServer.Tests.Accounts;

[TestClass]
public class OAuthControllerTest : IAsyncDisposable
{
	private readonly FakeHordeWebApp _app;
	private readonly IAccountCollection _accountCollection;

	public OAuthControllerTest()
	{
		Dictionary<string, string> settings = new() { { "Horde:AuthMethod", "Horde" } };
		_app = new FakeHordeWebApp(settings: settings);
		_accountCollection = _app.ServiceProvider.GetRequiredService<IAccountCollection>();
	}

	public async ValueTask DisposeAsync()
	{
		await _app.DisposeAsync();
		GC.SuppressFinalize(this);
	}

	[TestInitialize]
	public async Task InitAsync()
	{
		List<IUserClaim> claims = new()
		{
			new UserClaim("myClaimType1", "myClaimValue1"),
			new UserClaim("myClaimType2", "myClaimValue2"),
			new UserClaim(HordeClaimTypes.Group, "group1"),
			new UserClaim(HordeClaimTypes.Group, "group2"),
			new UserClaim(HordeClaimTypes.Group, "group3")
		};
		await _accountCollection.CreateAsync(new CreateAccountOptions("name1", "login1", claims, Email: "foo@horde", Description: "desc1", Password: "pass1"));
	}

	[TestMethod]
	public async Task AuthCodeFlow_ExchangeCodeForToken_ReturnsValidTokenWithClaims_Async()
	{
		using HttpClient httpClient = _app.CreateHttpClient(allowAutoRedirect: false);
		string authCode = await GetAuthCodeByLoggingInAsync(httpClient, "login1", "pass1");
		
		using FormUrlEncodedContent formData = new (
		[
			new KeyValuePair<string, string>("grant_type", "authorization_code"),
			new KeyValuePair<string, string>("code", authCode),
		]);
        
		HttpResponseMessage res = await httpClient.PostAsync("api/v1/oauth2/token", formData);
		Assert.AreEqual(HttpStatusCode.OK, res.StatusCode);
		OAuthGetTokenResponse? getTokenRes = await res.Content.ReadFromJsonAsync<OAuthGetTokenResponse>();
		Assert.IsNotNull(getTokenRes);
			
		JwtSecurityTokenHandler handler = new ();
		JwtSecurityToken jsonToken = handler.ReadJwtToken(getTokenRes.AccessToken);

		List<string> GetClaimValues(string claimType) => jsonToken.Claims.Where(x => x.Type == claimType).Select(x => x.Value).ToList();
		
		CollectionAssert.AreEquivalent(new [] {"myClaimValue1"}, GetClaimValues("myClaimType1"));
		CollectionAssert.AreEquivalent(new [] {"myClaimValue2"}, GetClaimValues("myClaimType2"));
		CollectionAssert.AreEquivalent(new [] {"group2", "group1", "group3"}, GetClaimValues(HordeClaimTypes.Group));
	}

	private static async Task<string> GetAuthCodeByLoggingInAsync(HttpClient httpClient, string username, string password)
	{
		using FormUrlEncodedContent formData = new (
		[
			new KeyValuePair<string, string>("username", username),
			new KeyValuePair<string, string>("password", password),
		]);
		HttpResponseMessage res = await httpClient.PostAsync("api/v1/oauth2/login?response_type=code&redirect_uri=http://horde/test-redirect", formData);
		
		Assert.AreEqual(HttpStatusCode.Found, res.StatusCode);
		Assert.IsNotNull(res.Headers.Location);
		NameValueCollection queryParams = HttpUtility.ParseQueryString(res.Headers.Location.Query);
		string? code = queryParams["code"];
		Assert.IsNotNull(code);
		return code;
	}
}