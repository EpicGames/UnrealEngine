// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Security.Claims;
using System.Threading.Tasks;
using HordeServer.Authentication;
using HordeServer.ServiceAccounts;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Primitives;
using Microsoft.Extensions.WebEncoders.Testing;
using Microsoft.Net.Http.Headers;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Authentication
{
	[TestClass]
	public class ServiceAccountAuthTest : DatabaseIntegrationTest
	{
		string _serviceAccountToken = null!;

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			services.AddSingleton<ILoggerFactory, LoggerFactory>();
			services.AddSingleton<IServiceAccountCollection, ServiceAccountCollection>();
		}

		[TestInitialize]
		public async Task InitializeAsync()
		{
			IServiceAccountCollection serviceAccountCollection = ServiceProvider.GetRequiredService<IServiceAccountCollection>();
			(_, _serviceAccountToken) = await serviceAccountCollection.CreateAsync(new CreateServiceAccountOptions("myName", "myDesc",
				Claims: new List<IUserClaim> { new UserClaim("myClaim", "myValue"), new UserClaim("foo", "bar") })
				);
		}

		private async Task<ServiceAccountAuthHandler> GetAuthHandlerAsync(string? headerValue)
		{
			ServiceAccountAuthOptions options = new ServiceAccountAuthOptions();

			ILoggerFactory loggerFactory = ServiceProvider.GetRequiredService<ILoggerFactory>();
			ServiceAccountAuthHandler handler = new ServiceAccountAuthHandler(new TestOptionsMonitor<ServiceAccountAuthOptions>(options), loggerFactory, new UrlTestEncoder(), ServiceProvider.GetRequiredService<IServiceAccountCollection>());
			AuthenticationScheme scheme = new AuthenticationScheme(ServiceAccountAuthHandler.AuthenticationScheme, "ServiceAccountAuth", handler.GetType());

			HttpContext httpContext = new DefaultHttpContext();
			if (headerValue != null)
			{
				httpContext.Request.Headers[HeaderNames.Authorization] = new StringValues(headerValue);
			}

			await handler.InitializeAsync(scheme, httpContext);

			return handler;
		}

		[TestMethod]
		public async Task ValidTokenAsync()
		{
			ServiceAccountAuthHandler handler = await GetAuthHandlerAsync($"ServiceAccount {_serviceAccountToken}");
			AuthenticateResult result = await handler.AuthenticateAsync();
			Assert.IsTrue(result.Succeeded);

			handler = await GetAuthHandlerAsync($"ServiceAccount   {_serviceAccountToken}        ");
			result = await handler.AuthenticateAsync();
			Assert.IsTrue(result.Succeeded);
		}

		[TestMethod]
		public async Task InvalidTokenAsync()
		{
			ServiceAccountAuthHandler handler = await GetAuthHandlerAsync("ServiceAccount doesNotExist");
			AuthenticateResult result = await handler.AuthenticateAsync();
			Assert.IsFalse(result.Succeeded);

			// Valid token but bad prefix
			handler = await GetAuthHandlerAsync("SriceAcct myToken");
			result = await handler.AuthenticateAsync();
			Assert.IsFalse(result.Succeeded);
		}

		[TestMethod]
		public async Task NoResultAsync()
		{
			// Valid token but bad prefix
			ServiceAccountAuthHandler handler = await GetAuthHandlerAsync("Bogus myToken");
			AuthenticateResult result = await handler.AuthenticateAsync();
			Assert.IsFalse(result.Succeeded);
			Assert.IsTrue(result.None);

			// No Authorization header at all
			handler = await GetAuthHandlerAsync(null);
			result = await handler.AuthenticateAsync();
			Assert.IsFalse(result.Succeeded);
			Assert.IsTrue(result.None);
		}

		[TestMethod]
		public async Task ClaimsAsync()
		{
			ServiceAccountAuthHandler handler = await GetAuthHandlerAsync($"ServiceAccount {_serviceAccountToken}");
			AuthenticateResult result = await handler.AuthenticateAsync();
			Assert.IsTrue(result.Succeeded);
			Assert.AreEqual(4, result.Ticket!.Principal.Claims.Count());
			Assert.AreEqual("myName", result.Ticket.Principal.FindFirst(ClaimTypes.Name)!.Value);
			Assert.AreEqual("myName", result.Ticket.Principal.FindFirst(HordeClaimTypes.User)!.Value);
			Assert.AreEqual("myValue", result.Ticket!.Principal.FindFirst("myClaim")!.Value);
			Assert.AreEqual("bar", result.Ticket!.Principal.FindFirst("foo")!.Value);
		}
	}
}