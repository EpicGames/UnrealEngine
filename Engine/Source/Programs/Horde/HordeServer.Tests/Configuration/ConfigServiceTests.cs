// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Runtime.InteropServices;
using HordeServer.Configuration;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Configuration;

[TestClass]
public class ConfigServiceTests
{
	[TestMethod]
	public void GetGlobalConfigUriTest()
	{
		// Paths are handled differently depending on OS deep inside Path.* methods in .NET SDK
		// So do different tests depending on platform

		if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
		{
			Console.WriteLine("Ran Windows");
			AssertConfig("globals.json", "C:\\SomeHordeDir", "file:///C:/SomeHordeDir/globals.json");
			AssertConfig("C:\\OtherDir\\globals.json", "C:\\SomeHordeDir", "file:///C:/OtherDir/globals.json");
		}

		if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
		{
			Console.WriteLine("Ran Linux");
			AssertConfig("globals.json", "/app", "file:///app/globals.json");
			AssertConfig("/some/dir/globals.json", "/app", "file:///some/dir/globals.json");
		}

		AssertConfig("//depot/dir1/globals.json", "C:\\SomeHordeDir", "perforce://default//depot/dir1/globals.json");
	}

	private static void AssertConfig(string configPath, string configDir, string expectedPath)
	{
		Uri uri = ConfigService.GetGlobalConfigUri(configPath, configDir);
		Assert.AreEqual(expectedPath, uri.AbsoluteUri);
		Assert.IsTrue(uri.IsAbsoluteUri);
	}
}

