// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Streams;
using HordeServer.Acls;
using HordeServer.Configuration;
using HordeServer.Plugins;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Configuration
{
	[TestClass]
	public class ConfigTests
	{
		class SubObject
		{
			public string? ValueA { get; set; }
			public int ValueB { get; set; }
			public SubObject? ValueC { get; set; }
		}

		[ConfigIncludeRoot]
		class ConfigObject
		{
			public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();
			public string? TestString { get; set; }
			public List<string> TestList { get; set; } = new List<string>();
			public SubObject? TestObject { get; set; }
		}

		[ConfigIncludeRoot]
		class NestedIncludeRoot
		{
			public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();
			public List<string> Items { get; set; } = new List<string>();
		}

		[ConfigIncludeRoot]
		class OuterWithNestedRoot
		{
			public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();
			public List<string> Items { get; set; } = new List<string>();
			public NestedIncludeRoot? Nested { get; set; }
		}

		readonly JsonSerializerOptions _jsonOptions = new JsonSerializerOptions { DefaultIgnoreCondition = System.Text.Json.Serialization.JsonIgnoreCondition.WhenWritingDefault };

		[TestMethod]
		public async Task IncludeTestAsync()
		{
			CancellationToken cancellationToken = CancellationToken.None;
			InMemoryConfigSource source = new InMemoryConfigSource();

			// memory:///bar
			Uri barUri = new Uri("memory:///bar");
			{
				ConfigObject obj = new ConfigObject();
				obj.TestList.Add("secondobj");
				obj.TestObject = new SubObject { ValueB = 123, ValueC = new SubObject { ValueB = 456 } };

				byte[] data2 = JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions);
				source.Add(barUri, data2);
			}

			// memory:///foo
			Uri fooUri = new Uri("memory:///foo");
			{
				ConfigObject obj = new ConfigObject();
				obj.Include.Add(new ConfigInclude { Path = barUri.ToString() });
				obj.TestString = "hello";
				obj.TestList.Add("there");
				obj.TestList.Add("world");
				obj.TestObject = new SubObject { ValueA = "hi", ValueC = new SubObject { ValueA = "yo" } };

				byte[] json1 = JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions);
				source.Add(fooUri, json1);
			}

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["memory"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			ConfigObject result = await context.ReadAsync<ConfigObject>(fooUri, cancellationToken);
			Assert.AreEqual("hello", result.TestString);
			Assert.IsTrue(result.TestList.SequenceEqual(new[] { "secondobj", "there", "world" }));
			Assert.AreEqual("hi", result.TestObject!.ValueA);
			Assert.AreEqual(123, result.TestObject!.ValueB);
			Assert.AreEqual("yo", result.TestObject!.ValueC!.ValueA);
		}

		[TestMethod]
		public async Task FileTestAsync()
		{
			CancellationToken cancellationToken = CancellationToken.None;

			DirectoryReference baseDir = new DirectoryReference("test");
			DirectoryReference.CreateDirectory(baseDir);

			FileConfigSource source = new FileConfigSource(baseDir);

			// file:test/foo
			Uri fooUri = new Uri($"file:///{FileReference.Combine(baseDir, "test.json")}");

			byte[] data;
			{
				ConfigObject obj = new ConfigObject();
				obj.TestString = "hello";
				data = JsonSerializer.SerializeToUtf8Bytes(obj, new JsonSerializerOptions { DefaultIgnoreCondition = System.Text.Json.Serialization.JsonIgnoreCondition.WhenWritingDefault });
			}
			await FileReference.WriteAllBytesAsync(new FileReference(fooUri.LocalPath), data, cancellationToken);

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["file"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			ConfigObject result = await context.ReadAsync<ConfigObject>(fooUri, cancellationToken);
			Assert.AreEqual("hello", result.TestString);

			// Check it returns the same object if the timestamp hasn't changed
			IConfigFile file1 = await source.GetAsync(fooUri, cancellationToken);
			await Task.Delay(TimeSpan.FromSeconds(1));
			IConfigFile file2 = await source.GetAsync(fooUri, cancellationToken);
			Assert.IsTrue(ReferenceEquals(file1, file2));

			// Check it returns a new object if the timestamp HAS changed
			await Task.Delay(TimeSpan.FromSeconds(1));
			await FileReference.WriteAllBytesAsync(new FileReference(fooUri.LocalPath), data, cancellationToken);
			IConfigFile file3 = await source.GetAsync(fooUri, cancellationToken);
			Assert.IsTrue(!ReferenceEquals(file1, file3));
		}

		private static NetworkConfig? GetNetworkConfig(ComputeConfig gc, string ip)
		{
			bool result = gc.TryGetNetworkConfig(IPAddress.Parse(ip), out NetworkConfig? networkConfig);
			return result ? networkConfig : null;
		}

		[TestMethod]
		public void NetworkConfig()
		{
			ComputeConfig gc = new()
			{
				Networks = new List<NetworkConfig>()
				{
					new() { CidrBlock = "10.0.0.0/31", Id = "foo" },
					new() { CidrBlock = "10.0.0.4/30", Id = "bar" },
					new() { CidrBlock = "192.168.0.0/16", Id = "baz" },
				}
			};

			Assert.AreEqual("foo", GetNetworkConfig(gc, "10.0.0.0")!.Id);
			Assert.AreEqual("foo", GetNetworkConfig(gc, "10.0.0.1")!.Id);
			Assert.AreEqual(null, GetNetworkConfig(gc, "10.0.0.2"));
			Assert.AreEqual(null, GetNetworkConfig(gc, "10.0.0.3"));

			Assert.AreEqual("bar", GetNetworkConfig(gc, "10.0.0.4")!.Id);
			Assert.AreEqual("bar", GetNetworkConfig(gc, "10.0.0.5")!.Id);
			Assert.AreEqual("bar", GetNetworkConfig(gc, "10.0.0.6")!.Id);
			Assert.AreEqual("bar", GetNetworkConfig(gc, "10.0.0.7")!.Id);
			Assert.AreEqual(null, GetNetworkConfig(gc, "10.0.0.8"));

			Assert.AreEqual("baz", GetNetworkConfig(gc, "192.168.0.0")!.Id);
			Assert.AreEqual("baz", GetNetworkConfig(gc, "192.168.0.1")!.Id);
			Assert.AreEqual("baz", GetNetworkConfig(gc, "192.168.255.254")!.Id);
			Assert.AreEqual("baz", GetNetworkConfig(gc, "192.168.255.255")!.Id);
			Assert.AreEqual(null, GetNetworkConfig(gc, "192.169.0.0"));
			Assert.AreEqual(null, GetNetworkConfig(gc, "192.169.0.1"));

			Assert.AreEqual(null, GetNetworkConfig(gc, "11.0.0.1"));

			gc = new()
			{
				Networks = new List<NetworkConfig>() { new() { CidrBlock = "0.0.0.0/0", Id = "global" } }
			};
			Assert.AreEqual("global", GetNetworkConfig(gc, "15.3.4.5")!.Id);
		}

		[TestMethod]
		public void WorkspaceConfig()
		{
			Dictionary<string, WorkspaceConfig> inputWorkspaces = new();
			inputWorkspaces["base"] = new WorkspaceConfig { Identifier = "base", Cluster = "myCluster", MinScratchSpace = 111 };
			inputWorkspaces["subType"] = new WorkspaceConfig { Base = "base", Identifier = "subType", ConformDiskFreeSpace = 222 };
			inputWorkspaces["subSubType"] = new WorkspaceConfig { Base = "subType", Identifier = "subSubType" };

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(new ProjectConfig { Streams = [new StreamConfig { WorkspaceTypes = inputWorkspaces }]});

			GlobalConfig gc = new();
			gc.Plugins.AddComputeConfig(new ComputeConfig());
			gc.Plugins.AddBuildConfig(buildConfig);
			gc.PostLoad(new ServerSettings(), Array.Empty<ILoadedPlugin>(), Array.Empty<IDefaultAclModifier>());

			Dictionary<string,WorkspaceConfig> workspaces = gc.Plugins.GetBuildConfig().Projects[0].Streams[0].WorkspaceTypes;
			Assert.AreEqual(3, workspaces.Count);
			Assert.AreEqual(111, workspaces["subType"].MinScratchSpace);
			Assert.AreEqual(111, workspaces["subSubType"].MinScratchSpace);
			Assert.AreEqual(222, workspaces["subSubType"].ConformDiskFreeSpace);
		}

		[TestMethod]
		public void WorkspaceInheritFromProjectConfig()
		{
			List<string> autoSdkViews = ["foo", "bar"];

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(new ProjectConfig
			{
				WorkspaceTypes =
				{
					{ "project1", new WorkspaceConfig { AutoSdkView = autoSdkViews } },
					{ "project2", new WorkspaceConfig { ConformDiskFreeSpace = 111 } },
					{ "project3", new WorkspaceConfig { MinScratchSpace = 222 } },
				},
				Streams = [
					new StreamConfig { WorkspaceTypes =
					{
						{"stream1", new WorkspaceConfig { Base = "project1", Stream = "myStream" }},
						{"project3", new WorkspaceConfig { Stream = "otherStream" }}
					}}
				]
			});

			GlobalConfig gc = new();
			gc.Plugins.AddComputeConfig(new ComputeConfig());
			gc.Plugins.AddBuildConfig(buildConfig);
			gc.PostLoad(new ServerSettings(), Array.Empty<ILoadedPlugin>(), Array.Empty<IDefaultAclModifier>());

			Dictionary<string, WorkspaceConfig> workspaces = gc.Plugins.GetBuildConfig().Projects[0].Streams[0].WorkspaceTypes;
			Assert.AreEqual(4, workspaces.Count);
			
			// Project-defined streams
			CollectionAssert.AreEquivalent(autoSdkViews, workspaces["project1"].AutoSdkView);
			Assert.AreEqual(111, workspaces["project2"].ConformDiskFreeSpace);
			Assert.AreEqual(111, workspaces["project2"].ConformDiskFreeSpace);
			
			// project3 should be overridden by stream workspace type with the same name
			Assert.AreEqual("otherStream", workspaces["project3"].Stream);
			Assert.IsNull(workspaces["project3"].MinScratchSpace);
			
			// stream1 inherits from project1 workspace type
			CollectionAssert.AreEquivalent(autoSdkViews, workspaces["stream1"].AutoSdkView);
			Assert.AreEqual("myStream", workspaces["stream1"].Stream);
		}

		[TestMethod]
		public void ConfigMergeAppend_NoInfiniteLoop()
		{
			// Referencing same workspace base from multiple streams should not cause an infinite loop
			// ConfigObject.MergeDefaults did not handle this
			
			BuildConfig buildConfig = new ();
			buildConfig.Projects.Add(new ProjectConfig
			{
				WorkspaceTypes =
				{
					{ "base", new WorkspaceConfig { View = ["myView"] } },
					{ "foo", new WorkspaceConfig { Identifier = "myFoo", Base = "base" } },
				},
				Streams = [
					new StreamConfig { Id = new StreamId("stream1"), WorkspaceTypes = { {"stream1", new WorkspaceConfig { Base = "foo", Stream = "myStream1" } } } },
					new StreamConfig { Id = new StreamId("stream2"), WorkspaceTypes = { {"stream2", new WorkspaceConfig { Base = "foo", Stream = "myStream2" } } } }
				]
			});
			
			GlobalConfig gc = new();
			gc.Plugins.AddComputeConfig(new ComputeConfig());
			gc.Plugins.AddBuildConfig(buildConfig);
			gc.PostLoad(new ServerSettings(), Array.Empty<ILoadedPlugin>(), Array.Empty<IDefaultAclModifier>());
		}

		class ObjectValue
		{
			public string Value { get; set; } = "";
		}

		[ConfigMacroScope]
		class BaseMacroScope
		{
			public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();
			public string Value { get; set; } = "";
			public List<string> ListValue { get; set; } = new List<string>();
			public ObjectValue ObjectValue { get; set; } = new ObjectValue();
			public BaseMacroScope? ChildScope { get; set; }
		}

		[TestMethod]
		public async Task DiamondIncludeTestAsync()
		{
			// Tests that diamond includes (A includes B and C, both B and C include D) only
			// merge D's content once, preventing duplicate array entries and dictionary key errors.
			CancellationToken cancellationToken = CancellationToken.None;
			InMemoryConfigSource source = new InMemoryConfigSource();

			// memory:///shared - the shared base included by both child1 and child2
			Uri sharedUri = new Uri("memory:///shared");
			{
				ConfigObject obj = new ConfigObject();
				obj.TestList.Add("shared-item");
				obj.TestObject = new SubObject { ValueA = "shared" };
				source.Add(sharedUri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			// memory:///child1 - includes shared
			Uri child1Uri = new Uri("memory:///child1");
			{
				ConfigObject obj = new ConfigObject();
				obj.Include.Add(new ConfigInclude { Path = sharedUri.ToString() });
				obj.TestList.Add("child1-item");
				source.Add(child1Uri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			// memory:///child2 - also includes shared (diamond)
			Uri child2Uri = new Uri("memory:///child2");
			{
				ConfigObject obj = new ConfigObject();
				obj.Include.Add(new ConfigInclude { Path = sharedUri.ToString() });
				obj.TestList.Add("child2-item");
				source.Add(child2Uri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			// memory:///root - includes child1 and child2
			Uri rootUri = new Uri("memory:///root");
			{
				ConfigObject obj = new ConfigObject();
				obj.Include.Add(new ConfigInclude { Path = child1Uri.ToString() });
				obj.Include.Add(new ConfigInclude { Path = child2Uri.ToString() });
				obj.TestList.Add("root-item");
				source.Add(rootUri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["memory"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			ConfigObject result = await context.ReadAsync<ConfigObject>(rootUri, cancellationToken);

			// "shared-item" should only appear once despite being included via both child1 and child2
			Assert.IsTrue(result.TestList.SequenceEqual(new[] { "shared-item", "child1-item", "child2-item", "root-item" }));
			Assert.AreEqual("shared", result.TestObject!.ValueA);

			// A warning should be generated about the diamond include, identifying both includers
			Assert.AreEqual(1, context.Warnings.Count);
			Assert.IsTrue(context.Warnings[0].Contains("Diamond include", StringComparison.Ordinal));
			Assert.IsTrue(context.Warnings[0].Contains(sharedUri.ToString(), StringComparison.Ordinal));
			Assert.IsTrue(context.Warnings[0].Contains(child1Uri.ToString(), StringComparison.Ordinal), "Warning should identify the first includer");
			Assert.IsTrue(context.Warnings[0].Contains(child2Uri.ToString(), StringComparison.Ordinal), "Warning should identify the duplicate includer");
			Assert.IsTrue(context.Warnings[0].Contains("Consider moving", StringComparison.Ordinal));
		}

		[TestMethod]
		public async Task CrossRootIncludeTestAsync()
		{
			// Tests that different [ConfigIncludeRoot] types processed in the same ConfigContext
			// can independently include the same file. The outer root and its nested root property
			// should both get the shared file's content without interference.
			CancellationToken cancellationToken = CancellationToken.None;
			InMemoryConfigSource source = new InMemoryConfigSource();

			// Shared file included by both outer and nested roots
			Uri sharedUri = new Uri("memory:///shared");
			{
				NestedIncludeRoot obj = new NestedIncludeRoot();
				obj.Items.Add("shared-item");
				source.Add(sharedUri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			// Outer config that includes shared AND has a nested property that also includes shared
			Uri outerUri = new Uri("memory:///outer");
			{
				OuterWithNestedRoot obj = new OuterWithNestedRoot();
				obj.Include.Add(new ConfigInclude { Path = sharedUri.ToString() });
				obj.Items.Add("outer-item");
				// The nested property is inline JSON, its Include will be processed as a separate IncludeRoot
				obj.Nested = new NestedIncludeRoot();
				obj.Nested.Include.Add(new ConfigInclude { Path = sharedUri.ToString() });
				obj.Nested.Items.Add("inner-item");
				source.Add(outerUri, JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions));
			}

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["memory"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			OuterWithNestedRoot result = await context.ReadAsync<OuterWithNestedRoot>(outerUri, cancellationToken);

			// Outer should have shared-item merged from the include
			Assert.IsTrue(result.Items.SequenceEqual(new[] { "shared-item", "outer-item" }));

			// Nested should ALSO have shared-item despite the outer root already including it.
			// Different include roots should have independent dedup scopes.
			Assert.IsNotNull(result.Nested);
			Assert.IsTrue(result.Nested!.Items.SequenceEqual(new[] { "shared-item", "inner-item" }));

			// No diamond warnings should be generated since these are independent include roots
			Assert.AreEqual(0, context.Warnings.Count);
		}

		[TestMethod]
		public async Task MacroTestAsync()
		{
			CancellationToken cancellationToken = CancellationToken.None;
			InMemoryConfigSource source = new InMemoryConfigSource();

			// memory:///bar
			Uri fooUri = new Uri("memory:///foo");
			{
				BaseMacroScope obj = new BaseMacroScope();
				obj.Macros.Add(new ConfigMacro { Name = "MacroName", Value = "MacroValue" });
				obj.Value = "This is a macro $(MacroName)";
				obj.ListValue.Add("List element macro $(MacroName)");
				obj.ObjectValue = new ObjectValue { Value = "Object macro $(MacroName)" };

				byte[] data2 = JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions);
				source.Add(fooUri, data2);
			}

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["memory"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			BaseMacroScope result = await context.ReadAsync<BaseMacroScope>(fooUri, cancellationToken);
			Assert.AreEqual("This is a macro MacroValue", result.Value);
			Assert.AreEqual("List element macro MacroValue", result.ListValue[0]);
			Assert.AreEqual("Object macro MacroValue", result.ObjectValue.Value);
		}

		[TestMethod]
		public async Task NestedMacroTestAsync()
		{
			CancellationToken cancellationToken = CancellationToken.None;
			InMemoryConfigSource source = new InMemoryConfigSource();

			// memory:///bar
			Uri fooUri = new Uri("memory:///foo");
			{
				BaseMacroScope obj = new BaseMacroScope();
				obj.Macros.Add(new ConfigMacro { Name = "MacroName", Value = "MacroValue" });

				obj.ChildScope = new BaseMacroScope();
				obj.ChildScope.Macros.Add(new ConfigMacro { Name = "MacroName2", Value = "MacroValue2" });
				obj.ChildScope.Value = "This is a macro $(MacroName) $(MacroName2)";

				obj.Value = "This is a macro $(MacroName) $(MacroName2)";

				byte[] data2 = JsonSerializer.SerializeToUtf8Bytes(obj, _jsonOptions);
				source.Add(fooUri, data2);
			}

			Dictionary<string, IConfigSource> sources = new Dictionary<string, IConfigSource>();
			sources["memory"] = source;

			ConfigContext context = new ConfigContext(_jsonOptions, sources, NullLogger.Instance);

			BaseMacroScope result = await context.ReadAsync<BaseMacroScope>(fooUri, cancellationToken);
			Assert.AreEqual("This is a macro MacroValue $(MacroName2)", result.Value);
			Assert.AreEqual("This is a macro MacroValue MacroValue2", result.ChildScope!.Value);
		}
	}
}
