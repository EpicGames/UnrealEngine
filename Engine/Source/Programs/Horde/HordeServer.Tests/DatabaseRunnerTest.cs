// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Server;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests
{
	[TestClass]
	public class DatabaseRunnerTest
	{
		[TestMethod]
		public void RunMongoDbTest()
		{
			using MongoDbRunnerLocal runner = new MongoDbRunnerLocal();
			runner.Start();
			Thread.Sleep(100);
			runner.Stop();
		}

		[TestMethod]
		[Ignore("Very slow to stop on Windows. Likely has to do with the specific Windows-version of redis-server.exe")]
		public async Task RunRedisTestAsync()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				await using RedisProcess runner = new RedisProcess(NullLogger.Instance);
				runner.Start("--save \"\" --appendonly no");
				await Task.Delay(100);
				await runner.StopAsync();
			}
		}
	}
}
