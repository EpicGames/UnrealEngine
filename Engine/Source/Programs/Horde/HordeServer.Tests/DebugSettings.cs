// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Tests
{
	/// <summary>
	/// Settings used for debug tests
	///
	/// Put into a separate file to avoid accidental commit of credentials.
	/// </summary>
	public static class DebugSettings
	{
		public static readonly string DbUsername = "username-not-set";
		public static readonly string DbPassword = "password-not-set";
		public static readonly string DbHostname = "";
	}
}