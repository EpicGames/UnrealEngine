<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <NeutralLanguage>en-US</NeutralLanguage>
    <GenerateProgramFile>false</GenerateProgramFile>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Testing" Version="10.0.6" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Serialization\EpicGames.Serialization.csproj" />
    <ProjectReference Include="..\HordeServer.Tests.Shared\HordeServer.Tests.Shared.csproj" />
    <ProjectReference Include="..\HordeAgent\HordeAgent.csproj">
      <Aliases>HordeAgent</Aliases>
    </ProjectReference>
    <ProjectReference Include="..\Drivers\JobDriver\JobDriver.csproj">
      <Aliases>JobDriver</Aliases>
    </ProjectReference>
    <ProjectReference Include="..\HordeServer\HordeServer.csproj" />
    <ProjectReference Include="..\Plugins\Analytics\HordeServer.Analytics.Tests\HordeServer.Analytics.Tests.csproj" />
    <ProjectReference Include="..\Plugins\Storage\HordeServer.Storage.Tests\HordeServer.Storage.Tests.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Compile Update="Properties\Resources.Designer.cs">
      <DesignTime>True</DesignTime>
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <EmbeddedResource Update="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
    </EmbeddedResource>
  </ItemGroup>

  <ItemGroup>
    <Folder Include="Resources\" />
    <Folder Include="Server\Notices\" />
  </ItemGroup>

</Project>
