// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Plugins
{
	[TestClass]
	public class PluginsTest
	{
		private static class PluginsWithDependencies
		{
			[Plugin("A", DependsOn = ["C"])]
			public class PluginA : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}

			[Plugin("B")]
			public class PluginB : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}

			[Plugin("C", DependsOn = ["B"])]
			public class PluginC : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}
		}

		private static class PluginsWithCycle
		{
			[Plugin("A", DependsOn = ["B"])]
			public class PluginA : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}

			[Plugin("B", DependsOn = ["A"])]
			public class PluginB : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}
		}

		private static class PluginsWithoutDependencies
		{
			[Plugin("A")]
			public class PluginA : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}

			[Plugin("B")]
			public class PluginB : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}

			[Plugin("C")]
			public class PluginC : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}
		}

		private static class PluginsMissingDependencies
		{
			[Plugin("A", DependsOn = ["Missing"])]
			public class PluginA : IPluginStartup
			{
				public void Configure(IApplicationBuilder app) { }
				public void ConfigureServices(IServiceCollection serviceCollection) { }
			}
		}

		[TestMethod]
		public void TopologicalSort()
		{
			PluginCollection pluginCollection = new();
			pluginCollection.Add(typeof(PluginsWithDependencies.PluginA));
			pluginCollection.Add(typeof(PluginsWithDependencies.PluginB));
			pluginCollection.Add(typeof(PluginsWithDependencies.PluginC));
			IReadOnlyList<ILoadedPlugin> sorted = PluginCollection.GetTopologicalSort(pluginCollection.LoadedPlugins);
			Assert.AreEqual("b", sorted[0].Name.ToString());
			Assert.AreEqual("c", sorted[1].Name.ToString());
			Assert.AreEqual("a", sorted[2].Name.ToString());
		}

		[TestMethod]
		public void TopologicalSortEmpty()
		{
			PluginCollection pluginCollection = new();
			pluginCollection.Add(typeof(PluginsWithoutDependencies.PluginA));
			pluginCollection.Add(typeof(PluginsWithoutDependencies.PluginB));
			pluginCollection.Add(typeof(PluginsWithoutDependencies.PluginC));
			IReadOnlyList<ILoadedPlugin> sorted = PluginCollection.GetTopologicalSort(pluginCollection.LoadedPlugins);
			Assert.AreEqual("a", sorted[0].Name.ToString());
			Assert.AreEqual("b", sorted[1].Name.ToString());
			Assert.AreEqual("c", sorted[2].Name.ToString());
		}

		[TestMethod]
		public void TopologicalSortErrorWithCycle()
		{
			PluginCollection pluginCollection = new();
			pluginCollection.Add(typeof(PluginsWithCycle.PluginA));
			pluginCollection.Add(typeof(PluginsWithCycle.PluginB));
			Assert.ThrowsExactly<InvalidOperationException>(() => _ = PluginCollection.GetTopologicalSort(pluginCollection.LoadedPlugins));
		}

		[TestMethod]
		public void TopologicalSortMissingDependency()
		{
			PluginCollection pluginCollection = new();
			pluginCollection.Add(typeof(PluginsMissingDependencies.PluginA));
			IReadOnlyList<ILoadedPlugin> sorted = PluginCollection.GetTopologicalSort(pluginCollection.LoadedPlugins);
			Assert.AreEqual("a", sorted[0].Name.ToString());
		}
	}
}
