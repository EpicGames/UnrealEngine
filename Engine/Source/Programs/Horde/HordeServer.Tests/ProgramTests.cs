// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Security.Cryptography.X509Certificates;
using EpicGames.Core;
using HordeServer.Commands;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests
{
	[TestClass]
	public class ProgramTests
	{
		[TestMethod]
		public void TestReadGrpcCertificate()
		{
			string friendlyName = "A testing cert";
			byte[] tempCertData = CertificateUtils.CreateSelfSignedCert("testing.epicgames.com", friendlyName);
			string tempCertPath = Path.GetTempFileName();
			try
			{
				File.WriteAllBytes(tempCertPath, tempCertData);
				using X509Certificate2 originalCert = X509CertificateLoader.LoadPkcs12(tempCertData, null);
				string expectedThumbprint = originalCert.Thumbprint;

				// No cert given
				Assert.IsNull(ServerCommand.ReadGrpcCertificate(new() { ServerPrivateCert = null }));

				// Cert as file path
				{
					using X509Certificate2? cert = ServerCommand.ReadGrpcCertificate(new() { ServerPrivateCert = tempCertPath });
					Assert.IsNotNull(cert);
					Assert.AreEqual(expectedThumbprint, cert!.Thumbprint);
				}

				// Cert as base64 data
				{
					string tempCertBase64 = Convert.ToBase64String(tempCertData);
					using X509Certificate2? cert = ServerCommand.ReadGrpcCertificate(new() { ServerPrivateCert = "base64:" + tempCertBase64 });
					Assert.IsNotNull(cert);
					Assert.AreEqual(expectedThumbprint, cert!.Thumbprint);
				}
			}
			finally
			{
				try
				{
					File.Delete(tempCertPath);
				}
				catch (IOException) { }
				catch (UnauthorizedAccessException) { }
			}
		}
	}
}
