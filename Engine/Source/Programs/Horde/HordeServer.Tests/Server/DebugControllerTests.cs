// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using System.Net.Http;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA2234 // Pass system uri objects instead of strings

namespace HordeServer.Tests.Server;

[TestClass]
public class DebugControllerTests
{
	[TestMethod]
	public async Task DebugEndpointsEnabledAsync()
	{
		await using FakeHordeWebApp app = CreateApp(true);
		using HttpResponseMessage response = await app.CreateHttpClient().GetAsync("/api/v1/debug/aclscopes");
		response.EnsureSuccessStatusCode();
	}
	
	[TestMethod]
	public async Task DebugEndpointsDisabledAsync()
	{
		await using FakeHordeWebApp app = CreateApp(null);
		using HttpResponseMessage response = await app.CreateHttpClient().GetAsync("/api/v1/debug/aclscopes");
		Assert.AreEqual(HttpStatusCode.Forbidden, response.StatusCode);
	}
	
	[TestMethod]
	public async Task DebugEndpointsDisabledByDefaultAsync()
	{
		await using FakeHordeWebApp app = CreateApp(null);
		using HttpResponseMessage response = await app.CreateHttpClient().GetAsync("/api/v1/debug/aclscopes");
		Assert.AreEqual(HttpStatusCode.Forbidden, response.StatusCode);
	}
	
	private static FakeHordeWebApp CreateApp(bool? enableDebugEndpoints)
	{
		Dictionary<string, string> settings = new();
		if (enableDebugEndpoints != null)
		{
			settings["Horde:EnableDebugEndpoints"] = Convert.ToString(enableDebugEndpoints.Value);
		}
		return new FakeHordeWebApp(settings: settings);
	}
}

