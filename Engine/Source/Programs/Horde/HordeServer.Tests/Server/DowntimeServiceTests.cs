// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Globalization;
using System.Text.Json;
using System.Threading.Tasks;
using HordeCommon;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class DowntimeServiceTest
	{
		private static ScheduledDowntime GetDowntimeFromJson(string json)
		{
			ScheduledDowntime scheduledDowntime = JsonSerializer.Deserialize<ScheduledDowntime>(json, JsonUtils.DefaultSerializerOptions)!;
			GlobalConfig config = new();
			config.Downtime.Add(scheduledDowntime);
			config.PostLoad(new ServerSettings(), [], []);
			return scheduledDowntime;
		}

		private static DateTimeOffset GetNow(int year, int month, int day, int hour, int minute)
		{
			DateTime clock = new(year, month, day, hour, minute, 0, DateTimeKind.Utc);
			DateTimeOffset now = TimeZoneInfo.ConvertTime(new DateTimeOffset(clock), TimeZoneInfo.Utc);
			return now;
		}

		private static DateTimeOffset DateTimeOffsetParse(string input)
		{
			return DateTimeOffset.Parse(input, CultureInfo.InvariantCulture);
		}

		[TestMethod]
		public async Task AdvanceToDowntimeIsActiveAsync()
		{
			FakeClock clock = new();
			GlobalConfig config = new();
			ScheduledDowntime downtime = new()
			{
				StartTime = clock.UtcNow + TimeSpan.FromSeconds(30),
				FinishTime = clock.UtcNow + TimeSpan.FromSeconds(90)
			};
			config.Downtime.Add(downtime);
			config.PostLoad(new ServerSettings(), [], []);
			TestOptionsMonitor<GlobalConfig> optionsMonitor = new(config);
			await using DowntimeService downtimeService = new(clock, optionsMonitor, NullLogger<DowntimeService>.Instance);
			Assert.IsFalse(downtimeService.IsDowntimeActive);
			await clock.AdvanceAsync(TimeSpan.FromSeconds(60));
			downtimeService.Tick();
			Assert.IsTrue(downtimeService.IsDowntimeActive);
		}

		[TestMethod]
		public void IsNotActive()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00"
				}
			""");
			Assert.IsFalse(downtime.IsActive(GetNow(2024, 12, 8, 2, 45)));
		}

		[TestMethod]
		public void IsActive()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00"
				}
			""");
			Assert.IsTrue(downtime.IsActive(GetNow(2020, 9, 6, 3, 15)));
		}

		[TestMethod]
		public void IsNotActiveDaily()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00",
					"frequency": "Daily"
				}
			""");
			Assert.IsFalse(downtime.IsActive(GetNow(2024, 12, 8, 2, 45)));
		}

		[TestMethod]
		public void IsActiveDaily()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00",
					"frequency": "Daily"
				}
			""");
			Assert.IsTrue(downtime.IsActive(GetNow(2024, 12, 8, 3, 15)));
		}

		[TestMethod]
		public void IsNotActiveWeekly()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00",
					"frequency": "Weekly"
				}
			""");
			Assert.IsFalse(downtime.IsActive(GetNow(2024, 12, 8, 2, 45)));
		}

		[TestMethod]
		public void IsActiveWeekly()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:45:00+00:00",
					"frequency": "Weekly"
				}
			""");
			Assert.IsTrue(downtime.IsActive(GetNow(2024, 12, 8, 3, 15)));
		}

		[TestMethod]
		public void FinishTimeFromDuration()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"duration": "0:45"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:45+0"), downtime.FinishTime);
		}

		[TestMethod]
		public void DurationFromFinishTime()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:30:00+00:00",
				}
			""");
			Assert.AreEqual(TimeSpan.FromMinutes(30), downtime.Duration);
		}

		[TestMethod]
		public void DurationHigherPrecedence()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "2020-09-06T03:00:00+00:00",
					"finishTime": "2020-09-06T03:30:00+00:00",
					"duration": "0:45"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:45+0"), downtime.FinishTime);
		}

		[TestMethod]
		public void StartTimeAndDurationMinuteAbbreviation()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Sun Sep 6, 2020, 3AM",
					"duration": "0:45"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:45"), downtime.FinishTime);
		}
		
		[TestMethod]
		public void WrongDayOfWeekNameError()
		{
			Assert.ThrowsExactly<FormatException>(() => GetDowntimeFromJson("""
				{
					"startTime": "Mon Sep 6, 2020, 3AM",
					"finishTime": "Sun Sep 6, 2020, 3:30AM",
				}
			"""));
		}

		[TestMethod]
		public void StartTimeAndDurationHourAbbreviation()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Sun Sep 6, 2020, 3AM",
					"duration": "1:30"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T04:30"), downtime.FinishTime);
		}

		[TestMethod]
		public void StartTimeAndFinishTimeAbbreviation()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Dec 1, 2024, 3AM",
					"finishTime": "Sunday December 1 2024, 3:15AM"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2024-12-01T03:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2024-12-01T03:15"), downtime.FinishTime);
		}

		[TestMethod]
		public void WithTimeZone()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Jan 1, 2020, 3AM",
					"duration": "1:30",
					"timezone": "Eastern Standard Time"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-01-01T03:00-05:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-01-01T04:30-05:00"), downtime.FinishTime);
		}

		[TestMethod]
		public void WithTimeZoneAndDaylightSaving()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Sun Sep 6, 2020, 3AM",
					"duration": "1:30",
					"timezone": "Eastern Standard Time"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:00-04:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T04:30-04:00"), downtime.FinishTime);
		}

		[TestMethod]
		public void WithTimeZoneIana()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Jan 1, 2020, 3AM",
					"duration": "1:30",
					"timezone": "America/New_York"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-01-01T03:00-05:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-01-01T04:30-05:00"), downtime.FinishTime);
		}

		[TestMethod]
		public void WithTimeZoneIanaAndDaylightSaving()
		{
			ScheduledDowntime downtime = GetDowntimeFromJson("""
				{
					"startTime": "Sun Sep 6, 2020, 3AM",
					"duration": "1:30",
					"timezone": "America/New_York"
				}
			""");
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:00-04:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T04:30-04:00"), downtime.FinishTime);
		}

		[TestMethod]
		public void WrongTimeZoneName()
		{
			Assert.ThrowsExactly<TimeZoneNotFoundException>(() => GetDowntimeFromJson("""
				{
					"startTime": "Jan 1, 2020, 3AM",
					"timezone": "ET"
				}
			"""));
		}

		[TestMethod]
		public void DowntimeToJson()
		{
			// -4 hours is Eastern Daylight Time
			string json = JsonSerializer.Serialize(GetDowntimeFromJson("""
					{
						"startTime": "Sun Sep 6, 2020, 3AM -4",
						"duration": "0:30"
					}
				"""),
				JsonUtils.DefaultSerializerOptions);
			ScheduledDowntime downtime = GetDowntimeFromJson(json);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:00:00-04:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2020-09-06T03:30:00-04:00"), downtime.FinishTime);
			Assert.AreEqual(TimeSpan.FromMinutes(30), downtime.Duration);
			Assert.AreEqual(ScheduledDowntimeFrequency.Once, downtime.Frequency);
			Assert.IsNull(downtime.TimeZone);
		}

		[TestMethod]
		public void DowntimeWithTimeZoneToJson()
		{
			string json = JsonSerializer.Serialize(GetDowntimeFromJson("""
					{
						"startTime": "Jan 1, 2025, 3AM",
						"duration": "0:30",
						"timezone": "Eastern Standard Time"
					}
				"""),
				JsonUtils.DefaultSerializerOptions);
			ScheduledDowntime downtime = GetDowntimeFromJson(json);
			Assert.AreEqual(DateTimeOffsetParse("2025-01-01T03:00:00-05:00"), downtime.StartTime);
			Assert.AreEqual(DateTimeOffsetParse("2025-01-01T03:30:00-05:00"), downtime.FinishTime);
			Assert.AreEqual(TimeSpan.FromMinutes(30), downtime.Duration);
			Assert.AreEqual(ScheduledDowntimeFrequency.Once, downtime.Frequency);
			Assert.AreEqual("Eastern Standard Time", downtime.TimeZone);
		}

		private static ScheduledDowntime GetWeeklyScheduledDowntime()
		{
			DateTimeOffset startTime = DateTimeOffset.Parse("Wednesday 1 Jan 2025 1AM");
			DateTimeOffset finishTime = startTime + TimeSpan.FromMinutes(5);
			ScheduledDowntime downtime = new()
			{
				StartTime = startTime,
				FinishTime = finishTime,
				Frequency = ScheduledDowntimeFrequency.Weekly
			};
			return downtime;
		}

		[TestMethod]
		[DataRow("18 Dec 2024 12:00", " 1 Jan 2025 1:00", DisplayName = "Two weeks before first schedule")]
		[DataRow("31 Dec 2024 12:00", " 1 Jan 2025 1:00", DisplayName = "One day before first schedule")]
		[DataRow(" 1 Jan 2025  1:05", " 8 Jan 2025 1:00", DisplayName = "End of first schedule")]
		[DataRow(" 2 Jan 2025 15:00", " 8 Jan 2025 1:00", DisplayName = "After first schedule")]
		[DataRow("11 Jan 2025 15:00", "15 Jan 2025 1:00", DisplayName = "After schedule")]
		[DataRow("11 Jan 2026 15:00", "14 Jan 2026 1:00", DisplayName = "One year later")]
		public void GetNextWeekly(string nowStr, string expectedStr)
		{
			DateTimeOffset now = DateTimeOffset.Parse(nowStr);
			DateTimeOffset expected = DateTimeOffset.Parse(expectedStr);

			ScheduledDowntime downtime = GetWeeklyScheduledDowntime();
			(DateTimeOffset startTime, DateTimeOffset finishTime) = downtime.GetNext(now);

			Assert.AreEqual(expected, startTime);
			Assert.AreEqual(expected + (finishTime - startTime), finishTime);
			Assert.IsFalse(downtime.IsActive(now));
		}

		[TestMethod]
		[DataRow(" 1 Jan 2025 1:00", " 1 Jan 2025 1:00", DisplayName = "Start of schedule")]
		[DataRow(" 1 Jan 2025 1:01", " 1 Jan 2025 1:00", DisplayName = "During schedule")]
		[DataRow("15 Jan 2025 1:00", "15 Jan 2025 1:00", DisplayName = "During schedule two weeks later")]
		[DataRow("14 Jan 2026 1:00", "14 Jan 2026 1:00", DisplayName = "During schedule one year later")]
		public void GetCurrentWeekly(string nowStr, string expectedStr)
		{
			DateTimeOffset now = DateTimeOffset.Parse(nowStr);
			DateTimeOffset expected = DateTimeOffset.Parse(expectedStr);

			ScheduledDowntime downtime = GetWeeklyScheduledDowntime();
			(DateTimeOffset startTime, DateTimeOffset finishTime) = downtime.GetNext(now);

			Assert.AreEqual(expected, startTime);
			Assert.AreEqual(expected + (finishTime - startTime), finishTime);
			Assert.IsTrue(downtime.IsActive(now));
		}

		private static ScheduledDowntime GetDailyScheduledDowntime()
		{
			DateTimeOffset startTime = DateTimeOffset.Parse("Wednesday 1 Jan 2025 1AM");
			DateTimeOffset finishTime = startTime + TimeSpan.FromMinutes(5);
			ScheduledDowntime downtime = new()
			{
				StartTime = startTime,
				FinishTime = finishTime,
				Frequency = ScheduledDowntimeFrequency.Daily
			};
			return downtime;
		}

		[TestMethod]
		[DataRow("30 Dec 2024  0:00", " 1 Jan 2025 1:00", DisplayName = "Two days before schedule")]
		[DataRow("31 Dec 2024  0:00", " 1 Jan 2025 1:00", DisplayName = "One day before schedule")]
		[DataRow(" 1 Jan 2025  1:06", " 2 Jan 2025 1:00", DisplayName = "End of schedule")]
		[DataRow("10 Jan 2025  1:06", "11 Jan 2025 1:00", DisplayName = "End of schedule")]
		[DataRow(" 2 Jan 2025 15:00", " 3 Jan 2025 1:00", DisplayName = "After schedule")]
		public void GetNextDaily(string nowStr, string expectedStr)
		{
			DateTimeOffset now = DateTimeOffset.Parse(nowStr);
			DateTimeOffset expected = DateTimeOffset.Parse(expectedStr);

			ScheduledDowntime downtime = GetDailyScheduledDowntime();
			(DateTimeOffset startTime, DateTimeOffset finishTime) = downtime.GetNext(now);

			Assert.AreEqual(expected, startTime);
			Assert.AreEqual(expected + (finishTime - startTime), finishTime);
			Assert.IsFalse(downtime.IsActive(now));
		}

		[TestMethod]
		[DataRow(" 1 Jan 2025  1:00", " 1 Jan 2025 1:00", DisplayName = "Start of schedule")]
		[DataRow(" 1 Jan 2025  1:03", " 1 Jan 2025 1:00", DisplayName = "During schedule")]
		[DataRow(" 1 Jan 2026  1:03", " 1 Jan 2026 1:00", DisplayName = "During schedule one year later")]
		public void GetCurrentDaily(string nowStr, string expectedStr)
		{
			DateTimeOffset now = DateTimeOffset.Parse(nowStr);
			DateTimeOffset expected = DateTimeOffset.Parse(expectedStr);

			ScheduledDowntime downtime = GetDailyScheduledDowntime();
			(DateTimeOffset startTime, DateTimeOffset finishTime) = downtime.GetNext(now);

			Assert.AreEqual(expected, startTime);
			Assert.AreEqual(expected + (finishTime - startTime), finishTime);
			Assert.IsTrue(downtime.IsActive(now));
		}

		[TestMethod]
		public void GetSuccessiveNext()
		{
			ScheduledDowntime downtime = new()
			{
				StartTime = DateTimeOffset.Parse("2024-11-07T10:10:00"),
				FinishTime = DateTimeOffset.Parse("2024-11-07T14:30:00"),
				Frequency = ScheduledDowntimeFrequency.Daily
			};

			DateTimeOffset now = DateTimeOffset.Parse("2021-11-07T10:40:00");
			(DateTimeOffset startTime0, DateTimeOffset finishTime0) = downtime.GetNext(now);
			(DateTimeOffset startTime1, DateTimeOffset finishTime1) = downtime.GetNext(now + TimeSpan.FromMinutes(1));
			(DateTimeOffset startTime2, DateTimeOffset finishTime2) = downtime.GetNext(now + TimeSpan.FromMinutes(2));

			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 10:10"), startTime0);
			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 14:30"), finishTime0);

			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 10:10"), startTime1);
			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 14:30"), finishTime1);

			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 10:10"), startTime2);
			Assert.AreEqual(DateTimeOffset.Parse("7 Nov 2024 14:30"), finishTime2);
		}
	}
}
