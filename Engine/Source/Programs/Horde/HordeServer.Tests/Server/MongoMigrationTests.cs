// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Server;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using MongoDB.Driver;

namespace HordeServer.Tests.Server;

[TestClass]
public class MongoMigratorTests : DatabaseIntegrationTest
{
	private abstract class TestMigration : IMongoMigration
	{
		public virtual Task UpAsync(MigrationContext context, CancellationToken token) { throw new NotImplementedException(); }
		public virtual Task DownAsync(MigrationContext context, CancellationToken token) { throw new NotImplementedException(); }
	}
	
	private class MissingAttributeMigration : TestMigration;
	
	[MongoMigration("duplicate", 1, "First upgrade", false)]
	private class DuplicateVersionMigrationA : TestMigration;
	
	[MongoMigration("duplicate", 1, "First upgrade again", false)]
	private class DuplicateVersionMigrationB : TestMigration;
	
	[MongoMigration("test", 0, "Invalid version", false)]
	private class InvalidVersionMigration : TestMigration { }
	
	[MongoMigration("test", 1, "ab", false)]
	private class ShortNameMigration : TestMigration { }
	
	[MongoMigration("failing", 1, "Failing migration", false)]
	private class FailingUpgradeMigration : TestMigration
	{
		public override Task UpAsync(MigrationContext context, CancellationToken token) { throw new Exception("Simulated failure"); }
	}
	
	[MongoMigration("failing", 1, "Failing migration", false)]
	private class FailingDowngradeMigration : TestMigration
	{
		public override Task UpAsync(MigrationContext context, CancellationToken token) { return Task.CompletedTask; }
		public override Task DownAsync(MigrationContext context, CancellationToken token) { throw new Exception("Simulated failure"); }
	}
	
	[MongoMigration("foo", 1, "Create collection")]
	private class FooMigration1 : TestMigration
	{
		public override async Task UpAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.CreateCollectionAsync("MyCollection-1", cancellationToken: token);
		}
		
		public override async Task DownAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.DropCollectionAsync("MyCollection-1", cancellationToken: token);
		}
	}
	
	[MongoMigration("foo", 2, "Rename collection 1 -> 2")]
	private class FooMigration2 : TestMigration
	{
		public override async Task UpAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.RenameCollectionAsync("MyCollection-1", "MyCollection-2", cancellationToken: token);
		}
		
		public override async Task DownAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.RenameCollectionAsync("MyCollection-2", "MyCollection-1", cancellationToken: token);
		}
	}
	
	[MongoMigration("foo", 3, "Rename collection 2 -> 3")]
	private class FooMigration3 : TestMigration
	{
		public override async Task UpAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.RenameCollectionAsync("MyCollection-2", "MyCollection-3", cancellationToken: token);
		}
		
		public override async Task DownAsync(MigrationContext c, CancellationToken token)
		{
			await c.Database.RenameCollectionAsync("MyCollection-3", "MyCollection-2", cancellationToken: token);
		}
	}
	
	private readonly MongoMigrator _migrator;
	private readonly IMongoService _mongoService;
	
	public MongoMigratorTests()
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});
		
		_migrator = new MongoMigrator(GetMongoServiceSingleton(), loggerFactory.CreateLogger<MongoMigrator>());
		_mongoService = GetMongoServiceSingleton();
	}
	
	[TestMethod]
	public void AddMigration_WithMissingAttribute()
	{
		MigrationException me = Assert.ThrowsExactly<MigrationException>(() => _migrator.AddMigration(new MissingAttributeMigration()));
		Assert.IsTrue(me.Message.Contains("is missing [MongoMigration] attribute", StringComparison.Ordinal));
	}
	
	[TestMethod]
	public void AddMigration_WithDuplicateVersion()
	{
		_migrator.AddMigration(new DuplicateVersionMigrationA());
		MigrationException me = Assert.ThrowsExactly<MigrationException>(() => _migrator.AddMigration(new DuplicateVersionMigrationB()));
		Assert.IsTrue(me.Message.Contains("already contains a migration with version 1", StringComparison.Ordinal));
	}
	
	[TestMethod]
	public void AddMigration_WithOutOfOrderVersions()
	{
		_migrator.AddMigration(new FooMigration2());
		_migrator.AddMigration(new FooMigration3());
		_migrator.AddMigration(new FooMigration1());
		IReadOnlyList<MigrationDefinition> migrations = _migrator.GetMigrations("foo");
		Assert.AreEqual(3, migrations.Count);
		Assert.AreEqual(1, migrations[0].Version);
		Assert.AreEqual(2, migrations[1].Version);
		Assert.AreEqual(3, migrations[2].Version);
	}
	
	[TestMethod]
	public void AutoAddMigrations_WithAllClasses()
	{
		_migrator.AutoAddMigrations();
		IReadOnlyList<MigrationDefinition> migrations = _migrator.GetMigrations("foo");
		Assert.AreEqual(3, migrations.Count);
		Assert.AreEqual(1, migrations[0].Version);
		Assert.AreEqual(2, migrations[1].Version);
		Assert.AreEqual(3, migrations[2].Version);
	}
	
	[TestMethod]
	public async Task Upgrade_WithVersionGap_Async()
	{
		_migrator.AddMigration(new FooMigration1());
		_migrator.AddMigration(new FooMigration3());
		MigrationException me = await Assert.ThrowsExactlyAsync<MigrationException>(() => _migrator.UpgradeAsync("foo", null, CancellationToken.None));
		Assert.IsTrue(me.Message.StartsWith("Gap detected", StringComparison.Ordinal));
	}
	
	[TestMethod]
	public void AddMigration_WithInvalidVersionNumber()
	{
		MigrationException me = Assert.ThrowsExactly<MigrationException>(() => _migrator.AddMigration(new InvalidVersionMigration()));
		Assert.IsTrue(me.Message.Contains("invalid version number", StringComparison.Ordinal));
	}
	
	[TestMethod]
	public void AddMigration_WithInvalidName()
	{
		MigrationException me = Assert.ThrowsExactly<MigrationException>(() => _migrator.AddMigration(new ShortNameMigration()));
		Assert.IsTrue(me.Message.Contains("too short/long name", StringComparison.Ordinal));
	}
	
	[TestMethod]
	public async Task UpgradeAsync()
	{
		_migrator.AddMigration(new FooMigration1());
		await _migrator.UpgradeAsync("foo", null, CancellationToken.None);
		AssertCollectionExists("MyCollection-1");
	}
	
	[TestMethod]
	public async Task Upgrade_WithSpecificVersion_Async()
	{
		// Test upgrading to a specific version, not all the way
		_migrator.AddMigration(new FooMigration1());
		_migrator.AddMigration(new FooMigration2());
		_migrator.AddMigration(new FooMigration3());
		
		await _migrator.UpgradeAsync("foo", 2, CancellationToken.None);
		
		AssertCollectionExists("MyCollection-2");
		SchemaVersionResult result = await _migrator.GetSchemaStateAsync("foo", CancellationToken.None);
		Assert.AreEqual(2, result.DatabaseVersion);
	}
	
	[TestMethod]
	public async Task UpgradeAndDowngrade_AppliesCorrectly_Async()
	{
		_migrator.AddMigration(new FooMigration1());
		await _migrator.UpgradeAsync("foo", null, CancellationToken.None);
		AssertCollectionExists("MyCollection-1");
		
		_migrator.AddMigration(new FooMigration2());
		await _migrator.UpgradeAsync("foo", null, CancellationToken.None);
		AssertCollectionExists("MyCollection-2");
		
		await _migrator.DowngradeAsync("foo", 1, CancellationToken.None);
		AssertCollectionExists("MyCollection-1");
	}
	
	[TestMethod]
	public async Task CheckSchemaVersion_WhenVersionsMatch_Async()
	{
		// Setup initial state at version 1
		_migrator.AddMigration(new FooMigration1());
		await _migrator.UpgradeAsync("foo", null, CancellationToken.None);
		
		// Check state
		SchemaVersionResult result = await _migrator.GetSchemaStateAsync("foo", CancellationToken.None);
		
		Assert.AreEqual(SchemaVersionState.VersionMatch, result.State);
		Assert.AreEqual(0, result.MissingMigrations.Count);
	}
	
	[TestMethod]
	public async Task CheckSchemaVersion_WhenDatabaseOutdated_Async()
	{
		// Add migrations but don't apply them yet
		_migrator.AddMigration(new FooMigration1());
		_migrator.AddMigration(new FooMigration2());
		_migrator.AddMigration(new FooMigration3());
		
		// Upgrade only to version 1
		await _migrator.UpgradeAsync("foo", 1, CancellationToken.None);
		
		// Check state - should show versions 2 and 3 as missing
		SchemaVersionResult result = await _migrator.GetSchemaStateAsync("foo", CancellationToken.None);
		
		Console.WriteLine("AppVersion: " + result.AppVersion);
		Console.WriteLine("DatabaseVersion: " + result.DatabaseVersion);
		
		Assert.AreEqual(SchemaVersionState.DatabaseOutdated, result.State);
		Assert.AreEqual(2, result.MissingMigrations.Count);
		Assert.AreEqual(2, result.MissingMigrations[0].Version);
		Assert.AreEqual(3, result.MissingMigrations[1].Version);
	}
	
	[TestMethod]
	public async Task CheckSchemaVersion_WhenApplicationOutdated_Async()
	{
		_migrator.AddMigration(new FooMigration1());
		_migrator.AddMigration(new FooMigration2());
		_migrator.AddMigration(new FooMigration3());
		await _migrator.UpgradeAsync("foo", null, CancellationToken.None);
		
		// Create new migrator instance with fewer migrations to simulate older app version
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder => builder.AddSimpleConsole());
		MongoMigrator newMigrator = new (_mongoService, loggerFactory.CreateLogger<MongoMigrator>());
		newMigrator.AddMigration(new FooMigration1());
		newMigrator.AddMigration(new FooMigration2());
		
		// Check state - should show version 3 as missing from application
		SchemaVersionResult result = await newMigrator.GetSchemaStateAsync("foo", CancellationToken.None);
		Assert.AreEqual(SchemaVersionState.ApplicationOutdated, result.State);
		Assert.AreEqual(1, result.MissingMigrations.Count);
		Assert.AreEqual(3, result.MissingMigrations[0].Version);
	}
	
	[TestMethod]
	public async Task CheckSchemaVersion_WithUnknownPlugin_Async()
	{
		MigrationException me = await Assert.ThrowsExactlyAsync<MigrationException>(() => _migrator.GetSchemaStateAsync("unknown-plugin", CancellationToken.None));
		Assert.AreEqual("Unknown plugin unknown-plugin", me.Message);
	}
	
	private void AssertCollectionExists(string name)
	{
		List<string> collectionNames = _mongoService.Database.ListCollectionNamesAsync().Result.ToList();
		CollectionAssert.Contains(collectionNames, name, $"Collection {name} does not exist");
	}
}