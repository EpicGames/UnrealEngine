// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class MongoTests
	{
		class TestClass : ISupportInitialize
		{
			public int Value { get; set; }

			[BsonExtraElements]
			public BsonDocument? ExtraElements { get; set; }

			public bool _endInitCalled;

			void ISupportInitialize.BeginInit() { }

			void ISupportInitialize.EndInit()
			{
				_endInitCalled = true;
			}
		}

		[TestMethod]
		public void TestInitializeCallbacks()
		{
			TestClass tc1 = new TestClass();
			tc1.Value = 123;
			tc1.ExtraElements = new BsonDocument(new BsonElement("hello", "world"));
			byte[] bytes = tc1.ToBson();

			TestClass tc2 = BsonSerializer.Deserialize<TestClass>(bytes);
			Assert.IsTrue(tc2._endInitCalled);
			Assert.AreEqual(123, tc2.Value);
			Assert.IsNotNull(tc2.ExtraElements);
			Assert.AreEqual("world", tc2.ExtraElements["hello"]);
		}
	}
}
