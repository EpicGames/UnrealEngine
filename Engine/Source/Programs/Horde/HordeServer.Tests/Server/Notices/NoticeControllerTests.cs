// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net.Http;
using System.Net.Http.Json;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Server.Notices;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Server.Notices;

[TestClass]
public class NoticeControllerTests : ControllerIntegrationTest
{
	[TestMethod]
	public async Task GetNoticesAsync()
	{
		CreateNoticeRequest notice = new("Hello, World!");
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.AreEqual("Hello, World!", result.Message);
		Assert.IsNull(result.StartTime);
		Assert.IsNull(result.FinishTime);
		Assert.IsTrue(result.Active);
	}

	[TestMethod]
	public async Task GetNoticesWithStartAndFinishTimesAsync()
	{
		IClock clock = ServiceProvider.GetRequiredService<IClock>();
		CreateNoticeRequest notice = new("Hello, World!")
		{
			StartTime = clock.UtcNow - TimeSpan.FromMinutes(1),
			FinishTime = clock.UtcNow + TimeSpan.FromMinutes(1)
		};
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.IsNotNull(result.StartTime);
		Assert.IsNotNull(result.FinishTime);
		Assert.IsTrue(result.Active);
	}

	[TestMethod]
	public async Task GetNoticesWithStartTimeAsync()
	{
		IClock clock = ServiceProvider.GetRequiredService<IClock>();
		CreateNoticeRequest notice = new("Hello, World!")
		{
			StartTime = clock.UtcNow - TimeSpan.FromMinutes(1)
		};
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.IsNotNull(result.StartTime);
		Assert.IsNull(result.FinishTime);
		Assert.IsTrue(result.Active);
	}

	[TestMethod]
	public async Task GetNoticesWithFinishTimeAsync()
	{
		IClock clock = ServiceProvider.GetRequiredService<IClock>();
		CreateNoticeRequest notice = new("Hello, World!")
		{
			FinishTime = clock.UtcNow + TimeSpan.FromMinutes(1)
		};
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.IsNull(result.StartTime);
		Assert.IsNotNull(result.FinishTime);
		Assert.IsTrue(result.Active);
	}

	[TestMethod]
	public async Task GetNoticesInactivePastAsync()
	{
		IClock clock = ServiceProvider.GetRequiredService<IClock>();
		CreateNoticeRequest notice = new("Hello, World!")
		{
			FinishTime = clock.UtcNow - TimeSpan.FromMinutes(1)
		};
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.IsFalse(result.Active);
	}

	[TestMethod]
	public async Task GetNoticesInactiveFutureAsync()
	{
		IClock clock = ServiceProvider.GetRequiredService<IClock>();
		CreateNoticeRequest notice = new("Hello, World!")
		{
			StartTime = clock.UtcNow + TimeSpan.FromMinutes(1)
		};
		using HttpResponseMessage response = await Client.PostAsJsonAsync("api/v1/notices", notice);
		response.EnsureSuccessStatusCode();
		List<GetNoticeResponse> notices = await HordeHttpRequest.GetAsync<List<GetNoticeResponse>>(Client, "api/v1/notices");
		GetNoticeResponse result = notices[0];
		Assert.IsFalse(result.Active);
	}
}
