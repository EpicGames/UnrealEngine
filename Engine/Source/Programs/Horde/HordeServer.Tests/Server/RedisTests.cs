// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq.Expressions;
using System.Threading.Tasks;
using EpicGames.Redis;
using EpicGames.Redis.Utility;
using HordeServer.Server;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using StackExchange.Redis;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class RedisTests : DatabaseIntegrationTest
	{
		class TestClass
		{
			public int Foo { get; set; }
			public int Bar { get; set; }
			public int Baz { get; set; }
		}

		[TestMethod]
		public async Task HashTestAsync()
		{
			IRedisService redisService = GetRedisServiceSingleton();
			IDatabase database = redisService.GetDatabase();

			RedisHashKey<TestClass> key = new RedisHashKey<TestClass>("test");
			await database.HashSetAsync(key, new TestClass { Foo = 123, Bar = 456, Baz = 789 });

			TestClass value = await database.HashGetAllAsync(key);
			Assert.AreEqual(123, value.Foo);
			Assert.AreEqual(456, value.Bar);
			Assert.AreEqual(789, value.Baz);

			TestClass value2 = await database.HashGetAsync(key, new Expression<Func<TestClass, object>>[] { x => x.Foo, x => x.Bar });
			Assert.AreEqual(123, value2.Foo);
			Assert.AreEqual(456, value2.Bar);
			Assert.AreEqual(0, value2.Baz);
		}

		[TestMethod]
		public async Task AsyncEventTestAsync()
		{
			IRedisService redisService = GetRedisServiceSingleton();

			await using RedisEvent asyncEvent = await RedisEvent.CreateAsync(redisService.GetConnection(), RedisChannel.Literal("hello-world"));

			Task task = asyncEvent.Task;
			Assert.IsFalse(task.IsCompleted);

			asyncEvent.Pulse();
			await task;
		}

		[TestMethod]
		public async Task QueueTestAsync()
		{
			IRedisService redisService = GetRedisServiceSingleton();

			await using RedisQueue<int> queue = await RedisQueue.CreateAsync<int>(redisService.GetConnection(), "queue", RedisChannel.Literal("queue-events"));
			await queue.PushAsync(1);
			await queue.PushAsync(2);
			await queue.PushAsync(3);

			int value = await queue.TryPopAsync();
			Assert.AreEqual(1, value);

			value = await queue.TryPopAsync();
			Assert.AreEqual(2, value);

			value = await queue.TryPopAsync();
			Assert.AreEqual(3, value);

			value = await queue.TryPopAsync();
			Assert.AreEqual(0, value);

			Task task = queue.WaitForDataAsync();
			Assert.IsFalse(task.IsCompleted);

			await queue.PushAsync(4);
			await task.WaitAsync(TimeSpan.FromSeconds(5.0));

			value = await queue.TryPopAsync();
			Assert.AreEqual(4, value);
		}
	}
}
