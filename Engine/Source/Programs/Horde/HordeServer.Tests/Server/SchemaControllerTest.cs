// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net.Http;
using System.Text.Json;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA2234 // Pass system uri objects instead of strings

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class SchemaControllerTest : ControllerIntegrationTest
	{
		class SchemaCatalog
		{
			public List<SchemaCatalogItem> Schemas { get; set; } = new List<SchemaCatalogItem>();
		}

		class SchemaCatalogItem
		{
			public string? Name { get; set; }
			public Uri Url { get; set; } = null!;
		}

		[TestMethod]
		public async Task TestCatalogAsync()
		{
			// Fetch the catalog
			SchemaCatalog catalog;
			using (HttpResponseMessage response = await Client.GetAsync("/api/v1/schema/catalog.json"))
			{
				response.EnsureSuccessStatusCode();
				byte[] data = await response.Content.ReadAsByteArrayAsync();
				catalog = JsonSerializer.Deserialize<SchemaCatalog>(data, new JsonSerializerOptions { PropertyNameCaseInsensitive = true })!;
			}

			// Fetch all the individual files
			foreach (SchemaCatalogItem schema in catalog.Schemas)
			{
				using HttpResponseMessage response = await Client.GetAsync(schema.Url);
				response.EnsureSuccessStatusCode();
			}
		}
	}
}
