// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeServer.Plugins;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class SchemaTests
	{
		[TestMethod]
		public void StreamSchema()
		{
			JsonSchemaCache cache = new JsonSchemaCache(new PluginCollection());
			JsonSchema schema = cache.CreateSchema(typeof(StreamConfig));
			_ = schema;
		}

		[TestMethod]
		public void ProjectSchema()
		{
			JsonSchemaCache cache = new JsonSchemaCache(new PluginCollection());
			JsonSchema schema = cache.CreateSchema(typeof(StreamConfig));
			_ = schema;
		}
	}
}
