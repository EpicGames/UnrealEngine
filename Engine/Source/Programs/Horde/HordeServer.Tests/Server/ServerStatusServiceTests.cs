// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Configuration;
using HordeServer.Server;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class ServerStatusServiceTest : ServerTestSetup
	{
		[TestMethod]
		public async Task UpdatesAreStoredNewToOldAsync()
		{
			IHealthMonitor<ConfigService> health = new HealthMonitor<ConfigService>(ServerStatusService);
			await health.UpdateAsync(HealthStatus.Unhealthy, "foo", DateTimeOffset.UtcNow - TimeSpan.FromSeconds(15));
			await health.UpdateAsync(HealthStatus.Healthy, "bar", DateTimeOffset.UtcNow);
			await health.UpdateAsync(HealthStatus.Unhealthy, "baz", DateTimeOffset.UtcNow - TimeSpan.FromSeconds(5));

			SubsystemStatus gcStatus = await GetSubsystemStatusAsync(typeof(ConfigService));
			Assert.AreEqual(3, gcStatus.Updates.Count);
			Assert.AreEqual("bar", gcStatus.Updates[0].Message);
			Assert.AreEqual("baz", gcStatus.Updates[1].Message);
			Assert.AreEqual("foo", gcStatus.Updates[2].Message);
		}

		[TestMethod]
		public async Task OnlyLastNUpdatesAreKeptAsync()
		{
			IHealthMonitor<ConfigService> health = new HealthMonitor<ConfigService>(ServerStatusService);

			for (int i = 0; i < ServerStatusService.MaxHistoryLength + 10; i++)
			{
				await health.UpdateAsync(HealthStatus.Healthy, "foo", DateTimeOffset.UtcNow);
			}

			Assert.AreEqual(ServerStatusService.MaxHistoryLength, (await GetSubsystemStatusAsync(typeof(ConfigService))).Updates.Count);
		}

		[TestMethod]
		public async Task MongoDbHealthCheckAsync()
		{
			// A MongoDB server is always present during test runs
			await ServerStatusService.UpdateMongoDbHealthAsync(CancellationToken.None);
			SubsystemStatus mongoDb = await GetSubsystemStatusAsync(typeof(MongoService));
			Assert.AreEqual(1, mongoDb.Updates.Count);
			Assert.AreEqual(HealthStatus.Healthy, mongoDb.Updates[0].Result);
		}

		[TestMethod]
		public async Task RedisHealthCheckAsync()
		{
			// A Redis server is always present during test runs
			await ServerStatusService.UpdateRedisHealthAsync(CancellationToken.None);
			SubsystemStatus redis = await GetSubsystemStatusAsync(typeof(RedisService));
			Assert.AreEqual(1, redis.Updates.Count);
			Assert.AreEqual(HealthStatus.Healthy, redis.Updates[0].Result);
		}

		private async Task<SubsystemStatus> GetSubsystemStatusAsync(Type type)
		{
			IReadOnlyList<SubsystemStatus> statuses = await ServerStatusService.GetSubsystemStatusesAsync();
			return statuses.First(x => x.Id == type.Name);
		}
	}
}
