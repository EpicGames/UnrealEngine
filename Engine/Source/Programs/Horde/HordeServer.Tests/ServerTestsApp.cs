// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;

namespace HordeServer.Tests
{
	class ServerTestsApp
	{
		/// <summary>
		/// Entry point
		/// </summary>
		public static Task Main()
		{
			// This custom entry point allows running custom code for debugging/profiling test runs.
			Console.WriteLine("Run tests through the MSTest framework.");
			return Task.CompletedTask;
		}
	}
}
