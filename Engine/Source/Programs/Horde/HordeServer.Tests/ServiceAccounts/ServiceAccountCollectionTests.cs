// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using HordeServer.Server;
using HordeServer.ServiceAccounts;
using HordeServer.Users;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.ServiceAccounts
{
	[TestClass]
	public class HordeAccountCollectionTests : DatabaseIntegrationTest
	{
		private readonly IServiceAccountCollection _serviceAccountCollection;
		private readonly IServiceAccount _serviceAccount;
		private readonly string _token;

		public HordeAccountCollectionTests()
		{
			IMongoService mongoService = GetMongoServiceSingleton();
			_serviceAccountCollection = new ServiceAccountCollection(mongoService);
			CreateServiceAccountOptions options = new (Name: "myName", Description: "myDesc", Claims: new List<IUserClaim> { new UserClaim("myClaim", "myValue") });
			(_serviceAccount, _token) = _serviceAccountCollection.CreateAsync(options).Result;
		}

		[TestMethod]
		public async Task AddAsync()
		{
			CreateServiceAccountOptions options = new (Name: "myName", Description: "myDesc", Claims: new List<IUserClaim> { new UserClaim("myClaim", "myValue") });
			(IServiceAccount sa, _) = await _serviceAccountCollection.CreateAsync(options);
			Assert.AreEqual(1, sa.Claims.Count);
			Assert.AreEqual("myValue", sa.Claims[0].Value);
			Assert.IsTrue(sa.Enabled);
			Assert.AreEqual("myName", sa.Name);
			Assert.AreEqual("myDesc", sa.Description);
		}

		[TestMethod]
		public async Task GetAsync()
		{
			IServiceAccount sa = (await _serviceAccountCollection.GetAsync(_serviceAccount.Id))!;
			Assert.AreEqual(_serviceAccount.Id, sa.Id);
		}

		[TestMethod]
		public async Task GetBySecretTokenAsync()
		{
			IServiceAccount sa = (await _serviceAccountCollection.FindBySecretTokenAsync(_token))!;
			Assert.AreEqual(_serviceAccount.Id, sa.Id);
		}

		[TestMethod]
		public async Task UpdateAsync()
		{
			IServiceAccount? account = await _serviceAccountCollection.GetAsync(_serviceAccount.Id);
			Assert.IsNotNull(account);

			List<UserClaim> newClaims = [new UserClaim("newClaim1", "newValue1"), new UserClaim("newClaim2", "newValue2")];
			(_, string? newToken) = await account.UpdateAsync(new UpdateServiceAccountOptions
			{
				Claims = newClaims,
				ResetToken = true,
				Enabled = false,
				Name = "newName",
				Description = "newDesc"
			});

			Assert.IsNotNull(newToken);
			Assert.IsNull(await _serviceAccountCollection.FindBySecretTokenAsync(_token));

			IServiceAccount? foundAccount = await _serviceAccountCollection.FindBySecretTokenAsync(newToken);
			Assert.IsNotNull(foundAccount);
			Assert.AreEqual(foundAccount.Id, _serviceAccount.Id);

			IServiceAccount? sa = await _serviceAccountCollection.GetAsync(_serviceAccount.Id);
			Assert.IsNotNull(sa);

			Assert.AreEqual(2, sa.Claims.Count);
			Assert.AreEqual("newValue1", sa.Claims[0].Value);
			Assert.AreEqual("newValue2", sa.Claims[1].Value);
			Assert.AreEqual(false, sa.Enabled);
			Assert.AreEqual("newName", sa.Name);
			Assert.AreEqual("newDesc", sa.Description);
		}

		[TestMethod]
		public async Task DeleteAsync()
		{
			await _serviceAccountCollection.DeleteAsync(_serviceAccount.Id);
			IServiceAccount? result = await _serviceAccountCollection.GetAsync(_serviceAccount.Id);
			Assert.IsNull(result);
		}
	}
}