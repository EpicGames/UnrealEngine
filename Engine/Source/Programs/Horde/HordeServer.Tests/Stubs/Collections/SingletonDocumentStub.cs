// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading;
using System.Threading.Tasks;
using HordeServer.Utilities;

namespace HordeServer.Tests.Stubs.Collections
{
	class SingletonDocumentStub<T> : ISingletonDocument<T> where T : SingletonBase, new()
	{
		T _document = new T();

		public Task<T> GetAsync(CancellationToken cancellationToken)
		{
			return Task.FromResult(_document);
		}

		public Task<bool> TryUpdateAsync(T value, CancellationToken cancellationToken)
		{
			_document = value;
			return Task.FromResult(true);
		}
	}
}
