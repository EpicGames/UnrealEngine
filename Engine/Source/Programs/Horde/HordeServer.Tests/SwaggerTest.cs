// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net.Http;
using System.Text.Json;
using System.Threading.Tasks;
using Microsoft.VisualStudio.TestTools.UnitTesting;

#pragma warning disable CA2234 // Pass system uri objects instead of strings

namespace HordeServer.Tests
{
	[TestClass]
	public class SwaggerTest : ControllerIntegrationTest
	{
		[TestMethod]
		public async Task ValidateSwaggerAsync()
		{
			HttpResponseMessage res = await Client.GetAsync("/swagger/v1/swagger.json");
			if (!res.IsSuccessStatusCode)
			{
				string rawJson = await res.Content.ReadAsStringAsync();
				JsonElement tempElement = JsonSerializer.Deserialize<JsonElement>(rawJson);
				string formattedJson = JsonSerializer.Serialize(tempElement, new JsonSerializerOptions { WriteIndented = true });
				await Console.Error.WriteLineAsync("Error result:\n" + formattedJson);
				res.EnsureSuccessStatusCode();
			}
		}
	}
}