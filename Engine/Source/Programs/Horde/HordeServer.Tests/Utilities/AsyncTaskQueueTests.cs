// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.VisualStudio.TestTools.UnitTesting;

namespace HordeServer.Tests.Utilities
{
	[TestClass]
	public class AsyncTaskQueueTests
	{
		[TestMethod]
		public async Task TestQueueAsync()
		{
			await using AsyncTaskQueue queue = new AsyncTaskQueue(NullLogger.Instance);

			bool executed1 = false;
			queue.Enqueue(async _ =>
			{
				await Task.Yield();
				executed1 = true;
			});

			bool executed2 = false;
			queue.Enqueue(async _ =>
			{
				await Task.Yield();
				executed2 = true;
			});

			await queue.FlushAsync();

			Assert.IsTrue(executed1);
			Assert.IsTrue(executed2);
		}
	}
}
