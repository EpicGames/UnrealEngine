// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using MongoDB.Bson;
using MongoDB.Driver;
using MongoDB.Driver.Core.Clusters;
using MongoDB.Driver.Core.Connections;
using MongoDB.Driver.Core.Events;
using MongoDB.Driver.Core.Servers;
using OpenTelemetry.Trace;

namespace HordeServer.Tests.Utilities;

[TestClass]
public class MongoCommandTracerTests
{
	private static BsonDocument CreateReply(long cursorId)
	{
		return new BsonDocument(new Dictionary<string, object>
		{
			{ "waitedMs", 0L },
			{
				"cursor",
				new BsonDocument(new Dictionary<string, object>()
				{
					{ "id", cursorId }
				})
			}
		});
	}
	
	[TestMethod]
	public void TestFindWithMultipleBatches()
	{
		DatabaseNamespace ns = new ("Horde");
		ConnectionId connectionId = new (new ServerId(new ClusterId(1), new DnsEndPoint("some-db-host.com", 123)));
		BsonDocument findCommand = new (new Dictionary<string, object> { { "find", "Agents" } });
		BsonDocument getMoreCommand = new (new Dictionary<string, object> { { "getMore", 452725158653L } });
		BsonDocument getMoreReply = CreateReply(452725158653);
		BsonDocument getMoreFinalReply = CreateReply(0);
		List<object> events =
		[
			new CommandStartedEvent("find", findCommand, ns, 4, 20, connectionId, null),
			new CommandSucceededEvent("find", getMoreReply, ns, 4, 20, connectionId, null, TimeSpan.FromMilliseconds(571.2163)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 21, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 21, connectionId, null, TimeSpan.FromMilliseconds(206.7805)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 22, connectionId, null), 
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 22, connectionId, TimeSpan.FromMilliseconds(205.5413)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 23, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 23, connectionId, TimeSpan.FromMilliseconds(204.9236)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 24, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 24, connectionId, TimeSpan.FromMilliseconds(204.98)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 25, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 25, connectionId, TimeSpan.FromMilliseconds(204.447)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 26, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 26, connectionId, TimeSpan.FromMilliseconds(204.2262)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 27, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 27, connectionId, TimeSpan.FromMilliseconds(204.1234)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 28, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 28, connectionId, TimeSpan.FromMilliseconds(204.043)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 29, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 29, connectionId, TimeSpan.FromMilliseconds(204.2582)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 30, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 30, connectionId, TimeSpan.FromMilliseconds(204.5859)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 31, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 31, connectionId, TimeSpan.FromMilliseconds(204.6983)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 32, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 32, connectionId, TimeSpan.FromMilliseconds(204.1839)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 33, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 33, connectionId, TimeSpan.FromMilliseconds(204.7504)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 34, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 34, connectionId, TimeSpan.FromMilliseconds(204.5998)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 35, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 35, connectionId, TimeSpan.FromMilliseconds(204.7395)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 36, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 36, connectionId, TimeSpan.FromMilliseconds(204.0108)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 37, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 37, connectionId, TimeSpan.FromMilliseconds(204.4958)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 38, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreReply, ns, 4, 38, connectionId, TimeSpan.FromMilliseconds(204.3534)),
			new CommandStartedEvent("getMore", getMoreCommand, ns, 4, 39, connectionId, null),
			new CommandSucceededEvent("getMore", getMoreFinalReply, ns, 4, 39, connectionId, TimeSpan.FromMilliseconds(203.8798)),
		];

		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
		{
			builder.SetMinimumLevel(LogLevel.Debug);
			builder.AddSimpleConsole(options => { options.SingleLine = true; });
		});
		
		MongoCommandTracer mongoTracer = new (TracerProvider.Default.GetTracer("hello"), loggerFactory.CreateLogger<MongoCommandTracer>());
		foreach (object e in events)
		{
			switch (e)
			{
				case CommandStartedEvent @started: mongoTracer.OnEvent(@started); break;
				case CommandSucceededEvent @succeeded: mongoTracer.OnEvent(@succeeded); break;
				case CommandFailedEvent @failed: mongoTracer.OnEvent(@failed); break;
			}
		}
		
		Assert.AreEqual(0, mongoTracer.GetSpans().Count);
	}
}

