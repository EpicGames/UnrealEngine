// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text.Json;
using HordeServer.Utilities;
using Microsoft.VisualStudio.TestTools.UnitTesting;
using Serilog.Events;
using Serilog.Formatting.Json;
using Serilog.Parsing;

namespace HordeServer.Tests.Utilities
{
	[TestClass]
	public class SizeLimitedJsonFormatterTests
	{
		readonly JsonFormatter _inner = new JsonFormatter(renderMessage: true);
		readonly SizeLimitedJsonFormatter _truncating = new SizeLimitedJsonFormatter(new JsonFormatter(renderMessage: true), maxBytes: 100);

		static LogEvent CreateLogEvent(string messageTemplate, Exception? exception = null, LogEventLevel level = LogEventLevel.Information, params LogEventProperty[] properties)
		{
			MessageTemplateParser parser = new MessageTemplateParser();
			return new LogEvent(DateTimeOffset.UtcNow, level, exception, parser.Parse(messageTemplate), properties);
		}

		/// <summary>
		/// Creates a log event with ExceptionInfo and ExceptionDetail properties, simulating the Serilog.Exceptions enricher.
		/// </summary>
		static LogEvent CreateLogEventWithExceptionProperties(string messageTemplate, Exception exception, string exceptionInfoValue, string exceptionDetailValue)
		{
			LogEventProperty infoProperty = new LogEventProperty("ExceptionInfo", new ScalarValue(exceptionInfoValue));
			LogEventProperty detailProperty = new LogEventProperty("ExceptionDetail", new ScalarValue(exceptionDetailValue));
			return CreateLogEvent(messageTemplate, exception, LogEventLevel.Error, infoProperty, detailProperty);
		}

		static string Format(Serilog.Formatting.ITextFormatter formatter, LogEvent logEvent)
		{
			using StringWriter writer = new StringWriter();
			formatter.Format(logEvent, writer);
			return writer.ToString();
		}

		[TestMethod]
		public void SmallEventPassesThroughUnchanged()
		{
			LogEvent logEvent = CreateLogEvent("Hello world");
			SizeLimitedJsonFormatter formatter = new SizeLimitedJsonFormatter(_inner, maxBytes: 256 * 1024);

			Assert.AreEqual(Format(_inner, logEvent), Format(formatter, logEvent));
		}

		[TestMethod]
		public void OversizedEventIsTruncated()
		{
			LogEvent logEvent = CreateLogEvent("This message template is long enough to exceed the tiny 100-byte limit we set for testing purposes");
			string result = Format(_truncating, logEvent);

			Assert.IsTrue(result.Contains("\"_truncated\":true", StringComparison.Ordinal), "Should contain _truncated marker");
			Assert.IsTrue(result.Contains("\"Level\":\"Information\"", StringComparison.Ordinal), "Should contain level");
			Assert.IsTrue(result.Contains("\"MessageTemplate\":", StringComparison.Ordinal), "Should contain message template");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should contain rendered message");
			Assert.IsFalse(result.Contains("ExceptionType", StringComparison.Ordinal), "Should not contain exception fields when no exception");
			Assert.IsFalse(result.Contains("ExceptionMessage", StringComparison.Ordinal), "Should not contain exception fields when no exception");
		}

		[TestMethod]
		public void TruncatedEventWithExceptionIncludesExceptionInfo()
		{
			InvalidOperationException exception = new InvalidOperationException("Something went wrong");
			LogEvent logEvent = CreateLogEvent("Operation failed", exception, LogEventLevel.Error);
			string result = Format(_truncating, logEvent);

			Assert.IsTrue(result.Contains("\"_truncated\":true", StringComparison.Ordinal), "Should contain _truncated marker");
			Assert.IsTrue(result.Contains("\"ExceptionType\":\"System.InvalidOperationException\"", StringComparison.Ordinal), "Should contain exception type");
			Assert.IsTrue(result.Contains("\"ExceptionMessage\":\"Something went wrong\"", StringComparison.Ordinal), "Should contain exception message");
			Assert.IsTrue(result.Contains("\"Exception\":", StringComparison.Ordinal), "Should contain full exception stack trace");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should contain rendered message");
		}

		[TestMethod]
		public void TruncatedEventIsValidJson()
		{
			Exception exception = new Exception("Bad \"things\"\nhappened");
			LogEvent logEvent = CreateLogEvent("Template with \"quotes\" and\nnewlines", exception, LogEventLevel.Error);
			string result = Format(_truncating, logEvent).TrimEnd();

			// Verify it parses as valid JSON
			JsonDocument.Parse(result);

			// Verify special characters were escaped
			Assert.IsFalse(result.Contains("\n\"", StringComparison.Ordinal), "Newlines in values should be escaped");
			Assert.IsTrue(result.Contains("\\n", StringComparison.Ordinal), "Should contain escaped newlines");
			Assert.IsTrue(result.Contains("\\\"quotes\\\"", StringComparison.Ordinal), "Should contain escaped quotes");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should contain rendered message");
		}

		[TestMethod]
		public void Tier2StripsEnricherPropertiesAndFits()
		{
			InvalidOperationException exception = new InvalidOperationException("Oops");
			string bulkyInfo = new string('X', 5000);
			string bulkyDetail = new string('Y', 5000);
			LogEvent logEvent = CreateLogEventWithExceptionProperties("Op failed", exception, bulkyInfo, bulkyDetail);

			// Inner formatted output includes the bulky properties
			string innerResult = Format(_inner, logEvent);
			int innerByteCount = System.Text.Encoding.UTF8.GetByteCount(innerResult);

			// Set maxBytes between stripped size (without ExceptionInfo/ExceptionDetail) and full size
			// The stripped version should be much smaller than innerByteCount
			int maxBytes = innerByteCount - 8000; // removing ~10K of bulky props should make it fit
			SizeLimitedJsonFormatter formatter = new SizeLimitedJsonFormatter(_inner, maxBytes: maxBytes);
			string result = Format(formatter, logEvent);

			Assert.IsTrue(result.Contains("\"_truncated\":true", StringComparison.Ordinal), "Should contain _truncated marker");
			Assert.IsTrue(result.Contains("\"_originalBytes\":", StringComparison.Ordinal), "Should contain original byte count");
			Assert.IsTrue(result.Contains("\"Properties\":", StringComparison.Ordinal), "Should preserve Properties object");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should preserve RenderedMessage");
			Assert.IsTrue(result.Contains("\"Exception\":", StringComparison.Ordinal), "Should preserve Exception field");
			Assert.IsFalse(result.Contains("ExceptionInfo", StringComparison.Ordinal), "Should have stripped ExceptionInfo");
			Assert.IsFalse(result.Contains("ExceptionDetail", StringComparison.Ordinal), "Should have stripped ExceptionDetail");

			// Verify valid JSON
			JsonDocument.Parse(result.TrimEnd());
		}

		[TestMethod]
		public void Tier2FallsToTier3WhenStillTooLarge()
		{
			InvalidOperationException exception = new InvalidOperationException("Oops");
			string bulkyInfo = new string('X', 1000);
			string bulkyDetail = new string('Y', 1000);
			LogEvent logEvent = CreateLogEventWithExceptionProperties("Op failed", exception, bulkyInfo, bulkyDetail);

			// Set maxBytes very small so even stripped Tier 2 output won't fit
			SizeLimitedJsonFormatter formatter = new SizeLimitedJsonFormatter(_inner, maxBytes: 100);
			string result = Format(formatter, logEvent);

			Assert.IsTrue(result.Contains("\"_truncated\":true", StringComparison.Ordinal), "Should contain _truncated marker");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should contain rendered message");
			Assert.IsTrue(result.Contains("\"Exception\":", StringComparison.Ordinal), "Should contain full exception");
			Assert.IsFalse(result.Contains("\"Properties\":", StringComparison.Ordinal), "Tier 3 should not contain Properties object");

			// Verify valid JSON
			JsonDocument.Parse(result.TrimEnd());
		}

		[TestMethod]
		public void InnerExceptionIsPreservedInTier3()
		{
			Exception inner = new InvalidOperationException("Inner cause");
			Exception outer = new AggregateException("Outer wrapper", inner);
			LogEvent logEvent = CreateLogEvent("Multi-layered failure", outer, LogEventLevel.Error);
			string result = Format(_truncating, logEvent);

			Assert.IsTrue(result.Contains("\"Exception\":", StringComparison.Ordinal), "Should contain Exception field");

			// Extract the Exception value and verify it contains both exception types
			JsonDocument doc = JsonDocument.Parse(result.TrimEnd());
			string exceptionText = doc.RootElement.GetProperty("Exception").GetString()!;
			Assert.IsTrue(exceptionText.Contains("AggregateException", StringComparison.Ordinal), "Should contain outer exception type");
			Assert.IsTrue(exceptionText.Contains("InvalidOperationException", StringComparison.Ordinal), "Should contain inner exception type");
			Assert.IsTrue(exceptionText.Contains("Inner cause", StringComparison.Ordinal), "Should contain inner exception message");
		}

		[TestMethod]
		public void NoEnricherPropertiesSkipsTier2()
		{
			// Exception without ExceptionInfo/ExceptionDetail should go straight to Tier 3
			InvalidOperationException exception = new InvalidOperationException("Plain exception");
			LogEvent logEvent = CreateLogEvent("Failed operation", exception, LogEventLevel.Error);

			// Use a small limit that forces truncation
			SizeLimitedJsonFormatter formatter = new SizeLimitedJsonFormatter(_inner, maxBytes: 100);
			string result = Format(formatter, logEvent);

			Assert.IsTrue(result.Contains("\"_truncated\":true", StringComparison.Ordinal), "Should contain _truncated marker");
			Assert.IsTrue(result.Contains("\"RenderedMessage\":", StringComparison.Ordinal), "Should contain rendered message");
			Assert.IsTrue(result.Contains("\"Exception\":", StringComparison.Ordinal), "Should contain full exception");
			Assert.IsTrue(result.Contains("\"ExceptionType\":\"System.InvalidOperationException\"", StringComparison.Ordinal), "Should contain exception type");
			Assert.IsFalse(result.Contains("\"Properties\":", StringComparison.Ordinal), "Should be Tier 3 output without Properties");

			// Verify valid JSON
			JsonDocument.Parse(result.TrimEnd());
		}

		[TestMethod]
		public void RenderedMessageContainsResolvedVariables()
		{
			LogEventProperty userProp = new LogEventProperty("User", new ScalarValue("Alice"));
			LogEventProperty actionProp = new LogEventProperty("Action", new ScalarValue("deploy"));
			LogEvent logEvent = CreateLogEvent("User {User} performed {Action}", null, LogEventLevel.Information, userProp, actionProp);

			SizeLimitedJsonFormatter formatter = new SizeLimitedJsonFormatter(_inner, maxBytes: 100);
			string result = Format(formatter, logEvent);

			JsonDocument doc = JsonDocument.Parse(result.TrimEnd());
			string messageTemplate = doc.RootElement.GetProperty("MessageTemplate").GetString()!;
			string renderedMessage = doc.RootElement.GetProperty("RenderedMessage").GetString()!;

			Assert.AreEqual("User {User} performed {Action}", messageTemplate, "MessageTemplate should contain raw template");
			Assert.IsTrue(renderedMessage.Contains("Alice", StringComparison.Ordinal), "RenderedMessage should contain resolved User value");
			Assert.IsTrue(renderedMessage.Contains("deploy", StringComparison.Ordinal), "RenderedMessage should contain resolved Action value");
			Assert.IsFalse(renderedMessage.Contains("{User}", StringComparison.Ordinal), "RenderedMessage should not contain unresolved placeholder");
			Assert.IsFalse(renderedMessage.Contains("{Action}", StringComparison.Ordinal), "RenderedMessage should not contain unresolved placeholder");
		}
	}
}
