{
  "version": 1,
  "isRoot": true,
  "tools": {
    "dotnet-ef": {
      "version": "6.0.10",
      "commands": [
        "dotnet-ef"
      ]
    }
  }
}