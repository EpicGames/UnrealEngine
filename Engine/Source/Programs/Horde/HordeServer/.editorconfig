# EditorConfig is awesome: http://EditorConfig.org

[*.cs]

# MA0032: Use an overload with a CancellationToken
dotnet_diagnostic.MA0032.severity = warning
