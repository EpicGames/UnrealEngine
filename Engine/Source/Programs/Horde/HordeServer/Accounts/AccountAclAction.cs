// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Accounts
{
	/// <summary>
	/// ACL actions valid for manipulating jobs
	/// </summary>
	public static class AccountAclAction
	{
		/// <summary>
		/// Ability to create new accounts
		/// </summary>
		public static AclAction CreateAccount { get; } = new AclAction("CreateAccount");

		/// <summary>
		/// Update an account settings
		/// </summary>
		public static AclAction UpdateAccount { get; } = new AclAction("UpdateAccount");

		/// <summary>
		/// Delete an account from the server
		/// </summary>
		public static AclAction DeleteAccount { get; } = new AclAction("DeleteAccount");

		/// <summary>
		/// Ability to view account information
		/// </summary>
		public static AclAction ViewAccount { get; } = new AclAction("ViewAccount");
	}
}
