// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Accounts;
using HordeServer.Acls;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Accounts
{
	/// <summary>
	/// Password hasher
	/// </summary>
	public static class PasswordHasher
	{
		private const int SaltSize = 16; // 128 bit 
		private const int KeySize = 32; // 256 bit
		private const int Iterations = 100000; // Number of iterations

		/// <summary>
		/// Generate a new salt for use with password hash
		/// </summary>
		/// <returns>A salt</returns>
		public static byte[] GenerateSalt()
		{
			using RandomNumberGenerator rng = RandomNumberGenerator.Create();
			byte[] salt = new byte[SaltSize];
			rng.GetBytes(salt);
			return salt;
		}

		/// <summary>
		/// Create a hash for the given password
		/// </summary>
		/// <param name="password">Clear text password</param>
		/// <param name="salt">Salt to hash with</param>
		/// <returns></returns>
		public static byte[] HashPassword(string password, byte[] salt)
		{
			return Rfc2898DeriveBytes.Pbkdf2(password, salt, Iterations, HashAlgorithmName.SHA256, KeySize);
		}

		/// <summary>
		/// Validate a password
		/// </summary>
		/// <param name="password">Clear text password</param>
		/// <param name="salt">Salt to hash with</param>
		/// <param name="correctHash">Correct hash</param>
		/// <returns>True if password matches correct hash</returns>
		public static bool ValidatePassword(string password, byte[] salt, byte[] correctHash)
		{
			byte[] hash = HashPassword(password, salt);
			return CryptographicOperations.FixedTimeEquals(hash, correctHash);
		}

		/// <summary>
		/// Convert a salt string to byte array
		/// </summary>
		/// <param name="saltString">Salt stored as a hex string</param>
		/// <returns>A byte array representation</returns>
		/// <exception cref="ArgumentException">If salt is invalid</exception>
		public static byte[] SaltFromString(string? saltString)
		{
			byte[] salt = Convert.FromHexString(saltString ?? "");
			if (salt.Length != 16)
			{
				throw new ArgumentException($"Invalid salt length: {salt.Length}");
			}
			return salt;
		}

		/// <summary>
		/// Convert a hash string to byte array
		/// </summary>
		/// <param name="hashString">Hash stored as a hex string</param>
		/// <returns>A byte array representation</returns>
		/// <exception cref="ArgumentException">If hash is invalid</exception>
		public static byte[] HashFromString(string? hashString)
		{
			byte[] hash = Convert.FromHexString(hashString ?? "");
			if (hash.Length != 32)
			{
				throw new ArgumentException($"Invalid hash length: {hash.Length}");
			}
			return hash;
		}
	}

	/// <summary>
	/// Collection of service account documents
	/// </summary>
	public class AccountCollection : IAccountCollection
	{
		record class ClaimDocument(string Type, string Value) : IUserClaim
		{
			public ClaimDocument(IUserClaim claim) : this(claim.Type, claim.Value)
			{ }

			public ClaimDocument(AclClaimConfig claim) : this(claim.Type, claim.Value)
			{ }
		}

		/// <summary>
		/// Concrete implementation of IHordeAccount
		/// </summary>
		private class AccountDocument : IAccount
		{
			[BsonIgnore]
			AccountCollection? _accountCollection;

			/// <inheritdoc/>
			[BsonRequired, BsonId]
			public AccountId Id { get; set; }

			/// <inheritdoc/>
			public string Name { get; set; } = "";

			/// <inheritdoc/>
			public string Login { get; set; } = "";

			/// <inheritdoc/>
			public string NormalizedLogin { get; set; } = "";

			/// <inheritdoc/>
			public string? Email { get; set; }

			/// <inheritdoc/>
			public string? PasswordHash { get; set; }

			/// <inheritdoc/>
			public string? PasswordSalt { get; set; }

			[BsonElement("Claims2")]
			public List<ClaimDocument> Claims { get; set; } = new List<ClaimDocument>();

			/// <inheritdoc/>
			public bool Enabled { get; set; }

			/// <inheritdoc/>
			public string Description { get; set; } = "";

			/// <inheritdoc/>
			public string SessionKey { get; set; } = "";

			[BsonIgnoreIfDefault, BsonDefaultValue(0)]
			public int UpdateIndex { get; set; }

			IReadOnlyList<IUserClaim> IAccount.Claims => Claims;

			[BsonConstructor]
			private AccountDocument()
			{
			}

			public AccountDocument(AccountId id, string name, string login)
			{
				Id = id;
				Name = name;
				Login = login;
				NormalizedLogin = NormalizeLogin(login);
			}

			public void PostLoad(AccountCollection accountCollection)
			{
				_accountCollection = accountCollection;
			}

			public bool ValidatePassword(string password)
			{
				if (String.IsNullOrEmpty(PasswordHash))
				{
					return String.IsNullOrEmpty(password);
				}
				else
				{
					return PasswordSalt != null && PasswordHasher.ValidatePassword(password, PasswordHasher.SaltFromString(PasswordSalt), PasswordHasher.HashFromString(PasswordHash));
				}
			}

			public async Task<IAccount?> RefreshAsync(CancellationToken cancellationToken)
				=> await _accountCollection!.GetAsync(Id, cancellationToken);

			public async Task<IAccount?> TryUpdateAsync(UpdateAccountOptions options, CancellationToken cancellationToken)
				=> await _accountCollection!.TryUpdateAsync(this, options, cancellationToken);
		}

		const int DuplicateKeyErrorCode = 11000;

		private static readonly AccountId s_defaultAdminAccountId = AccountId.Parse("65d4f282ff286703e0609ccd");

		private bool _hasCreatedAdminAccount = false;
		private readonly IMongoCollection<AccountDocument> _accounts;

		static string NormalizeLogin(string login)
			=> login.ToUpperInvariant();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service</param>
		public AccountCollection(IMongoService mongoService)
		{
			_accounts = mongoService.GetCollection<AccountDocument>("Accounts", keys => keys.Ascending(x => x.NormalizedLogin), unique: true);
		}

		/// <inheritdoc/>
		public async Task<IAccount> CreateAsync(
			CreateAccountOptions options,
			CancellationToken cancellationToken = default)
		{
			AccountDocument account = new(new AccountId(BinaryIdUtils.CreateNew()), options.Name, options.Login)
			{
				Email = options.Email,
				Description = options.Description ?? "",
				Enabled = options.Enabled ?? true
			};

			if (options.Claims != null)
			{
				account.Claims = options.Claims.ConvertAll(x => new ClaimDocument(x));
			}

			if (options.Password != null)
			{
				(string passwordSalt, string passwordHash) = CreateSaltAndHashPassword(options.Password);
				account.PasswordSalt = passwordSalt;
				account.PasswordHash = passwordHash;
			}

			try
			{
				await _accounts.InsertOneAsync(account, (InsertOneOptions?)null, cancellationToken);
			}
			catch (MongoWriteException ex) when (ex.WriteError.Code == DuplicateKeyErrorCode)
			{
				throw new LoginAlreadyTakenException(options.Login);
			}

			account.PostLoad(this);
			return account;
		}

		/// <inheritdoc/>
		public async ValueTask CreateAdminAccountAsync(string password, CancellationToken cancellationToken)
		{
			if (!_hasCreatedAdminAccount)
			{
				(string passwordSalt, string passwordHash) = CreateSaltAndHashPassword(password);

				const string AdminLogin = "Admin";
				UpdateDefinition<AccountDocument> update = Builders<AccountDocument>.Update
					.SetOnInsert(x => x.Name, "Admin")
					.SetOnInsert(x => x.Login, AdminLogin)
					.SetOnInsert(x => x.NormalizedLogin, NormalizeLogin(AdminLogin))
					.SetOnInsert(x => x.Description, "Default administrator account")
					.SetOnInsert(x => x.Claims, new List<ClaimDocument> { new ClaimDocument(HordeClaims.AdminClaim) })
					.SetOnInsert(x => x.PasswordSalt, passwordSalt)
					.SetOnInsert(x => x.PasswordHash, passwordHash)
					.SetOnInsert(x => x.Enabled, true);

				await _accounts.UpdateOneAsync(x => x.Id == s_defaultAdminAccountId, update, new UpdateOptions { IsUpsert = true }, cancellationToken);
				_hasCreatedAdminAccount = true;
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IAccount>> FindAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			List<IAccount> accounts = new List<IAccount>();
			await foreach (AccountDocument account in _accounts.Find(FilterDefinition<AccountDocument>.Empty).Range(index, count).ToAsyncEnumerable(cancellationToken))
			{
				account.PostLoad(this);
				accounts.Add(account);
			}

			return accounts;
		}

		/// <inheritdoc/>
		public async Task<IAccount?> GetAsync(AccountId id, CancellationToken cancellationToken = default)
		{
			AccountDocument? account = await _accounts.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			account?.PostLoad(this);
			return account;
		}

		/// <inheritdoc/>
		public async Task<IAccount?> FindByLoginAsync(string login, CancellationToken cancellationToken = default)
		{
			string normalizedLogin = NormalizeLogin(login);

			AccountDocument? account = await _accounts.Find(x => x.NormalizedLogin == normalizedLogin).FirstOrDefaultAsync(cancellationToken);
			account?.PostLoad(this);
			return account;
		}

		/// <inheritdoc/>
		async Task<AccountDocument?> TryUpdateAsync(AccountDocument document, UpdateAccountOptions options, CancellationToken cancellationToken = default)
		{
			UpdateDefinition<AccountDocument> update = Builders<AccountDocument>.Update.Set(x => x.UpdateIndex, document.UpdateIndex + 1);

			if (options.Name != null)
			{
				update = update.Set(x => x.Name, options.Name);
			}
			if (options.Login != null)
			{
				update = update.Set(x => x.Login, options.Login).Set(x => x.NormalizedLogin, NormalizeLogin(options.Login));
			}
			if (options.Email != null)
			{
				update = update.Set(x => x.Email, options.Email);
			}
			if (options.Password != null)
			{
				(string salt, string hash) = CreateSaltAndHashPassword(options.Password);
				update = update.Set(x => x.PasswordSalt, salt).Set(x => x.PasswordHash, hash);
			}
			if (options.Claims != null)
			{
				update = update.Set(x => x.Claims, options.Claims.ConvertAll(x => new ClaimDocument(x)));
			}
			if (options.Enabled != null)
			{
				update = update.Set(x => x.Enabled, options.Enabled);
			}
			if (options.Description != null)
			{
				update = update.Set(x => x.Description, options.Description);
			}
			if (options.SessionKey != null)
			{
				update = update.Set(x => x.SessionKey, options.SessionKey);
			}

			FilterDefinition<AccountDocument> filter;
			if (document.UpdateIndex == 0)
			{
				filter = Builders<AccountDocument>.Filter.Eq(x => x.Id, document.Id) & Builders<AccountDocument>.Filter.Exists(x => x.UpdateIndex, false);
			}
			else
			{
				filter = Builders<AccountDocument>.Filter.Eq(x => x.Id, document.Id) & Builders<AccountDocument>.Filter.Eq(x => x.UpdateIndex, document.UpdateIndex);
			}

			try
			{
				return await _accounts.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<AccountDocument, AccountDocument?> { ReturnDocument = ReturnDocument.After }, cancellationToken);
			}
			catch (MongoCommandException ex) when (options.Login != null && ex.Code == DuplicateKeyErrorCode)
			{
				throw new LoginAlreadyTakenException(options.Login);
			}
		}

		/// <inheritdoc/>
		public Task DeleteAsync(AccountId id, CancellationToken cancellationToken = default)
		{
			if (id == s_defaultAdminAccountId)
			{
				throw new InvalidOperationException("The default administrator account cannot be deleted.");
			}
			return _accounts.DeleteOneAsync(x => x.Id == id, cancellationToken);
		}

		static (string Salt, string Hash) CreateSaltAndHashPassword(string password)
		{
			byte[] salt = PasswordHasher.GenerateSalt();
			byte[] hashedPassword = PasswordHasher.HashPassword(password, salt);
			return (Convert.ToHexString(salt), Convert.ToHexString(hashedPassword));
		}
	}

	/// <summary>
	/// Exception thrown when a user name is already taken
	/// </summary>
	public class LoginAlreadyTakenException : Exception
	{
		internal LoginAlreadyTakenException(string name)
			: base($"The login '{name}' is already taken")
		{
		}
	}
}
