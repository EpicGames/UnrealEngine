// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Server;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Accounts
{
	/// <summary>
	/// Controller for the /api/v1/accounts endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	public class AccountsController : HordeControllerBase
	{
		readonly IAccountCollection _accountCollection;
		readonly GlobalConfig _globalConfig;
		readonly IOptionsMonitor<ServerSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public AccountsController(IAccountCollection accountCollection, IOptionsSnapshot<GlobalConfig> globalConfig, IOptionsMonitor<ServerSettings> settings)
		{
			_accountCollection = accountCollection;
			_globalConfig = globalConfig.Value;
			_settings = settings;
		}

		/// <summary>
		/// Create a new account
		/// </summary>
		[HttpPost]
		[Route("/api/v1/accounts")]
		[ProducesResponseType(typeof(CreateAccountResponse), 200)]
		public async Task<ActionResult<CreateAccountResponse>> CreateAccountAsync([FromBody] CreateAccountRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.CreateAccount, User))
			{
				return Forbid(AccountAclAction.CreateAccount);
			}

			List<IUserClaim> claims = request.Claims.ConvertAll<IUserClaim>(x => new UserClaim(x.Type, x.Value));
			IAccount account = await _accountCollection.CreateAsync(new CreateAccountOptions(request.Name, request.Login, claims, request.Description, request.Email, request.Password, request.Enabled), cancellationToken);
			return new CreateAccountResponse(account.Id);
		}

		/// <summary>
		/// Gets a list of accounts
		/// </summary>
		[HttpGet]
		[Route("/api/v1/accounts")]
		[ProducesResponseType(typeof(List<GetAccountResponse>), 200)]
		public async Task<ActionResult<List<GetAccountResponse>>> FindAccountsAsync([FromQuery] int? index = null, [FromQuery] int? count = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.ViewAccount, User))
			{
				return Forbid(AccountAclAction.ViewAccount);
			}

			List<GetAccountResponse> responses = new List<GetAccountResponse>();

			IReadOnlyList<IAccount> accounts = await _accountCollection.FindAsync(index, count, cancellationToken);
			foreach (IAccount account in accounts)
			{
				responses.Add(CreateGetAccountResponse(account));
			}

			return responses;
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpGet]
		[Route("/api/v1/accounts/current")]
		[ProducesResponseType(typeof(GetAccountResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<object>> GetCurrentAccountAsync([FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			AccountId? accountId = User.GetAccountId();
			if (accountId == null)
			{
				return BadRequest("User is not logged in through a Horde account");
			}

			IAccount? account = await _accountCollection.GetAsync(accountId.Value, cancellationToken);
			if (account == null)
			{
				return NotFound(accountId.Value);
			}

			GetAccountResponse response = CreateGetAccountResponse(account);
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpPut]
		[Route("/api/v1/accounts/current")]
		[ProducesResponseType(200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult> UpdateCurrentAccountAsync(UpdateCurrentAccountRequest request, CancellationToken cancellationToken = default)
		{
			AccountId? accountId = User.GetAccountId();
			if (accountId == null)
			{
				return BadRequest("User is not logged in through a Horde account");
			}

			for (; ; )
			{
				IAccount? account = await _accountCollection.GetAsync(accountId.Value, cancellationToken);
				if (account == null)
				{
					return NotFound(accountId.Value);
				}
				if (request.NewPassword != null && !account.ValidatePassword(request.OldPassword ?? String.Empty))
				{
					return Unauthorized($"Invalid password for user");
				}

				account = await account.TryUpdateAsync(new UpdateAccountOptions { Password = request.NewPassword }, cancellationToken);
				if (account != null)
				{
					break;
				}
			}

			return Ok();
		}

		/// <summary>
		/// Gets information about an account by id
		/// </summary>
		[HttpGet]
		[Route("/api/v1/accounts/{id}")]
		[ProducesResponseType(typeof(GetAccountResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<object>> GetAccountAsync(AccountId id, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.ViewAccount, User))
			{
				return Forbid(AccountAclAction.ViewAccount);
			}

			IAccount? account = await _accountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			GetAccountResponse response = CreateGetAccountResponse(account);
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpGet]
		[Route("/api/v1/accounts/{id}/entitlements")]
		[ProducesResponseType(typeof(GetAccountEntitlementsResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<object>> GetAccountEntitlementsAsync(AccountId id, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.ViewAccount, User))
			{
				return Forbid(AccountAclAction.ViewAccount);
			}

			IAccount? account = await _accountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			GetAccountEntitlementsResponse response = AccountController.CreateGetAccountEntitlementsResponse(_globalConfig.Acl, claim => account.HasClaim(claim));
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Updates an account by id
		/// </summary>
		[HttpPut]
		[Route("/api/v1/accounts/{id}")]
		[ProducesResponseType(200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult> UpdateAccountAsync(AccountId id, UpdateAccountRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.UpdateAccount, User))
			{
				return Forbid(AccountAclAction.UpdateAccount);
			}

			IAccount? account = await _accountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			IReadOnlyList<IUserClaim>? claims = null;
			if (request.Claims != null)
			{
				claims = request.Claims.ConvertAll(x => new UserClaim(x.Type, x.Value));
			}

			UpdateAccountOptions options = new UpdateAccountOptions(request.Name, request.Login, claims, request.Description, request.Email, request.Password, request.Enabled);

			account = await account.UpdateAsync(options, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			return Ok();
		}

		/// <summary>
		/// Deletes an account by id
		/// </summary>
		[HttpDelete]
		[Route("/api/v1/accounts/{id}")]
		[ProducesResponseType(200)]
		public async Task<ActionResult> DeleteAccountAsync(AccountId id, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(AccountAclAction.DeleteAccount, User))
			{
				return Forbid(AccountAclAction.DeleteAccount);
			}

			await _accountCollection.DeleteAsync(id, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpPost]
		[AllowAnonymous]
		[Route("/api/v1/accounts/admin/create")]
		public async Task<ActionResult> CreateAdminAccountAsync(CreateAdminAccountRequest request, CancellationToken cancellationToken = default)
		{
			if (_settings.CurrentValue.AuthMethod != AuthMethod.Horde)
			{
				return NotFound();
			}

			IAccount? account = await _accountCollection.FindByLoginAsync("Admin", cancellationToken);
			if (account != null)
			{
				return Forbid("Admin account already exists");
			}

			await _accountCollection.CreateAdminAccountAsync(request.Password, cancellationToken);

			account = await _accountCollection.FindByLoginAsync("Admin", cancellationToken);
			if (account != null)
			{
				return Ok(new CreateAccountResponse(account.Id));
			}

			return Forbid("Unable to find Admin account after creation");
		}

		static GetAccountResponse CreateGetAccountResponse(IAccount account)
		{
			List<AccountClaimMessage> claims = new List<AccountClaimMessage>();
			foreach (IUserClaim claim in account.Claims)
			{
				claims.Add(new AccountClaimMessage(claim.Type, claim.Value));
			}
			return new GetAccountResponse(account.Id, account.Name, account.Login, claims, account.Description, account.Email, account.Enabled);
		}
	}
}
