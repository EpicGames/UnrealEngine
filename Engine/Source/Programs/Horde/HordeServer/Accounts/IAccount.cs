// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using HordeServer.Acls;
using HordeServer.Users;

namespace HordeServer.Accounts
{
	/// <summary>
	/// An internal Horde account representing a user or service
	///
	/// Service-to-service authentication always use this for authentication (for example, Robomerge accessing Horde)
	/// When external authentication is enabled (such as OpenID Connect) users cannot be authenticated through this.
	/// </summary>
	public interface IAccount
	{
		/// <summary>
		/// Unique internal ID for this Horde account
		/// </summary>
		AccountId Id { get; }

		/// <summary>
		/// Full name of the user
		/// </summary>
		string Name { get; }

		/// <summary>
		/// A login ID or username
		/// </summary>
		string Login { get; }

		/// <summary>
		/// Email associated with account
		/// </summary>
		string? Email { get; }

		/// <summary>
		/// Hashed password
		/// </summary>
		string? PasswordHash { get; }

		/// <summary>
		/// Salt for password hash (if PasswordHash is set)
		/// </summary>
		string? PasswordSalt { get; }

		/// <summary>
		/// If the account is active
		/// </summary>
		bool Enabled { get; }

		/// <summary>
		/// Description of the account (who is it for, is there an owner etc)
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Get list of claims
		/// </summary>
		/// <returns>List of claims</returns>
		IReadOnlyList<IUserClaim> Claims { get; }

		/// <summary>
		/// Key for the current session. Refresh tokens are considered invalid when the session key no longer matches.
		/// </summary>
		string SessionKey { get; }

		/// <summary>
		/// Validate that a password is correct for this account
		/// </summary>
		bool ValidatePassword(string password);

		/// <summary>
		/// Get the latest version of this account
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New account object</returns>
		Task<IAccount?> RefreshAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to update settings for the account
		/// </summary>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>On success, returns the updated account object</returns>
		Task<IAccount?> TryUpdateAsync(UpdateAccountOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Options for updating an account
	/// </summary>
	/// <param name="Name">If set, name of account to update</param>
	/// <param name="Login">If set, login ID/username to update</param>
	/// <param name="Claims">If set, claims to update</param>
	/// <param name="Description">If set, description to update</param>
	/// <param name="Email">If set, email to update</param>
	/// <param name="Password">If set, password hash to update</param>
	/// <param name="Enabled">If set, enabled flag to update</param>
	/// <param name="SessionKey">If set, identifier for the current session</param>
	public record class UpdateAccountOptions(
		string? Name = null,
		string? Login = null,
		IReadOnlyList<IUserClaim>? Claims = null,
		string? Description = null,
		string? Email = null,
		string? Password = null,
		bool? Enabled = null,
		string? SessionKey = null);

	/// <summary>
	/// Extension methods for accounts
	/// </summary>
	public static class AccountExtensions
	{
		/// <summary>
		/// Test whether a user has a particular claim
		/// </summary>
		/// <param name="account">Account to test</param>
		/// <param name="type">Claim type to check for</param>
		/// <param name="value">Claim value to check for</param>
		/// <returns>True if the user has the claim</returns>
		public static bool HasClaim(this IAccount account, string type, string value)
		{
			foreach (IUserClaim claim in account.Claims)
			{
				if (claim.Type.Equals(type, StringComparison.OrdinalIgnoreCase) && claim.Value.Equals(value, StringComparison.Ordinal))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Test whether a user has a particular claim
		/// </summary>
		/// <param name="account">Account to test</param>
		/// <param name="claim">Claim to test for</param>
		/// <returns>True if the user has the claim</returns>
		public static bool HasClaim(this IAccount account, AclClaimConfig claim)
			=> HasClaim(account, claim.Type, claim.Value);

		/// <summary>
		/// Update settings for the account, retrying if the account object has changed
		/// </summary>
		/// <param name="account">The account to update</param>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>On success, returns the updated account object</returns>
		public static async Task<IAccount?> UpdateAsync(this IAccount account, UpdateAccountOptions options, CancellationToken cancellationToken = default)
		{
			IAccount? updatedAccount = account;
			while (updatedAccount != null)
			{
				updatedAccount = await updatedAccount.TryUpdateAsync(options, cancellationToken);
				if (updatedAccount != null)
				{
					return updatedAccount;
				}

				updatedAccount = await account.RefreshAsync(cancellationToken);
			}
			return null;
		}
	}
}
