// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using HordeServer.Users;

namespace HordeServer.Accounts
{
	/// <summary>
	/// Interface for a collection of accounts
	/// </summary>
	public interface IAccountCollection
	{
		/// <summary>
		/// Adds a new account to the collection
		/// </summary>
		/// <param name="options">Options for the new account</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IAccount> CreateAsync(CreateAccountOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches service accounts
		/// </summary>
		/// <param name="index">Index of the first account</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IReadOnlyList<IAccount>> FindAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get an account via login ID
		/// </summary>
		/// <param name="login">Login or username to use for searching</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IAccount?> FindByLoginAsync(string login, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get service account via ID
		/// </summary>
		/// <param name="id">The unique service account id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IAccount?> GetAsync(AccountId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Delete a service account from the collection
		/// </summary>
		/// <param name="id">The service account</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(AccountId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Creates the admin account when using Horde authentication
		/// </summary>
		/// <param name="password"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		ValueTask CreateAdminAccountAsync(string password, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Options for a new account
	/// </summary>
	/// <param name="Name">Name of the account (for example a given full name or name of service)</param>
	/// <param name="Login">Login ID or username</param>
	/// <param name="Claims">Optional list of claims</param>
	/// <param name="Description">Optional description</param>
	/// <param name="Email">Optional e-mail address</param>
	/// <param name="Password">Optional password for interactive login</param>
	/// <param name="Enabled">Whether the account should be enabled</param>
	public record class CreateAccountOptions(
			string Name,
			string Login,
			IReadOnlyList<IUserClaim>? Claims = null,
			string? Description = null,
			string? Email = null,
			string? Password = null,
			bool? Enabled = null);
}
