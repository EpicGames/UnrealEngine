// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IdentityModel.Tokens.Jwt;
using System.Linq;
using System.Security.Claims;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Web;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Server;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using IdentityModel;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Filters;
using Microsoft.AspNetCore.WebUtilities;
using Microsoft.Extensions.Options;
using Microsoft.IdentityModel.Tokens;

namespace HordeServer.Accounts
{
	/// <summary>
	/// Controller for /api/v1/oauth2 endpoints
	/// </summary>
	[ApiController]
	[Authorize]
	[ServiceFilter<OAuthControllerFilter>]
	public class OAuthController : Controller
	{
		readonly GlobalsService _globalsService;
		readonly IAccountCollection _accountCollection;
		readonly ServerSettings _serverSettings;

		/// <summary>
		/// Constructor
		/// </summary>
		public OAuthController(GlobalsService globalsService, IAccountCollection accountCollection, IOptionsSnapshot<ServerSettings> serverSettings)
		{
			_globalsService = globalsService;
			_accountCollection = accountCollection;
			_serverSettings = serverSettings.Value;
		}

		/// <summary>
		/// Implements the OIDC discovery endpoint
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/oauth2/.well-known/openid-configuration")]
		public ActionResult GetOpenIdConfiguration()
		{
			// Return the oidc discovery document
			// https://openid.net/specs/openid-connect-discovery-1_0.html

			return Ok(new
			{
				issuer = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2"),
				authorization_endpoint = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2/authorize"),
				token_endpoint = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2/token"),
				userinfo_endpoint = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2/userinfo"),
				end_session_endpoint = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2/logout"),
				jwks_uri = new Uri(_serverSettings.ServerUrl, "api/v1/oauth2/.well-known/jwks.json"),
				response_types_supported = new string[] { "code" },
				subject_types_supported = new[] { "public" },
				id_token_signing_alg_values_supported = new[] { "RS256" }
			});
		}

		/// <summary>
		/// Query public signing keys for our JWTs
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/oauth2/.well-known/jwks.json")]
		public async Task<ActionResult> GetJwksAsync(CancellationToken cancellationToken)
		{
			IGlobals globals = await _globalsService.GetAsync(cancellationToken);
			RsaSecurityKey rsaSecurityKey = globals.RsaSigningKey;

			object response = new
			{
				keys = new object[]
				{
					new
					{
						kty = "RSA",
						alg = "RS256",
						kid = rsaSecurityKey.KeyId,
						use = "sig",
						e = Base64UrlEncoder.Encode(rsaSecurityKey.Parameters.Exponent),
						n = Base64UrlEncoder.Encode(rsaSecurityKey.Parameters.Modulus)
					}
				}
			};

			return Ok(response);
		}

		/// <summary>
		/// Entry point for authorization. Displays the login page.
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/oauth2/authorize")]
		public ActionResult Authorize()
		{
			return View("~/Server/HordeAccountLogin.cshtml", new HordeAccountLoginViewModel
			{
				FormPostUrl = $"/api/v1/oauth2/login{Request.QueryString}"
			});
		}
		
		private static bool TryParseUrl(string candidate, out Uri uri)
		{
			uri = null!;
			if (!Uri.TryCreate(candidate, UriKind.Absolute, out Uri? parsed))
			{
				return false;
			}
			
			// Disallow fragment and userinfo per OAuth spec
			bool hasFragment = !String.IsNullOrEmpty(parsed.Fragment);
			bool hasUserInfo = !String.IsNullOrEmpty(parsed.UserInfo);
			
			if (hasFragment || hasUserInfo)
			{
				return false;
			}
			
			uri = parsed;
			return true;
		}
		
		/// <summary>
		/// Post interactive login credentials to the server in exchange for an authorization token
		/// </summary>
		[HttpPost]
		[AllowAnonymous]
		[Route("/api/v1/oauth2/login")]
		public async Task<ActionResult> LoginAsync(OAuthAuthorizeRequest request, [FromForm(Name = "username")] string? userName, [FromForm(Name = "password")] string? password, CancellationToken cancellationToken = default)
		{
			// We only support the authorization code flow
			if (request.ResponseType != "code")
			{
				return BadRequest($"Unsupported response_type '{request.ResponseType}'");
			}
			if (String.IsNullOrEmpty(userName))
			{
				return BadRequest("Missing username from form post");
			}

			if (!TryParseUrl(request.RedirectUri, out Uri _))
			{
				return BadRequest("Invalid redirect URI");
			}

			// Check the login credentials and update the session key
			IAccount? account = null;
			while (account == null)
			{
				account = await _accountCollection.FindByLoginAsync(userName, cancellationToken);
				if (account == null)
				{
					return Unauthorized();
				}
				if (!account.ValidatePassword(password ?? String.Empty))
				{
					return Unauthorized();
				}
				account = await account.TryUpdateAsync(new UpdateAccountOptions { SessionKey = RandomNumberGenerator.GetHexString(16) }, cancellationToken);
			}

			// Create a short-lived authorization token with information from the request
			IGlobals globals = await _globalsService.GetAsync(cancellationToken);
			string authToken = CreateAuthorizationToken(globals, account, request.PkceCodeChallenge, request.PkceCodeChallengeMethod, request.Nonce);

			// Build the response
			Dictionary<string, string> responseFields = new Dictionary<string, string>();
			responseFields.Add("code", authToken);
			if (request.State != null)
			{
				responseFields.Add("state", request.State);
			}

			// Send the response
			if (request.ResponseMode.Equals("query", StringComparison.Ordinal))
			{
				// Redirect directly to the user's requested URL
				string redirectUri = request.RedirectUri;
				foreach ((string key, string value) in responseFields)
				{
					redirectUri = QueryHelpers.AddQueryString(redirectUri, key, value);
				}
				return Redirect(redirectUri);
			}
			else if (request.ResponseMode.Equals("form_post", StringComparison.Ordinal))
			{
				// Return a page that automatically does a form post on the client side
				// See https://openid.net/specs/oauth-v2-form-post-response-mode-1_0.html

				StringBuilder html = new StringBuilder();
				html.AppendLine($"<html>");
				html.AppendLine($"  <body onload=\"javascript:document.forms[0].submit()\">");
				html.AppendLine($"    <form method=\"post\" action=\"{HttpUtility.HtmlAttributeEncode(request.RedirectUri)}\">");
				foreach ((string key, string value) in responseFields)
				{
					html.AppendLine($"      <input type=\"hidden\" name=\"{HttpUtility.HtmlAttributeEncode(key)}\" value=\"{HttpUtility.HtmlAttributeEncode(value)}\"/>");
				}
				html.AppendLine($"    </form>");
				html.AppendLine($"  </body>");
				html.AppendLine($"</html>");

				return Content(html.ToString(), "text/html", Encoding.UTF8);
			}
			else
			{
				return BadRequest($"Invalid response_mode '{request.ResponseMode}'");
			}
		}

		/// <summary>
		/// Logs out of the current session
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/oauth2/logout")]
		public async Task<ActionResult> LogoutAsync(CancellationToken cancellationToken)
		{
			AccountId? accountId = User.GetAccountId();
			if (accountId == null)
			{
				return Unauthorized("User does not have an account id claim");
			}

			for (; ; )
			{
				IAccount? account = await _accountCollection.GetAsync(accountId.Value, cancellationToken);
				if (account == null)
				{
					return NotFound($"Account {accountId.Value} not found");
				}
				if (String.IsNullOrEmpty(account.SessionKey))
				{
					return Ok();
				}
				if (await account.TryUpdateAsync(new UpdateAccountOptions { SessionKey = "" }, cancellationToken) != null)
				{
					return Ok();
				}
			}
		}

		/// <summary>
		/// Exchange a token for another token
		/// </summary>
		[HttpPost]
		[AllowAnonymous]
		[Route("/api/v1/oauth2/token")]
		public async Task<ActionResult<OAuthGetTokenResponse>> ExchangeTokenAsync([FromForm] OAuthGetTokenRequest request, CancellationToken cancellationToken)
		{
			// Validate the supplied token
			IGlobals globals = await _globalsService.GetAsync(cancellationToken);

			JwtPayload payload;
			if (request.GrantType == "authorization_code")
			{
				payload = await ValidateTokenAsync(globals, request.AuthorizationToken);

				// Check the supplied token is correct for the operation we're performing
				string? purpose = GetClaimOrDefault(payload, PurposeClaim);
				if (purpose != AuthorizationCodePurpose)
				{
					OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_request", ErrorDescription = $"Expected '{AuthorizationCodePurpose}' purpose, not '{purpose}'" };
					return BadRequest(error);
				}

				// Perform the PKCE challenge
				string? pkceChallenge = GetClaimOrDefault(payload, PkceCodeChallengeClaim);
				if (!String.IsNullOrEmpty(pkceChallenge))
				{
					string? method = GetClaimOrDefault(payload, PkceCodeChallengeMethodClaim);

					string verifier = ComputePkceVerifier(method, request.PkceCodeVerifier ?? String.Empty);
					if (!String.Equals(pkceChallenge, verifier, StringComparison.OrdinalIgnoreCase))
					{
						OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_request", ErrorDescription = $"PKCE verification failure" };
						return Unauthorized(error);
					}
				}
			}
			else if (request.GrantType == "refresh_token")
			{
				payload = await ValidateTokenAsync(globals, request.RefreshToken);

				// Validate that we have a refresh token
				string? purpose = GetClaimOrDefault(payload, PurposeClaim);
				if (purpose != RefreshTokenPurpose)
				{
					OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_request", ErrorDescription = $"Expected '{RefreshTokenPurpose}' purpose, not '{purpose}'" };
					return BadRequest(error);
				}
			}
			else
			{
				OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "unsupported_grant_type", ErrorDescription = $"Unsupported grant type: '{request.GrantType}'" };
				return BadRequest(error);
			}

			// Get the matching account and check the session key is still valid
			AccountId accountId;
			if (!TryParseAccountIdFromSubject(payload, out accountId))
			{
				OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_grant", ErrorDescription = $"Missing account-id subject" };
				return Unauthorized(error);
			}

			IAccount? account = await _accountCollection.GetAsync(accountId, cancellationToken);
			if (account == null)
			{
				OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_grant", ErrorDescription = $"Invalid account-id ({accountId})" };
				return Unauthorized(error);
			}

			string? session = GetClaimOrDefault(payload, SessionClaim);
			if (session != account.SessionKey)
			{
				OAuthGetTokenResponse error = new OAuthGetTokenResponse { Error = "invalid_grant", ErrorDescription = $"Invalid session key ('{session}')" };
				return Unauthorized(error);
			}

			// Create the tokens and response object
			Response.Headers.CacheControl = "no-cache";

			string? nonce = GetClaimOrDefault(payload, NonceClaim);

			OAuthGetTokenResponse response = new OAuthGetTokenResponse();
			response.ExpiresIn = 30;
			response.AccessToken = CreateAccessToken(globals, account, response.ExpiresIn.Value, nonce);
			response.TokenType = "Bearer";
			response.RefreshTokenExpiresIn = 7 * 24 * 60 * 60;
			response.RefreshToken = CreateRefreshToken(globals, account, response.RefreshTokenExpiresIn.Value, nonce);
			response.IdToken = CreateIdToken(globals, account, nonce);
			return response;
		}

		async Task<JwtPayload> ValidateTokenAsync(IGlobals globals, string? token)
		{
			TokenValidationParameters validationParameters = new TokenValidationParameters()
			{
				ValidAudience = _serverSettings.JwtIssuer,
				ValidIssuer = _serverSettings.JwtIssuer,
				IssuerSigningKey = globals.RsaSigningKey
			};

			JwtSecurityTokenHandler tokenHandler = new JwtSecurityTokenHandler();

			TokenValidationResult result = await tokenHandler.ValidateTokenAsync(token, validationParameters);
			if (!result.IsValid)
			{
				throw result.Exception;
			}

			return ((JwtSecurityToken)result.SecurityToken).Payload;
		}

		/// <summary>
		/// Get information about the logged in user
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/oauth2/userinfo")]
		public async Task<ActionResult> GetUserInfoAsync(CancellationToken cancellationToken)
		{
			AccountId? accountId = User.GetAccountId();
			if (accountId == null)
			{
				return Unauthorized("User does not have an account id claim");
			}

			IAccount? account = await _accountCollection.GetAsync(accountId.Value, cancellationToken);
			if (account == null)
			{
				return NotFound($"Account {accountId.Value} not found");
			}

			Dictionary<string, object> response = new Dictionary<string, object>();
			response["sub"] = $"{AccountSubjectPrefix}{accountId}";
			GetUserInfoClaims(account, response);

			return Ok(response);
		}

		static string? GetClaimOrDefault(JwtPayload payload, string claimType, string? claimDefault = null)
		{
			if (payload.TryGetValue(claimType, out object? claimObj) && claimObj is string claimValue)
			{
				return claimValue;
			}
			else
			{
				return claimDefault;
			}
		}

		static bool TryParseAccountIdFromSubject(JwtPayload payload, out AccountId accountId)
		{
			if (payload.Sub == null || !payload.Sub.StartsWith(AccountSubjectPrefix, StringComparison.Ordinal))
			{
				accountId = default;
				return false;
			}
			return AccountId.TryParse(payload.Sub.AsSpan(AccountSubjectPrefix.Length), out accountId);
		}

		static string ComputePkceVerifier(string? method, string code)
		{
			if (method == null || method.Equals("plain", StringComparison.Ordinal))
			{
				return code;
			}
			else if (method.Equals("S256", StringComparison.Ordinal))
			{
				return Base64Url.Encode(SHA256.HashData(Encoding.UTF8.GetBytes(code)));
			}
			else
			{
				throw new NotSupportedException($"Pkce method '{method}' is not supported");
			}
		}

		#region Tokens

		// Claim names
		const string PurposeClaim = "purpose";
		const string PkceCodeChallengeClaim = "pkce-code-challenge";
		const string PkceCodeChallengeMethodClaim = "pkce-code-challenge-method";
		const string SessionClaim = "session";
		const string NonceClaim = "nonce";

		// Token purpose values (stored in the "purpose" claim)
		const string AuthorizationCodePurpose = "auth";
		const string AccessTokenPurpose = "access";
		const string RefreshTokenPurpose = "refresh";
		const string IdTokenPurpose = "id";

		// Prefix for "sub" values referring to a Horde account
		const string AccountSubjectPrefix = "account-id:";

		string CreateAuthorizationToken(IGlobals globals, IAccount subject, string? pkceCode, string? pkceMethod, string? nonce)
		{
			JwtPayload payload = CreateJwtPayload(AuthorizationCodePurpose, subject, 30, nonce);

			payload[PkceCodeChallengeClaim] = pkceCode;
			payload[PkceCodeChallengeMethodClaim] = pkceMethod;
			payload[NonceClaim] = nonce;

			return CreateAndSignJwt(globals, payload);
		}

		string CreateAccessToken(IGlobals globals, IAccount subject, long expiresIn, string? nonce)
		{
			JwtPayload payload = CreateJwtPayload(AccessTokenPurpose, subject, expiresIn, nonce);

			payload["name"] = subject.Name;
			payload["preferred_username"] = subject.Login;
			payload["email"] = subject.Email;

			payload[ClaimTypes.Name] = subject.Name;
			payload[ClaimTypes.Email] = subject.Email;
			payload[HordeClaimTypes.AccountId] = subject.Id.ToString();
			payload[HordeClaimTypes.User] = subject.Login;
			
			// A single claim type can occur multiple times, so ensure all values captured and grouped as an array-value when more than one exists
			foreach (IGrouping<string, IUserClaim> group in subject.Claims.GroupBy(c => c.Type))
			{
				string[] values = group.Select(c => c.Value).ToArray();
				payload[group.Key] = values.Length == 1 ? values[0] : values;
			}
			
			return CreateAndSignJwt(globals, payload);
		}

		string CreateRefreshToken(IGlobals globals, IAccount subject, long expiresIn, string? nonce)
		{
			JwtPayload payload = CreateJwtPayload(RefreshTokenPurpose, subject, expiresIn, nonce);
			return CreateAndSignJwt(globals, payload);
		}

		string CreateIdToken(IGlobals globals, IAccount subject, string? nonce)
		{
			JwtPayload payload = CreateJwtPayload(IdTokenPurpose, subject, TimeSpan.FromDays(1.0), nonce);

			GetUserInfoClaims(subject, payload);

			return CreateAndSignJwt(globals, payload);
		}

		static void GetUserInfoClaims(IAccount account, Dictionary<string, object> properties)
		{
			properties["name"] = account.Name;
			properties["preferred_username"] = account.Login;

			properties[ClaimTypes.Name] = account.Name;
			if (!String.IsNullOrEmpty(account.Email))
			{
				properties["email"] = account.Email;
				properties[ClaimTypes.Email] = account.Email;
			}
			properties[HordeClaimTypes.AccountId] = account.Id.ToString();
			properties[HordeClaimTypes.User] = account.Login;

			// A single claim type can occur multiple times, so ensure all values captured and grouped as an array-value when more than one exists
			foreach (IGrouping<string, IUserClaim> group in account.Claims.GroupBy(c => c.Type))
			{
				string[] values = group.Select(c => c.Value).ToArray();
				properties[group.Key] = values.Length == 1 ? values[0] : values;
			}
		}

		JwtPayload CreateJwtPayload(string purpose, IAccount subject, TimeSpan expiresIn, string? nonce)
		{
			return CreateJwtPayload(purpose, subject, (long)expiresIn.TotalSeconds, nonce);
		}

		JwtPayload CreateJwtPayload(string purpose, IAccount subject, long expiresIn, string? nonce)
		{
			long issuedAt = (long)(DateTime.UtcNow - DateTime.UnixEpoch).TotalSeconds;

			JwtPayload payload = new JwtPayload();
			payload["iss"] = _serverSettings.JwtIssuer; // by me
			payload["aud"] = _serverSettings.JwtIssuer; // for me
			payload["iat"] = issuedAt;
			payload["exp"] = issuedAt + expiresIn;
			payload["sub"] = $"{AccountSubjectPrefix}{subject.Id}";

			payload[PurposeClaim] = purpose;
			payload[SessionClaim] = subject.SessionKey;
			if (!String.IsNullOrEmpty(nonce))
			{
				payload[NonceClaim] = nonce;
			}

			return payload;
		}

		static string CreateAndSignJwt(IGlobals globals, JwtPayload payload)
		{
			SigningCredentials signingCredentials = new SigningCredentials(globals.RsaSigningKey, SecurityAlgorithms.RsaSha256);

			JwtHeader header = new JwtHeader(signingCredentials);
			JwtSecurityToken token = new JwtSecurityToken(header, payload);

			return new JwtSecurityTokenHandler().WriteToken(token);
		}

		#endregion
	}

	/// <summary>
	/// Filters requests to the OAuth2 controller
	/// </summary>
	sealed class OAuthControllerFilter : ActionFilterAttribute
	{
		public IOptionsSnapshot<ServerSettings> Settings { get; }

		public OAuthControllerFilter(IOptionsSnapshot<ServerSettings> settings)
		{
			Settings = settings;
		}

		public override void OnActionExecuting(ActionExecutingContext context)
		{
			if (Settings.Value.AuthMethod != AuthMethod.Horde)
			{
				context.Result = new BadRequestObjectResult(new
				{
					message = "This endpoint is only available when authentication method is set to 'Horde'."
				});
			}
		}
	}
}
