// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json.Serialization;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Accounts
{
	/// <summary>
	/// Request to authorize a user using OAuth2
	/// </summary>
	public class OAuthAuthorizeRequest
	{
		/// <summary>
		/// Client identifier
		/// </summary>
		[FromQuery(Name = "client_id")]
		public string ClientId { get; set; } = String.Empty;

		/// <summary>
		/// Redirect URI that the response will be sent to
		/// </summary>
		[FromQuery(Name = "redirect_uri")]
		public string RedirectUri { get; set; } = String.Empty;

		/// <summary>
		/// OAuth response type. Should be "code".
		/// </summary>
		[FromQuery(Name = "response_type")]
		public string ResponseType { get; set; } = String.Empty;

		/// <summary>
		/// Space separated list of the requested scope values. Should include openid.
		/// </summary>
		[FromQuery(Name = "scope")]
		public string Scope { get; set; } = String.Empty;

		/// <summary>
		/// PKCE code challenge.
		/// </summary>
		[FromQuery(Name = "code_challenge")]
		public string? PkceCodeChallenge { get; set; }

		/// <summary>
		/// PKCE code challenge method. Should be "plain" or "S256".
		/// </summary>
		[FromQuery(Name = "code_challenge_method")]
		public string? PkceCodeChallengeMethod { get; set; }

		/// <summary>
		/// Response method. Only "query" is supported by Horde.
		/// </summary>
		[FromQuery(Name = "response_mode")]
		public string ResponseMode { get; set; } = "query";

		/// <summary>
		/// Random string used to prevent replay attacks
		/// </summary>
		[FromQuery(Name = "nonce")]
		public string? Nonce { get; set; }

		/// <summary>
		/// Optional application-defined state value.
		/// </summary>
		[FromQuery(Name = "state")]
		public string? State { get; set; }
	}

	/// <summary>
	/// Response from a token exchange operation
	/// </summary>
	public class OAuthGetTokenRequest
	{
		/// <summary>
		/// Type of token being specified. Should be "authorization_code" or "refresh_token".
		/// </summary>
		[FromForm(Name = "grant_type")]
		public string? GrantType { get; set; }

		/// <summary>
		/// Access token specified in the redirect.
		/// </summary>
		[FromForm(Name = "code")]
		public string? AuthorizationToken { get; set; }

		/// <summary>
		/// Refresh token specified in the redirect.
		/// </summary>
		[FromForm(Name = "refresh_token")]
		public string? RefreshToken { get; set; }

		/// <summary>
		/// 
		/// </summary>
		[FromForm(Name = "redirect_url")]
		public string? RedirectUrl { get; set; }

		/// <summary>
		/// Data that the PKCE code challenge was generated from
		/// </summary>
		[FromForm(Name = "code_verifier")]
		public string? PkceCodeVerifier { get; set; }
	}

	/// <summary>
	/// Response from the OAuth2 token endpoint (https://datatracker.ietf.org/doc/html/rfc6749#section-5.1)
	/// </summary>
	public class OAuthGetTokenResponse
	{
		/// <summary>
		/// The access token return value
		/// </summary>
		[JsonPropertyName("access_token")]
		public string? AccessToken { get; set; }

		/// <summary>
		/// Type of <see cref="AccessToken"/>. Typically "Bearer" for Horde responses.
		/// </summary>
		[JsonPropertyName("token_type")]
		public string? TokenType { get; set; }

		/// <summary>
		/// Expiry time in seconds for the access token
		/// </summary>
		[JsonPropertyName("expires_in")]
		public int? ExpiresIn { get; set; }

		/// <summary>
		/// 
		/// </summary>
		[JsonPropertyName("scope")]
		public string? Scope { get; set; }

		/// <summary>
		/// Refresh token
		/// </summary>
		[JsonPropertyName("refresh_token")]
		public string? RefreshToken { get; set; }

		/// <summary>
		/// TTL for the refresh token
		/// </summary>
		[JsonPropertyName("refresh_token_expires_in")]
		public int? RefreshTokenExpiresIn { get; set; }

		/// <summary>
		/// JWT with information about the authorized user
		/// </summary>
		[JsonPropertyName("id_token")]
		public string? IdToken { get; set; }

		/// <summary>
		/// Error code
		/// </summary>
		[JsonPropertyName("error")]
		public string? Error { get; set; }

		/// <summary>
		/// The error description text
		/// </summary>
		[JsonPropertyName("error_description")]
		public string? ErrorDescription { get; set; }
	}
}
