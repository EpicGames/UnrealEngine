// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IdentityModel.Tokens.Jwt;
using System.Linq;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;
using Microsoft.IdentityModel.Tokens;
using MongoDB.Driver;

namespace HordeServer.Acls
{
	/// <summary>
	/// Wraps functionality for manipulating permissions
	/// </summary>
	public class AclService : IAclService
	{
		private readonly GlobalsService _globalsService;
		private readonly IOptionsMonitor<GlobalConfig> _globalConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public AclService(GlobalsService globalsService, IOptionsMonitor<GlobalConfig> globalConfig)
		{
			_globalsService = globalsService;
			_globalConfig = globalConfig;
		}

		/// <summary>
		/// Issues a bearer token with the given claims
		/// </summary>
		/// <param name="claims">List of claims to include</param>
		/// <param name="expiry">Time that the token expires</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>JWT security token with a claim for creating new agents</returns>
		public async ValueTask<string> IssueBearerTokenAsync(IEnumerable<Claim> claims, TimeSpan? expiry, CancellationToken cancellationToken = default)
		{
			IGlobals globals = await _globalsService.GetAsync(cancellationToken);
			SigningCredentials signingCredentials = new(globals.JwtSigningKey, SecurityAlgorithms.HmacSha256);

			JwtSecurityToken token = new(globals.JwtIssuer, null, claims.DistinctBy(x => (x.Type, x.Value)), null, DateTime.UtcNow + expiry, signingCredentials);
			return new JwtSecurityTokenHandler().WriteToken(token);
		}

		/// <summary>
		/// Gets the agent id associated with a particular user
		/// </summary>
		/// <param name="user"></param>
		/// <returns></returns>
		public static AgentId? GetAgentId(ClaimsPrincipal user)
		{
			Claim? claim = user.Claims.FirstOrDefault(x => x.Type == HordeClaimTypes.Agent);
			if (claim == null)
			{
				return null;
			}
			else
			{
				return new AgentId(claim.Value);
			}
		}

		/// <inheritdoc/>
		public bool TryGetAclScope(AclScopeName scopeName, [NotNullWhen(true)] out AclConfig? scopeConfig)
			=> _globalConfig.CurrentValue.TryGetAclScope(scopeName, out scopeConfig);
	}
}
