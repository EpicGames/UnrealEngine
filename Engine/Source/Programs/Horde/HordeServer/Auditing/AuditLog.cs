// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.CompilerServices;
using System.Text;
using System.Threading;
using System.Threading.Channels;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Auditing
{
	class AuditLog<TSubject> : IAuditLog<TSubject>, IAsyncDisposable
	{
		class AuditLogMessage : IAuditLogMessage<TSubject>
		{
			public ObjectId Id { get; set; }

			[BsonElement("s")]
			public TSubject Subject { get; set; }

			[BsonElement("t")]
			public DateTime TimeUtc { get; set; }

			[BsonElement("l")]
			public LogLevel Level { get; set; }

			[BsonElement("d")]
			public string Data { get; set; }

			public AuditLogMessage()
			{
				Subject = default!;
				Data = String.Empty;
			}

			public AuditLogMessage(TSubject subject, DateTime timeUtc, LogLevel level, string data)
			{
				Id = ObjectId.GenerateNewId();
				Subject = subject;
				TimeUtc = timeUtc;
				Level = level;
				Data = data;
			}
		}

		class AuditLogChannel : IAuditLogChannel<TSubject>
		{
			sealed class Scope : IDisposable
			{
				readonly AuditLogChannel _owner;

				public LogEvent Message { get; }

				public Scope(AuditLogChannel owner, LogEvent message)
				{
					_owner = owner;
					Message = message;

					_owner._scopes.Add(this);
				}

				public void Dispose() => _owner._scopes.Remove(this);
			}

			public readonly AuditLog<TSubject> Outer;
			public TSubject Subject { get; }

			readonly List<Scope> _scopes = new List<Scope>();

			public AuditLogChannel(AuditLog<TSubject> outer, TSubject subject)
			{
				Outer = outer;
				Subject = subject;
			}

			public IDisposable? BeginScope<TState>(TState state) where TState : notnull => new Scope(this, LogEvent.FromState(LogLevel.Information, default, state, null, (x, y) => x?.ToString() ?? String.Empty));

			public bool IsEnabled(LogLevel logLevel) => true;

			LogEvent CreateEvent<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
			{
				LogEvent logEvent = LogEvent.FromState(logLevel, eventId, state, exception, formatter);
				if (_scopes.Count > 0)
				{
					StringBuilder message = new StringBuilder();
					StringBuilder format = new StringBuilder();
					Dictionary<string, object?> properties = new Dictionary<string, object?>(StringComparer.Ordinal);

					for (int idx = 0; idx < _scopes.Count; idx++)
					{
						message.Append('[');
						format.Append('[');
						AppendMessage(_scopes[idx].Message, idx + 1, message, format, properties);
						message.Append(']');
						format.Append(']');
					}

					message.Append(' ');
					format.Append(' ');

					AppendMessage(logEvent, 0, message, format, properties);

					logEvent.Message = message.ToString();
					logEvent.Format = format.ToString();
					logEvent.Properties = properties;
				}
				return logEvent;
			}

			static void AppendMessage(LogEvent logEvent, int id, StringBuilder message, StringBuilder format, Dictionary<string, object?> properties)
			{
				message.Append(logEvent.Message);
				if (logEvent.Format == null)
				{
					format.Append($"{{_scope{id}}}");
					properties.Add($"_scope{id}", logEvent.Message);
				}
				else
				{
					format.Append(logEvent.Format);
					if (logEvent.Properties != null)
					{
						foreach (KeyValuePair<string, object?> item in logEvent.Properties)
						{
							properties[item.Key] = item.Value;
						}
					}
				}
			}

			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
			{
				DateTime time = DateTime.UtcNow;

				LogEvent logEvent = CreateEvent(logLevel, eventId, state, exception, formatter);
				string data = logEvent.ToJson();
				AuditLogMessage message = new AuditLogMessage(Subject, time, logLevel, data);
				Outer._messageChannel.Writer.TryWrite(message);

#pragma warning disable CA2254 // Template should be a static expression
				using (IDisposable? _ = Outer._logger.BeginScope($"Subject: {{{Outer._subjectProperty}}}", Subject))
				{
					Outer._logger.Log(logLevel, eventId, state, exception, formatter);
				}
#pragma warning restore CA2254 // Template should be a static expression
			}

			public IAsyncEnumerable<IAuditLogMessage> FindAsync(DateTime? minTime, DateTime? maxTime, int? index, int? count, CancellationToken cancellationToken = default)
				=> Outer.FindAsync(Subject, minTime, maxTime, index, count, cancellationToken);

			public Task<long> DeleteAsync(DateTime? minTime, DateTime? maxTime, CancellationToken cancellationToken = default)
				=> Outer.DeleteAsync(Subject, minTime, maxTime, cancellationToken);

			public Task FlushAsync(CancellationToken cancellationToken)
				=> Outer.FlushAsync(cancellationToken);
		}

		readonly IMongoCollection<AuditLogMessage> _messages;
		readonly Channel<AuditLogMessage?> _messageChannel;
		readonly string _subjectProperty;
		readonly ILogger _logger;
		readonly BackgroundTask _backgroundTask;

		TaskCompletionSource? _flushEvent;

		public IAuditLogChannel<TSubject> this[TSubject subject] => new AuditLogChannel(this, subject);

		public AuditLog(IMongoService mongoService, string collectionName, string subjectProperty, ILogger logger)
		{
			List<MongoIndex<AuditLogMessage>> indexes = new List<MongoIndex<AuditLogMessage>>();
			indexes.Add(builder => builder.Ascending(x => x.Subject).Descending(x => x.TimeUtc));

			_messages = mongoService.GetCollection<AuditLogMessage>(collectionName, indexes);

			_messageChannel = Channel.CreateUnbounded<AuditLogMessage?>();
			_subjectProperty = subjectProperty;
			_logger = logger;

			_backgroundTask = BackgroundTask.StartNew(ctx => WriteMessagesAsync(ctx));
		}

		public async ValueTask DisposeAsync()
		{
			_messageChannel.Writer.TryComplete();
			await _backgroundTask.DisposeAsync();
		}

		/// <summary>
		/// Flush any pending messages to database
		/// Exposed as internal for use in tests
		/// </summary>
		internal async Task<int> FlushMessagesInternalAsync(CancellationToken cancellationToken)
		{
			List<AuditLogMessage> newMessages = new();
			while (_messageChannel.Reader.TryRead(out AuditLogMessage? newMessage))
			{
				if (newMessage != null)
				{
					newMessages.Add(newMessage);
				}
			}
			if (newMessages.Count > 0)
			{
				await _messages.InsertManyAsync(newMessages, null, cancellationToken);
			}

			return newMessages.Count;
		}

		async Task WriteMessagesAsync(CancellationToken cancellationToken)
		{
			while (await _messageChannel.Reader.WaitToReadAsync(cancellationToken))
			{
				TaskCompletionSource? flushEvent = Interlocked.Exchange(ref _flushEvent, null);

				List<AuditLogMessage> newMessages = new();
				while (_messageChannel.Reader.TryRead(out AuditLogMessage? newMessage))
				{
					if (newMessage != null)
					{
						newMessages.Add(newMessage);
					}
				}
				if (newMessages.Count > 0)
				{
					await _messages.InsertManyAsync(newMessages, null, cancellationToken);
				}

				flushEvent?.TrySetResult();
			}
		}

		public async Task FlushAsync(CancellationToken cancellationToken)
		{
			// Get the existing task completion source, or create a new one
			TaskCompletionSource newTcs = new TaskCompletionSource(TaskCreationOptions.RunContinuationsAsynchronously);
			TaskCompletionSource? tcs = Interlocked.CompareExchange(ref _flushEvent, newTcs, null) ?? newTcs;

			// Force the background task to run once 
			await _messageChannel.Writer.WriteAsync(null, cancellationToken);

			// Wait for the event to trigger
			await tcs.Task;
		}

		async IAsyncEnumerable<IAuditLogMessage<TSubject>> FindAsync(TSubject subject, DateTime? minTime = null, DateTime? maxTime = null, int? index = null, int? count = null, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			FilterDefinition<AuditLogMessage> filter = Builders<AuditLogMessage>.Filter.Eq(x => x.Subject, subject);
			if (minTime != null)
			{
				filter &= Builders<AuditLogMessage>.Filter.Gte(x => x.TimeUtc, minTime.Value);
			}
			if (maxTime != null)
			{
				filter &= Builders<AuditLogMessage>.Filter.Lte(x => x.TimeUtc, maxTime.Value);
			}

			using (IAsyncCursor<AuditLogMessage> cursor = await _messages.Find(filter).SortByDescending(x => x.TimeUtc).Range(index, count).ToCursorAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (AuditLogMessage message in cursor.Current)
					{
						yield return message;
					}
				}
			}
		}

		async Task<long> DeleteAsync(TSubject subject, DateTime? minTime = null, DateTime? maxTime = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<AuditLogMessage> filter = Builders<AuditLogMessage>.Filter.Eq(x => x.Subject, subject);
			if (minTime != null)
			{
				filter &= Builders<AuditLogMessage>.Filter.Gte(x => x.TimeUtc, minTime.Value);
			}
			if (maxTime != null)
			{
				filter &= Builders<AuditLogMessage>.Filter.Lte(x => x.TimeUtc, maxTime.Value);
			}

			DeleteResult result = await _messages.DeleteManyAsync(filter, cancellationToken);
			return result.DeletedCount;
		}
	}

	class AuditLogFactory<TSubject> : IAuditLogFactory<TSubject>
	{
		readonly IMongoService _mongoService;
		readonly ILogger<AuditLog<TSubject>> _logger;

		public AuditLogFactory(IMongoService mongoService, ILogger<AuditLog<TSubject>> logger)
		{
			_mongoService = mongoService;
			_logger = logger;
		}

		public IAuditLog<TSubject> Create(string collectionName, string subjectProperty)
		{
			return new AuditLog<TSubject>(_mongoService, collectionName, subjectProperty, _logger);
		}
	}
}
