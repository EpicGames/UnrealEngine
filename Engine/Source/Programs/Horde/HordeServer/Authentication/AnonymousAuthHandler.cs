// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Security.Claims;
using System.Text.Encodings.Web;
using System.Threading.Tasks;
using EpicGames.Horde.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Authentication
{
	class AnonymousAuthenticationOptions : AuthenticationSchemeOptions
	{
	}

	class AnonymousAuthHandler : AuthenticationHandler<AnonymousAuthenticationOptions>
	{
		public const string AuthenticationScheme = "Anonymous";

		public AnonymousAuthHandler(IOptionsMonitor<AnonymousAuthenticationOptions> options, ILoggerFactory logger, UrlEncoder encoder)
			: base(options, logger, encoder)
		{
		}

		protected override Task<AuthenticateResult> HandleAuthenticateAsync()
		{
			List<Claim> claims = new List<Claim>();
			claims.Add(new Claim(HordeClaimTypes.Version, HordeClaimTypes.CurrentVersion));
			claims.Add(new Claim(ClaimTypes.Name, AuthenticationScheme));
			claims.Add(new Claim(HordeClaimTypes.UserId, UserId.Anonymous.ToString()));
			claims.Add(new Claim(HordeClaims.AdminClaim.Type, HordeClaims.AdminClaim.Value));

			ClaimsIdentity identity = new ClaimsIdentity(claims, Scheme.Name);

			ClaimsPrincipal principal = new ClaimsPrincipal(identity);
			AuthenticationTicket ticket = new AuthenticationTicket(principal, Scheme.Name);

			return Task.FromResult(AuthenticateResult.Success(ticket));
		}
	}

	static class AnonymousExtensions
	{
		public static AuthenticationBuilder AddAnonymous(this AuthenticationBuilder builder, Action<AnonymousAuthenticationOptions> configure)
		{
			return builder.AddScheme<AnonymousAuthenticationOptions, AnonymousAuthHandler>(AnonymousAuthHandler.AuthenticationScheme, configure);
		}
	}
}
