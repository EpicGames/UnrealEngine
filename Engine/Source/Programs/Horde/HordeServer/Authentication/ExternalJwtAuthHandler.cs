// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.IdentityModel.Tokens.Jwt;
using System.Linq;
using System.Security.Claims;
using System.Threading.Tasks;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.JwtBearer;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.IdentityModel.Tokens;

namespace HordeServer.Authentication
{
	/// <summary>
	/// JWT bearer handler capable of processing tokens generated by external OIDC providers (ie. access tokens issued by the OIDC server rather than the Horde server)
	/// </summary>
	public class ExternalJwtAuthHandler
	{
		/// <summary>
		/// Default name of the authentication scheme
		/// </summary>
		public const string AuthenticationScheme = "ExternalJwt";

		// Caches the lookup of "sub" claim in access token to actual user used internally by Horde.
		// Required to avoid database lookup on every request when authenticating.
		private readonly ConcurrentDictionary<string, IUser> _subToUser = new();

		readonly ServerSettings _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public ExternalJwtAuthHandler(ServerSettings settings)
		{
			_settings = settings;
		}

		/// <summary>
		/// Callback when JWT bearer request has been received
		/// Looks up and extracts the token from Authorization header.
		/// This cannot be done by the default JwtBearer handler as it can only handle "Bearer" prefix.
		/// </summary>
		/// <param name="context">Message context</param>
		private Task OnMessageReceivedAsync(MessageReceivedContext context)
		{
			string? authorization = context.Request.Headers.Authorization;
			if (!String.IsNullOrEmpty(authorization))
			{
				string prefix = AuthenticationScheme + " ";
				if (authorization.StartsWith(prefix, StringComparison.OrdinalIgnoreCase))
				{
					context.Token = authorization.Substring(prefix.Length).Trim();
				}
			}

			return Task.CompletedTask;
		}
		
		private Task OnAuthenticationFailedAsync(AuthenticationFailedContext context)
		{
			if (_settings.OidcDebugMode)
			{
				IServiceProvider serviceProvider = context.HttpContext.RequestServices;
				ILogger<ExternalJwtAuthHandler> logger = serviceProvider.GetRequiredService<ILogger<ExternalJwtAuthHandler>>();
				logger.LogError(context.Exception, "JWT bearer auth failed. Auth header: {AuthHeader}", context.Request.Headers.Authorization.ToString());	
			}
			return Task.CompletedTask;
		}
		
		/// <summary>
		/// Callback when JWT bearer token is being validated
		/// If user is not cached, it will look up additional info via /userinfo and cache it
		/// </summary>
		/// <param name="context">Token validation context</param>
		private async Task OnTokenValidatedAsync(TokenValidatedContext context)
		{
			IServiceProvider serviceProvider = context.HttpContext.RequestServices;

			ILogger<ExternalJwtAuthHandler> logger = serviceProvider.GetRequiredService<ILogger<ExternalJwtAuthHandler>>();
			logger.LogInformation("Running HordeJwtBearerHandler.OnTokenValidatedAsync()");

			if (context.Principal == null)
			{
				ReportError(logger, context, "Principal not set in context");
				return;
			}

			JwtSecurityToken? accessToken = context.SecurityToken as JwtSecurityToken;
			if (accessToken == null)
			{
				ReportError(logger, context, "Unable to read access token");
				return;
			}

			ClaimsIdentity? identity = (ClaimsIdentity?)context.Principal?.Identity;
			if (identity == null)
			{
				ReportError(logger, context, "No identity specified");
				return;
			}

			if (_settings.OidcAuthority == null)
			{
				ReportError(logger, context, "OidcAuthority not set in settings");
				return;
			}

			logger.LogInformation("Using HordeJwtBearerHandler");
			if (!_subToUser.TryGetValue(accessToken.Subject, out IUser? user))
			{
				string login = accessToken.Subject;
				string? name = accessToken.Claims.FirstOrDefault(c => c.Type == "name")?.Value;
				string? email = accessToken.Claims.FirstOrDefault(c => c.Type == "email")?.Value;

				IUserCollection userCollection = serviceProvider.GetRequiredService<IUserCollection>();
				user = await userCollection.FindOrAddUserByLoginAsync(login, name, email);

				await userCollection.UpdateClaimsAsync(user.Id, accessToken.Claims.Select(x => new UserClaim(x.Type, x.Value)), context.HttpContext.RequestAborted);
				_subToUser[accessToken.Subject] = user;
			}
			
			identity.AddClaim(new Claim(HordeClaimTypes.Version, HordeClaimTypes.CurrentVersion));
			identity.AddClaim(new Claim(HordeClaimTypes.UserId, user.Id.ToString()));

			foreach (Claim claim in accessToken.Claims.Where(x => x.Type == "groups"))
			{
				identity.AddClaim(new Claim(ClaimTypes.Role, claim.Value));
			}
			OidcAuthHandler.MapAdminClaim(_settings, identity);
		}

		private static void ReportError(ILogger logger, ResultContext<JwtBearerOptions> context, string message)
		{
#pragma warning disable CA2254 // Template should be a static expression
			logger.LogError(message);
#pragma warning restore CA2254
			context.Fail(message);
		}

		/// <summary>
		/// Registers this instance as a JWT handler
		/// </summary>
		/// <param name="authBuilder">Authentication builder</param>
		public void AddHordeJwtBearerConfiguration(AuthenticationBuilder authBuilder)
		{
			authBuilder.AddJwtBearer(AuthenticationScheme, options =>
			{
				options.Authority = _settings.OidcAuthority;
				options.Audience = _settings.OidcAudience;
				options.Events = new JwtBearerEvents() { OnAuthenticationFailed = OnAuthenticationFailedAsync, OnMessageReceived = OnMessageReceivedAsync, OnTokenValidated = OnTokenValidatedAsync };
				options.RequireHttpsMetadata = !_settings.OidcDebugMode;

				options.TokenValidationParameters.ValidAudience = _settings.OidcAudience;
				options.TokenValidationParameters.RequireExpirationTime = true;
				options.TokenValidationParameters.RequireSignedTokens = true;
				options.TokenValidationParameters.ValidateIssuer = true;
				options.TokenValidationParameters.ValidIssuer = _settings.OidcAuthority;
				options.TokenValidationParameters.ValidateAudience = true;
				options.TokenValidationParameters.ValidateIssuerSigningKey = true;
				options.TokenValidationParameters.ValidateLifetime = true;

				options.TokenHandlers.Clear();
				options.TokenHandlers.Add(new StrictTokenHandler());
			});
		}

		private class StrictTokenHandler : JwtSecurityTokenHandler
		{
			public override ClaimsPrincipal ValidateToken(string token, TokenValidationParameters validationParameters, out SecurityToken validatedToken)
			{
				ClaimsPrincipal claimsPrincipal = base.ValidateToken(token, validationParameters, out validatedToken);
				JwtSecurityToken jwtSecurityToken = ReadJwtToken(token);
				if (jwtSecurityToken.Header?.Alg is not SecurityAlgorithms.RsaSha256)
				{
					throw new SecurityTokenValidationException("The JWT algorithm is not RS256.");
				}

				return claimsPrincipal;
			}
		}
	}
}
