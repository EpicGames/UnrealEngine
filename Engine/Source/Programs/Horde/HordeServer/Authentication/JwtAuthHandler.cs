// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IdentityModel.Tokens.Jwt;
using System.Text.Encodings.Web;
using System.Threading.Tasks;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.JwtBearer;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Authentication
{
	/// <summary>
	/// JWT handler for server-issued bearer tokens. These tokens are signed using a randomly generated key per DB instance.
	/// </summary>
	class JwtAuthHandler : JwtBearerHandler, IAsyncDisposable
	{
		/// <summary>
		/// Default name of the authentication scheme
		/// </summary>
		public const string AuthenticationScheme = "ServerJwt";

		readonly AsyncCachedValue<IGlobals> _globals;

		public JwtAuthHandler(ILoggerFactory logger, UrlEncoder encoder, IOptionsMonitor<ServerSettings> settings, GlobalsService globalsService, IOptionsMonitorCache<JwtBearerOptions> optionsCache)
			: base(GetOptionsMonitor(settings.CurrentValue, optionsCache), logger, encoder)
		{
			_globals = new AsyncCachedValue<IGlobals>(async ctx => await globalsService.GetAsync(ctx), TimeSpan.FromSeconds(30.0));
		}

		public async ValueTask DisposeAsync()
		{
			await _globals.DisposeAsync();
		}

		private static IOptionsMonitor<JwtBearerOptions> GetOptionsMonitor(ServerSettings settings, IOptionsMonitorCache<JwtBearerOptions> optionsCache)
		{
			ConfigureNamedOptions<JwtBearerOptions> namedOptions = new (AuthenticationScheme, options =>
			{
				options.RequireHttpsMetadata = !settings.OidcDebugMode;
			});
			OptionsFactory<JwtBearerOptions> optionsFactory = new (new[] { namedOptions }, Array.Empty<IPostConfigureOptions<JwtBearerOptions>>());
			return new OptionsMonitor<JwtBearerOptions>(optionsFactory, Array.Empty<IOptionsChangeTokenSource<JwtBearerOptions>>(), optionsCache);
		}

		protected override async Task<AuthenticateResult> HandleAuthenticateAsync()
		{
			// Get the current state
			IGlobals globals = await _globals.GetAsync();

			Options.TokenValidationParameters.ValidateAudience = false; // Don't need to validate audience because we only issue tokens for our own consumption.

			Options.TokenValidationParameters.RequireExpirationTime = false;
			Options.TokenValidationParameters.ValidateLifetime = true;

			Options.TokenValidationParameters.ValidIssuer = globals.JwtIssuer;
			Options.TokenValidationParameters.ValidateIssuer = true;

			Options.TokenValidationParameters.ValidateIssuerSigningKey = true;
			Options.TokenValidationParameters.IssuerSigningKeys = new[] { globals.JwtSigningKey, globals.RsaSigningKey };

			// Silent fail if this JWT is not issued by the server
			string? token;
			if (!JwtUtils.TryGetBearerToken(Request, "Bearer ", out token))
			{
				return AuthenticateResult.NoResult();
			}

			// Validate that it's from the correct issuer, and silent fail if not. Allows multiple handlers for bearer tokens.
			JwtSecurityToken? jwtToken;
			if (!JwtUtils.TryParseJwt(token, out jwtToken) || !String.Equals(jwtToken.Issuer, globals.JwtIssuer, StringComparison.Ordinal))
			{
				return AuthenticateResult.NoResult();
			}

			// Pass it to the base class
			AuthenticateResult result = await base.HandleAuthenticateAsync();
			return result;
		}
	}
}
