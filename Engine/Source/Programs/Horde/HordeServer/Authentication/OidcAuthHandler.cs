// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Security.Claims;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Threading.Tasks;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.OAuth.Claims;
using Microsoft.AspNetCore.Authentication.OpenIdConnect;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.DependencyInjection.Extensions;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.IdentityModel.Protocols.OpenIdConnect;

namespace HordeServer.Authentication
{
	class OidcAuthHandler : OpenIdConnectHandler
	{
		readonly IUserCollection _userCollection;

		public OidcAuthHandler(IOptionsMonitor<OpenIdConnectOptions> options, ILoggerFactory logger, HtmlEncoder htmlEncoder, UrlEncoder encoder, IUserCollection userCollection)
			: base(options, logger, htmlEncoder, encoder)
		{
			_userCollection = userCollection;
		}

		/// <summary>
		/// Try to map a field found in user info as a claim
		///
		/// If the claim is already set, it is skipped.
		/// </summary>
		/// <param name="userInfo">User info from the OIDC /userinfo endpoint </param>
		/// <param name="identity">Identity which claims to modify</param>
		/// <param name="claimName">Name of claim</param>
		/// <param name="userInfoFields">List of field names to try when mapping</param>
		/// <exception cref="Exception">Thrown if no field name matches</exception>
		private static void MapUserInfoFieldToClaim(JsonElement userInfo, ClaimsIdentity identity, string claimName, string[] userInfoFields)
		{
			if (identity.FindFirst(claimName) != null)
			{
				return;
			}

			foreach (string userDataField in userInfoFields)
			{
				if (userInfo.TryGetProperty(userDataField, out JsonElement fieldElement))
				{
					identity.AddClaim(new Claim(claimName, fieldElement.ToString()));
					return;
				}
			}

			string message = $"Unable to map a field from user info to claim '{claimName}' using list [{String.Join(", ", userInfoFields)}].";
			message += " UserInfo: " + userInfo;
			throw new Exception(message);
		}

		public static void AddUserInfoClaims(ServerSettings settings, JsonElement userInfo, ClaimsIdentity identity)
		{
			MapUserInfoFieldToClaim(userInfo, identity, ClaimTypes.Name, settings.OidcClaimNameMapping);
			MapUserInfoFieldToClaim(userInfo, identity, ClaimTypes.Email, settings.OidcClaimEmailMapping);
			MapUserInfoFieldToClaim(userInfo, identity, HordeClaimTypes.User, settings.OidcClaimHordeUserMapping);
			MapUserInfoFieldToClaim(userInfo, identity, HordeClaimTypes.PerforceUser, settings.OidcClaimHordePerforceUserMapping);

			if (userInfo.TryGetProperty("groups", out JsonElement groupsElement) && groupsElement.ValueKind == JsonValueKind.Array)
			{
				for (int idx = 0; idx < groupsElement.GetArrayLength(); idx++)
				{
					identity.AddClaim(new Claim(ClaimTypes.Role, groupsElement[idx].ToString()!));
				}
			}

			MapAdminClaim(settings, identity);
		}

		public static void MapAdminClaim(ServerSettings settings, ClaimsIdentity identity)
		{
			if (!String.IsNullOrEmpty(settings.AdminClaimType) && !String.IsNullOrEmpty(settings.AdminClaimValue))
			{
				if (identity.HasClaim(settings.AdminClaimType, settings.AdminClaimValue))
				{
					identity.AddClaim(HordeClaims.AdminClaim.ToClaim());
				}
			}
		}

		protected override async Task<HandleRequestResult> HandleRemoteAuthenticateAsync()
		{
			// Authenticate with the OIDC provider
			HandleRequestResult result = await base.HandleRemoteAuthenticateAsync();
			if (!result.Succeeded)
			{
				return result;
			}

			ClaimsIdentity? identity = (ClaimsIdentity?)result.Principal?.Identity;
			if (identity == null)
			{
				return HandleRequestResult.Fail("No identity specified");
			}

			string login = identity.FindFirst(ClaimTypes.Name)!.Value;
			string? name = identity.FindFirst("name")?.Value;
			string? email = identity.FindFirst(ClaimTypes.Email)?.Value;

			IUser user = await _userCollection.FindOrAddUserByLoginAsync(login, name, email);
			identity.AddClaim(new Claim(HordeClaimTypes.Version, HordeClaimTypes.CurrentVersion));
			identity.AddClaim(new Claim(HordeClaimTypes.UserId, user.Id.ToString()));

			await _userCollection.UpdateClaimsAsync(user.Id, identity.Claims.Select(x => new UserClaim(x.Type, x.Value)), Request.HttpContext.RequestAborted);

			return result;
		}
	}

	static class OpenIdConnectHandlerExtensions
	{
		class MapRolesClaimAction : ClaimAction
		{
			private readonly ServerSettings _settings;
			public MapRolesClaimAction(ServerSettings settings) : base(ClaimTypes.Role, ClaimTypes.Role) { _settings = settings; }
			public override void Run(JsonElement userData, ClaimsIdentity identity, string issuer) { OidcAuthHandler.AddUserInfoClaims(_settings, userData, identity); }
		}

		static void ApplyDefaultOptions(ServerSettings settings, OpenIdConnectOptions options, Action<OpenIdConnectOptions> handler)
		{
			options.ResponseType = OpenIdConnectResponseType.Code;
			options.GetClaimsFromUserInfoEndpoint = true;
			options.SaveTokens = false;
			options.TokenValidationParameters.NameClaimType = "name";
			options.ClaimActions.Add(new MapRolesClaimAction(settings));
			options.RequireHttpsMetadata = !settings.OidcDebugMode;

			handler(options);
		}

		public static void AddHordeOpenId(this AuthenticationBuilder builder, ServerSettings settings, string authenticationScheme, string displayName, Action<OpenIdConnectOptions> handler)
		{
			builder.Services.TryAddEnumerable(ServiceDescriptor.Singleton<IPostConfigureOptions<OpenIdConnectOptions>, OpenIdConnectPostConfigureOptions>());
			builder.AddRemoteScheme<OpenIdConnectOptions, OidcAuthHandler>(authenticationScheme, displayName, options => ApplyDefaultOptions(settings, options, handler));
		}
	}
}
