// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Security.Claims;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Threading.Tasks;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.OAuth.Claims;
using Microsoft.AspNetCore.Authentication.OpenIdConnect;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.DependencyInjection.Extensions;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.IdentityModel.Protocols.OpenIdConnect;

namespace HordeServer.Authentication
{
	class OktaAuthHandler : OpenIdConnectHandler
	{
		public const string AuthenticationScheme = "Okta" + OpenIdConnectDefaults.AuthenticationScheme;

		readonly IUserCollection _userCollection;

		public OktaAuthHandler(IOptionsMonitor<OpenIdConnectOptions> options, ILoggerFactory logger, HtmlEncoder htmlEncoder, UrlEncoder encoder, IUserCollection userCollection)
			: base(options, logger, htmlEncoder, encoder)
		{
			_userCollection = userCollection;
		}

		public static void AddUserInfoClaims(JsonElement userInfo, ServerSettings settings, ClaimsIdentity identity)
		{
			JsonElement nameElement;
			if (userInfo.TryGetProperty("name", out nameElement))
			{
				if (identity.FindFirst(ClaimTypes.Name) == null)
				{
					identity.AddClaim(new Claim(ClaimTypes.Name, nameElement.ToString()!));
				}
			}

			JsonElement userElement;
			if (userInfo.TryGetProperty("preferred_username", out userElement))
			{
				if (identity.FindFirst(HordeClaimTypes.User) == null)
				{
					identity.AddClaim(new Claim(HordeClaimTypes.User, userElement.ToString()!));
				}
				if (identity.FindFirst(HordeClaimTypes.PerforceUser) == null)
				{
					identity.AddClaim(new Claim(HordeClaimTypes.PerforceUser, userElement.ToString()!));
				}
			}

			JsonElement groupsElement;
			if (userInfo.TryGetProperty("groups", out groupsElement) && groupsElement.ValueKind == JsonValueKind.Array)
			{
				for (int idx = 0; idx < groupsElement.GetArrayLength(); idx++)
				{
					identity.AddClaim(new Claim(ClaimTypes.Role, groupsElement[idx].ToString()!));
				}
			}

			JsonElement emailElement;
			if (userInfo.TryGetProperty("email", out emailElement) && identity.FindFirst(ClaimTypes.Email) == null)
			{
				identity.AddClaim(new Claim(ClaimTypes.Email, emailElement.ToString()!));
			}

			if (!String.IsNullOrEmpty(settings.AdminClaimType) && !String.IsNullOrEmpty(settings.AdminClaimValue))
			{
				if (identity.HasClaim(settings.AdminClaimType, settings.AdminClaimValue))
				{
					identity.AddClaim(HordeClaims.AdminClaim.ToClaim());
				}
			}
		}

		protected override async Task<HandleRequestResult> HandleRemoteAuthenticateAsync()
		{
			// Authenticate with the OIDC provider
			HandleRequestResult result = await base.HandleRemoteAuthenticateAsync();
			if (!result.Succeeded)
			{
				return result;
			}

			ClaimsIdentity? identity = (ClaimsIdentity?)result.Principal?.Identity;
			if (identity == null)
			{
				return HandleRequestResult.Fail("No identity specified");
			}

			string login = identity.FindFirst(HordeClaimTypes.User)!.Value;
			string? name = identity.FindFirst(ClaimTypes.Name)?.Value;
			string? email = identity.FindFirst(ClaimTypes.Email)?.Value;

			IUser user = await _userCollection.FindOrAddUserByLoginAsync(login, name, email);
			identity.AddClaim(new Claim(HordeClaimTypes.Version, HordeClaimTypes.CurrentVersion));
			identity.AddClaim(new Claim(HordeClaimTypes.UserId, user.Id.ToString()));

			await _userCollection.UpdateClaimsAsync(user.Id, identity.Claims.Select(x => new UserClaim(x.Type, x.Value)), Request.HttpContext.RequestAborted);

			return result;
		}
	}

	static class OktaExtensions
	{
		class MapRolesClaimAction : ClaimAction
		{
			readonly ServerSettings _settings;

			public MapRolesClaimAction(ServerSettings settings)
				: base(ClaimTypes.Role, ClaimTypes.Role)
			{
				_settings = settings;
			}

			public override void Run(JsonElement userData, ClaimsIdentity identity, string issuer)
			{
				OktaAuthHandler.AddUserInfoClaims(userData, _settings, identity);
			}
		}

		static void ApplyDefaultOktaOptions(OpenIdConnectOptions options, ServerSettings settings, Action<OpenIdConnectOptions> handler)
		{
			options.Scope.Add("profile");
			options.Scope.Add("groups");
			options.Scope.Add("email");
			options.ResponseType = OpenIdConnectResponseType.Code;
			options.GetClaimsFromUserInfoEndpoint = true;
			options.SaveTokens = false;
			options.TokenValidationParameters.NameClaimType = "name";
			options.ClaimActions.Add(new MapRolesClaimAction(settings));

			handler(options);
		}

		public static void AddOkta(this AuthenticationBuilder builder, ServerSettings settings, string authenticationScheme, string displayName, Action<OpenIdConnectOptions> handler)
		{
			builder.Services.TryAddEnumerable(ServiceDescriptor.Singleton<IPostConfigureOptions<OpenIdConnectOptions>, OpenIdConnectPostConfigureOptions>());
			builder.AddRemoteScheme<OpenIdConnectOptions, OktaAuthHandler>(authenticationScheme, displayName, options => ApplyDefaultOktaOptions(options, settings, handler));
		}
	}
}
