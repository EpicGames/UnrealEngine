// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Security.Claims;
using System.Text.Encodings.Web;
using System.Threading.Tasks;
using HordeServer.ServiceAccounts;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Microsoft.Net.Http.Headers;

namespace HordeServer.Authentication
{
	class ServiceAccountAuthOptions : AuthenticationSchemeOptions
	{
	}

	class ServiceAccountAuthHandler : AuthenticationHandler<ServiceAccountAuthOptions>
	{
		public const string AuthenticationScheme = "ServiceAccount";
		public const string Prefix = "ServiceAccount";

		private readonly IServiceAccountCollection _serviceAccountCollection;

		public ServiceAccountAuthHandler(IOptionsMonitor<ServiceAccountAuthOptions> options,
			ILoggerFactory logger, UrlEncoder encoder, IServiceAccountCollection serviceAccountCollection)
			: base(options, logger, encoder)
		{
			_serviceAccountCollection = serviceAccountCollection;
		}

		protected override async Task<AuthenticateResult> HandleAuthenticateAsync()
		{
			if (!Context.Request.Headers.TryGetValue(HeaderNames.Authorization, out Microsoft.Extensions.Primitives.StringValues headerValue))
			{
				return AuthenticateResult.NoResult();
			}

			if (headerValue.Count < 1)
			{
				return AuthenticateResult.NoResult();
			}

			string? header = headerValue[0];
			if (header == null || !header.StartsWith(Prefix, StringComparison.Ordinal))
			{
				return AuthenticateResult.NoResult();
			}

			string token = header.Replace(Prefix, "", StringComparison.Ordinal).Trim();
			IServiceAccount? serviceAccount = await _serviceAccountCollection.FindBySecretTokenAsync(token);

			if (serviceAccount == null)
			{
				return AuthenticateResult.Fail($"Service account for token {token} not found");
			}

			List<Claim> claims = new List<Claim>(10);
			claims.Add(new Claim(ClaimTypes.Name, serviceAccount.Name));
			claims.Add(new Claim(HordeClaimTypes.User, serviceAccount.Name));
			claims.AddRange(serviceAccount.Claims.Select(claimPair => new Claim(claimPair.Type, claimPair.Value)));

			ClaimsIdentity identity = new ClaimsIdentity(claims, Scheme.Name);
			ClaimsPrincipal principal = new ClaimsPrincipal(identity);
			AuthenticationTicket ticket = new AuthenticationTicket(principal, Scheme.Name);

			return AuthenticateResult.Success(ticket);
		}
	}

	static class ServiceAccountAuthExtensions
	{
		public static AuthenticationBuilder AddServiceAccounts(this AuthenticationBuilder builder, Action<ServiceAccountAuthOptions> config)
		{
			return builder.AddScheme<ServiceAccountAuthOptions, ServiceAccountAuthHandler>(ServiceAccountAuthHandler.AuthenticationScheme, config);
		}
	}
}
