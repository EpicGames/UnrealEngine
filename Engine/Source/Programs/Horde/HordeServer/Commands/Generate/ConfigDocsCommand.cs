// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer.Commands.Generate
{
	[Command("generate", "configdocs", "Writes Markdown docs for server settings")]
	class DocsCommand : Command
	{
		[CommandLine]
		[Description("Output directory for generated markdown files")]
		public DirectoryReference OutputDir { get; set; } = DirectoryReference.Combine(ServerApp.AppDir, "Docs");

		[CommandLine]
		[Description("Path to the agent executable, for gathering agent settings.")]
		public FileReference? Agent { get; set; }

		record class PageInfo(string Title, string LinkRail, string FileName, string? Introduction = null);

		readonly IPluginCollection _pluginCollection;

		public DocsCommand(IPluginCollection pluginCollection)
			=> _pluginCollection = pluginCollection;

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			DirectoryReference.CreateDirectory(OutputDir);

			JsonSchemaCache schemaCache = new JsonSchemaCache(_pluginCollection);

			HashSet<Assembly> assemblies = new HashSet<Assembly>();
			assemblies.Add(Assembly.GetExecutingAssembly());
			assemblies.UnionWith(_pluginCollection.LoadedPlugins.Select(x => x.Assembly));

			if (Agent == null)
			{
				logger.LogWarning("Not generating documentation for agent settings. Specify path to agent assembly with -Agent=...");
			}
			else
			{
				assemblies.Add(Assembly.LoadFile(Agent.FullName));
			}

			Dictionary<JsonSchemaType, PageInfo> typeToPageInfo = new Dictionary<JsonSchemaType, PageInfo>();
			foreach (Type type in assemblies.SelectMany(x => x.GetExportedTypes()))
			{
				ConfigDocAttribute? attribute = type.GetCustomAttribute<ConfigDocAttribute>();
				if (attribute != null)
				{
					JsonSchema schema = schemaCache.CreateSchema(type);
					typeToPageInfo.Add(schema.RootType, new PageInfo(attribute.Title, attribute.LinkRail, attribute.FileName, attribute.Introduction));
				}
			}

			Dictionary<string, string> typeNameToLink = typeToPageInfo.ToDictionary(x => x.Key.Name!, x => Path.GetFileName(x.Value.FileName), StringComparer.Ordinal);
			foreach ((JsonSchemaType type, PageInfo pageInfo) in typeToPageInfo)
			{
				await WriteDocAsync(type, pageInfo.Title, pageInfo.FileName, pageInfo.LinkRail, pageInfo.Introduction, typeNameToLink, logger);
			}

			await WriteAclDocAsync(assemblies, logger);
			return 0;
		}

		class ObjectQueue
		{
			readonly Stack<JsonSchemaType> _stack = new Stack<JsonSchemaType>();
			readonly HashSet<string> _visited = new HashSet<string>(StringComparer.Ordinal);

			public void Add(JsonSchemaType type)
			{
				if (type.Name != null && _visited.Add(type.Name))
				{
					_stack.Push(type);
				}
			}

			public void Ignore(JsonSchemaType type)
			{
				if (type.Name != null)
				{
					_visited.Add(type.Name);
				}
			}

			public bool TryPop([NotNullWhen(true)] out JsonSchemaType? obj) => _stack.TryPop(out obj);
		}

		async Task WriteDocAsync(JsonSchemaType rootType, string title, string fileName, string linkRail, string? introduction, Dictionary<string, string> typeNameToLink, ILogger logger)
		{
			FileReference file = FileReference.Combine(OutputDir, fileName);
			DirectoryReference.CreateDirectory(file.Directory);

			using (FileStream stream = FileReference.Open(file, FileMode.Create, FileAccess.Write))
			{
				using (StreamWriter writer = new StreamWriter(stream))
				{
					await writer.WriteLineAsync($"{linkRail} > {title}");
					await writer.WriteLineAsync();
					await writer.WriteLineAsync($"# {title}");

					if (introduction != null)
					{
						await writer.WriteLineAsync();
						await writer.WriteLineAsync(introduction);
					}

					HashSet<string> visitedTypeNames = new HashSet<string>(typeNameToLink.Keys, StringComparer.Ordinal);
					visitedTypeNames.Remove(rootType.Name!);

					List<JsonSchemaType> types = new List<JsonSchemaType>();
					FindCustomTypes(rootType, types, visitedTypeNames);

					foreach (JsonSchemaType schemaType in types)
					{
						if (schemaType != rootType)
						{
							await writer.WriteLineAsync();
							await writer.WriteLineAsync($"## {GetHeadingName(schemaType)}");
						}

						await writer.WriteLineAsync();

						if (schemaType.Description != null && (introduction == null || schemaType != rootType))
						{
							string description = schemaType.Description.ReplaceLineEndings();
							await writer.WriteLineAsync(description);
							await writer.WriteLineAsync();
						}

						if (schemaType is JsonSchemaObject schemaObj)
						{
							if (schemaObj.Properties.Count > 0)
							{
								await writer.WriteLineAsync("Name | Description");
								await writer.WriteLineAsync("---- | -----------");

								foreach (JsonSchemaProperty property in schemaObj.Properties)
								{
									string name = property.CamelCaseName;
									string type = GetMarkdownType(property.Type, typeNameToLink);
									string description = GetMarkdownDescription(property.Description);
									await writer.WriteLineAsync($"`{name}` | {type}<br>{description}");
								}
							}
						}
						else if (schemaType is JsonSchemaEnum schemaEnum)
						{
							await writer.WriteLineAsync("Name | Description");
							await writer.WriteLineAsync("---- | -----------");

							for (int idx = 0; idx < schemaEnum.Values.Count; idx++)
							{
								string name = schemaEnum.Values[idx];
								string description = GetMarkdownDescription(schemaEnum.Descriptions[idx]);
								await writer.WriteLineAsync($"`{name}` | {description}");
							}
						}
						else
						{
							throw new NotImplementedException();
						}
					}
				}
			}
			logger.LogInformation("Written {File}", file);
		}

		static void FindCustomTypes(JsonSchemaType type, List<JsonSchemaType> types, HashSet<string> visitedTypeNames)
		{
			switch (type)
			{
				case JsonSchemaOneOf oneOf:
					foreach (JsonSchemaType oneOfType in oneOf.Types)
					{
						FindCustomTypes(oneOfType, types, visitedTypeNames);
					}
					break;
				case JsonSchemaArray array:
					FindCustomTypes(array.ItemType, types, visitedTypeNames);
					break;
				case JsonSchemaEnum _:
					if (type.Name != null && visitedTypeNames.Add(type.Name))
					{
						types.Add(type);
					}
					break;
				case JsonSchemaObject obj:
					if (type.Name != null)
					{
						if (visitedTypeNames.Add(type.Name))
						{
							types.Add(type);
						}
						else
						{
							break;
						}
					}
					if (obj.AdditionalProperties != null)
					{
						FindCustomTypes(obj.AdditionalProperties, types, visitedTypeNames);
					}
					foreach (JsonSchemaProperty property in obj.Properties)
					{
						FindCustomTypes(property.Type, types, visitedTypeNames);
					}
					break;
			}
		}

		static string GetMarkdownType(JsonSchemaType type, Dictionary<string, string> typeNameToLink)
		{
			switch (type)
			{
				case JsonSchemaBoolean _:
					return "`boolean`";
				case JsonSchemaInteger _:
					return "`integer`";
				case JsonSchemaNumber _:
					return "`number`";
				case JsonSchemaString _:
					return "`string`";
				case JsonSchemaOneOf oneOf:
					return String.Join("/", oneOf.Types.Select(x => GetMarkdownType(x, typeNameToLink)));
				case JsonSchemaArray array:
					string elementType = GetMarkdownType(array.ItemType, typeNameToLink);
					if (elementType.EndsWith("`", StringComparison.Ordinal))
					{
						return elementType.Insert(elementType.Length - 1, "[]");
					}
					else
					{
						return elementType + "`[]`";
					}

				case JsonSchemaEnum en:
					if (type.Name == null)
					{
						return String.Join("<br>", en.Values);
					}
					else
					{
						return $"[{type.Name}]({GetLink(type, typeNameToLink)})";
					}
				case JsonSchemaObject obj:
					if (obj.AdditionalProperties != null)
					{
						return "`string` `->` " + GetMarkdownType(obj.AdditionalProperties, typeNameToLink);
					}
					else if (type.Name == null)
					{
						return "`object`";
					}
					else
					{
						return $"[{type.Name}]({GetLink(type, typeNameToLink)})";
					}
				default:
					return type.GetType().Name;
			}
		}

		static string GetMarkdownDescription(string? description)
		{
			return (description ?? String.Empty).Replace("|", @"\|", StringComparison.Ordinal).Replace("\n", "<br>", StringComparison.Ordinal);
		}

		static string GetHeadingName(JsonSchemaType type)
		{
			if (type.Name == null)
			{
				throw new NotImplementedException("Unknown type");
			}

			switch (type)
			{
				case JsonSchemaEnum _:
					return $"{type.Name} (Enum)";
				default:
					return type.Name;
			}
		}

		static string GetLink(JsonSchemaType type, Dictionary<string, string> typeNameToLink)
		{
			string? link;
			if (type.Name != null && typeNameToLink.TryGetValue(type.Name, out link))
			{
				return link;
			}
			else
			{
				return GetAnchorName(type);
			}
		}

		[SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase")]
		static string GetAnchorName(JsonSchemaType type)
		{
			string anchor = GetHeadingName(type).ToLowerInvariant();
			anchor = Regex.Replace(anchor, @"[^a-z0-9]+", "-");
			return "#" + anchor.Trim('-');
		}

		async Task WriteAclDocAsync(IEnumerable<Assembly> assemblies, ILogger logger)
		{
			FileReference file = FileReference.Combine(OutputDir, "Config/Schema/AclActions.md");
			DirectoryReference.CreateDirectory(file.Directory);

			using (FileStream stream = FileReference.Open(file, FileMode.Create, FileAccess.Write))
			{
				using (StreamWriter writer = new StreamWriter(stream))
				{
					await writer.WriteLineAsync($"[Horde](../../../README.md) > [Configuration](../../Config.md) > ACL Actions");
					await writer.WriteLineAsync();
					await writer.WriteLineAsync($"# ACL Actions");

					Dictionary<string, List<PropertyInfo>> categoryToProperties = new Dictionary<string, List<PropertyInfo>>();

					List<PropertyInfo> newProperties = new List<PropertyInfo>();
					foreach (Assembly assembly in assemblies)
					{
						foreach (Type type in assembly.GetTypes())
						{
							if (type.IsClass)
							{
								foreach (PropertyInfo propertyInfo in type.GetProperties(BindingFlags.Static | BindingFlags.Public))
								{
									if (propertyInfo.PropertyType == typeof(AclAction))
									{
										newProperties.Add(propertyInfo);
									}
								}
								if (newProperties.Count > 0)
								{
									string category = type.Namespace ?? "Default";

									int topNamespaceIdx = category.LastIndexOf('.');
									if (topNamespaceIdx != -1)
									{
										category = category.Substring(topNamespaceIdx + 1);
									}

									List<PropertyInfo>? existingProperties;
									if (categoryToProperties.TryGetValue(category, out existingProperties))
									{
										existingProperties.AddRange(newProperties);
										newProperties.Clear();
									}
									else
									{
										categoryToProperties.Add(category, newProperties);
										newProperties = new List<PropertyInfo>();
									}
								}
							}
						}

						XmlDocument documentation = new XmlDocument();

						FileReference inputDocumentationFile = new FileReference(assembly.Location).ChangeExtension(".xml");
						if (FileReference.Exists(inputDocumentationFile))
						{
							documentation.Load(inputDocumentationFile.FullName);
						}

						foreach ((string category, List<PropertyInfo> properties) in categoryToProperties.OrderBy(x => x.Key))
						{
							await writer.WriteLineAsync();
							await writer.WriteLineAsync($"## {category}");
							await writer.WriteLineAsync();
							await writer.WriteLineAsync("| Name | Description |");
							await writer.WriteLineAsync("| ---- | ----------- |");
							foreach (PropertyInfo property in properties)
							{
								AclAction action = (AclAction)property.GetValue(null)!;
								string description = String.Empty;

								string selector = $"//member[@name='P:{property.DeclaringType!.FullName}.{property.Name}']/summary";
								XmlNode? node = documentation.SelectSingleNode(selector);
								if (node != null)
								{
									description = node.InnerText.Trim().Replace("\r\n", "\n", StringComparison.Ordinal);
								}

								await writer.WriteLineAsync($"| `{action.Name}` | {description} |");
							}
						}
					}
				}
			}

			logger.LogInformation("Written {File}", file);
		}
	}
}
