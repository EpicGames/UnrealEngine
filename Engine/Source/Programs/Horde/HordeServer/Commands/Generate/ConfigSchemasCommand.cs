// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.ComponentModel;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer.Commands.Generate
{
	[Command("generate", "configschemas", "Writes JSON schemas for server settings")]
	class SchemasCommand : Command
	{
		[CommandLine("-OutputDir=")]
		[Description("Output directory to write schemas to. Defaults to the 'Schemas' subfolder of the application directory.")]
		DirectoryReference? _outputDir = null!;

		readonly IPluginCollection _pluginCollection;

		public SchemasCommand(IPluginCollection pluginCollection)
			=> _pluginCollection = pluginCollection;

		public override Task<int> ExecuteAsync(ILogger logger)
		{
			_outputDir ??= DirectoryReference.Combine(ServerApp.AppDir, "Schemas");

			JsonSchemaCache schemaCache = new JsonSchemaCache(_pluginCollection);

			DirectoryReference.CreateDirectory(_outputDir);
			foreach (Type schemaType in SchemaController.ConfigSchemas)
			{
				FileReference outputFile = FileReference.Combine(_outputDir, $"{schemaType.Name}.json");
				schemaCache.CreateSchema(schemaType).Write(outputFile);
			}

			return Task.FromResult(0);
		}
	}
}
