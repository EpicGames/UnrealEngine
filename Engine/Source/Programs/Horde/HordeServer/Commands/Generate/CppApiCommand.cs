// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;

namespace HordeServer.Commands.Generate
{
	[Command("generate", "cppapi", "Outputs C++ source files describing the Horde API")]
	class CppApiCommand : Command
	{
		record class ApiPropertyInfo(string Name, Type Type, string Description);
		record class ApiTypeInfo(string Name, Type Type, string Description, List<ApiPropertyInfo> Properties);
		record class ApiGroupInfo(string Name, Dictionary<Type, ApiTypeInfo> Types);

		static readonly string[] s_boilerplateLines = new[]
		{
			"// ---------------------------------------------------------------------------------------------------",
			"// This file is autogenerated by the Horde Server 'generate cppapi' command. Do not modify it by hand.",
			"// ---------------------------------------------------------------------------------------------------",
			""
		};

		[CommandLine("-OutputDir=")]
		[Description("Specifies the output directory for generated files.")]
		public DirectoryReference? OutputDir { get; set; }

		readonly IPluginCollection _pluginCollection;

		public CppApiCommand(IPluginCollection pluginCollection)
		{
			_pluginCollection = pluginCollection;
		}

		public override Task<int> ExecuteAsync(ILogger logger)
		{
			// Try to find the output directory assuming we're being run from the default output directory in the engine source tree
			if (OutputDir == null)
			{
				FileReference assemblyFile = new FileReference(Assembly.GetExecutingAssembly().Location);

				DirectoryReference defaultHordeDir = DirectoryReference.Combine(assemblyFile.Directory, "../../../../../../Developer/Horde");
				if (!FileReference.Exists(FileReference.Combine(defaultHordeDir, "Horde.build.cs")))
				{
					throw new ArgumentException("No output directory specified. Use -OutputDir=...");
				}

				OutputDir = defaultHordeDir;
			}

			// Find all the assemblies containing potential controller types
			HashSet<Assembly> inputAssemblies = new HashSet<Assembly>();
			inputAssemblies.Add(Assembly.GetExecutingAssembly());
			inputAssemblies.UnionWith(_pluginCollection.LoadedPlugins.Select(x => x.Assembly));

			// Find all the referenced types from each controller
			HashSet<Type> inputTypes = new HashSet<Type>();
			foreach (Assembly inputAssembly in inputAssemblies)
			{
				foreach (Type exportedType in inputAssembly.GetExportedTypes())
				{
					if (exportedType.GetCustomAttribute<CppApiAttribute>() != null)
					{
						FindTypesFromController(exportedType, inputTypes);
					}
				}
			}

			// Create all the types 
			Dictionary<string, List<Type>> namespaceToTypes = new Dictionary<string, List<Type>>();
			foreach (Type inputType in inputTypes)
			{
				List<Type>? typesList;
				if (!namespaceToTypes.TryGetValue(inputType.Namespace!, out typesList))
				{
					typesList = new List<Type>();
					namespaceToTypes.Add(inputType.Namespace!, typesList);
				}
				typesList.Add(inputType);
			}

			// Process each namespace in order, creating new files from each
			XmlDocReader xmlDocReader = new XmlDocReader();
			foreach ((string namespaceName, List<Type> namespaceTypes) in namespaceToTypes)
			{
				const string ExpectedPrefix = "EpicGames.Horde.";
				if (namespaceName.StartsWith(ExpectedPrefix, StringComparison.Ordinal))
				{
					string outputPath = namespaceName.Substring(ExpectedPrefix.Length).Replace('.', '/');
					string outputName = namespaceName.Substring(ExpectedPrefix.Length).Replace(".", "", StringComparison.Ordinal).TrimEnd('s') + "Messages";

					List<ApiTypeInfo> apiTypes = FindApiTypes(namespaceTypes, xmlDocReader);

					FileReference headerFile = FileReference.Combine(OutputDir, "Public", outputPath, outputName + ".h");
					WriteHeader(headerFile, apiTypes, logger);

					FileReference sourceFile = FileReference.Combine(OutputDir, "Private", outputPath, outputName + ".cpp");
					WriteSource(sourceFile, apiTypes, logger);
				}
			}

			return Task.FromResult(0);
		}

		static readonly Dictionary<Type, string> s_knownCppTypes = new Dictionary<Type, string>
		{
			{ typeof(bool), "bool" },
			{ typeof(int), "int32" },
			{ typeof(string), "FString" },
			{ typeof(StringId), "FString" },
		};

		public static string GetCppType(Type type)
		{
			string? name;
			if (s_knownCppTypes.TryGetValue(type, out name))
			{
				return name;
			}
			if (type.IsEnum)
			{
				return $"E{type.Name}";
			}
			if (type.IsArray)
			{
				Type elementType = type.GetElementType()!;
				return $"TArray<{GetCppType(elementType)}>";
			}
			if (type.IsGenericType)
			{
				Type genericType = type.GetGenericTypeDefinition();
				if (genericType == typeof(List<>))
				{
					Type elementType = type.GetGenericArguments()[0];
					return $"TArray<{GetCppType(elementType)}>";
				}
				if (genericType == typeof(Nullable<>))
				{
					Type elementType = type.GetGenericArguments()[0];
					return $"TOptional<{GetCppType(elementType)}>";
				}
			}
			if (type.Namespace!.StartsWith("EpicGames.Horde.", StringComparison.Ordinal))
			{
				return $"F{type.Name}";
			}
			throw new NotImplementedException();
		}

		static void FindTypesFromController(Type controllerType, HashSet<Type> apiTypes)
		{
			foreach (MethodInfo method in controllerType.GetMethods(BindingFlags.Public | BindingFlags.Instance | BindingFlags.DeclaredOnly))
			{
				foreach (ParameterInfo parameter in method.GetParameters())
				{
					Type parameterType = parameter.ParameterType;
					if (parameterType.GetCustomAttribute<FromQueryAttribute>() == null && parameterType != typeof(CancellationToken))
					{
						apiTypes.Add(parameterType);
					}
				}

				foreach (ProducesResponseTypeAttribute attribute in method.GetCustomAttributes<ProducesResponseTypeAttribute>())
				{
					apiTypes.Add(attribute.Type);
				}

				Type returnType = method.ReturnType;
				if (returnType != typeof(void) && returnType != typeof(Task) && returnType != typeof(ActionResult))
				{
					if (returnType.IsGenericType && returnType.GetGenericTypeDefinition() == typeof(Task<>))
					{
						returnType = returnType.GetGenericArguments()[0];
					}
					if (returnType.IsGenericType && returnType.GetGenericTypeDefinition() == typeof(ActionResult<>))
					{
						returnType = returnType.GetGenericArguments()[0];
					}
					apiTypes.Add(returnType);
				}
			}
		}

		static List<ApiTypeInfo> FindApiTypes(List<Type> types, XmlDocReader xmlDocReader)
		{
			List<ApiTypeInfo> apiTypes = new List<ApiTypeInfo>();
			foreach (Type type in types)
			{
				List<ApiPropertyInfo> properties = new List<ApiPropertyInfo>();
				foreach (PropertyInfo propertyInfo in type.GetProperties(BindingFlags.Public | BindingFlags.Instance))
				{
					JsonPropertyNameAttribute? attribute = propertyInfo.GetCustomAttribute<JsonPropertyNameAttribute>();
					string jsonName = attribute?.Name ?? propertyInfo.Name;
					string description = xmlDocReader.GetDescription(propertyInfo) ?? "";
					properties.Add(new ApiPropertyInfo(jsonName, propertyInfo.PropertyType, description));
				}

				string className = $"F{type.Name}";
				apiTypes.Add(new ApiTypeInfo(className, type, "Unknown", properties));
			}
			return apiTypes;
		}

		static StreamWriter OpenOutputFile(FileReference outputFile, ILogger logger)
		{
			DirectoryReference.CreateDirectory(outputFile.Directory);
			logger.LogInformation("Writing {File}", outputFile);
			return new StreamWriter(outputFile.FullName);
		}

		static void WriteBoilerplate(StreamWriter writer)
		{
			foreach (string boilerplateLine in s_boilerplateLines)
			{
				writer.WriteLine(boilerplateLine);
			}
		}

		static void WriteHeader(FileReference outputFile, List<ApiTypeInfo> apiTypes, ILogger logger)
		{
			using StreamWriter writer = OpenOutputFile(outputFile, logger);

			writer.WriteLine("// Copyright Epic Games, Inc. All Rights Reserved.");
			writer.WriteLine();
			writer.WriteLine("#pragma once");
			writer.WriteLine();
			writer.WriteLine("#include \"CoreMinimal.h\"");
			writer.WriteLine("#include \"Serialization/JsonSerializerMacros.h\"");
			writer.WriteLine();

			WriteBoilerplate(writer);

			foreach (ApiTypeInfo apiType in apiTypes)
			{
				writer.WriteLine();
				writer.WriteLine($"struct {apiType.Name} : FJsonSerializable");
				writer.WriteLine("{");

				foreach (ApiPropertyInfo property in apiType.Properties)
				{
					writer.WriteLine($"\t/** {property.Description} */");
					writer.WriteLine($"\t{GetCppType(property.Type)} {property.Name};");
					writer.WriteLine();
				}

				writer.WriteLine($"\tvirtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override;");
				writer.WriteLine("};");
			}
		}

		static void WriteSource(FileReference outputFile, List<ApiTypeInfo> apiTypes, ILogger logger)
		{
			using StreamWriter writer = OpenOutputFile(outputFile, logger);
			writer.WriteLine("// Copyright Epic Games, Inc. All Rights Reserved.");

			WriteBoilerplate(writer);

			bool first = true;
			foreach (ApiTypeInfo apiType in apiTypes)
			{
				if (first)
				{
					first = false;
				}
				else
				{
					writer.WriteLine("// ----------------------------------------------");
				}

				writer.WriteLine();
				writer.WriteLine($"void {apiType.Name}::Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override");
				writer.WriteLine("{");
				foreach (ApiPropertyInfo property in apiType.Properties)
				{
					writer.WriteLine($"\t\tJSON_SERIALIZE({property.Name}, \"{property.Name}\")");
				}
				writer.WriteLine("}");
			}
		}
	}
}
