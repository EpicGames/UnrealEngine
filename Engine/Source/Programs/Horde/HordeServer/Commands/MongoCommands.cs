// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Server;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Commands;

[Command("mongo", "upgrade", "Upgrade database schema(s) by applying migration scripts")]
class MongoUpgradeCommand(IConfiguration config) : Command
{
	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		using IHost host = ServerCommand.CreateMinimalHostBuilder(config).Build();
		MongoMigrator migrator = host.Services.GetRequiredService<MongoMigrator>();
		
		try
		{
			migrator.AutoAddMigrations();
			await migrator.UpgradeAsync(CancellationToken.None);
		}
		catch (Exception e)
		{
			logger.LogError(e, "Database schema upgrade failed: {Exception}", e);
			return 1;
		}
		
		logger.LogInformation("Database schemas successfully upgraded!");
		return 0;
	}
}

[Command("mongo", "validate", "Validate database schema versions")]
class MongoValidateCommand(IConfiguration config) : Command
{
	/// <inheritdoc/>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		using IHost host = ServerCommand.CreateMinimalHostBuilder(config).Build();
		MongoMigrator migrator = host.Services.GetRequiredService<MongoMigrator>();

		try
		{
			migrator.AutoAddMigrations();
			Dictionary<string,SchemaVersionResult> results = await migrator.GetAllSchemaStateAsync(CancellationToken.None);
			foreach ((string plugin, SchemaVersionResult svr) in results)
			{
				logger.LogInformation("Plugin {Plugin}: {State}", plugin, svr.State);
				if (svr.MissingMigrations.Count > 0)
				{
					logger.LogInformation("\tMissing migrations:");
					foreach (MigrationDefinition md in svr.MissingMigrations)
					{
						logger.LogInformation("\t{Version,3}: {Class} - {Name}", md.Version, md.Migration.GetType().FullName, md.Name);
					}
				}
			}
		}
		catch (Exception e)
		{
			logger.LogError(e, "Database validation failed: {Exception}", e);
			return 1;
		}
		
		return 0;
	}
}

