// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Commands
{
	/// <summary>
	/// Opens a web browser to the server homepage
	/// </summary>
	[Command("open", "Open web browser to the server homepage")]
	public class OpenCommand : Command
	{
		readonly IOptions<ServerSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public OpenCommand(IOptions<ServerSettings> settings)
		{
			_settings = settings;
		}

		/// <inheritdoc/>
		public override Task<int> ExecuteAsync(ILogger logger)
		{
			ProcessStartInfo startInfo = new ProcessStartInfo();
			startInfo.FileName = $"http://localhost:{_settings.Value.HttpPort}";
			startInfo.UseShellExecute = true;
			using Process? process = Process.Start(startInfo);
			return Task.FromResult((process != null) ? 0 : 1);
		}
	}
}
