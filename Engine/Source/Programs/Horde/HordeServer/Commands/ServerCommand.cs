// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Security.Cryptography;
using System.Security.Cryptography.X509Certificates;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Server;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.Server.Kestrel.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Hosting.WindowsServices;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Serilog;

namespace HordeServer.Commands
{
	using ILogger = Microsoft.Extensions.Logging.ILogger;

	[Command("server", "Runs the Horde Build server (default)")]
	class ServerCommand : Command
	{
		readonly ServerSettings _hordeSettings;
		readonly IConfiguration _config;
		string[] _args = Array.Empty<string>();

		public ServerCommand(ServerSettings settings, IConfiguration config)
		{
			_hordeSettings = settings;
			_config = config;
		}
		
		/// <inheritdoc/>
		public override void Configure(CommandLineArguments arguments, ILogger logger)
		{
			base.Configure(arguments, logger);
			_args = arguments.GetRawArray();
		}
		
		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			logger.LogInformation("Server version: {Version}", ServerApp.Version);
			logger.LogInformation("App directory: {AppDir}", ServerApp.AppDir);
			logger.LogInformation("Data directory: {DataDir}", ServerApp.DataDir);
			logger.LogInformation("Server config: {ConfigFile}", ServerApp.ServerConfigFile);
			
			using X509Certificate2? grpcCertificate = ReadGrpcCertificate(_hordeSettings);
			using IHost host = CreateHostBuilderWithCert(_args, _config, _hordeSettings, grpcCertificate).Build();
			
			if (_hordeSettings.MongoMigrationsEnabled)
			{
				MongoMigrator migrator = host.Services.GetRequiredService<MongoMigrator>();
				migrator.AutoAddMigrations();
				bool schemasValid = await migrator.ValidateAllSchemasAsync(_hordeSettings.MongoMigrationsAutoUpgrade, CancellationToken.None);
				if (!schemasValid)
				{
					return 1;
				}
			}
			
			await host.RunAsync();
			return 0;
		}

		static IHostBuilder CreateHostBuilderWithCert(string[] args, IConfiguration config, ServerSettings serverSettings, X509Certificate2? sslCert)
		{
			AppContext.SetSwitch("System.Net.Http.SocketsHttpHandler.Http2UnencryptedSupport", true);

			IHostBuilder hostBuilder = Host.CreateDefaultBuilder(args)
				.UseSerilog()
				.ConfigureHostOptions(options => options.ShutdownTimeout = TimeSpan.FromSeconds(30.0))
				.ConfigureAppConfiguration(builder => builder.AddConfiguration(config))
				.ConfigureWebHostDefaults(webBuilder =>
				{
					webBuilder.UseShutdownTimeout(TimeSpan.FromSeconds(30.0));

					webBuilder.UseUrls(); // Disable default URLs; we will configure each port directly.

					webBuilder.UseWebRoot("DashboardApp");

					webBuilder.ConfigureKestrel(options =>
					{
						options.Limits.MaxRequestBodySize = 256 * 1024 * 1024;

						// When agents are saturated with work (CPU or I/O), slow sending of gRPC data can happen.
						// Kestrel protects against this behavior by default as it's commonly used for malicious attacks.
						// Setting a more generous data rate should prevent incoming HTTP connections from being closed prematurely.
						options.Limits.MinRequestBodyDataRate = new MinDataRate(10, TimeSpan.FromSeconds(60));

						options.Limits.KeepAliveTimeout = TimeSpan.FromSeconds(220); // 10 seconds more than agent's timeout

						int httpPort = serverSettings.HttpPort;
						if (httpPort != 0)
						{
							options.ListenAnyIP(httpPort, configure => { configure.Protocols = HttpProtocols.Http1; });
						}

						int httpsPort = serverSettings.HttpsPort;
						if (httpsPort != 0)
						{
							options.ListenAnyIP(httpsPort, configure =>
							{
								if (sslCert != null)
								{
									configure.UseHttps(sslCert);
								}
								else
								{
									configure.UseHttps();
								}
							});
						}

						// To serve HTTP/2 with gRPC *without* TLS enabled, a separate port for HTTP/2 must be used.
						// This is useful when having a load balancer in front that terminates TLS.
						int http2Port = serverSettings.Http2Port;
						if (http2Port != 0)
						{
							options.ListenAnyIP(http2Port, configure => { configure.Protocols = HttpProtocols.Http2; });
						}
					});
					webBuilder.UseStartup<Startup>();
				});

			if (OperatingSystem.IsWindows() && WindowsServiceHelpers.IsWindowsService())
			{
				// Attempt to setup this process as a Windows service. A race condition inside Microsoft.Extensions.Hosting.WindowsServices.WindowsServiceHelpers.IsWindowsService
				// can result in accessing the parent process after it's terminated, so catch any exceptions that it throws.
				try
				{
					// Register the default WindowsServiceLifetime
					hostBuilder = hostBuilder.UseWindowsService();

#pragma warning disable CA1416
					// Replace the default WindowsServiceLifetime (if there is one; we may not be running as a service) with a custom one
					// that waits for all application startup before the service enters the running state. See https://github.com/dotnet/runtime/issues/50019
					hostBuilder = hostBuilder.ConfigureServices(services =>
					{
						ServiceDescriptor descriptor = services.First(x => x.ImplementationType == typeof(WindowsServiceLifetime));
						services.Remove(descriptor);
						services.AddSingleton<IHostLifetime, CustomWindowsServiceLifetime>();
					});
#pragma warning restore CA1416
				}
				catch (InvalidOperationException)
				{
				}
			}

			return hostBuilder;
		}

		// Custom service lifetime to wait for startup before continuing
		[SupportedOSPlatform("windows")]
		sealed class CustomWindowsServiceLifetime : WindowsServiceLifetime, IHostLifetime
		{
			readonly IHostApplicationLifetime _applicationLifetime;
			readonly ManualResetEventSlim _applicationStarted = new ManualResetEventSlim(false);
			readonly TaskCompletionSource _serviceStarting = new TaskCompletionSource(TaskCreationOptions.RunContinuationsAsynchronously);
			readonly ILogger _logger;

			public CustomWindowsServiceLifetime(IHostEnvironment environment, IHostApplicationLifetime applicationLifetime, ILoggerFactory loggerFactory, IOptions<HostOptions> optionsAccessor)
				: base(environment, applicationLifetime, loggerFactory, optionsAccessor)
			{
				_applicationLifetime = applicationLifetime;
				_applicationLifetime.ApplicationStarted.Register(() => _applicationStarted.Set());

				_logger = loggerFactory.CreateLogger<CustomWindowsServiceLifetime>();
			}

			protected override void Dispose(bool disposing)
			{
				if (disposing)
				{
					_applicationStarted.Dispose();
				}

				base.Dispose(disposing);
			}

			public new async Task WaitForStartAsync(CancellationToken cancellationToken)
			{
				Task baseStartTask = base.WaitForStartAsync(cancellationToken);
				await await Task.WhenAny(baseStartTask, _serviceStarting.Task);
			}

			protected override void OnStart(string[] args)
			{
				// Win32 service is starting; need to synchronously wait until application has finished startup.
				_logger.LogInformation("Win32 service starting...");

				_serviceStarting.TrySetResult();
				_applicationStarted.Wait();

				base.OnStart(args);

				_logger.LogInformation("Win32 service startup complete.");
			}
		}

		/// <summary>
		/// Gets the certificate to use for Grpc endpoints
		/// </summary>
		/// <returns>Custom certificate to use for Grpc endpoints, or null for the default.</returns>
		public static X509Certificate2? ReadGrpcCertificate(ServerSettings hordeSettings)
		{
			string base64Prefix = "base64:";

			if (hordeSettings.ServerPrivateCert == null)
			{
				return null;
			}
			else if (hordeSettings.ServerPrivateCert.StartsWith(base64Prefix, StringComparison.Ordinal))
			{
				try
				{
					byte[] certData = Convert.FromBase64String(hordeSettings.ServerPrivateCert.Substring(base64Prefix.Length));
					return X509CertificateLoader.LoadPkcs12(certData, null);
				}
				catch (Exception ex) when (ex is FormatException or CryptographicException)
				{
					throw new InvalidOperationException($"Failed to load PKCS#12 certificate from base64 value in setting '{nameof(ServerSettings.ServerPrivateCert)}': {ex.Message}", ex);
				}
			}
			else
			{
				FileReference? serverPrivateCert;
				if (!Path.IsPathRooted(hordeSettings.ServerPrivateCert))
				{
					serverPrivateCert = FileReference.Combine(ServerApp.AppDir, hordeSettings.ServerPrivateCert);
				}
				else
				{
					serverPrivateCert = new FileReference(hordeSettings.ServerPrivateCert);
				}

				try
				{
					return X509CertificateLoader.LoadPkcs12(FileReference.ReadAllBytes(serverPrivateCert), null);
				}
				catch (Exception ex) when (ex is IOException or UnauthorizedAccessException or CryptographicException)
				{
					throw new InvalidOperationException($"Failed to load PKCS#12 certificate from path '{serverPrivateCert}' (setting '{nameof(ServerSettings.ServerPrivateCert)}'): {ex.Message}", ex);
				}
			}
		}

		public static IHostBuilder CreateHostBuilderForTesting(string[] args)
		{
			ServerSettings hordeSettings = new ServerSettings();
			return CreateHostBuilderWithCert(args, new ConfigurationBuilder().Build(), hordeSettings, null);
		}
		
		public static IHostBuilder CreateMinimalHostBuilder(IConfiguration config)
		{
			return Host.CreateDefaultBuilder()
				.UseSerilog()
				.ConfigureAppConfiguration(builder => builder.AddConfiguration(config))
				.ConfigureWebHostDefaults(webBuilder => { webBuilder.UseStartup<Startup>(); });
		}
	}
}
