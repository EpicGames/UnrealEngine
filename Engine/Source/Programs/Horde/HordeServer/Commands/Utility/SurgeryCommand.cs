// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Driver;
using MongoDB.Driver.Linq;

namespace HordeServer.Commands
{
	/// <summary>
	/// Scaffolding to construct a DB instance for scoped DB modifications
	/// </summary>
	[Command("surgery", "Placeholder command to allow DB surgery", Advertise = false)]
	public class SurgeryCommand : Command
	{
		readonly IOptions<ServerSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public SurgeryCommand(IOptions<ServerSettings> settings)
		{
			_settings = settings;
		}

		/// <inheritdoc/>
		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			string connectionString = _settings.Value.MongoConnectionString ?? "mongodb://localhost:27017";

			MongoClientSettings mongoSettings = MongoClientSettings.FromConnectionString(connectionString);
			mongoSettings.LinqProvider = LinqProvider.V2;

			MongoClient client = new MongoClient(mongoSettings);
			IMongoDatabase database = client.GetDatabase(_settings.Value.MongoDatabaseName);

			await RunAsync(database, logger);
			return 0;
		}

		async Task RunAsync(IMongoDatabase database, ILogger logger)
		{
			_ = this;
			_ = database;
			_ = logger;

			await Task.Yield();
			throw new NotImplementedException();
		}
	}
}
