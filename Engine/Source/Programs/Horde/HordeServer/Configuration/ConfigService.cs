// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Text.Json;
using System.Text.Json.Nodes;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Secrets;
using EpicGames.Horde.Users;
using EpicGames.Redis;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using ProtoBuf;
using StackExchange.Redis;

namespace HordeServer.Configuration
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Service which processes runtime configuration data.
	/// </summary>
	public sealed class ConfigService : IConfigService, IOptionsFactory<GlobalConfig>, IOptionsChangeTokenSource<GlobalConfig>, IHostedService, IAsyncDisposable
	{
		// Index to all current config files 
		[ProtoContract]
		class ConfigSnapshot
		{
			[ProtoMember(1)]
			public byte[] Data { get; set; } = null!;

			[ProtoMember(4)]
			public Dictionary<Uri, string> Dependencies { get; set; } = new Dictionary<Uri, string>();

			[ProtoMember(5)]
			public string ServerVersion { get; set; } = String.Empty;
		}

		// Implements a token for config change notifications
		class ChangeToken : IChangeToken
		{
			class Registration : IDisposable
			{
				readonly List<Registration> _registrations;
				readonly Action<object?> _callback;
				readonly object? _state;

				public Registration(List<Registration> registrations, Action<object?> callback, object? state)
				{
					_registrations = registrations;
					_callback = callback;
					_state = state;

					lock (registrations)
					{
						registrations.Add(this);
					}
				}

				public void Dispose()
				{
					lock (_registrations)
					{
						_registrations.Remove(this);
					}
				}

				public void Trigger() => _callback(_state);
			}

			readonly List<Registration> _registrations = new List<Registration>();

			public bool ActiveChangeCallbacks => true;
			public bool HasChanged { get; private set; }

			public void TriggerChange()
			{
				HasChanged = true;

				Registration[] registrations;
				lock (_registrations)
				{
					registrations = _registrations.ToArray();
				}
				foreach (Registration registration in registrations)
				{
					registration.Trigger();
				}

				HasChanged = false;
			}

			public IDisposable RegisterChangeCallback(Action<object?> callback, object? state)
			{
				return new Registration(_registrations, callback, state);
			}
		}

		// The current config
		record class ConfigState(IoHash Hash, GlobalConfig GlobalConfig);

		readonly IRedisService _redisService;
		readonly ServerSettings _serverSettings;
		readonly Dictionary<string, IConfigSource> _sources;
		readonly IPluginCollection _pluginCollection;
		readonly JsonSerializerOptions _jsonOptions;
		readonly RedisKey _snapshotKey = "config-v2";
		readonly IEnumerable<IDefaultAclModifier> _defaultAclModifiers;
		readonly IHealthMonitor _health;
		readonly ILogger _logger;

		readonly ITicker _ticker;
		readonly TimeSpan _tickInterval = TimeSpan.FromMinutes(1.0);

		readonly RedisChannel _updateChannel = RedisChannel.Literal("config-update-v2");
		readonly BackgroundTask _updateTask;

		Task<ConfigState> _stateTask;
		readonly ChangeToken _changeToken = new ChangeToken();

		readonly IServiceProvider _serviceProvider;

		/// <inheritdoc/>
		string IOptionsChangeTokenSource<GlobalConfig>.Name => String.Empty;

		/// <summary>
		/// Event for notifications that the config has been updated
		/// </summary>
		public event Action<ConfigUpdateInfo>? OnConfigUpdate;

		/// <summary>
		/// Constructor
		/// </summary>
		public ConfigService(IRedisService redisService, IOptions<ServerSettings> serverSettings, IEnumerable<IConfigSource> sources, IPluginCollection pluginCollection, IEnumerable<IDefaultAclModifier> aclModifiers, IClock clock, IHealthMonitor<ConfigService> health, ILogger<ConfigService> logger, IServiceProvider serviceProvider)
		{
			_redisService = redisService;
			_serverSettings = serverSettings.Value;
			_sources = sources.ToDictionary(x => x.Scheme, x => x, StringComparer.OrdinalIgnoreCase);
			_defaultAclModifiers = aclModifiers;
			_pluginCollection = pluginCollection;
			_health = health;
			_health.SetName("GlobalConfig");
			_logger = logger;
			_serviceProvider = serviceProvider;

			_jsonOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonOptions);

			PluginConfigCollectionConverter pluginConfigConverter = new PluginConfigCollectionConverter();
			foreach (ILoadedPlugin plugin in pluginCollection.LoadedPlugins)
			{
				if (plugin.GlobalConfigType != null)
				{
					pluginConfigConverter.NameToType[plugin.Name] = plugin.GlobalConfigType;
				}
			}
			_jsonOptions.Converters.Add(pluginConfigConverter);

			_ticker = clock.AddSharedTicker<ConfigService>(_tickInterval, TickSharedAsync, logger);

			_updateTask = new BackgroundTask(WaitForUpdatesAsync);

			_stateTask = Task.Run(() => GetStartupStateAsync());
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _stateTask;
			await _updateTask.DisposeAsync();
			await _ticker.DisposeAsync();
		}

		/// <summary>
		/// Wait for the initial config state to be read.
		/// </summary>
		public async Task<GlobalConfig> WaitForInitialConfigAsync(CancellationToken cancellationToken = default)
		{
			ConfigState state = await _stateTask.WaitAsync(cancellationToken);
			return state.GlobalConfig;
		}

		class OverrideConfigFile : IConfigFile
		{
			readonly byte[] _data;

			public Uri Uri { get; }
			public string Revision => "New";
			public IUser? Author => null;
			public bool WasRead { get; private set; }

			public OverrideConfigFile(Uri uri, byte[] data)
			{
				Uri = uri;
				_data = data;
			}

			public ValueTask<ReadOnlyMemory<byte>> ReadAsync(CancellationToken cancellationToken)
			{
				WasRead = true;
				return new ValueTask<ReadOnlyMemory<byte>>(_data);
			}
		}

		class OverrideConfigSource : IConfigSource
		{
			readonly IConfigSource _inner;
			readonly Dictionary<Uri, OverrideConfigFile> _overrides;

			public string Scheme => _inner.Scheme;

			public TimeSpan UpdateInterval => TimeSpan.FromSeconds(1.0);

			public OverrideConfigSource(IConfigSource inner, Dictionary<Uri, OverrideConfigFile> overrides)
			{
				_inner = inner;
				_overrides = overrides;
			}

			public void Add(OverrideConfigFile file) => _overrides.Add(file.Uri, file);

			public async Task<IConfigFile[]> GetFilesAsync(Uri[] uris, CancellationToken cancellationToken)
			{
				IConfigFile[] files = new IConfigFile[uris.Length];
				for (int idx = 0; idx < uris.Length; idx++)
				{
					IConfigFile? file;
					if (_overrides.TryGetValue(uris[idx], out OverrideConfigFile? overrideFile))
					{
						file = overrideFile;
					}
					else
					{
						file = (await _inner.GetFilesAsync(new[] { uris[idx] }, cancellationToken))[0];
					}
					files[idx] = file;
				}
				return files;
			}

			/// <inheritdoc/>
			public Task GetUpdateInfoAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, ConfigUpdateInfo updateInfo, CancellationToken cancellationToken)
				=> Task.CompletedTask;
		}

		ConfigContext CreateConfigContext(IReadOnlyDictionary<string, IConfigSource> sources, ILogger logger)
		{
			ConfigContext context = new ConfigContext(_jsonOptions, sources, logger);
			context.MacroScopes.Add(new Dictionary<string, string> { ["HordeDir"] = ServerApp.AppDir.FullName });
			return context;
		}

		(Dictionary<Uri, OverrideConfigFile> OverrideFiles, ConfigContext Context) CreateOverrideContext(Dictionary<Uri, byte[]> files)
		{
			Dictionary<Uri, OverrideConfigFile> overrideFiles = new Dictionary<Uri, OverrideConfigFile>();
			foreach ((Uri uri, byte[] data) in files)
			{
				overrideFiles[uri] = new OverrideConfigFile(uri, data);
			}

			Dictionary<string, IConfigSource> overrideSources = new Dictionary<string, IConfigSource>(_sources.Count, _sources.Comparer);
			foreach ((string schema, IConfigSource source) in _sources)
			{
				overrideSources.Add(schema, new OverrideConfigSource(source, overrideFiles));
			}

			ConfigContext context = CreateConfigContext(overrideSources, NullLogger.Instance);
			return (overrideFiles, context);
		}

		async Task<GlobalConfig> ReadAndPostLoadOverridesAsync(ConfigContext context, CancellationToken cancellationToken)
		{
			GlobalConfig globalConfig = await ReadGlobalConfigAsync(context, cancellationToken);
			globalConfig.PostLoad(_serverSettings, _pluginCollection.LoadedPlugins, _defaultAclModifiers, _logger);
			await ResolveConfigSecretsAsync(globalConfig, cancellationToken);
			return globalConfig;
		}

		string FormatConfigException(Exception ex, ConfigContext context, string logMessage)
		{
			StringBuilder message = new StringBuilder(ex.Message);
			message.Append("\n\nLocation:\n");
			foreach (IConfigFile include in context.IncludeStack)
			{
				string path = include.GetUserFormattedPath();
				message.Append($"  {path}\n");
			}
			if (ex is not ConfigException && ex is not JsonException)
			{
				message.Append($"\n\nTrace:\n{ex.StackTrace}");
				_logger.LogWarning(ex, "{Message}: {ExMessage}", logMessage, ex.Message);
			}
			return message.ToString();
		}

		/// <summary>
		/// Validate a new set of config files. Parses and runs PostLoad methods on them.
		/// </summary>
		public async Task<ConfigValidationResult> ValidateAsync(Dictionary<Uri, byte[]> files, CancellationToken cancellationToken)
		{
			(Dictionary<Uri, OverrideConfigFile> overrideFiles, ConfigContext context) = CreateOverrideContext(files);
			try
			{
				await ReadAndPostLoadOverridesAsync(context, cancellationToken);

				foreach (OverrideConfigFile file in overrideFiles.Values)
				{
					if (!file.WasRead)
					{
						return new ConfigValidationResult { Error = $"File {file.Uri} was not read by server", Warnings = context.Warnings };
					}
				}

				return new ConfigValidationResult { Warnings = context.Warnings };
			}
			catch (Exception ex)
			{
				return new ConfigValidationResult { Error = FormatConfigException(ex, context, "Unhandled exception while validating config files"), Warnings = context.Warnings };
			}
		}

		/// <inheritdoc/>
		public async Task<ConfigResolveResult> ResolveWithOverridesAsync(Dictionary<Uri, byte[]> files, CancellationToken cancellationToken)
		{
			(Dictionary<Uri, OverrideConfigFile> overrideFiles, ConfigContext context) = CreateOverrideContext(files);
			try
			{
				GlobalConfig globalConfig = await ReadAndPostLoadOverridesAsync(context, cancellationToken);

				// Collect unread files as warnings (not errors)
				List<Uri> unreadFiles = new List<Uri>();
				foreach (OverrideConfigFile file in overrideFiles.Values)
				{
					if (!file.WasRead)
					{
						unreadFiles.Add(file.Uri);
					}
				}

				return new ConfigResolveResult { Config = globalConfig, Plugins = globalConfig.Plugins, UnreadFiles = unreadFiles, Warnings = context.Warnings };
			}
			catch (Exception ex)
			{
				return new ConfigResolveResult { Error = FormatConfigException(ex, context, "Unhandled exception while resolving config overrides"), Warnings = context.Warnings };
			}
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			// Wait for the initial config update to complete
			await _stateTask;

			if (_serverSettings.IsRunModeActive(RunMode.Worker))
			{
				await _ticker.StartAsync();
			}
			_updateTask.Start();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _updateTask.StopAsync(cancellationToken);
			if (_serverSettings.IsRunModeActive(RunMode.Worker))
			{
				await _ticker.StopAsync();
			}
		}

		/// <summary>
		/// Accessor for tests to update the global config
		/// </summary>
		/// <param name="globalConfig">New config value</param>
		public void OverrideConfig(GlobalConfig globalConfig)
		{
			Set(IoHash.Zero, globalConfig);
		}

		/// <summary>
		/// Updates the current config
		/// </summary>
		/// <param name="hash">Hash for the config data</param>
		/// <param name="globalConfig">New config value</param>
		void Set(IoHash hash, GlobalConfig globalConfig)
		{
			// Set the new state
			_stateTask = Task.FromResult(new ConfigState(hash, globalConfig));

			// Notify any watchers of the new config object
			_changeToken.TriggerChange();
		}

		/// <inheritdoc/>
		public GlobalConfig Create(string name)
		{
#pragma warning disable VSTHRD002 // Synchronous wait
			return _stateTask.Result.GlobalConfig;
#pragma warning restore VSTHRD002 // Synchronous wait
		}

		/// <inheritdoc/>
		public IChangeToken GetChangeToken() => _changeToken;

		async Task<ConfigState> GetStartupStateAsync()
		{
			try
			{
				bool forceConfigUpdate = _serverSettings.ForceConfigUpdateOnStartup;
				if (forceConfigUpdate && _redisService.ReadOnlyMode)
				{
					_logger.LogInformation("Ignoring flag to force config update on startup due to Redis configured in read-only mode.");
					forceConfigUpdate = false;
				}

				ReadOnlyMemory<byte> data = ReadOnlyMemory<byte>.Empty;
				if (!forceConfigUpdate)
				{
					data = await ReadSnapshotDataAsync();
				}
				if (data.Length == 0)
				{
					ConfigSnapshot snapshot = await CreateSnapshotAsync(CancellationToken.None);
					await WriteSnapshotAsync(snapshot);
					data = SerializeSnapshot(snapshot);
				}

				ConfigState state = new ConfigState(IoHash.Compute(data.Span), await CreateGlobalConfigAsync(data, CancellationToken.None));
				await _health.UpdateAsync(HealthStatus.Healthy);

				return state;
			}
			catch (Exception ex)
			{
				await _health.UpdateAsync(HealthStatus.Unhealthy, ex.Message);
				throw;
			}
		}

		async ValueTask TickSharedAsync(CancellationToken cancellationToken)
		{
			// Read a new copy of the current snapshot in the context of being elected to perform updates. This
			// avoids any latency with receiving notifications on the pub/sub channel about changes.
			ReadOnlyMemory<byte> initialData = await ReadSnapshotDataAsync();

			// Print info about the initial snapshot
			ConfigSnapshot? snapshot = null;
			if (initialData.Length > 0)
			{
				snapshot = DeserializeSnapshot(initialData);
				_logger.LogDebug("Initial snapshot for update: {@Info}", GetSnapshotInfo(initialData.Span, snapshot));
			}

			// Update the snapshot until we're asked to stop
			while (!cancellationToken.IsCancellationRequested)
			{
				if (snapshot == null || await IsOutOfDateAsync(snapshot, cancellationToken))
				{
					snapshot = await CreateSnapshotGuardedAsync(snapshot?.Dependencies, cancellationToken);
					if (snapshot != null)
					{
						try
						{
							await WriteSnapshotAsync(snapshot);
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Error while updating config: {Message}", ex.Message);
						}
					}
				}

				TimeSpan tickInterval = GetConfigUpdateInterval(snapshot);
				await Task.Delay(tickInterval, cancellationToken);
			}
		}

		async Task WaitForUpdatesAsync(CancellationToken cancellationToken)
		{
			AsyncEvent updateEvent = new AsyncEvent();
			await using (RedisSubscription subscription = await _redisService.SubscribeAsync(_updateChannel, _ => updateEvent.Pulse()))
			{
				Task cancellationTask = Task.Delay(-1, cancellationToken);
				while (!cancellationToken.IsCancellationRequested)
				{
					Task updateTask = updateEvent.Task;

					try
					{
						RedisValue value = await _redisService.GetDatabase().StringGetAsync(_snapshotKey);
						if (!value.IsNullOrEmpty)
						{
							await UpdateConfigObjectAsync(value);
						}
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while updating config from Redis: {Message}", ex.Message);
						await Task.Delay(TimeSpan.FromMinutes(1.0), cancellationToken);
						continue;
					}

					await await Task.WhenAny(updateTask, cancellationTask);
				}
			}
		}

		/// <summary>
		/// Create a new snapshot object, catching any exceptions and sending notifications
		/// </summary>
		/// <param name="prevFiles">Previous snapshot data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New config snapshot</returns>
		async Task<ConfigSnapshot?> CreateSnapshotGuardedAsync(IReadOnlyDictionary<Uri, string>? prevFiles, CancellationToken cancellationToken)
		{
			try
			{
				ConfigSnapshot snapshot = await CreateSnapshotAsync(cancellationToken);
				await _health.UpdateAsync(HealthStatus.Healthy);
				await NotifyConfigUpdateAsync(snapshot.Dependencies, prevFiles, null, cancellationToken);
				return snapshot;
			}
			catch (ConfigException ex)
			{
				_logger.LogError(ex, "Exception while updating config: {Message}", ex.Message);
				await _health.UpdateAsync(HealthStatus.Unhealthy, ex.Message);
				await NotifyConfigUpdateAsync(ex.GetContext().Files.ToDictionary(x => x.Key, x => x.Value.Revision), prevFiles, ex, cancellationToken);
				return null;
			}
		}

		async Task NotifyConfigUpdateAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, Exception? exception, CancellationToken cancellationToken)
		{
			ConfigUpdateInfo info = new ConfigUpdateInfo(new List<string>(), new HashSet<UserId>(), exception);
			foreach (IConfigSource source in _sources.Values)
			{
				await source.GetUpdateInfoAsync(files, prevFiles, info, cancellationToken);
			}
			OnConfigUpdate?.Invoke(info);
		}

		internal Uri GetGlobalConfigUri()
		{
			return GetGlobalConfigUri(_serverSettings.ConfigPath, ServerApp.ConfigDir.FullName);
		}

		/// <summary>
		/// Gets the path to the root config file
		/// </summary>
		internal static Uri GetGlobalConfigUri(string configPath, string defaultConfigDir)
		{
			bool isPerforcePath = configPath.StartsWith("//", StringComparison.Ordinal);
			if (isPerforcePath)
			{
				return ConfigNode.CombinePaths(new Uri(FileReference.Combine(new DirectoryReference(defaultConfigDir), "_").FullName), configPath);
			}

			Uri? uri;
			if (Uri.TryCreate(configPath, UriKind.Absolute, out uri))
			{
				return uri;
			}

			// Path is relative
			FileReference fileReference = FileReference.Combine(new DirectoryReference(defaultConfigDir), configPath);
			return new UriBuilder("file", String.Empty) { Path = fileReference.FullName }.Uri;
		}

		/// <summary>
		/// Get the appropriate update interval for checking of new config updates
		/// </summary>
		TimeSpan GetConfigUpdateInterval(ConfigSnapshot? snapshot)
		{
			TimeSpan interval = TimeSpan.FromSeconds(5.0);
			foreach (Uri sourceUri in GetConfigSourceUris(snapshot))
			{
				if (_sources.TryGetValue(sourceUri.Scheme, out IConfigSource? configSource))
				{
					TimeSpan sourceInterval = configSource.UpdateInterval;
					if (sourceInterval > interval)
					{
						interval = sourceInterval;
					}
				}
			}
			return interval;
		}

		/// <summary>
		/// Enumerates all the known URIs for reading the given config snapshot.
		/// </summary>
		IEnumerable<Uri> GetConfigSourceUris(ConfigSnapshot? snapshot)
		{
			if (snapshot == null)
			{
				return new[] { GetGlobalConfigUri() };
			}
			else
			{
				return snapshot.Dependencies.Keys;
			}
		}

		/// <summary>
		/// Create a new snapshot object
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New config snapshot</returns>
		async Task<ConfigSnapshot> CreateSnapshotAsync(CancellationToken cancellationToken)
		{
			// Read the config files
			ConfigContext context = CreateConfigContext(_sources, _logger);
			try
			{
				// Bind it to the target object
				GlobalConfig globalConfig = await ReadGlobalConfigAsync(context, cancellationToken);
				if (globalConfig.VersionEnum < ConfigVersion.Latest)
				{
					List<string> message = new List<string>();
					message.Add($"Support for the following features will be removed in an upcoming release:");
					message.Add("");
					if (globalConfig.VersionEnum < ConfigVersion.PoolsInConfigFiles)
					{
						message.Add($"- v{(int)ConfigVersion.PoolsInConfigFiles}: Pools should now be configured through the globals.json file rather than REST API or database.");
					}
					message.Add("");
					message.Add($"Please migrate your installation and update the 'Version' property in globals.json to {(int)ConfigVersion.Latest}");

					_logger.LogWarning("Global config file is using old version number ({Version}<{LatestVersion})\n\n{DeprecatedFeaturesMessage}\n", globalConfig.Version, (int)ConfigVersion.Latest, String.Join("\n", message));
				}

				// Create the snapshot data, with the config serialized back out to a byte array
				ConfigSnapshot snapshot = new ConfigSnapshot();
				snapshot.ServerVersion = ServerApp.Version.ToString();
				snapshot.Data = JsonSerializer.SerializeToUtf8Bytes(globalConfig, context.JsonOptions);

				// Save all the dependencies
				foreach ((Uri depUri, IConfigFile depFile) in context.Files)
				{
					snapshot.Dependencies.Add(depUri, depFile.Revision);
				}

				// Execute a PostLoad before returning so we can validate that everything is valid
				globalConfig.PostLoad(_serverSettings, _pluginCollection.LoadedPlugins, _defaultAclModifiers, _logger);

				await ResolveConfigSecretsAsync(globalConfig, cancellationToken);

				return snapshot;
			}
			catch (Exception ex) when (ex is not ConfigException)
			{
				throw new ConfigException(context, ex.Message, ex);
			}
		}

		async Task<GlobalConfig> ReadGlobalConfigAsync(ConfigContext context, CancellationToken cancellationToken)
		{
			// Read the new config in
			Uri globalConfigUri = GetGlobalConfigUri();

			// Generate the schema for the preprocessor
			ObjectConfigNode configNode = new ObjectConfigNode(typeof(GlobalConfig));

			ObjectConfigNode pluginsObj = new ObjectConfigNode(false, false);
			configNode.Properties["Plugins"] = pluginsObj;

			foreach (ILoadedPlugin plugin in _pluginCollection.LoadedPlugins)
			{
				if (plugin.GlobalConfigType != null)
				{
					pluginsObj.Properties.Add(plugin.Name.ToString(), new ObjectConfigNode(plugin.GlobalConfigType));
				}
			}

			// Copy the schema from any remapped nodes to the source node
			foreach (KeyValuePair<string, string> remapConfigPair in s_remapConfigValues)
			{
				if (configNode.TryGetChildNode(remapConfigPair.Value, out ConfigNode? node))
				{
					configNode.AddChildNode(remapConfigPair.Key, node);
				}
			}

			// Preprocess the file
			JsonObject configObj = await context.PreprocessFileAsync(globalConfigUri, configNode, cancellationToken);

			// Copy the preprocessed data into the right location
			foreach ((string source, string target) in s_remapConfigValues)
			{
				CopyJsonNode(configObj, source, target);
			}

			// Bind it to the target object
			return JsonSerializer.Deserialize<GlobalConfig>(configObj, _jsonOptions)!;
		}

		static readonly KeyValuePair<string, string>[] s_remapConfigValues = new[]
		{
			// Analytics
			KeyValuePair.Create("TelemetryStores", "Plugins.Analytics.Stores"),

			// Build
			KeyValuePair.Create("PerforceClusters", "Plugins.Build.PerforceClusters"),
			KeyValuePair.Create("Devices", "Plugins.Build.Devices"),
			KeyValuePair.Create("MaxConformCount", "Plugins.Build.MaxConformCount"),
			KeyValuePair.Create("AgentShutdownIfDisabledGracePeriod", "Plugins.Build.AgentShutdownIfDisabledGracePeriod"),
			KeyValuePair.Create("ArtifactTypes", "Plugins.Build.ArtifactTypes"),
			KeyValuePair.Create("Projects", "Plugins.Build.Projects"),
			KeyValuePair.Create("IssueFixedTag", "Plugins.Build.IssueFixedTag"),

			// Compute
			KeyValuePair.Create("Rates", "Plugins.Compute.Rates"),
			KeyValuePair.Create("Compute", "Plugins.Compute.Clusters"),
			KeyValuePair.Create("Pools", "Plugins.Compute.Pools"),
			KeyValuePair.Create("Software", "Plugins.Compute.Software"),
			KeyValuePair.Create("Networks", "Plugins.Compute.Network"),

			// Secrets
			KeyValuePair.Create("Secrets", "Plugins.Secrets.Secrets"),

			// Storage
			KeyValuePair.Create("Storage", "Plugins.Storage"),

			// Tools
			KeyValuePair.Create("Tools", "Plugins.Tools.Tools")
		};

		static void CopyJsonNode(JsonObject rootObj, string sourcePath, string targetPath)
		{
			JsonNode? sourceNode = rootObj;

			string[] sourceFragments = sourcePath.Split('.');
			foreach (string sourceFragment in sourceFragments)
			{
				sourceNode = sourceNode[sourceFragment];
				if (sourceNode == null)
				{
					return;
				}
			}

			JsonObject? targetObj = rootObj;

			string[] targetFragments = targetPath.Split('.');
			for (int idx = 0; idx < targetFragments.Length - 1; idx++)
			{
				string targetFragment = targetFragments[idx];

				JsonObject? nextObj = targetObj[targetFragment] as JsonObject;
				if (nextObj == null)
				{
					nextObj = new JsonObject();
					targetObj[targetFragment] = nextObj;
				}

				targetObj = nextObj;
			}

			targetObj[targetFragments[^1]] = sourceNode.DeepClone();
		}

		internal byte[] Serialize(GlobalConfig config)
		{
			return JsonSerializer.SerializeToUtf8Bytes(config, _jsonOptions);
		}

		internal GlobalConfig? Deserialize(byte[] data, bool withData)
		{
			JsonSerializerOptions options = new JsonSerializerOptions(_jsonOptions);
			options.Converters.Add(new ConfigResourceSerializer(withData));

			return JsonSerializer.Deserialize<GlobalConfig>(data, options);
		}

		/// <summary>
		/// Writes information about the current snapshot to the logger
		/// </summary>
		/// <param name="span">Data for the snapshot</param>
		/// <param name="snapshot">Snapshot to log information on</param>
		static object GetSnapshotInfo(ReadOnlySpan<byte> span, ConfigSnapshot snapshot)
		{
			return new { Revision = IoHash.Compute(span).ToString(), ServerVersion = snapshot.ServerVersion.ToString(), Dependencies = snapshot.Dependencies.ToArray() };
		}

		/// <summary>
		/// Checks if the given snapshot is out of date
		/// </summary>
		/// <param name="snapshot">The current config snapshot</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the snapshot is out of date</returns>
		async Task<bool> IsOutOfDateAsync(ConfigSnapshot snapshot, CancellationToken cancellationToken)
		{
			try
			{
				// Always re-read the config file when switching server versions
				string newServerVersion = ServerApp.Version.ToString();
				if (!snapshot.ServerVersion.Equals(newServerVersion, StringComparison.Ordinal))
				{
					_logger.LogInformation("Config is out of date (server version {OldVersion} -> {NewVersion})", snapshot.ServerVersion, newServerVersion);
					return true;
				}

				// Group the dependencies by scheme in order to allow the source to batch-query them
				foreach (IGrouping<string, KeyValuePair<Uri, string>> group in snapshot.Dependencies.GroupBy(x => x.Key.Scheme))
				{
					KeyValuePair<Uri, string>[] pairs = group.ToArray();

					IConfigSource? source;
					if (!_sources.TryGetValue(group.Key, out source))
					{
						_logger.LogInformation("Config is out of date (missing source '{Source}')", group.Key);
						return true;
					}

					IConfigFile[] files = await source.GetFilesAsync(pairs.ConvertAll(x => x.Key), cancellationToken);
					for (int idx = 0; idx < pairs.Length; idx++)
					{
						if (!files[idx].Revision.Equals(pairs[idx].Value, StringComparison.Ordinal))
						{
							_logger.LogInformation("Config is out of date (file {Path}: {OldVersion} -> {NewVersion})", files[idx].Uri, pairs[idx].Value, files[idx].Revision);
							return true;
						}
					}
				}
				return false;
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Exception while checking for config files; assuming out of date");
				return true;
			}
		}

		async Task WriteSnapshotAsync(ConfigSnapshot snapshot)
		{
			ReadOnlyMemory<byte> data = SerializeSnapshot(snapshot);
			IoHash hash = IoHash.Compute(data.Span);

			await _redisService.GetDatabase().StringSetAsync(_snapshotKey, data);
			await _redisService.PublishAsync(_updateChannel, RedisValue.EmptyString);

			_logger.LogInformation("Published new config snapshot (hash: {Hash}, server: {Server}, size: {Size})", hash.ToString(), ServerApp.Version.ToString(), data.Length);
		}

		async Task<ReadOnlyMemory<byte>> ReadSnapshotDataAsync()
		{
			RedisValue value = await _redisService.GetDatabase().StringGetAsync(_snapshotKey);
			return value.IsNullOrEmpty ? ReadOnlyMemory<byte>.Empty : (ReadOnlyMemory<byte>)value;
		}

		static ReadOnlyMemory<byte> SerializeSnapshot(ConfigSnapshot snapshot)
		{
			using (MemoryStream outputStream = new MemoryStream())
			{
				using GZipStream zipStream = new GZipStream(outputStream, CompressionMode.Compress, false);
				ProtoBuf.Serializer.Serialize(zipStream, snapshot);
				zipStream.Flush();
				return outputStream.ToArray();
			}
		}

		static ConfigSnapshot DeserializeSnapshot(ReadOnlyMemory<byte> data)
		{
			using (ReadOnlyMemoryStream outputStream = new ReadOnlyMemoryStream(data))
			{
				using GZipStream zipStream = new GZipStream(outputStream, CompressionMode.Decompress, false);
				return ProtoBuf.Serializer.Deserialize<ConfigSnapshot>(zipStream);
			}
		}

		async Task UpdateConfigObjectAsync(ReadOnlyMemory<byte> data)
		{
			ConfigState state = await _stateTask;

			// Hash the data and only update if it changes; this prevents any double-updates due to time between initialization and the hosted service starting.
			IoHash hash = IoHash.Compute(data.Span);
			if (hash != state.Hash && state.Hash != IoHash.Zero) // Don't replace any explicit config override
			{
				GlobalConfig globalConfig = await CreateGlobalConfigAsync(data, CancellationToken.None);
				_logger.LogInformation("Updating config state from Redis (hash: {OldHash} -> {NewHash}, new size: {Size})", state.Hash, hash, data.Length);
				Set(hash, globalConfig);
			}
		}

		async Task<GlobalConfig> CreateGlobalConfigAsync(ReadOnlyMemory<byte> data, CancellationToken cancellationToken = default)
		{
			// Decompress the snapshot object
			ConfigSnapshot snapshot = DeserializeSnapshot(data);

			// Build the new config and store the project and stream configs inside it
			GlobalConfig globalConfig = JsonSerializer.Deserialize<GlobalConfig>(snapshot.Data, _jsonOptions)!;
			globalConfig.Revision = IoHash.Compute(data.Span).ToString();

			// Run the postload callbacks on all the config objects
			globalConfig.PostLoad(_serverSettings, _pluginCollection.LoadedPlugins, _defaultAclModifiers, _logger);

			await ResolveConfigSecretsAsync(globalConfig, cancellationToken);

			return globalConfig;
		}

		class SecretReplacementInfo
		{
			public required string SecretName { get; init; }
			public required object SourceObject { get; init; }
			public required PropertyInfo Property { get; init; }
		}

		/// <summary>
		/// Convert string representations of secrets to concrete values in the config
		/// </summary>
		/// <param name="config">The config to update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task ResolveConfigSecretsAsync(GlobalConfig config, CancellationToken cancellationToken)
		{
			if (!config.ResolveSecrets)
			{
				return;
			}

			SecretsConfig? secretsConfig = config.Plugins.Values.OfType<SecretsConfig>().FirstOrDefault();
			if (secretsConfig == null)
			{
				return;
			}

			await using AsyncServiceScope scope = _serviceProvider.CreateAsyncScope();
			ISecretCollection? secretCollection = scope.ServiceProvider.GetService<ISecretCollection>();
			if (secretCollection == null)
			{
				return;
			}

			Dictionary<SecretId, List<SecretReplacementInfo>> secrets = [];
			foreach ((object obj, PropertyInfo prop) in GetPropertiesWithAttribute<string, ResolveSecretAttribute>(config))
			{
				string? value = prop.GetValue(obj) as string;
				if (String.IsNullOrEmpty(value))
				{
					continue;
				}

				Match match = Regex.Match(value, @"^horde:secret:(?<SecretId>[^\.]+)\.(?<PropertyName>[^\.]+)?$");
				if (!match.Success)
				{
					continue;
				}
				SecretId secretId = new(match.Groups["SecretId"].Value);
				string secretName = match.Groups["PropertyName"].Value;

				if (!secrets.TryGetValue(secretId, out List<SecretReplacementInfo>? replacementInfo))
				{
					replacementInfo = [];
					secrets.Add(secretId, replacementInfo);
				}
				replacementInfo.Add(new SecretReplacementInfo {
					SecretName = secretName, Property = prop, SourceObject = obj,
				});
			}

			foreach ((SecretId secretId, List<SecretReplacementInfo> replacementInfos) in secrets)
			{
				ISecret? secret = await secretCollection.GetAsync(secretId, secretsConfig, cancellationToken);
				if (secret == null)
				{
					throw new KeyNotFoundException($"The given secret '{secretId}' was not found");
				}

				foreach (SecretReplacementInfo replacementInfo in replacementInfos)
				{
					if (!secret.Data.TryGetValue(replacementInfo.SecretName, out string? value))
					{
						throw new KeyNotFoundException($"The given property '{replacementInfo.SecretName}' for the secret '{secretId}' was not found");
					}
					replacementInfo.Property.SetValue(replacementInfo.SourceObject, value);
				}
			}
		}

		static List<KeyValuePair<object, PropertyInfo>> GetPropertiesWithAttribute<T, TAttribute>(object obj) where TAttribute : Attribute
		{
			List<KeyValuePair<object, PropertyInfo>> results = [];
			Stack<object?> stack = new([obj]);
			HashSet<object?> visited = [];

			while (stack.Count > 0)
			{
				object? currentObj = stack.Pop();
				if (currentObj == null || visited.Contains(currentObj))
				{
					continue;
				}

				if (currentObj is not string && currentObj is IEnumerable elems)
				{
					foreach (object? elem in elems)
					{
						if (elem != null)
						{
							stack.Push(elem);
						}
					}
				}
				else
				{
					PropertyInfo[] props = currentObj.GetType().GetProperties(BindingFlags.Public | BindingFlags.Instance);
					foreach (PropertyInfo prop in props)
					{
						if (!prop.CanRead)
						{
							continue;
						}

						if (prop.PropertyType == typeof(T))
						{
							if (prop.GetCustomAttributes<TAttribute>().Any())
							{
								KeyValuePair<object, PropertyInfo> result = new(currentObj, prop);
								results.Add(result);
							}
						}
						else
						{
							// Check if reached properties that should not be enumerated any further
							if (!(prop.PropertyType.IsPrimitive || prop.GetIndexParameters().Length > 0 || prop.PropertyType.IsByRefLike))
							{
								object? value = prop.GetValue(currentObj);
								if (value != null)
								{
									stack.Push(value);
								}
							}
						}
					}
				}

				visited.Add(currentObj);
			}

			return results;
		}
	}
}
