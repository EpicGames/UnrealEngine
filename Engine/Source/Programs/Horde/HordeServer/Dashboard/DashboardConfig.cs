// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using EpicGames.Horde.Common;
using HordeServer.Configuration;
using HordeServer.Utilities;

namespace HordeServer.Dashboard
{
	/// <summary>
	/// Configuration for dashboard features
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/dashboard")]
	[JsonSchemaCatalog("Horde Dashboard", "Horde dashboard configuration file", new[] { "*.dashboard.json", "Dashboard/*.json" })]
	[ConfigDoc("*.dashboard.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Dashboard.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class DashboardConfig
	{
		/// <summary>
		/// Navigate to the landing page by default
		/// </summary>
		public bool ShowLandingPage { get; set; } = false;

		/// <summary>
		/// Custom landing page route to direct users to
		/// </summary>
		public string LandingPageRoute { get; set; } = String.Empty;

		/// <summary>
		/// Enable CI functionality
		/// </summary>
		public bool ShowCI { get; set; } = true;

		/// <summary>
		/// Whether to show functionality related to agents, pools, and utilization on the dashboard.
		/// </summary>
		public bool ShowAgents { get; set; } = true;

		/// <summary>
		/// Whether to show the agent registration page. When using registration tokens from elsewhere this is not needed.
		/// </summary>
		public bool ShowAgentRegistration { get; set; } = true;

		/// <summary>
		/// Show the Perforce server option on the server menu
		/// </summary>
		public bool ShowPerforceServers { get; set; } = true;

		/// <summary>
		/// Show the device manager on the server menu
		/// </summary>
		public bool ShowDeviceManager { get; set; } = true;

		/// <summary>
		/// Show automated tests (legacy) on the server menu
		/// </summary>
		public bool ShowTests { get; set; } = false;

		/// <summary>
		/// Configuration for different agent pages
		/// </summary>
		public List<DashboardAgentCategoryConfig> AgentCategories { get; set; } = new List<DashboardAgentCategoryConfig>();

		/// <summary>
		/// Configuration for different pool pages
		/// </summary>
		public List<DashboardPoolCategoryConfig> PoolCategories { get; set; } = new List<DashboardPoolCategoryConfig>();

		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Macros within this configuration
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

	}

	/// <summary>
	/// Configuration for a category of agents
	/// </summary>
	public class DashboardAgentCategoryConfig
	{
		/// <summary>
		/// Name of the category
		/// </summary>
		public string Name { get; set; } = "Unnamed";

		/// <summary>
		/// Condition string to be evaluated for this page
		/// </summary>
		public Condition? Condition { get; set; }
	}

	/// <summary>
	/// Configuration for a category of pools
	/// </summary>
	public class DashboardPoolCategoryConfig
	{
		/// <summary>
		/// Name of the category
		/// </summary>
		public string Name { get; set; } = "Unnamed";

		/// <summary>
		/// Condition string to be evaluated for this page
		/// </summary>
		public Condition? Condition { get; set; }
	}
}
