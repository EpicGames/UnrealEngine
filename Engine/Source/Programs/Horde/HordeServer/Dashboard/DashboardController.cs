// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Dashboard;
using EpicGames.Horde.Server;
using HordeServer.Accounts;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.Cookies;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Dashboard
{
	/// <summary>	
	/// Dashboard authorization challenge controller	
	/// </summary>	
	[ApiController]
	[Route("[controller]")]
	public class DashboardController : Controller
	{
		readonly string _authenticationScheme;
		readonly ServerSettings _settings;
		readonly IDashboardPreviewCollection _previewCollection;
		readonly IAccountCollection _hordeAccounts;
		readonly IEnumerable<IPluginResponseFilter> _responseFilters;
		readonly IOptionsSnapshot<GlobalConfig> _globalConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public DashboardController(IDashboardPreviewCollection previewCollection, IAccountCollection hordeAccounts, IEnumerable<IPluginResponseFilter> responseFilters, IOptionsMonitor<ServerSettings> serverSettings, IOptionsSnapshot<GlobalConfig> globalConfig)
		{
			_authenticationScheme = AccountController.GetAuthScheme(serverSettings.CurrentValue.AuthMethod);
			_previewCollection = previewCollection;
			_hordeAccounts = hordeAccounts;
			_responseFilters = responseFilters;
			_settings = serverSettings.CurrentValue;
			_globalConfig = globalConfig;
		}

		/// <summary>	
		/// Challenge endpoint for the dashboard, using cookie authentication scheme	
		/// </summary>	
		/// <returns>Ok on authorized, otherwise will 401</returns>	
		[HttpGet]
		[Route("/api/v1/dashboard/challenge")]
		public async Task<IActionResult> GetChallengeAsync()
		{
			bool needsFirstTimeSetup = false;
			if (_settings.AuthMethod == AuthMethod.Horde)
			{
				IAccount? defaultAdminAccount = await _hordeAccounts.GetAsync(AccountId.Parse("65d4f282ff286703e0609ccd"));
				if (defaultAdminAccount == null)
				{
					needsFirstTimeSetup = true;
				}
			}

			return Ok(new GetDashboardChallengeResponse { NeedsFirstTimeSetup = needsFirstTimeSetup, NeedsAuthorization = User.Identity == null || !User.Identity.IsAuthenticated });
		}

		/// <summary>
		/// Login to server, redirecting to the specified URL on success
		/// </summary>
		/// <param name="redirect"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/dashboard/login")]
		public IActionResult Login([FromQuery] string? redirect)
		{
			return new ChallengeResult(_authenticationScheme, new AuthenticationProperties { RedirectUri = redirect ?? "/" });
		}

		/// <summary>
		/// Login to server, redirecting to the specified Base64 encoded URL, which fixes some escaping issues on some auth providers, on success
		/// </summary>
		/// <param name="redirect"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v2/dashboard/login")]
		public IActionResult LoginV2([FromQuery] string? redirect)
		{
			if (_settings.AuthMethod == AuthMethod.Horde)
			{
				return Redirect("/login" + (redirect == null ? String.Empty : $"?redirect={redirect}"));
			}

			string? redirectUri = null;

			if (redirect != null)
			{
				byte[] data = Convert.FromBase64String(redirect);
				redirectUri = Encoding.UTF8.GetString(data);
			}

			return new ChallengeResult(_authenticationScheme, new AuthenticationProperties { RedirectUri = redirectUri ?? "/index" });
		}

		/// <summary>
		/// Logout of the current account
		/// </summary>
		/// /// <param name="dashboard"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/dashboard/logout")]
		public async Task<ActionResult> LogoutAsync([FromQuery] bool? dashboard)
		{
			await HttpContext.SignOutAsync(CookieAuthenticationDefaults.AuthenticationScheme);
			try
			{
				await HttpContext.SignOutAsync(_authenticationScheme);
			}
			catch
			{
			}

			if (dashboard == false && _settings.AuthMethod == AuthMethod.Horde)
			{
				return Redirect("/");
			}

			return Ok();
		}

		/// <summary>
		/// Query all the projects
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/dashboard/config")]
		public ActionResult<GetDashboardConfigResponse> GetDashbordConfig()
		{
			GetDashboardConfigResponse dashboardConfigResponse = new GetDashboardConfigResponse();

			dashboardConfigResponse.AuthMethod = _settings.AuthMethod;

			dashboardConfigResponse.HelpEmailAddress = _settings.HelpEmailAddress;
			dashboardConfigResponse.HelpSlackChannel = _settings.HelpSlackChannel;

			foreach (DashboardAgentCategoryConfig category in _globalConfig.Value.Dashboard.AgentCategories)
			{
				dashboardConfigResponse.AgentCategories.Add(new GetDashboardAgentCategoryResponse { Name = category.Name, Condition = category.Condition });
			}

			foreach (DashboardPoolCategoryConfig category in _globalConfig.Value.Dashboard.PoolCategories)
			{
				dashboardConfigResponse.PoolCategories.Add(new GetDashboardPoolCategoryResponse { Name = category.Name, Condition = category.Condition });
			}

			dashboardConfigResponse.ArtifactTypes = _globalConfig.Value.Plugins.GetBuildConfig().ArtifactTypes.Select(a => a.Type.ToString()).ToList();

			dashboardConfigResponse.ArtifactTypeInfo = _globalConfig.Value.Plugins.GetBuildConfig().ArtifactTypes.Select(a => new GetDashboardArtifactTypeResponse { Type = a.Type.ToString(), KeepDays = a.KeepDays}).ToList();

			dashboardConfigResponse.ConfigVersion = _globalConfig.Value.Version;

			foreach (IPluginResponseFilter responseFilter in _responseFilters)
			{
				responseFilter.Apply(HttpContext, dashboardConfigResponse);
			}

			return dashboardConfigResponse;
		}

		/// <summary>
		/// Create a new dashboard preview item
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		[HttpPost]
		[Authorize]
		[Route("/api/v1/dashboard/preview")]
		public async Task<ActionResult<GetDashboardPreviewResponse>> CreateDashbordPreviewAsync([FromBody] CreateDashboardPreviewRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid();
			}

			IDashboardPreview preview = await _previewCollection.AddPreviewAsync(request.Summary, cancellationToken);

			if (!String.IsNullOrEmpty(request.ExampleLink) || !String.IsNullOrEmpty(request.DiscussionLink) || !String.IsNullOrEmpty(request.TrackingLink))
			{
				IDashboardPreview? updated = await _previewCollection.UpdatePreviewAsync(preview.Id, null, null, null, request.ExampleLink, request.DiscussionLink, request.TrackingLink, cancellationToken);
				if (updated == null)
				{
					return NotFound(preview.Id);
				}

				return CreatePreviewResponse(updated);
			}

			return CreatePreviewResponse(preview);
		}

		/// <summary>
		/// Update a dashboard preview item
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		[HttpPut]
		[Authorize]
		[Route("/api/v1/dashboard/preview")]
		public async Task<ActionResult<GetDashboardPreviewResponse>> UpdateDashbordPreviewAsync([FromBody] UpdateDashboardPreviewRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid();
			}

			IDashboardPreview? preview = await _previewCollection.UpdatePreviewAsync(request.Id, request.Summary, request.DeployedCL, request.Open, request.ExampleLink, request.DiscussionLink, request.TrackingLink, cancellationToken);

			if (preview == null)
			{
				return NotFound(request.Id);
			}

			return CreatePreviewResponse(preview);
		}

		/// <summary>
		/// Query dashboard preview items
		/// </summary>
		/// <returns>Config information needed by the dashboard</returns>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/dashboard/previews")]
		public async Task<ActionResult<List<GetDashboardPreviewResponse>>> GetDashbordPreviewsAsync([FromQuery] bool open = true, CancellationToken cancellationToken = default)
		{
			List<IDashboardPreview> previews = await _previewCollection.FindPreviewsAsync(open, cancellationToken);
			return previews.Select(CreatePreviewResponse).ToList();
		}

		static GetDashboardPreviewResponse CreatePreviewResponse(IDashboardPreview preview)
		{
			GetDashboardPreviewResponse response = new GetDashboardPreviewResponse();
			response.Id = preview.Id;
			response.CreatedAt = preview.CreatedAt;
			response.Summary = preview.Summary;
			response.DeployedCL = preview.DeployedCL;
			response.Open = preview.Open;
			response.ExampleLink = preview.ExampleLink;
			response.DiscussionLink = preview.DiscussionLink;
			response.TrackingLink = preview.TrackingLink;
			return response;
		}

		/// <summary>
		/// Returns a list of valid user-defined groups from the current server config. These are any claims with
		/// the <see cref="HordeClaimTypes.Group"/> type.
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/dashboard/accountgroups")]
		public ActionResult<List<AccountClaimMessage>> GetAccountGroupClaims()
		{
			if (!_globalConfig.Value.Authorize(AccountAclAction.CreateAccount, User) && !_globalConfig.Value.Authorize(AccountAclAction.UpdateAccount, User))
			{
				return Forbid();
			}

			return Ok(_globalConfig.Value.GetValidAccountGroupClaims().Select(x => new AccountClaimMessage(HordeClaimTypes.Group, x)).ToList());
		}
	}
}
