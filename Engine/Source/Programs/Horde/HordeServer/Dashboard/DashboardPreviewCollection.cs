// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Dashboard
{

	/// <summary>
	/// 
	/// </summary>
	public interface IDashboardPreview
	{
		/// <summary>
		/// The unique ID of the preview item
		/// </summary>
		public int Id { get; }

		/// <summary>
		/// When the preview item was created
		/// </summary>
		public DateTime CreatedAt { get; }

		/// <summary>
		/// A summary of what the preview item changes
		/// </summary>
		public string Summary { get; }

		/// <summary>
		/// The CL the preview was deployed in
		/// </summary>
		public int? DeployedCL { get; }

		/// <summary>
		/// Whather the preview is under consideration, if false the preview item didn't pass muster
		/// </summary>
		public bool Open { get; }

		/// <summary>
		/// An example of the preview site users can view the changes
		/// </summary>
		public string? ExampleLink { get; }

		/// <summary>
		/// Optional Link for discussion the preview item
		/// </summary>
		public string? DiscussionLink { get; }

		/// <summary>
		/// Optional Link for discussing the preview item
		/// </summary>
		public string? TrackingLink { get; }
	}

	/// <summary>
	/// Collection of preview documents
	/// </summary>
	public interface IDashboardPreviewCollection
	{

		/// <summary>
		/// Add a preview item
		/// </summary>
		/// <param name="summary"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<IDashboardPreview> AddPreviewAsync(string summary, CancellationToken cancellationToken);

		/// <summary>
		/// Update a dashboard preview item
		/// </summary>
		/// <param name="previewId"></param>
		/// <param name="summary"></param>
		/// <param name="deployedCL"></param>
		/// <param name="open"></param>
		/// <param name="exampleLink"></param>
		/// <param name="discussionLink"></param>
		/// <param name="trackingLink"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<IDashboardPreview?> UpdatePreviewAsync(int previewId, string? summary = null, int? deployedCL = null, bool? open = null, string? exampleLink = null, string? discussionLink = null, string? trackingLink = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds preview items with the given criterial
		/// </summary>
		/// <param name="open"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<List<IDashboardPreview>> FindPreviewsAsync(bool? open = null, CancellationToken cancellationToken = default);
	}

	internal class DashboardPreviewCollection : IDashboardPreviewCollection
	{
		[SingletonDocument("dashboard-preview-ledger", "63f8e2154a2e859581b5bb24")]
		internal class PreviewLedger : SingletonBase
		{
			public int NextId { get; set; }
		}

		internal class PreviewDocument : IDashboardPreview
		{
			[BsonId]
			public int Id { get; set; }

			public DateTime CreatedAt { get; set; }

			public string Summary { get; set; }

			public bool Open { get; set; }

			[BsonIgnoreIfNull]
			public int? DeployedCL { get; set; }

			[BsonIgnoreIfNull]
			public string? ExampleLink { get; set; }

			[BsonIgnoreIfNull]
			public string? DiscussionLink { get; set; }

			[BsonIgnoreIfNull]
			public string? TrackingLink { get; set; }

			[BsonConstructor]
			private PreviewDocument()
			{
				Summary = String.Empty;
			}

			public PreviewDocument(int id, string summary)
			{
				Id = id;
				Summary = summary;
				CreatedAt = DateTime.UtcNow;
				Open = true;
			}
		}

		readonly IMongoCollection<PreviewDocument> _previews;
		readonly ISingletonDocument<PreviewLedger> _ledgerSingleton;

		public DashboardPreviewCollection(IMongoService mongoService)
		{
			_ledgerSingleton = new SingletonDocument<PreviewLedger>(mongoService);
			_previews = mongoService.GetCollection<PreviewDocument>("DashboardPreviews");
		}

		public async Task<IDashboardPreview> AddPreviewAsync(string summary, CancellationToken cancellationToken)
		{
			PreviewLedger ledger = await _ledgerSingleton.UpdateAsync(x => x.NextId++, cancellationToken);

			PreviewDocument newPreview = new PreviewDocument(ledger.NextId, summary);
			await _previews.InsertOneAsync(newPreview, (InsertOneOptions?)null, cancellationToken);

			return newPreview;
		}

		public async Task<IDashboardPreview> UpdatePreviewAsync(string summary, CancellationToken cancellationToken)
		{
			PreviewLedger ledger = await _ledgerSingleton.UpdateAsync(x => x.NextId++, cancellationToken);

			PreviewDocument newPreview = new PreviewDocument(ledger.NextId, summary);
			await _previews.InsertOneAsync(newPreview, (InsertOneOptions?)null, cancellationToken);

			return newPreview;
		}

		public async Task<IDashboardPreview?> UpdatePreviewAsync(int previewId, string? summary = null, int? deployedCL = null, bool? open = null, string? exampleLink = null, string? discussionLink = null, string? trackingLink = null, CancellationToken cancellationToken = default)
		{
			UpdateDefinitionBuilder<PreviewDocument> updateBuilder = Builders<PreviewDocument>.Update;
			List<UpdateDefinition<PreviewDocument>> updates = new List<UpdateDefinition<PreviewDocument>>();

			if (summary != null)
			{
				updates.Add(updateBuilder.Set(x => x.Summary, summary));
			}

			if (deployedCL != null)
			{
				updates.Add(updateBuilder.Set(x => x.DeployedCL, deployedCL));
			}

			if (open != null)
			{
				updates.Add(updateBuilder.Set(x => x.Open, open));
			}

			if (exampleLink != null)
			{
				updates.Add(updateBuilder.Set(x => x.ExampleLink, exampleLink));
			}

			if (discussionLink != null)
			{
				updates.Add(updateBuilder.Set(x => x.DiscussionLink, discussionLink));
			}

			if (trackingLink != null)
			{
				updates.Add(updateBuilder.Set(x => x.TrackingLink, trackingLink));
			}

			if (updates.Count == 0)
			{
				await _previews.Find(x => x.Id == previewId).FirstOrDefaultAsync(cancellationToken);
			}

			if (await _previews.FindOneAndUpdateAsync(x => x.Id == previewId, updateBuilder.Combine(updates), null, cancellationToken) == null)
			{
				return null;
			}

			return await _previews.Find(x => x.Id == previewId).FirstOrDefaultAsync(cancellationToken);
		}

		public async Task<List<IDashboardPreview>> FindPreviewsAsync(bool? open = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<PreviewDocument> filter = Builders<PreviewDocument>.Filter.Empty;
			if (open != null)
			{
				filter &= Builders<PreviewDocument>.Filter.Eq(x => x.Open, open);
			}

			List<PreviewDocument> results = await _previews.Find(filter).ToListAsync(cancellationToken);
			return results.Select<PreviewDocument, IDashboardPreview>(x => x).ToList();
		}
	}
}

