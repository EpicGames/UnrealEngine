<!DOCTYPE html>
<html>
<head>
	<title>Horde Server</title>
	<style>
		body { font-family: 'Segoe UI', 'Roboto', arial, sans-serif; }
		p { margin:20px; font-size:13px; }
		h1 { margin:20px; font-size:32px; font-weight:200; }
	</style>
	<link xmlns="http://www.w3.org/1999/xhtml" rel="shortcut icon" href="favicon.svg" />
</head>
<body>
	<h1>Horde Server</h1>
	<p>To have the Horde server host the full dashboard app, copy the built dashboard files into the DashboardApp folder.</p>
	<p><a href="/account">Account Page</a></p>
	<p><a href="/swagger">API Documentation</a></p>
</body>
</html>
