{
	"macros": [
		{
			"name": "IsBuildMachine",
			"value": "@.Payload.Config_IsBuildMachine==true"
		},
		{
			"name": "IsUserMachine",
			"value": "@.Payload.Config_IsBuildMachine==false && @.Payload.Config_IsRunningCommandlet==false"
		},
		{
			"name": "IsLoadTimeInRange",
			"value": "@.Payload.LoadingSeconds>5 && @.Payload.LoadingSeconds<3600"
		},
		{
			"name": "IsUBAExecutor",
			"value": "@.Payload.EventName == 'Core.UBT.Executor' && search(@.Payload.Executor, \"Unreal Build Accelerator\")"
		},
		{
			"name": "IsEditorPerf",
			"value": "search(@.Payload.Horde_TemplateName, \"EditorPerf Test\")"
		}
	],
	"metrics": [
		{
			"id": "core-loading-build-timetoedit-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToEdit')]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-build-timetopie-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToPIE')]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-build-timetoedit-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToEdit')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-build-timetopie-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToPIE')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-build-timetoedit",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToEdit')]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-build-timetopie",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsEditorPerf) && $(IsLoadTimeInRange) && @.Payload.LoadingName=='TimeToPIE')]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-loadingtime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) && $(IsLoadTimeInRange) )]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.LoadingName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-loadingcount",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.LoadingName",
			"function": "Count",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-editorboottime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Boot' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-editorstartuptime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Initialize' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-loadmaptime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.LoadMap' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-loadmap-count",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.LoadMap' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"function": "Count",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-pieloadmaptime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='PIE.LoadMap' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-timetoedit",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToEdit' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-timetopie",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToPIE' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-piestartuptime",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='PIE.Startup' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"function": "Average",
			"interval": "03:00:00",
			"topN": 10
		},
		{
			"id": "core-loading-user-editorboottime-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Boot' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-editorstartuptime-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Initialize' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-loadmaptime-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.LoadMap' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-piestartuptime-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='PIE.Startup' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-timetoedit-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToEdit' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-timetopie-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToPIE' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.LoadingSeconds",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-cloudlocalddchitratio",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) && @.Payload.DDC_Summary_CloudGetTotal>0 && $(IsLoadTimeInRange))]",
			"property": "$.Payload.DDC_Summary_CloudGetHitPct",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-zenlocalddchitratio",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) && @.Payload.DDC_Summary_ZenLocalGetTotal>0 && $(IsLoadTimeInRange))]",
			"property": "$.Payload.DDC_Summary_ZenLocalGetHitPct",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-sharedddchitratio",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) && @.Payload.DDC_Summary_SharedGetTotal>0 && $(IsLoadTimeInRange))]",
			"property": "$.Payload.DDC_Summary_SharedGetHitPct",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-totalddchitratio",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"property": "$.Payload.DDC_Summary_TotalGetHitPct",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-ddc-asset-efficiency",
			"filter": "$[?(@.Payload.EventName=='Core.DDC.Resource' && $(IsUserMachine) && @.Payload.AssetType!='Total')]",
			"property": "$.Payload.Efficiency",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.AssetType",
			"function": "Average",
			"interval": "03:00:00",
			"bottomN": 25
		},
		{
			"id": "core-loading-user-timetoedit-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToEdit' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-timetopie-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='TimeToPIE' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-editorboottime-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Boot' && $(IsUserMachine) && $(IsLoadTimeInRange) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-editorstartuptime-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.Initialize' && $(IsUserMachine) && $(IsLoadTimeInRange) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-loadmaptime-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='Editor.LoadMap' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-loading-user-piestartuptime-percentile",
			"filter": "$[?(@.Payload.EventName == 'Core.Loading' && @.Payload.LoadingName=='PIE.Startup' && $(IsUserMachine) && $(IsLoadTimeInRange))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.MapName",
			"property": "$.Payload.LoadingSeconds",
			"function": "Percentile",
			"percentile": 75,
			"interval": "03:00:00"
		},
		{
			"id": "core-cooking-standard-cook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Standard Build\") && search(@.Payload.Horde_StepName, \"Cook\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-standard-shaders-compiled",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Standard Build\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-incremental-cook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_TemplateName=='Incremental CI')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "core-cooking-incremental-shaders-compiled",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_TemplateName=='Incremental CI')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Horde_StepName",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "core-cooking-nightly-cook-time-windows-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\") && @.Payload.Horde_StepName=='Cook Windows Client')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-nightly-cook-time-ps5-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\") && @.Payload.Horde_StepName=='Cook PS5 Client')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-nightly-cook-shaders-compiled-windows-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\") && @.Payload.Horde_StepName=='Cook Windows Client')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-nightly-cook-shaders-compiled-ps5-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\") && @.Payload.Horde_StepName=='Cook PS5 Client')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-nightly-cook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\") && search(@.Payload.Horde_StepName, \"Cook\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-nightly-shaders-compiled",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Nightly Build\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-autotest-cook-time-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest Cook')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-autotest-cook-shaders-compiled-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest Cook')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-autotest-cook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest Cook')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "core-cooking-autotest-shaders-compiled",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest Cook')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "core-cooking-autotest-fastcook-time-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest FastCook')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-autotest-fastcook-shaders-compiled-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest FastCook')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-autotest-fastcook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && @.Payload.Horde_StepName=='AutoTest FastCook')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-lyra-time-windows-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='Lyra Cook Win64')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-ancientgame-time-windows-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='AncientGame Cook Win64')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-citysample-time-windows-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='CitySample Cook Win64')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-lyra-time-ps5-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='Lyra Cook PS5')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-ancientgame-time-ps5-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='AncientGame Cook PS5')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-citysample-time-ps5-kpi",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && @.Payload.Horde_StepName=='CitySample Cook PS5')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-package-and-test-cook-time",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\") && search(@.Payload.Horde_StepName, \"Cook\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Cook_Profile_CookWallTimeSec",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-cooking-package-and-test-shaders-compiled",
			"filter": "$[?(@.Payload.EventName == 'Core.Cooking' && search(@.Payload.Horde_TemplateName, \"Package and Test\"))]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_StepName,$.Payload.Cook_Profile_TargetPlatforms",
			"property": "$.Payload.Shaders_ShadersCompiled",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "core-memory-llm-systems",
			"filter": "$[?(@.Payload.EventName == 'Core.Memory.LLM' && @.Payload.TagSet=='None')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Hardware_Platform,$.Payload.LoadingName,$.Payload.Name",
			"property": "$.Payload.Size",
			"function": "Average",
			"interval": "00:15:00",
			"topN": 25
		},
		{
			"id": "core-memory-llm-assetclasses",
			"filter": "$[?(@.Payload.EventName == 'Core.Memory.LLM' && @.Payload.TagSet=='AssetClasses')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Hardware_Platform,$.Payload.LoadingName,$.Payload.Name",
			"property": "$.Payload.Size",
			"function": "Average",
			"interval": "00:15:00",
			"topN": 25
		},
		{
			"id": "core-memory-llm-assets",
			"filter": "$[?(@.Payload.EventName == 'Core.Memory.LLM' && @.Payload.TagSet=='Assets')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.ProjectName,$.Payload.Hardware_Platform,$.Payload.LoadingName,$.Payload.Name",
			"property": "$.Payload.Size",
			"function": "Average",
			"interval": "00:15:00",
			"topN": 25
		},
		{
			"id": "core-content-events",
			"filter": "$[?(@.Payload.EventName == 'Core.VirtualAssets.MountPoint')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Session_Label,$.Payload.ProjectName",
			"property": "$.Payload.SchemaVersion",
			"function": "Count",
			"interval": "00:05:00"
		},
		{
			"id": "core-content-totalsize",
			"filter": "$[?(@.Payload.EventName == 'Core.VirtualAssets.MountPoint')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.TotalFileSize",
			"function": "Sum",
			"interval": "03:00:00"
		},
		{
			"id": "core-content-totalsize-mp",
			"filter": "$[?(@.Payload.EventName == 'Core.VirtualAssets.MountPoint')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Name",
			"property": "$.Payload.TotalFileSize",
			"function": "Sum",
			"interval": "03:00:00",
			"topN": 25
		},
		{
			"id": "core-content-virtualization",
			"filter": "$[?(@.Payload.EventName == 'Core.VirtualAssets.MountPoint')]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.VirtualizedPercent",
			"function": "Average",
			"interval": "03:00:00"
		},
		{
			"id": "core-content-virtualization-mp",
			"filter": "$[?(@.Payload.EventName == 'Core.VirtualAssets.MountPoint')]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Name",
			"property": "$.Payload.VirtualizedPercent",
			"function": "Average",
			"interval": "03:00:00",
			"bottomN": 25
		},
		{
			"id": "horde-incremental-build-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && @.Payload.JobName=='Incremental Build' && search(@.Payload.StepName, \"Incremental\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-incremental-ci-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && @.Payload.JobName=='Incremental CI' && search(@.Payload.StepName, \"Incremental\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-autotest-ci-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && @.Payload.JobName=='Incremental CI' && search(@.Payload.StepName, \"AutoTest\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-package-and-test-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && search(@.Payload.JobName, \"Package and Test\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.JobName,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-nightly-build-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && search(@.Payload.JobName, \"Nightly Build\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.JobName,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-monolithic-compile-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && @.Payload.JobName=='Editor, Tools & Monolithics' && search(@.Payload.StepName, \"Compile\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "horde-incremental-compile-jobs",
			"filter": "$[?(@.Payload.EventName == 'State.JobStepRef' && @.Payload.JobName=='Incremental CI' && search(@.Payload.StepName, \"Incremental Compile\") && @.Payload.Duration>0 && @.Payload.State=='Completed')]",
			"groupBy": "$.Payload.StreamId,$.Payload.StepName",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "01:00:00"
		},
		{
			"id": "ubt-uba-user-duration-avg",
			"filter": "$[?( $(IsUserMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-user-remote-usage-avg",
			"filter": "$[?( $(IsUserMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteUsage",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-user-remote-actions-sum",
			"filter": "$[?( $(IsUserMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-user-total-actions-sum",
			"filter": "$[?( $(IsUserMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.TotalActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-build-duration-avg",
			"filter": "$[?( $(IsBuildMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-build-remote-usage-avg",
			"filter": "$[?( $(IsBuildMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteUsage",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-build-remote-actions-sum",
			"filter": "$[?( $(IsBuildMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-build-total-actions-sum",
			"filter": "$[?( $(IsBuildMachine) && $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.TotalActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-duration",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-remote-usage",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteUsage",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-remote-actions",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-local-usage",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.LocalUsage",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-local-actions",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.LocalActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-total-actions",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.TotalActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-remote-actions-avg",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.RemoteActions",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-cache-rate-avg",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.CacheRate",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-local-actions-avg",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.LocalActions",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-total-actions-avg",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.TotalActions",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-failure-rate",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.FailureRate",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-remote-usage-all",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.RemoteUsage",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-total-actions-all",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.TotalActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-remote-actions-all",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.RemoteActions",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-failure-rate-all",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.FailureRate",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-uba-cache-rate-all",
			"filter": "$[?( $(IsUBAExecutor) )]",
			"groupBy": "$.Payload.Build_BranchName",
			"property": "$.Payload.CacheRate",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "ubt-completed-duration",
			"filter": "$[?(@.Payload.EventName == 'Core.UBT.Completed' )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Target,$.Payload.Mode,$.Payload.Configuration,$.Payload.Platform",
			"property": "$.Payload.Duration",
			"function": "Average",
			"interval": "00:30:00"
		},
		{
			"id": "package-size",
			"filter": "$[?(@.Payload.EventName == 'Core.IoStoreChunk' && @.Payload.SchemaVersion>=8 )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_TemplateName,$.Payload.Platform,$.Payload.InstallType,$.Payload.PakChunk",
			"property": "$.Payload.CompressedSize",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "package-total-size",
			"filter": "$[?(@.Payload.EventName == 'Core.IoStoreChunk' && @.Payload.SchemaVersion>=8 )]",
			"groupBy": "$.Payload.Build_BranchName,$.Payload.Horde_TemplateName,$.Payload.Platform,$.Payload.InstallType",
			"property": "$.Payload.CompressedSize",
			"function": "Sum",
			"interval": "00:30:00"
		},
		{
			"id": "submit-tool-session-count",
			"filter": "$[?(@.Payload.EventName == 'SubmitTool.StandAlone.Start')]",
			"groupBy": "$.Payload.Stream,$.Payload.Version",
			"function": "Count",
			"interval": "00:30:00"
		}
	]
}