{
	"views": [
		{
			"id": "engine",
			"name": "Engine",
			"telemetryStoreId": "engine",
			"variables": [
				{
					"name": "Stream",
					"group": "Build_BranchName",
					"defaults": [
						"++ue5+main"
					]
				},
				{
					"name": "Project",
					"group": "ProjectName",
					"defaults": [
						"Lyra",
						"CitySample"
					]
				},
				{
					"name": "Map",
					"group": "MapName"
				},
				{
					"name": "LoadingName",
					"group": "LoadingName",
					"defaults": [
						"TimeToEdit",
						"TimeToPIE"
					]
				},
				{
					"name": "AssetType",
					"group": "AssetType"
				},
				{
					"name": "Cook Platform",
					"group": "Cook_Profile_TargetPlatforms"
				},
				{
					"name": "Horde TemplateName",
					"group": "Horde_TemplateName"
				},
				{
					"name": "Horde JobName",
					"group": "Horde_JobName"
				},
				{
					"name": "Horde StepName",
					"group": "Horde_StepName"
				},
				{
					"name": "StreamId",
					"group": "StreamId"
				},
				{
					"name": "JobName",
					"group": "JobName"
				},
				{
					"name": "StepName",
					"group": "StepName"
				},
				{
					"name": "Event",
					"group": "EventName"
				},
				{
					"name": "Label",
					"group": "Session_Label"
				},
				{
					"name": "Target",
					"group": "Target",
					"defaults": [
						"CitySample",
						"CityEditor",
						"LyraEditor",
						"LyraGame"
					]
				},
				{
					"name": "Mode",
					"group": "Mode",
					"defaults": [
						"Build",
						"Rebuild"
					]
				},
				{
					"name": "Platform",
					"group": "Platform",
					"defaults": [
						"Win64"
					]
				},
				{
					"name": "Configuration",
					"group": "Configuration",
					"defaults": [
						"Development"
					]
				},
				{
					"name": "Hardware Platform",
					"group": "Hardware_Platform"
				},
				{
					"name": "Install Type",
					"group": "InstallType"
				}
			],
			"categories": [
				{
					"name": "Live",
					"charts": [
						{
							"name": "User Load Time (75th Percentile)",
							"metrics": [
								{
									"id": "core-loading-user-timetoedit-percentile",
									"alias": "TimeToEdit",
									"threshold": 120
								},
								{
									"id": "core-loading-user-timetopie-percentile",
									"alias": "TimeToPIE",
									"threshold": 360
								}
							],
							"graph": "indicator",
							"max": 720
						},
						{
							"name": "User Load Time (Average)",
							"metrics": [
								{
									"id": "core-loading-user-timetoedit",
									"alias": "TimeToEdit",
									"threshold": 120
								},
								{
									"id": "core-loading-user-timetopie",
									"alias": "TimeToPIE",
									"threshold": 360
								}
							],
							"graph": "indicator",
							"max": 900
						},
						{
							"name": "User Loading Time (KPIs)",
							"metrics": [
								{
									"id": "core-loading-user-timetoedit-kpi",
									"alias": "TimeToEdit"
								},
								{
									"id": "core-loading-user-timetopie-kpi",
									"alias": "TimeToPIE"
								}
							]
						},
						{
							"name": "User Loading Count (Live)",
							"display": "Value",
							"max": 100,
							"metrics": [
								{
									"id": "core-loading-user-loadingcount"
								}
							]
						},
						{
							"name": "User Editor Boot Time (Live)",
							"metrics": [
								{
									"id": "core-loading-user-editorboottime"
								}
							]
						},
						{
							"name": "User Editor LoadMap Time (Live)",
							"metrics": [
								{
									"id": "core-loading-user-loadmaptime"
								}
							]
						},
						{
							"name": "User Editor LoadMap Count (Live)",
							"display": "Value",
							"metrics": [
								{
									"id": "core-loading-user-loadmap-count"
								}
							]
						},
						{
							"name": "User PIE Startup Time (Live)",
							"metrics": [
								{
									"id": "core-loading-user-piestartuptime"
								}
							]
						},
						{
							"name": "User PIE LoadMap Time (Live)",
							"metrics": [
								{
									"id": "core-loading-user-pieloadmaptime"
								}
							]
						},
						{
							"name": "User Boot To PIE Time (Live)",
							"metrics": [
								{
									"id": "core-loading-user-timetopie"
								}
							]
						}
					]
				},
				{
					"name": "Load",
					"charts": [
						{
							"name": "Load Time (75th Percentile)",
							"metrics": [
								{
									"id": "core-loading-build-timetoedit-percentile",
									"alias": "TimeToEdit",
									"threshold": 120
								},
								{
									"id": "core-loading-build-timetopie-percentile",
									"alias": "TimeToPIE",
									"threshold": 360
								}
							],
							"graph": "indicator",
							"max": 720
						},
						{
							"name": "Load Time (Average)",
							"metrics": [
								{
									"id": "core-loading-build-timetoedit",
									"alias": "TimeToEdit",
									"threshold": 120
								},
								{
									"id": "core-loading-build-timetopie",
									"alias": "TimeToPIE",
									"threshold": 360
								}
							],
							"graph": "indicator",
							"max": 900
						},
						{
							"name": "Load Time (KPIs)",
							"metrics": [
								{
									"id": "core-loading-build-timetoedit-kpi",
									"alias": "TimeToEdit"
								},
								{
									"id": "core-loading-build-timetopie-kpi",
									"alias": "TimeToPIE"
								}
							]
						}
					]
				},
				{
					"name": "Cook",
					"charts": [
						{
							"name": "Cook Time (KPIs)",
							"metrics": [
								{
									"id": "core-cooking-lyra-time-windows-kpi",
									"alias": "Lyra Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-ancientgame-time-windows-kpi",
									"alias": "AncientGame Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-citysample-time-windows-kpi",
									"alias": "CitySample Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-lyra-time-ps5-kpi",
									"alias": "Lyra Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-ancientgame-time-ps5-kpi",
									"alias": "AncientGame Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-citysample-time-ps5-kpi",
									"alias": "CitySample Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-nightly-cook-time-windows-kpi",
									"alias": "Nightly Cook Windows",
									"threshold": 7200
								},
								{
									"id": "core-cooking-nightly-cook-time-ps5-kpi",
									"alias": "Nightly Cook PS5",
									"threshold": 7200
								},
								{
									"id": "core-cooking-autotest-cook-time-kpi",
									"alias": "AutoTest Cook",
									"threshold": 4500
								},
								{
									"id": "core-cooking-autotest-fastcook-time-kpi",
									"alias": "AutoTest FastCook",
									"threshold": 1800
								}
							],
							"graph": "indicator",
							"max": 10800
						},
						{
							"name": "Cook Shaders Compiled (KPIs)",
							"display": "Value",
							"metrics": [
								{
									"id": "core-cooking-nightly-cook-shaders-compiled-windows-kpi",
									"alias": "Nightly Cook Windows",
									"threshold": 1000
								},
								{
									"id": "core-cooking-nightly-cook-shaders-compiled-ps5-kpi",
									"alias": "Nightly Cook PS5",
									"threshold": 1000
								},
								{
									"id": "core-cooking-autotest-cook-shaders-compiled-kpi",
									"alias": "AutoTest Cook",
									"threshold": 1000
								},
								{
									"id": "core-cooking-autotest-fastcook-shaders-compiled-kpi",
									"alias": "AutoTest FastCook",
									"threshold": 1000
								}
							],
							"graph": "indicator",
							"max": 2000
						},
						{
							"name": "Cook Time (KPIs)",
							"metrics": [
								{
									"id": "core-cooking-nightly-cook-time-windows-kpi",
									"alias": "Nightly Cook Windows"
								},
								{
									"id": "core-cooking-nightly-cook-time-ps5-kpi",
									"alias": "Nightly Cook PS5"
								},
								{
									"id": "core-cooking-autotest-cook-time-kpi",
									"alias": "AutoTest Cook"
								},
								{
									"id": "core-cooking-autotest-fastcook-time-kpi",
									"alias": "AutoTest FastCook"
								}
							]
						},
						{
							"name": "Cook Shaders Compiled (KPIs)",
							"display": "Value",
							"metrics": [
								{
									"id": "core-cooking-nightly-cook-shaders-compiled-windows-kpi",
									"alias": "Nightly Cook Windows"
								},
								{
									"id": "core-cooking-nightly-cook-shaders-compiled-ps5-kpi",
									"alias": "Nightly Cook PS5"
								},
								{
									"id": "core-cooking-autotest-cook-shaders-compiled-kpi",
									"alias": "AutoTest Cook"
								},
								{
									"id": "core-cooking-autotest-fastcook-shaders-compiled-kpi",
									"alias": "AutoTest FastCook"
								}
							]
						},
						{
							"name": "Incremental Cook Time",
							"metrics": [
								{
									"id": "core-cooking-incremental-cook-time"
								}
							]
						},
						{
							"name": "Incremental Cook Shaders Compiled",
							"metrics": [
								{
									"id": "core-cooking-incremental-shaders-compiled"
								}
							]
						},
						{
							"name": "Package And Test Cook Time",
							"metrics": [
								{
									"id": "core-cooking-package-and-test-cook-time"
								}
							]
						},
						{
							"name": "Package And Test Shaders Compiled",
							"display": "Value",
							"metrics": [
								{
									"id": "core-cooking-package-and-test-shaders-compiled"
								}
							]
						},
						{
							"name": "Nightly Cook Time",
							"metrics": [
								{
									"id": "core-cooking-nightly-cook-time"
								}
							]
						},
						{
							"name": "Nightly Cook Shaders Compiled",
							"display": "Value",
							"metrics": [
								{
									"id": "core-cooking-nightly-shaders-compiled"
								}
							]
						},
						{
							"name": "Standard Cook Time",
							"metrics": [
								{
									"id": "core-cooking-standard-cook-time"
								}
							]
						},
						{
							"name": "Standard Cook Shaders Compiled",
							"metrics": [
								{
									"id": "core-cooking-standard-shaders-compiled"
								}
							]
						}
					]
				},
				{
					"name": "Cache",
					"charts": [
						{
							"name": "User DDC Backend Efficiency",
							"display": "Ratio",
							"metrics": [
								{
									"id": "core-loading-user-totalddchitratio",
									"alias": "Root"
								},
								{
									"id": "core-loading-user-cloudlocalddchitratio",
									"alias": "Cloud"
								},
								{
									"id": "core-loading-user-zenlocalddchitratio",
									"alias": "Local"
								},
								{
									"id": "core-loading-user-sharedddchitratio",
									"alias": "Shared"
								}
							]
						},
						{
							"name": "User DDC Asset Efficiency",
							"display": "Ratio",
							"metrics": [
								{
									"id": "core-loading-user-ddc-asset-efficiency"
								}
							]
						}
					]
				},
				{
					"name": "Memory",
					"charts": [
						{
							"name": "Memory By System",
							"display": "Value",
							"metrics": [
								{
									"id": "core-memory-llm-systems"
								}
							]
						},
						{
							"name": "Memory By AssetClass",
							"display": "Value",
							"metrics": [
								{
									"id": "core-memory-llm-assetclasses"
								}
							]
						},
						{
							"name": "Memory By Asset",
							"display": "Value",
							"metrics": [
								{
									"id": "core-memory-llm-assets"
								}
							]
						}
					]
				},
				{
					"name": "Content",
					"charts": [
						{
							"name": "Content Size Total",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "core-content-totalsize"
								}
							]
						},
						{
							"name": "Content Size By Mount Point",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "core-content-totalsize-mp"
								}
							]
						},
						{
							"name": "Virtualization By Mount Point",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "core-content-virtualization-mp"
								}
							]
						}
					]
				},
				{
					"name": "Compile",
					"charts": [
						{
							"name": "Remote Distibution All (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-remote-usage-all"
								}
							]
						},
						{
							"name": "Cache Rate All (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-cache-rate-all"
								}
							]
						},
						{
							"name": "UB Duration",
							"display": "Value",
							"metrics": [
								{
									"id": "ubt-completed-duration",
									"alias": "FullDuration"
								}
							]
						},
						{
							"name": "Actions All (Sum)",
							"display": "Value",
							"metrics": [
								{
									"id": "ubt-uba-total-actions-all",
									"alias": "Total"
								},
								{
									"id": "ubt-uba-remote-actions-all",
									"alias": "Remote"
								}
							]
						},
						{
							"name": "Failure Rate All (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-failure-rate-all"
								}
							]
						},
						{
							"name": "WallTime (Average)",
							"metrics": [
								{
									"id": "ubt-uba-duration"
								}
							]
						},
						{
							"name": "Remote Distribution (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-remote-usage"
								}
							]
						},
						{
							"name": "Cache Rate (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-cache-rate-avg"
								}
							]
						},
						{
							"name": "Actions (Average)",
							"display": "Value",
							"metrics": [
								{
									"id": "ubt-uba-total-actions-avg",
									"alias": "Total"
								},
								{
									"id": "ubt-uba-remote-actions-avg",
									"alias": "Remote"
								}
							]
						},
						{
							"name": "Actions (Sum)",
							"display": "Value",
							"metrics": [
								{
									"id": "ubt-uba-total-actions",
									"alias": "Total"
								},
								{
									"id": "ubt-uba-remote-actions",
									"alias": "Remote"
								}
							]
						},
						{
							"name": "Failure Rate (Average)",
							"display": "Ratio",
							"metrics": [
								{
									"id": "ubt-uba-failure-rate"
								}
							]
						}
					]
				},
				{
					"name": "Build",
					"charts": [
						{
							"name": "Full Compile",
							"metrics": [
								{
									"id": "horde-monolithic-compile-jobs"
								}
							]
						},
						{
							"name": "Incremental Compile",
							"metrics": [
								{
									"id": "horde-incremental-compile-jobs"
								}
							]
						},
						{
							"name": "Incremental Build",
							"metrics": [
								{
									"id": "horde-incremental-build-jobs"
								},
								{
									"id": "horde-incremental-ci-jobs"
								}
							]
						},
						{
							"name": "AutoTest Build",
							"metrics": [
								{
									"id": "horde-autotest-ci-jobs"
								}
							]
						},
						{
							"name": "Package And Test Build",
							"metrics": [
								{
									"id": "horde-package-and-test-jobs"
								}
							]
						},
						{
							"name": "Nighty Build",
							"metrics": [
								{
									"id": "horde-nightly-build-jobs"
								}
							]
						}
					]
				},
				{
					"name": "Package",
					"charts": [
						{
							"name": "Size By Install Type",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "package-total-size"
								}
							]
						},
						{
							"name": "Size Per Package",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "package-size"
								}
							]
						}
					]
				},
				{
					"name": "Submit",
					"charts": [
						{
							"name": "Session Count",
							"display": "Value",
							"min": 0,
							"metrics": [
								{
									"id": "submit-tool-session-count"
								}
							]
						}
					]
				},
				{
					"name": "Debug",
					"charts": [
						{
							"name": "MountPoint Test",
							"display": "Value",
							"metrics": [
								{
									"id": "core-content-events"
								}
							]
						}
					]
				}
			]
		}
	]
}