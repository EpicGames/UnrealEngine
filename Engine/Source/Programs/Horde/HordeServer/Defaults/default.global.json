{
	// Settings for the dashboard
	"dashboard": {

		// Whether to show the default landing page with documentation when logging on to the server. Shows a per-user homepage if disabled, showing recent jobs, projects, and so on.
		"showLandingPage": true,

		"showCI": true,
		"showAgents": true,
		"showPerforceServers": true,
		"showDeviceManager": true,
		"showTests": true
	},

	// Settings for various plugins
	"plugins": {

		"compute": {
			"clusters": [
				{
					"id": "default",
					"namespaceId": "horde.compute"
				}
			],
			"networks": [
				{
					"id": "default",
					"cidrBlock": "0.0.0.0/0",
					"description": "Default network (matching all IPs)",
					"computeId": "default"
				}
			],
			"pools": [
				{
					"name": "Win-UE5",
					"condition": "Platform == 'Win64'",
					"color": "Blue"
				},
				{
					"name": "Win-UE5-GPU",
					"condition": "Platform == 'Win64' && Interactive == true",
					"color": "Blue"
				},
				{
					"name": "Mac-UE5",
					"condition": "Platform == 'Mac'",
					"color": "Green"
				},
				{
					"name": "Linux-UE5",
					"condition": "Platform == 'Linux'",
					"color": "Orange"
				}
			]
		},

		"storage": {
			"backends": [
				{
					"id": "default-backend",
					"type": "FileSystem",
					"baseDir": "Storage" // Default base directory is C:\ProgramData\HordeServer
				},
				{
					"id": "memory-backend", // Used for automated tests
					"type": "Memory"
				}
			],
			"namespaces": [
				{
					"id": "default",
					"backend": "default-backend"
				},
				{
					"id": "horde-artifacts",
					"prefix": "Artifacts/",
					"backend": "default-backend"
				},
				{
					"id": "horde-perforce",
					"prefix": "Perforce/",
					"backend": "default-backend"
				},
				// The 'horde-logs' namespace is used internally by Horde to store logs from the CI
				// and remote execution systems. It must be configured in order for Horde to function
				// correctly.
				{
					"id": "horde-logs",
					"prefix": "Logs/",
					"backend": "default-backend"
				},
				{
					"id": "horde-tools",
					"prefix": "Tools/",
					"backend": "default-backend"
				},
				{
					"id": "memory",
					"backend": "memory-backend",
					"enableAliases": true
				}
			]
		},

		"build": {
			"artifactTypes": [
				{
					"name": "step-output",
					"keepDays": 1
				}
			],
			"devices": {
				"platforms": [
					{
						"id": "android",
						"name": "Android"
					}
				],
				"pools": [
					{
						"id": "ue5",
						"name": "UE5",
						"poolType": "Automation"
					},
					{
						"id": "remote-ue5",
						"name": "Remote UE5",
						"poolType": "Shared"
					}
				]
			}
		},
		"exampleplugin": {
			"greetingMessage": "Hello from the Horde Example Plugin"
		}
	},

	// Default ACL groups
	"acl": {
		"entries": [
			{
				"claim": {
					"type": "http://epicgames.com/ue/horde/group", 
					"value": "View"
				},
				"profiles": [
					"default-read"
				]
			},
			{
				"claim": {
					"type": "http://epicgames.com/ue/horde/group", 
					"value": "Run"
				},
				"profiles": [
					"default-run"
				]
			}
		]
	}
}
