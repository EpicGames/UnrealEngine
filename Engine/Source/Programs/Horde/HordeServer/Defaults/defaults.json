// This file exists for backwards compatibility with older server versions. New installations should directly include the default.global.json file.
{
	"include": [
		{
			"path": "default.global.json"
		}
	]
}
