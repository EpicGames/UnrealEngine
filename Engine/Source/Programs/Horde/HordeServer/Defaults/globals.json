{
	// Schema version number for the config file. New Horde versions may bump this number and introduce schema changes. Upgrade info
	// will be printed to the log on startup.
	"version": 2,

	// Include default settings from the Horde installation folder
	"include": [
		{
			"path": "$(HordeDir)/Defaults/default.global.json"
		}
	],

	"plugins": {
		"build": {
			// Definitions for projects in the CI system
			"projects": [
				// Uncomment this to enable a default //UE5/Dev-Main-Horde stream with some simple CI jobs.
//				{
//					"id": "ue5",
//					"path": "ue5.project.json"
//				}
			],

			// Configuration for different Perforce servers. Horde allows running against multiple 'logical' Perforce servers, each of which may be accessed through different edge servers. Server
			// selection is performed to evenly distribute load, and servers can expose their health/status via a separate health check service.
			"perforceClusters": 
			[
//				{
//					// Name for this cluster. Streams can be configured to use a particular cluster using the ClusterName property in the stream config. 
//					"name": "Default",
//
//					// Default account to use inside the Horde Server for querying submitted changes etc..
//					// If this value is not specified, the first item from the credentials below will be used.
//					"serviceAccount": null,
//
//					// Whether the service account supports generating login tickets for other users, which requires a priviledged account on the server. Impersonating other users
//					// allows Horde to submit changes on behalf of users.
//					"canImpersonate": false,
//
//					// List of servers to access
//					"servers": 
//					[
//						{
//							// DNS name and port for the server.
//							// If connecting using "ssl:" prefix, ensure P4 server's fingerprint/certificate is trusted (see docs for "p4 trust")
//							// See Horde's documentation on connecting to SSL-enabled Perforce servers.
//							"serverAndPort": "perforce:1666"
//						}
//					],
//
//					// List of credentials for the server. The first credentials will be used by default, but can be overridden by setting the userName property in the workspace.
//					"credentials": 
//					[
//						{
//							"userName": "test.user",
//
//							// Specify either the password or ticket property below (but not both)
//							"password": "password123",
//							// "ticket": "ticket if not using password"
//						}
//					]
//				}
			]
		}
	},

	// Default deployment parameters for other tools bundled with Horde. These can be queried at runtime without auth using the default Horde server.
	"parameters": {
		"ugs": 
		{
			// Set the default Perforce server for anyone using UGS.
//			"defaultPerforceServer": "perforce:1666"
		}
	}
}
