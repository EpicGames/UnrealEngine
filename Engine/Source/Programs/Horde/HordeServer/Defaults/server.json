// Changes to this file requires a restart of the server
{
	"Horde": {

		// How to authorize users to log in to the server. Auth is disabled by default ("Anonymous"), but can also be configured to use accounts managed 
		// by Horde itself ("Horde") or to use an external OIDC auth provider ("OpenIdConnect" or "Okta").
		"AuthMethod": "Anonymous",

		// Path to read the global Horde configuration from. This can be a path on disk, or path to a file in Perforce if that is configured below.
		// The specified file is continuously monitored for updates and will be hot-reloaded without restarting the Horde server
		"ConfigPath": "globals.json",

		// List of Perforce servers for use specifically by the server (eg. for bootstrapping configuration read from P4, rather than executing jobs).
		// Configuration for the server may be stored in a more secure location than standard agents have access to.
		"Perforce": [
//			{
//				"id": "Default",
//				"serverAndPort": "perforce:1666",
//				"credentials": {
//					"userName": "test.user",
//					"password": "password (optional)",
//					"ticket": "ticket (optional)"
//				}
//			}
		],

		// Configure an editor telemetry sink that stores data in the bundled MongoDB instance.
		// This stores raw telemetry data; aggregated metrics can be computed dynamically without
		// needing to scan this data, and is expired separately.
		"Telemetry": [
			{
				"Type": "Mongo",
				"RetainDays": 1
			}
		],

		// Whether to use the local Perforce environment for reading configuration data
		"UseLocalPerforceEnv": false
	}
}
