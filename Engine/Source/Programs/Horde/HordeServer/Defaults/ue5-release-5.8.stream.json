{
	// Update this to the name of your Perforce stream
	"name": "//UE5/Release-5.8-HordeSync",

	// Macros for strings in the rest of this file (as $(Project) etc...)
	"macros":
	[
		{
			"name": "Project",
			"value": "Lyra"
		},
		{
			"name": "ProjectPath",
			"value": "Samples/Games/Lyra/Lyra.uproject"
		},
		{
			"name": "GameTarget",
			"value": "LyraGame"
		},
		{
			"name": "EditorTarget",
			"value": "LyraEditor"
		}
	],

	// Default job to run when triggering a preflight from P4VUtils (can be installed through UGS)
	"defaultPreflight":
	{
		"templateId": "default-presubmit-tests",

		// Set the base changelist number to the last incremental build that succeeded
		"change":
		{
			"templateId": "incremental-build",
			"target": "Incremental Build Minimal",
			"outcomes":
			[
				"Success",
				"Warnings"
			]
		}
	},

	// Tabs to display in the dashboard for different jobs. Each job needs to be added to a tab to be shown in the New Build dialog.
	"tabs":
	[
		{
			"title": "Incremental",
			"style": "Compact",
			"templates":
			[
				"incremental-build"
			],
			"columns":
			[
				{
					"heading": "Editors",
					"category": "Editors",
					"relativeWidth": 2
				},
				{
					"heading": "Target Platforms",
					"category": "Platforms",
					"relativeWidth": 4
				},
				{
					"heading": "Other",
					"category": "Other",
					"relativeWidth": 5
				}
			]
		},
		{
			"title": "Packaged Builds",
			"style": "Normal",
			"templates":
			[
				"packaged-build"
			],
			"columns":
			[
				{
					"heading": "Editors",
					"category": "Editors",
					"relativeWidth": 1
				},
				{
					"heading": "Clients",
					"category": "Clients",
					"relativeWidth": 1
				},
				{
					"heading": "Servers",
					"category": "Servers",
					"relativeWidth": 1
				},
				{
					"heading": "Tests",
					"category": "Tests",
					"relativeWidth": 1
				},
				{
					"heading": "Other",
					"category": "Other",
					"relativeWidth": 1
				}
			]
		},
		{
			"title": "Presubmit",
			"style": "Normal",
			"showPreflights": true,
			"templates":
			[
				"default-presubmit-tests"
			],
			"columns":
			[
				{
					"heading": "Editors",
					"category": "Editors",
					"relativeWidth": 1
				},
				{
					"heading": "Target Platforms",
					"category": "Platforms",
					"relativeWidth": 2
				},
				{
					"heading": "Other",
					"category": "Other",
					"relativeWidth": 1
				}
			]
		},
		{
			"title": "Utility",
			"style": "Normal",
			"templates":
			[
				"installed-build",
				"parameter-demo",
				"remote-execution-test",
				"test-executor",
				"custom"
			],
			"columns":
			[
				{
					"heading": "Editors",
					"category": "Editors",
					"relativeWidth": 1
				},
				{
					"heading": "Target Platforms",
					"category": "Platforms",
					"relativeWidth": 2
				},
				{
					"heading": "Other",
					"category": "Other",
					"relativeWidth": 1
				}
			]
		}
	],

	// Definitions for different agent types. Agent types are referenced by BuildGraph scripts, and map onto pools and workspace definitions configured here.
	"agentTypes":
	{
		"Win64":
		{
			"pool": "win-ue5",
			"workspace": "Full"
		},
		"Win64_Licensee": // Used for installed engine build
		{
			"pool": "win-ue5",
			"workspace": "Full"
		},
		"TestWin64":
		{
			"pool": "win-ue5-gpu",
			"workspace": "Full"
		},
		"IncrementalWin64":
		{
			"pool": "win-ue5",
			"workspace": "IncrementalCompileOnly"
		},
		"DevkitAutomation":
		{
			"pool": "win-devkitautomation",
			"workspace": "Full"			
		}
	},

	// Different workspace types, meaning Perforce client views that should be synced by particular agent types.
	"workspaceTypes":
	{
		"Full":
		{
			// Set a unique identifier to prevent overlap with other streams; will cause workspace to be synced to a different folder
			"identifier": "Demo",

			// Use the native Perforce library to avoid p4.exe dependency
			"method": "name=managedWorkspace&preferNativeClient=true"
		},
		"IncrementalCompileOnly":
		{
			// Set a unique identifier to prevent overlap with other streams; will cause workspace to be synced to a different folder
			"identifier": "Demo-Inc",

			// Don't clean the contents of the workspace after building
			"incremental": true,

			// Filter out files not needed to compilation. Wildcards for these rules is similar to P4 clientspecs.
			"view":
			[
				"-*.uasset",
				"-*.umap"
			],

			// Use the native Perforce library to avoid p4.exe dependency
			"method": "name=managedWorkspace&preferNativeClient=true"
		}
	},

	// Artifact types that can be uploaded for this stream, and their retention policies
	"artifactTypes": [
		{
			"type": "packaged-build",
			"keepDays": 14
		},
		{
			"type": "staged-build",
			"keepDays": 14
		},
		{
			"type": "ugs-pcb",
			"keepCount": 5
		}
	],

	// Templates for all the jobs that can be run, as well as any parameters to show in the dashboard's New Build dialog
	"templates":
	[
		{
			"id": "incremental-build",
			"name": "Incremental Build",
			"description": "Performs an incremental editor compile, and uploads the produced binaries to Horde for syncing with UnrealGameSync.",
			"initialAgentType": "Win64",
			"showUgsBadges": true,
			"arguments":
			[
				"-Script=Engine/Build/Graph/Examples/BuildEditorAndTools.xml",
				"-Target=Submit PCBs",
				"-set:UProjectPath=$(ProjectPath)",
				"-set:EditorTarget=$(EditorTarget)",
				"-set:PreferredAgent=IncrementalWin64;CompileWin64;Win64"
			]
		},
		{
			"id": "packaged-build",
			"name": "Packaged $(Project) Build",
			"description": "Creates a packaged build of the configured project, and runs tests for it.",
			"initialAgentType": "Win64",
			"arguments":
			[
				"-Script=Engine/Build/Graph/Tasks/BuildAndTestProject.xml",
				"-Target=BuildAndTest $(Project)",
				"-set:ProjectFile=$(ProjectPath)",
				"-set:TargetName=$(GameTarget)",
				"-set:EditorTargetName=$(EditorTarget)",
				"-set:WithDeviceReservation=false",
				"-set:PublishBuilds=false",
				"-set:PublishHordeArtifacts=true"				
			],
			"parameters":
			[
				{

					"type": "List",
					"label": "Editors",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Editor",
							"text": "Linux",
							"argumentIfEnabled": "-set:WithLinuxEditor=true",
							"default": false
						},
						{
							"group": "Editor",
							"text": "Mac",
							"argumentIfEnabled": "-set:WithMacEditor=true",
							"default": false
						},
						{
							"group": "Editor",
							"text": "Win64",
							"argumentIfEnabled": "-set:WithWin64Editor=true",
							"default": true
						}
					]
				},
				{
					"type": "List",
					"label": "Editor Tests",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Test List",
							"text": "Boot Test",
							"argumentIfEnabled": "-set:EditorBootTest=true",
							"argumentIfDisabled": "-set:EditorBootTest=false",
							"default": true
						},
						{
							"group": "Test List",
							"text": "PIE Test",
							"argumentIfEnabled": "-set:EditorTestList=UE.EditorAutomation(RunTest=Project.Maps.PIE)",
							"default": true
						},
						{
							"group": "Test List",
							"text": "Project Tests (RunTest=Project.*)",
							"argumentIfEnabled": "-set:EditorTestList=UE.EditorAutomation(RunTest=Project.)",
							"default": false
						}
					]
				},
				{
					"type": "List",
					"label": "Target Platforms",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Desktop",
							"text": "Win64",
							"argumentIfEnabled": "-set:WithWin64=true",
							"argumentIfDisabled": "-set:WithWin64=false",
							"default": true
						},
						{
							"group": "Desktop",
							"text": "Mac",
							"argumentIfEnabled": "-set:WithMac=true",
							"argumentIfDisabled": "-set:WithMac=false",
							"default": false
						},
						{
							"group": "Desktop",
							"text": "Linux",
							"argumentIfEnabled": "-set:WithLinux=true",
							"argumentIfDisabled": "-set:WithLinux=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "Android",
							"argumentIfEnabled": "-set:WithAndroid=true",
							"argumentIfDisabled": "-set:WithAndroid=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "IOS",
							"argumentIfEnabled": "-set:WithIOS=true",
							"argumentIfDisabled": "-set:WithIOS=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "TVOS",
							"argumentIfEnabled": "-set:WithTVOS=true",
							"argumentIfDisabled": "-set:WithTVOS=false",
							"default": false
						}
					]
				},
				{
					"type": "List",
					"label": "Target Configurations",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Build Configurations",
							"text": "Build Debug Executables",
							"argumentIfEnabled": "-set:WithDebugTargets=true",
							"default": false
						},
						{
							"group": "Build Configurations",
							"text": "Build Development Executables",
							"argumentIfEnabled": "-set:WithDevelopmentTargets=true",
							"default": true
						},
						{
							"group": "Build Configurations",
							"text": "Build Test Executables",
							"argumentIfEnabled": "-set:WithTestTargets=true",
							"default": false
						},
						{
							"group": "Build Configurations",
							"text": "Build Shipping Executables",
							"argumentIfEnabled": "-set:WithShippingTargets=true",
							"default": false
						}
					]
				},
				{
					"type": "List",
					"label": "Target Packages",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Package Configurations",
							"text": "Create Debug Packages",
							"argumentIfEnabled": "-set:WithDebugPackages=true",
							"default": false
						},
						{
							"group": "Package Configurations",
							"text": "Create Development Packages",
							"argumentIfEnabled": "-set:WithDevelopmentPackages=true",
							"default": true
						},
						{
							"group": "Package Configurations",
							"text": "Create Test Packages",
							"argumentIfEnabled": "-set:WithTestPackages=true",
							"default": false
						},
						{
							"group": "Package Configurations",
							"text": "Create Shipping Packages",
							"argumentIfEnabled": "-set:WithShippingPackages=true",
							"default": false
						}
					]
				},
				{
					"type": "List",
					"label": "Target Tests",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Configuration",
							"text": "Test Debug Packages",
							"argumentIfEnabled": "-set:WithDebugTests=true",
							"default": false
						},
						{
							"group": "Configuration",
							"text": "Test Development Packages",
							"argumentIfEnabled": "-set:WithDevelopmentTests=true",
							"default": true
						},
						{
							"group": "Configuration",
							"text": "Test Test Packages",
							"argumentIfEnabled": "-set:WithTestTests=true",
							"default": false
						},
						{
							"group": "Configuration",
							"text": "Test Shipping Packages",
							"argumentIfEnabled": "-set:WithShippingTests=true",
							"default": false
						},
						{
							"group": "Test List",
							"text": "Boot Test",
							"argumentIfEnabled": "-set:TargetBootTest=true",
							"argumentIfDisabled": "-set:TargetBootTest=false",
							"default": true
						},
						{
							"group": "Test List",
							"text": "Map Tests",
							"argumentIfEnabled": "-set:TargetTestList=UE.TargetAutomation(RunTest=Project.Maps.Cycle)",
							"default": false
						},
						{
							"group": "Test List",
							"text": "Project Tests (RunTest=Project.*)",
							"argumentIfEnabled": "-set:TargetTestList=UE.TargetAutomation(RunTest=Project.)",
							"default": false
						}
					]
				},
				{
					"type": "Text",
					"label": "Device Manager URL",
					"default": "http://localhost:13340",
					"argument": "-set:GauntletDeviceService="
				},
				{
					"type": "Text",
					"label": "Device Pool",
					"default": "UE5",
					"argument": "-set:GauntletDevicePool="
				}				
			]
		},
		{
			"id": "default-presubmit-tests",
			"name": "Default Presubmit Tests",
			"description": "Runs some quick tests on a shelved change, with the option to submit it if the job succeeds.",
			"arguments":
			[
				"-Target=Presubmit Tests",
				"-Script=Engine/Build/Graph/Examples/PresubmitTests.xml",
				"-set:ProjectPath=$(ProjectPath)",
				"-set:EditorTarget=$(EditorTarget)"
			]
		},
		{
			"id": "installed-build",
			"name": "Installed Engine Build",
			"description": "Creates a standalone build of the Unreal Engine that can be installed as a standalone SDK.",
			"initialAgentType": "Win64",
			"arguments":
			[
				"-Target=Installed Engine Build",
				"-Script=Engine/Build/InstalledEngineBuild.xml"
			],
			"parameters":
			[
				{
					"type": "List",
					"label": "Host Platforms",
					"style": "MultiList",
					"items":
					[
						{
							"text": "Win64",
							"argumentIfEnabled": "-set:WithInstalledWin64=true",
							"argumentIfDisabled": "-set:WithInstalledWin64=false",
							"default": true
						},
						{
							"text": "Mac",
							"argumentIfEnabled": "-set:WithInstalledMac=true",
							"argumentIfDisabled": "-set:WithInstalledMac=false",
							"default": false
						},
						{
							"text": "Linux",
							"argumentIfEnabled": "-set:WithInstalledLinux=true",
							"argumentIfDisabled": "-set:WithInstalledLinux=false",
							"default": false
						}
					]
				},
				{
					"type": "List",
					"label": "Target Platforms",
					"style": "MultiList",
					"items":
					[
						{
							"group": "Desktop",
							"text": "Win64",
							"argumentIfEnabled": "-set:WithWin64=true",
							"argumentIfDisabled": "-set:WithWin64=false",
							"default": false
						},
						{
							"group": "Desktop",
							"text": "Mac",
							"argumentIfEnabled": "-set:WithMac=true",
							"argumentIfDisabled": "-set:WithMac=false",
							"default": false
						},
						{
							"group": "Desktop",
							"text": "Linux",
							"argumentIfEnabled": "-set:WithLinux=true",
							"argumentIfDisabled": "-set:WithLinux=false",
							"default": false
						},
						{
							"group": "Desktop",
							"text": "LinuxArm64",
							"argumentIfEnabled": "-set:WithLinuxArm64=true",
							"argumentIfDisabled": "-set:WithLinuxArm64=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "Android",
							"argumentIfEnabled": "-set:WithAndroid=true",
							"argumentIfDisabled": "-set:WithAndroid=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "IOS",
							"argumentIfEnabled": "-set:WithIOS=true",
							"argumentIfDisabled": "-set:WithIOS=false",
							"default": false
						},
						{
							"group": "Mobile",
							"text": "TVOS",
							"argumentIfEnabled": "-set:WithTVOS=true",
							"argumentIfDisabled": "-set:WithTVOS=false",
							"default": false
						}
					]
				},
				{
					"type": "Bool",
					"label": "Build DDC",
					"argumentIfEnabled": "-set:WithDDC=true",
					"argumentIfDisabled": "-set:WithDDC=false",
					"default": true
				}
			]
		},
		{
			"id": "parameter-demo",
			"name": "Parameter Demo",
			"description": "Sample showing how to configure different parameter types and access them from BuildGraph scripts.",
			"initialAgentType": "Win64",
			"arguments":
			[
				"-Script=Engine/Build/Graph/Examples/Parameters.xml",
				"-Target=Print Parameters"
			],
			"parameters":
			[
				{
					"type": "Text",
					"label": "Text Parameter",
					"argument": "-set:TextParameter=",
					"default": "this-is-a-text-parameter",
					"validation": "[a-zA-Z0-9_-]+",
					"validationError": "Should be a sequence of alphanumeric characters",
					"hint": "Enter a string of alphanumeric characters",
					"toolTip": "Tooltip for text parameter"
				},
				{
					"type": "List",
					"label": "List Parameter (Default Style)",
					"toolTip": "Tooltip for list parameter",
					"items":
					[
						{
							"text": "Option 1",
							"argumentIfEnabled": "-set:ListParameter=Option1"
						},
						{
							"text": "Option 2",
							"argumentIfEnabled": "-set:ListParameter=Option2",
							"default": true
						},
						{
							"text": "Option 3",
							"argumentIfEnabled": "-set:ListParameter=Option3"
						}
					]
				},
				{
					"type": "List",
					"style": "MultiList",
					"label": "List Parameter (MultiList Style)",
					"toolTip": "Tooltip for list parameter (MultiList)",
					"items":
					[
						{
							"group": "First group",
							"text": "Option 1",
							"argumentIfEnabled": "-set:MultiListOption1=true",
							"default": true
						},
						{
							"group": "First group",
							"text": "Option 2",
							"argumentIfEnabled": "-set:MultiListOption2=true"
						},
						{
							"group": "Second group",
							"text": "Option 3",
							"argumentIfEnabled": "-set:MultiListOption3=true",
							"default": true
						},
						{
							"group": "Second group",
							"text": "Option 4",
							"argumentIfEnabled": "-set:MultiListOption4=true"
						}
					]
				},
				{
					"type": "List",
					"style": "TagPicker",
					"label": "List Parameter (TagPicker Style)",
					"toolTip": "Tooltip for list parameter (TagPicker)",
					"items":
					[
						{
							"text": "Option 1",
							"argumentIfEnabled": "-set:TagPickerOption1=true",
							"default": true
						},
						{
							"text": "Option 2",
							"argumentIfEnabled": "-set:TagPickerOption2=true"
						},
						{
							"text": "Option 3",
							"argumentIfEnabled": "-set:TagPickerOption3=true",
							"default": true
						},
						{
							"text": "Option 4",
							"argumentIfEnabled": "-set:TagPickerOption4=true"
						}
					]
				},
				{
					"type": "Bool",
					"label": "Bool Parameter",
					"toolTip": "Tooltip for bool parameter",
					"argumentIfEnabled": "-set:BoolParameter=true",
					"argumentIfDisabled": "-set:BoolParameter=false"
				}
			]
		},
		{
			"id": "remote-execution-test",
			"name": "Remote Execution Test",
			"description": "Builds a target with all compilation steps forced to execute remotely using Unreal Build Accelerator.",
			"arguments":
			[
				"-Script=Engine/Build/Graph/Tasks/RemoteExecutionTest.xml",
				"-Target=Compile Target"
			],
			"parameters":
			[
				{
					"type": "Text",
					"label": "Target",
					"argument": "-set:Target=",
					"default": "UnrealEditor"
				},
				{
					"type": "Text",
					"label": "Configuration",
					"argument": "-set:Configuration=",
					"default": "Development"
				},
				{
					"type": "Text",
					"label": "Platforms",
					"argument": "-set:Platforms=",
					"default": "Win64"
				},
				{
					"type": "Text",
					"label": "Arguments",
					"argument": "-set:Arguments=",
					"default": "-UBA -UBAForceRemote -UBAStrict"
				},
				{
					"type": "Bool",
					"label": "Clean before build",
					"argumentIfEnabled": "-set:Clean=true",
					"argumentIfDisabled": "-set:Clean=false",
					"default": true
				}
			]
		},
		{
			"id": "test-executor",
			"name": "Run Test Executor",
			"description": "Simulate a simple job without syncing any data from revision control.",
			"initialAgentType": "Win64",
			"arguments":
			[
				"-Target=Full Build",
				"-Project=$(Project)"
			],
			"parameters":
			[
				{
					"type": "Bool",
					"label": "Simulate Warning",
					"argumentIfEnabled": "-set:SimulateWarning=true",
					"argumentIfDisabled": "-set:SimulateWarning=false",
					"default": true
				},
				{
					"type": "Bool",
					"label": "Simulate Error",
					"argumentIfEnabled": "-set:SimulateError=true",
					"argumentIfDisabled": "-set:SimulateError=false",
					"default": true
				}
			],
			"jobOptions":
			{
				"executor": "Test"
			}
		},
		{
			"id": "custom",
			"name": "Custom...",
			"arguments":
			[
			],
			"parameters":
			[
				{
					"type": "Text",
					"label": "Script",
					"argument": "-Script=",
					"default": "Foo.xml"
				},
				{
					"type": "Text",
					"label": "Target",
					"argument": "-Target=",
					"default": ""
				}
			]
		}
	]
}
