{
	"name": "UE5",
	"logo": "ue5.project.png",
	"logoDarkTheme": "ue5.project-darktheme.png",
	"streams": [
		{
		  "id": "ue5-release-5.8",
		  "path": "ue5-release-5.8.stream.json"
		}
	],
	"categories": [
		{
			"name": "General",
			"row": 0,
			"showOnNavMenu": true,
			"includePatterns": [
				"//UE5/Release-5.8"
			]
		}
	]
}
