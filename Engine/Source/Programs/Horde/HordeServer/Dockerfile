#
# NOTE: This Dockerfile must be built with the Engine directory as CWD.
#       In the Horde build script we stage the required files before running:
#
#         docker build -f Source/Programs/Horde/HordeServer/Dockerfile . 
#
#       Running directly from the source tree will cause a lot of unnecessary
#       files to be uploaded to Docker before building.

ARG code_coverage="false"

FROM redis:7.2 AS redis

# Build image
FROM mcr.microsoft.com/dotnet/sdk:10.0 AS build-env
WORKDIR /app
COPY Source/Programs/Horde/nuget.config ./Source/Programs/Horde/nuget.config
RUN DEBIAN_FRONTEND=noninteractive \
	apt-get -qq update && \
	apt-get -qq install curl zip && \
	dotnet tool install --global JetBrains.dotCover.CommandLineTools && \
	ln -s /root/.dotnet/tools/dotCover /usr/local/bin/dotnet-dotcover

COPY --from=redis /usr/local/bin/redis-server /usr/local/bin/redis-server

# Since the .deb does not install in this image, just download it and extract the static binary
RUN wget https://repo.mongodb.org/apt/debian/dists/bookworm/mongodb-org/7.0/main/binary-amd64/mongodb-org-server_7.0.4_amd64.deb && \
	dpkg -x mongodb-org-server_7.0.4_amd64.deb /tmp/mongodb && \
	cp /tmp/mongodb/usr/bin/mongod /usr/local/bin/mongod

# Copy build dependencies
COPY Binaries/DotNET/EpicGames.Perforce.Native/ Binaries/DotNET/EpicGames.Perforce.Native/
COPY Source/Programs/Shared/ ./Source/Programs/Shared/
COPY Source/Programs/Horde/* Source/Programs/Horde/
COPY Source/Programs/Horde/Plugins/ Source/Programs/Horde/Plugins/
COPY Source/Programs/Horde/HordeServer/ Source/Programs/Horde/HordeServer/
COPY Source/Programs/Horde/HordeServer.Shared/ Source/Programs/Horde/HordeServer.Shared/

# Restore all the C# projects
RUN dotnet restore Source/Programs/Horde/HordeServer/HordeServer.csproj -p:WithHordeStorage=false

# Build the server
ARG msbuild_args
RUN dotnet publish Source/Programs/Horde/HordeServer/HordeServer.csproj -c Release -o out -p:WithHordeStorage=false $msbuild_args

# Copy test dependencies
COPY Source/Programs/Horde/Drivers/JobDriver/ Source/Programs/Horde/Drivers/JobDriver/
COPY Source/Programs/Horde/HordeAgent/ Source/Programs/Horde/HordeAgent/
COPY Source/Programs/Horde/HordeServer.Tests/ Source/Programs/Horde/HordeServer.Tests/
COPY Source/Programs/Horde/HordeServer.Tests.Shared/ Source/Programs/Horde/HordeServer.Tests.Shared/
COPY Source/Programs/Horde/Scripts/test.sh Source/Programs/Horde/Scripts/test.sh

# Run tests
ARG code_coverage
RUN bash Source/Programs/Horde/Scripts/test.sh

# Remove native libs not used on Linux x86_64
RUN rm -rf /app/out/runtimes/osx* && \
    rm -rf /app/out/runtimes/win* && \
    rm -rf /app/out/runtimes/linux-arm* && \
    rm -rf /app/out/runtimes/linux/native/libgrpc_csharp_ext.x86.so


# Placeholder stage that's used to produce smaller output (only specific files from build-env stage is copied)
# To retrieve code coverage reports, docker buildx can be run with `--target build-env-output --output type=local,dest=docker-output`
FROM scratch AS build-env-output
# Provide at least one file that exists (/tmp/empty), so COPY operation won't fail
COPY --from=build-env /tmp/empty /tmp/dotcover-report/*.zip /

# Create the runtime image
FROM mcr.microsoft.com/dotnet/aspnet:10.0 AS runtime
WORKDIR /app
ENV ASPNETCORE_URLS=""
ENV ASPNETCORE_HTTP_PORTS=""
COPY --from=build-env /app/out .
ENTRYPOINT [ "dotnet", "HordeServer.dll" ]
