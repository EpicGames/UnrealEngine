FROM horde-server-bare:latest
COPY Dashboard/ /app/DashboardApp/
COPY Server/win-x64/Tools/ /app/Tools/
COPY Server/win-x64/appsettings.json /app/