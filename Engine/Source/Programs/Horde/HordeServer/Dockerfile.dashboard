FROM horde-server-bare:latest
COPY --from=hordedashboard-public:latest /usr/share/nginx/html/ /app/DashboardApp/