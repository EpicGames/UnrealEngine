# Ignore everything
*
**

# Include necessary dependencies
!Source/Programs/Shared/**
!Source/Programs/AutomationTool/AutomationUtils/Matchers/**
!Source/Programs/UnrealBuildTool/Matchers/**
!Source/Programs/Horde/*
!Source/Programs/Horde/HordeAgent/**
!Source/Programs/Horde/HordeServer/**
!Source/Programs/Horde/HordeServer.Tests/**
!Source/Programs/Horde/Scripts/**
!Binaries/DotNET/EpicGames.Perforce.Native/**
!Source/Programs/Horde/nuget.config

# Ensure bin/obj files generated by .NET builds on the host are ignored
**/bin
**/obj
**/uatbin
**/uatobj

# Ignore Visual Studio directory
**/.vs

# Exclude local config files
appsettings.Local.json
