// Copyright Epic Games, Inc. All Rights Reserved.

// This file is used by Code Analysis to maintain SuppressMessage
// attributes that are applied to this project.
// Project-level suppressions either have no target or are given
// a specific target and scoped to a namespace, type, member, etc.

using System.Diagnostics.CodeAnalysis;

[assembly: SuppressMessage("Reliability", "CA2007:Consider calling ConfigureAwait on the awaited task", Justification = "Not necessary in NET Core; no SynchronizationContext.", Scope = "module")]
[assembly: SuppressMessage("Usage", "CA2227:Collection properties should be read only", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1062:Validate arguments of public methods", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Globalization", "CA1303:Do not pass literals as localized parameters", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Performance", "CA1819:Properties should not return arrays", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Naming", "CA1716:Identifiers should not match keywords", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1030:Use events where appropriate", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1031:Do not catch general exception types", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1032:Implement standard exception constructors", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1056:URI-like properties should not be strings", Justification = "<Pending>", Scope = "module")]
[assembly: SuppressMessage("Design", "CA1055:URI-like return values should not be strings", Justification = "<Pending>", Scope = "module")]
