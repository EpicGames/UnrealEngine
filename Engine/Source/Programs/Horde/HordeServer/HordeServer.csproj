<Project Sdk="Microsoft.NET.Sdk.Web">

  <PropertyGroup>
    <AssemblyName>HordeServer</AssemblyName>
    <DocumentationFile>HordeServer.xml</DocumentationFile>
    <ApplicationIcon>HordeServer.ico</ApplicationIcon>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.AutoScaling" Version="3.7.301.9" />
    <PackageReference Include="AWSSDK.CloudWatch" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.EC2" Version="3.7.306.6" />
    <PackageReference Include="AWSSDK.Extensions.NETCore.Setup" Version="3.7.300" />
    <PackageReference Include="AWSSDK.S3" Version="3.7.304.1" />
    <PackageReference Include="AWSSDK.SimpleSystemsManagement" Version="3.7.304.24" />
    <PackageReference Include="AWSSDK.SQS" Version="3.7.300.12" />
    <PackageReference Include="AWSSDK.SecurityToken" Version="3.7.300.13" />
    <PackageReference Include="Azure.Storage.Blobs" Version="12.19.1" />
    <PackageReference Include="Crc32.NET" Version="1.2.0" />
    <PackageReference Include="Grpc.AspNetCore" Version="2.59.0" />
    <PackageReference Include="Grpc.AspNetCore.Server.Reflection" Version="2.59.0" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="JetBrains.Profiler.SelfApi" Version="2.5.0" />
    <PackageReference Include="JsonPath.Net" Version="0.6.8" />
    <PackageReference Include="Meziantou.Analyzer" Version="2.0.220">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.AspNetCore.Authentication.JwtBearer" Version="10.0.6" />
    <PackageReference Include="Microsoft.AspNetCore.Authentication.OpenIdConnect" Version="10.0.6" />
    <PackageReference Include="Microsoft.AspNetCore.DataProtection.StackExchangeRedis" Version="10.0.6" />
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Razor.RuntimeCompilation" Version="10.0.6" />
    <PackageReference Include="Microsoft.AspNetCore.SpaServices.Extensions" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Hosting.WindowsServices" Version="10.0.6" />
    <PackageReference Include="Microsoft.Extensions.Http.Polly" Version="10.0.6" />
    <PackageReference Include="Microsoft.VisualStudio.Threading.Analyzers" Version="17.14.15">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="MongoDB.Driver" Version="2.22.0" />
    <PackageReference Include="OpenTelemetry" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.Console" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Exporter.OpenTelemetryProtocol" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Extensions.Hosting" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.AspNetCore" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.GrpcNetClient" Version="1.8.0-beta.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.Http" Version="1.8.1" />
    <PackageReference Include="OpenTelemetry.Instrumentation.StackExchangeRedis" Version="1.0.0-rc9.14" />
    <PackageReference Include="Polly" Version="8.2.1" />
    <PackageReference Include="Polly.Extensions.Http" Version="3.0.0" />
    <PackageReference Include="Serilog.AspNetCore" Version="8.0.3" />
    <PackageReference Include="Serilog.Exceptions.Grpc" Version="8.4.0" />
    <PackageReference Include="Serilog.Extensions.Hosting" Version="8.0.0" />
    <PackageReference Include="Serilog.Sinks.Console" Version="5.0.1" />
    <PackageReference Include="Serilog.Sinks.File" Version="5.0.0" />
    <PackageReference Include="SerilogAnalyzer" Version="0.15.0" />
    <PackageReference Include="SharpZipLib" Version="1.4.2" />
    <PackageReference Include="StackExchange.Redis" Version="2.8.58" />
    <PackageReference Include="Swashbuckle.AspNetCore" Version="6.5.0" />
    <PackageReference Include="System.Data.Odbc" Version="10.0.6" />
    <PackageReference Include="T-Digest.NET" Version="1.0.3" />
    <PackageReference Include="TimeZoneConverter" Version="6.1.0" />
    <PackageReference Include="ZstdSharp.Port" Version="0.8.6" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="..\..\Shared\EpicGames.Horde.Common\Protos\**\*.*">
      <Link>Protos\%(RecursiveDir)%(FileName)%(Extension)</Link>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <None Remove="HordeServer.xml" />
  </ItemGroup>

  <ItemGroup>
    <Protobuf Include="Protos\grpc\health\v1\health.proto" ProtoRoot="Protos" GrpcServices="Server" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.AspNet\EpicGames.AspNet.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.BuildGraph\EpicGames.BuildGraph.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Oodle\EpicGames.Oodle.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Serialization\EpicGames.Serialization.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Slack\EpicGames.Slack.csproj" />
    <ProjectReference Include="..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\Plugins\Analytics\HordeServer.Analytics\HordeServer.Analytics.csproj" />
    <ProjectReference Include="..\Plugins\Build\HordeServer.Build\HordeServer.Build.csproj" />
    <ProjectReference Include="..\Plugins\ClickHouseTelemetry\HordeServer.Epic.ClickHouseTelemetry\HordeServer.Epic.ClickHouseTelemetry.csproj" />
    <ProjectReference Include="..\Plugins\Compute\HordeServer.Compute\HordeServer.Compute.csproj" />
    <ProjectReference Include="..\Plugins\ExamplePlugin\HordeServer.ExamplePlugin\HordeServer.ExamplePlugin.csproj" />
    <ProjectReference Include="..\Plugins\Experimental\HordeServer.Experimental\HordeServer.Experimental.csproj" />
    <ProjectReference Include="..\Plugins\OdbcTelemetry\HordeServer.OdbcTelemetry\HordeServer.OdbcTelemetry.csproj" />
    <ProjectReference Include="..\Plugins\PerformanceTrends\HordeServer.PerformanceTrends\HordeServer.PerformanceTrends.csproj" />
    <ProjectReference Include="..\Plugins\Secrets\HordeServer.Secrets\HordeServer.Secrets.csproj" />
    <ProjectReference Include="..\Plugins\Storage\HordeServer.Storage\HordeServer.Storage.csproj" />
    <ProjectReference Include="..\Plugins\StructuredAnalytics\HordeServer.StructuredAnalytics\HordeServer.StructuredAnalytics.csproj" />
    <ProjectReference Include="..\Plugins\Symbols\HordeServer.Symbols\HordeServer.Symbols.csproj" />
    <ProjectReference Include="..\Plugins\Tools\HordeServer.Tools\HordeServer.Tools.csproj" />
  </ItemGroup>

  <ItemGroup Condition="Exists('$(MSBuildThisFileDirectory)..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\EpicPlugins\BuildHealth\HordeServer.Epic.BuildHealth\HordeServer.Epic.BuildHealth.csproj')">
    <ProjectReference Include="..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\EpicPlugins\BuildHealth\HordeServer.Epic.BuildHealth\HordeServer.Epic.BuildHealth.csproj" />
  </ItemGroup>

  <ItemGroup Condition="Exists('$(MSBuildThisFileDirectory)..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\EpicPlugins\EpicSandbox\HordeServer.Epic.EpicSandbox\HordeServer.Epic.EpicSandbox.csproj')">
    <ProjectReference Include="..\..\..\..\Restricted\NotForLicensees\Source\Programs\Horde\EpicPlugins\EpicSandbox\HordeServer.Epic.EpicSandbox\HordeServer.Epic.EpicSandbox.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Content Include="..\ThirdParty\**" Condition="$([MSBuild]::IsOSPlatform('Windows'))">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
      <Link>ThirdParty/%(RecursiveDir)/%(Filename)%(Extension)</Link>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <None Include="DashboardApp\**">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </None>
  </ItemGroup>

  <ItemGroup>
    <Content Include="Defaults\ue5.project.png">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
    <Content Include="Defaults\ue5.project-darktheme.png">
      <CopyToOutputDirectory>Always</CopyToOutputDirectory>
    </Content>
  </ItemGroup>

  <ItemGroup>
    <Content Update="appsettings.Local.json" CopyToPublishDirectory="Never" />
  </ItemGroup>

</Project>
