// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.Json.Nodes;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Parameters
{
	/// <summary>
	/// Controller for the /api/v1/parameters endpoint. Provides configuration data to other tools.
	/// </summary>
	[ApiController]
	[AllowAnonymous]
	public class ParametersController : HordeControllerBase
	{
		private readonly IOptionsSnapshot<GlobalConfig> _globalConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public ParametersController(IOptionsSnapshot<GlobalConfig> globalConfig)
		{
			_globalConfig = globalConfig;
		}

		/// <summary>
		/// Query side-wide parameters published for automatic configuration of external tools.
		/// </summary>
		/// <param name="path">Base path for the object to return</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Parameters matching the requested filter</returns>
		[HttpGet]
		[Route("/api/v1/parameters/{*path}")]
		[ProducesResponseType(typeof(object), 200)]
		public ActionResult<object> GetParameters(string? path = null, [FromQuery] PropertyFilter? filter = null)
		{
			JsonObject? parameters = _globalConfig.Value.Parameters;
			if (parameters != null && !String.IsNullOrEmpty(path))
			{
				foreach (string fragment in path.Split('/'))
				{
					parameters = parameters[fragment] as JsonObject;
					if (parameters == null)
					{
						break;
					}
				}
			}
			return PropertyFilter.Apply(parameters ?? new JsonObject(), filter);
		}
	}
}
