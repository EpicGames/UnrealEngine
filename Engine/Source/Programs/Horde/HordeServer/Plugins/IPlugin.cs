// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface for manipulating a plugin
	/// </summary>
	public interface IPlugin
	{
		/// <summary>
		/// Name of the plugin
		/// </summary>
		PluginName Name { get; }

		/// <summary>
		/// Metadata for the plugin
		/// </summary>
		IPluginMetadata Metadata { get; }

		/// <summary>
		/// Load this plugin
		/// </summary>
		ILoadedPlugin Load();
	}

	/// <summary>
	/// Plugin which has been loaded
	/// </summary>
	public interface ILoadedPlugin : IPlugin
	{
		/// <summary>
		/// Assembly containing the plugin's implementation
		/// </summary>
		Assembly Assembly { get; }

		/// <summary>
		/// Type used for the server config.
		/// </summary>
		Type ServerConfigType { get; }

		/// <summary>
		/// Type used for the global config.
		/// </summary>
		Type GlobalConfigType { get; }

		/// <summary>
		/// Configure services for the plugin
		/// </summary>
		/// <param name="configuration">Static configuration data</param>
		/// <param name="serverInfo">Information about the server</param>
		/// <param name="serviceCollection">Service collection instance</param>
		void ConfigureServices(IConfiguration configuration, IServerInfo serverInfo, IServiceCollection serviceCollection);
	}

	/// <summary>
	/// Extension methods for plugins
	/// </summary>
	public static class PluginExtensions
	{
		/// <summary>
		/// Adds a plugin's services to a service collection
		/// </summary>
		public static void AddPlugin(this IServiceCollection serviceCollection, IServerInfo serverInfo, ILoadedPlugin plugin, IConfiguration configuration)
			=> plugin.ConfigureServices(configuration, serverInfo, serviceCollection);
	}
}
