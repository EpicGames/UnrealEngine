// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Interface for querying the state of plugins on the server
	/// </summary>
	public interface IPluginCollection
	{
		/// <summary>
		/// List of available plugins
		/// </summary>
		IReadOnlyList<IPlugin> Plugins { get; }

		/// <summary>
		/// List of the enabled plugins
		/// </summary>
		IReadOnlyList<ILoadedPlugin> LoadedPlugins { get; }
	}
}
