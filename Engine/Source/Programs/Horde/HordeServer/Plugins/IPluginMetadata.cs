// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Information about a Horde plugin
	/// </summary>
	public interface IPluginMetadata
	{
		/// <summary>
		/// Name of the plugin
		/// </summary>
		PluginName Name { get; }

		/// <summary>
		/// Description of the plugin
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Other plugins that this plugin depends on
		/// </summary>
		IReadOnlyList<string> DependsOn { get; }

		/// <summary>
		/// Unique implementations of singleton features which this plugin provides
		/// </summary>
		IReadOnlyList<string>? ImplementsSingletons { get; }
	}

	/// <summary>
	/// Information about a Horde plugin
	/// </summary>
	public class PluginMetadata : IPluginMetadata
	{
		/// <inheritdoc/>
		public PluginName Name { get; set; }

		/// <inheritdoc/>
		public string Description { get; set; } = String.Empty;

		/// <inheritdoc/>
		public List<string> DependsOn { get; set; } = new List<string>();
		IReadOnlyList<string> IPluginMetadata.DependsOn => DependsOn;

		/// <inheritdoc/>
		public List<string>? ImplementsSingletons { get; set; }
		IReadOnlyList<string>? IPluginMetadata.ImplementsSingletons => ImplementsSingletons;
	}
}
