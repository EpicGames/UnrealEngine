// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Reflection;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Concrete implementation of <see cref="IPluginCollection"/>
	/// </summary>
	public class PluginCollection : IPluginCollection
	{
		// Plugin which is compiled into the application or already loaded
		class StaticPlugin : IPlugin
		{
			public PluginName Name => _metadata.Name;
			public IPluginMetadata Metadata => _metadata;

			readonly IPluginMetadata _metadata;
			readonly ILoadedPlugin _loadedPlugin;

			public StaticPlugin(IPluginMetadata metadata, ILoadedPlugin loadedPlugin)
			{
				_metadata = metadata;
				_loadedPlugin = loadedPlugin;
			}

			public ILoadedPlugin Load()
				=> _loadedPlugin;
		}

		[DebuggerDisplay("{Name}")]
		class LoadedPlugin<TServerConfig, TGlobalConfig, TStartup> : ILoadedPlugin
			where TServerConfig : PluginServerConfig, new()
			where TGlobalConfig : class, IPluginConfig, new()
			where TStartup : class, IPluginStartup
		{
			readonly IPluginMetadata _metadata;

			public PluginName Name => _metadata.Name;
			public IPluginMetadata Metadata => _metadata;
			public Assembly Assembly => typeof(TStartup).Assembly;
			public Type ServerConfigType => typeof(TServerConfig);
			public Type GlobalConfigType => typeof(TGlobalConfig);

			public LoadedPlugin(IPluginMetadata metadata)
				=> _metadata = metadata;

			public void ConfigureServices(IConfiguration config, IServerInfo serverInfo, IServiceCollection serviceCollection)
			{
				serviceCollection.Configure<TServerConfig>(config);
				serviceCollection.AddPluginConfig<TGlobalConfig>(Name);

				TStartup startup = CreateStartup(config, serverInfo);
				serviceCollection.AddSingleton<TStartup>(startup);
				serviceCollection.AddSingleton<IPluginStartup>(startup);

				startup.ConfigureServices(serviceCollection);
			}

			static TStartup CreateStartup(IConfiguration configuration, IServerInfo serverInfo)
			{
				ConstructorInfo? chosenConstructor = null;
				ParameterInfo[]? chosenConstructorParams = null;

				ConstructorInfo[] constructors = typeof(TStartup).GetConstructors(BindingFlags.Public | BindingFlags.Instance);
				foreach (ConstructorInfo constructor in constructors)
				{
					ParameterInfo[] parameters = constructor.GetParameters();
					if (IsValidConstructor(parameters))
					{
						if (chosenConstructorParams == null || parameters.Length > chosenConstructorParams.Length)
						{
							chosenConstructor = constructor;
							chosenConstructorParams = parameters;
						}
					}
				}

				TStartup startup;
				if (chosenConstructor == null || chosenConstructorParams == null)
				{
					startup = Activator.CreateInstance<TStartup>();
				}
				else
				{
					object[] arguments = new object[chosenConstructorParams.Length];
					for (int idx = 0; idx < chosenConstructorParams.Length; idx++)
					{
						ParameterInfo parameter = chosenConstructorParams[idx];
						if (parameter.ParameterType == typeof(IConfiguration))
						{
							arguments[idx] = configuration;
						}
						else if (parameter.ParameterType == typeof(IServerInfo))
						{
							arguments[idx] = serverInfo;
						}
						else if (parameter.ParameterType == typeof(TServerConfig))
						{
							arguments[idx] = new TServerConfig();
							configuration.Bind(arguments[idx]);
						}
						else
						{
							throw new NotImplementedException();
						}
					}

					startup = (TStartup)chosenConstructor.Invoke(arguments);
				}

				return startup;
			}

			static bool IsValidConstructor(ParameterInfo[] parameters)
			{
				foreach (ParameterInfo parameter in parameters)
				{
					if (parameter.ParameterType != typeof(IServerInfo)
						&& parameter.ParameterType != typeof(TServerConfig))
					{
						return false;
					}
				}
				return true;
			}

			public ILoadedPlugin Load()
				=> this;
		}

		/// <inheritdoc/>
		public IReadOnlyList<IPlugin> Plugins => _plugins;

		/// <inheritdoc/>
		public IReadOnlyList<ILoadedPlugin> LoadedPlugins => _loadedPlugins;

		readonly List<IPlugin> _plugins = new List<IPlugin>();
		readonly List<ILoadedPlugin> _loadedPlugins = new List<ILoadedPlugin>();
		readonly HashSet<PluginName> _loadedPluginNames = new HashSet<PluginName>();

		/// <summary>
		/// Adds a plugin with the given startup class
		/// </summary>
		public ILoadedPlugin Add(Type startupType)
		{
			PluginAttribute attr = startupType.GetCustomAttribute<PluginAttribute>()
				?? throw new InvalidOperationException($"Cannot add {startupType.Name} as a plugin. No {nameof(PluginAttribute)} was found.");

			IPluginMetadata metadata = new PluginMetadata { Name = new PluginName(attr.Name), DependsOn = [..attr.DependsOn] };
			if (!_loadedPluginNames.Add(metadata.Name))
			{
				throw new InvalidOperationException($"An implementation of the {metadata.Name} plugin has already been added");
			}

			Type pluginType = typeof(LoadedPlugin<,,>).MakeGenericType(attr.ServerConfigType ?? typeof(PluginServerConfig), attr.GlobalConfigType ?? typeof(EmptyPluginConfig), startupType);
			ILoadedPlugin loadedPlugin = (ILoadedPlugin)Activator.CreateInstance(pluginType, metadata)!;
			_loadedPlugins.Add(loadedPlugin);

			return loadedPlugin;
		}

		private enum DependencyNodeState
		{
			None,
			InProgress,
			Done,
		}

		private class DependencyNode(ILoadedPlugin plugin, IReadOnlyList<PluginName> dependsOn)
		{
			public readonly ILoadedPlugin Plugin = plugin;
			public readonly IReadOnlyList<PluginName> DependsOn = dependsOn;
			public DependencyNodeState State { get; set; } = DependencyNodeState.None;
		}

		private static void DependencyNodeVisit(DependencyNode current, Dictionary<PluginName, DependencyNode> nodes, List<ILoadedPlugin> result)
		{
			if (current.State == DependencyNodeState.Done)
			{
				return;
			}
			if (current.State == DependencyNodeState.InProgress)
			{
				throw new InvalidOperationException($"Cycle found in plugins for '{current.Plugin.Name}' during dependency sort");
			}
			current.State = DependencyNodeState.InProgress;

			foreach (PluginName dependsOn in current.DependsOn)
			{
				if (nodes.TryGetValue(dependsOn, out DependencyNode? next))
				{
					DependencyNodeVisit(next, nodes, result);
				}
			}

			current.State = DependencyNodeState.Done;
			result.Add(current.Plugin);
		}

		/// <summary>
		/// Returns a topological sorted sequence of plugins by plugins they depend upon
		/// </summary>
		public static IReadOnlyList<ILoadedPlugin> GetTopologicalSort(IReadOnlyList<ILoadedPlugin> plugins)
		{
			List<ILoadedPlugin> result = [];
			Dictionary<PluginName, DependencyNode> nodes = new();

			foreach (ILoadedPlugin plugin in plugins)
			{
				List<PluginName> dependsOn = [];
				foreach (string pluginName in plugin.Metadata.DependsOn)
				{
					dependsOn.Add(new PluginName(pluginName));
				}
				DependencyNode node = new(plugin, dependsOn);
				nodes.Add(plugin.Name, node);
			}

			foreach (DependencyNode node in nodes.Values)
			{
				DependencyNodeVisit(node, nodes, result);
			}
			return result;
		}

		/// <summary>
		/// Adds a plugin with the given startup class
		/// </summary>
		/// <typeparam name="T">Type of the startup class</typeparam>
		public ILoadedPlugin Add<T>() where T : class, IPluginStartup
			=> Add(typeof(T));
	}
}
