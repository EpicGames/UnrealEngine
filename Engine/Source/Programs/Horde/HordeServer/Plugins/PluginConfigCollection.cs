// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Horde;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Collection of <see cref="IPluginConfig"/> objects.
	/// </summary>
	[JsonConverter(typeof(PluginConfigCollectionConverter))]
	public class PluginConfigCollection : Dictionary<PluginName, IPluginConfig>
	{
		/// <summary>
		/// Attempts to get a plugin config of a specific type
		/// </summary>
		public bool TryGetValue<T>(PluginName name, [NotNullWhen(true)] out T? value) where T : class, IPluginConfig
		{
			IPluginConfig? config;
			if (base.TryGetValue(name, out config) && config is T typedConfig)
			{
				value = typedConfig;
				return true;
			}
			else
			{
				value = null;
				return false;
			}
		}
	}

	/// <summary>
	/// Serializer to and from strongly typed plugin config classes
	/// </summary>
	class PluginConfigCollectionConverter : JsonConverter<PluginConfigCollection>
	{
		/// <summary>
		/// Map of plugin names to the expected config type
		/// </summary>
		public Dictionary<PluginName, Type> NameToType { get; } = new Dictionary<PluginName, Type>();

		/// <inheritdoc/>
		public override PluginConfigCollection? Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
		{
			if (reader.TokenType != JsonTokenType.StartObject)
			{
				throw new JsonException("Expected an object for plugin config type");
			}

			PluginConfigCollection result = new PluginConfigCollection();
			while (reader.Read() && reader.TokenType == JsonTokenType.PropertyName)
			{
				string? nameStr = reader.GetString();
				if (nameStr == null || !reader.Read())
				{
					throw new JsonException("Invalid plugin name");
				}

				PluginName name = new PluginName(new StringId(nameStr));
				if (NameToType.TryGetValue(name, out Type? type))
				{
					object? obj = JsonSerializer.Deserialize(ref reader, type, options);
					if (obj != null)
					{
						result.Add(name, (IPluginConfig)obj);
					}
				}
				else
				{
					reader.Skip();
				}
			}

			if (reader.TokenType != JsonTokenType.EndObject)
			{
				throw new JsonException("Invalid token while parsing plugin config");
			}
			return result;
		}

		/// <inheritdoc/>
		public override void Write(Utf8JsonWriter writer, PluginConfigCollection value, JsonSerializerOptions options)
		{
			writer.WriteStartObject();
			foreach ((PluginName name, IPluginConfig config) in value)
			{
				writer.WritePropertyName(name.ToString());
				JsonSerializer.Serialize(writer, config, config.GetType(), options);
			}
			writer.WriteEndObject();
		}
	}
}
