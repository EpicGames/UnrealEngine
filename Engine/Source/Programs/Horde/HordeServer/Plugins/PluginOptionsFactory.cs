// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using HordeServer.Server;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;

namespace HordeServer.Plugins
{
	/// <summary>
	/// Factory for accessing plugin config instances
	/// </summary>
	/// <typeparam name="T">Type of the options object to return</typeparam>
	sealed class PluginOptionsFactory<T> : IOptionsFactory<T>, IOptionsChangeTokenSource<T>
		where T : class, IPluginConfig, new()
	{
		readonly PluginName _pluginName;
		readonly IOptionsMonitor<GlobalConfig> _globalConfig;
		readonly IOptionsChangeTokenSource<GlobalConfig> _globalConfigChangeTokenSource;

		/// <inheritdoc/>
		string IOptionsChangeTokenSource<T>.Name => String.Empty;

		/// <summary>
		/// Constructor
		/// </summary>
		public PluginOptionsFactory(PluginName pluginName, IServiceProvider serviceProvider)
			: this(pluginName, serviceProvider.GetRequiredService<IOptionsMonitor<GlobalConfig>>(), serviceProvider.GetRequiredService<IOptionsChangeTokenSource<GlobalConfig>>())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public PluginOptionsFactory(PluginName pluginName, IOptionsMonitor<GlobalConfig> globalConfig, IOptionsChangeTokenSource<GlobalConfig> globalConfigChangeTokenSource)
		{
			_pluginName = pluginName;
			_globalConfig = globalConfig;
			_globalConfigChangeTokenSource = globalConfigChangeTokenSource;
		}

		/// <inheritdoc/>
		public T Create(string name)
			=> (T)_globalConfig.CurrentValue.Plugins[_pluginName];

		/// <inheritdoc/>
		public IChangeToken GetChangeToken()
			=> _globalConfigChangeTokenSource.GetChangeToken();
	}

	/// <summary>
	/// Helper methods for registering 
	/// </summary>
	public static class PluginOptionsFactoryExtensions
	{
		/// <summary>
		/// Register an options factory for the given plugin options type
		/// </summary>
		public static void AddPluginConfig<T>(this IServiceCollection services, PluginName name) where T : class, IPluginConfig, new()
		{
			services.AddSingleton<PluginOptionsFactory<T>>(sp => new PluginOptionsFactory<T>(name, sp));
			services.AddSingleton<IOptionsFactory<T>>(sp => sp.GetRequiredService<PluginOptionsFactory<T>>());
			services.AddSingleton<IOptionsChangeTokenSource<T>>(sp => sp.GetRequiredService<PluginOptionsFactory<T>>());
		}
	}
}
