// Copyright Epic Games, Inc. All Rights Reserved.

using System.Resources;
using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("HordeServer.Tests")]
[assembly: InternalsVisibleTo("HordeServer.Tests.Shared")]
[assembly: NeutralResourcesLanguage("en-US")]
