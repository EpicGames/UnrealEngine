<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121.
-->
<Project>
  <PropertyGroup>
    <DeleteExistingFiles>true</DeleteExistingFiles>
    <ExcludeApp_Data>false</ExcludeApp_Data>
    <LaunchSiteAfterPublish>true</LaunchSiteAfterPublish>
    <LastUsedBuildConfiguration>Release</LastUsedBuildConfiguration>
    <LastUsedPlatform>Any CPU</LastUsedPlatform>
    <PublishProvider>FileSystem</PublishProvider>
    <PublishUrl>bin\Release\net6.0\publish\</PublishUrl>
    <WebPublishMethod>FileSystem</WebPublishMethod>
    <_TargetId>Folder</_TargetId>
    <SiteUrlToLaunchAfterPublish />
    <TargetFramework>net6.0</TargetFramework>
    <ProjectGuid>199841ba-0c0d-4019-9795-08618f7cd0d3</ProjectGuid>
    <SelfContained>false</SelfContained>
  </PropertyGroup>
</Project>