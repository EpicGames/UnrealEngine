<?xml version="1.0" encoding="utf-8"?>
<!--
https://go.microsoft.com/fwlink/?LinkID=208121.
-->
<Project>
  <PropertyGroup>
    <_PublishTargetUrl>C:\P4 Horde\Engine\Source\Programs\Horde\Horde.Build\bin\Release\net6.0\publish\</_PublishTargetUrl>
    <History>True|2022-10-25T14:32:02.3283843Z;</History>
    <LastFailureDetails />
  </PropertyGroup>
</Project>