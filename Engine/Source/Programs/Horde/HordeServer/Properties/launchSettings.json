{
  "profiles": {
    "HordeServer": {
      "commandName": "Project",
      "commandLineArgs": "",
      "environmentVariables": {
        "ASPNETCORE_ENVIRONMENT": "Local",
        "Horde__Perforce__1__Id": "psyonix"
      },
      "hotReloadEnabled": false
    }
  }
}