// Copyright 2015 The gRPC Authors
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

// The canonical version of this proto can be found at
// https://github.com/grpc/grpc-proto/blob/master/grpc/health/v1/health.proto

syntax = "proto3";

package grpc.health.v1;

option csharp_namespace = "Grpc.Health.V1";
option go_package = "google.golang.org/grpc/health/grpc_health_v1";
option java_multiple_files = true;
option java_outer_classname = "HealthProto";
option java_package = "io.grpc.health.v1";

message HealthCheckRequest {
  string service = 1;
}

message HealthCheckResponse {
  enum ServingStatus {
    UNKNOWN = 0;
    SERVING = 1;
    NOT_SERVING = 2;
    SERVICE_UNKNOWN = 3;  // Used only by the Watch method.
  }
  ServingStatus status = 1;
}

service Health {
  // If the requested service is unknown, the call will fail with status
  // NOT_FOUND.
  rpc Check(HealthCheckRequest) returns (HealthCheckResponse);

  // Performs a watch for the serving status of the requested service.
  // The server will immediately send back a message indicating the current
  // serving status.  It will then subsequently send a new message whenever
  // the service's serving status changes.
  //
  // If the requested service is unknown when the call is received, the
  // server will send a message setting the serving status to
  // SERVICE_UNKNOWN but will *not* terminate the call.  If at some
  // future point, the serving status of the service becomes known, the
  // server will send a new message with the service's serving status.
  //
  // If the call terminates with status UNIMPLEMENTED, then clients
  // should assume this method is not supported and should not retry the
  // call.  If the call terminates with any other status (including OK),
  // clients should retry the call with appropriate exponential backoff.
  rpc Watch(HealthCheckRequest) returns (stream HealthCheckResponse);
}