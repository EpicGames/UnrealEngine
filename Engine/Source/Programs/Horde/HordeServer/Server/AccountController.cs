// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Security.Claims;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Server;
using EpicGames.Horde.Users;
using HordeServer.Accounts;
using HordeServer.Acls;
using HordeServer.Authentication;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.Cookies;
using Microsoft.AspNetCore.Authentication.OpenIdConnect;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

#pragma warning disable CA1054 // URI-like parameters should not be strings

namespace HordeServer.Server
{
	/// <summary>
	/// Model for Horde account login view
	/// </summary>
	public class HordeAccountLoginViewModel
	{
		/// <summary>
		/// Where to post the form
		/// </summary>
		public string? FormPostUrl { get; set; }

		/// <summary>
		/// Optional error message to display
		/// </summary>
		public string? ErrorMessage { get; set; }
	}
	
	/// <summary>
	/// Model for Horde account state view
	/// </summary>
	public class AccountStateViewModel
	{
		/// <summary>Whether the current user is authenticated</summary>
		public bool IsAuthenticated { get; set; }
		
		/// <summary>Authenticated user's display name</summary>
		public string? UserName { get; set; }
		
		/// <summary>Whether the user has administrative write permissions</summary>
		public bool IsAdmin { get; set; }
		
		/// <summary>List of security claims associated with the authenticated user</summary>
		public List<ClaimViewModel> Claims { get; set; } = new();
		
		/// <summary>
		/// Gets or sets the user's ACL permissions grouped by scope.
		/// Each scope contains a list of permissions showing which actions are authorized.
		/// Only scopes with at least one permission are displayed in the view.
		/// </summary>
		public Dictionary<string, List<UserAclPermission>> ScopeToPermissions { get; set; } = new();
	}
	
	/// <summary>
	/// Model for Horde account state claim view
	/// </summary>
	public class ClaimViewModel
	{
		/// <summary>
		/// Gets or sets the claim type identifier (e.g., "name", "role", "sub").
		/// </summary>
		public string Type { get; set; } = "";
		
		/// <summary>
		/// Gets or sets the claim value associated with the claim type.
		/// </summary>
		public string Value { get; set; } = "";
	}

	/// <summary>
	/// Controller managing account status
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public class AccountController : Controller
	{
		/// <summary>
		/// Style sheet for HTML responses
		/// </summary>
		const string StyleSheet =
			"body { font-family: 'Segoe UI', 'Roboto', arial, sans-serif; } " +
			"p { margin:20px; font-size:13px; } " +
			"h1 { margin: 20px; font-size: 32px; font-weight: 200; } " +
			"h2 { margin: 20px; font-size: 24px; font-weight: 200; } " +
			"table { margin:10px 20px; } " +
			"th { margin: 5px; font-size: 13px; vertical-align: top; padding: 3px; text-align: left; }" +
			"td { margin:5px; font-size:13px; vertical-align: top; padding: 3px; font-family: Consolas, Courier New, monospace; }";

		readonly IUserCollection _users;
		readonly IAccountCollection _hordeAccounts;
		readonly string _authenticationScheme;
		readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		readonly IOptionsMonitor<ServerSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public AccountController(IUserCollection users, IAccountCollection hordeAccounts, IOptionsMonitor<ServerSettings> serverSettings, IOptionsSnapshot<GlobalConfig> globalConfig, IOptionsMonitor<ServerSettings> settings)
		{
			_users = users;
			_hordeAccounts = hordeAccounts;
			_authenticationScheme = GetAuthScheme(serverSettings.CurrentValue.AuthMethod);
			_globalConfig = globalConfig;
			_settings = settings;
		}

		/// <summary>
		/// Get auth scheme name for a given auth method
		/// </summary>
		/// <param name="method">Authentication method</param>
		/// <returns>Name of authentication scheme</returns>
		public static string GetAuthScheme(AuthMethod method)
		{
			return method switch
			{
				AuthMethod.Anonymous => AnonymousAuthHandler.AuthenticationScheme,
				AuthMethod.Okta => OktaAuthHandler.AuthenticationScheme,
				AuthMethod.OpenIdConnect => OpenIdConnectDefaults.AuthenticationScheme,
				AuthMethod.Horde => CookieAuthenticationDefaults.AuthenticationScheme,
				_ => throw new ArgumentOutOfRangeException(nameof(method), method, null)
			};
		}

		/// <summary>
		/// Displays the current user's authentication status and permissions
		/// </summary>
		[HttpGet]
		[Route("/account")]
		public ActionResult State()
		{
			AccountStateViewModel model = new()
			{
				IsAuthenticated = User.Identity?.IsAuthenticated ?? false,
				UserName = User.Identity?.Name,
				IsAdmin = _globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User),
			};
			
			if (model.IsAuthenticated)
			{
				model.Claims = User.Claims.Select(claim => new ClaimViewModel { Type = claim.Type, Value = claim.Value }).ToList();
				List<UserAclPermission> aclPermissions = UserController.GetUserAclPermissions(_globalConfig.Value, User);
				model.ScopeToPermissions = aclPermissions
					.GroupBy(permission => permission.Scope)
					.ToDictionary(group => group.Key, group => group.ToList());
			}

			return View("~/Server/HordeAccountState.cshtml", model);
		}

		/// <summary>
		/// Logged out page
		/// </summary>
		/// <returns>HTML</returns>
		[HttpGet]
		[Route("/account/logged-out")]
		public ViewResult LoggedOut()
		{
			return View("~/Server/HordeAccountLoggedOut.cshtml");
		}

		/// <summary>
		/// Show login form for username/password login
		/// </summary>
		/// <returns>HTML for a login form</returns>
		[HttpGet]
		[Route("/account/login/horde")]
		public IActionResult UserPassLoginForm(string? returnUrl = null)
		{
			if (User.Identity is { IsAuthenticated: true })
			{
				// Redirect if already logged in
				return Redirect(returnUrl ?? "/");
			}

			return View("~/Server/HordeAccountLogin.cshtml", new HordeAccountLoginViewModel
			{
				FormPostUrl = Url.Action("UserPassLogin", "Account", returnUrl != null ? new { returnUrl } : null)
			});
		}

		/// <summary>
		/// Perform a login with username/password credentials
		/// </summary>
		/// <returns>An HTTP redirect if successful</returns>
		[HttpPost]
		[Route("/account/login/horde")]
		public async Task<IActionResult> UserPassLoginAsync(string? returnUrl = null)
		{
			if (_globalConfig.Value.ServerSettings.AuthMethod != AuthMethod.Horde)
			{
				return Forbid("Horde built-in authentication is disabled");
			}

			const string ErrorMsg = "Invalid username or password";
			string? username = Request.Form["username"];
			string password = (string?)Request.Form["password"] ?? String.Empty;

			bool success = await SignInAsync(username, password);

			if (!success)
			{
				return LoginFormError(ErrorMsg, returnUrl);
			}

			return Redirect(returnUrl ?? "/");
		}

		/// <summary>
		/// Dashboard login 
		/// </summary>
		/// <param name="request"></param>
		/// <returns></returns>
		[HttpPost]
		[Route("/account/login/dashboard")]
		public async Task<IActionResult> UserDashboardLoginAsync([FromBody] DashboardLoginRequest request)
		{
			if (_globalConfig.Value.ServerSettings.AuthMethod != AuthMethod.Horde)
			{
				return Forbid();
			}

			bool success = await SignInAsync(request.Username, request.Password);

			if (!success)
			{
				return Forbid();
			}

			return Redirect(request.ReturnUrl ?? "/");
		}

		private ViewResult LoginFormError(string message, string? returnUrl = null, HttpStatusCode statusCode = HttpStatusCode.BadRequest)
		{
			Response.StatusCode = (int)statusCode;
			return View("~/Server/HordeAccountLogin.cshtml", new HordeAccountLoginViewModel
			{
				FormPostUrl = Url.Action("UserPassLogin", "Account", returnUrl != null ? new { returnUrl } : null),
				ErrorMessage = message
			});
		}

		/// <summary>
		/// Sign into a Horde auth account
		/// </summary>
		/// <param name="login"></param>
		/// <param name="password"></param>
		/// <returns></returns>
		async Task<bool> SignInAsync(string? login, string? password)
		{
			if (String.IsNullOrEmpty(login))
			{
				return false;
			}

			IAccount? account = await _hordeAccounts.FindByLoginAsync(login);
			if (account is not { Enabled: true })
			{
				return false;
			}

			if (!String.IsNullOrEmpty(account.PasswordHash))
			{
				byte[] correctHash = PasswordHasher.HashFromString(account.PasswordHash);
				byte[] salt = PasswordHasher.SaltFromString(account.PasswordSalt);
				if (!PasswordHasher.ValidatePassword(password ?? "", salt, correctHash))
				{
					return false;
				}
			}

			IUser user = await _users.FindOrAddUserByLoginAsync(account.Login, account.Name, account.Email);
			List<Claim> claims = new()
			{
				new Claim(HordeClaimTypes.Version, HordeClaimTypes.CurrentVersion),
				new Claim(HordeClaimTypes.AccountId, account.Id.ToString()),
				new Claim(ClaimTypes.Name, account.Name),
				new Claim(HordeClaimTypes.User, account.Login),
				new Claim(HordeClaimTypes.UserId, user.Id.ToString()),
			};
			if (!String.IsNullOrEmpty(account.Email))
			{
				claims.Add(new Claim(ClaimTypes.Email, account.Email));
			}
			
			if (!String.IsNullOrEmpty(_settings.CurrentValue.AdminClaimType) &&
			    !String.IsNullOrEmpty(_settings.CurrentValue.AdminClaimValue) &&
			    account.HasClaim(_settings.CurrentValue.AdminClaimType, _settings.CurrentValue.AdminClaimValue))
			{
				claims.Add(HordeClaims.AdminClaim.ToClaim());
			}

			foreach (IUserClaim claim in account.Claims)
			{
				claims.Add(new Claim(claim.Type, claim.Value));
			}

			ClaimsIdentity claimsIdentity = new(claims, CookieAuthenticationDefaults.AuthenticationScheme);
			AuthenticationProperties authProperties = new()
			{
				IsPersistent = true,
				ExpiresUtc = DateTimeOffset.UtcNow.AddDays(7)
			};

			await HttpContext.SignInAsync(
				CookieAuthenticationDefaults.AuthenticationScheme,
				new ClaimsPrincipal(claimsIdentity),
				authProperties);

			return true;

		}

		/// <summary>
		/// Login to the server
		/// </summary>
		/// <returns>Http result</returns>
		[HttpGet]
		[Route("/account/login")]
		public IActionResult Login()
		{
			return new ChallengeResult(_authenticationScheme, new AuthenticationProperties { RedirectUri = "/account" });
		}

		/// <summary>
		/// Logout of the current account
		/// </summary>
		/// <returns>Http result</returns>
		[HttpGet]
		[Route("/account/logout")]
		public async Task<IActionResult> LogoutAsync()
		{
			await HttpContext.SignOutAsync(CookieAuthenticationDefaults.AuthenticationScheme);
			try
			{
				await HttpContext.SignOutAsync(_authenticationScheme);
			}
			catch
			{
			}

			string content = $"<html><style>{StyleSheet}</style><body onload=\"setTimeout(function(){{ window.location = '/account'; }}, 2000)\"><p>User has been logged out. Returning to login page.</p></body></html>";
			return new ContentResult { ContentType = "text/html", StatusCode = (int)HttpStatusCode.OK, Content = content };
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpGet]
		[Route("/account/entitlements")]
		[ProducesResponseType(typeof(GetAccountEntitlementsResponse), 200)]
		public ActionResult<object> GetCurrentAccountEntitlements([FromQuery] PropertyFilter? filter = null)
		{
			GetAccountEntitlementsResponse response = CreateGetAccountEntitlementsResponse(_globalConfig.Value.Acl, claim => User.HasClaim(claim.Type, claim.Value));
			return PropertyFilter.Apply(response, filter);
		}

		internal static GetAccountEntitlementsResponse CreateGetAccountEntitlementsResponse(AclConfig rootAclConfig, Predicate<AclClaimConfig> predicate)
		{
			Dictionary<AclScopeName, HashSet<AclAction>> scopeToActions = rootAclConfig.FindEntitlements(predicate);

			List<GetAccountScopeEntitlementsResponse> scopes = new List<GetAccountScopeEntitlementsResponse>();
			foreach ((AclScopeName scopeName, HashSet<AclAction> actions) in scopeToActions)
			{
				scopes.Add(new GetAccountScopeEntitlementsResponse(scopeName.Text, actions.OrderBy(x => x.Name).ToList()));
			}

			return new GetAccountEntitlementsResponse(predicate(HordeClaims.AdminClaim), scopes);
		}

		/// <summary>
		/// Tests whether a user has a particular entitlement
		/// </summary>
		[HttpGet]
		[Route("/account/access")]
		public ActionResult<object> GetAccess([FromQuery] AclScopeName scope, [FromQuery] AclAction action, [FromQuery] PropertyFilter? filter = null)
		{
			AclConfig? scopeConfig;
			if (!_globalConfig.Value.TryGetAclScope(scope, out scopeConfig))
			{
				return NotFound();
			}

			bool access = scopeConfig.Authorize(action, User);

			List<object> scopes = new List<object>();
			for (AclConfig? testScopeConfig = scopeConfig; testScopeConfig != null; testScopeConfig = testScopeConfig.Parent)
			{
				scopes.Add(new { Name = testScopeConfig.ScopeName, Access = testScopeConfig.Authorize(action, User) });
			}

			return PropertyFilter.Apply(new { access, scopes }, filter);
		}
	}
}
