// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Acls;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Server
{
	/// <summary>
	/// Object containing settings for the server
	/// </summary>
	public class AdminSettings
	{
		/// <summary>
		/// The default perforce server
		/// </summary>
		public string? DefaultServerAndPort { get; set; }

		/// <summary>
		/// The default perforce username
		/// </summary>
		public string? DefaultUserName { get; set; }

		/// <summary>
		/// The default perforce password
		/// </summary>
		public string? DefaultPassword { get; set; }
	}

	/// <summary>
	/// The conform limit value
	/// </summary>
	public class ConformSettings
	{
		/// <summary>
		/// Maximum number of conforms allowed at once
		/// </summary>
		public int MaxCount { get; set; }
	}

	/// <summary>
	/// Controller managing account status
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class AdminController : HordeControllerBase
	{
		readonly IAclService _aclService;
		readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		readonly IOptions<ServerSettings> _serverSettings;

		/// <summary>
		/// Constructor
		/// </summary>
		public AdminController(IAclService aclService, IOptionsSnapshot<GlobalConfig> globalConfig, IOptions<ServerSettings> serverSettings)
		{
			_aclService = aclService;
			_globalConfig = globalConfig;
			_serverSettings = serverSettings;
		}

		/// <summary>
		/// Issues a token for the given roles. Issues a token for the current user if not specified.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/token")]
		public async Task<ActionResult<string>> GetTokenAsync(CancellationToken cancellationToken)
		{
			bool issueBearerToken = _globalConfig.Value.Authorize(ServerAclAction.IssueBearerToken, User);
			bool issueExtendedBearer = _globalConfig.Value.Authorize(ServerAclAction.IssueExtendedBearerToken, User); 

			if (!issueBearerToken && !issueExtendedBearer)
			{
				return Forbid(ServerAclAction.IssueBearerToken);
			}

			TimeSpan? expires = GetDefaultExpiryTime(issueExtendedBearer);

			return await _aclService.IssueBearerTokenAsync(User.Claims, expires, cancellationToken);
		}

		/// <summary>
		/// Issues a token for the given roles. Issues a token for the current user if not specified.
		/// </summary>
		/// <param name="roles">Roles for the new token</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/roletoken")]
		public async Task<ActionResult<string>> GetRoleTokenAsync([FromQuery] string roles, CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<Claim> claims = new List<Claim>();
			claims.AddRange(roles.Split('+').Select(x => new Claim(ClaimTypes.Role, x)));

			return await _aclService.IssueBearerTokenAsync(claims, GetDefaultExpiryTime(), cancellationToken);
		}

		/// <summary>
		/// Issues a token for the given roles. Issues a token for the current user if not specified.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/registrationtoken")]
		public async Task<ActionResult<string>> GetRegistrationTokenAsync(CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(ClaimTypes.Name, User.Identity?.Name ?? "Unknown"));
			claims.Add(HordeClaims.AgentRegistrationClaim);

			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Issues a token valid to upload new versions of the agent software.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/softwaretoken")]
		public async Task<ActionResult<string>> GetSoftwareTokenAsync(CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(ClaimTypes.Name, User.Identity?.Name ?? "Unknown"));
			claims.Add(HordeClaims.UploadToolsClaim);

			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Issues a token valid to download new versions of the agent software.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/softwaredownloadtoken")]
		public async Task<ActionResult<string>> GetSoftwareDownloadTokenAsync(CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminRead, User))
			{
				return Forbid(AdminAclAction.AdminRead);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(ClaimTypes.Name, User.Identity?.Name ?? "Unknown"));
			claims.Add(HordeClaims.DownloadSoftwareClaim);

			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Issues a token valid to configure streams and projects
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/configtoken")]
		public async Task<ActionResult<string>> GetConfigTokenAsync(CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminRead, User))
			{
				return Forbid(AdminAclAction.AdminRead);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(ClaimTypes.Name, User.Identity?.Name ?? "Unknown"));
			claims.Add(HordeClaims.ConfigureProjectsClaim);

			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Issues a token valid to start chained jobs
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Administrative settings for the server</returns>
		[HttpGet]
		[Route("/api/v1/admin/chainedjobtoken")]
		public async Task<ActionResult<string>> GetChainedJobTokenAsync(CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(AdminAclAction.AdminRead, User))
			{
				return Forbid(AdminAclAction.AdminRead);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			//Claims.Add(new AclClaim(ClaimTypes.Name, User.Identity.Name ?? "Unknown"));
			claims.Add(HordeClaims.StartChainedJobClaim);

			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Gets the default expiry time for a token
		/// </summary>		
		/// <param name="extended">Whether to use the extended bearer expiryTime</param>
		/// <returns></returns>
		private TimeSpan? GetDefaultExpiryTime(bool extended = false)
		{
			ServerSettings serverSettings = _serverSettings.Value;

			int expireyHours = extended ? serverSettings.ExtendedJwtExpiryTimeHours : serverSettings.JwtExpiryTimeHours;

			TimeSpan? expiryTime = null;
			if (expireyHours != -1)
			{
				expiryTime = TimeSpan.FromHours(expireyHours);
			}
			return expiryTime;
		}
	}
}
