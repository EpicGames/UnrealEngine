// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Web;
using EpicGames.Core;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Utilities;
using JetBrains.Profiler.SelfApi;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Features;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Driver;

namespace HordeServer.Server
{
	/// <summary>
	/// Controller providing some debug functionality
	/// </summary>
	[ApiController]
	[Authorize]
	[DebugEndpoint]
	[Tags("Debug")]
	public class DebugController : HordeControllerBase
	{
		private static readonly Random s_random = new();

		private readonly IMongoService _mongoService;
		private readonly ConfigService _configService;
		private readonly IAgentScheduler _agentScheduler;
		private readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		private readonly ILogger<DebugController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public DebugController(
			IMongoService mongoService,
			ConfigService configService,
			IAgentScheduler agentScheduler,
			IOptionsSnapshot<GlobalConfig> globalConfig,
			ILogger<DebugController> logger)
		{
			_mongoService = mongoService;
			_configService = configService;
			_agentScheduler = agentScheduler;
			_globalConfig = globalConfig;
			_logger = logger;
		}

		/// <summary>
		/// Prints all the environment variables
		/// </summary>
		/// <returns>Http result</returns>
		[HttpGet]
		[Route("/api/v1/debug/environment")]
		public ActionResult GetServerEnvVars()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			StringBuilder content = new StringBuilder();
			content.AppendLine("<html><body><pre>");
			foreach (System.Collections.DictionaryEntry? pair in System.Environment.GetEnvironmentVariables())
			{
				if (pair != null)
				{
					content.AppendLine(HttpUtility.HtmlEncode($"{pair.Value.Key}={pair.Value.Value}"));
				}
			}
			content.Append("</pre></body></html>");
			return new ContentResult { ContentType = "text/html", StatusCode = (int)HttpStatusCode.OK, Content = content.ToString() };
		}

		/// <summary>
		/// Converts all legacy pools into config entries
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/aclscopes")]
		public ActionResult<object> GetAclScopes()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			return new { scopes = _globalConfig.Value.AclScopes.Keys.ToList() };
		}

		/// <summary>
		/// Returns the fully parsed config object.
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/appsettings")]
		public ActionResult<object> GetAppSettings()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			return _globalConfig.Value.ServerSettings;
		}

		/// <summary>
		/// Returns the fully parsed config object.
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/config")]
		public ActionResult<object> GetConfig()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			// Duplicate the config, so we can redact stuff that we don't want to return through the browser
			byte[] data = _configService.Serialize(_globalConfig.Value);
			GlobalConfig config = _configService.Deserialize(data, false)!;
			return config;
		}

		/// <summary>
		/// Returns the fully parsed config object.
		/// </summary>
		[HttpGet]
		[Route("/api/v1/debug/randomdata")]
		public async Task<ActionResult> GetDataAsync([FromQuery] string size = "1mb", CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			long sizeBytes;
			try
			{
				sizeBytes = StringUtils.ParseBytesString(size);
			}
			catch
			{
				return BadRequest("Size must have a well-known binary suffix (eg. gb, mb, kb).");
			}

			// Disable buffering for the response
			IHttpResponseBodyFeature? responseBodyFeature = HttpContext.Features.Get<IHttpResponseBodyFeature>();
			responseBodyFeature?.DisableBuffering();

			// Write the response directly to the writer
			HttpResponse response = HttpContext.Response;
			response.ContentType = "application/octet-stream";
			response.StatusCode = (int)HttpStatusCode.OK;

			await response.StartAsync(cancellationToken);

			Random rnd = new Random();
			for (long offsetBytes = 0; offsetBytes < sizeBytes;)
			{
				int chunkSize = (int)Math.Min(64 * 1024, sizeBytes - offsetBytes);

				Memory<byte> chunkData = response.BodyWriter.GetMemory(chunkSize);
				rnd.NextBytes(chunkData.Span.Slice(0, chunkSize));
				response.BodyWriter.Advance(chunkSize);

				offsetBytes += chunkSize;
			}

			await response.CompleteAsync();
			return Empty;
		}

		/// <summary>
		/// Generate log message of varying size
		/// </summary>
		/// <returns>Information about the log message generated</returns>
		[HttpGet]
		[Route("/api/v1/debug/generate-log-msg")]
		public ActionResult GenerateLogMessage(
			[FromQuery] string? logLevel = null,
			[FromQuery] int messageLen = 0,
			[FromQuery] int exceptionMessageLen = 0,
			[FromQuery] int argCount = 0,
			[FromQuery] int argLen = 10)
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			string RandomString(int length)
			{
				const string Chars = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789";
				return new string(Enumerable.Repeat(Chars, length).Select(s => s[s_random.Next(s.Length)]).ToArray());
			}

			if (!Enum.TryParse(logLevel, out LogLevel logLevelInternal))
			{
				logLevelInternal = LogLevel.Information;
			}

			Exception? exception = null;
			string message = "Message generated by /api/v1/debug/generate-log-msg";
			message += RandomString(messageLen);

			if (exceptionMessageLen > 0)
			{
				exception = new Exception("Exception from /api/v1/debug/generate-log-msg " + RandomString(exceptionMessageLen));
			}

			Dictionary<string, object> args = new();
			if (argCount > 0)
			{
				for (int i = 0; i < argCount; i++)
				{
					args["Arg" + i] = "Arg 1 - " + RandomString(argLen);
				}
			}

			using IDisposable? logScope = _logger.BeginScope(args);

			// Ignore warning as we explicitly want to build this message manually
#pragma warning disable CA2254 // Template should be a static expression
			_logger.Log(logLevelInternal, exception, message);
#pragma warning restore CA2254

			return Ok($"Log message generated logLevel={logLevelInternal} messageLen={messageLen} exceptionMessageLen={exceptionMessageLen} argCount={argCount} argLen={argLen}");
		}

		/// <summary>
		/// Populate the database with test data
		/// </summary>
		/// <returns>Async task</returns>
		[HttpGet]
		[Route("/api/v1/debug/collections/{Name}")]
		public async Task<ActionResult<object>> GetDocumentsAsync(string name, [FromQuery] string? filter = null, [FromQuery] int index = 0, [FromQuery] int count = 10)
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			IMongoCollection<Dictionary<string, object>> collection = _mongoService.GetCollection<Dictionary<string, object>>(name);
			List<Dictionary<string, object>> documents = await collection.Find(filter ?? "{}").Skip(index).Limit(count).ToListAsync();
			return documents;
		}

		/// <summary>
		/// Start a CPU profiler session using dotTrace
		/// Only one profiling session can run at a time.
		/// </summary>
		/// <returns>Status description</returns>
		[HttpGet]
		[Route("/api/v1/debug/profiler/cpu/start")]
		public async Task<ActionResult> StartCpuProfilerAsync()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			// Downloads dotTrace executable if not available
			Stopwatch sw = Stopwatch.StartNew();
			await DotTrace.EnsurePrerequisiteAsync();
			_logger.LogInformation("dotTrace prerequisites step finished in {SetupTimeMs} ms", sw.ElapsedMilliseconds);

			string snapshotDir = Path.Join(Path.GetTempPath(), "horde-cpu-profiler-snapshots");
			if (!Directory.Exists(snapshotDir))
			{
				Directory.CreateDirectory(snapshotDir);
			}

			DotTrace.Config config = new();
			config.SaveToDir(snapshotDir);
			DotTrace.Attach(config);
			DotTrace.StartCollectingData();

			return new ContentResult { ContentType = "text/plain", StatusCode = (int)HttpStatusCode.OK, Content = "CPU profiling session started. Using dir " + snapshotDir };
		}

		/// <summary>
		/// Stops a CPU profiler session
		/// </summary>
		/// <returns>Text message</returns>
		[HttpGet]
		[Route("/api/v1/debug/profiler/cpu/stop")]
		public ActionResult StopProfiler()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			DotTrace.SaveData();
			DotTrace.Detach();
			return new ContentResult { ContentType = "text/plain", StatusCode = (int)HttpStatusCode.OK, Content = "CPU profiling session stopped" };
		}

		/// <summary>
		/// Downloads the captured CPU profiling snapshots
		/// </summary>
		/// <returns>A .zip file containing the profiling snapshots</returns>
		[HttpGet]
		[Route("/api/v1/debug/profiler/cpu/download")]
		public ActionResult DownloadProfilingData()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			string snapshotZipFile = DotTrace.GetCollectedSnapshotFilesArchive(false);
			if (!System.IO.File.Exists(snapshotZipFile))
			{
				return NotFound("The generated snapshot .zip file was not found");
			}

			return PhysicalFile(snapshotZipFile, "application/zip", Path.GetFileName(snapshotZipFile));
		}

		/// <summary>
		/// Take a memory snapshot using dotTrace
		/// </summary>
		/// <returns>A .dmw file containing the memory snapshot</returns>
		[HttpGet]
		[Route("/api/v1/debug/profiler/mem/snapshot")]
		public async Task<ActionResult> TakeMemorySnapshotAsync()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			// Downloads dotMemory executable if not available
			Stopwatch sw = Stopwatch.StartNew();
			await DotMemory.EnsurePrerequisiteAsync();
			_logger.LogInformation("dotMemory prerequisites step finished in {SetupTimeMs} ms", sw.ElapsedMilliseconds);

			string snapshotDir = Path.Join(Path.GetTempPath(), "horde-mem-profiler-snapshots");
			if (!Directory.Exists(snapshotDir))
			{
				Directory.CreateDirectory(snapshotDir);
			}

			sw.Restart();
			DotMemory.Config config = new();
			config.SaveToDir(snapshotDir);
			string workspaceFilePath = DotMemory.GetSnapshotOnce(config);
			_logger.LogInformation("dotMemory snapshot captured in {CaptureTimeMs} ms", sw.ElapsedMilliseconds);

			if (!System.IO.File.Exists(workspaceFilePath))
			{
				return NotFound("The generated workspace file was not found");
			}

			return PhysicalFile(workspaceFilePath, "application/octet-stream", Path.GetFileName(workspaceFilePath));
		}

		/// <summary>
		/// Throws an exception to debug error handling
		/// </summary>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/debug/exception")]
		public ActionResult ThrowException()
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			int numberArg = 42;
			string stringArg = "hello";
			throw new Exception($"Message: numberArg:{numberArg}, stringArg:{stringArg}");
		}

		/// <summary>
		/// Clean up orphaned agent sessions
		/// </summary>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/debug/agent-orphaned-session-cleanup")]
		public async Task<ActionResult<object>> AgentSessionCleanupAsync(CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			return Ok(new 
			{ 
				SessionsRemoved = await _agentScheduler.RemoveOrphanedSessionsAsync(cancellationToken),
			});
		}
	}
}
