// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Server
{
	/// <summary>
	/// Service which manages the downtime schedule
	/// </summary>
	public sealed class DowntimeService : IDowntimeService, IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Whether the server is currently in downtime
		/// </summary>
		public bool IsDowntimeActive
		{
			get;
			private set;
		}

		readonly IClock _clock;
		readonly IOptionsMonitor<GlobalConfig> _globalConfig;
		readonly ILogger _logger;

		readonly ITicker _ticker;

		/// <summary>
		/// Constructor
		/// </summary>
		public DowntimeService(IClock clock, IOptionsMonitor<GlobalConfig> globalConfig, ILogger<DowntimeService> logger)
		{
			_clock = clock;
			_globalConfig = globalConfig;
			_logger = logger;

			// Ensure the initial value to be correct
			Tick();

			_ticker = clock.AddTicker<DowntimeService>(TimeSpan.FromMinutes(1.0), TickAsync, logger);
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		/// <inheritdoc/>
		public async ValueTask DisposeAsync() => await _ticker.DisposeAsync();

		/// <summary>
		/// Periodically called tick function
		/// </summary>
		/// <param name="stoppingToken">Token indicating that the service should stop</param>
		/// <returns>Async task</returns>
		ValueTask TickAsync(CancellationToken stoppingToken)
		{
			Tick();
			return new ValueTask();
		}

		internal void Tick()
		{
			GlobalConfig globalConfig = _globalConfig.CurrentValue;

			DateTimeOffset now = TimeZoneInfo.ConvertTime(new DateTimeOffset(_clock.UtcNow), _clock.TimeZone);
			bool isActive = globalConfig.Downtime.Any(x => x.IsActive(now));

			DateTimeOffset? next = null;
			foreach (ScheduledDowntime schedule in globalConfig.Downtime)
			{
				DateTimeOffset start = schedule.GetNext(now).StartTime;
				if (start >= now && (next == null || start < next))
				{
					next = start;
				}
			}

			if (next != null)
			{
				_logger.LogInformation("Server time: {Time}. Downtime: {Downtime}. Next: {Next}.", now, isActive, next.Value);
			}

			if (isActive != IsDowntimeActive)
			{
				IsDowntimeActive = isActive;
				if (IsDowntimeActive)
				{
					_logger.LogInformation("Entering downtime");
				}
				else
				{
					_logger.LogInformation("Leaving downtime");
				}
			}
		}
	}
}
