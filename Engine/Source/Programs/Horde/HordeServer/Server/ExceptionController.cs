// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Diagnostics;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;

namespace HordeServer.Server
{
	/// <summary>
	/// Middleware that logs any unhandled exception and then rethrows it,
	/// so upstream handlers (developer exception page, exception filters, etc.) still see it.
	/// </summary>
	public sealed class ExceptionLoggingMiddleware(RequestDelegate next, ILogger<ExceptionLoggingMiddleware> logger)
	{
		/// <summary>
		/// Name of key to store in HttpContext to track if an exception has been logged
		/// </summary>
		public const string LoggedContextKey = "ExceptionLogged"; 
		
		/// <summary>
		/// Invokes the next middleware and logs any unhandled exception with request method, path, and trace identifier for correlation.
		/// </summary>
		/// <exception cref="Exception">
		/// Always rethrows the original exception after logging, to preserve default error handling.
		/// </exception>
		public async Task InvokeAsync(HttpContext context)
		{
			try
			{
				await next(context);
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Unhandled exception for {Method} {Path}: {ExceptionMessage}", context.Request.Method, context.Request.Path, ex.Message);
				context.Items[LoggedContextKey] = true;
				throw;
			}
		}
	}
	
	/// <summary>
	/// Controller managing account status
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public class ExceptionController(ILogger<ExceptionController> logger) : HordeControllerBase
	{
		/// <summary>
		/// Outputs a diagnostic error response for an exception
		/// </summary>
		[Route("/api/v1/exception")]
		[ApiExplorerSettings(IgnoreApi = true)]
		public ActionResult Exception()
		{
			IExceptionHandlerPathFeature? feature = HttpContext.Features.Get<IExceptionHandlerPathFeature>();
			bool wasAlreadyLogged = HttpContext.Items.ContainsKey(ExceptionLoggingMiddleware.LoggedContextKey);
			int statusCode = (int)HttpStatusCode.InternalServerError;

			LogEvent logEvent;
			if (feature?.Error == null)
			{
				const string Msg = "Exception handler path feature is missing or no error code.";
				logEvent = LogEvent.Create(LogLevel.Error, Msg);
				logger.LogError(Msg);
			}
			else if (feature.Error is StructuredHttpException structuredHttpEx)
			{
				(logEvent, statusCode) = (structuredHttpEx.ToLogEvent(), structuredHttpEx.StatusCode);
				if (!wasAlreadyLogged)
				{
					logger.Log(logEvent.Level, structuredHttpEx, "Structured HTTP exception: {Message}", structuredHttpEx.Message);
				}
			}
			else if (feature.Error is StructuredException structuredEx)
			{
				logEvent = structuredEx.ToLogEvent();
				if (!wasAlreadyLogged)
				{
					logger.Log(logEvent.Level, structuredEx, "Structured exception: {Message}", structuredEx.Message);	
				}
			}
			else
			{
				logEvent = LogEvent.Create(LogLevel.Error, default, feature.Error, "Unhandled exception: {Message}", feature.Error.Message);
				if (!wasAlreadyLogged)
				{
					logger.LogError(feature.Error, "Unhandled exception on {Path}: {Message}", feature.Path, feature.Error.Message);
				}
			}

			return new ObjectResult(logEvent) { StatusCode = statusCode };
		}
	}
}
