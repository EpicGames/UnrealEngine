// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.Security.Claims;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using HordeServer.Accounts;
using HordeServer.Acls;
using HordeServer.Configuration;
using HordeServer.Dashboard;
using HordeServer.Plugins;
using HordeServer.Server.Notices;
using HordeServer.ServiceAccounts;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer.Server
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Global configuration
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/global")]
	[JsonSchemaCatalog("Horde Globals", "Horde global configuration file", new[] { "globals.json", "*.global.json" })]
	[ConfigDoc("Globals.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Globals.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class GlobalConfig
	{
		/// <summary>
		/// Global server settings object
		/// </summary>
		[JsonIgnore]
		public ServerSettings ServerSettings { get; private set; } = null!;

		/// <summary>
		/// Unique identifier for this config revision. Useful to detect changes.
		/// </summary>
		[JsonIgnore]
		public string Revision { get; set; } = String.Empty;

		/// <summary>
		/// Version number for the server. Values are indicated by the <see cref="ConfigVersion"/>.
		/// </summary>
		public int Version { get; set; }

		/// <summary>
		/// Version number for the server, as an enum.
		/// </summary>
		[JsonIgnore]
		public ConfigVersion VersionEnum => (ConfigVersion)Version;

		/// <summary>
		/// Other paths to include
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Macros within the global scope
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

		/// <summary>
		/// Settings for the dashboard
		/// </summary>
		public DashboardConfig Dashboard { get; set; } = new DashboardConfig();

		/// <summary>
		/// List of scheduled downtime
		/// </summary>
		public List<ScheduledDowntime> Downtime { get; set; } = new List<ScheduledDowntime>();

		/// <summary>
		/// Plugin config objects
		/// </summary>
		public PluginConfigCollection Plugins { get; set; } = new PluginConfigCollection();

		/// <summary>
		/// General parameters for other tools. Can be queried through the api/v1/parameters endpoint.
		/// </summary>
		public JsonObject Parameters { get; set; } = new JsonObject();

		/// <summary>
		/// Whether to enable resolving secrets in config properties
		/// </summary>
		public bool ResolveSecrets { get; set; }

		/// <summary>
		/// Access control list
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Accessor for the ACL scope lookup
		/// </summary>
		[JsonIgnore]
		public IReadOnlyDictionary<AclScopeName, AclConfig> AclScopes => _aclLookup;

		private readonly Dictionary<AclScopeName, AclConfig> _aclLookup = new Dictionary<AclScopeName, AclConfig>();

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);

		/// <summary>
		/// Called after the config file has been read
		/// </summary>
		public void PostLoad(ServerSettings serverSettings, IReadOnlyList<ILoadedPlugin> loadedPlugins, IEnumerable<IDefaultAclModifier> defaultAclModifiers, ILogger? logger = null)
		{
			ServerSettings = serverSettings;
			AclAction[] aclActions = AclConfig.GetActions(
			[
				typeof(AccountAclAction),
				typeof(ServiceAccountAclAction),
				typeof(NoticeAclAction),
				typeof(ServerAclAction),
				typeof(AdminAclAction),
			]);

			AclConfig defaultAcl = CreateRootAcl(defaultAclModifiers);
			Acl.PostLoad(defaultAcl, defaultAcl.ScopeName, aclActions);

			// Ensure that all plugins have an entry in the global config so they can register their ACLs
			foreach (ILoadedPlugin loadedPlugin in loadedPlugins)
			{
				if (!Plugins.TryGetValue(loadedPlugin.Name, out _))
				{
					IPluginConfig pluginConfig = (IPluginConfig)Activator.CreateInstance(loadedPlugin.GlobalConfigType)!;
					Plugins.Add(loadedPlugin.Name, pluginConfig);
				}
			}

			IReadOnlyCollection<IPluginConfig> pluginConfigs;
			if (loadedPlugins.Count == 0)
			{
				pluginConfigs = Plugins.Values;
			}
			else
			{
				List<IPluginConfig> sortedPluginConfigs = [];
				HashSet<PluginName> seen = [];
				foreach (ILoadedPlugin loadedPlugin in PluginCollection.GetTopologicalSort(loadedPlugins))
				{
					sortedPluginConfigs.Add(Plugins[loadedPlugin.Name]);
					seen.Add(loadedPlugin.Name);
				}
				// Add configs that do not have a plugin
				foreach (KeyValuePair<PluginName, IPluginConfig> kvp in Plugins)
				{
					if (!seen.Contains(kvp.Key))
					{
						sortedPluginConfigs.Add(kvp.Value);
					}
				}
				pluginConfigs = sortedPluginConfigs;
			}

			PluginConfigOptions pluginConfigOptions = new PluginConfigOptions(VersionEnum, Plugins.Values, Acl, logger);
			foreach (IPluginConfig pluginConfig in pluginConfigs)
			{
				pluginConfig.PostLoad(pluginConfigOptions);
			}

			foreach (ScheduledDowntime downtime in Downtime)
			{
				downtime.PostLoad();
			}

			_aclLookup.Clear();
			BuildAclScopeLookup(Acl, _aclLookup);
		}

		/// <summary>
		/// Creates the default root ACL
		/// </summary>
		static AclConfig CreateRootAcl(IEnumerable<IDefaultAclModifier> defaultAclModifiers)
		{
			DefaultAclBuilder defaultAclBuilder = new DefaultAclBuilder();
			defaultAclBuilder.AddDefaultReadAction(ServerAclAction.IssueBearerToken);

			foreach (IDefaultAclModifier defaultAclModifier in defaultAclModifiers)
			{
				defaultAclModifier.Apply(defaultAclBuilder);
			}

			AclConfig defaultAcl = defaultAclBuilder.Build();
			defaultAcl.PostLoad(null, AclScopeName.Root, []);

			return defaultAcl;
		}

		static void BuildAclScopeLookup(AclConfig acl, Dictionary<AclScopeName, AclConfig> aclLookup)
		{
			aclLookup.Add(acl.ScopeName, acl);
			if (acl.LegacyScopeNames != null)
			{
				foreach (AclScopeName legacyScopeName in acl.LegacyScopeNames)
				{
					aclLookup.Add(legacyScopeName, acl);
				}
			}
			if (acl.Children != null)
			{
				foreach (AclConfig childAcl in acl.Children)
				{
					BuildAclScopeLookup(childAcl, aclLookup);
				}
			}
		}

		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="scopeName">Name of the scope to auth against</param>
		/// <param name="scopeConfig">Configuration for the scope</param>
		public bool TryGetAclScope(AclScopeName scopeName, [NotNullWhen(true)] out AclConfig? scopeConfig)
			=> _aclLookup.TryGetValue(scopeName, out scopeConfig);

		/// <summary>
		/// Authorizes a user to perform a given action
		/// </summary>
		/// <param name="scopeName">Name of the scope to auth against</param>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to validate</param>
		public bool Authorize(AclScopeName scopeName, AclAction action, ClaimsPrincipal user)
			=> _aclLookup.TryGetValue(scopeName, out AclConfig? scopeConfig) && scopeConfig.Authorize(action, user);

		IReadOnlyList<string>? _cachedGroupClaims;

		/// <summary>
		/// Gets all the valid <see cref="HordeClaimTypes.Group"/> claims referenced by ACL entries within the config object.
		/// </summary>
		public IReadOnlyList<string> GetValidAccountGroupClaims()
		{
			if (_cachedGroupClaims == null)
			{
				HashSet<string> groups = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
				FindGroupClaimsFromObject(Acl, groups);
				_cachedGroupClaims = groups.ToArray();
			}
			return _cachedGroupClaims;
		}

		static void FindGroupClaimsFromObject(AclConfig config, HashSet<string> groups)
		{
			if (config.Entries != null)
			{
				foreach (AclEntryConfig entry in config.Entries)
				{
					AclClaimConfig claim = entry.Claim;
					if (claim.Type.Equals(HordeClaimTypes.Group, StringComparison.OrdinalIgnoreCase))
					{
						groups.Add(claim.Value);
					}
				}
			}
			if (config.Children != null)
			{
				foreach (AclConfig childConfig in config.Children)
				{
					FindGroupClaimsFromObject(childConfig, groups);
				}
			}
		}
	}

	/// <summary>
	/// How frequently the maintenance window repeats
	/// </summary>
	public enum ScheduledDowntimeFrequency
	{
		/// <summary>
		/// Once
		/// </summary>
		Once,

		/// <summary>
		/// Every day
		/// </summary>
		Daily,

		/// <summary>
		/// Every week
		/// </summary>
		Weekly,
	}

	/// <summary>
	/// Settings for the maintenance window
	/// </summary>
	public class ScheduledDowntime
	{
		/// <summary>
		/// Start time
		/// </summary>
		public DateTimeOffset StartTime { get; set; }

		/// <summary>
		/// Finish time
		/// </summary>
		public DateTimeOffset FinishTime { get; set; }

		/// <summary>
		/// Duration of the maintenance window. An alternative to FinishTime.
		/// </summary>
		public TimeSpan? Duration { get; set; }

		/// <summary>
		/// The name of time zone to set the Coordinated Universal Time (UTC) offset for StartTime and FinishTime.
		/// </summary>
		public string? TimeZone { get; set; }

		/// <summary>
		/// Frequency that the window repeats
		/// </summary>
		public ScheduledDowntimeFrequency Frequency { get; set; } = ScheduledDowntimeFrequency.Once;

		internal void PostLoad()
		{
			if (TimeZone != null)
			{
				TimeZoneInfo timeZoneInfo = TimeZoneInfo.FindSystemTimeZoneById(TimeZone);
				TimeSpan offset = timeZoneInfo.GetUtcOffset(StartTime);
				StartTime = new DateTimeOffset(StartTime.Year, StartTime.Month, StartTime.Day,
					StartTime.Hour, StartTime.Minute, StartTime.Second, offset);
				FinishTime = new DateTimeOffset(FinishTime.Year, FinishTime.Month, FinishTime.Day,
					FinishTime.Hour, FinishTime.Minute, FinishTime.Second, offset);
			}

			if (Duration != null)
			{
				FinishTime = StartTime + Duration.Value;
			}
			else
			{
				Duration = FinishTime - StartTime;
			}
		}

		/// <summary>
		/// Gets the next scheduled downtime or the current one if it is active
		/// </summary>
		/// <param name="now">The current time</param>
		/// <returns>Start and finish time</returns>
		public (DateTimeOffset StartTime, DateTimeOffset FinishTime) GetNext(DateTimeOffset now)
		{
			TimeSpan offset = TimeSpan.Zero;
			if (Frequency is ScheduledDowntimeFrequency.Daily or ScheduledDowntimeFrequency.Weekly)
			{
				double days = (now - StartTime).TotalDays;
				if (IsActive(now))
				{
					// Clamp active schedule to "now"
					offset = TimeSpan.FromDays(Math.Floor(days));
				}
				else
				{
					// Round up to the next schedule window after "now"
					if (days > 0.0)
					{
						if (Frequency == ScheduledDowntimeFrequency.Daily)
						{
							offset = TimeSpan.FromDays(Math.Ceiling(days));
						}
						else if (Frequency == ScheduledDowntimeFrequency.Weekly)
						{
							offset = TimeSpan.FromDays(Math.Ceiling(days / 7.0) * 7.0);
						}
					}
				}
			}
			return (StartTime + offset, FinishTime + offset);
		}

		/// <summary>
		/// Determines if this schedule is active
		/// </summary>
		/// <param name="now">The current time</param>
		/// <returns>True if downtime is active</returns>
		public bool IsActive(DateTimeOffset now)
		{
			if (Frequency == ScheduledDowntimeFrequency.Once)
			{
				return now >= StartTime && now < FinishTime;
			}
			else if (Frequency == ScheduledDowntimeFrequency.Daily)
			{
				double days = (now - StartTime).TotalDays;
				if (days < 0.0)
				{
					return false;
				}
				else
				{
					return (days % 1.0) < (FinishTime - StartTime).TotalDays;
				}
			}
			else if (Frequency == ScheduledDowntimeFrequency.Weekly)
			{
				double days = (now - StartTime).TotalDays;
				if (days < 0.0)
				{
					return false;
				}
				else
				{
					return (days % 7.0) < (FinishTime - StartTime).TotalDays;
				}
			}
			else
			{
				return false;
			}
		}
	}
}

