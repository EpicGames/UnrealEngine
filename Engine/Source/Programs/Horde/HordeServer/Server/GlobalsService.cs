// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;
using Microsoft.IdentityModel.Tokens;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Server
{
	/// <summary>
	/// Accessor for the global <see cref="Globals"/> singleton
	/// </summary>
	public class GlobalsService
	{
		/// <summary>
		/// Global server settings
		/// </summary>
		[SingletonDocument("globals", "5e3981cb28b8ec59cd07184a")]
		class Globals : SingletonBase, IGlobals
		{
			[BsonIgnore]
			public GlobalsService _owner = null!;

			public ObjectId InstanceId { get; set; }
			public string ConfigRevision { get; set; } = String.Empty;
			public byte[]? JwtSigningKey { get; set; }
			public RSAParameters? RsaParameters { get; set; }
			public int? SchemaVersion { get; set; }

			[BsonIgnore]
			string IGlobals.JwtIssuer => _owner._jwtIssuer;

			[BsonIgnore]
			SecurityKey IGlobals.JwtSigningKey => new SymmetricSecurityKey(JwtSigningKey!);

			[BsonIgnore]
			RsaSecurityKey IGlobals.RsaSigningKey => new RsaSecurityKey(RsaParameters!.Value) { KeyId = InstanceId.ToString() };

			public Globals()
			{
				InstanceId = ObjectId.GenerateNewId();
			}

			public Globals Clone()
			{
				return (Globals)MemberwiseClone();
			}

			public void RotateSigningKey()
			{
				JwtSigningKey = RandomNumberGenerator.GetBytes(128);
			}

			public void RotateRsaParameters()
			{
				using RSACryptoServiceProvider rsaProvider = new RSACryptoServiceProvider(2048);
				rsaProvider.PersistKeyInCsp = false;
				RsaParameters = rsaProvider.ExportParameters(true);
			}
		}

		readonly IMongoService _mongoService;
		readonly string _jwtIssuer;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService"></param>
		/// <param name="settings">Global settings instance</param>
		public GlobalsService(IMongoService mongoService, IOptions<ServerSettings> settings)
		{
			_mongoService = mongoService;

			if (String.IsNullOrEmpty(settings.Value.JwtIssuer))
			{
				_jwtIssuer = Dns.GetHostName();
			}
			else
			{
				_jwtIssuer = settings.Value.JwtIssuer;
			}
		}

		/// <summary>
		/// Gets the current globals instance
		/// </summary>
		/// <returns>Globals instance</returns>
		public async ValueTask<IGlobals> GetAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				Globals globals = await _mongoService.GetSingletonAsync<Globals>(() => CreateGlobals(), cancellationToken);
				globals._owner = this;

				if (globals.RsaParameters != null)
				{
					return globals;
				}

				globals.RotateRsaParameters();

				if (await _mongoService.TryUpdateSingletonAsync<Globals>(globals, cancellationToken))
				{
					return globals;
				}
			}
		}

		static Globals CreateGlobals()
		{
			Globals globals = new Globals();
			globals.RotateSigningKey();
			globals.RotateRsaParameters();
			return globals;
		}

		/// <summary>
		/// Try to update the current globals object
		/// </summary>
		/// <param name="globals">The current options value</param>
		/// <param name="configRevision"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async ValueTask<IGlobals?> TryUpdateAsync(IGlobals globals, string? configRevision, CancellationToken cancellationToken)
		{
			Globals concreteGlobals = ((Globals)globals).Clone();
			if (configRevision != null)
			{
				concreteGlobals.ConfigRevision = configRevision;
			}
			if (!await _mongoService.TryUpdateSingletonAsync(concreteGlobals, cancellationToken))
			{
				return null;
			}
			return concreteGlobals;
		}
	}
}
