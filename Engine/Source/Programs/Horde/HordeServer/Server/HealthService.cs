// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using Grpc.Core;
using Grpc.Health.V1;
using static Grpc.Health.V1.HealthCheckResponse.Types;

namespace HordeServer.Server
{
	/// <summary>
	/// Implements the gRPC health checking protocol
	/// See https://github.com/grpc/grpc/blob/master/doc/health-checking.md for details
	/// </summary>
	public class HealthService : Health.HealthBase
	{
		readonly LifetimeService _lifetimeService;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="lifetimeService">The application lifetime</param>
		public HealthService(LifetimeService lifetimeService)
		{
			_lifetimeService = lifetimeService;
		}

		/// <summary>
		/// Return server status, primarily intended for load balancers to decide if traffic should be routed to this process
		/// The supplied service name is currently ignored.
		///
		/// For example, the gRPC health check for AWS ALB will pick up and react to the gRPC status code returned.
		/// </summary>
		/// <param name="request">Empty placeholder request (for now)</param>
		/// <param name="context">Context for the call</param>
		/// <returns>Return status code 'unavailable' if stopping</returns>
		public override Task<HealthCheckResponse> Check(HealthCheckRequest request, ServerCallContext context)
		{
			ServingStatus status = ServingStatus.Serving;

			bool isStopping = _lifetimeService.IsPreStopping || _lifetimeService.IsStopping;
			if (isStopping)
			{
				context.Status = new Status(StatusCode.Unavailable, "Server is stopping");
				status = ServingStatus.NotServing;
			}

			return Task.FromResult(new HealthCheckResponse { Status = status });
		}

		/// <summary>
		/// Stream the server health status (not implemented)
		/// </summary>
		/// <param name="request"></param>
		/// <param name="responseStream"></param>
		/// <param name="context"></param>
		/// <returns></returns>
		public override Task Watch(HealthCheckRequest request, IServerStreamWriter<HealthCheckResponse> responseStream, ServerCallContext context)
		{
			return Task.FromException(new RpcException(new Status(StatusCode.Unimplemented, "Watch() not implemented")));
		}
	}
}
