@model HordeServer.Server.HordeAccountLoginViewModel
@{
	ViewBag.Title = "Login";
	Layout = "~/Server/_Layout.cshtml";
}

<style>
	.container {
		display: flex;
		justify-content: center;
		align-items: center;
		width: 100%;
	}
	.login-container {
		display: flex;
		flex-direction: column; /* Ensures the elements are stacked vertically */
		align-items: center; /* Centers the items horizontally */
		justify-content: center; /* Centers the items vertically */
		width: 300px;
		padding: 20px;
		background-color: #444;
		box-shadow: 0 2px 4px rgba(0, 0, 0, 0.5);
	}
	form { width: 100%; }
	.form-group { margin-bottom: 15px; }
	.error {
		display: block;
		width: 100%;
		padding: 8px;
		background-color: #CC6666;
		margin-bottom: 15px;
		font-size: 14px;
		box-sizing: border-box;
	}
	label { display: block; margin-bottom: 5px; font-size: 14px; }
	input:focus { outline: 1px solid #b4b4b4; border: 1px solid #b4b4b4; }
	.login-button {
		width: 100%;
		padding: 10px;
		font-size: 14px;
		font-weight: bold;
		background-color: #007bff;
		color: white;
		border: none;
		cursor: pointer;
	}
	.login-button:hover { background-color: #0056b3; }
	.logo { object-fit: contain; width: 70%; height: 30%; opacity: 0.5; margin-top: 10px; }
	.footer { color: #b4b4b4; margin-top: 20px; }
</style>
<div class="container">
	<div class="login-container">
		<img class="logo" alt="Unreal Engine Logo" src="data:image/png;base64,iVBORw0KGgoAAAANSUhEUgAAAasAAAFYCAMAAADaySPoAAAAM1BMVEUAAAD///////////////////////////////////////////////////////////////+3leKCAAAAEHRSTlMAgBDwwECg0GDgMCCQcLBQZ954NQAAEL1JREFUeNrsmtuC6yAIRVHwEo2a///a89I5TWqbXtMqst4nI27BDRUEQRAEQRAEQRAEoSKaChAaIhivZiK33AKJrFIi2y+JRlnC5QkoKy+SfRmTLC2vMmVlNAjHE/3slveZcpIMO5KQMi4fhJTodQTa22k5AEoBhI8mlLtf1YhoVhcQ0QN/aQsIHyHMewnlSKliAuyhjUkq73lGtF7sxoFC0ZxMhKcwXtHN72XJrjeIabrR3aoS4VW0UfnGd614jdfwdP1y8QHeR5f5+udVBOE54oxXdYrwQcrspBa+S6FvmWvtLdZnIonReAxd31KYj7RpZq7/oZVSeJ+osNq3AjscZTez+Ix9oq227FuNT6huSBK19pT67cVR8qVaHoRHlEIb4NvENElu3UXbRrxYyaLWLvrCUVCBX1H3diSecIXfbs7PDbNW03ZB0m+dMK7B1sZv1EIFAkDM201p5ghv1Zrk2gKFbSpVq5WbyPffYVxT99S+WphgXPTc/LndGlQ37NsMM/XQxmwP1JgeY7MHU8vDnEiDp9YmqdqyFDVlGjm1VF9zAd3Zej9IXNk/bLn8nQnU3ZI/PlLKjZe/MwnHGzqtJ+rY0zuUSKNZjOh6TKq/1BqqDhbsM6nqgzYDc1TvdmoVgOusKjyHzv23KavGEBlfWqsKgq2OlO6j8wCXVsBz/eu6fKT+q8MdPLKJ0JxDscAQ37P/u0Q7JiXiKpZZG2n52kHL7hwmZofvD20Z1veCHL37qrpzerIQGIqlHdN+RDtuYvGLiG1o3OLZYjkFpx1Pu8RQLO3YtiHcxOIvFR+xLH+pAGYWYg0h1er3R+w3zEGk4hCo7z6CYcQaSCoA2/W88/+0DLt8AzOSWBE5mKMnyMuJ7mae2g0m1aqV7O3ZT7+nbLjjqcaTCiBgj2aqdHzRvkFYTmTohoD9rfkfOWeCJDcMQlGsfbPF/U+bpZKokq70tAaQcM87wLhHX8AHyWYh3e7428Q75gIW2t38RVhbY/0ZnJp3Nq57TQYT/qKDNDWFiIhOj/H6k1Mc3IBi11yENj1khTeui73PPXATV/iKel5arwak+5Ss39U1G5DCnKpfY/2zAup+2T94/EUBIfql/Zs78R7NpbGyDUY9Mv5A9Vjkz9ha9/swl6gL8gFRv1QA6Q7GvUv+SO/wHlIBXPrnNkYw+L3DfVLVflznzEJk9VkwTBRVglJrZ27lvPL007z2LOilzGq98JEAggyd7OdiuOn2gibLNIHmwMHCNz97yOMLD1oWg4lD5ts33eID8hduerD4B1sISQYUUkVSdHU4WFWsSrOINKkAmuJu3UlYn9Pi38hPsU2KiHSpwFi1V+76WEWxoBpUEKI2iySpBl1tk5X5d1Gy+B8ayFADDsjmwCm1Fwd7djYXPjBbEulKYaJX8AiqMJbb9ZSMSAkrulL02cihchbWhrFgzH/EakVv4jAQO06FJ1mV21gEfMIF7CQrYjeTQt8eeDe8ifiMDsyUiCgzG3HqxoJ1ZAyxUjWwwItp+ADXnTmvLrAcq18vFnFhCvT5Wfy+W2Dxbp6EgwXjpSbrNquyD4SPvbNCKvSSlWoQeUu5jsAqI6xWSIXC9o/5gndVVbHCCKsVUmXgwoRnmVbj8mjaOAk/xgETNT57isr1UbRvEi7Uyj+zm7aqXCAihm/bJKRqRX+YQBBUNVPBg23XFLtQqyZeEwdBy2m+Hf0DWap1WgX5tmBQlFwWTFyTQBPxNSL3vFF+OOx0vD4XuSaBDl9FXCqswIsn/F3+X1HE6scjVVqqANxkDRc7A1N0J3ydLiaV2PZPCmy7GXaU3VcQZ6rV+/M4vS/TUgVgx9j9tv3kOVEyESfIk0cdMSQz86gK/IT9t2Qij2FvOIWfvwd1pZelCiBAHXV9E4VnJ3acw31mC+SfasWJnaBxV5PjwZGT+STnRIoeZA+B2L7R3QXsgqdiXjjLeGKdGk/EiT/NjLF7ZxedxYl2/ASh/lTqsEwd9sCADGGqGup8vMn4KaJzERG582sAIfzeFoslrBuS+HiJ5uggRd7ZYnWOUX9BGi8E/xQgRtv5hk/gsKEOabC5TPGVLOTdQE+BRdRY0OPgIHQDrOSpLKsvBWYkwty+VZCj7XOCjSEFnkgFWAPLgiBl7hnKQtrYFVpVFeVqrJiHD1G4Sw4kw+tfDnhEmRkjpK+wN6yQN9N6kKTvOhhx9NbuwDVaVRXW4ju7rl0wjM/sIq3A7u+Ef3LtGV14ejwnXKWVI5w4K6sbhPzVaLaIDmuydSBL2XMB15GLsccvpxXYLQWLnuAvfVo1ECbsKFievBEr6tPqAGHSjoJ1kP+54ytqVXZ0WBd5wJS/olZgP+509D3T45fU6lo/EqzkWA74Nzmc3vsz2PfWilA7iJOtwDWzyGmU3/zOWvn1B/kH9RS1P1mj9sZamVe7YUWdcMCB9fA36X21grzcXFhqJ5yffqAlva9W39i7092EYRgAwGmO3off/2mn0aFONCyRwixf/j1W4Cu1m6QOenHhW0uLudBLZxdrVVdcUMqQXWF6zwepVgl75GJr/WixtMhrlWq1YD+SvzaOWvjiYpRFqpXDXtEZG1dxHlDcfWmSahWRC8Gp8dzoymtRdqlWPXIh2HpDF8vzRZ1Uqw53CmtuzY9wRvAKrQ6EQ93rzrXRGkan0KqiaKdUso/P36VGK49YtF8f/Wgs+ZNKq7pkT6aUiT9nlk6rAfUGKzaunArny5Vaxb/eOLWj+Z/kSs/q/XtiO9Ie2qzS+bMiaIWS8DtUq8YPNp4DgcqtDpcNYlaPN5vUWlXd8VAZtlgfL1ZrleqsaAxbRAAYzcplgtzBIsDk1Fv1LhvErALAptjq75lhYlYA4BVb1ZRmZJbiAPTOrN4fi9LNHEAyKx5WCSZnVlysNrNiY7WYFRer6MyKi9VoVlysZm9WXKycMyuzMiuzcmZlVvlgNwltVnwmoc2KxJyIWZkVE6vZrNhYlb8+bgsGzIrPUylmxSc5yrW6ellhRL5rjFkRGkooPIZvVhStoln9V3MQlkf7rBWRPSrqznR2v2K5VqF8KGbtNORaVV5uGbXTEGu1VFmxGrgQa1XxRBS3gQuxVhtyY87sVo1mRWemttROQ5OV3/69BTejFg2krfzQUT7PS0W7Jis/QGLSjsm5MZMf9Vj5AYDHrPC7ncrUWH1T7ZRP83IhqMXqmwo2yukjnyC9u0KJ1YMKFjZlYLYQ1GF1Uk1cRm7zj+LrsDqpYMXZ5I3IhiI8reahdebuwN8I1d+LCwVWc2i+OVqvhTFoMdzaqMq3elLBRntKvbwBtXirEZ6x8Nn5Jb8BtXSr63/spAcR7jHfrtyyrXx//dFBuX7ORXi9HIi2OgvA9vMzXtqIsb8mLMlWR/hQE3c4wzvU2F4Tllwr38OvCL45XQ2uGKwS1ket9lqrnEOaPtZvf73WxaDG9HKnQNfqi7xzW24bhoEoCN4vkvH/X9t60paaphkp9VKE5POWpzizBLALMmN2chS/U1QvTujYf8upBEACB2jFtEfJcpj4lxJsLPBrLKp8QGeTIL13/Ne1LPId7EKd2pWClFXq+excGOBpAFpZhvS/Tl7qh1Bplc8EgHde6HTirmsfr1U/pftm+zjWua8kroDTXel0lhcqGqPV3jGtTsA0wL8BRDqfCmiCCK16ZX/2BWAsA9xYownkkU3QyItipSxwFsRartAEGqAJQrSSdopSkgnQAjPNoACaIEYryYk21GZlBB6xQW10kOs0QSPfIzdPT9i3KGNYIRmn0BTawJszJ/+Bi7JFk7Gg1FvgDOrAwBBFGwly+BpNIg675qyiDaf3XB9iGVbXRpRhGTIvIs2CZdCddLWijAfGhy00jXVMxGJ10yoQIcKVME3jAerCy7qU/lNSV1WRMbY20EQyyF04Eeua8d4vIYs2bAFZJU8TMaCH7axPoA0JdImeaSYV9dcUdX2v0+hF2E53Fk8Cyosm0YpDZRvLNBUvH/j7RSqUryDKCpzFkwhbCgbRiK2wllEJAOKDlDvuALsFBJSVo+n8LvA7RmARr2lMoAan1DuKlXDXO5Hmw7YX1t1iVgK6r0QKMJvCulfMSgTAacjBIwqLiqLKSnS3suqFVS6wY7cnS0VZU1n1wnL6L0QCR6RZ3yfpKqteWF67WKFPxH2p7lhWvbAiPdG7wQiH7UusBGHRVla9sJLq5xahX5DusLLOQ0zAXJSZMCQrcMLhlX4jEE3PyqKTtv9mqNO7h6NlaxOBqKJmE7glonfJvIKlOjoQYyEU7k+YUYXHH6EFLlVnla9oTCgeouTe6m9W/FvBEsFS7ecC+0AOcR3XwV/35swEgxtOqj2xuv9DGwtD6jDdROHwGStVp9ghRdUpoi0GdziPmKRsQFLtixWYkESNfr3biyHBrzq4VP/KxNkTjWgzK6lkHdSgHxks1edMbBfCUkSrsfiA7ag8YSxQqo5BG/VOFBXPN/cTUWQCw8aipeqZOFRCY3RuLLa4rRfUoFbYbdtdqREdcPqTwK+pdqD5YZPRUhGbSng4a++AT5axIzVFgFTjCfo74BM3+GOWYNVLlS7QAZ+wHV3+nJxuqYrVdxm8kzA9DaMuUa9UHHWn4C3tlBRYl1WnVH+GVVaagrdwPGuyPlrUJ1USzXvAL/t1o+GwN85qkqqI3puQf5Hk3OFavQnOqpCK7TXseifIjIcGxSdjmvtJo0lwvNCw+kW8SMIgusMhBe2a4nXOF4AmV0lWW7xcrW8DSNPH5Xt97tfOp74X0d9r3Y3eg2Kv5yt+4y7ZvF+Xyl7IV3T/+k5isb3UvuJv2L6PWJc/mP0F3oPuDceLrZY+U+TCPfxbUt3A9aa3EOsWUr2HWBxvEvyTXHzoHpHqLgu1H+3d25azIAwFYAIJiAju93/a/zAX2tbp6sFawHwPMGvoriFBu+w9rI6iMoa6DmuMHUVlzAS08GjjS0bXVVTGDN+/X/sh0ltUF2H1sqT/BnQXVaeLWu3E3NGqVmHFbgYt32tpX96g7fo4HLSh16iMGV1X7eDo0Pxx7e/G2FGHMaDrqXFdNULjm5ZHZxX9hl0tseVvYwoddkp3ulxMzdbB2XU5gdwYXOt10E4VPDZ/iHWH4VrsB8eArvrZu2xBw+M+ASe4e7ogtNpFJW75e/bm6TRQGlpyduh5AN5mGe117+P6n27ygc0XZSy4hR9pWcLp6t9WPwWqfu0S0Xb/utucglh3j5EK0Ppc+B6JbRRCSw4nbCo2Lq3qD9+HePaL6ocELFyN25ZEnHmnukAOFffvwmijTh8jMVZiTWkJo53+5xhzrDItYaCpueIY5LASa/hYrpIqZy9/C+ux5iiZrxoi1vhMR0rPv93Ai/kWSxFVVuV6CONCGKz5AvHQpJ5Py/nRHMvmCKD2ma8OwrgUsjWHmQuA6rqcegnjSjmmFo7eQavfk5LH4XGNU8SVoEk9IpHDtZKT+ZDZOwDVtKHNGQJuhEnM3sZc8E9d411rtl9GwbRfXmnwEbdYi99+L6Ngmq15k+TisCHqJfWqlCM2xUJzMq+wkn3AJqe71N4t2oI9zfL4XxKa2OEXzus9jx2MFHAP80RZRH5JSAYi/glJgzpAygUPcbyIeEiYzvsQxafIFLC36AdtJj7DzhNrTg2RXCLe45hmPZb9uKXzZrwiFJr1cjqelUyFHR4SeCLRLuLbkggRFWYOWIv810Q0iw65SimllFJKKaVUe/4AalbOLn1CQYsAAAAASUVORK5CYII=">
		<h2>Horde</h2>
		
		@if (Model.ErrorMessage != null)
		{
			<div class="error">@Model.ErrorMessage</div>
		}
		<form action="@Model.FormPostUrl" method="post">
			<div class="form-group">
				<label for="username">Username</label>
				<input type="text" id="username" name="username" required>
			</div>
			<div class="form-group">
				<label for="password">Password</label>
				<input type="password" id="password" name="password">
			</div>
			<button type="submit" class="login-button">Log In</button>
			<div class="footer">
				To obtain an account or reset your password, contact your administrator.
			</div>
		</form>
	</div>
</div>