@model HordeServer.Server.AccountStateViewModel

<!DOCTYPE html>
<html lang="en">
<head>
	<title>Horde Account</title>
	<style>
		body { font-family: 'Segoe UI', 'Roboto', arial, sans-serif; }
		p { margin:20px; font-size:13px; }
		h1 { margin: 20px; font-size: 32px; font-weight: 200; }
		h2 { margin: 20px; font-size: 24px; font-weight: 200; }
		table { margin:10px 20px; }
		th { margin: 5px; font-size: 13px; vertical-align: top; padding: 3px; text-align: left; }
		td { margin:5px; font-size:13px; vertical-align: top; padding: 3px; font-family: Consolas, Courier New, monospace; }
	</style>
</head>
<body>
<h1>Horde Server</h1>

@if (Model.IsAuthenticated)
{
	<p>User <b>@Model.UserName</b> is logged in. <a href="/account/logout">Log out</a></p>
	
	@if (Model.IsAdmin)
	{
		<p>
			<a href="/api/v1/admin/token">Get bearer token</a><br/>
			<a href="/api/v1/admin/registrationtoken">Get agent registration token</a><br/>
			<a href="/api/v1/admin/softwaretoken">Get agent software upload token</a><br/>
			<a href="/api/v1/admin/softwaredownloadtoken">Get agent software download token</a><br/>
			<a href="/api/v1/admin/configtoken">Get configuration token</a><br/>
			<a href="/api/v1/admin/chainedjobtoken">Get chained job token</a><br/>
		</p>
	}
	
	<h2>Claims for @Model.UserName:</h2>
	<table>
		<tr>
			<th>Type</th><th>Value</th>
		</tr>
		@foreach (var claim in Model.Claims)
		{
			<tr>
				<td>@claim.Type</td>
				<td>@claim.Value</td>
			</tr>
		}
	</table>
	
	<h2>ACL Permissions</h2>
	<p>Only scopes with at least one authorized action are shown.</p>
	<table class="acl-scopes">
		<tr>
			<th>Scope</th><th>Action</th><th>Authorized</th>
		</tr>
		
		@foreach (var scopeGroup in Model.ScopeToPermissions)
		{
			var scope = scopeGroup.Key;
			var permissions = scopeGroup.Value;
			
			if (permissions.Count > 0)
			{
				for (int i = 0; i < permissions.Count; i++)
				{
					var uap = permissions[i];
					var color = uap.IsAuthorized ? "#c8ffc8" : "#ffc8c8";
					var authorized = uap.IsAuthorized ? "Yes" : "No";
					
					if (i == 0)
					{
						<tr>
							<td rowspan="@(permissions.Count)">@scope</td>
							<td>@uap.Action</td>
							<td style="background-color: @color; text-align: center;">@authorized</td>
						</tr>
					}
					else
					{
						<tr>
							<td>@uap.Action</td>
							<td style="background-color: @color; text-align: center;">@authorized</td>
						</tr>
					}
				}
				
				<tr style="border-bottom:1px solid black">
					<td colspan="100%"></td>
				</tr>
				<tr>
					<td colspan="100%" style="display: block; height: 20px;"></td>
				</tr>
			}
		}
		
		@if (Model.ScopeToPermissions.Count == 0)
		{
			<tr>
				<td colspan="100%">No scopes</td>
			</tr>
		}
	</table>
	
	<p>Built from Perforce</p>
}
else
{
	<p>
		<a href="/account/login">
			<b>Login</b>
		</a>
	</p>
}
</body>
</html>