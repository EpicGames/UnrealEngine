// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.IdentityModel.Tokens;
using MongoDB.Bson;

namespace HordeServer.Server
{
	/// <summary>
	/// Global server settings
	/// </summary>
	public interface IGlobals
	{
		/// <summary>
		/// Unique instance id of this database
		/// </summary>
		ObjectId InstanceId { get; }

		/// <summary>
		/// Issuer for JWT keys. Can be overridden by config file.
		/// </summary>
		string JwtIssuer { get; }

		/// <summary>
		/// The signing key for this server cluster
		/// </summary>
		public SecurityKey JwtSigningKey { get; }

		/// <summary>
		/// RSA security key for this cluster
		/// </summary>
		public RsaSecurityKey RsaSigningKey { get; }
	}
}
