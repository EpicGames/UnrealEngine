// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Server
{
	/// <summary>
	/// Controller for app lifetime related routes
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public class LifetimeController : ControllerBase
	{
		/// <summary>
		/// Singleton instance of the lifetime service
		/// </summary>
		readonly LifetimeService _lifetimeService;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="lifetimeService">The lifetime service singleton</param>
		public LifetimeController(LifetimeService lifetimeService)
		{
			_lifetimeService = lifetimeService;
		}

		/// <summary>
		/// Readiness check for server
		/// 
		/// When a SIGTERM has been received, the server is shutting down.
		/// To communicate the server is stopping to load balancers/orchestrators, this route will return either 503 or 200.
		/// The server will continue to serve requests, but it's assumed the load balancer has reacted and stopped sending
		/// traffic by the time the server process exits.
		/// </summary>
		/// <returns>Status code 503 is server is stopping, else 200 OK</returns>
		[HttpGet]
		[Route("/health/ready")]
		public Task<ActionResult> ServerReadinessAsync()
		{
			int statusCode = 200;
			string content = "ok";

			if (_lifetimeService.IsPreStopping)
			{
				statusCode = 503; // Service Unavailable
				content = "stopping";
			}

			return Task.FromResult<ActionResult>(new ContentResult { ContentType = "text/plain", StatusCode = statusCode, Content = content });
		}
	}
}
