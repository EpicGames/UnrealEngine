// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using StackExchange.Redis;

namespace HordeServer.Server
{
	/// <summary>
	/// Service containing an async task that allows long polling operations to complete early if the server is shutting down
	/// </summary>
	public sealed class LifetimeService : ILifetimeService, IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Writer for log output
		/// </summary>
		readonly ILogger<LifetimeService> _logger;

		/// <summary>
		/// Task source for the server stopping
		/// </summary>
		readonly TaskCompletionSource<bool> _stoppingTaskCompletionSource;

//		/// <summary>
//		/// Task source for the server stopping
//		/// </summary>
//		readonly TaskCompletionSource<bool> _preStoppingTaskCompletionSource;

		/// <summary>
		/// Registration token for the stopping event
		/// </summary>
		readonly CancellationTokenRegistration _registration;

		readonly IHostApplicationLifetime _lifetime;
		readonly IMongoService _mongoService;
		readonly IRedisService _redisService;
		readonly ITicker _ticker;

		/// <inheritdoc cref="ServerSettings.ShutdownMemoryThreshold" />
		readonly int? _shutdownMemoryThreshold = null;

		/*
		/// <summary>
		/// Max time to wait for any outstanding requests to finish
		/// </summary>
		readonly TimeSpan RequestGracefulTimeout = TimeSpan.FromMinutes(5);
		
		/// <summary>
		/// Initial delay before attempting the shutdown. This to ensure any load balancers/ingress will detect
		/// the server is unavailable to serve new requests in the event of no outstanding requests to wait for.
		/// </summary>
		readonly TimeSpan InitialStoppingDelay = TimeSpan.FromSeconds(35);
		*/

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="settings">Server settings</param>
		/// <param name="lifetime">Application lifetime interface</param>
		/// <param name="env">Current ASP.NET environment</param>
		/// <param name="mongoService">Database singleton service</param>
		/// <param name="redisService">Redis singleton service</param>
		/// <param name="clock"></param>
		/// <param name="logger">Logging interface</param>
		public LifetimeService(IOptionsMonitor<ServerSettings> settings, IHostApplicationLifetime lifetime, IHostEnvironment env, IMongoService mongoService, IRedisService redisService, IClock clock, ILogger<LifetimeService> logger)
		{
			_shutdownMemoryThreshold = settings.CurrentValue.ShutdownMemoryThreshold;
			_lifetime = lifetime;
			_mongoService = mongoService;
			_redisService = redisService;
			_logger = logger;
			_stoppingTaskCompletionSource = new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously);
//			_preStoppingTaskCompletionSource = new TaskCompletionSource<bool>(TaskCreationOptions.RunContinuationsAsynchronously);

			if (env.IsProduction() || env.IsDevelopment())
			{
				_registration = lifetime.ApplicationStopping.Register(ApplicationStopping);
			}

			_ticker = clock.AddTicker<LifetimeService>(TimeSpan.FromMinutes(5), CheckMemoryUsageAsync, _logger);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _registration.DisposeAsync();
			await _ticker.DisposeAsync();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		private ValueTask CheckMemoryUsageAsync(CancellationToken cancellationToken)
		{
			if (_shutdownMemoryThreshold == null || _shutdownMemoryThreshold <= 0)
			{
				return ValueTask.CompletedTask;
			}

			// Force a garbage collection and wait for it to complete for a more accurate reading
			// Can be a heavy operation but this ticker method is run infrequently
			GC.Collect();
			GC.WaitForPendingFinalizers();

			long totalMemoryUsageMb = GC.GetTotalMemory(forceFullCollection: false) / 1024 / 1024;
			if (totalMemoryUsageMb > _shutdownMemoryThreshold)
			{
				_logger.LogWarning("Memory usage exceeded {MemoryThreshold} MB. Stopping process...", _shutdownMemoryThreshold);
				_lifetime.StopApplication();
			}

			return ValueTask.CompletedTask;
		}

		/// <summary>
		/// Callback for the application stopping
		/// </summary>
		void ApplicationStopping()
		{
			_logger.LogInformation("Shutdown/SIGTERM signal received");
			IsPreStopping = true;
			IsStopping = true;

//			_preStoppingTaskCompletionSource.TrySetResult(true);
			_stoppingTaskCompletionSource.TrySetResult(true);
/*
			int shutdownDelayMs = 30 * 1000;
			_logger.LogInformation("Delaying shutdown by sleeping {ShutdownDelayMs} ms...", shutdownDelayMs);
			Thread.Sleep(shutdownDelayMs);
			_logger.LogInformation("Server process now shutting down...");
*/
			/*
			if (PreStoppingTaskCompletionSource.TrySetResult(true))
			{
				
				WaitAndTriggerStoppingTask = Task.Run(() => ExecStoppingTask());
				
				Logger.LogInformation("App is stopping. Waiting an initial {InitialDelay} secs before waiting on any requests...", (int)InitialStoppingDelay.TotalSeconds);
				Thread.Sleep(InitialStoppingDelay);
				Logger.LogInformation("Blocking shutdown for up to {MaxGraceTimeout} secs until all request have finished...", (int)RequestGracefulTimeout.TotalSeconds);
				
				DateTime StartTime = DateTime.UtcNow;
				do
				{
					RequestTrackerService.LogRequestsInProgress();
					Thread.Sleep(5000);
				}
				while (DateTime.UtcNow < StartTime + RequestGracefulTimeout && RequestTrackerService.GetRequestsInProgress().Count > 0);

				if (RequestTrackerService.GetRequestsInProgress().Count == 0)
				{
					Logger.LogInformation("All open requests finished gracefully after {TimeTaken} secs", (DateTime.UtcNow - StartTime).TotalSeconds);	
				}
				else
				{
					Logger.LogInformation("One or more requests did not finish within the grace period of {TimeTaken} secs. Shutdown will now resume with risk of interrupting those requests!", (DateTime.UtcNow - StartTime).TotalSeconds);
					RequestTrackerService.LogRequestsInProgress();
				}
			}
			*/
		}

		/// <summary>
		/// Returns true if the server is stopping
		/// </summary>
		public bool IsStopping { get; private set; }

		/// <summary>
		/// Returns true if the server is stopping, but may not be removed from the load balancer yet
		/// </summary>
		public bool IsPreStopping { get; private set; }

		/// <summary>
		/// Gets an awaitable task for the server stopping
		/// </summary>
		public Task StoppingTask => _stoppingTaskCompletionSource.Task;

		/// <summary>
		/// Check if MongoDB can be reached
		/// </summary>
		/// <returns>True if communication works</returns>
		public async Task<bool> IsMongoDbConnectionHealthyAsync()
		{
			using CancellationTokenSource cancelSource = new CancellationTokenSource(10000);
			bool isHealthy = false;
			try
			{
				await _mongoService.Database.ListCollectionNamesAsync(null, cancelSource.Token);
				isHealthy = true;
			}
			catch (Exception e)
			{
				_logger.LogError(e, "MongoDB call failed during health check");
			}

			return isHealthy;
		}

		/// <summary>
		/// Check if Redis can be reached
		/// </summary>
		/// <returns>True if communication works</returns>
		public async Task<bool> IsRedisConnectionHealthyAsync()
		{
			using CancellationTokenSource cancelSource = new CancellationTokenSource(10000);
			bool isHealthy = false;
			try
			{
				string key = "HordeLifetimeService-Health-Check";
				IDatabase redis = _redisService.GetDatabase();
				await redis.StringSetAsync(key, "ok");
				await redis.StringGetAsync(key);
				isHealthy = true;
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Redis call failed during health check");
			}

			return isHealthy;
		}
	}
}