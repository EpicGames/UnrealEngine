// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.Metrics;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Hosting;

namespace HordeServer.Server
{
	/// <summary>
	/// Periodically send metrics for the CLR and other services that cannot be collected on a per-request basis
	/// </summary>
	public sealed class MetricService : IHostedService, IDisposable
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public MetricService(Meter meter)
		{
			// Force a collection to ensure metrics are populated as some of them require a GC to have been run
			GC.Collect();

			meter.CreateObservableGauge("horde.clr.threadpool.io.min", () => new ThreadMetrics().IoMin);
			meter.CreateObservableGauge("horde.clr.threadpool.io.max", () => new ThreadMetrics().IoMax);
			meter.CreateObservableGauge("horde.clr.threadpool.io.free", () => new ThreadMetrics().IoFree);
			meter.CreateObservableGauge("horde.clr.threadpool.io.busy", () => new ThreadMetrics().IoBusy);

			meter.CreateObservableGauge("horde.clr.threadpool.worker.min", () => new ThreadMetrics().WorkerMin);
			meter.CreateObservableGauge("horde.clr.threadpool.worker.max", () => new ThreadMetrics().WorkerMax);
			meter.CreateObservableGauge("horde.clr.threadpool.worker.free", () => new ThreadMetrics().WorkerFree);
			meter.CreateObservableGauge("horde.clr.threadpool.worker.busy", () => new ThreadMetrics().WorkerBusy);

			meter.CreateObservableGauge("horde.clr.gc.totalMemory", () => GC.GetTotalMemory(false));
			meter.CreateObservableGauge("horde.clr.gc.totalAllocated", () => GC.GetTotalAllocatedBytes());
			meter.CreateObservableGauge("horde.clr.gc.heapSize", () => GC.GetGCMemoryInfo().HeapSizeBytes);
			meter.CreateObservableGauge("horde.clr.gc.fragmented", () => GC.GetGCMemoryInfo().FragmentedBytes);
			meter.CreateObservableGauge("horde.clr.gc.memoryLoad", () => GC.GetGCMemoryInfo().MemoryLoadBytes);
			meter.CreateObservableGauge("horde.clr.gc.totalAvailableMemory", () => GC.GetGCMemoryInfo().TotalAvailableMemoryBytes);
			meter.CreateObservableGauge("horde.clr.gc.highMemoryLoadThreshold", () => GC.GetGCMemoryInfo().HighMemoryLoadThresholdBytes);
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) { return Task.CompletedTask; }

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) { return Task.CompletedTask; }

		/// <inheritdoc/>
		public void Dispose() { }

		private struct ThreadMetrics
		{
			public readonly int WorkerMin;
			public readonly int WorkerMax;
			public readonly int WorkerFree;
			public readonly int WorkerBusy;
			public readonly int IoMin;
			public readonly int IoMax;
			public readonly int IoFree;
			public readonly int IoBusy;

			public ThreadMetrics()
			{
				ThreadPool.GetMinThreads(out WorkerMin, out IoMin);
				ThreadPool.GetMaxThreads(out WorkerMax, out IoMax);
				ThreadPool.GetAvailableThreads(out WorkerFree, out IoFree);
				WorkerBusy = WorkerMax - WorkerFree;
				IoBusy = WorkerMax - WorkerFree;
			}
		}
	}
}
