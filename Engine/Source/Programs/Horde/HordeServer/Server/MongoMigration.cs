// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Threading;
using System.Threading.Tasks;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer.Server;

/// <summary>
/// Indicates the version relationship between the application's defined schema and the database's applied schema.
/// Used to determine if migrations or code updates are required.
/// </summary>
public enum SchemaVersionState
{
	/// <summary>
	/// The application and database schemas are at the same version.
	/// <remarks>
	/// - No action required
	/// - Application can operate normally
	/// - Represents the ideal state
	/// </remarks>
	/// </summary>
	VersionMatch = 0,
	
	/// <summary>
	/// The application schema version is newer than the database version.
	/// <remarks>
	/// - Database requires migration
	/// - Application should not run or process requests
	/// - Requires manual (or automated) migration intervention
	/// </remarks>
	/// </summary>
	DatabaseOutdated,
	
	/// <summary>
	/// The database schema version is newer than the application version.
	/// <remarks>
	/// - Application requires deployment of a newer version of itself
	/// - Can be a valid temporary state during rolling updates
	/// - Safe to continue operation
	/// </remarks>
	/// </summary>
	ApplicationOutdated,
}

/// <summary>
/// Results of comparing application and database schema versions
/// </summary>
/// <param name="State">Current version state</param>
/// <param name="AppVersion">Application schema version</param>
/// <param name="DatabaseVersion">Database schema version</param>
/// <param name="MissingMigrations">List of migrations needed to reconcile versions</param>
public record SchemaVersionResult(SchemaVersionState State, int AppVersion, int DatabaseVersion, List<MigrationDefinition> MissingMigrations);

/// <summary>
/// A database migration with its metadata
/// </summary>
public class MigrationDefinition(IMongoMigration migration, string plugin, int version, string name)
{
	/// <inheritdoc cref="IMongoMigration"/>
	public IMongoMigration Migration { get; } = migration;
	
	/// <inheritdoc cref="MongoMigrationAttribute.Plugin"/>
	public string Plugin { get; } = plugin;
	
	/// <inheritdoc cref="MongoMigrationAttribute.Version"/>
	public int Version { get; } = version;
	
	/// <inheritdoc cref="MongoMigrationAttribute.Name"/>
	public string Name { get; } = name;
	
	/// <inheritdoc/>
	public override string ToString()
	{
		return $"Plugin: {Plugin}, Version: {Version}, Name: {Name}";
	}
	
	/// <inheritdoc/>
	public override bool Equals(object? obj)
	{
		if (ReferenceEquals(null, obj))
		{
			return false;
		}
		
		if (ReferenceEquals(this, obj))
		{
			return true;
		}
		
		return obj.GetType() == GetType() && Equals((MigrationDefinition)obj);
	}
	
	/// <inheritdoc/>
	public override int GetHashCode()
	{
		return HashCode.Combine(Plugin, Version);
	}
	
	private bool Equals(MigrationDefinition other)
	{
		return Plugin == other.Plugin && Version == other.Version;
	}
}

/// <summary>
/// Exception thrown during migration operations
/// </summary>
public class MigrationException : Exception
{
	/// <inheritdoc/>
	public MigrationException(string message) : base(message) {}
	
	/// <inheritdoc/>
	public MigrationException(string message, Exception innerException) : base(message, innerException) { }
}

/// <summary>
/// Handles database schema migrations for MongoDB database
/// This class is not thread-safe
/// </summary>
public class MongoMigrator
{
	private readonly Dictionary<string, HashSet<MigrationDefinition>> _migrations = [];
	private readonly IMongoService _mongoService;
	private readonly ILogger<MongoMigrator> _logger;
	
	[SingletonDocument("mongo-migrations")]
	private class MigrationState : SingletonBase
	{
		public Dictionary<string, int> Versions { get; set; } = new ();
	}
	
	/// <summary>
	/// Constructor
	/// </summary>
	public MongoMigrator(IMongoService mongoService, ILogger<MongoMigrator> logger)
	{
		_mongoService = mongoService;
		_logger = logger;
	}
	
	/// <summary>
	/// Load all migrations
	/// Will scan all loaded assemblies via reflection and add any class decorated with MongoMigrationAttribute
	/// </summary>
	/// <exception cref="MigrationException">If migration is invalid or conflicts</exception>
	public void AutoAddMigrations()
	{
		foreach (Assembly assembly in AppDomain.CurrentDomain.GetAssemblies())
		{
			// Skip most assemblies as iterating types is slow
			bool isHordePlugin = assembly.GetName().Name?.Contains("Horde", StringComparison.InvariantCultureIgnoreCase) ?? false;
			if (isHordePlugin)
			{
				foreach (Type type in assembly.GetTypes())
				{
					MongoMigrationAttribute? mma = type.GetCustomAttribute<MongoMigrationAttribute>();
					if (mma is {AutoLoad: true} && Activator.CreateInstance(type) is IMongoMigration migration)
					{
						AddMigration(migration);
					}
				}
			}
		}
	}
	
	/// <summary>
	/// Adds a new migration to be managed by this migrator
	/// </summary>
	/// <param name="migration">Migration implementation to add</param>
	/// <exception cref="MigrationException">If migration is invalid or conflicts</exception>
	public void AddMigration(IMongoMigration migration)
	{
		MongoMigrationAttribute? attribute = migration.GetType().GetCustomAttribute<MongoMigrationAttribute>();
		if (attribute == null)
		{
			throw new MigrationException($"Migration class {migration.GetType()} is missing [MongoMigration] attribute");
		}

		MigrationDefinition definition = new (migration, attribute.Plugin, attribute.Version, attribute.Name);
		if (!_migrations.TryGetValue(definition.Plugin, out HashSet<MigrationDefinition>? pluginMigrations))
		{
			pluginMigrations = new HashSet<MigrationDefinition>();
			_migrations[definition.Plugin] = pluginMigrations;
		}

		if (definition.Version is < 1 or > 50000)
		{
			throw new MigrationException($"Migration {definition.Plugin}/{definition.Name} has an invalid version number {definition.Version}");
		}
		
		if (definition.Name.Length is <= 3 or > 200)
		{
			throw new MigrationException($"Migration {definition.Plugin}/{definition.Name} has a too short/long name");
		}

		if (!pluginMigrations.Add(definition))
		{
			throw new MigrationException($"Migrations for plugin {definition.Plugin} already contains a migration with version {definition.Version}");
		}
	}
	
	/// <summary>
	/// Validates database schema versions for all plugins against application versions
	/// Logs error for outdated schemas with migration instructions.
	/// </summary>
	/// <param name="autoUpgrade">Whether automatic upgrade of the schemas should be performed</param>
	/// <param name="cancellationToken">Token to cancel the validation operation</param>
	/// <returns>True if all schemas are valid or newer than app, false if any schema is outdated</returns>
	public async Task<bool> ValidateAllSchemasAsync(bool autoUpgrade, CancellationToken cancellationToken)
	{
		Dictionary<string, SchemaVersionResult> schemaResults = await GetAllSchemaStateAsync(cancellationToken);
		bool databaseOutdated = false;
		foreach ((string plugin, SchemaVersionResult svr) in schemaResults)
		{
			string dbVersion = svr.DatabaseVersion > 0 ? Convert.ToString(svr.DatabaseVersion) : "(not applied)";
			switch (svr.State)
			{
				case SchemaVersionState.VersionMatch: continue;
				case SchemaVersionState.DatabaseOutdated:
					_logger.Log(autoUpgrade ? LogLevel.Warning : LogLevel.Error, "Database schema for plugin {Plugin} is outdated. Schema versions: app = {AppVersion} database = {DatabaseVersion}", plugin, svr.AppVersion, dbVersion);
					databaseOutdated = true;
					break;
				case SchemaVersionState.ApplicationOutdated:
					_logger.LogWarning("Database schema for plugin {Plugin} newer than app. Schema versions: app = {AppVersion} database = {DatabaseVersion}", plugin, svr.AppVersion, dbVersion);
					break;
				default: throw new ArgumentOutOfRangeException($"Unknown schema state {svr}");
			}
		}
		
		if (autoUpgrade)
		{
			_logger.LogInformation("Automatically upgrading database schemas...");
			await UpgradeAsync(cancellationToken);
		}
		else if (databaseOutdated)
		{
			_logger.LogInformation("To apply migration scripts, run: dotnet HordeServer.dll mongo upgrade (or equivalent)");
			_logger.LogInformation("For dev or test deployments, auto-apply of migrations can be enabled in server settings. This setting is *not* recommended for production use.");
		}
		
		return !databaseOutdated;
	}
	
	/// <summary>
	/// Retrieves the schema version states for all registered plugins
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>Schema version comparison results per plugin</returns>
	public async Task<Dictionary<string, SchemaVersionResult>> GetAllSchemaStateAsync(CancellationToken cancellationToken)
	{
		Dictionary<string, SchemaVersionResult> result = new ();
		foreach (string plugin in _migrations.Keys)
		{
			SchemaVersionResult svr = await GetSchemaStateAsync(plugin, cancellationToken);
			result[plugin] = svr;
		}
		
		return result;
	}
	
	/// <summary>
	/// Checks the current schema version state for a plugin
	/// </summary>
	/// <param name="plugin">Plugin identifier</param>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>Schema version comparison results</returns>
	public async Task<SchemaVersionResult> GetSchemaStateAsync(string plugin, CancellationToken cancellationToken)
	{
		IReadOnlyList<MigrationDefinition> migrations = GetMigrations(plugin);
		int appVersion = migrations.Count == 0 ? -1 : migrations.Max(x => x.Version);
		int dbVersion = await GetVersionAsync(plugin, cancellationToken);
		
		if (appVersion > dbVersion)
		{
			List<MigrationDefinition> missingMigrations = migrations
				.Where(x => x.Version > dbVersion && x.Version <= appVersion)
				.OrderBy(x => x.Version)
				.ToList();
			
			return new SchemaVersionResult(SchemaVersionState.DatabaseOutdated, appVersion, dbVersion, missingMigrations);
		}
		if (appVersion < dbVersion)
		{
			List<MigrationDefinition> missingMigrations = [];
			for (int v = appVersion + 1; v <= dbVersion; v++)
			{
				missingMigrations.Add(new MigrationDefinition(null!, plugin, v, $"Unknown migration {v}"));
			}
			
			return new SchemaVersionResult(SchemaVersionState.ApplicationOutdated, appVersion, dbVersion, missingMigrations);
		}
		
		return new SchemaVersionResult(SchemaVersionState.VersionMatch, appVersion, dbVersion, []);
	}
	
	/// <summary>
	/// Gets all migrations registered for a plugin
	/// </summary>
	/// <param name="plugin">Plugin identifier</param>
	/// <returns>List of migrations ordered by version</returns>
	/// <exception cref="MigrationException">If plugin is unknown</exception>
	public IReadOnlyList<MigrationDefinition> GetMigrations(string plugin)
	{
		if (_migrations.TryGetValue(plugin, out HashSet<MigrationDefinition>? pluginMigrations))
		{
			return new List<MigrationDefinition>(pluginMigrations).OrderBy(x => x.Version).ToList();
		}
		
		throw new MigrationException($"Unknown plugin {plugin}");
	}
	
	/// <summary>
	/// Upgrades all database schemas to latest version
	/// </summary>
	/// <param name="cancellationToken">Cancellation token</param>
	public async Task UpgradeAsync(CancellationToken cancellationToken)
	{
		foreach (string plugin in _migrations.Keys)
		{
			await UpgradeAsync(plugin, toVersion: null, cancellationToken);
		}
	}
	
	/// <summary>
	/// Upgrades database schema for a plugin to specified version or latest if not specified
	/// </summary>
	/// <param name="plugin">Plugin identifier</param>
	/// <param name="toVersion">Target version, or null for latest</param>
	/// <param name="cancellationToken">Cancellation token</param>
	public async Task UpgradeAsync(string plugin, int? toVersion, CancellationToken cancellationToken)
	{
		IReadOnlyList<MigrationDefinition> migrations = GetMigrations(plugin);
		ValidateMigrations(plugin, migrations);
		int appliedVersion = await GetVersionAsync(plugin, cancellationToken);
		MigrationContext context = new (_mongoService, _logger);
		
		foreach (MigrationDefinition mi in migrations.Where(x => x.Version > appliedVersion && (!toVersion.HasValue || x.Version <= toVersion.Value)))
		{
			try
			{
				_logger.LogInformation("Upgrading {Plugin} to version {Version} using migration: {Name} ({ClassName})", mi.Plugin, mi.Version, mi.Name, mi.Migration.GetType().FullName);
				await mi.Migration.UpAsync(context, cancellationToken);
				await SetVersionAsync(plugin, mi.Version, cancellationToken);
			}
			catch (Exception e)
			{
				throw new MigrationException($"Failed to upgrade {mi.Plugin} to version {mi.Version} using migration: {mi.Name}", e);
			}
		}
	}
	
	/// <summary>
	/// Downgrades database schema for a plugin to specified version
	/// </summary>
	/// <param name="plugin">Plugin identifier</param>
	/// <param name="toVersion">Target version</param>
	/// <param name="cancellationToken">Cancellation token</param>
	public async Task DowngradeAsync(string plugin, int toVersion, CancellationToken cancellationToken)
	{
		IReadOnlyList<MigrationDefinition> migrations = GetMigrations(plugin);
		ValidateMigrations(plugin, migrations);
		MigrationContext context = new (_mongoService, _logger);
		
		IEnumerable<MigrationDefinition> migrationsToApply = migrations
			.OrderByDescending(x => x.Version).Where(x => x.Version > toVersion);
		
		foreach (MigrationDefinition mi in migrationsToApply)
		{
			try
			{
				_logger.LogInformation("Downgrading {Plugin} to version {Version} using migration: {Name}", mi.Plugin, mi.Version, mi.Name);
				await mi.Migration.DownAsync(context, cancellationToken);
				await SetVersionAsync(plugin, mi.Version, cancellationToken);
			}
			catch (Exception e)
			{
				throw new MigrationException($"Failed to downgrade {mi.Plugin} to version {mi.Version} using migration: {mi.Name}", e);
			}
		}
	}
	
	private static void ValidateMigrations(string plugin, IReadOnlyList<MigrationDefinition> migrations)
	{
		if (migrations.Count == 0)
		{
			throw new MigrationException($"No migrations found for {plugin}");
		}
		
		if (migrations[0].Version != 1)
		{
			throw new MigrationException($"Missing migration version 1 for {plugin}");
		}
		
		// Check for gaps in version numbers
		for (int i = 0; i < migrations.Count - 1; i++)
		{
			int currentVersion = migrations[i].Version;
			int nextVersion = migrations[i + 1].Version;
			
			if (nextVersion != currentVersion + 1)
			{
				throw new MigrationException($"Gap detected in migration versions for {plugin}. Expected version {currentVersion + 1} but found version {nextVersion}");
			}
		}
	}
	
	private async Task SetVersionAsync(string plugin, int version, CancellationToken cancellationToken)
	{
		await _mongoService.UpdateSingletonAsync<MigrationState>(state => state.Versions[plugin] = version, cancellationToken);
	}
	
	/// <summary>
	/// Get currently applied version of migrations for a plugin as stored in database
	/// </summary>
	private async Task<int> GetVersionAsync(string plugin, CancellationToken cancellationToken)
	{
		MigrationState state = await _mongoService.GetSingletonAsync<MigrationState>(cancellationToken);
		return state.Versions.TryGetValue(plugin, out int version) ? version : -1;
	}
}