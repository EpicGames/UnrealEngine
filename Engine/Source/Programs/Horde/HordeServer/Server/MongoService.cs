// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Globalization;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Net.Security;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Security.Cryptography.X509Certificates;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Redis.Utility;
using HordeServer.Utilities;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Driver;
using MongoDB.Driver.Core.Events;
using MongoDB.Driver.Linq;
using OpenTelemetry.Trace;
using StackExchange.Redis;

namespace HordeServer.Server
{
	/// <summary>
	/// Task used to update indexes
	/// </summary>
	record class MongoUpgradeTask(Func<CancellationToken, Task> UpgradeAsync, TaskCompletionSource CompletionSource);

	/// <summary>
	/// Singleton for accessing the database
	/// </summary>
	public sealed class MongoService : IMongoService, IHealthCheck, IAsyncDisposable
	{
		/// <summary>
		/// The database instance
		/// </summary>
		public IMongoDatabase Database { get; private set; }

		/// <summary>
		/// Collection of singleton documents
		/// </summary>
		IMongoCollection<BsonDocument> SingletonsV1 { get; }

		/// <summary>
		/// Collection of singleton documents
		/// </summary>
		IMongoCollection<BsonDocument> SingletonsV2 { get; }

		/// <summary>
		/// Settings for the application
		/// </summary>
		ServerSettings Settings { get; }

		/// <summary>
		/// Logger for this instance
		/// </summary>
		readonly ILogger<MongoService> _logger;

		/// <summary>
		/// Access the database in a read-only mode (don't create indices or modify content)
		/// </summary>
		public bool ReadOnlyMode { get; }

		/// <summary>
		/// The mongo process group
		/// </summary>
		ManagedProcessGroup? _mongoProcessGroup;

		/// <summary>
		/// The mongo process
		/// </summary>
		ManagedProcess? _mongoProcess;

		/// <summary>
		/// Task to read from the mongo process
		/// </summary>
		Task? _mongoOutputTask;

		/// <summary>
		/// Factory for creating logger instances
		/// </summary>
		readonly ILoggerFactory _loggerFactory;

		/// <summary>
		/// Tracer
		/// </summary>
		readonly Tracer _tracer;

		/// <summary>
		/// Default port for MongoDB connections
		/// </summary>
		const int DefaultMongoPort = 27017;

		static readonly RedisKey s_schemaLockKey = new RedisKey("server/schema-upgrade/lock");

		readonly MongoClient _client;
		readonly IRedisService _redisService;
#pragma warning disable CA2213 // Disposable fields should be disposed
		readonly SemaphoreSlim _upgradeSema = new SemaphoreSlim(1);
#pragma warning restore CA2213 // Disposable fields should be disposed
		CancellationTokenSource _upgradeCancellationSource = new CancellationTokenSource();
		readonly Dictionary<string, Task> _collectionUpgradeTasks = new Dictionary<string, Task>(StringComparer.Ordinal);
		readonly Task<bool> _setSchemaVersionTask;

		static string? s_existingInstance;

		/// <summary>
		/// Constructor
		/// </summary>
		public MongoService(IOptions<ServerSettings> settingsSnapshot, IRedisService redisService, MongoCommandTracer mongoTracer, Tracer tracer, ILogger<MongoService> logger, ILoggerFactory loggerFactory)
		{
			if (s_existingInstance != null)
			{
				throw new Exception("Existing instance on MongoService!");
			}

			s_existingInstance = Environment.StackTrace;

			Settings = settingsSnapshot.Value;
			_redisService = redisService;
			_tracer = tracer;
			_logger = logger;
			_loggerFactory = loggerFactory;

			try
			{
				ReadOnlyMode = Settings.MongoReadOnlyMode;
				if (Settings.MongoPublicCertificate != null)
				{
					X509Store localTrustStore = new X509Store(StoreName.Root);
					try
					{
						localTrustStore.Open(OpenFlags.ReadWrite);

						X509Certificate2Collection collection = ImportCertificateBundle(Settings.MongoPublicCertificate);
						foreach (X509Certificate2 certificate in collection)
						{
							_logger.LogInformation("Importing certificate for {Subject}", certificate.Subject);
						}

						localTrustStore.AddRange(collection);
					}
					finally
					{
						localTrustStore.Close();
					}
				}

				string? connectionString = Settings.MongoConnectionString;
				if (connectionString == null)
				{
					if (IsPortInUse(DefaultMongoPort))
					{
						connectionString = "mongodb://localhost:27017";
					}
					else if (TryStartMongoServer(_logger))
					{
						connectionString = "mongodb://localhost:27017/?readPreference=primary&appname=Horde&ssl=false";
					}
					else
					{
						throw new Exception($"Unable to connect to MongoDB server. Setup a MongoDB server and set the connection string in {ServerApp.ServerConfigFile}");
					}
				}

				MongoClientSettings mongoSettings = MongoClientSettings.FromConnectionString(connectionString);
				mongoSettings.ClusterConfigurator = clusterBuilder =>
				{
					if (_logger.IsEnabled(LogLevel.Trace))
					{
						clusterBuilder.Subscribe<CommandStartedEvent>(ev => TraceMongoCommand(ev.Command));
					}

					if (settingsSnapshot.Value.OpenTelemetry.Enabled)
					{
						mongoTracer.Register(clusterBuilder);
					}
				};

				mongoSettings.LinqProvider = LinqProvider.V2;
				mongoSettings.SslSettings = new SslSettings();
				mongoSettings.SslSettings.ServerCertificateValidationCallback = CertificateValidationCallBack;

				// Increase defaults to handle larger deployments of Horde interfacing with MongoDB
				// Making room for slower running commands potentially hogging the queue 
				mongoSettings.MinConnectionPoolSize = 50; // Default is 0
				mongoSettings.MaxConnectionPoolSize = 300; // Default is 100
#pragma warning disable CS0618 // Type or member is obsolete
				mongoSettings.WaitQueueSize = 5000;
#pragma warning restore CS0618 // Type or member is obsolete
				mongoSettings.MaxConnecting = 10; // Default is 2

				//TestSslConnection(MongoSettings.Server.Host, MongoSettings.Server.Port, Logger);

				_client = new MongoClient(mongoSettings);
				Database = _client.GetDatabase(Settings.MongoDatabaseName);

				SingletonsV1 = GetCollection<BsonDocument>("Singletons");
				SingletonsV2 = GetCollection<BsonDocument>("SingletonsV2");
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while initializing MongoService");
				throw;
			}

			_setSchemaVersionTask = SetSchemaVersionAsync(ServerApp.Version, CancellationToken.None);
		}

		internal const int CtrlCEvent = 0;

		[DllImport("kernel32.dll")]
		internal static extern bool GenerateConsoleCtrlEvent(int eventId, int processGroupId);

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_upgradeCancellationSource != null)
			{
				await _upgradeCancellationSource.CancelAsync();

				if (_collectionUpgradeTasks.Count > 0)
				{
					_logger.LogInformation("Waiting for upgrade tasks to cancel...");
					try
					{
						await Task.WhenAll(_collectionUpgradeTasks.Values);
					}
					catch (Exception ex)
					{
						_logger.LogInformation(ex, "Discarded upgrade task exception: {Message}", ex.Message);
					}
				}

				_upgradeCancellationSource.Dispose();
				_upgradeCancellationSource = null!;
			}

			if (_mongoProcess != null)
			{
				_logger.LogInformation("Stopping MongoDB...");
				try
				{
					_logger.LogDebug("  Sent shutdown command");
					IMongoDatabase adminDb = _client.GetDatabase("admin");
					await adminDb.RunCommandAsync(new JsonCommand<BsonDocument>("{shutdown: 1}"));
				}
				catch
				{
					// Ignore errors due to connection termination
				}

				try
				{
					_logger.LogInformation("  Waiting for MongoDB to exit");
					await _mongoProcess.WaitForExitAsync().WaitAsync(TimeSpan.FromSeconds(5.0));

					_mongoProcess.Dispose();
					_mongoProcess = null;

					if (_mongoOutputTask != null)
					{
						_logger.LogInformation("  Waiting for logger task");
						await _mongoOutputTask;
						_mongoOutputTask = null;
					}
				}
				catch (Exception ex)
				{
					_logger.LogInformation(ex, "  Unable to terminate mongo process: {Message}", ex.Message);
				}
				_logger.LogInformation("Done");
			}

			if (_mongoProcessGroup != null)
			{
				_mongoProcessGroup.Dispose();
				_mongoProcessGroup = null;
			}

			//_upgradeSema.Dispose();
			s_existingInstance = null;
		}

		/// <summary>
		/// Checks if the given port is in use
		/// </summary>
		/// <param name="portNumber"></param>
		/// <returns></returns>
		static bool IsPortInUse(int portNumber)
		{
			IPGlobalProperties ipGlobalProperties = IPGlobalProperties.GetIPGlobalProperties();

			IPEndPoint[] listeners = ipGlobalProperties.GetActiveTcpListeners();
			if (listeners.Any(x => x.Port == portNumber))
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Attempts to start a local instance of MongoDB
		/// </summary>
		/// <param name="logger"></param>
		/// <returns></returns>
		bool TryStartMongoServer(ILogger logger)
		{
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return false;
			}

			FileReference mongoExe = FileReference.Combine(ServerApp.AppDir, "ThirdParty", "Mongo", "mongod.exe");
			if (!FileReference.Exists(mongoExe))
			{
				logger.LogWarning("Unable to find Mongo executable.");
				return false;
			}

			DirectoryReference mongoDir = DirectoryReference.Combine(ServerApp.DataDir, "Mongo");

			DirectoryReference mongoDataDir = DirectoryReference.Combine(mongoDir, "Data");
			DirectoryReference.CreateDirectory(mongoDataDir);

			FileReference mongoLogFile = FileReference.Combine(mongoDir, "mongod.log");

			FileReference configFile = FileReference.Combine(mongoDir, "mongod.conf");

			// Check mongo log and config files for non-latin characters, mongodb.exe cannout read these paths
			if (Regex.IsMatch(mongoLogFile.ToString(), "[^a-zA-Z0-9:\\\\\\/.\\-_]"))
			{
				logger.LogError("MongoDB log file contains non-latin characters, which mongodb.exe cannot read: {MongoLogFile}", mongoLogFile.ToString());
			}

			if (Regex.IsMatch(configFile.ToString(), "[^a-zA-Z0-9:\\\\\\/.\\-_]"))
			{
				logger.LogError("MongoDB config file contains non-latin characters, which mongodb.exe cannot read: {MongoConfigFile}", configFile.ToString());
			}

			if (!FileReference.Exists(configFile))
			{
				DirectoryReference.CreateDirectory(configFile.Directory);
				using (StreamWriter writer = new StreamWriter(configFile.FullName))
				{
					writer.WriteLine("# mongod.conf");
					writer.WriteLine();
					writer.WriteLine("# for documentation of all options, see:");
					writer.WriteLine("# http://docs.mongodb.org/manual/reference/configuration-options/");
					writer.WriteLine();
					writer.WriteLine("storage:");
					writer.WriteLine("    dbPath: {0}", mongoDataDir.FullName);
					writer.WriteLine();
					writer.WriteLine("net:");
					writer.WriteLine("    port: {0}", DefaultMongoPort);
					writer.WriteLine("    bindIp: 127.0.0.1");
				}
			}

			_mongoProcessGroup = new ManagedProcessGroup();
			try
			{
				_mongoProcess = new ManagedProcess(_mongoProcessGroup, mongoExe.FullName, $"--config \"{configFile}\"", null, null, ProcessPriorityClass.Normal);
				_mongoProcess.StdIn.Close();
				_mongoOutputTask = Task.Run(() => RelayMongoOutputAsync());
				return true;
			}
			catch (Exception ex)
			{
				logger.LogWarning(ex, "Unable to start Mongo server process");
				return false;
			}
		}

		/// <summary>
		/// Copies output from the mongo process to the logger
		/// </summary>
		/// <returns></returns>
		async Task RelayMongoOutputAsync()
		{
			ILogger mongoLogger = _loggerFactory.CreateLogger("MongoDB");

			Dictionary<string, ILogger> channelLoggers = new Dictionary<string, ILogger>();
			for (; ; )
			{
				string? line = await _mongoProcess!.ReadLineAsync();
				if (line == null)
				{
					break;
				}
				if (line.Length > 0)
				{
					Match match = Regex.Match(line, @"^\s*[^\s]+\s+([A-Z])\s+([^\s]+)\s+(.*)");
					if (match.Success)
					{
						ILogger? channelLogger;
						if (!channelLoggers.TryGetValue(match.Groups[2].Value, out channelLogger))
						{
							channelLogger = _loggerFactory.CreateLogger($"MongoDB.{match.Groups[2].Value}");
							channelLoggers.Add(match.Groups[2].Value, channelLogger);
						}
						channelLogger.Log(ParseMongoLogLevel(match.Groups[1].Value), "{Message}", match.Groups[3].Value.TrimEnd());
					}
					else
					{
						mongoLogger.Log(LogLevel.Information, "{Message}", line);
					}
				}
			}
			mongoLogger.LogInformation("Exit code {ExitCode}", _mongoProcess.ExitCode);
		}

		static LogLevel ParseMongoLogLevel(string text)
		{
			if (text.Equals("I", StringComparison.Ordinal))
			{
				return LogLevel.Information;
			}
			else if (text.Equals("E", StringComparison.Ordinal))
			{
				return LogLevel.Error;
			}
			else
			{
				return LogLevel.Warning;
			}
		}

		/// <summary>
		/// Logs a mongodb command, removing any fields we don't care about
		/// </summary>
		/// <param name="command">The command document</param>
		void TraceMongoCommand(BsonDocument command)
		{
			List<string> names = new List<string>();
			List<string> values = new List<string>();

			foreach (BsonElement element in command)
			{
				if (element.Value != null && !element.Name.Equals("$db", StringComparison.Ordinal) && !element.Name.Equals("lsid", StringComparison.Ordinal))
				{
					names.Add($"{element.Name}: {{{values.Count}}}");
					values.Add(element.Value.ToString()!);
				}
			}

#pragma warning disable CA2254 // Template should be a static expression
			_logger.LogTrace($"MongoDB: {String.Join(", ", names)}", values.ToArray());
#pragma warning restore CA2254 // Template should be a static expression
		}

		/// <summary>
		/// Imports one or more certificates from a single PEM file
		/// </summary>
		/// <param name="fileName">File to import</param>
		/// <returns>Collection of certificates</returns>
		static X509Certificate2Collection ImportCertificateBundle(string fileName)
		{
			X509Certificate2Collection collection = new X509Certificate2Collection();

			string text = File.ReadAllText(fileName);
			for (int offset = 0; offset < text.Length;)
			{
				int nextOffset = text.IndexOf("-----BEGIN CERTIFICATE-----", offset + 1, StringComparison.Ordinal);
				if (nextOffset == -1)
				{
					nextOffset = text.Length;
				}

				string certificateText = text.Substring(offset, nextOffset - offset);
				collection.Add(X509CertificateLoader.LoadCertificate(Encoding.UTF8.GetBytes(certificateText)));

				offset = nextOffset;
			}

			return collection;
		}

		/// <summary>
		/// Provides additional diagnostic information for SSL certificate validation
		/// </summary>
		/// <param name="sender"></param>
		/// <param name="certificate"></param>
		/// <param name="chain"></param>
		/// <param name="sslPolicyErrors"></param>
		/// <returns>True if the certificate is allowed, false otherwise</returns>
		bool CertificateValidationCallBack(object sender, X509Certificate? certificate, X509Chain? chain, SslPolicyErrors sslPolicyErrors)
		{
			// If the certificate is a valid, signed certificate, return true.
			if (sslPolicyErrors == SslPolicyErrors.None)
			{
				return true;
			}

			// Generate diagnostic information
			StringBuilder builder = new StringBuilder();
			if (sender != null)
			{
				string senderInfo = StringUtils.Indent(sender.ToString() ?? String.Empty, "    ");
				builder.Append(CultureInfo.InvariantCulture, $"\nSender:\n{senderInfo}");
			}
			if (certificate != null)
			{
				builder.Append(CultureInfo.InvariantCulture, $"\nCertificate: {certificate.Subject}");
			}
			if (chain != null)
			{
				if (chain.ChainStatus != null && chain.ChainStatus.Length > 0)
				{
					builder.Append("\nChain status:");
					foreach (X509ChainStatus status in chain.ChainStatus)
					{
						builder.Append(CultureInfo.InvariantCulture, $"\n  {status.StatusInformation}");
					}
				}
				if (chain.ChainElements != null)
				{
					builder.Append("\nChain elements:");
					for (int idx = 0; idx < chain.ChainElements.Count; idx++)
					{
						X509ChainElement element = chain.ChainElements[idx];
						builder.Append(CultureInfo.InvariantCulture, $"\n  {idx,4} - Certificate: {element.Certificate.Subject}");
						if (element.ChainElementStatus != null && element.ChainElementStatus.Length > 0)
						{
							foreach (X509ChainStatus status in element.ChainElementStatus)
							{
								builder.Append(CultureInfo.InvariantCulture, $"\n         Status: {status.StatusInformation} ({status.Status})");
							}
						}
						if (!String.IsNullOrEmpty(element.Information))
						{
							builder.Append(CultureInfo.InvariantCulture, $"\n         Info: {element.Information}");
						}
					}
				}
			}

			// Print out additional diagnostic information
			_logger.LogError("TLS certificate validation failed ({Errors}).{AdditionalInfo}", sslPolicyErrors, StringUtils.Indent(builder.ToString(), "    "));
			return false;
		}

		/// <summary>
		/// Get the MongoDB client
		/// </summary>
		/// <returns>A MongoDB client instance</returns>
		public MongoClient GetClient()
		{
			return _client;
		}

		/// <inheritdoc/>
		public IMongoCollection<T> GetCollection<T>(string name)
		{
			return GetCollection<T>(name, Enumerable.Empty<MongoIndex<T>>());
		}

		/// <inheritdoc/>
		public IMongoCollection<T> GetCollection<T>(string name, Func<IndexKeysDefinitionBuilder<T>, IndexKeysDefinition<T>> keysFunc, bool unique = false)
		{
			List<MongoIndex<T>> indexes = new List<MongoIndex<T>>();
			indexes.Add(MongoIndex.Create<T>(keysFunc, unique));
			return GetCollection(name, indexes);
		}

		/// <inheritdoc/>
		public IMongoCollection<T> GetCollection<T>(string name, IEnumerable<MongoIndex<T>> indexes)
		{
			IMongoCollection<T> collection = Database.GetCollection<T>(name);

			Task? upgradeTask = Task.CompletedTask;
			if (indexes.Any())
			{
				lock (_collectionUpgradeTasks)
				{
					if (!_collectionUpgradeTasks.TryGetValue(name, out upgradeTask))
					{
						_logger.LogDebug("Queuing update for collection {Name}", name);

						MongoIndex<T>[] indexesCopy = indexes.ToArray();
						upgradeTask = Task.Run(() => UpdateIndexesAsync(name, collection, indexesCopy, CancellationToken.None));

						_collectionUpgradeTasks.Add(name, upgradeTask);
					}
				}
			}

			return new MongoTracingCollection<T>(Database.GetCollection<T>(name), upgradeTask, _tracer);
		}

		private async Task UpdateIndexesAsync<T>(string collectionName, IMongoCollection<T> collection, MongoIndex<T>[] newIndexes, CancellationToken cancellationToken)
		{
			// Check we're allowed to upgrade the DB
			if (!await _setSchemaVersionTask)
			{
				return;
			}

			int attemptIdx = 1;
			for (; ; )
			{
				try
				{
					await UpdateIndexesInternalAsync<T>(collectionName, collection, newIndexes, cancellationToken);
					break;
				}
				catch (Exception ex) when (attemptIdx < 3)
				{
					_logger.LogError(ex, "Error updating indexes ({Message}) - retrying (attempt {Attempt})", ex.Message, attemptIdx);
					attemptIdx++;
				}
			}
		}

		private async Task UpdateIndexesInternalAsync<T>(string collectionName, IMongoCollection<T> collection, MongoIndex<T>[] newIndexes, CancellationToken cancellationToken)
		{
			// Find all the current indexes, excluding the default
			Dictionary<string, MongoIndex> nameToExistingIndex = new Dictionary<string, MongoIndex>(StringComparer.Ordinal);
			using (IAsyncCursor<BsonDocument> cursor = await collection.Indexes.ListAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (BsonDocument document in cursor.Current)
					{
						MongoIndex indexDocument = BsonSerializer.Deserialize<MongoIndex>(document);
						if (!indexDocument.Name.Equals("_id_", StringComparison.Ordinal))
						{
							nameToExistingIndex.Add(indexDocument.Name, indexDocument);
						}
					}
				}
			}

			// Figure out which indexes to drop
			List<MongoIndex<T>> createIndexes = new List<MongoIndex<T>>();
			HashSet<string> removeIndexNames = new HashSet<string>(nameToExistingIndex.Keys, StringComparer.Ordinal);
			foreach (MongoIndex<T> newIndex in newIndexes)
			{
				MongoIndex? existingIndex;
				if (nameToExistingIndex.TryGetValue(newIndex.Name, out existingIndex))
				{
					if (existingIndex.Equals(newIndex))
					{
						removeIndexNames.Remove(newIndex.Name);
						continue;
					}
				}
				createIndexes.Add(newIndex);
			}

			if (removeIndexNames.Count > 0 || createIndexes.Count > 0)
			{
				await _upgradeSema.WaitAsync(cancellationToken);
				try
				{
					// Aquire a lock for updating the DB
					await using RedisLock schemaLock = new(_redisService.GetDatabase(), s_schemaLockKey);
					while (!await schemaLock.AcquireAsync(TimeSpan.FromMinutes(5.0)))
					{
						_logger.LogDebug("Unable to acquire lock for upgrade task; pausing for 1s");
						await Task.Delay(TimeSpan.FromSeconds(1.0), cancellationToken);
					}

					_logger.LogDebug("Updating indexes for collection {CollectionName}", collectionName);

					// Drop any indexes that are no longer needed
					foreach (string removeIndexName in removeIndexNames)
					{
						_logger.LogWarning("Collection {CollectionName} index {IndexName} is no longer required and should be removed. This may adversely affect performance.", collectionName, removeIndexName);
					}

					// Create all the new indexes
					foreach (MongoIndex<T> createIndex in createIndexes)
					{
						if (ReadOnlyMode)
						{
							_logger.LogWarning("Would create index {CollectionName}.{IndexName} - skipping due to read-only setting.", collectionName, createIndex.Name);
						}
						else
						{
							_logger.LogInformation("Creating index {IndexName} in {CollectionName}", createIndex.Name, collectionName);

							CreateIndexOptions<T> options = new CreateIndexOptions<T>();
							options.Name = createIndex.Name;
							options.Unique = createIndex.Unique;
							options.Sparse = createIndex.Sparse;
							options.Background = true;

							CreateIndexModel<T> model = new CreateIndexModel<T>(createIndex.Keys, options);

							try
							{
								string result = await collection.Indexes.CreateOneAsync(model, cancellationToken: cancellationToken);
								_logger.LogInformation("Created index {IndexName}", result);
							}
							catch (Exception ex)
							{
								_logger.LogError(ex, "Unable to create index {IndexName}: {Message}", createIndex.Name, ex.Message);
								throw;
							}
						}
					}
				}
				finally
				{
					_upgradeSema.Release();
				}

				_logger.LogInformation("Finished updating indexes for collection {CollectionName}", collectionName);
			}
		}

		async Task<bool> SetSchemaVersionAsync(SemVer schemaVersion, CancellationToken cancellationToken)
		{
			// Check we're not downgrading the data
			for (; ; )
			{
				MongoSchemaDocument currentSchema = await GetSingletonAsync<MongoSchemaDocument>(cancellationToken);
				if (!String.IsNullOrEmpty(currentSchema.Version))
				{
					SemVer currentVersion = SemVer.Parse(currentSchema.Version);
					if (schemaVersion < currentVersion)
					{
						_logger.LogInformation("Ignoring upgrade command; server is older than current schema version ({ProgramVer} < {CurrentVer})", ServerApp.Version, currentVersion);
						return false;
					}
					if (schemaVersion == currentVersion)
					{
						return true;
					}
				}

				_logger.LogInformation("Upgrading schema version {OldVersion} -> {NewVersion}", currentSchema.Version, schemaVersion.ToString());
				currentSchema.Version = schemaVersion.ToString();

				if (ReadOnlyMode)
				{
					return false;
				}
				if (await TryUpdateSingletonStaticAsync(SingletonsV2, currentSchema, cancellationToken))
				{
					return true;
				}
			}
		}

		class SingletonInfo<T> where T : SingletonBase
		{
			public static readonly SingletonDocumentAttribute Attribute = GetAttribute();

			static SingletonDocumentAttribute GetAttribute()
			{
				SingletonDocumentAttribute? attribute = typeof(T).GetCustomAttribute<SingletonDocumentAttribute>();
				if (attribute == null)
				{
					throw new Exception($"Type {typeof(T).Name} is missing a {nameof(SingletonDocumentAttribute)} annotation");
				}
				return attribute;
			}
		}

		/// <inheritdoc/>
		public Task<T> GetSingletonAsync<T>(CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			return GetSingletonStaticAsync(SingletonsV1, SingletonsV2, () => new T(), cancellationToken);
		}

		/// <inheritdoc/>
		public Task<T> GetSingletonAsync<T>(Func<T> constructor, CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			return GetSingletonStaticAsync(SingletonsV1, SingletonsV2, constructor, cancellationToken);
		}

		/// <inheritdoc/>
		public Task UpdateSingletonAsync<T>(Action<T> updater, CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			bool Update(T instance)
			{
				updater(instance);
				return true;
			}
			return UpdateSingletonAsync<T>(Update, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task UpdateSingletonAsync<T>(Func<T, bool> updater, CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			for (; ; )
			{
				T document = await GetSingletonStaticAsync(SingletonsV1, SingletonsV2, () => new T(), cancellationToken);
				if (!updater(document))
				{
					break;
				}
				if (await TryUpdateSingletonStaticAsync(SingletonsV2, document, cancellationToken))
				{
					break;
				}
			}
		}

		/// <inheritdoc/>
		public Task<bool> TryUpdateSingletonAsync<T>(T singletonObject, CancellationToken cancellationToken) where T : SingletonBase
		{
			return TryUpdateSingletonStaticAsync(SingletonsV2, singletonObject, cancellationToken);
		}

		/// <summary>
		/// Gets a singleton document by id
		/// </summary>
		/// <param name="singletonsV1">Collection of Singleton documents V1</param>
		/// <param name="singletonsV2">Collection of Singleton documents V2</param>
		/// <param name="constructor">Method to use to construct a new object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The document</returns>
		internal static async Task<T> GetSingletonStaticAsync<T>(IMongoCollection<BsonDocument> singletonsV1, IMongoCollection<BsonDocument> singletonsV2, Func<T> constructor, CancellationToken cancellationToken) where T : SingletonBase, new()
		{
			SingletonDocumentAttribute attribute = SingletonInfo<T>.Attribute;

			FilterDefinition<BsonDocument> filter = new BsonDocument(new BsonElement("_id", attribute.Id));
			for (; ; )
			{
				BsonDocument? document = await singletonsV2.Find(filter).FirstOrDefaultAsync(cancellationToken);
				if (document != null)
				{
					T item = BsonSerializer.Deserialize<T>(document);
					item.PostLoad();
					return item;
				}

				T? newItem = null;
				if (attribute.LegacyId != null)
				{
					BsonDocument? legacyDocument = await singletonsV1.Find(new BsonDocument(new BsonElement("_id", ObjectId.Parse(attribute.LegacyId)))).FirstOrDefaultAsync(cancellationToken);
					if (legacyDocument != null)
					{
						legacyDocument.Remove("_id");
						newItem = BsonSerializer.Deserialize<T>(legacyDocument);
						newItem.PostLoad();
					}
				}
				newItem ??= constructor();

				newItem.Id = new SingletonId(attribute.Id);
				await singletonsV2.InsertOneIgnoreDuplicatesAsync(newItem.ToBsonDocument(), cancellationToken);
			}
		}

		/// <summary>
		/// Attempts to update a singleton object
		/// </summary>
		/// <param name="singletonsV2">Collection of Singleton documents V2</param>
		/// <param name="singletonObject">The singleton object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the singleton document was updated</returns>
		internal static async Task<bool> TryUpdateSingletonStaticAsync<T>(IMongoCollection<BsonDocument> singletonsV2, T singletonObject, CancellationToken cancellationToken) where T : SingletonBase
		{
			int prevRevision = singletonObject.Revision++;

			BsonDocument filter = new BsonDocument { new BsonElement("_id", singletonObject.Id.ToString()), new BsonElement(nameof(SingletonBase.Revision), prevRevision) };
			try
			{
				ReplaceOneResult result = await singletonsV2.ReplaceOneAsync(filter, singletonObject.ToBsonDocument(), new ReplaceOptions { IsUpsert = true }, cancellationToken);
				return result.MatchedCount > 0;
			}
			catch (MongoWriteException ex)
			{
				// Duplicate key error occurs if filter fails to match because revision is not the same.
				if (ex.WriteError != null && ex.WriteError.Category == ServerErrorCategory.DuplicateKey)
				{
					return false;
				}
				else
				{
					throw;
				}
			}
		}

		/// <inheritdoc/>
		public async Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = default)
		{
			try
			{
				await Database.RunCommandAsync((Command<BsonDocument>)"{ping:1}", cancellationToken: cancellationToken);
				return HealthCheckResult.Healthy();
			}
			catch (Exception ex)
			{
				return HealthCheckResult.Unhealthy("Unable to ping MongoDB", ex);
			}
		}
	}

	/// <summary>
	/// Stores the version number of the latest server instance to have upgraded the database schema
	/// </summary>
	[SingletonDocument("mongo-schema", "62470d8508d48eddfac7b55d")]
	public class MongoSchemaDocument : SingletonBase
	{
		/// <summary>
		/// Current version number
		/// </summary>
		public string? Version { get; set; }
	}
}
