// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Users;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// User notice
	/// </summary>
	public interface INotice
	{
		/// <summary>
		/// Unique id for this notice
		/// </summary>
		public ObjectId Id { get; }

		/// <summary>
		/// User id of who created this notice, null for system created notices
		/// </summary>
		[BsonIgnoreIfNull]
		public UserId? UserId { get; }

		/// <summary>
		/// Start time to display this message
		/// </summary>
		public DateTime? StartTime { get; }

		/// <summary>
		/// Finish time to display this message
		/// </summary>
		public DateTime? FinishTime { get; }

		/// <summary>
		/// Message to display
		/// </summary>
		public string Message { get; }
	}
}
