// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// A collection of user specified notices
	/// </summary>
	public interface INoticeCollection
	{
		/// <summary>
		/// Add a notice to the collection
		/// </summary>
		/// <param name="message"></param>
		/// <param name="userId"></param>
		/// <param name="startTime"></param>
		/// <param name="finishTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<INotice?> AddNoticeAsync(string message, UserId? userId, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update an existing notice
		/// </summary>
		/// <param name="id"></param>
		/// <param name="message"></param>
		/// <param name="startTime"></param>
		/// <param name="finishTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<bool> UpdateNoticeAsync(ObjectId id, string? message, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get a notice by id
		/// </summary>
		/// <param name="noticeId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<INotice?> GetNoticeAsync(ObjectId noticeId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get all notices
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<List<INotice>> GetNoticesAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Remove a notice
		/// </summary>
		/// <param name="id"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<bool> RemoveNoticeAsync(ObjectId id, CancellationToken cancellationToken = default);
	}
}