// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// ACL actions relating to notices
	/// </summary>
	public static class NoticeAclAction
	{
		/// <summary>
		/// Ability to create new notices
		/// </summary>
		public static AclAction CreateNotice { get; } = new AclAction("CreateNotice");

		/// <summary>
		/// Ability to modify notices on the server
		/// </summary>
		public static AclAction UpdateNotice { get; } = new AclAction("UpdateNotice");

		/// <summary>
		/// Ability to delete notices
		/// </summary>
		public static AclAction DeleteNotice { get; } = new AclAction("DeleteNotice");
	}
}
