// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Users;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// Collection of notice documents
	/// </summary>
	public class NoticeCollection : INoticeCollection
	{

		/// <summary>
		/// Document representing a notice 
		/// </summary>
		class NoticeDocument : INotice
		{
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonIgnoreIfNull]
			public UserId? UserId { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? StartTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? FinishTime { get; set; }

			public string Message { get; set; } = String.Empty;

			[BsonConstructor]
			private NoticeDocument()
			{
			}

			public NoticeDocument(ObjectId id, string message, UserId? userId, DateTime? startTime, DateTime? finishTime)
			{
				Id = id;
				Message = message;
				UserId = userId;
				StartTime = startTime;
				FinishTime = finishTime;
			}
		}

		readonly IMongoCollection<NoticeDocument> _notices;

		/// <summary>
		/// Constructor
		/// </summary>
		public NoticeCollection(IMongoService mongoService)
		{
			_notices = mongoService.GetCollection<NoticeDocument>("Notices");
		}

		/// <inheritdoc/>
		public async Task<INotice?> AddNoticeAsync(string message, UserId? userId, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken)
		{
			NoticeDocument newNotice = new NoticeDocument(ObjectId.GenerateNewId(), message, userId, startTime, finishTime);
			await _notices.InsertOneAsync(newNotice, null, cancellationToken);
			return newNotice;
		}

		/// <inheritdoc/>
		public async Task<bool> UpdateNoticeAsync(ObjectId id, string? message, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken)
		{
			UpdateDefinitionBuilder<NoticeDocument> updateBuilder = Builders<NoticeDocument>.Update;
			List<UpdateDefinition<NoticeDocument>> updates = new List<UpdateDefinition<NoticeDocument>>();

			if (message != null)
			{
				updates.Add(updateBuilder.Set(x => x.Message, message));
			}

			updates.Add(updateBuilder.Set(x => x.StartTime, startTime));

			updates.Add(updateBuilder.Set(x => x.FinishTime, finishTime));

			NoticeDocument? document = await _notices.FindOneAndUpdateAsync(x => x.Id == id, updateBuilder.Combine(updates), null, cancellationToken);

			return document != null;
		}

		/// <inheritdoc/>
		public async Task<INotice?> GetNoticeAsync(ObjectId noticeId, CancellationToken cancellationToken)
		{
			return await _notices.Find(x => x.Id == noticeId).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<List<INotice>> GetNoticesAsync(CancellationToken cancellationToken)
		{
			List<NoticeDocument> results = await _notices.Find(x => true).ToListAsync(cancellationToken);
			return results.Select<NoticeDocument, INotice>(x => x).ToList();
		}

		/// <inheritdoc/>
		public async Task<bool> RemoveNoticeAsync(ObjectId id, CancellationToken cancellationToken)
		{
			DeleteResult result = await _notices.DeleteOneAsync(x => x.Id == id, cancellationToken);
			return result.DeletedCount > 0;
		}
	}
}

