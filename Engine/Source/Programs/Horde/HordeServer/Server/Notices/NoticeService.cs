// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// Notice service
	/// </summary>
	public sealed class NoticeService
	{
		/// <summary>
		/// Notice collection
		/// </summary>
		readonly INoticeCollection _notices;

		/// <summary>
		/// Device service constructor
		/// </summary>
		public NoticeService(INoticeCollection notices)
		{
			_notices = notices;
		}

		/// <summary>
		/// Add a notice
		/// </summary>
		/// <param name="message"></param>
		/// <param name="userId"></param>
		/// <param name="startTime"></param>
		/// <param name="finishTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<INotice?> AddNoticeAsync(string message, UserId? userId, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken)
		{
			return _notices.AddNoticeAsync(message, userId, startTime, finishTime, cancellationToken);
		}

		/// <summary>
		/// Update an existing notice
		/// </summary>
		/// <param name="id"></param>
		/// <param name="message"></param>
		/// <param name="startTime"></param>
		/// <param name="finishTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<bool> UpdateNoticeAsync(ObjectId id, string? message, DateTime? startTime, DateTime? finishTime, CancellationToken cancellationToken)
		{
			return _notices.UpdateNoticeAsync(id, message, startTime, finishTime, cancellationToken);
		}

		/// <summary>
		/// Get a notice by id
		/// </summary>
		/// <param name="noticeId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<INotice?> GetNoticeAsync(ObjectId noticeId, CancellationToken cancellationToken)
		{
			return _notices.GetNoticeAsync(noticeId, cancellationToken);
		}

		/// <summary>
		/// Get all notices
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<List<INotice>> GetNoticesAsync(CancellationToken cancellationToken)
		{
			return _notices.GetNoticesAsync(cancellationToken);
		}

		/// <summary>
		/// Remove an existing notice
		/// </summary>
		/// <param name="id"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<bool> RemoveNoticeAsync(ObjectId id, CancellationToken cancellationToken)
		{
			return _notices.RemoveNoticeAsync(id, cancellationToken);
		}
	}
}