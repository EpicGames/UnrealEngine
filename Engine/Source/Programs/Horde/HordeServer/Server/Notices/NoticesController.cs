// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.Server.Notices;
using EpicGames.Horde.Users;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Options;
using MongoDB.Bson;

namespace HordeServer.Server.Notices
{
	/// <summary>
	/// Controller for the /api/v1/notices endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class NoticesController : ControllerBase
	{
		private readonly IUserCollection _userCollection;
		private readonly NoticeService _noticeService;
		private readonly ServerStatusService _statusService;
		private readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		private readonly IClock _clock;

		/// <summary>
		/// Constructor
		/// </summary>
		public NoticesController(NoticeService noticeService, ServerStatusService statusService, IUserCollection userCollection, IClock clock, IOptionsSnapshot<GlobalConfig> globalConfig)
		{
			_userCollection = userCollection;
			_noticeService = noticeService;
			_statusService = statusService;
			_clock = clock;
			_globalConfig = globalConfig;
		}

		/// <summary>
		/// Add a status message
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost("/api/v1/notices")]
		public async Task<ActionResult> AddNoticeAsync(CreateNoticeRequest request, CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(NoticeAclAction.CreateNotice, User))
			{
				return Forbid();
			}

			INotice? notice = await _noticeService.AddNoticeAsync(request.Message, User.GetUserId(), request.StartTime, request.FinishTime, cancellationToken);

			return notice == null ? NotFound() : Ok();

		}

		/// <summary>
		/// Update a status message
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPut("/api/v1/notices")]
		public async Task<ActionResult> UpdateNoticeAsync(UpdateNoticeRequest request, CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(NoticeAclAction.UpdateNotice, User))
			{
				return Forbid();
			}

			return await _noticeService.UpdateNoticeAsync(new ObjectId(request.Id), request.Message, request.StartTime, request.FinishTime, cancellationToken) ? Ok() : NotFound();
		}

		/// <summary>
		/// Remove a manually added status message
		/// </summary>
		/// <returns></returns>
		[HttpDelete("/api/v1/notices/{id}")]
		public async Task<ActionResult> DeleteNoticeAsync(string id, CancellationToken cancellationToken)
		{
			if (!_globalConfig.Value.Authorize(NoticeAclAction.DeleteNotice, User))
			{
				return Forbid();
			}

			await _noticeService.RemoveNoticeAsync(new ObjectId(id), cancellationToken);

			return Ok();
		}

		/// <summary>
		/// Gets the current status messages
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The status messages</returns>
		[HttpGet("/api/v1/notices")]
		public async Task<List<GetNoticeResponse>> GetNoticesAsync(CancellationToken cancellationToken)
		{
			List<GetNoticeResponse> messages = new List<GetNoticeResponse>();

			DateTimeOffset now = TimeZoneInfo.ConvertTime(new DateTimeOffset(_clock.UtcNow), _clock.TimeZone);

			foreach (ScheduledDowntime schedule in _globalConfig.Value.Downtime)
			{
				DateTimeOffset start = TimeZoneInfo.ConvertTimeBySystemTimeZoneId(schedule.GetNext(now).StartTime, "UTC");
				DateTimeOffset finish = TimeZoneInfo.ConvertTimeBySystemTimeZoneId(schedule.GetNext(now).FinishTime, "UTC");
				messages.Add(new GetNoticeResponse() { ScheduledDowntime = true, StartTime = new DateTime(start.Ticks, DateTimeKind.Utc), FinishTime = new DateTime(finish.Ticks, DateTimeKind.Utc), Active = schedule.IsActive(now) });
			}

			// Add user notices
			List<INotice> notices = await _noticeService.GetNoticesAsync(cancellationToken);
			for (int i = 0; i < notices.Count; i++)
			{
				INotice notice = notices[i];
				GetThinUserInfoResponse? userInfo = null;

				if (notice.UserId != null)
				{
					userInfo = (await _userCollection.GetCachedUserAsync(notice.UserId, cancellationToken))?.ToThinApiResponse();
				}

				DateTime? startTime = null;
				if (notice.StartTime != null)
				{
					startTime = TimeZoneInfo.ConvertTimeBySystemTimeZoneId(notice.StartTime.Value, "UTC");
				}
				DateTime? finishTime = null;
				if (notice.FinishTime != null)
				{
					finishTime = TimeZoneInfo.ConvertTimeBySystemTimeZoneId(notice.FinishTime.Value, "UTC");
				}
				bool active = _clock.UtcNow >= (startTime ?? DateTime.MinValue) && _clock.UtcNow < (finishTime ?? DateTime.MaxValue);

				messages.Add(new GetNoticeResponse() { Id = notice.Id.ToString(), Active = active, StartTime = startTime, FinishTime = finishTime, Message = notice.Message, CreatedByUser = userInfo });
			}

			// Add any status notices for admins
			if (User.HasAdminClaim())
			{
				IReadOnlyList<SubsystemStatus> statuses = await _statusService.GetSubsystemStatusesAsync();
				foreach (SubsystemStatus status in statuses)
				{
					SubsystemStatusUpdate? update = status.Updates.FirstOrDefault();
					if (update != null)
					{
						if (update.Result != HealthStatus.Healthy)
						{
							string message = $"Server is reporting a {status.Name} issue.";

							// Add links to relevant status pages within Horde
							if (status.Name == "Perforce")
							{
								message += " See[Perforce servers page](/perforce/servers) for more info.";
							}
							else
							{
								message += " See[status page](/serverstatus) for more info.";
							}

							messages.Add(new GetNoticeResponse { Active = true, Message = message });
						}
					}
				}
			}

			return messages;

		}
	}
}
