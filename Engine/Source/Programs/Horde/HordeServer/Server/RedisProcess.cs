// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.Net;
using System.Net.Sockets;
using System.Runtime.InteropServices;
using System.Runtime.Versioning;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace HordeServer.Server
{
	/// <summary>
	/// Runs a local instance of redis
	/// </summary>
	public sealed class RedisProcess : IAsyncDisposable
	{
		readonly DirectoryReference _tempDir;
		readonly ILogger _logger;

		int? _port;
		ManagedProcessGroup? _processGroup;
		ManagedProcess? _process;
		BackgroundTask? _relayTask;

		/// <summary>
		/// Path to the redis executable
		/// </summary>
		[SupportedOSPlatform("windows")]
		public static FileReference RedisExe => FileReference.Combine(ServerApp.AppDir, "ThirdParty", "Redis", "redis-server.exe");

		/// <summary>
		/// Selected port for the service
		/// </summary>
		public int Port => _port ?? throw new Exception("Redis process has not been started");

		/// <summary>
		/// Constructor
		/// </summary>
		public RedisProcess(ILogger logger)
		{
			_tempDir = DirectoryReference.Combine(ServerApp.DataDir, "Redis");
			_logger = logger;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await StopAsync();
		}

		/// <summary>
		/// Starts the redis process
		/// </summary>
		[SupportedOSPlatform("windows")]
		public void Start(string arguments)
		{
			if (_process != null)
			{
				throw new Exception("Redis process has already been started");
			}

			FileReference redisExe = RedisExe;
			if (!FileReference.Exists(redisExe))
			{
				throw new Exception($"Couldn't find bundled Redis executable at {redisExe}");
			}

			// For some reason Redis takes quite a while to shut down, and is in a Zombie state until it is. Copy it to a temp directory first.
			DeleteTempFiles();
			DirectoryReference.CreateDirectory(_tempDir);

			FileReference tempRedisExe = FileReference.Combine(_tempDir, $"redis-server.{Guid.NewGuid():N}.exe");
			FileReference.Copy(redisExe, tempRedisExe);

			// Find a free port on the local machine
			_port = GetAvailablePort();

			// Launch the child process
			_processGroup = new ManagedProcessGroup();
			_process = new ManagedProcess(_processGroup, tempRedisExe.FullName, $"{arguments} --bind 127.0.0.1 --port {_port} --save \"\" --appendonly no --databases 3000", redisExe.Directory.FullName, null, ProcessPriorityClass.Normal);
			_process.StdIn.Close();

			_relayTask = BackgroundTask.StartNew(RelayOutputAsync);
		}

		/// <summary>
		/// Stops the current process
		/// </summary>
		public async ValueTask StopAsync()
		{
			_logger.LogInformation("Stopping Redis...");
			if (_processGroup != null)
			{
				_processGroup.Dispose();
				_processGroup = null;
			}
			if (_process != null)
			{
				_process.Dispose();
				_process = null;
			}
			if (_relayTask != null)
			{
				await _relayTask.DisposeAsync();
				_relayTask = null;
			}

			DeleteTempFiles();
			_logger.LogInformation("Done.");
		}

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern uint DeleteFile(string lpFileName);

		void DeleteTempFiles()
		{
			if (DirectoryReference.Exists(_tempDir))
			{
				foreach (FileReference file in DirectoryReference.EnumerateFiles(_tempDir, "*.exe", System.IO.SearchOption.TopDirectoryOnly))
				{
					_ = DeleteFile(file.FullName);
				}
			}
		}

		/// <summary>
		/// Gets an unused port that can host the redis server
		/// </summary>
		static int GetAvailablePort()
		{
			using TcpListener listener = new(IPAddress.Loopback, 0);
			listener.Start();
			int port = ((IPEndPoint)listener.LocalEndpoint).Port;
			listener.Stop();
			return port;
		}

		/// <summary>
		/// Copies output from the redis process to the logger
		/// </summary>
		async Task RelayOutputAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				string? line = await _process!.ReadLineAsync(cancellationToken);
				if (line == null)
				{
					break;
				}
				if (line.Length > 0)
				{
					_logger.Log(LogLevel.Information, "{Output}", line);
				}
			}
		}
	}
}
