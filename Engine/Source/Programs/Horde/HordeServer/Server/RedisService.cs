// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Redis;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Server
{
	/// <summary>
	/// Manages the lifetime of a bundled Redis instance
	/// </summary>
	public sealed class RedisService : IRedisService, IAsyncDisposable
	{
		/// <summary>
		/// Default Redis port
		/// </summary>
		const int RedisPort = 6379;

		/// <summary>
		/// Connection pool
		/// </summary>
		public RedisConnectionPool ConnectionPool { get; }

		/// <summary>
		/// Flag for whether the connection is read-only
		/// </summary>
		public bool ReadOnlyMode { get; }

		RedisProcess? _redisProcess;
		readonly ILogger<RedisService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public RedisService(IOptions<ServerSettings> options, ILogger<RedisService> logger)
			: this(options.Value.RedisConnectionString, -1, logger)
		{
			ReadOnlyMode = options.Value.RedisReadOnlyMode;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="connectionString">Redis connection string. If null, we will start a temporary redis instance on the local machine.</param>
		/// <param name="dbNum">Override for the database to use. Set to -1 to use the default from the connection string.</param>
		/// <param name="logger"></param>
		public RedisService(string? connectionString, int dbNum, ILogger<RedisService> logger)
		{
			_logger = logger;

			if (connectionString == null)
			{
				if (IsRunningOnDefaultPort())
				{
					connectionString = $"127.0.0.1:{RedisPort}";
				}
				else if (TryStartRedisProcess())
				{
					connectionString = $"127.0.0.1:{_redisProcess!.Port},allowAdmin=true";
				}
				else
				{
					throw new Exception($"Unable to connect to Redis. Please set {nameof(ServerSettings.RedisConnectionString)} in {ServerApp.ServerConfigFile}");
				}
			}

			ConnectionPool = new RedisConnectionPool(20, connectionString, dbNum);
		}

		/// <inheritdoc/>
		public async Task<HealthCheckResult> CheckHealthAsync(HealthCheckContext context, CancellationToken cancellationToken = default)
		{
			try
			{
				TimeSpan span = await ConnectionPool.GetDatabase().PingAsync();
				return HealthCheckResult.Healthy(data: new Dictionary<string, object> { ["Latency"] = span.ToString() });
			}
			catch (Exception ex)
			{
				return HealthCheckResult.Unhealthy("Unable to ping Redis", ex);
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_redisProcess != null)
			{
				_logger.LogInformation("Sending shutdown command...");
				ConnectionPool.GetConnection().GetServers().FirstOrDefault()?.Shutdown();
			}

			ConnectionPool.Dispose();

			if (_redisProcess != null)
			{
				await _redisProcess.DisposeAsync();
				_redisProcess = null;
			}
		}

		/// <summary>
		/// Checks if Redis is already running on the default port
		/// </summary>
		/// <returns></returns>
		static bool IsRunningOnDefaultPort()
		{
			IPGlobalProperties ipGlobalProperties = IPGlobalProperties.GetIPGlobalProperties();

			IPEndPoint[] listeners = ipGlobalProperties.GetActiveTcpListeners();
			if (listeners.Any(x => x.Port == RedisPort))
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Attempts to start a local instance of Redis
		/// </summary>
		/// <returns></returns>
		bool TryStartRedisProcess()
		{
			if (_redisProcess != null)
			{
				return true;
			}
			if (!RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				return false;
			}

			FileReference redisConfigFile = FileReference.Combine(RedisProcess.RedisExe.Directory, "redis.conf");
			_redisProcess = new RedisProcess(_logger);
			_redisProcess.Start($"\"{redisConfigFile}\"");
			return true;
		}
	}
}
