// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Globalization;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Logging;

namespace HordeServer.Server
{
	/// <summary>
	/// Tracks open HTTP requests to ASP.NET
	/// Will block a pending shutdown until all requests in progress are finished (or graceful timeout is reached)
	/// This avoids interrupting long running requests such as artifact uploads.
	/// </summary>
	public class RequestTrackerService
	{
		/// <summary>
		/// Writer for log output
		/// </summary>
		private readonly ILogger<RequestTrackerService> _logger;

		readonly ConcurrentDictionary<string, TrackedRequest> _requestsInProgress = new ConcurrentDictionary<string, TrackedRequest>();

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">Logger</param>
		public RequestTrackerService(ILogger<RequestTrackerService> logger)
		{
			_logger = logger;
		}

		/// <summary>
		/// Called by the middleware when a request is started
		/// </summary>
		/// <param name="context">HTTP Context</param>
		public void RequestStarted(HttpContext context)
		{
			_requestsInProgress[context.TraceIdentifier] = new TrackedRequest(context.Request);
		}

		/// <summary>
		/// Called by the middleware when a request is finished (no matter if an exception occurred or not)
		/// </summary>
		/// <param name="context">HTTP Context</param>
		public void RequestFinished(HttpContext context)
		{
			_requestsInProgress.Remove(context.TraceIdentifier, out _);
		}

		/// <summary>
		/// Get current requests in progress
		/// </summary>
		/// <returns>The requests in progress</returns>
		public IReadOnlyDictionary<string, TrackedRequest> GetRequestsInProgress()
		{
			return _requestsInProgress;
		}

		private string GetRequestsInProgressAsString()
		{
			List<KeyValuePair<string, TrackedRequest>> requests = GetRequestsInProgress().ToList();
			requests.Sort((a, b) => a.Value.StartedAt.CompareTo(b.Value.StartedAt));
			StringBuilder content = new StringBuilder();
			foreach (KeyValuePair<string, TrackedRequest> pair in requests)
			{
				int ageInMs = pair.Value.GetTimeSinceStartInMs();
				string path = pair.Value.Request.Path;
				content.AppendLine(CultureInfo.InvariantCulture, $"{ageInMs,9}  {path}");
			}

			return content.ToString();
		}

		/// <summary>
		/// Log all requests currently in progress
		/// </summary>
		public void LogRequestsInProgress()
		{
			if (GetRequestsInProgress().Count == 0)
			{
				_logger.LogInformation("There are no requests in progress!");
			}
			else
			{
				_logger.LogInformation("Current open requests are:\n{RequestsInProgress}", GetRequestsInProgressAsString());
			}
		}
	}

	/// <summary>
	/// Value object for tracked requests
	/// </summary>
	public class TrackedRequest
	{
		/// <summary>
		/// When the request was received
		/// </summary>
		public DateTime StartedAt { get; }

		/// <summary>
		/// The HTTP request being tracked
		/// </summary>
		public HttpRequest Request { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="request">HTTP request being tracked</param>
		public TrackedRequest(HttpRequest request)
		{
			StartedAt = DateTime.UtcNow;
			Request = request;
		}

		/// <summary>
		/// How long the request has been running
		/// </summary>
		/// <returns>Time elapsed in milliseconds since request started</returns>
		public int GetTimeSinceStartInMs()
		{
			return (int)(DateTime.UtcNow - StartedAt).TotalMilliseconds;
		}
	}

	/// <summary>
	/// ASP.NET Middleware to track open requests
	/// </summary>
	public class RequestTrackerMiddleware
	{
		private readonly RequestDelegate _next;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="next">Next middleware to call</param>
		public RequestTrackerMiddleware(RequestDelegate next)
		{
			_next = next;
		}

		/// <summary>
		/// Invoked by ASP.NET framework itself
		/// </summary>
		/// <param name="context">HTTP Context</param>
		/// <param name="service">The RequestTrackerService singleton</param>
		/// <returns></returns>
		public async Task InvokeAsync(HttpContext context, RequestTrackerService service)
		{
			if (!context.Request.Path.StartsWithSegments("/health", StringComparison.Ordinal))
			{
				try
				{
					service.RequestStarted(context);
					await _next(context);
				}
				finally
				{
					service.RequestFinished(context);
				}
			}
			else
			{
				// Ignore requests to /health/*
				await _next(context);
			}
		}
	}
}