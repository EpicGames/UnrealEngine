// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using Microsoft.AspNetCore.Builder;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Server;

/// <summary>
/// Middleware that detects if the server is running behind a load balancer or reverse proxy.
/// If detected, logs an error if <see cref="ServerSettings.ServerUrl"/> is not configured.
/// </summary>
/// <param name="next">Next middleware in the pipeline</param>
/// <param name="logger">Logger</param>
/// <param name="settings">Settings</param>
public class ReverseProxyDetectionMiddleware(RequestDelegate next, ILogger<ReverseProxyDetectionMiddleware> logger, IOptionsMonitor<ServerSettings> settings)
{
	private const int MaxWarnings = 5;
	private const int MaxRequests = 5000;
	private readonly List<string> _headers = ["X-Forwarded-For", "X-Forwarded-Proto", "X-Forwarded-Host", "X-Forwarded-Server",];
	private int _requestCount;
	private int _warningCount;
	
	/// <summary>
	/// Processes an HTTP request to detect reverse proxy headers and validate server configuration
	/// </summary>
	public async Task InvokeAsync(HttpContext context)
	{
		// Only check the first X number of requests for performance
		if (_requestCount < MaxRequests && _warningCount < MaxWarnings)
		{
			_requestCount++;
			bool isReverseProxyRequest = _headers.Exists(header => context.Request.Headers.ContainsKey(header));
			if (isReverseProxyRequest && settings.CurrentValue.IsDefaultServerUrlUsed())
			{
				_warningCount++;
				logger.LogError("Load balancer or reverse proxy detected. Ensure {Setting} is configured in server settings",
					nameof(ServerSettings.ServerUrl));
			}
		}
		
		await next(context);
	}
}

/// <summary>
/// Extension methods for configuring the reverse proxy detection middleware.
/// </summary>
public static class ReverseProxyDetectionMiddlewareExtensions
{
	/// <summary>
	/// Adds middleware to detect if the application is running behind a reverse proxy
	/// </summary>
	public static IApplicationBuilder UseReverseProxyDetection(this IApplicationBuilder builder)
	{
		return builder.UseMiddleware<ReverseProxyDetectionMiddleware>();
	}
}