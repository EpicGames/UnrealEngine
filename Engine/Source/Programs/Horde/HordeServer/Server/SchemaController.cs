// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Reflection;
using System.Text.Json.Serialization;
using EpicGames.Core;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Server
{
	/// <summary>
	/// Controller for the /api/v1/schema endpoint
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public class SchemaController : ControllerBase
	{
		class CatalogItem
		{
			public string? Name { get; set; }
			public string? Description { get; set; }
			public string[]? FileMatch { get; set; }
			public Uri? Url { get; set; }
		}

		class CatalogRoot
		{
			[JsonPropertyName("$schema")]
			public string Schema { get; set; } = "https://json.schemastore.org/schema-catalog.json";
			public int Version { get; set; } = 1;
			public List<CatalogItem> Schemas { get; set; } = new List<CatalogItem>();
		}

		/// <summary>
		/// Array of types included in the schema
		/// </summary>
		public static Type[] ConfigSchemas { get; } = FindSchemaTypes();

		readonly JsonSchemaCache _schemaCache;

		/// <summary>
		/// Constructor
		/// </summary>
		public SchemaController(JsonSchemaCache schemaCache)
		{
			_schemaCache = schemaCache;
		}

		static Type[] FindSchemaTypes()
		{
			List<Type> schemaTypes = new List<Type>();
			foreach (Assembly assembly in AppDomain.CurrentDomain.GetAssemblies())
			{
				foreach (Type type in assembly.GetTypes())
				{
					if (type.GetCustomAttribute<JsonSchemaAttribute>() != null)
					{
						schemaTypes.Add(type);
					}
				}
			}
			return schemaTypes.ToArray();
		}

		/// <summary>
		/// Get the catalog for config schema
		/// </summary>
		/// <returns>Information about all the schedules</returns>
		[HttpGet]
		[Route("/api/v1/schema/catalog.json")]
		public ActionResult GetCatalog()
		{
			CatalogRoot root = new CatalogRoot();
			foreach (Type schemaType in ConfigSchemas)
			{
				JsonSchemaAttribute? schemaAttribute = schemaType.GetCustomAttribute<JsonSchemaAttribute>();
				if (schemaAttribute != null)
				{
					JsonSchemaCatalogAttribute? catalogAttribute = schemaType.GetCustomAttribute<JsonSchemaCatalogAttribute>();
					if (catalogAttribute != null)
					{
						Uri url = new Uri($"{Request.Scheme}://{Request.Host}/api/v1/schema/types/{schemaType.Name}.json");
						root.Schemas.Add(new CatalogItem { Name = catalogAttribute.Name, Description = catalogAttribute.Description, FileMatch = catalogAttribute.FileMatch, Url = url });
					}
				}
			}
			return Ok(root);
		}

		/// <summary>
		/// Gets a specific schema
		/// </summary>
		/// <param name="typeName">The type name</param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/schema/types/{typeName}.json")]
		public ActionResult GetSchema(string typeName)
		{
			foreach (Type schemaType in ConfigSchemas)
			{
				if (schemaType.Name.Equals(typeName, StringComparison.OrdinalIgnoreCase))
				{
					JsonSchema schema = _schemaCache.CreateSchema(schemaType);

					using MemoryStream stream = new MemoryStream();
					schema.Write(stream);

					return new FileContentResult(stream.ToArray(), "application/json");
				}
			}
			return NotFound();
		}
	}
}
