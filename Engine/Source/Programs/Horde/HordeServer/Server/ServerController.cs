// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Reflection;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Server;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Server
{
	/// <summary>
	/// Controller managing account status
	/// </summary>
	[ApiController]
	[Authorize]
	[CppApi]
	[Route("[controller]")]
	public class ServerController : HordeControllerBase
	{
		readonly IAgentVersionProvider? _agentVersionProvider;
		readonly IPluginCollection _pluginCollection;
		readonly IServerInfo _serverInfo;
		readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		readonly ILogger<ServerController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ServerController(IEnumerable<IAgentVersionProvider> agentVersionProviders, IPluginCollection pluginCollection, IServerInfo serverInfo, IOptionsSnapshot<GlobalConfig> globalConfig, ILogger<ServerController> logger)
		{
			_agentVersionProvider = agentVersionProviders.FirstOrDefault();
			_pluginCollection = pluginCollection;
			_serverInfo = serverInfo;
			_globalConfig = globalConfig;
			_logger = logger;
		}

		/// <summary>
		/// Get server version
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/server/version")]
		public ActionResult GetVersion()
		{
			FileVersionInfo fileVersionInfo = FileVersionInfo.GetVersionInfo(Assembly.GetExecutingAssembly().Location);
			return Ok(fileVersionInfo.ProductVersion);
		}

		/// <summary>
		/// Get server information
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/server/info")]
		[ProducesResponseType(typeof(GetServerInfoResponse), 200)]
		public async Task<ActionResult<GetServerInfoResponse>> GetServerInfoAsync()
		{
			GetServerInfoResponse response = new GetServerInfoResponse();
			response.ApiVersion = HordeApiVersion.Latest;

			FileVersionInfo versionInfo = FileVersionInfo.GetVersionInfo(Assembly.GetExecutingAssembly().Location);
			response.ServerVersion = versionInfo.ProductVersion ?? String.Empty;

			if (_agentVersionProvider != null)
			{
				response.AgentVersion = await _agentVersionProvider.GetAsync(HttpContext.RequestAborted);
			}

			response.Plugins = _pluginCollection.LoadedPlugins.ConvertAll(plugin => {
				FileVersionInfo pluginVersion = FileVersionInfo.GetVersionInfo(plugin.Assembly.Location);
				string[]? userAcls = GetUserAclsForPlugin(plugin);
				return new ServerPluginInfoResponse(plugin.Metadata.Name.ToString(), plugin.Metadata.Description, true, pluginVersion.ProductVersion ?? String.Empty, userAcls);
			}).ToArray();

			response.Environment = _serverInfo.Environment;

			return response;
		}

		/// <summary>
		/// Gets the list of ACL actions the current user has for a given plugin.
		/// Returns null if plugin doesn't support dashboard ACLs, or an array (possibly empty)
		/// of the user's granted ACL actions if it does.
		/// </summary>
		private string[]? GetUserAclsForPlugin(ILoadedPlugin plugin)
		{
			// Only return ACLs if the user is authenticated
			if (User?.Identity?.IsAuthenticated != true)
			{
				return null;
			}

			// Check if the plugin config implements IPluginDashboardAcls
			if (!_globalConfig.Value.Plugins.TryGetValue(plugin.Name, out IPluginConfig? pluginConfig))
			{
				return null;
			}

			if (pluginConfig is not IPluginDashboardAcls dashboardAcls)
			{
				return null;
			}

			// Get the dashboard-relevant ACL actions and check which ones the user has
			AclAction[] actions = dashboardAcls.GetDashboardAclActions();

			// Admin users get all dashboard ACL actions
			if (User.HasAdminClaim())
			{
				return [.. actions.Select(a => a.Name)];
			}

			List<string> userAcls = [];

			foreach (AclAction action in actions)
			{
				if (dashboardAcls.AuthorizeDashboardAction(action, User))
				{
					userAcls.Add(action.Name);
				}
			}

			// Return the array (possibly empty) - this distinguishes from plugins that don't use ACLs (null)
			return userAcls.ToArray();
		}

		/// <summary>
		/// Gets connection information
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/server/connection")]
		public ActionResult<GetConnectionResponse> GetConnection()
		{
			GetConnectionResponse response = new GetConnectionResponse();
			response.Ip = HttpContext.Connection.RemoteIpAddress?.ToString();
			response.Port = HttpContext.Connection.RemotePort;
			return response;
		}

		/// <summary>
		/// Gets ports used by the server
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route("/api/v1/server/ports")]
		public ActionResult<GetPortsResponse> GetPorts()
		{
			ServerSettings serverSettings = _globalConfig.Value.ServerSettings;

			GetPortsResponse response = new GetPortsResponse();
			response.Http = serverSettings.HttpPort;
			response.Https = serverSettings.HttpsPort;
			response.UnencryptedHttp2 = serverSettings.Http2Port;
			return response;
		}

		private const string ServerAuthRoute = "/api/v1/server/auth";
		
		/// <summary>
		/// Returns settings for automating auth against this server
		/// </summary>
		[HttpGet]
		[AllowAnonymous]
		[Route(ServerAuthRoute)]
		public ActionResult<GetAuthConfigResponse> GetAuthConfig()
		{
			ServerSettings settings = _globalConfig.Value.ServerSettings;

			GetAuthConfigResponse response = new GetAuthConfigResponse();
			response.Method = settings.AuthMethod;
			response.ProfileName = settings.OidcProfileName;
			if (settings.AuthMethod == AuthMethod.Horde)
			{
				response.ServerUrl = new Uri(_globalConfig.Value.ServerSettings.ServerUrl, "api/v1/oauth2").ToString();
				response.ClientId = "default";
			}
			else
			{
				if (!String.IsNullOrEmpty(settings.OidcClientSecret))
				{
					_logger.LogWarning(
						"OIDC config mismatch: Command-line auth requires a public OAuth/OIDC client, but a confidential client is configured ({ClientSecret} is set). " +
						"This will prevent Horde's C# client from signing in and block usage of Unreal Build Accelerator. " +
						"To fix: Configure your OAuth/OIDC client as public (SPA/mobile/desktop) and remove the client secret",
						nameof(ServerSettings.OidcClientSecret)
					);
				}
				response.ServerUrl = settings.OidcAuthority;
				response.ClientId = settings.OidcClientId;
				response.Scopes = settings.OidcApiRequestedScopes;
			}
			response.LocalRedirectUrls = settings.OidcLocalRedirectUrls;
			return response;
		}
	}
}
