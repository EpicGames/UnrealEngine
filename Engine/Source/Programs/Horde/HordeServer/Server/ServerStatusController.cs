// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Horde.Server;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Diagnostics.HealthChecks;

namespace HordeServer.Server;

/// <summary>
/// ASP.NET view model for server status updates
/// </summary>
public class ServerStatusUpdatesViewModel
{
	/// <summary>
	/// Status for each subsystem
	/// </summary>
	public IReadOnlyList<SubsystemStatus> SubsystemStatuses { get; init; } = Array.Empty<SubsystemStatus>();

	/// <summary>
	/// Format a date/time to human-readable relative date
	/// </summary>
	/// <param name="dateTime">Date/time to convert</param>
	/// <returns>Relative time</returns>
	public static string ToRelativeTime(DateTimeOffset dateTime)
	{
		TimeSpan timeSpan = DateTime.UtcNow - dateTime;
		if (timeSpan <= TimeSpan.FromSeconds(60))
		{
			return $"{timeSpan.Seconds} seconds ago";
		}
		if (timeSpan <= TimeSpan.FromMinutes(60))
		{
			return $"{timeSpan.Minutes} minutes ago";
		}
		if (timeSpan <= TimeSpan.FromHours(24))
		{
			return $"{timeSpan.Hours} hours ago";
		}
		if (timeSpan <= TimeSpan.FromDays(30))
		{
			return $"{timeSpan.Days} days ago";
		}
		if (timeSpan <= TimeSpan.FromDays(365))
		{
			return $"{timeSpan.Days / 30} months ago";
		}

		return $"{timeSpan.Days / 365} years ago";
	}
}

/// <summary>
/// Controller managing server status
/// </summary>
[ApiController]
[Authorize]
[Tags("Server")]
public class ServerStatusController : Controller
{
	private readonly ServerStatusService _serverStatus;

	/// <summary>
	/// Constructor
	/// </summary>
	public ServerStatusController(ServerStatusService serverStatus)
	{
		_serverStatus = serverStatus;
	}

	/// <summary>
	/// Get the server status of Horde's internal subsystems
	/// </summary>
	/// <returns>Http result</returns>
	[HttpGet]
	[Route("/api/v1/server/status")]
	[ProducesResponseType(typeof(ServerStatusResponse), 200)]
	public async Task<ActionResult<ServerStatusResponse>> GetUpdatesAsync([FromQuery] string? format = null)
	{
		IReadOnlyList<SubsystemStatus> subsystemStatuses = await _serverStatus.GetSubsystemStatusesAsync();

		if (format == "html")
		{
			return GetUpdatesHtml(subsystemStatuses);
		}

		return new ServerStatusResponse
		{
			Statuses = subsystemStatuses.Select(x =>
			{
				return new ServerStatusSubsystem()
				{
					Name = x.Name,
					Updates = x.Updates.Select(
						u => new ServerStatusUpdate()
						{
							Result = ConvertSubsystemResult(u.Result),
							Message = u.Message ?? "Operating normally.",
							UpdatedAt = u.UpdatedAt
						}).ToArray()
				};
			}).ToArray(),
		};
	}

	private ActionResult GetUpdatesHtml(IReadOnlyList<SubsystemStatus> subsystemStatuses)
	{
		return View("~/Server/ServerStatusUpdates.cshtml", new ServerStatusUpdatesViewModel
		{
			SubsystemStatuses = subsystemStatuses
		});
	}

	private static ServerStatusResult ConvertSubsystemResult(HealthStatus result)
	{
		return result switch
		{
			HealthStatus.Healthy => ServerStatusResult.Healthy,
			HealthStatus.Unhealthy => ServerStatusResult.Unhealthy,
			HealthStatus.Degraded => ServerStatusResult.Degraded,
			_ => throw new Exception($"Unknown result: {result}")
		};
	}
}