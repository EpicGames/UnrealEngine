// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.Json;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Server;

internal class HealthMonitor<T> : IHealthMonitor<T>
{
	private readonly ServerStatusService _statusService;
	private string _name;

	public HealthMonitor(ServerStatusService statusService) : this(statusService, typeof(T).Name)
	{
	}

	public HealthMonitor(ServerStatusService statusService, string name)
	{
		_statusService = statusService;
		_name = name;
	}

	public void SetName(string name)
	{
		_name = name;
	}

	public async Task UpdateAsync(HealthStatus result, string? message, DateTimeOffset? timestamp)
	{
		await _statusService.ReportAsync(typeof(T), _name, result, message, timestamp);
	}
}

/// <summary>
/// Represents status of a subsystem inside Horde
/// </summary>
/// <param name="Id">Unique ID</param>
/// <param name="Name">Human-readable name</param>
/// <param name="Updates">List of updates</param>
public record SubsystemStatus(string Id, string Name, List<SubsystemStatusUpdate> Updates)
{
	/// <inheritdoc/>
	public override string ToString()
	{
		string updates = Updates.Count > 0 ? Updates.First().ToString() : "<no updates>";
		return $"Subsystem(Id={Id} Name={Name} LastUpdate={updates})";
	}
}

/// <summary>
/// An individual status update for a subsystem
/// </summary>
/// <param name="Result"></param>
/// <param name="Message"></param>
/// <param name="UpdatedAt"></param>
public record SubsystemStatusUpdate(HealthStatus Result, string? Message, DateTimeOffset UpdatedAt);

/// <summary>
/// Tracks health and status of the Horde server itself
/// Such as connectivity to external systems (MongoDB, Redis, Perforce etc).
/// </summary>
public class ServerStatusService : IHostedService
{
	/// <summary>
	/// Max historical status updates to keep
	/// </summary>
	public const int MaxHistoryLength = 10;

	private readonly IClock _clock;
	private readonly IMongoService _mongoService;
	private readonly IRedisService _redisService;

	private readonly IHealthMonitor<MongoService> _mongoDbHealth;
	private readonly ITicker _mongoDbHealthTicker;

	private readonly IHealthMonitor<RedisService> _redisHealth;
	private readonly ITicker _redisHealthTicker;

	private static string RedisHashKey() => "server-status";

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="redisService"></param>
	/// <param name="clock"></param>
	/// <param name="mongoService"></param>
	/// <param name="logger"></param>
	public ServerStatusService(IMongoService mongoService, IRedisService redisService, IClock clock, ILogger<ServerStatusService> logger)
	{
		_mongoService = mongoService;
		_redisService = redisService;
		_clock = clock;

		_mongoDbHealth = new HealthMonitor<MongoService>(this, "MongoDB");
		_mongoDbHealthTicker = clock.AddTicker($"{nameof(ServerStatusService)}.MongoDb", TimeSpan.FromSeconds(30.0), UpdateMongoDbHealthAsync, logger);

		_redisHealth = new HealthMonitor<RedisService>(this, "Redis");
		_redisHealthTicker = clock.AddTicker($"{nameof(ServerStatusService)}.Redis", TimeSpan.FromSeconds(30.0), UpdateRedisHealthAsync, logger);
	}

	/// <inheritdoc/>
	public async Task StartAsync(CancellationToken cancellationToken)
	{
		await _mongoDbHealthTicker.StartAsync();
		await _redisHealthTicker.StartAsync();
	}

	/// <inheritdoc/>
	public async Task StopAsync(CancellationToken cancellationToken)
	{
		await _mongoDbHealthTicker.StopAsync();
		await _redisHealthTicker.StopAsync();
	}

	/// <summary>
	/// Checks health and connectivity to MongoDB database
	/// </summary>
	internal async ValueTask UpdateMongoDbHealthAsync(CancellationToken cancellationToken)
	{
		HealthCheckResult result = await _mongoService.CheckHealthAsync(new HealthCheckContext(), cancellationToken);
		await _mongoDbHealth.UpdateAsync(result.Status, result.Description);
	}

	/// <summary>
	/// Checks health and connectivity to Redis database
	/// </summary>
	internal async ValueTask UpdateRedisHealthAsync(CancellationToken cancellationToken)
	{
		HealthCheckResult result = await _redisService.CheckHealthAsync(new HealthCheckContext(), cancellationToken);
		await _redisHealth.UpdateAsync(result.Status, result.Description);
	}

	/// <summary>
	/// Report a status update for a given subsystem
	/// </summary>
	/// <param name="type">Service type reporting health</param>
	/// <param name="name">Human-readable name</param>
	/// <param name="result">Result of the update</param>
	/// <param name="message">Human-readable message</param>
	/// <param name="timestamp">Optional timestamp to be associated with the report. Defaults to UtcNow</param>
	public async Task ReportAsync(Type type, string name, HealthStatus result, string? message = null, DateTimeOffset? timestamp = null)
	{
		string id = type.Name;
		IDatabase redis = _redisService.GetDatabase();
		SubsystemStatus status = await GetSubsystemStatusFromRedisAsync(redis, id, name);
		SubsystemStatusUpdate update = new(result, message, timestamp ?? _clock.UtcNow);
		status.Updates.Add(update);
		status.Updates.Sort((a, b) => b.UpdatedAt.CompareTo(a.UpdatedAt));

		if (status.Updates.Count > MaxHistoryLength)
		{
			status.Updates.RemoveRange(MaxHistoryLength, status.Updates.Count - MaxHistoryLength);
		}

		string data = JsonSerializer.Serialize(status);
		if (!_redisService.ReadOnlyMode)
		{
			await redis.HashSetAsync(RedisHashKey(), id, data);
		}
	}

	private static async Task<SubsystemStatus> GetSubsystemStatusFromRedisAsync(IDatabase redis, string id, string name)
	{
		try
		{
			string? rawJson = await redis.HashGetAsync(RedisHashKey(), id);
			if (rawJson != null)
			{
				return JsonSerializer.Deserialize<SubsystemStatus>(rawJson) ?? throw new JsonException("Unable to parse JSON: " + rawJson);
			}
		}
		catch (Exception)
		{
			// Ignored
		}

		return new SubsystemStatus(id, name, []);
	}

	/// <summary>
	/// Get a list of status and updates for each subsystem
	/// </summary>
	/// <returns>A list of statuses</returns>
	public async Task<IReadOnlyList<SubsystemStatus>> GetSubsystemStatusesAsync()
	{
		HashEntry[] entries = await _redisService.GetDatabase().HashGetAllAsync(RedisHashKey());
		List<SubsystemStatus> subsystems = [];
		foreach (HashEntry entry in entries)
		{
			try
			{
				SubsystemStatus status = JsonSerializer.Deserialize<SubsystemStatus>(entry.Value.ToString()) ?? throw new JsonException("Failed parsing JSON");
				subsystems.Add(status);
			}
			catch (JsonException) { /* Ignored */ }
		}

		return subsystems;
	}
}
