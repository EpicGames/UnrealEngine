@using HordeServer.Server
@using Microsoft.Extensions.Diagnostics.HealthChecks
@model HordeServer.Server.ServerStatusUpdatesViewModel
@{
	ViewBag.Title = "Horde Server Status";
	Layout = "~/Server/_Layout.cshtml";
}

@functions
{
	private static string GetResultIcon(HealthStatus result, string? tooltip = null)
	{
		string title = tooltip == null ? "" : $" title=\"{tooltip}\"";
		return result switch
		{
			HealthStatus.Healthy => $"<div class=\"green-circle\"{title}></div>",
			HealthStatus.Unhealthy => $"<div class=\"red-square\"{title}></div>",
			_ => ""
		};
	}

	private static string? SafeSubstring(string? str)
	{
		int maxLength = 150;
		if (str != null && str.Length > maxLength)
		{
			return str.Substring(0, maxLength) + " ...";
		}
		return str;
	}
}

<style>
	.container {
		display: flex;
		justify-content: center;
		width: 100%;
	}
	.status-container {
		display: flex;
		flex-direction: column; /* Ensures the elements are stacked vertically */
		align-items: center; /* Centers the items horizontally */
		width: 1280px;
		padding: 20px;
		margin: 20px;
		background-color: #444;
		box-shadow: 0 2px 4px rgba(0, 0, 0, 0.5);
	}
	
	.status {
		width: 100%;
		border-collapse: collapse;
	}
	.status td, th {
		text-align: left;
		padding: 10px 35px;
		line-height: 20px;
		vertical-align: middle;
	}
	.status th {
		border-bottom: 1px solid rgb(160 160 160);
		white-space: nowrap;
		color: #AAA;
	}
	
	.green-circle {
		display: inline-block;
		width: 20px;
		height: 20px;
		border-radius: 50%;
		background-color: rgba(0, 255, 0, 0.7);
		margin-right: 3px;
		vertical-align: middle;
	}
	
	.red-square {
		display: inline-block;
		width: 20px;
		height: 20px;
		background-color: rgba(255, 0, 0, 0.7);
		margin-right: 3px;
		vertical-align: middle;
	}
	
	.triangle-up {
		display: inline-block;
		vertical-align: middle;
		width: 0;
		height: 0;
		border-left: 10px solid transparent; /* Adjust size */
		border-right: 10px solid transparent; /* Adjust size */
		border-bottom: 20px solid yellow; /* Adjust color and size */
	}
</style>
<div class="container">
	<div class="status-container">
		<h2>Horde Server Status</h2>
		
		@if (Model.SubsystemStatuses.Count > 0) 
		{
			<table class="status">
				<thead>
				<tr>
					<th>Name</th>
					<th>Last Result</th>
					<th>Last Update</th>
					<th>Last Message</th>
					<th>Previous updates</th>
				</tr>
				</thead>
				@foreach (SubsystemStatus status in Model.SubsystemStatuses)
				{
					<tr>
						<td>@status.Name</td>
						@if (status.Updates.Count >= 1)
						{
							<td style="white-space: nowrap;">@Html.Raw(GetResultIcon(status.Updates.First().Result)) <b>@status.Updates.First().Result</b></td>
							<td style="white-space: nowrap;">@ServerStatusUpdatesViewModel.ToRelativeTime(status.Updates.First().UpdatedAt)</td>
							<td>@SafeSubstring(status.Updates.First().Message)</td>
						}
						else
						{
							<td></td>
							<td></td>
							<td></td>
						}
						<td>
							@foreach (SubsystemStatusUpdate update in status.Updates)
							{
								@Html.Raw(GetResultIcon(update.Result, update.Message + " - " + ServerStatusUpdatesViewModel.ToRelativeTime(update.UpdatedAt)))
							}
						</td>
					</tr>
				}
			
			</table>
		}
		else
		{
			<tr><td>No status updates available</td></tr>
		}
	</div>
</div>