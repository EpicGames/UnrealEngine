<!DOCTYPE html>
<html lang="en">
<head>
	<meta charset="UTF-8">
	<meta name="viewport" content="width=device-width, initial-scale=1.0">
	<title>@ViewBag.Title - Horde</title>
	<style>
		body {
			font-family: Arial, sans-serif;
			background-color: #333;
			color: #fff;
			margin: 0;
			padding: 0;
			display: flex;
			height: 100vh;
			font-size: 14px;
		}
		a { color: #69adf7; }
		h2 { font-size: 24px; text-align: center; }
		input[type="text"], input[type="password"] {
			width: 100%;
			padding: 8px;
			margin-bottom: 10px;
			border: 1px solid rgb(140, 140, 140);
			background-color: #555;
			color: #fff;
			box-sizing: border-box;
			font-size: 14px;
			outline: none;
		}
		input:focus { outline: 1px solid #b4b4b4; border: 1px solid #b4b4b4; }
	</style>
</head>
<body>
	@RenderBody()
</body>
</html>