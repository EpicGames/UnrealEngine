// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using HordeServer.Commands;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Configuration.Memory;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;
using Microsoft.Win32;
using Serilog;
using Serilog.Configuration;
using Serilog.Exceptions;
using Serilog.Exceptions.Core;
using Serilog.Exceptions.Grpc.Destructurers;
using Serilog.Extensions.Logging;
using Serilog.Formatting.Json;
using Serilog.Sinks.SystemConsole.Themes;

namespace HordeServer
{
	using ILogger = Microsoft.Extensions.Logging.ILogger;

	static class LoggerExtensions
	{
		class DatadogVersionLogEnricher : Serilog.Core.ILogEventEnricher
		{
			public void Enrich(Serilog.Events.LogEvent logEvent, Serilog.Core.ILogEventPropertyFactory propertyFactory)
			{
				logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.version", ServerApp.Version));
			}
		}

		public static LoggerConfiguration Console(this LoggerSinkConfiguration sinkConfig, ServerSettings settings)
		{
			if (settings.LogJsonToStdOut)
			{
				return sinkConfig.Console(new SizeLimitedJsonFormatter(new JsonFormatter(renderMessage: true)));
			}
			else
			{
				ConsoleTheme theme;
				if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && Environment.OSVersion.Version < new Version(10, 0))
				{
					theme = SystemConsoleTheme.Literate;
				}
				else
				{
					theme = AnsiConsoleTheme.Code;
				}
				return sinkConfig.Console(outputTemplate: "[{Timestamp:HH:mm:ss} {Level:w3}] {Indent}{Message:l}{NewLine}{Exception}", theme: theme, restrictedToMinimumLevel: Serilog.Events.LogEventLevel.Debug);
			}
		}

		public static LoggerConfiguration WithHordeConfig(this LoggerConfiguration configuration, ServerSettings settings)
		{
			if (settings.OpenTelemetry.EnableDatadogCompatibility)
			{
				configuration = configuration.Enrich.With<DatadogVersionLogEnricher>();
				configuration = configuration.Enrich.With<OpenTelemetryDatadogLogEnricher>();
			}

			return configuration;
		}
	}

	class ServerApp
	{
		public static SemVer Version { get; } = GetVersion();

		public static string DeploymentEnvironment { get; } = GetEnvironment();

		public static string SessionId { get; } = Guid.NewGuid().ToString("n");

		public static DirectoryReference AppDir { get; } = GetAppDir();

		public static DirectoryReference DataDir => s_dataDir;

		public static DirectoryReference ConfigDir => s_configDir;

		public static FileReference ServerConfigFile => s_serverConfigFile ?? throw new InvalidOperationException("ServerConfigFile has not been initialized");

		public static Type[] ConfigSchemas = FindSchemaTypes();

		public static IPluginCollection Plugins => s_pluginCollection ?? throw new InvalidOperationException("Plugin collection has not been initialized");

		private static DirectoryReference s_dataDir = DirectoryReference.Combine(GetAppDir(), "Data");
		private static DirectoryReference s_configDir = DirectoryReference.Combine(GetAppDir(), "Defaults");
		private static FileReference? s_serverConfigFile;
		private static IPluginCollection? s_pluginCollection;

		static Type[] FindSchemaTypes()
		{
			List<Type> schemaTypes = new List<Type>();
			foreach (Type type in Assembly.GetExecutingAssembly().GetTypes())
			{
				if (type.GetCustomAttribute<JsonSchemaAttribute>() != null)
				{
					schemaTypes.Add(type);
				}
			}
			return schemaTypes.ToArray();
		}

		static SemVer GetVersion()
		{
			FileVersionInfo versionInfo = FileVersionInfo.GetVersionInfo(Assembly.GetExecutingAssembly().Location);
			if (String.IsNullOrEmpty(versionInfo.ProductVersion))
			{
				return SemVer.Parse("0.0.0");
			}
			else
			{
				return SemVer.Parse(versionInfo.ProductVersion);
			}
		}

		public static async Task<int> Main(string[] args)
		{
			CommandLineArguments arguments = new CommandLineArguments(args);

			// Create the base configuration data by just reading from the application directory. We need to check some settings before
			// being able to read user configuration files.
			IConfiguration baseConfig = CreateConfig(false, null);

			ServerSettings baseServerSettings = new ServerSettings();
			Startup.BindServerSettings(baseConfig, baseServerSettings);

			// Set the default data directory
			if (baseServerSettings.DataDir != null)
			{
				if (Path.IsPathRooted(baseServerSettings.DataDir))
				{
					DirectoryReference? dataDir = DirectoryReference.FromString(baseServerSettings.DataDir);
					if (dataDir != null)
					{
						s_dataDir = dataDir;
					}
				}
				else
				{
					s_dataDir = DirectoryReference.Combine(GetAppDir(), baseServerSettings.DataDir);
				}
			}
			else if (baseServerSettings.Installed && RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				DirectoryReference? commonDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData);
				if (commonDataDir != null)
				{
					s_dataDir = DirectoryReference.Combine(commonDataDir, "Epic", "Horde", "Server");
				}
			}

			// For installed builds, copy default config files to the data dir and use that as the config dir instead
			if (baseServerSettings.Installed)
			{
				await CopyDefaultConfigFilesAsync(s_configDir, s_dataDir, CancellationToken.None);
				s_configDir = s_dataDir;
			}

			// Create the final configuration, including the server.json file
			s_serverConfigFile = FileReference.Combine(s_configDir, "server.json");
			IConfiguration config = CreateConfig(baseServerSettings.Installed, s_serverConfigFile);

			// Bind the complete settings
			ServerSettings serverSettings = new ServerSettings();
			Startup.BindServerSettings(config, serverSettings);

			DirectoryReference logDir = DirectoryReference.Combine(DataDir, "Logs");
			Serilog.Log.Logger = new LoggerConfiguration()
				.WithHordeConfig(serverSettings)
				.Enrich.FromLogContext()
				.Enrich.WithExceptionDetails(new DestructuringOptionsBuilder()
					.WithDefaultDestructurers()
					.WithDestructurers([new RpcExceptionDestructurer()])
					.WithDestructuringDepth(3))
				.WriteTo.Console(serverSettings)
				.WriteTo.File(Path.Combine(logDir.FullName, "Log.txt"), outputTemplate: "[{Timestamp:HH:mm:ss} {Level:w3}] {Indent}{Message:l}{NewLine}{Exception}", rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, fileSizeLimitBytes: 20 * 1024 * 1024, retainedFileCountLimit: 10)
				.WriteTo.File(new SizeLimitedJsonFormatter(new JsonFormatter(renderMessage: true)), Path.Combine(logDir.FullName, "Log.json"), rollingInterval: RollingInterval.Day, rollOnFileSizeLimit: true, fileSizeLimitBytes: 20 * 1024 * 1024, retainedFileCountLimit: 10)
				.ReadFrom.Configuration(config)
				.CreateLogger();

#pragma warning disable CA2000 // Dispose objects before losing scope
			ILogger startupLogger = new SerilogLoggerFactory().CreateLogger(typeof(ServerApp).FullName ?? "ServerApp");
#pragma warning restore CA2000 // Dispose objects before losing scope
			
			serverSettings.Validate(startupLogger);

			try
			{
				ServiceCollection services = new ServiceCollection();
				services.AddCommandsFromAssembly(Assembly.GetExecutingAssembly());
				services.AddLogging(builder => builder.AddSerilog());
				services.AddSingleton<IConfiguration>(config);
				services.AddSingleton<ServerSettings>(serverSettings);
				services.Configure<ServerSettings>(x => Startup.BindServerSettings(config, x));
				services.AddTransient<IServerStartup, Startup>();

				ServerInfo serverInfo = new ServerInfo(config, Options.Create(serverSettings));
				services.AddSingleton<IServerInfo>(serverInfo);

				s_pluginCollection = CreatePluginCollection(config, startupLogger);
				services.AddSingleton<IPluginCollection>(s_pluginCollection);

				foreach (Assembly pluginAssembly in s_pluginCollection.LoadedPlugins.Select(x => x.Assembly).Distinct())
				{
					services.AddCommandsFromAssembly(pluginAssembly);
				}

#pragma warning disable ASP0000 // Do not call 'IServiceCollection.BuildServiceProvider' in 'ConfigureServices'
				await using (ServiceProvider serviceProvider = services.BuildServiceProvider())
				{
					return await CommandHost.RunAsync(arguments, serviceProvider, typeof(ServerCommand));
				}
#pragma warning restore ASP0000 // Do not call 'IServiceCollection.BuildServiceProvider' in 'ConfigureServices'
			}
			catch (Exception ex)
			{
				startupLogger.LogError(ex, "Uncaught exception: {Message}", ex.Message);
				throw;
			}
		}

		internal static void InitializePluginsForTests()
		{
			s_pluginCollection ??= CreatePluginCollection(new ConfigurationBuilder().Build(), NullLogger.Instance);
		}
		
		/// <summary>
		/// Creates a plugin collection by loading and configuring plugins from assemblies based on configuration settings.
		///
		/// The method performs the following steps:
		/// 1. Loads plugin configurations from the "Horde.Plugins" configuration section
		/// 2. Scans the application directory for DLL files matching the pattern "HordeServer.*.dll"
		/// 3. Loads each matching assembly and searches for types decorated with <see cref="PluginAttribute"/>
		/// 4. Enables plugins based on configuration settings and default values
		/// 5. Validates that all plugins enabled in configuration are found
		/// </summary>
		/// <returns>An initialized <see cref="IPluginCollection"/> containing all successfully loaded and enabled plugins.</returns>
		/// <exception cref="InvalidOperationException">Thrown when one or more plugins enabled in the configuration file cannot be found.</exception>
		static IPluginCollection CreatePluginCollection(IConfiguration configuration, ILogger logger)
		{
			Dictionary<string, PluginServerConfig> pluginConfigs = new Dictionary<string, PluginServerConfig>(StringComparer.OrdinalIgnoreCase);
			configuration.GetSection("Horde").GetSection("Plugins").Bind(pluginConfigs);

			List<FileReference> files = new List<FileReference>();
			foreach (FileInfo fileInfo in AppDir.ToDirectoryInfo().EnumerateFiles())
			{
				const string Prefix = "HordeServer.";
				const string Suffix = ".dll";
				if (fileInfo.Name.StartsWith(Prefix, StringComparison.OrdinalIgnoreCase) && fileInfo.Name.EndsWith(Suffix, StringComparison.OrdinalIgnoreCase) && fileInfo.Name.Length > Prefix.Length + Suffix.Length)
				{
					files.Add(new FileReference(fileInfo));
				}
			}

			HashSet<string> missingPlugins = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach ((string name, PluginServerConfig pluginConfig) in pluginConfigs)
			{
				if (pluginConfig.Enabled ?? true)
				{
					missingPlugins.Add(name);
				}
			}

			PluginCollection pluginCollection = new PluginCollection();
			foreach (FileReference file in files)
			{
				logger.LogInformation("Loading {File}", file);
				try
				{
					Assembly assembly = Assembly.LoadFrom(file.FullName);
					foreach (Type type in assembly.GetExportedTypes())
					{
						PluginAttribute? pluginAttribute = type.GetCustomAttribute<PluginAttribute>();
						if (pluginAttribute != null)
						{
							bool? enabled = null;
							if (pluginConfigs.TryGetValue(pluginAttribute.Name, out PluginServerConfig? pluginConfig))
							{
								enabled = pluginConfig.Enabled;
							}
							
							if (enabled ?? pluginAttribute.EnabledByDefault)
							{
								logger.LogDebug("Added plugin '{Plugin}'", pluginAttribute.Name);
								pluginCollection.Add(type);
								missingPlugins.Remove(pluginAttribute.Name);
							}
						}
					}
				}
				catch (Exception ex)
				{
					logger.LogError(ex, "Error loading plugins from {File}: {Message}", file, ex.Message);
					throw;
				}
			}

			if (missingPlugins.Count > 0)
			{
				throw new InvalidOperationException($"Unable to find plugin(s) enabled in config file: {StringUtils.FormatList(missingPlugins)}");
			}

			return pluginCollection;
		}

		// Used by WebApplicationFactory in controller tests. Uses reflection to call this exact function signature.
		public static IHostBuilder CreateHostBuilder(string[] args) => ServerCommand.CreateHostBuilderForTesting(args);

		/// <summary>
		/// Gets the current environment
		/// </summary>
		/// <returns></returns>
		static string GetEnvironment()
		{
			string? environment = Environment.GetEnvironmentVariable("ASPNETCORE_ENVIRONMENT");
			if (String.IsNullOrEmpty(environment))
			{
				environment = "Production";
			}
			return environment;
		}

		/// <summary>
		/// Get the application directory
		/// </summary>
		/// <returns></returns>
		static DirectoryReference GetAppDir()
		{
			return new FileReference(Assembly.GetExecutingAssembly().Location).Directory;
		}

		/// <summary>
		/// List of config settings to rename to new locations
		/// </summary>
		static readonly KeyValuePair<string, string>[] s_renamedConfigValues = new[]
		{
			// Build plugin
			KeyValuePair.Create("Horde:UseLocalPerforceEnv", "Horde:Plugins:Build:UseLocalPerforceEnv"),
			KeyValuePair.Create("Horde:PerforceConnectionPoolSize", "Horde:Plugins:Build:PerforceConnectionPoolSize"),
			KeyValuePair.Create("Horde:EnableConformTasks", "Horde:Plugins:Build:EnableConformTasks"),
			KeyValuePair.Create("Horde:P4SwarmUrl", "Horde:Plugins:Build:P4SwarmUrl"),
			KeyValuePair.Create("Horde:RobomergeUrl", "Horde:Plugins:Build:RobomergeUrl"),
            KeyValuePair.Create("Horde:CommitsViewerUrl", "Horde:Plugins:Build:CommitsViewerUrl"),
            KeyValuePair.Create("Horde:JiraUsername", "Horde:Plugins:Build:JiraUsername"),
			KeyValuePair.Create("Horde:JiraApiToken", "Horde:Plugins:Build:JiraApiToken"),
			KeyValuePair.Create("Horde:JiraUrl", "Horde:Plugins:Build:JiraUrl"),
			KeyValuePair.Create("Horde:SharedDeviceCheckoutDays", "Horde:Plugins:Build:SharedDeviceCheckoutDays"),
			KeyValuePair.Create("Horde:DeviceProblemCooldownMinutes", "Horde:Plugins:Build:DeviceProblemCooldownMinutes"),
			KeyValuePair.Create("Horde:DeviceReportChannel", "Horde:Plugins:Build:DeviceReportChannel"),
			KeyValuePair.Create("Horde:DisableSchedules", "Horde:Plugins:Build:DisableSchedules"),
			KeyValuePair.Create("Horde:SlackToken", "Horde:Plugins:Build:SlackToken"),
			KeyValuePair.Create("Horde:SlackSocketToken", "Horde:Plugins:Build:SlackSocketToken"),
			KeyValuePair.Create("Horde:SlackAdminToken", "Horde:Plugins:Build:SlackAdminToken"),
			KeyValuePair.Create("Horde:SlackUsers", "Horde:Plugins:Build:SlackUsers"),
			KeyValuePair.Create("Horde:SlackErrorPrefix", "Horde:Plugins:Build:SlackErrorPrefix"),
			KeyValuePair.Create("Horde:SlackWarningPrefix", "Horde:Plugins:Build:SlackWarningPrefix"),
			KeyValuePair.Create("Horde:ConfigNotificationChannel", "Horde:Plugins:Build:ConfigNotificationChannel"),
			KeyValuePair.Create("Horde:UpdateStreamsNotificationChannel", "Horde:Plugins:Build:UpdateStreamsNotificationChannel"),
			KeyValuePair.Create("Horde:JobNotificationChannel", "Horde:Plugins:Build:JobNotificationChannel"),
			KeyValuePair.Create("Horde:AgentNotificationChannel", "Horde:Plugins:Build:AgentNotificationChannel"),
			KeyValuePair.Create("Horde:TestDataRetainMonths", "Horde:Plugins:Build:TestDataRetainMonths"),
			KeyValuePair.Create("Horde:BlockCacheDir", "Horde:Plugins:Build:BlockCacheDir"),
			KeyValuePair.Create("Horde:BlockCacheSize", "Horde:Plugins:Build:BlockCacheSize"),
			KeyValuePair.Create("Horde:Perforce:", "Horde:Plugins:Build:Perforce:"),
			KeyValuePair.Create("Horde:Commits:", "Horde:Plugins:Build:Commits:"),

			// Compute plugin
			KeyValuePair.Create("Horde:EnableUpgradeTasks", "Horde:Plugins:Compute:EnableUpgradeTasks"),
			KeyValuePair.Create("Horde:FleetManagerV2", "Horde:Plugins:Compute:FleetManagerV2"),
			KeyValuePair.Create("Horde:FleetManagerV2Config", "Horde:Plugins:Compute:FleetManagerV2Config"),
			KeyValuePair.Create("Horde:AwsAutoScalingQueueUrls", "Horde:Plugins:Compute:AwsAutoScalingQueueUrls"),
			KeyValuePair.Create("Horde:AutoEnrollAgents", "Horde:Plugins:Compute:AutoEnrollAgents"),
			KeyValuePair.Create("Horde:DefaultAgentPoolSizeStrategy", "Horde:Plugins:Compute:DefaultAgentPoolSizeStrategy"),
			KeyValuePair.Create("Horde:AgentPoolScaleOutCooldownSeconds", "Horde:Plugins:Compute:AgentPoolScaleOutCooldownSeconds"),
			KeyValuePair.Create("Horde:AgentPoolScaleInCooldownSeconds", "Horde:Plugins:Compute:AgentPoolScaleInCooldownSeconds"),
			KeyValuePair.Create("Horde:ComputeTunnelPort", "Horde:Plugins:Compute:ComputeTunnelPort"),
			KeyValuePair.Create("Horde:ComputeTunnelAddress", "Horde:Plugins:Compute:ComputeTunnelAddress"),
			KeyValuePair.Create("Horde:WithAws", "Horde:Plugins:Compute:WithAws"),

			// Secrets plugin
			KeyValuePair.Create("Horde:WithAws", "Horde:Plugins:Secrets:WithAws"),

			// Storage plugin
			KeyValuePair.Create("Horde:BundleCacheDir", "Horde:Plugins:Storage:BundleCacheDir"),
			KeyValuePair.Create("Horde:BundleCacheSize", "Horde:Plugins:Storage:BundleCacheSize"),
			KeyValuePair.Create("Horde:Backends:", "Horde:Plugins:Storage:Backends:"),
	
			// Tools plugin
			KeyValuePair.Create("Horde:BundledTools:", "Horde:Plugins:Tools:BundledTools:")
		};

		/// <summary>
		/// Constructs a configuration object for the current environment
		/// </summary>
		/// <returns></returns>
		static IConfiguration CreateConfig(bool readInstalledConfig, FileReference? serverConfigFile)
		{
			IConfigurationBuilder builder = new ConfigurationBuilder();

			string? regDataDir = null;
			if (OperatingSystem.IsWindows())
			{
				if (readInstalledConfig)
				{
					builder = builder.Add(new RegistryConfigurationSource(Registry.LocalMachine, "SOFTWARE\\Epic Games\\Horde\\Server", ServerSettings.SectionName));
				}
				else
				{
					// Check the registry for an installed server, if this is running we need to set the data directory if available
					string? installedServerExecutable = Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Epic Games\\Horde\\Server", "InstalledServerExecutable", null) as string;
					if (!String.IsNullOrEmpty(installedServerExecutable))
					{
						FileReference installedServer = new FileReference(installedServerExecutable);
						FileReference executingServer = new FileReference(Assembly.GetExecutingAssembly().Location);

						if (installedServer == executingServer)
						{
							regDataDir = Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Epic Games\\Horde\\Server", "DataDir", null) as string;
						}
					}
				}
			}

			builder.SetBasePath(AppDir.FullName)
				.AddJsonFile("appsettings.json", optional: false)
				.AddJsonFile("appsettings.Build.json", optional: true) // specific settings for builds (installer/dockerfile)
				.AddJsonFile($"appsettings.{DeploymentEnvironment}.json", optional: true) // environment variable overrides, also used in k8s setups with Helm
				.AddJsonFile("appsettings.User.json", optional: true);

			if (serverConfigFile != null)
			{
				builder = builder.AddJsonFile(serverConfigFile.FullName, optional: true, reloadOnChange: true);
			}

			builder = builder.AddEnvironmentVariables();

			// Create a temporary configuration object and apply any upgrades and key renames
			IConfiguration configuration = builder.Build();

			List<KeyValuePair<string, string?>> remappedValues = new List<KeyValuePair<string, string?>>();
			foreach ((string source, string target) in s_renamedConfigValues)
			{
				if (source.EndsWith(':'))
				{
					foreach (KeyValuePair<string, string?> pair in configuration.AsEnumerable().Where(x => x.Key.StartsWith(source, StringComparison.OrdinalIgnoreCase)))
					{
						if (pair.Value != null)
						{
							remappedValues.Add(new KeyValuePair<string, string?>(target + pair.Key.Substring(source.Length), pair.Value));
						}
					}
				}
				else
				{
					string? value = configuration[source];
					if (value != null && configuration[target] == null)
					{
						remappedValues.Add(new KeyValuePair<string, string?>(target, value));
					}
				}
			}

			// set data directory if we have one from registry
			if (!readInstalledConfig && !String.IsNullOrEmpty(regDataDir))
			{
				remappedValues.Add(new KeyValuePair<string, string?>("Horde:DataDir", regDataDir));
			}

			builder.Add(new MemoryConfigurationSource { InitialData = remappedValues });
			return builder.Build();
		}

		static async Task CopyDefaultConfigFilesAsync(DirectoryReference sourceDir, DirectoryReference targetDir, CancellationToken cancellationToken)
		{
			DirectoryReference.CreateDirectory(targetDir);
			foreach (FileReference sourceFile in DirectoryReference.EnumerateFiles(sourceDir))
			{
				if ((sourceFile.HasExtension(".json") || sourceFile.HasExtension(".png")) && !sourceFile.GetFileName().StartsWith("default", StringComparison.OrdinalIgnoreCase))
				{
					FileReference targetFile = FileReference.Combine(targetDir, sourceFile.GetFileName());
					if (!FileReference.Exists(targetFile))
					{
						// Copy the data to the output file. Create a new file to reset permissions.
						using (FileStream targetStream = FileReference.Open(targetFile, FileMode.Create, FileAccess.Write, FileShare.Read))
						{
							using FileStream sourceStream = FileReference.Open(sourceFile, FileMode.Open, FileAccess.Read, FileShare.Read);
							await sourceStream.CopyToAsync(targetStream, cancellationToken);
						}
					}
				}
			}
		}
	}
}
