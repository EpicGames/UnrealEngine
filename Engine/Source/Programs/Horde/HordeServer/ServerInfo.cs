// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.Options;

namespace HordeServer
{
	/// <summary>
	/// Concrete implementation of <see cref="IServerInfo"/>
	/// </summary>
	class ServerInfo : IServerInfo
	{
		/// <inheritdoc/>
		public SemVer Version => ServerApp.Version;

		/// <inheritdoc/>
		public string Environment => ServerApp.DeploymentEnvironment;

		/// <inheritdoc/>
		public string SessionId => ServerApp.SessionId;

		/// <inheritdoc/>
		public DirectoryReference AppDir => ServerApp.AppDir;

		/// <inheritdoc/>
		public DirectoryReference DataDir => ServerApp.DataDir;

		/// <inheritdoc/>
		public IConfiguration Configuration => _configuration;

		/// <inheritdoc/>
		public bool EnableDebugEndpoints => _serverSettings.Value.EnableDebugEndpoints;

		/// <inheritdoc/>
		public bool ReadOnlyMode => _serverSettings.Value.MongoReadOnlyMode;

		/// <inheritdoc/>
		public Uri ServerUrl => _serverSettings.Value.ServerUrl;

		/// <inheritdoc/>
		public Uri DashboardUrl => _serverSettings.Value.DashboardUrl;

		readonly IConfiguration _configuration;
		readonly IOptions<ServerSettings> _serverSettings;

		/// <summary>
		/// Constructor
		/// </summary>
		public ServerInfo(IConfiguration configuration, IOptions<ServerSettings> serverSettings)
		{
			_configuration = configuration;
			_serverSettings = serverSettings;
		}

		/// <inheritdoc/>
		public bool IsRunModeActive(RunMode mode)
			=> _serverSettings.Value.IsRunModeActive(mode);
	}
}
