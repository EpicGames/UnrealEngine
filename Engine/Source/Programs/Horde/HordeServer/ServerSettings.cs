// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net;
using EpicGames.Horde.Server;
using EpicGames.Horde.Utilities;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

namespace HordeServer
{
	/// <summary>
	/// Feature flags to aid rollout of new features.
	///
	/// Once a feature is running in its intended state and is stable, the flag should be removed.
	/// A name and date of when the flag was created is noted next to it to help encourage this behavior.
	/// Try having them be just a flag, a boolean.
	/// </summary>
	public class FeatureFlagSettings
	{
	}

	/// <summary>
	/// Global settings for the application
	/// </summary>
	[ConfigDoc("Server.json", "[Horde](../../README.md) > [Deployment](../Deployment.md) > [Server](Server.md)", "Deployment/ServerSettings.md", 
		Introduction = "All Horde-specific settings are stored in a root object called `Horde`. Other .NET functionality may be configured using properties in the root of this file."
	)]
	public class ServerSettings
	{
		/// <summary>
		/// Name of the section containing these settings
		/// </summary>
		public const string SectionName = "Horde";

		/// <summary>
		/// Modes that the server should run in. Runmodes can be used in a multi-server deployment to limit the operations that a particular instance will try to perform.
		/// </summary>
		public RunMode[]? RunModes { get; set; } = null;

		/// <summary>
		/// Override the data directory used by Horde. Defaults to C:\ProgramData\HordeServer on Windows, {AppDir}/Data on other platforms.
		/// </summary>
		public string? DataDir { get; set; } = null;

		/// <summary>
		/// Whether the server is running in 'installed' mode. In this mode, on Windows, the default data directory will use the common 
		/// application data folder (C:\ProgramData\Epic\Horde), and configuration data will be read from here and the registry.
		/// This setting is overridden to false for local builds from appsettings.Local.json.
		/// </summary>
		public bool Installed { get; set; } = true;

		/// <summary>
		/// Main port for serving HTTP.
		/// </summary>
		public int HttpPort { get; set; } = 5000;

		/// <summary>
		/// Port for serving HTTP with TLS enabled. Disabled by default.
		/// </summary>
		public int HttpsPort { get; set; } = 0;

		/// <summary>
		/// Dedicated port for serving only HTTP/2.
		/// </summary>
		public int Http2Port { get; set; } = 5002;

		/// <summary>
		/// Connection string for the Mongo database
		/// </summary>
		public string? MongoConnectionString { get; set; }

		/// <summary>
		/// MongoDB connection string
		/// </summary>
		[Obsolete("Use MongoConnectionString instead")]
		public string? DatabaseConnectionString
		{
			get => MongoConnectionString;
			set => MongoConnectionString = value;
		}

		/// <summary>
		/// MongoDB database name
		/// </summary>
		public string MongoDatabaseName { get; set; } = "Horde";

		/// <inheritdoc cref="DatabaseName"/>
		[Obsolete("Replace references to DatabaseName with MongoDatabaseName")]
		public string DatabaseName
		{
			get => MongoDatabaseName;
			set => MongoDatabaseName = value;
		}

		/// <summary>
		/// Optional certificate to trust in order to access the database (eg. AWS public cert for TLS)
		/// </summary>
		public string? MongoPublicCertificate { get; set; }

		/// <inheritdoc cref="MongoPublicCertificate"/>
		[Obsolete("Replace DatabasePublicCert with MongoPublicCertificate")]
		public string? DatabasePublicCert
		{
			get => MongoPublicCertificate;
			set => MongoPublicCertificate = value;
		}

		/// <summary>
		/// Access the database in read-only mode (avoids creating indices or updating content)
		/// Useful for debugging a local instance of HordeServer against a production database.
		/// </summary>
		public bool MongoReadOnlyMode { get; set; } = false;

		/// <inheritdoc cref="MongoReadOnlyMode"/>
		[Obsolete("Replace DatabaseReadOnlyMode with MongoReadOnlyMode")]
		public bool DatabaseReadOnlyMode
		{
			get => MongoReadOnlyMode;
			set => MongoReadOnlyMode = value;
		}
		
		/// <summary>
		/// Whether database schema migrations are enabled
		/// </summary>
		public bool MongoMigrationsEnabled { get; set; } = false;
		
		/// <summary>
		/// Whether database schema should automatically be applied
		/// Only recommended for dev or test environments
		/// </summary>
		public bool MongoMigrationsAutoUpgrade { get; set; } = false;

		/// <summary>
		/// Shutdown the current server process if memory usage reaches this threshold (specified in MB)
		///
		/// Usually set to 80-90% of available memory to avoid CLR heap using all of it.
		/// If a memory leak was to occur, it's usually better to restart the process rather than to let the GC
		/// work harder and harder trying to recoup memory.
		/// 
		/// Should only be used when multiple server processes are running behind a load balancer
		/// and one can be safely restarted automatically by the underlying process handler (Docker, Kubernetes, AWS ECS, Supervisor etc).
		/// The shutdown behaves similar to receiving a SIGTERM and will wait for outstanding requests to finish.
		/// </summary>
		public int? ShutdownMemoryThreshold { get; set; } = null;

		/// <summary>
		/// Optional PFX certificate to use for encrypting agent SSL traffic. This can be a self-signed certificate, as long as it's trusted by agents.
		/// </summary>
		public string? ServerPrivateCert { get; set; }

		/// <summary>
		/// Type of authentication (e.g anonymous, OIDC, built-in Horde accounts)
		/// If "Horde" auth mode is used, be sure to configure "ServerUrl" as well.
		/// </summary>
		/// <see cref="ServerUrl"/>
		public AuthMethod AuthMethod { get; set; } = AuthMethod.Anonymous;
		
		/// <summary>
		/// Optional profile name to report through the /api/v1/server/auth endpoint. Allows sharing auth tokens between providers configured through
		/// the same profile name in OidcToken.exe config files.
		/// </summary>
		public string? OidcProfileName { get; set; }

		/// <summary>
		/// OpenID Connect (OIDC) authority URL (required when OIDC is enabled)
		/// </summary>
		public string? OidcAuthority { get; set; }

		/// <summary>
		/// Audience for validating externally issued tokens (required when OIDC is enabled)
		/// </summary>
		public string? OidcAudience { get; set; }

		/// <summary>
		/// Client ID for the OIDC authority (required when OIDC is enabled)
		/// </summary>
		public string? OidcClientId { get; set; }

		/// <summary>
		/// Client secret for authenticating with the OIDC provider.
		/// Note: If you need authentication support in Unreal Build Tool or Unreal Game Sync,
		/// configure your OIDC client as a public client (using PKCE flow without a client secret) instead of a confidential client.
		/// These tools utilize the EpicGames.OIDC library which only supports public clients with authorization code flow + PKCE.
		/// </summary>
		public string? OidcClientSecret { get; set; }

		/// <summary>
		/// Optional redirect url provided to OIDC login
		/// </summary>
		public string? OidcSigninRedirect { get; set; }

		/// <summary>
		/// Optional redirect url provided to OIDC login for external tools (typically to a local server)
		/// Default value is the local web server started during signin by EpicGames.OIDC library
		/// </summary>
		public string[]? OidcLocalRedirectUrls { get; set; } = ["http://localhost:8749/ugs.client"];
		
		/// <summary>
		/// Debug mode for OIDC which logs reasons for why JWT tokens fail to authenticate
		/// Also turns off HTTPS requirement for OIDC metadata fetching.
		/// NOT FOR PRODUCTION USE!
		/// </summary>
		public bool OidcDebugMode { get; set; } = false;
		
		/// <summary>
		/// OpenID Connect scopes to request when signing in
		/// </summary>
		public string[] OidcRequestedScopes { get; set; } = [];

		/// <summary>
		/// List of fields in /userinfo endpoint to try map to the standard name claim (see System.Security.Claims.ClaimTypes.Name)
		/// </summary>
		public string[] OidcClaimNameMapping { get; set; } = [];

		/// <summary>
		/// List of fields in /userinfo endpoint to try map to the standard email claim (see System.Security.Claims.ClaimTypes.Email)
		/// </summary>
		public string[] OidcClaimEmailMapping { get; set; } = [];

		/// <summary>
		/// List of fields in /userinfo endpoint to try map to the Horde user claim (see HordeClaimTypes.User)
		/// </summary>
		public string[] OidcClaimHordeUserMapping { get; set; } = [];

		/// <summary>
		/// List of fields in /userinfo endpoint to try map to the Horde Perforce user claim (see HordeClaimTypes.PerforceUser)
		/// </summary>
		public string[] OidcClaimHordePerforceUserMapping { get; set; } = [];
		
		/// <summary>
		/// API scopes to request when acquiring OIDC access tokens
		/// </summary>
		public string[] OidcApiRequestedScopes { get; set; } = [];
		
		/// <summary>
		/// Add common scopes and mappings to above OIDC config fields
		/// Provided as a workaround since .NET config will only *merge* array entries when combining multiple config sources.
		/// Due to this unwanted behavior, having hard-coded defaults makes such fields unchangeable.
		/// See https://github.com/dotnet/runtime/issues/36569 
		/// <see cref="OidcRequestedScopes" />
		/// <see cref="OidcClaimNameMapping" />
		/// <see cref="OidcClaimEmailMapping" />
		/// <see cref="OidcClaimHordeUserMapping" />
		/// <see cref="OidcClaimHordePerforceUserMapping" />
		/// </summary>
		public bool OidcAddDefaultScopesAndMappings { get; set; } = true;
		
		/// <summary>
		/// Adds default scopes and claim mappings to the OIDC configuration if enabled.
		/// This method should be called after the initial configuration is loaded.
		/// </summary>
		public void AddDefaultOidcScopesAndMappings()
		{
			IReadOnlyList<string> defaultScopes = ["profile", "email", "openid"];
			IReadOnlyList<string> defaultMappings = ["preferred_username", "email"];
			IReadOnlyList<string> defaultEmailMapping = ["email"];
			IReadOnlyList<string> defaultApiScopes = ["offline_access", "openid"];
			
			if (OidcAddDefaultScopesAndMappings)
			{
				OidcRequestedScopes = OidcRequestedScopes.Concat(defaultScopes).ToArray();
				OidcClaimNameMapping = OidcClaimNameMapping.Concat(defaultMappings).ToArray();
				OidcClaimEmailMapping = OidcClaimEmailMapping.Concat(defaultEmailMapping).ToArray();
				OidcClaimHordeUserMapping = OidcClaimHordeUserMapping.Concat(defaultMappings).ToArray();
				OidcClaimHordePerforceUserMapping = OidcClaimHordePerforceUserMapping.Concat(defaultMappings).ToArray();
				OidcApiRequestedScopes = OidcApiRequestedScopes.Concat(defaultApiScopes).ToArray();
			}
		}

		/// <summary>
		/// Base URL this Horde server is accessible from
		/// For example https://horde.mystudio.com. If not set, a default is used based on current hostname.
		/// It's important this URL matches where users and agents access the server as it's used for signing auth tokens etc.
		/// Must be configured manually when running behind a reverse proxy or load balancer
		/// </summary>
		public Uri ServerUrl
		{
			get => _serverUrl ?? GetDefaultServerUrl();
			set => _serverUrl = value;
		}

		/// <summary>
		/// Name of the issuer in bearer tokens from the server
		/// </summary>
		public string? JwtIssuer
		{
			get => _jwtIssuer ?? ServerUrl.ToString();
			set => _jwtIssuer = value;
		}
		
		internal bool IsDefaultServerUrlUsed() =>_serverUrl == GetDefaultServerUrl();

		Uri? _serverUrl;
		string? _jwtIssuer;

		Uri GetDefaultServerUrl()
		{
			string hostName = Dns.GetHostName();
			if (HttpsPort == 443)
			{
				return new Uri($"https://{hostName}");
			}
			else if (HttpsPort != 0)
			{
				return new Uri($"https://{hostName}:{HttpsPort}");
			}
			else if (HttpPort == 80)
			{
				return new Uri($"http://{hostName}");
			}
			else
			{
				return new Uri($"http://{hostName}:{HttpPort}");
			}
		}

		/// <summary>
		/// Length of time before JWT tokens expire, in hours
		/// </summary>
		public int JwtExpiryTimeHours { get; set; } = 8;

		/// <summary>
		/// Length of time before extended JWT tokens expire, in hours, defaults to 1 week
		/// </summary>
		public int ExtendedJwtExpiryTimeHours { get; set; } = 168;

		/// <summary>
		/// The claim type for administrators
		/// </summary>
		public string? AdminClaimType { get; set; }

		/// <summary>
		/// Value of the claim type for administrators
		/// </summary>
		public string? AdminClaimValue { get; set; }

		/// <summary>
		/// Whether to enable Cors, generally for development purposes
		/// </summary>
		public bool CorsEnabled { get; set; } = false;

		/// <summary>
		/// Allowed Cors origin 
		/// </summary>
		public string CorsOrigin { get; set; } = null!;

		/// <summary>
		/// Whether to enable debug/administrative REST API endpoints
		/// </summary>
		public bool EnableDebugEndpoints { get; set; } = false;

		/// <summary>
		/// Whether to automatically enable new agents by default. If false, new agents must manually be enabled before they can take on work.
		/// </summary>
		public bool EnableNewAgentsByDefault { get; set; } = false;

		/// <summary>
		/// Interval between rebuilding the schedule queue with a DB query.
		/// </summary>
		public TimeSpan SchedulePollingInterval { get; set; } = TimeSpan.FromSeconds(60.0);

		/// <summary>
		/// Interval between polling for new jobs
		/// </summary>
		public TimeSpan NoResourceBackOffTime { get; set; } = TimeSpan.FromSeconds(30.0);

		/// <summary>
		/// Interval between attempting to assign agents to take on jobs
		/// </summary>
		public TimeSpan InitiateJobBackOffTime { get; set; } = TimeSpan.FromSeconds(180.0);

		/// <summary>
		/// Interval between scheduling jobs when an unknown error occurs
		/// </summary>
		public TimeSpan UnknownErrorBackOffTime { get; set; } = TimeSpan.FromSeconds(120.0);

		/// <summary>
		/// Config for connecting to Redis server(s).
		/// Setting it to null will disable Redis use and connection
		/// See format at https://stackexchange.github.io/StackExchange.Redis/Configuration.html
		/// </summary>
		public string? RedisConnectionString { get; set; }

		/// <inheritdoc cref="RedisConnectionString"/>
		[Obsolete("Use RedisConnectionString instead")]
		public string? RedisConnectionConfig
		{
			get => RedisConnectionString;
			set => RedisConnectionString = value;
		}

		/// <summary>
		/// Whether to disable writes to Redis.
		/// </summary>
		public bool RedisReadOnlyMode { get; set; }

		/// <summary>
		/// Overridden settings for storage backends. Useful for running against a production server with custom backends.
		/// </summary>
		public string LogServiceWriteCacheType { get; set; } = "InMemory";

		/// <summary>
		/// Whether to log json to stdout
		/// </summary>
		public bool LogJsonToStdOut { get; set; } = false;

		/// <summary>
		/// Whether to log requests to the UpdateSession and QueryServerState RPC endpoints
		/// </summary>
		public bool LogSessionRequests { get; set; } = false;

		/// <summary>
		/// Timezone for evaluating schedules
		/// </summary>
		public string? ScheduleTimeZone { get; set; }

		/// <summary>
		/// The URl to use for generating links back to the dashboard.
		/// </summary>
		public Uri DashboardUrl { get; set; } = new Uri("https://localhost:3000");

		/// <summary>
		/// Help email address that users can contact with issues
		/// </summary>
		public string? HelpEmailAddress { get; set; }

		/// <summary>
		/// Help slack channel that users can use for issues
		/// </summary>
		public string? HelpSlackChannel { get; set; }

		/// <summary>
		/// Set the minimum size of the global thread pool
		/// This value has been found in need of tweaking to avoid timeouts with the Redis client during bursts
		/// of traffic. Default is 16 for .NET Core CLR. The correct value is dependent on the traffic the Horde Server
		/// is receiving. For Epic's internal deployment, this is set to 40.
		/// </summary>
		public int? GlobalThreadPoolMinSize { get; set; }

		/// <summary>
		/// Path to the root config file. Relative to the server.json file by default.
		/// </summary>
		public string ConfigPath { get; set; } = "globals.json";

		/// <summary>
		/// Forces configuration data to be read and updated as part of appplication startup, rather than on a schedule. Useful when running locally.
		/// </summary>
		public bool ForceConfigUpdateOnStartup { get; set; }

		/// <summary>
		/// Whether to open a browser on startup
		/// </summary>
		public bool OpenBrowser { get; set; } = false;

		/// <summary>
		/// Experimental features to enable on the server.
		/// </summary>
		public FeatureFlagSettings FeatureFlags { get; set; } = new();

		/// <summary>
		/// Options for OpenTelemetry
		/// </summary>
		public OpenTelemetrySettings OpenTelemetry { get; set; } = new OpenTelemetrySettings();

		/// <summary>
		/// Helper method to check if this process has activated the given mode
		/// </summary>
		/// <param name="mode">Run mode</param>
		/// <returns>True if mode is active</returns>
		public bool IsRunModeActive(RunMode mode)
		{
			if (RunModes == null)
			{
				return true;
			}
			return RunModes.Contains(mode);
		}

		/// <summary>
		/// Validate the settings object does not contain any invalid fields
		/// </summary>
		/// <exception cref="ArgumentException"></exception>
		public void Validate(ILogger? logger = null)
		{
			if (RunModes != null && IsRunModeActive(RunMode.None))
			{
				throw new ArgumentException($"Settings key '{nameof(RunModes)}' contains one or more invalid entries");
			}
			
			if (AuthMethod != AuthMethod.Anonymous && IsDefaultServerUrlUsed() && logger != null)
			{
				logger.LogError("Configure {Setting} in server settings when non-anonymous auth method is used", nameof(ServerUrl));
			}
		}
	}
}
