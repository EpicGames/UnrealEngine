// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.ServiceAccounts;
using HordeServer.Acls;
using HordeServer.Users;

namespace HordeServer.ServiceAccounts
{
	/// <summary>
	/// An internal Horde account representing a service
	///
	/// Service-to-service authentication always use this for authentication (for example, Robomerge accessing Horde).
	/// </summary>
	public interface IServiceAccount
	{
		/// <summary>
		/// Unique internal ID for this Horde account
		/// </summary>
		ServiceAccountId Id { get; }
		
		/// <summary>
		/// Human-readable name for identifying this account
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Description of the account (who is it for, is there an owner etc)
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Get list of claims
		/// </summary>
		/// <returns>List of claims</returns>
		IReadOnlyList<IUserClaim> Claims { get; }

		/// <summary>
		/// If the account is active
		/// </summary>
		bool Enabled { get; }

		/// <summary>
		/// Get the latest version of this account
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New account object</returns>
		Task<IServiceAccount?> RefreshAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to update settings for the account
		/// </summary>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>On success, returns the updated account object</returns>
		Task<(IServiceAccount?, string?)> TryUpdateAsync(UpdateServiceAccountOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Options for updating an account
	/// </summary>
	/// <param name="Name">If set, name to update</param>
	/// <param name="Description">If set, description to update</param>
	/// <param name="Claims">If set, claims to update</param>
	/// <param name="ResetToken">Whether to reset the secret token for this account</param>
	/// <param name="Enabled">If set, enabled flag to update</param>
	public record UpdateServiceAccountOptions(
		string? Name = null,
		string? Description = null,
		IReadOnlyList<IUserClaim>? Claims = null,
		bool? ResetToken = null,
		bool? Enabled = null);

	/// <summary>
	/// Extension methods for accounts
	/// </summary>
	public static class AccountExtensions
	{
		/// <summary>
		/// Test whether a user has a particular claim
		/// </summary>
		/// <param name="account">Account to test</param>
		/// <param name="type">Claim type to check for</param>
		/// <param name="value">Claim value to check for</param>
		/// <returns>True if the user has the claim</returns>
		public static bool HasClaim(this IServiceAccount account, string type, string value)
		{
			foreach (IUserClaim claim in account.Claims)
			{
				if (claim.Type.Equals(type, StringComparison.OrdinalIgnoreCase) && claim.Value.Equals(value, StringComparison.Ordinal))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Test whether a user has a particular claim
		/// </summary>
		/// <param name="account">Account to test</param>
		/// <param name="claim">Claim to test for</param>
		/// <returns>True if the user has the claim</returns>
		public static bool HasClaim(this IServiceAccount account, AclClaimConfig claim)
			=> HasClaim(account, claim.Type, claim.Value);

		/// <summary>
		/// Update settings for the account, retrying if the account object has changed
		/// </summary>
		/// <param name="account">The account to update</param>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>On success, returns the updated account object</returns>
		public static async Task<(IServiceAccount?, string?)> UpdateAsync(this IServiceAccount account, UpdateServiceAccountOptions options, CancellationToken cancellationToken = default)
		{
			IServiceAccount? updatedAccount = account;
			while (updatedAccount != null)
			{
				(IServiceAccount? newAccount, string? newToken) = await updatedAccount.TryUpdateAsync(options, cancellationToken);
				if (newAccount != null)
				{
					return (newAccount, newToken);
				}

				updatedAccount = await account.RefreshAsync(cancellationToken);
			}
			return (null, null);
		}
	}
}
