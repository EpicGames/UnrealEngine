// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.ServiceAccounts;
using HordeServer.Users;

namespace HordeServer.ServiceAccounts
{
	/// <summary>
	/// Interface for a collection of accounts
	/// </summary>
	public interface IServiceAccountCollection
	{
		/// <summary>
		/// Adds a new account to the collection
		/// </summary>
		/// <param name="options">Options for the new account</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<(IServiceAccount, string)> CreateAsync(CreateServiceAccountOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches service accounts
		/// </summary>
		/// <param name="index">Index of the first account</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IReadOnlyList<IServiceAccount>> FindAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get service account via secret token
		/// </summary>
		/// <param name="secretToken">Secret token to use for searching</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IServiceAccount?> FindBySecretTokenAsync(string secretToken, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get service account via ID
		/// </summary>
		/// <param name="id">The unique service account id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The service account</returns>
		Task<IServiceAccount?> GetAsync(ServiceAccountId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Delete a service account from the collection
		/// </summary>
		/// <param name="id">The service account</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(ServiceAccountId id, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Options for creating a new service account
	/// </summary>
	/// <param name="Name">Name of the account</param>
	/// <param name="Description">Description for the account</param>
	/// <param name="Claims">Optional list of claims</param>
	/// <param name="Enabled">Whether the account should be enabled</param>
	public record CreateServiceAccountOptions(string Name, string Description, IReadOnlyList<IUserClaim>? Claims = null, bool? Enabled = null);
}
