// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Security.Cryptography;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Horde.ServiceAccounts;
using HordeServer.Acls;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.ServiceAccounts
{
	/// <summary>
	/// Collection of service account documents
	/// </summary>
	public class ServiceAccountCollection : IServiceAccountCollection
	{
		record class ClaimDocument(string Type, string Value) : IUserClaim
		{
			public ClaimDocument(IUserClaim claim) : this(claim.Type, claim.Value)
			{ }

			public ClaimDocument(AclClaimConfig claim) : this(claim.Type, claim.Value)
			{ }
		}

		/// <summary>
		/// Concrete implementation of IHordeAccount
		/// </summary>
		private class ServiceAccountDocument : IServiceAccount
		{
			[BsonIgnore]
			ServiceAccountCollection? _accountCollection;

			/// <inheritdoc/>
			[BsonRequired, BsonId]
			public ServiceAccountId Id { get; set; }

			/// <inheritdoc/>
			public string Name { get; set; } = "<not-set>";
			
			public string SecretToken { get; set; } = String.Empty;

			[BsonElement("Claims2")]
			public List<ClaimDocument> Claims { get; set; } = new List<ClaimDocument>();

			/// <inheritdoc/>
			public bool Enabled { get; set; }

			/// <inheritdoc/>
			public string Description { get; set; } = "";

			[BsonIgnoreIfDefault, BsonDefaultValue(0)]
			public int UpdateIndex { get; set; }

			IReadOnlyList<IUserClaim> IServiceAccount.Claims => Claims;

			[BsonConstructor]
			private ServiceAccountDocument()
			{
			}

			public ServiceAccountDocument(ServiceAccountId id, string name, string description)
			{
				Id = id;
				Name = name;
				Description = description;
			}

			public void PostLoad(ServiceAccountCollection accountCollection)
			{
				_accountCollection = accountCollection;
			}

			public async Task<IServiceAccount?> RefreshAsync(CancellationToken cancellationToken)
				=> await _accountCollection!.GetAsync(Id, cancellationToken);

			public async Task<(IServiceAccount?, string?)> TryUpdateAsync(UpdateServiceAccountOptions options, CancellationToken cancellationToken)
				=> await _accountCollection!.TryUpdateAsync(this, options, cancellationToken);
		}

		private readonly IMongoCollection<ServiceAccountDocument> _accounts;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service</param>
		public ServiceAccountCollection(IMongoService mongoService)
		{
			_accounts = mongoService.GetCollection<ServiceAccountDocument>("ServiceAccounts", keys => keys.Ascending(x => x.SecretToken));
		}

		static string CreateToken()
			=> StringUtils.FormatHexString(RandomNumberGenerator.GetBytes(32));

		/// <inheritdoc/>
		public async Task<(IServiceAccount, string)> CreateAsync(CreateServiceAccountOptions options, CancellationToken cancellationToken = default)
		{
			string newToken = CreateToken();
			ServiceAccountDocument account = new (new ServiceAccountId(BinaryIdUtils.CreateNew()), options.Name, options.Description)
			{
				SecretToken = newToken,
				Enabled = options.Enabled ?? true,
			};

			if (options.Claims != null)
			{
				account.Claims = options.Claims.ConvertAll(x => new ClaimDocument(x));
			}

			await _accounts.InsertOneAsync(account, (InsertOneOptions?)null, cancellationToken);

			account.PostLoad(this);
			return (account, newToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IServiceAccount>> FindAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			List<IServiceAccount> accounts = new List<IServiceAccount>();
			await foreach (ServiceAccountDocument account in _accounts.Find(FilterDefinition<ServiceAccountDocument>.Empty).Range(index, count).ToAsyncEnumerable(cancellationToken))
			{
				account.PostLoad(this);
				accounts.Add(account);
			}

			return accounts;
		}

		/// <inheritdoc/>
		public async Task<IServiceAccount?> GetAsync(ServiceAccountId id, CancellationToken cancellationToken = default)
		{
			ServiceAccountDocument? account = await _accounts.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			account?.PostLoad(this);
			return account;
		}

		/// <inheritdoc/>
		public async Task<IServiceAccount?> FindBySecretTokenAsync(string secretToken, CancellationToken cancellationToken = default)
		{
			ServiceAccountDocument? account = await _accounts.Find(x => x.SecretToken == secretToken).FirstOrDefaultAsync(cancellationToken);
			account?.PostLoad(this);
			return account;
		}

		/// <inheritdoc/>
		async Task<(ServiceAccountDocument?, string?)> TryUpdateAsync(ServiceAccountDocument document, UpdateServiceAccountOptions options, CancellationToken cancellationToken = default)
		{
			string? newToken = null;
			UpdateDefinition<ServiceAccountDocument> update = Builders<ServiceAccountDocument>.Update.Set(x => x.UpdateIndex, document.UpdateIndex + 1);
			
			if (options.Name != null)
			{
				update = update.Set(x => x.Name, options.Name);
			}
			if (options.Description != null)
			{
				update = update.Set(x => x.Description, options.Description);
			}
			if (options.Claims != null)
			{
				update = update.Set(x => x.Claims, options.Claims.ConvertAll(x => new ClaimDocument(x)));
			}
			if (options.ResetToken ?? false)
			{
				newToken = CreateToken();
				update = update.Set(x => x.SecretToken, newToken);
			}
			if (options.Enabled != null)
			{
				update = update.Set(x => x.Enabled, options.Enabled);
			}

			FilterDefinition<ServiceAccountDocument> filter;
			if (document.UpdateIndex == 0)
			{
				filter = Builders<ServiceAccountDocument>.Filter.Eq(x => x.Id, document.Id) & Builders<ServiceAccountDocument>.Filter.Exists(x => x.UpdateIndex, false);
			}
			else
			{
				filter = Builders<ServiceAccountDocument>.Filter.Eq(x => x.Id, document.Id) & Builders<ServiceAccountDocument>.Filter.Eq(x => x.UpdateIndex, document.UpdateIndex);
			}

			ServiceAccountDocument? updated = await _accounts.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<ServiceAccountDocument, ServiceAccountDocument?> { ReturnDocument = ReturnDocument.After }, cancellationToken);
			if (updated == null)
			{
				return (null, null);
			}

			return (updated, newToken);
		}

		/// <inheritdoc/>
		public Task DeleteAsync(ServiceAccountId id, CancellationToken cancellationToken = default)
		{
			return _accounts.DeleteOneAsync(x => x.Id == id, cancellationToken);
		}
	}
}
