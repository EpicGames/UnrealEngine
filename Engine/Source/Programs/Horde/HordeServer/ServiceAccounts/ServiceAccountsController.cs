// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Accounts;
using EpicGames.Horde.ServiceAccounts;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.ServiceAccounts
{
	/// <summary>
	/// Controller for the /api/v1/serviceaccounts endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class ServiceAccountsController : HordeControllerBase
	{
		readonly IServiceAccountCollection _serviceAccountCollection;
		readonly GlobalConfig _globalConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public ServiceAccountsController(IServiceAccountCollection accountCollection, IOptionsSnapshot<GlobalConfig> globalConfig)
		{
			_serviceAccountCollection = accountCollection;
			_globalConfig = globalConfig.Value;
		}

		/// <summary>
		/// Create a new service account
		/// </summary>
		[HttpPost]
		[Route("/api/v1/serviceaccounts")]
		[ProducesResponseType(typeof(CreateServiceAccountResponse), 200)]
		public async Task<ActionResult<CreateServiceAccountResponse>> CreateServiceAccountAsync([FromBody] CreateServiceAccountRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.CreateAccount, User))
			{
				return Forbid(ServiceAccountAclAction.CreateAccount);
			}

			List<IUserClaim> claims = request.Claims.ConvertAll<IUserClaim>(x => new UserClaim(x.Type, x.Value));
			CreateServiceAccountOptions options = new (request.Name, request.Description, claims, request.Enabled);
			(IServiceAccount account, string secretToken) = await _serviceAccountCollection.CreateAsync(options, cancellationToken);
			return new CreateServiceAccountResponse(account.Id, secretToken);
		}

		/// <summary>
		/// Gets a list of accounts
		/// </summary>
		[HttpGet]
		[Route("/api/v1/serviceaccounts")]
		[ProducesResponseType(typeof(List<GetServiceAccountResponse>), 200)]
		public async Task<ActionResult<List<GetServiceAccountResponse>>> FindAccountsAsync([FromQuery] int? index = null, [FromQuery] int? count = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.ViewAccount, User))
			{
				return Forbid(ServiceAccountAclAction.ViewAccount);
			}

			List<GetServiceAccountResponse> responses = new List<GetServiceAccountResponse>();

			IReadOnlyList<IServiceAccount> accounts = await _serviceAccountCollection.FindAsync(index, count, cancellationToken);
			foreach (IServiceAccount account in accounts)
			{
				responses.Add(CreateGetAccountResponse(account));
			}

			return responses;
		}

		/// <summary>
		/// Gets information about an account by id
		/// </summary>
		[HttpGet]
		[Route("/api/v1/serviceaccounts/{id}")]
		[ProducesResponseType(typeof(GetServiceAccountResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<object>> GetAccountAsync(ServiceAccountId id, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.ViewAccount, User))
			{
				return Forbid(ServiceAccountAclAction.ViewAccount);
			}

			IServiceAccount? account = await _serviceAccountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			GetServiceAccountResponse response = CreateGetAccountResponse(account);
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Gets information about the current account
		/// </summary>
		[HttpGet]
		[Route("/api/v1/serviceaccounts/{id}/entitlements")]
		[ProducesResponseType(typeof(GetAccountEntitlementsResponse), 200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<object>> GetAccountEntitlementsAsync(ServiceAccountId id, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.ViewAccount, User))
			{
				return Forbid(ServiceAccountAclAction.ViewAccount);
			}

			IServiceAccount? account = await _serviceAccountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			GetAccountEntitlementsResponse response = AccountController.CreateGetAccountEntitlementsResponse(_globalConfig.Acl, claim => account.HasClaim(claim));
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Updates an account by id
		/// </summary>
		[HttpPut]
		[Route("/api/v1/serviceaccounts/{id}")]
		[ProducesResponseType(200)]
		[ProducesResponseType(404)]
		public async Task<ActionResult<UpdateServiceAccountResponse>> UpdateAccountAsync(ServiceAccountId id, UpdateServiceAccountRequest request, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.UpdateAccount, User))
			{
				return Forbid(ServiceAccountAclAction.UpdateAccount);
			}

			IServiceAccount? account = await _serviceAccountCollection.GetAsync(id, cancellationToken);
			if (account == null)
			{
				return NotFound(id);
			}

			IReadOnlyList<IUserClaim>? claims = null;
			if (request.Claims != null)
			{
				claims = request.Claims.ConvertAll(x => new UserClaim(x.Type, x.Value));
			}

			UpdateServiceAccountOptions options = new (request.Name, request.Description, claims, request.ResetToken, request.Enabled);
			(IServiceAccount? newAccount, string? newToken) = await account.UpdateAsync(options, cancellationToken);
			if (newAccount == null)
			{
				return NotFound(id);
			}

			return Ok(new UpdateServiceAccountResponse { NewSecretToken = newToken });
		}

		/// <summary>
		/// Deletes an account by id
		/// </summary>
		[HttpDelete]
		[Route("/api/v1/serviceaccounts/{id}")]
		[ProducesResponseType(200)]
		public async Task<ActionResult> DeleteAccountAsync(ServiceAccountId id, CancellationToken cancellationToken = default)
		{
			if (!_globalConfig.Authorize(ServiceAccountAclAction.DeleteAccount, User))
			{
				return Forbid(ServiceAccountAclAction.DeleteAccount);
			}

			await _serviceAccountCollection.DeleteAsync(id, cancellationToken);
			return Ok();
		}

		static GetServiceAccountResponse CreateGetAccountResponse(IServiceAccount account)
		{
			List<AccountClaimMessage> claims = new List<AccountClaimMessage>();
			foreach (IUserClaim claim in account.Claims)
			{
				claims.Add(new AccountClaimMessage(claim.Type, claim.Value));
			}
			return new GetServiceAccountResponse(account.Id, claims, account.Description, account.Enabled);
		}
	}
}
