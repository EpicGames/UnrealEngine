{
    "display_information": {
        "name": "Horde",
        "description": "Allow for interaction with the Horde build system.",
        "background_color": "#000000"
    },
    "features": {
        "bot_user": {
            "display_name": "Horde",
            "always_online": false
        }
    },
    "oauth_config": {
        "scopes": {
            "user": [
                "admin.conversations:write"
            ],
            "bot": [
                "chat:write",
                "chat:write.public",
                "reactions:read",
                "reactions:write",
                "users.profile:read",
                "users:read",
                "users:read.email",
                "channels:manage"
            ]
        }
    },
    "settings": {
        "interactivity": {
            "is_enabled": true
        },
        "org_deploy_enabled": true,
        "socket_mode_enabled": true,
        "token_rotation_enabled": false
    }
}
