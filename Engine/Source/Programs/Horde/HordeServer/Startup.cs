// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net.Mime;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text.Json;
using System.Text.Json.Nodes;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.AspNet;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Server;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using EpicGames.Redis;
using Grpc.Core;
using Grpc.Core.Interceptors;
using HordeCommon;
using HordeServer.Accounts;
using HordeServer.Acls;
using HordeServer.Auditing;
using HordeServer.Authentication;
using HordeServer.Configuration;
using HordeServer.Dashboard;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Server.Notices;
using HordeServer.ServiceAccounts;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authentication;
using Microsoft.AspNetCore.Authentication.Cookies;
using Microsoft.AspNetCore.Authentication.JwtBearer;
using Microsoft.AspNetCore.Authentication.OpenIdConnect;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Builder;
using Microsoft.AspNetCore.DataProtection;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.Hosting.Server.Features;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Features;
using Microsoft.AspNetCore.HttpOverrides;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.Mvc.Controllers;
using Microsoft.AspNetCore.Mvc.Filters;
using Microsoft.AspNetCore.Mvc.ModelBinding;
using Microsoft.AspNetCore.ResponseCompression;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;
using Microsoft.Extensions.Primitives;
using Microsoft.OpenApi.Models;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Conventions;
using MongoDB.Bson.Serialization.Serializers;
using Serilog;
using Serilog.Events;
using StackExchange.Redis;
using Swashbuckle.AspNetCore.SwaggerGen;
using Status = Grpc.Core.Status;
using StatusCode = Grpc.Core.StatusCode;

#pragma warning disable CA1505 // 'ConfigureServices' has a maintainability index of '9'. Rewrite or refactor the code to increase its maintainability index (MI) above '9'.

namespace HordeServer
{
	using ContentHash = EpicGames.Core.ContentHash;
	using ILogger = Microsoft.Extensions.Logging.ILogger;
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	class Startup : IServerStartup
	{
		static Startup()
		{
			ProtoBuf.Meta.RuntimeTypeModel.Default[typeof(ProjectId)].SetSurrogate(typeof(StringIdProto<ProjectId, ProjectIdConverter>));
			ProtoBuf.Meta.RuntimeTypeModel.Default[typeof(StreamId)].SetSurrogate(typeof(StringIdProto<StreamId, StreamIdConverter>));
		}

		class GrpcExceptionInterceptor : Interceptor
		{
			readonly ILogger<GrpcExceptionInterceptor> _logger;

			public GrpcExceptionInterceptor(ILogger<GrpcExceptionInterceptor> logger)
			{
				_logger = logger;
			}

			public override Task<TResponse> UnaryServerHandler<TRequest, TResponse>(TRequest request, ServerCallContext context, UnaryServerMethod<TRequest, TResponse> continuation)
			{
				return GuardAsync(context, () => base.UnaryServerHandler(request, context, continuation));
			}

			public override Task<TResponse> ClientStreamingServerHandler<TRequest, TResponse>(IAsyncStreamReader<TRequest> requestStream, ServerCallContext context, ClientStreamingServerMethod<TRequest, TResponse> continuation) where TRequest : class where TResponse : class
			{
				return GuardAsync(context, () => base.ClientStreamingServerHandler(requestStream, context, continuation));
			}

			public override Task ServerStreamingServerHandler<TRequest, TResponse>(TRequest request, IServerStreamWriter<TResponse> responseStream, ServerCallContext context, ServerStreamingServerMethod<TRequest, TResponse> continuation) where TRequest : class where TResponse : class
			{
				return GuardAsync(context, () => base.ServerStreamingServerHandler(request, responseStream, context, continuation));
			}

			public override Task DuplexStreamingServerHandler<TRequest, TResponse>(IAsyncStreamReader<TRequest> requestStream, IServerStreamWriter<TResponse> responseStream, ServerCallContext context, DuplexStreamingServerMethod<TRequest, TResponse> continuation) where TRequest : class where TResponse : class
			{
				return GuardAsync(context, () => base.DuplexStreamingServerHandler(requestStream, responseStream, context, continuation));
			}

			async Task<T> GuardAsync<T>(ServerCallContext context, Func<Task<T>> callFunc) where T : class
			{
				T result = null!;
				await GuardAsync(context, async () => { result = await callFunc(); });
				return result;
			}

			async Task GuardAsync(ServerCallContext context, Func<Task> callFunc)
			{
				HttpContext httpContext = context.GetHttpContext();

				AgentId? agentId = AclService.GetAgentId(httpContext.User);
				if (agentId != null)
				{
					using IDisposable? scope = _logger.BeginScope("Agent: {AgentId}, RemoteIP: {RemoteIP}, Method: {Method}", agentId.Value, httpContext.Connection.RemoteIpAddress, context.Method);
					await GuardInnerAsync(context, callFunc);
				}
				else
				{
					using IDisposable? scope = _logger.BeginScope("RemoteIP: {RemoteIP}, Method: {Method}", httpContext.Connection.RemoteIpAddress, context.Method);
					await GuardInnerAsync(context, callFunc);
				}
			}

			async Task GuardInnerAsync(ServerCallContext context, Func<Task> callFunc)
			{
				try
				{
					await callFunc();
				}
				catch (OperationCanceledException ex)
				{
					_logger.LogDebug(ex, "Rpc call to {Method} was cancelled", context.Method);
					throw;
				}
				catch (StructuredRpcException ex)
				{
#pragma warning disable CA2254 // Template should be a static expression
					_logger.LogError(ex, ex.Format, ex.Args);
#pragma warning restore CA2254 // Template should be a static expression
					throw;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception in call to {Method}", context.Method);
					throw new RpcException(new Status(StatusCode.Internal, $"An exception was thrown on the server: {ex}"));
				}
			}
		}

		class BsonSerializationProvider : IBsonSerializationProvider
		{
			public IBsonSerializer? GetSerializer(Type type)
			{
				if (type == typeof(ContentHash))
				{
					return new ContentHashSerializer();
				}
				if (type == typeof(DateTimeOffset))
				{
					return new DateTimeOffsetStringSerializer();
				}
				return null;
			}
		}

		class ObsoleteLoggingFilter : IActionFilter
		{
			public void OnActionExecuted(ActionExecutedContext context)
			{
			}

			public void OnActionExecuting(ActionExecutingContext context)
			{
				ControllerActionDescriptor? actionDescriptor = context.ActionDescriptor as ControllerActionDescriptor;
				if (actionDescriptor?.MethodInfo.GetCustomAttribute<ObsoleteAttribute>() != null
					|| actionDescriptor?.MethodInfo?.DeclaringType?.GetCustomAttribute<ObsoleteAttribute>() != null)
				{
					ILogger? logger = context.HttpContext.RequestServices.GetService<ILogger<ObsoleteLoggingFilter>>();
					logger?.LogDebug("Using obsolete endpoint: {RequestPath} (Source: {RemoteIp})", context.HttpContext.Request.Path, context.HttpContext.Connection.RemoteIpAddress);
				}
			}
		}

		public Startup(IConfiguration configuration)
		{
			Configuration = configuration;
		}

		public IConfiguration Configuration { get; }

		public static void BindServerSettings(IConfiguration configuration, ServerSettings settings, ILogger? logger = null)
		{
			IConfigurationSection hordeSection = configuration.GetSection("Horde");
			hordeSection.Bind(settings);
			settings.Validate(logger);
			settings.AddDefaultOidcScopesAndMappings();
		}

		/// <summary>
		/// Converter to/from Redis values
		/// </summary>
		sealed class AgentIdRedisConverter : IRedisConverter<AgentId>
		{
			/// <inheritdoc/>
			public AgentId FromRedisValue(RedisValue value) => new AgentId((string)value!);

			/// <inheritdoc/>
			public RedisValue ToRedisValue(AgentId value) => value.ToString();
		}

		/// <summary>
		/// Configure custom types for the public API documentation
		/// </summary>
		class SwaggerSchemaFilter : ISchemaFilter
		{
			public void Apply(OpenApiSchema schema, SchemaFilterContext context)
			{
				if (context.Type == typeof(AgentId))
				{
				}
				if (context.Type == typeof(Utf8String) || context.Type.GetCustomAttribute<JsonSchemaStringAttribute>() != null)
				{
					schema.Type = "string";
					schema.Properties.Clear();
				}
			}
		}

		// This method gets called *multiple times* by the runtime. Use this method to add services to the container.
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<MemoryMappedFileCache>();
			services.AddSingleton<JsonSchemaCache>();

			// IOptionsMonitor pattern for live updating of configuration settings
			services.Configure<ServerSettings>(x => BindServerSettings(Configuration, x));

			// We may upload a large number of references with posts to storage endpoints. Increase the max number of form values to allow this (the default is 1024).
			services.Configure<FormOptions>(options =>
			{
				options.ValueCountLimit = 100_000;
			});

			// Bind the settings again for local variable access in this method
			ServerSettings settings = new();
			BindServerSettings(Configuration, settings);

			ServerInfo serverInfo = new ServerInfo(Configuration, Options.Create(settings));
			services.AddSingleton<IServerInfo>(serverInfo);

			// Register the plugin collection
			IPluginCollection pluginCollection = ServerApp.Plugins;
			services.AddSingleton<IPluginCollection>(pluginCollection);

			// Register all the plugin services
			IConfigurationSection pluginsConfig = Configuration.GetSection("Horde").GetSection("Plugins");
			foreach (ILoadedPlugin plugin in pluginCollection.LoadedPlugins)
			{
				IConfigurationSection pluginConfig = pluginsConfig.GetSection(plugin.Name.ToString());
				plugin.ConfigureServices(pluginConfig, serverInfo, services);
			}

			OpenTelemetryHelper.Configure(services, settings.OpenTelemetry);

			if (settings.GlobalThreadPoolMinSize != null)
			{
				// Min thread pool size is set to combat timeouts seen with the Redis client.
				// See comments for <see cref="ServerSettings.GlobalThreadPoolMinSize" /> and
				// https://github.com/StackExchange/StackExchange.Redis/issues/1680
				int min = settings.GlobalThreadPoolMinSize.Value;
				ThreadPool.SetMinThreads(min, min);
			}

			RedisSerializer.RegisterConverter<AgentId, AgentIdRedisConverter>();

#pragma warning disable CA2000 // Dispose objects before losing scope
			RedisService redisService;
			using (Serilog.Extensions.Logging.SerilogLoggerFactory loggerFactory = new Serilog.Extensions.Logging.SerilogLoggerFactory(Serilog.Log.Logger))
			{
				redisService = new RedisService(Options.Create(settings), loggerFactory.CreateLogger<RedisService>());
			}
#pragma warning restore CA2000 // Dispose objects before losing scope
			services.AddSingleton<RedisService>(sp => redisService);
			services.AddSingleton<IRedisService>(sp => sp.GetRequiredService<RedisService>());
			services.AddDataProtection().PersistKeysToStackExchangeRedis(() => redisService.GetDatabase(), "aspnet-data-protection");

			services.AddResponseCompression(options =>
			{
				options.EnableForHttps = true;
				options.Providers.Add<GzipCompressionProvider>();
				options.Providers.Add<BrotliCompressionProvider>();
				options.Providers.Add<ZstdCompressionProvider>();
			});

			if (settings.CorsEnabled)
			{
				services.AddCors(options =>
				{
					options.AddPolicy("CorsPolicy",
						builder => builder.WithOrigins(settings.CorsOrigin.Split(";"))
						.AllowAnyMethod()
						.AllowAnyHeader()
						.AllowCredentials());
				});
			}

			services.AddGrpc(options =>
			{
				options.EnableDetailedErrors = true;
				options.MaxReceiveMessageSize = 200 * 1024 * 1024; // 100 MB (packaged builds of Horde agent can be large) 
				options.Interceptors.Add(typeof(GrpcExceptionInterceptor));
			});
			services.AddGrpcReflection();

			services.AddSingleton<IAccountCollection, AccountCollection>();
			services.AddSingleton<IServiceAccountCollection, ServiceAccountCollection>();
			services.AddSingleton<IUserCollection, UserCollectionV2>();
			services.AddSingleton<INoticeCollection, NoticeCollection>();
			services.AddSingleton<IDashboardPreviewCollection, DashboardPreviewCollection>();

			services.AddSingleton<IConfigSource, InMemoryConfigSource>();
			services.AddSingleton<IConfigSource, FileConfigSource>();

			services.AddSingleton<ConfigService>();
			services.AddSingleton<IConfigService>(sp => sp.GetRequiredService<ConfigService>());
			services.AddSingleton<IOptionsFactory<GlobalConfig>>(sp => sp.GetRequiredService<ConfigService>());
			services.AddSingleton<IOptionsChangeTokenSource<GlobalConfig>>(sp => sp.GetRequiredService<ConfigService>());

			// Always run the hosted config service, regardless of the server role, so we receive updates on the latest config values.
			services.AddHostedService(provider => provider.GetRequiredService<ConfigService>());

			// Auditing
			services.AddSingleton(typeof(IAuditLogFactory<>), typeof(AuditLogFactory<>));
			services.AddSingleton(typeof(ISingletonDocument<>), typeof(SingletonDocument<>));

			services.AddSingleton<IAclService, AclService>();
			services.AddSingleton<RequestTrackerService>();
			services.AddSingleton<MongoCommandTracer>();
			services.AddSingleton<MongoService>();
			services.AddSingleton<IMongoService>(sp => sp.GetRequiredService<MongoService>());
			services.AddSingleton<MongoMigrator>();
			services.AddSingleton<GlobalsService>();
			services.AddSingleton<IClock, Clock>();
			services.AddSingleton<IDowntimeService, DowntimeService>();
			services.AddSingleton<LifetimeService>();
			services.AddSingleton<ILifetimeService>(sp => sp.GetRequiredService<LifetimeService>());
			services.AddHostedService(provider => provider.GetRequiredService<LifetimeService>());
			services.AddSingleton(typeof(IHealthMonitor<>), typeof(HealthMonitor<>));
			services.AddSingleton<ServerStatusService>();
			services.AddHostedService(provider => provider.GetRequiredService<ServerStatusService>());

			services.AddScoped<OAuthControllerFilter>();

			services.AddSingleton<NoticeService>();

			AuthenticationBuilder authBuilder = services.AddAuthentication(options =>
				{
					switch (settings.AuthMethod)
					{
						case AuthMethod.Anonymous:
							options.DefaultAuthenticateScheme = AnonymousAuthHandler.AuthenticationScheme;
							options.DefaultSignInScheme = AnonymousAuthHandler.AuthenticationScheme;
							options.DefaultChallengeScheme = AnonymousAuthHandler.AuthenticationScheme;
							break;

						case AuthMethod.Okta:
							// If an authentication cookie is present, use it to get authentication information
							options.DefaultAuthenticateScheme = CookieAuthenticationDefaults.AuthenticationScheme;
							options.DefaultSignInScheme = CookieAuthenticationDefaults.AuthenticationScheme;

							// If authentication is required, and no cookie is present, use OIDC to sign in
							options.DefaultChallengeScheme = OktaAuthHandler.AuthenticationScheme;
							break;

						case AuthMethod.OpenIdConnect:
							// If an authentication cookie is present, use it to get authentication information
							options.DefaultAuthenticateScheme = CookieAuthenticationDefaults.AuthenticationScheme;
							options.DefaultSignInScheme = CookieAuthenticationDefaults.AuthenticationScheme;

							// If authentication is required, and no cookie is present, use OIDC to sign in
							options.DefaultChallengeScheme = OpenIdConnectDefaults.AuthenticationScheme;
							break;

						case AuthMethod.Horde:
							options.DefaultAuthenticateScheme = CookieAuthenticationDefaults.AuthenticationScheme;
							options.DefaultSignInScheme = CookieAuthenticationDefaults.AuthenticationScheme;
							options.DefaultChallengeScheme = CookieAuthenticationDefaults.AuthenticationScheme;
							break;

						default:
							throw new ArgumentException($"Invalid auth method {settings.AuthMethod}");
					}
				});

			List<string> schemes = new List<string>();

			authBuilder.AddCookie(options =>
			{
				if (settings.AuthMethod == AuthMethod.Horde)
				{
					options.Cookie.Name = CookieAuthenticationDefaults.AuthenticationScheme;
					options.LoginPath = "/account/login/horde";
					options.LogoutPath = "/";

					// .NET 10 breaking change: cookie auth no longer redirects to login page for [ApiController] endpoints by default.
					// AccountController.Login() is a web navigation endpoint that must redirect to the login form, so restore the redirect behavior.
					options.Events.OnRedirectToLogin = context =>
					{
						context.Response.Redirect(context.RedirectUri);
						return Task.CompletedTask;
					};
				}

				options.Events.OnValidatePrincipal = context =>
				{
					if (!String.Equals(context.Principal?.FindFirst(HordeClaimTypes.Version)?.Value, HordeClaimTypes.CurrentVersion, StringComparison.Ordinal))
					{
						context.RejectPrincipal();
					}
					if (context.Principal?.FindFirst(HordeClaimTypes.UserId) == null)
					{
						context.RejectPrincipal();
					}
					return Task.CompletedTask;
				};

				options.Events.OnRedirectToAccessDenied = context =>
				{
					context.Response.StatusCode = StatusCodes.Status403Forbidden;
					return context.Response.CompleteAsync();
				};
			});
			schemes.Add(CookieAuthenticationDefaults.AuthenticationScheme);

			authBuilder.AddServiceAccounts(options => { });
			schemes.Add(ServiceAccountAuthHandler.AuthenticationScheme);
			
			ValidateOidcSettings(settings);
			switch (settings.AuthMethod)
			{
				case AuthMethod.Anonymous:
					authBuilder.AddAnonymous(options => { });
					schemes.Add(AnonymousAuthHandler.AuthenticationScheme);
					break;

				case AuthMethod.Okta:
					authBuilder.AddOkta(settings, OktaAuthHandler.AuthenticationScheme, OpenIdConnectDefaults.DisplayName, options =>
						{
							options.Authority = settings.OidcAuthority;
							options.ClientId = settings.OidcClientId;
							options.Scope.Remove("groups");

							if (!String.IsNullOrEmpty(settings.OidcSigninRedirect))
							{
								options.Events = new OpenIdConnectEvents
								{
									OnRedirectToIdentityProvider = async redirectContext =>
									{
										redirectContext.ProtocolMessage.RedirectUri = settings.OidcSigninRedirect;
										await Task.CompletedTask;
									}
								};
							}
						});
					schemes.Add(OktaAuthHandler.AuthenticationScheme);
					break;

				case AuthMethod.OpenIdConnect:
					authBuilder.AddHordeOpenId(settings, OpenIdConnectDefaults.AuthenticationScheme, OpenIdConnectDefaults.DisplayName, options =>
						{
							options.Authority = settings.OidcAuthority;
							options.ClientId = settings.OidcClientId;
							options.ClientSecret = settings.OidcClientSecret;
							foreach (string scope in settings.OidcRequestedScopes)
							{
								options.Scope.Add(scope);
							}

							if (!String.IsNullOrEmpty(settings.OidcSigninRedirect))
							{
								options.Events = new OpenIdConnectEvents
								{
									OnRedirectToIdentityProvider = async redirectContext =>
									{
										redirectContext.ProtocolMessage.RedirectUri = settings.OidcSigninRedirect;
										await Task.CompletedTask;
									}
								};
							}
						});
					schemes.Add(OpenIdConnectDefaults.AuthenticationScheme);
					break;

				case AuthMethod.Horde:
					authBuilder.AddHordeOpenId(settings, OpenIdConnectDefaults.AuthenticationScheme, OpenIdConnectDefaults.DisplayName, options =>
					{
						options.Authority = new Uri(settings.ServerUrl, "api/v1/oauth2").ToString();
						options.ClientId = "default";
						if (settings.HttpsPort == 0)
						{
							options.RequireHttpsMetadata = false;
						}
						foreach (string scope in settings.OidcRequestedScopes)
						{
							options.Scope.Add(scope);
						}
					});
					schemes.Add(OpenIdConnectDefaults.AuthenticationScheme);
					break;

				default:
					throw new ArgumentException($"Invalid auth method {settings.AuthMethod}");
			}

			authBuilder.AddScheme<JwtBearerOptions, JwtAuthHandler>(JwtAuthHandler.AuthenticationScheme, options => { });
			schemes.Add(JwtAuthHandler.AuthenticationScheme);

			if (!String.IsNullOrEmpty(settings.OidcAuthority) && !String.IsNullOrEmpty(settings.OidcAudience))
			{
				ExternalJwtAuthHandler hordeJwtBearer = new(settings);
				hordeJwtBearer.AddHordeJwtBearerConfiguration(authBuilder);
				schemes.Add(ExternalJwtAuthHandler.AuthenticationScheme);
			}

			services.AddAuthorization(options =>
				{
					options.DefaultPolicy = new AuthorizationPolicyBuilder(schemes.ToArray())
						.RequireAuthenticatedUser()
						.Build();
				});

			// Hosted service that needs to run no matter the run mode of the process (server vs worker)
			services.AddHostedService(provider => (DowntimeService)provider.GetRequiredService<IDowntimeService>());

			if (settings.IsRunModeActive(RunMode.Worker) && !settings.MongoReadOnlyMode)
			{
				services.AddHostedService<MetricService>();
			}

			// Allow longer to shutdown so we can debug missing cancellation tokens
			services.Configure<HostOptions>(options =>
			{
				options.ShutdownTimeout = TimeSpan.FromSeconds(30.0);
			});

			// Allow forwarded headers
			services.Configure<ForwardedHeadersOptions>(options =>
			{
				options.ForwardedHeaders = ForwardedHeaders.XForwardedFor | ForwardedHeaders.XForwardedProto;
				options.KnownProxies.Clear();
				options.KnownIPNetworks.Clear();
			});

			IMvcBuilder mvcBuilder = services.AddMvc()
				.AddJsonOptions(options => JsonUtils.ConfigureJsonSerializer(options.JsonSerializerOptions));

			foreach (Assembly pluginAssembly in pluginCollection.LoadedPlugins.Select(x => x.Assembly).Distinct())
			{
				mvcBuilder.AddApplicationPart(pluginAssembly);
			}

			services.AddControllersWithViews(options => options.Filters.Add(new ObsoleteLoggingFilter()))
				.AddRazorRuntimeCompilation();

			services.AddControllers(options =>
			{
				options.InputFormatters.Add(new CbInputFormatter());
				options.OutputFormatters.Add(new CbOutputFormatter());
				options.OutputFormatters.Insert(0, new CbPreferredOutputFormatter());
				options.OutputFormatters.Add(new RawOutputFormatter(NullLogger.Instance));
				options.FormatterMappings.SetMediaTypeMappingForFormat("raw", MediaTypeNames.Application.Octet);
				options.FormatterMappings.SetMediaTypeMappingForFormat("uecb", CustomMediaTypeNames.UnrealCompactBinary);
				options.FormatterMappings.SetMediaTypeMappingForFormat("uecbpkg", CustomMediaTypeNames.UnrealCompactBinaryPackage);
			}).ConfigureApiBehaviorOptions(options =>
			{
				options.InvalidModelStateResponseFactory = context =>
				{
					BadRequestObjectResult result = new BadRequestObjectResult(context.ModelState);
					// always return errors as json objects
					// we could allow more types here, but we do not want raw for instance
					result.ContentTypes.Add(MediaTypeNames.Application.Json);

					return result;
				};
			});

			services.AddSwaggerGen(config =>
			{
				config.SchemaGeneratorOptions.SchemaFilters.Add(new SwaggerSchemaFilter());
				config.SwaggerDoc("v1", new OpenApiInfo { Title = "Horde Server API", Version = "v1" });

				string xmlDocFile = Path.Combine(AppContext.BaseDirectory, $"{Assembly.GetExecutingAssembly().GetName().Name}.xml");
				if (File.Exists(xmlDocFile))
				{
					config.IncludeXmlComments(xmlDocFile);
				}
			});

			services.Configure<ApiBehaviorOptions>(options =>
			{
				options.InvalidModelStateResponseFactory = context =>
				{
					foreach (KeyValuePair<string, ModelStateEntry> pair in context.ModelState)
					{
						ModelError? error = pair.Value.Errors.FirstOrDefault();
						if (error != null)
						{
							string message = error.ErrorMessage;
							if (String.IsNullOrEmpty(message))
							{
								message = error.Exception?.Message ?? "Invalid error object";
							}
							return new BadRequestObjectResult(EpicGames.Core.LogEvent.Create(LogLevel.Error, KnownLogEvents.None, error.Exception, "Invalid value for {Name}: {Message}", pair.Key, message));
						}
					}
					return new BadRequestObjectResult(context.ModelState);
				};
			});

			DirectoryReference dashboardDir = DirectoryReference.Combine(ServerApp.AppDir, "DashboardApp");
			if (DirectoryReference.Exists(dashboardDir))
			{
				services.AddSpaStaticFiles(config => { config.RootPath = "DashboardApp"; });
			}

			ConfigureMongoDbClient();
			ConfigureFormatters();

			OnAddHealthChecks(services);
		}
		
		private static void ValidateOidcSettings(ServerSettings settings)
		{
			if (settings.AuthMethod is AuthMethod.OpenIdConnect or AuthMethod.Okta)
			{
				if (settings.OidcAuthority == null)
				{
					throw new ArgumentException($"Key '{nameof(ServerSettings.OidcAuthority)}' in server settings must be set when auth mode {settings.AuthMethod} is used");
				}
				
				if (settings.OidcAudience == null)
				{
					throw new ArgumentException($"Key '{nameof(ServerSettings.OidcAudience)}' in server settings must be set when auth mode {settings.AuthMethod} is used");
				}
				
				if (settings.OidcClientId == null)
				{
					throw new ArgumentException($"Key '{nameof(ServerSettings.OidcClientId)}' in server settings must be set when auth mode {settings.AuthMethod} is used");
				}
			}
		}
		
		public static void ConfigureFormatters()
		{
			LogValueFormatter.RegisterTypeAnnotation<AgentId>("AgentId");
			LogValueFormatter.RegisterTypeAnnotation<JobId>("JobId");
			LogValueFormatter.RegisterTypeAnnotation<LeaseId>("LeaseId");
			LogValueFormatter.RegisterTypeAnnotation<LogId>("LogId");
			LogValueFormatter.RegisterTypeAnnotation<UserId>("UserId");
		}

		public sealed class CommitIdBsonSerializer : SerializerBase<CommitId>
		{
			/// <inheritdoc/>
			public override CommitId Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				if (context.Reader.GetCurrentBsonType() == BsonType.Int32)
				{
					return CommitId.FromPerforceChange(context.Reader.ReadInt32());
				}
				else
				{
					return new CommitId(context.Reader.ReadString());
				}
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, CommitId value)
			{
				context.Writer.WriteString(value.Name);
			}
		}

		public sealed class BlobLocatorBsonSerializer : SerializerBase<BlobLocator>
		{
			/// <inheritdoc/>
			public override BlobLocator Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return new BlobLocator(context.Reader.ReadString());
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, BlobLocator value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		public sealed class RefNameBsonSerializer : SerializerBase<RefName>
		{
			/// <inheritdoc/>
			public override RefName Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return new RefName(context.Reader.ReadString());
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, RefName value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		public sealed class IoHashBsonSerializer : SerializerBase<IoHash>
		{
			/// <inheritdoc/>
			public override IoHash Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return IoHash.Parse(context.Reader.ReadString());
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, IoHash value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		public sealed class NamespaceIdBsonSerializer : SerializerBase<NamespaceId>
		{
			/// <inheritdoc/>
			public override NamespaceId Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return new NamespaceId(context.Reader.ReadString());
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, NamespaceId value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		public sealed class AgentIdBsonSerializer : SerializerBase<AgentId>
		{
			/// <inheritdoc/>
			public override AgentId Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				string argument;
				if (context.Reader.CurrentBsonType == MongoDB.Bson.BsonType.ObjectId)
				{
					argument = context.Reader.ReadObjectId().ToString();
				}
				else
				{
					argument = context.Reader.ReadString();
				}
				return new AgentId(argument);
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, AgentId value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		public sealed class AclActionBsonSerializer : SerializerBase<AclAction>
		{
			/// <inheritdoc/>
			public override AclAction Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
				=> new AclAction(context.Reader.ReadString());

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, AclAction value)
				=> context.Writer.WriteString(value.Name);
		}

		public sealed class AclScopeNameBsonSerializer : SerializerBase<AclScopeName>
		{
			/// <inheritdoc/>
			public override AclScopeName Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
				=> new AclScopeName(context.Reader.ReadString());

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, AclScopeName value)
				=> context.Writer.WriteString(value.Text ?? String.Empty);
		}

		sealed class SubResourceIdBsonSerializer<TValue, TConverter> : SerializerBase<TValue> where TValue : struct where TConverter : SubResourceIdConverter<TValue>, new()
		{
			readonly TConverter _converter = new TConverter();

			/// <inheritdoc/>
			public override TValue Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
				=> _converter.FromSubResourceId(new SubResourceId((ushort)context.Reader.ReadInt32()));

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, TValue value)
				=> context.Writer.WriteInt32(_converter.ToSubResourceId(value).Value);
		}

		/// <summary>
		/// Serializer for subresource ids
		/// </summary>
		public class SubResourceIdSerializer : IBsonSerializer<SubResourceId>
		{
			/// <inheritdoc/>
			public Type ValueType => typeof(SubResourceId);

			/// <inheritdoc/>
			public object Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return new SubResourceId((ushort)context.Reader.ReadInt32());
			}

			/// <inheritdoc/>
			public void Serialize(BsonSerializationContext context, BsonSerializationArgs args, object value)
			{
				SubResourceId id = (SubResourceId)value;
				context.Writer.WriteInt32((int)id.Value);
			}

			/// <inheritdoc/>
			public void Serialize(BsonSerializationContext context, BsonSerializationArgs args, SubResourceId id)
			{
				context.Writer.WriteInt32((int)id.Value);
			}

			/// <inheritdoc/>
			SubResourceId IBsonSerializer<SubResourceId>.Deserialize(BsonDeserializationContext context, BsonDeserializationArgs vrgs)
			{
				return new SubResourceId((ushort)context.Reader.ReadInt32());
			}
		}

		sealed class SubResourceIdBsonSerializationProvider : BsonSerializationProviderBase
		{
			/// <inheritdoc/>
			public override IBsonSerializer? GetSerializer(Type type, IBsonSerializerRegistry serializerRegistry)
			{
				SubResourceIdConverterAttribute? attribute = type.GetCustomAttribute<SubResourceIdConverterAttribute>();
				if (attribute == null)
				{
					return null;
				}
				return (IBsonSerializer?)Activator.CreateInstance(typeof(SubResourceIdBsonSerializer<,>).MakeGenericType(type, attribute.ConverterType));
			}
		}

		sealed class CommitTagBsonSerializer : SerializerBase<CommitTag>
		{
			/// <inheritdoc/>
			public override CommitTag Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				return new CommitTag(context.Reader.ReadString());
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, CommitTag value)
			{
				context.Writer.WriteString(value.ToString());
			}
		}

		sealed class JsonObjectBsonSerializer : SerializerBase<JsonObject>
		{
			/// <inheritdoc/>
			public override JsonObject Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
			{
				string str = context.Reader.ReadString();
				if (String.IsNullOrWhiteSpace(str))
				{
					return new JsonObject();
				}
				else
				{
					return (JsonObject)JsonObject.Parse(str, new JsonNodeOptions { PropertyNameCaseInsensitive = true }, new JsonDocumentOptions { AllowTrailingCommas = true, CommentHandling = JsonCommentHandling.Skip })!;
				}
			}

			/// <inheritdoc/>
			public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, JsonObject value)
				=> context.Writer.WriteString(value.ToJsonString());
		}

		static int s_haveConfiguredMongoDb = 0;

		public static void ConfigureMongoDbClient()
		{
			if (Interlocked.CompareExchange(ref s_haveConfiguredMongoDb, 1, 0) == 0)
			{
				// Ignore extra elements on deserialized documents
				ConventionPack conventionPack = new ConventionPack();
				conventionPack.Add(new IgnoreExtraElementsConvention(true));
				conventionPack.Add(new EnumRepresentationConvention(BsonType.String));
				ConventionRegistry.Register("Horde", conventionPack, type => true);

				// Register the custom serializers
				BsonSerializer.RegisterSerializer(new CommitIdBsonSerializer());
				BsonSerializer.RegisterSerializer(new BlobLocatorBsonSerializer());
				BsonSerializer.RegisterSerializer(new RefNameBsonSerializer());
				BsonSerializer.RegisterSerializer(new IoHashBsonSerializer());
				BsonSerializer.RegisterSerializer(new NamespaceIdBsonSerializer());
				BsonSerializer.RegisterSerializer(new AgentIdBsonSerializer());
				BsonSerializer.RegisterSerializer(new AclActionBsonSerializer());
				BsonSerializer.RegisterSerializer(new AclScopeNameBsonSerializer());
				BsonSerializer.RegisterSerializer(new ConditionSerializer());
				BsonSerializer.RegisterSerializer(new SubResourceIdSerializer());
				BsonSerializer.RegisterSerializer(new CommitTagBsonSerializer());
				BsonSerializer.RegisterSerializer(new JsonObjectBsonSerializer());
				BsonSerializer.RegisterSerializationProvider(new BsonSerializationProvider());
				BsonSerializer.RegisterSerializationProvider(new StringIdBsonSerializationProvider());
				BsonSerializer.RegisterSerializationProvider(new BinaryIdBsonSerializationProvider());
				BsonSerializer.RegisterSerializationProvider(new ObjectIdBsonSerializationProvider());
				BsonSerializer.RegisterSerializationProvider(new SubResourceIdBsonSerializationProvider());
			}
		}

		private static void OnAddHealthChecks(IServiceCollection services)
		{
			services.AddHealthChecks()
				.AddCheck("self", () => HealthCheckResult.Healthy(), tags: new[] { "self" })
				.AddCheck<RedisService>("redis")
				.AddCheck<MongoService>("mongo");
		}

		// This method gets called by the runtime. Use this method to configure the HTTP request pipeline.
		public static void Configure(IApplicationBuilder app, IWebHostEnvironment env, Microsoft.Extensions.Hosting.IHostApplicationLifetime lifetime, IOptions<ServerSettings> settings)
		{
			app.UseForwardedHeaders();
			app.UseResponseCompression();

			// Used for allowing auth cookies in combination with OpenID Connect auth (for example, Google Auth did not work with these unset)
			app.UseCookiePolicy(new CookiePolicyOptions()
			{
				MinimumSameSitePolicy = SameSiteMode.None,
				CheckConsentNeeded = _ => true
			});

			if (settings.Value.CorsEnabled)
			{
				app.UseCors("CorsPolicy");
			}

			// Enable middleware to serve generated Swagger as a JSON endpoint.
			app.UseSwagger();

			// Enable serilog request logging
			app.UseSerilogRequestLogging(options =>
			{
				options.GetLevel = GetRequestLoggingLevel;
				options.EnrichDiagnosticContext = (diagnosticContext, httpContext) =>
				{
					diagnosticContext.Set("RemoteIP", httpContext?.Connection?.RemoteIpAddress);

					// Header sent by the dashboard to indicate how long a user has been inactive for a particular browser page (in seconds)
					if (httpContext?.Request.Headers.TryGetValue("X-Horde-LastUserActivity", out StringValues values) is true)
					{
						string? value = values.FirstOrDefault();
						if (!String.IsNullOrEmpty(value) && Int32.TryParse(value, out int lastUserActivity))
						{
							diagnosticContext.Set("LastUserActivity", lastUserActivity);
						}
					}
				};
			});

			// Enable middleware to serve swagger-ui (HTML, JS, CSS, etc.),
			// specifying the Swagger JSON endpoint.
			app.UseSwaggerUI(c =>
			{
				c.SwaggerEndpoint("/swagger/v1/swagger.json", "Horde Server API");
				c.RoutePrefix = "swagger";
			});

			if (!env.IsDevelopment())
			{
				app.UseMiddleware<RequestTrackerMiddleware>();
			}
			
			app.UseReverseProxyDetection();
			app.UseExceptionHandler("/api/v1/exception");
			app.UseMiddleware<ExceptionLoggingMiddleware>();

			DirectoryReference dashboardDir = DirectoryReference.Combine(ServerApp.AppDir, "DashboardApp");
			if (DirectoryReference.Exists(dashboardDir))
			{
				app.UseDefaultFiles();
				app.UseStaticFiles();
				app.UseWhen(IsSpaRequest, builder => builder.UseSpaStaticFiles());
			}

			app.UseRouting();

			app.UseAuthentication();
			app.UseAuthorization();

			app.UseEndpoints(endpoints =>
			{
				endpoints.MapGrpcService<HealthService>();

				endpoints.MapGrpcReflectionService();

				endpoints.MapControllers();

				endpoints.MapHealthChecks("/health/live");
			});

			if (DirectoryReference.Exists(dashboardDir))
			{
				app.MapWhen(IsSpaRequest, builder => builder.UseSpa(spa => spa.Options.SourcePath = "DashboardApp"));
			}

			foreach (IPluginStartup pluginStartup in app.ApplicationServices.GetRequiredService<IEnumerable<IPluginStartup>>())
			{
				pluginStartup.Configure(app);
			}

			if (settings.Value.OpenBrowser && RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				lifetime.ApplicationStarted.Register(() => LaunchBrowser(app));
			}
		}

		static bool IsSpaRequest(HttpContext context)
		{
			return !context.Request.Path.StartsWithSegments("/api", StringComparison.OrdinalIgnoreCase);
		}

		static void LaunchBrowser(IApplicationBuilder app)
		{
			IServerAddressesFeature? feature = app.ServerFeatures.Get<IServerAddressesFeature>();
			if (feature != null && feature.Addresses.Count > 0)
			{
				// with a development cert, host will be set by default to localhost, otherwise there will be no host in address
				string address = feature.Addresses.First().Replace("[::]", System.Net.Dns.GetHostName(), StringComparison.OrdinalIgnoreCase);
				Process.Start(new ProcessStartInfo { FileName = address, UseShellExecute = true });
			}
		}

		static LogEventLevel GetRequestLoggingLevel(HttpContext context, double elapsedMs, Exception? ex)
		{
			if (context.Request != null && context.Request.Path.HasValue)
			{
				string requestPath = context.Request.Path;

				if (requestPath.Equals("/Horde.HordeRpc/QueryServerStateV2", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Verbose;
				}

				if (requestPath.Equals("/Horde.HordeRpc/UpdateSession", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Verbose;
				}

				if (requestPath.Equals("/Horde.HordeRpc/CreateEvents", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Verbose;
				}

				if (requestPath.Equals("/Horde.HordeRpc/WriteOutput", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Information;
				}

				if (requestPath.StartsWith("/Horde.HordeRpc", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Debug;
				}

				if (requestPath.Equals("/Horde.Relay.RelayRpc/GetPortMappings", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Verbose;
				}

				if (requestPath.StartsWith("/health", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Debug;
				}

				if (requestPath.StartsWith("/grpc.health", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Debug;
				}

				if (requestPath.StartsWith("/api/v1", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Debug;
				}

				if (requestPath.StartsWith("/ugs/api", StringComparison.OrdinalIgnoreCase))
				{
					return LogEventLevel.Verbose;
				}

				if (IsSpaRequest(context))
				{
					return LogEventLevel.Verbose;
				}
			}
			return LogEventLevel.Information;
		}

		class HostApplicationLifetime : IHostApplicationLifetime
		{
			public CancellationToken ApplicationStarted => CancellationToken.None;
			public CancellationToken ApplicationStopped => CancellationToken.None;
			public CancellationToken ApplicationStopping => CancellationToken.None;

			public void StopApplication() { }
		}

		public static void AddServices(IServiceCollection serviceCollection, IConfiguration configuration)
		{
			Startup startup = new Startup(configuration);
			startup.ConfigureServices(serviceCollection);
			serviceCollection.AddSingleton<IHostApplicationLifetime, HostApplicationLifetime>();
		}

		public static ServiceCollection CreateServiceCollection(IConfiguration configuration)
		{
			ServiceCollection serviceCollection = new ServiceCollection();
			AddServices(serviceCollection, configuration);
			return serviceCollection;
		}

		public static ServiceCollection CreateServiceCollection(IConfiguration configuration, ILoggerProvider loggerProvider)
		{
			ServiceCollection services = new ServiceCollection();
			services.AddSingleton(configuration);
			services.AddSingleton(loggerProvider);
			AddServices(services, configuration);
			return services;
		}
	}
}
