// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.CompilerServices;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Users
{
	/// <summary>
	/// Manages user documents
	/// </summary>
	class UserCollectionV1 : IUserCollection
	{
		class UserDocument : IUser, IUserClaims, IUserSettings
		{
			public UserId Id { get; set; }

			public ClaimDocument PrimaryClaim { get; set; } = null!;
			public List<ClaimDocument> Claims { get; set; } = new List<ClaimDocument>();

			[BsonDefaultValue(false), BsonIgnoreIfDefault]
			public bool EnableExperimentalFeatures { get; set; }

			[BsonDefaultValue(false), BsonIgnoreIfDefault]
			public bool AlwaysTagPreflightCL { get; set; }

			public BsonValue DashboardSettings { get; set; } = BsonNull.Value;
			public List<JobId> PinnedJobIds { get; set; } = new List<JobId>();
			public List<BisectTaskId> PinnedBisectTaskIds { get; set; } = new List<BisectTaskId>();

			string IUser.Name => Claims.FirstOrDefault(x => String.Equals(x.Type, "name", StringComparison.Ordinal))?.Value ?? PrimaryClaim.Value;
			string IUser.Login => Claims.FirstOrDefault(x => String.Equals(x.Type, ClaimTypes.Name, StringComparison.Ordinal))?.Value ?? PrimaryClaim.Value;
			string? IUser.Email => Claims.FirstOrDefault(x => String.Equals(x.Type, ClaimTypes.Email, StringComparison.Ordinal))?.Value;

			UserId IUserClaims.UserId => Id;
			IReadOnlyList<IUserClaim> IUserClaims.Claims => Claims;

			UserId IUserSettings.UserId => Id;
			IReadOnlyList<JobId> IUserSettings.PinnedJobIds => PinnedJobIds;
			IReadOnlyList<BisectTaskId> IUserSettings.PinnedBisectTaskIds => PinnedBisectTaskIds;

			IReadOnlyList<IUserJobTemplateSettings>? IUserSettings.JobTemplateSettings => null;
		}

		class ClaimDocument : IUserClaim
		{
			public string Type { get; set; }
			public string Value { get; set; }

			private ClaimDocument()
			{
				Type = null!;
				Value = null!;
			}

			public ClaimDocument(string type, string value)
			{
				Type = type;
				Value = value;
			}

			public ClaimDocument(IUserClaim other)
			{
				Type = other.Type;
				Value = other.Value;
			}
		}

		readonly IMongoCollection<UserDocument> _users;

		/// <summary>
		/// Static constructor
		/// </summary>
		static UserCollectionV1()
		{
			BsonSerializer.RegisterDiscriminatorConvention(typeof(UserDocument), NullDiscriminatorConvention.Instance);
			BsonSerializer.RegisterDiscriminatorConvention(typeof(ClaimDocument), NullDiscriminatorConvention.Instance);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService"></param>
		public UserCollectionV1(IMongoService mongoService)
		{
			_users = mongoService.GetCollection<UserDocument>("Users", keys => keys.Ascending(x => x.PrimaryClaim), unique: true);
		}

		/// <inheritdoc/>
		public async Task<IUser?> GetUserAsync(UserId id, CancellationToken cancellationToken)
		{
			return await _users.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask<IUser?> GetCachedUserAsync(UserId? id, CancellationToken cancellationToken)
		{
			if (id == null)
			{
				return null;
			}
			else
			{
				return await GetUserAsync(id.Value, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IUser>> FindUsersAsync(IEnumerable<UserId>? ids, string? nameRegex, int? index, int? count, CancellationToken cancellationToken)
		{
			FilterDefinition<UserDocument> filter = Builders<UserDocument>.Filter.In(x => x.Id, ids);
			return await _users.Find(filter).Range(index, count).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUser?> FindUserByLoginAsync(string login, CancellationToken cancellationToken)
		{
			ClaimDocument primaryClaim = new ClaimDocument(HordeClaimTypes.User, login);
			return await _users.Find(x => x.PrimaryClaim == primaryClaim).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public Task<IUser?> FindUserByEmailAsync(string email, CancellationToken cancellationToken)
		{
			return Task.FromResult<IUser?>(null);
		}

		/// <inheritdoc/>
		public async Task<IUser> FindOrAddUserByLoginAsync(string login, string? name, string? email, CancellationToken cancellationToken)
		{
			ClaimDocument newPrimaryClaim = new ClaimDocument(ClaimTypes.Name, login);
			UpdateDefinition<UserDocument> update = Builders<UserDocument>.Update.SetOnInsert(x => x.Id, new UserId(BinaryIdUtils.CreateNew()));
			return await _users.FindOneAndUpdateAsync<UserDocument>(x => x.PrimaryClaim == newPrimaryClaim, update, new FindOneAndUpdateOptions<UserDocument> { IsUpsert = true, ReturnDocument = ReturnDocument.After }, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUserClaims> GetClaimsAsync(UserId userId, CancellationToken cancellationToken)
		{
			return await _users.Find(x => x.Id == userId).FirstOrDefaultAsync(cancellationToken) ?? new UserDocument { Id = userId };
		}

		/// <inheritdoc/>
		public async Task UpdateClaimsAsync(UserId userId, IEnumerable<IUserClaim> claims, CancellationToken cancellationToken)
		{
			List<ClaimDocument> newClaims = claims.Select(x => new ClaimDocument(x)).ToList();
			await _users.FindOneAndUpdateAsync(x => x.Id == userId, Builders<UserDocument>.Update.Set(x => x.Claims, newClaims), cancellationToken: cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUserSettings> GetSettingsAsync(UserId userId, CancellationToken cancellationToken)
		{
			return await _users.Find(x => x.Id == userId).FirstOrDefaultAsync(cancellationToken) ?? new UserDocument { Id = userId };
		}

		/// <inheritdoc/>
		public async Task UpdateSettingsAsync(UserId userId, bool? enableExperimentalFeatures, bool? alwaysTagPreflightCL = null, BsonValue? dashboardSettings = null, IEnumerable<JobId>? addPinnedJobIds = null, IEnumerable<JobId>? removePinnedJobIds = null, UpdateUserJobTemplateOptions? templateOptions = null, IEnumerable<BisectTaskId>? addBisectTaskIds = null, IEnumerable<BisectTaskId>? removeBisectTaskIds = null, CancellationToken cancellationToken = default)
		{
			if (addPinnedJobIds != null)
			{
				foreach (JobId pinnedJobId in addPinnedJobIds)
				{
					FilterDefinition<UserDocument> filter = Builders<UserDocument>.Filter.Eq(x => x.Id, userId) & Builders<UserDocument>.Filter.AnyNin<JobId>(x => x.PinnedJobIds, new[] { pinnedJobId });
					UpdateDefinition<UserDocument> update = Builders<UserDocument>.Update.PushEach(x => x.PinnedJobIds, new[] { pinnedJobId }, 50);
					await _users.UpdateOneAsync(filter, update, null, cancellationToken);
				}
			}

			List<UpdateDefinition<UserDocument>> updates = new List<UpdateDefinition<UserDocument>>();
			if (enableExperimentalFeatures != null)
			{
				updates.Add(Builders<UserDocument>.Update.SetOrUnsetNull(x => x.EnableExperimentalFeatures, enableExperimentalFeatures));
			}
			if (alwaysTagPreflightCL != null)
			{
				updates.Add(Builders<UserDocument>.Update.SetOrUnsetNull(x => x.AlwaysTagPreflightCL, alwaysTagPreflightCL));
			}
			if (dashboardSettings != null)
			{
				updates.Add(Builders<UserDocument>.Update.Set(x => x.DashboardSettings, dashboardSettings));
			}
			if (removePinnedJobIds != null && removePinnedJobIds.Any())
			{
				updates.Add(Builders<UserDocument>.Update.PullAll(x => x.PinnedJobIds, removePinnedJobIds));
			}
			if (updates.Count > 0)
			{
				await _users.UpdateOneAsync<UserDocument>(x => x.Id == userId, Builders<UserDocument>.Update.Combine(updates), (UpdateOptions?)null, cancellationToken);
			}
		}

		/// <summary>
		/// Enumerate all the documents in this collection
		/// </summary>
		/// <returns></returns>
		public async IAsyncEnumerable<(IUser, IUserClaims, IUserSettings)> EnumerateDocumentsAsync([EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using (IAsyncCursor<UserDocument> cursor = await _users.Find(FilterDefinition<UserDocument>.Empty).ToCursorAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (UserDocument document in cursor.Current)
					{
						if (document.Claims.Count > 0)
						{
							yield return (document, document, document);
						}
					}
				}
			}
		}
	}
}
