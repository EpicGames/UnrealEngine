// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using MongoDB.Driver.Linq;

namespace HordeServer.Users
{
	/// <summary>
	/// Manages user documents
	/// </summary>
	class UserCollectionV2 : IUserCollection, IDisposable
	{
		class UserDocument : IUser
		{
			public UserId Id { get; set; }

			public string Name { get; set; }
			public string Login { get; set; }
			public string LoginUpper { get; set; }
			public string? Email { get; set; }
			public string? EmailUpper { get; set; }

			[BsonIgnoreIfDefault]
			public bool? Hidden { get; set; }

			[BsonConstructor]
			private UserDocument()
			{
				Name = null!;
				Login = null!;
				LoginUpper = null!;
			}

			public UserDocument(IUser other)
				: this(other.Id, other.Name, other.Login, other.Email)
			{
			}

			public UserDocument(UserId id, string name, string login, string? email)
			{
				Id = id;
				Name = name;
				Login = login;
				LoginUpper = login.ToUpperInvariant();
				Email = email;
				EmailUpper = email?.ToUpperInvariant();
			}
		}

		class UserClaimsDocument : IUserClaims
		{
			public UserId Id { get; set; }
			public List<UserClaim> Claims { get; set; } = new List<UserClaim>();

			UserId IUserClaims.UserId => Id;
			IReadOnlyList<IUserClaim> IUserClaims.Claims => Claims;

			[BsonConstructor]
			private UserClaimsDocument()
			{
			}

			public UserClaimsDocument(UserId id)
			{
				Id = id;
			}

			public UserClaimsDocument(IUserClaims other)
				: this(other.UserId)
			{
				Claims.AddRange(other.Claims.Select(x => new UserClaim(x)));
			}
		}

		class JobTemplateSettingsDocument : IUserJobTemplateSettings
		{
			public StreamId StreamId { get; set; }

			public TemplateId TemplateId { get; set; }

			public string TemplateHash { get; set; } = String.Empty;

			public List<string> Arguments { get; set; } = new List<string>();
			IReadOnlyList<string> IUserJobTemplateSettings.Arguments => Arguments;

			public DateTime UpdateTimeUtc { get; set; }

			[BsonConstructor]
			private JobTemplateSettingsDocument()
			{

			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="streamId"></param>
			/// <param name="templateId"></param>
			/// <param name="templateHash"></param>
			/// <param name="arguments"></param>
			public JobTemplateSettingsDocument(StreamId streamId, TemplateId templateId, string templateHash, List<string> arguments)
			{
				StreamId = streamId;
				TemplateId = templateId;
				TemplateHash = templateHash;
				Arguments = arguments;
				UpdateTimeUtc = DateTime.UtcNow;
			}
		}

		class UserSettingsDocument : IUserSettings
		{
			public UserId Id { get; set; }

			[BsonDefaultValue(false), BsonIgnoreIfDefault]
			public bool EnableExperimentalFeatures { get; set; }

			[BsonDefaultValue(false), BsonIgnoreIfDefault]
			public bool AlwaysTagPreflightCL { get; set; }

			public BsonValue DashboardSettings { get; set; } = BsonNull.Value;
			public List<JobId> PinnedJobIds { get; set; } = new List<JobId>();
			public List<BisectTaskId> PinnedBisectTaskIds { get; set; } = new List<BisectTaskId>();

			UserId IUserSettings.UserId => Id;
			IReadOnlyList<JobId> IUserSettings.PinnedJobIds => PinnedJobIds;
			IReadOnlyList<BisectTaskId> IUserSettings.PinnedBisectTaskIds => PinnedBisectTaskIds;

			public List<JobTemplateSettingsDocument> JobTemplateSettings { get; set; } = new List<JobTemplateSettingsDocument>();
			IReadOnlyList<IUserJobTemplateSettings>? IUserSettings.JobTemplateSettings => JobTemplateSettings;

			[BsonConstructor]
			private UserSettingsDocument()
			{
			}

			public UserSettingsDocument(UserId id)
			{
				Id = id;
			}

			public UserSettingsDocument(IUserSettings other)
				: this(other.UserId)
			{
				EnableExperimentalFeatures = other.EnableExperimentalFeatures;
				DashboardSettings = other.DashboardSettings;
				PinnedJobIds = new List<JobId>(other.PinnedJobIds);
				PinnedBisectTaskIds = new List<BisectTaskId>(other.PinnedBisectTaskIds);
			}
		}

		class ClaimDocument : IUserClaim
		{
			public string Type { get; set; }
			public string Value { get; set; }

			private ClaimDocument()
			{
				Type = null!;
				Value = null!;
			}

			public ClaimDocument(string type, string value)
			{
				Type = type;
				Value = value;
			}

			public ClaimDocument(IUserClaim other)
			{
				Type = other.Type;
				Value = other.Value;
			}
		}

		readonly IMongoCollection<UserDocument> _users;
		readonly IMongoCollection<UserClaimsDocument> _userClaims;
		readonly IMongoCollection<UserSettingsDocument> _userSettings;
		readonly ILogger<UserCollectionV2> _logger;
		readonly MemoryCache _userCache;

		/// <summary>
		/// Static constructor
		/// </summary>
		static UserCollectionV2()
		{
			BsonSerializer.RegisterDiscriminatorConvention(typeof(UserDocument), NullDiscriminatorConvention.Instance);
			BsonSerializer.RegisterDiscriminatorConvention(typeof(UserClaimsDocument), NullDiscriminatorConvention.Instance);
			BsonSerializer.RegisterDiscriminatorConvention(typeof(UserSettingsDocument), NullDiscriminatorConvention.Instance);

			BsonSerializer.RegisterDiscriminatorConvention(typeof(ClaimDocument), NullDiscriminatorConvention.Instance);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService"></param>
		/// <param name="logger"></param>
		public UserCollectionV2(IMongoService mongoService, ILogger<UserCollectionV2> logger)
		{
			_logger = logger;

			List<MongoIndex<UserDocument>> userIndexes = new List<MongoIndex<UserDocument>>();
			userIndexes.Add(keys => keys.Ascending(x => x.LoginUpper), unique: true);
			userIndexes.Add(keys => keys.Ascending(x => x.EmailUpper));
			_users = mongoService.GetCollection<UserDocument>("UsersV2", userIndexes);

			_userClaims = mongoService.GetCollection<UserClaimsDocument>("UserClaimsV2");
			_userSettings = mongoService.GetCollection<UserSettingsDocument>("UserSettingsV2");

			MemoryCacheOptions options = new MemoryCacheOptions();
			_userCache = new MemoryCache(options);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_userCache.Dispose();
		}

		/// <inheritdoc/>
		public async Task<IUser?> GetUserAsync(UserId id, CancellationToken cancellationToken)
		{
			if (id == UserId.Anonymous)
			{
				// Anonymous user is handled as a special case so we can debug with local anonymous login against a production DB in read-only mode.
				return new UserDocument(id, "Anonymous", "anonymous", null);
			}

			IUser? user = await _users.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			using (ICacheEntry entry = _userCache.CreateEntry(id))
			{
				entry.SetValue(user);
				entry.SetSlidingExpiration(TimeSpan.FromMinutes(30.0));
			}
			return user;
		}

		/// <inheritdoc/>
		public async ValueTask<IUser?> GetCachedUserAsync(UserId? id, CancellationToken cancellationToken)
		{
			IUser? user;
			if (id == null)
			{
				return null;
			}
			else if (_userCache.TryGetValue(id.Value, out user))
			{
				return user;
			}
			else
			{
				return await GetUserAsync(id.Value, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IUser>> FindUsersAsync(IEnumerable<UserId>? ids, string? nameRegex = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<UserDocument> filter = FilterDefinition<UserDocument>.Empty;
			if (ids != null)
			{
				filter &= Builders<UserDocument>.Filter.In(x => x.Id, ids);
			}

			if (nameRegex != null)
			{
				BsonRegularExpression regex = new BsonRegularExpression(nameRegex, "i");
				filter &= Builders<UserDocument>.Filter.Regex(x => x.Name, regex);
			}

			filter &= Builders<UserDocument>.Filter.Ne(x => x.Hidden, true);

			return await _users.Find(filter).Range(index, count ?? 100).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUser?> FindUserByLoginAsync(string login, CancellationToken cancellationToken)
		{
			string loginUpper = login.ToUpperInvariant();
			FilterDefinition<UserDocument> filter = Builders<UserDocument>.Filter.Eq(x => x.LoginUpper, loginUpper) & Builders<UserDocument>.Filter.Ne(x => x.Hidden, true);
			return await _users.Find(filter).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUser?> FindUserByEmailAsync(string email, CancellationToken cancellationToken)
		{
			string emailUpper = email.ToUpperInvariant();
			FilterDefinition<UserDocument> filter = Builders<UserDocument>.Filter.Eq(x => x.EmailUpper, emailUpper) & Builders<UserDocument>.Filter.Ne(x => x.Hidden, true);
			return await _users.Find(filter).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUser> FindOrAddUserByLoginAsync(string login, string? name, string? email, CancellationToken cancellationToken)
		{
			UserId newUserId = new UserId(BinaryIdUtils.CreateNew());
			UpdateDefinition<UserDocument> update = Builders<UserDocument>.Update.SetOnInsert(x => x.Id, newUserId).SetOnInsert(x => x.Login, login).Unset(x => x.Hidden);

			if (name == null)
			{
				update = update.SetOnInsert(x => x.Name, login);
			}
			else
			{
				update = update.Set(x => x.Name, name);
			}

			if (email != null)
			{
				update = update.Set(x => x.Email, email).Set(x => x.EmailUpper, email.ToUpperInvariant());
			}

			string loginUpper = login.ToUpperInvariant();

			IUser user = await _users.FindOneAndUpdateAsync<UserDocument>(x => x.LoginUpper == loginUpper, update, new FindOneAndUpdateOptions<UserDocument, UserDocument> { IsUpsert = true, ReturnDocument = ReturnDocument.After }, cancellationToken);
			if (user.Id == newUserId)
			{
				_logger.LogInformation("Added new user {Name} ({UserId}, {Login}, {Email})", user.Name, user.Id, user.Login, user.Email);
			}

			return user;
		}

		/// <inheritdoc/>
		public async Task<IUserClaims> GetClaimsAsync(UserId userId, CancellationToken cancellationToken)
		{
			IUserClaims? claims = await _userClaims.Find(x => x.Id == userId).FirstOrDefaultAsync(cancellationToken);
			claims ??= new UserClaimsDocument(userId);
			return claims;
		}

		/// <inheritdoc/>
		public async Task UpdateClaimsAsync(UserId userId, IEnumerable<IUserClaim> claims, CancellationToken cancellationToken)
		{
			UserClaimsDocument newDocument = new UserClaimsDocument(userId);
			newDocument.Claims.AddRange(claims.Select(x => new UserClaim(x)));
			await _userClaims.ReplaceOneAsync(x => x.Id == userId, newDocument, new ReplaceOptions { IsUpsert = true }, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IUserSettings> GetSettingsAsync(UserId userId, CancellationToken cancellationToken)
		{
			IUserSettings? settings = await _userSettings.Find(x => x.Id == userId).FirstOrDefaultAsync(cancellationToken);
			settings ??= new UserSettingsDocument(userId);
			return settings;
		}

		/// <inheritdoc/>
		public async Task UpdateSettingsAsync(UserId userId, bool? enableExperimentalFeatures = null, bool? alwaysTagPreflightCL = null, BsonValue? dashboardSettings = null, IEnumerable<JobId>? addPinnedJobIds = null, IEnumerable<JobId>? removePinnedJobIds = null, UpdateUserJobTemplateOptions? templateOptions = null, IEnumerable<BisectTaskId>? addBisectTaskIds = null, IEnumerable<BisectTaskId>? removeBisectTaskIds = null, CancellationToken cancellationToken = default)
		{
			List<UpdateDefinition<UserSettingsDocument>> updates = new List<UpdateDefinition<UserSettingsDocument>>();
			if (enableExperimentalFeatures != null)
			{
				updates.Add(Builders<UserSettingsDocument>.Update.SetOrUnsetNull(x => x.EnableExperimentalFeatures, enableExperimentalFeatures));
			}
			if (alwaysTagPreflightCL != null)
			{
				updates.Add(Builders<UserSettingsDocument>.Update.SetOrUnsetNull(x => x.AlwaysTagPreflightCL, alwaysTagPreflightCL));
			}
			if (dashboardSettings != null)
			{
				updates.Add(Builders<UserSettingsDocument>.Update.Set(x => x.DashboardSettings, dashboardSettings));
			}
			if (addPinnedJobIds != null && addPinnedJobIds.Any())
			{
				updates.Add(Builders<UserSettingsDocument>.Update.AddToSetEach(x => x.PinnedJobIds, addPinnedJobIds));
			}
			if (removePinnedJobIds != null && removePinnedJobIds.Any())
			{
				updates.Add(Builders<UserSettingsDocument>.Update.PullAll(x => x.PinnedJobIds, removePinnedJobIds));
			}

			if (addBisectTaskIds != null && addBisectTaskIds.Any())
			{
				updates.Add(Builders<UserSettingsDocument>.Update.AddToSetEach(x => x.PinnedBisectTaskIds, addBisectTaskIds));
			}
			if (removeBisectTaskIds != null && removeBisectTaskIds.Any())
			{
				updates.Add(Builders<UserSettingsDocument>.Update.PullAll(x => x.PinnedBisectTaskIds, removeBisectTaskIds));
			}

			if (templateOptions != null)
			{
				JobTemplateSettingsDocument doc = new JobTemplateSettingsDocument(templateOptions.StreamId, templateOptions.TemplateId, templateOptions.TemplateHash, templateOptions.Arguments.ToList());
				FilterDefinition<UserSettingsDocument> filter = Builders<UserSettingsDocument>.Filter.Eq(x => x.Id, userId) & Builders<UserSettingsDocument>.Filter.ElemMatch(x => x.JobTemplateSettings, t => t.StreamId == templateOptions.StreamId && t.TemplateId == templateOptions.TemplateId);
				UpdateResult result = await _userSettings.UpdateOneAsync(filter, Builders<UserSettingsDocument>.Update.Set(x => x.JobTemplateSettings[-1], doc), null, cancellationToken);
				if (result.ModifiedCount == 0)
				{
					updates.Add(Builders<UserSettingsDocument>.Update.PushEach(x => x.JobTemplateSettings, new[] { doc }, -100));
				}
			}

			if (updates.Count > 0)
			{
				await _userSettings.UpdateOneAsync<UserSettingsDocument>(x => x.Id == userId, Builders<UserSettingsDocument>.Update.Combine(updates), new UpdateOptions { IsUpsert = true }, cancellationToken);
			}
		}

		/// <summary>
		/// Upgrade from V1 collection
		/// </summary>
		/// <param name="userCollectionV1"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task ResaveDocumentsAsync(UserCollectionV1 userCollectionV1, CancellationToken cancellationToken)
		{
			await foreach ((IUser user, IUserClaims claims, IUserSettings settings) in userCollectionV1.EnumerateDocumentsAsync(cancellationToken))
			{
				try
				{
					await _users.ReplaceOneAsync(x => x.Id == user.Id, new UserDocument(user), new ReplaceOptions { IsUpsert = true }, cancellationToken);
					await _userClaims.ReplaceOneAsync(x => x.Id == user.Id, new UserClaimsDocument(claims), new ReplaceOptions { IsUpsert = true }, cancellationToken);
					await _userSettings.ReplaceOneAsync(x => x.Id == user.Id, new UserSettingsDocument(settings), new ReplaceOptions { IsUpsert = true }, cancellationToken);
					_logger.LogDebug("Updated user {UserId}", user.Id);
				}
				catch (MongoWriteException ex)
				{
					_logger.LogWarning(ex, "Unable to resave user {UserId}", user.Id);
				}

				if (settings.PinnedJobIds.Count > 0)
				{
					await UpdateSettingsAsync(user.Id, addPinnedJobIds: settings.PinnedJobIds, cancellationToken: cancellationToken);
				}
			}
		}
	}
}
