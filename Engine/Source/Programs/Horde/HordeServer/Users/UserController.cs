// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Security.Claims;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Users;
using HordeServer.Accounts;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Server;
using HordeServer.Server.Notices;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Users
{
	/// <summary>
	/// Controller for the /api/v1/user endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class UserController : ControllerBase
	{
		readonly IUserCollection _userCollection;
		readonly IAvatarService? _avatarService;
		readonly IEnumerable<IPluginResponseFilter> _responseFilters;
		readonly IOptionsSnapshot<GlobalConfig> _globalConfig;
		readonly IOptionsMonitor<ServerSettings> _settings;

		/// <summary>
		/// Constructor
		/// </summary>
		public UserController(IUserCollection userCollection, IAvatarService? avatarService, IEnumerable<IPluginResponseFilter> responseFilters, IOptionsSnapshot<GlobalConfig> globalConfig, IOptionsMonitor<ServerSettings> settings)
		{
			_userCollection = userCollection;
			_avatarService = avatarService;
			_responseFilters = responseFilters;
			_globalConfig = globalConfig;
			_settings = settings;
		}

		/// <summary>
		/// Gets information about the logged in user
		/// </summary>
		/// <returns>Http result code</returns>
		[HttpGet]
		[Route("/api/v1/user")]
		[ProducesResponseType(typeof(GetUserResponse), 200)]
		public async Task<ActionResult<object>> GetUserAsync([FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IUser? internalUser = await _userCollection.GetUserAsync(User, cancellationToken);
			if (internalUser == null)
			{
				return NotFound();
			}

			IAvatar? avatar = (_avatarService == null) ? (IAvatar?)null : await _avatarService.GetAvatarAsync(internalUser, cancellationToken);
			IUserClaims claims = await _userCollection.GetClaimsAsync(internalUser.Id, cancellationToken);
			IUserSettings settings = await _userCollection.GetSettingsAsync(internalUser.Id, cancellationToken);

			GetUserResponse response = internalUser.ToApiResponse(avatar, claims, settings);
			response.DashboardFeatures = GetDashboardFeatures(_globalConfig.Value, _settings.CurrentValue, User);

			return PropertyFilter.Apply(response, filter);
		}

		GetDashboardFeaturesResponse GetDashboardFeatures(GlobalConfig globalConfig, ServerSettings settings, ClaimsPrincipal principal)
		{
			GetDashboardFeaturesResponse response = new GetDashboardFeaturesResponse();
			response.ShowLandingPage = globalConfig.Dashboard.ShowLandingPage;
			response.LandingPageRoute = globalConfig.Dashboard.LandingPageRoute;
			response.ShowCI = globalConfig.Dashboard.ShowCI;
			response.ShowAgents = globalConfig.Dashboard.ShowAgents;
			response.ShowAgentRegistration = globalConfig.Dashboard.ShowAgentRegistration;
			response.ShowPerforceServers = globalConfig.Dashboard.ShowPerforceServers;
			response.ShowDeviceManager = globalConfig.Dashboard.ShowDeviceManager;
			response.ShowTests = globalConfig.Dashboard.ShowTests;
			response.ShowNoticeEditor = globalConfig.Authorize(NoticeAclAction.CreateNotice, principal) || globalConfig.Authorize(NoticeAclAction.UpdateNotice, principal);
			response.ShowAccounts = settings.AuthMethod == EpicGames.Horde.Server.AuthMethod.Horde && globalConfig.Authorize(AccountAclAction.UpdateAccount, principal);

			foreach (IPluginResponseFilter responseFilter in _responseFilters)
			{
				responseFilter.Apply(HttpContext, response);
			}

			return response;
		}

		/// <summary>
		/// Updates the logged in user
		/// </summary>
		/// <returns>Http result code</returns>
		[HttpPut]
		[Route("/api/v1/user")]
		public async Task<ActionResult> UpdateUserAsync(UpdateUserRequest request)
		{
			UserId? userId = User.GetUserId();
			if (userId == null)
			{
				return BadRequest("Current user does not have a registered profile");
			}

			await _userCollection.UpdateSettingsAsync(userId.Value, request.EnableExperimentalFeatures, request.AlwaysTagPreflightCL, request.DashboardSettings?.ToBsonValue(), request.AddPinnedJobIds?.Select(x => JobId.Parse(x)), request.RemovePinnedJobIds?.Select(x => JobId.Parse(x)), null, request.AddPinnedBisectTaskIds?.Select(x => BisectTaskId.Parse(x)), request.RemovePinnedBisectTaskIds?.Select(x => BisectTaskId.Parse(x)));
			return Ok();
		}

		/// <summary>
		/// Gets claims for the current user
		/// </summary>
		[HttpGet]
		[Route("/api/v1/user/claims")]
		public ActionResult<object[]> GetUserClaims()
		{
			return User.Claims.Select(x => new { x.Type, x.Value }).ToArray();
		}
		
		/// <summary>
		/// Gets ACL permissions for the current user
		/// </summary>
		[HttpGet]
		[Route("/api/v1/user/acl-permissions")]
		public ActionResult<GetUserAclPermissionsResponse> GetUserAclPermissions()
		{
			return new GetUserAclPermissionsResponse(GetUserAclPermissions(_globalConfig.Value, User));
		}
		
		/// <summary>
		/// Get a list of ACL permissions for a user
		/// A scope is only included if the user is authorized for at least one action inside of it
		/// </summary>
		/// <param name="globalConfig">Global config</param>
		/// <param name="user">User to inspect</param>
		/// <returns></returns>
		public static List<UserAclPermission> GetUserAclPermissions(GlobalConfig globalConfig, ClaimsPrincipal user)
		{
			List<UserAclPermission> result = [];
			foreach (AclConfig aclConfig in globalConfig.AclScopes.Values)
			{
				string scope = aclConfig.ScopeName.ToString();
				List<UserAclPermission> permissions = (aclConfig.Actions ?? [])
					.Select(x => new UserAclPermission(scope, x.Name, aclConfig.Authorize(x, user)))
					.ToList();
				
				bool hasPermission = permissions.Exists(x => x.IsAuthorized);
				if (hasPermission)
				{
					result.AddRange(permissions);
				}
			}
			
			return result;
		}
	}
}
