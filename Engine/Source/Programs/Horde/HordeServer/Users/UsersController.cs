// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Horde.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Users
{
	/// <summary>
	/// Controller for the /api/v1/users endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class UsersController : HordeControllerBase
	{
		/// <summary>
		/// The user collection instance
		/// </summary>
		IUserCollection UserCollection { get; set; }

		/// <summary>
		/// The avatar service
		/// </summary>
		IAvatarService? AvatarService { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="userCollection"></param>
		/// <param name="avatarService"></param>
		public UsersController(IUserCollection userCollection, IAvatarService? avatarService)
		{
			UserCollection = userCollection;
			AvatarService = avatarService;
		}

		/// <summary>
		/// Gets information about a user by id, specify "current" for id to get the currently logged in user
		/// </summary>
		/// <returns>Http result code</returns>
		[HttpGet]
		[Route("/api/v1/users/{id}")]
		[ProducesResponseType(typeof(List<GetUserResponse>), 200)]
		public async Task<ActionResult<object>> GetUserAsync(string id, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			UserId? userId = ParseUserId(id);
			if (userId == null)
			{
				return BadRequest("Invalid user id '{Id}'", id);
			}

			IUser? user = await UserCollection.GetUserAsync(userId.Value, cancellationToken);
			if (user == null)
			{
				return NotFound(userId.Value);
			}

			IAvatar? avatar = (AvatarService == null) ? (IAvatar?)null : await AvatarService.GetAvatarAsync(user, cancellationToken);
			IUserClaims? claims = await UserCollection.GetClaimsAsync(user.Id, cancellationToken);
			IUserSettings? settings = await UserCollection.GetSettingsAsync(user.Id, cancellationToken);
			return PropertyFilter.Apply(user.ToApiResponse(avatar, claims, settings), filter);
		}

		/// <summary>
		/// Gets a list of users
		/// </summary>
		/// <returns>List of user responses</returns>
		[HttpGet]
		[Route("/api/v1/users")]
		[ProducesResponseType(typeof(List<GetUserResponse>), 200)]
		public async Task<ActionResult<List<GetUserResponse>>> FindUsersAsync(
			[FromQuery] string[]? ids = null,
			[FromQuery] string? nameRegex = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 100,
			[FromQuery] bool includeClaims = false,
			[FromQuery] bool includeAvatar = false,
			CancellationToken cancellationToken = default)
		{

			UserId[]? userIds = null;
			if (ids != null && ids.Length > 0)
			{
				userIds = ids.Select(x => UserId.Parse(x)).ToArray();
			}

			IReadOnlyList<IUser> users = await UserCollection.FindUsersAsync(userIds, nameRegex, index, count, cancellationToken);

			List<GetUserResponse> response = new List<GetUserResponse>();
			foreach (IUser user in users)
			{
				IAvatar? avatar = (AvatarService == null || !includeAvatar) ? (IAvatar?)null : await AvatarService.GetAvatarAsync(user, cancellationToken);
				IUserClaims? claims = (!includeClaims) ? null : await UserCollection.GetClaimsAsync(user.Id, cancellationToken);
				response.Add(user.ToApiResponse(avatar, claims, null));
			}

			return response;
		}

		UserId? ParseUserId(string id)
		{
			if (id.Equals("current", StringComparison.OrdinalIgnoreCase))
			{
				return User.GetUserId();
			}
			else if (UserId.TryParse(id, out UserId result))
			{
				return result;
			}
			return null;
		}
	}
}
