// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Text.Json;

#nullable disable

namespace HordeServer.Utilities
{
	static class AgentUtilities
	{
		/// <summary>
		/// Reads the version number from an archive
		/// </summary>
		/// <param name="data">The archive data</param>
		/// <returns></returns>
		public static string ReadVersion(byte[] data)
		{
			MemoryStream inputStream = new MemoryStream(data);
			using (ZipArchive inputArchive = new ZipArchive(inputStream, ZipArchiveMode.Read, true))
			{
				foreach (ZipArchiveEntry inputEntry in inputArchive.Entries)
				{
					if (inputEntry.FullName.Equals("HordeAgent.dll", StringComparison.OrdinalIgnoreCase))
					{
						string tempFile = Path.GetTempFileName();
						try
						{
							inputEntry.ExtractToFile(tempFile, true);
							return FileVersionInfo.GetVersionInfo(tempFile).ProductVersion;
						}
						finally
						{
							File.Delete(tempFile);
						}
					}
				}
			}
			throw new Exception("Unable to find HordeAgent.dll in archive");
		}

		/// <summary>
		/// Updates the agent app settings within the archive data
		/// </summary>
		/// <param name="data">Data for the zip archive</param>
		/// <param name="settings">The settings to update</param>
		/// <returns>New agent app data</returns>
		public static byte[] UpdateAppSettings(byte[] data, Dictionary<string, object> settings)
		{
			bool writtenClientId = false;

			MemoryStream outputStream = new MemoryStream();
			using (ZipArchive outputArchive = new ZipArchive(outputStream, ZipArchiveMode.Create, true))
			{
				MemoryStream inputStream = new MemoryStream(data);
				using (ZipArchive inputArchive = new ZipArchive(inputStream, ZipArchiveMode.Read, true))
				{
					foreach (ZipArchiveEntry inputEntry in inputArchive.Entries)
					{
						ZipArchiveEntry outputEntry = outputArchive.CreateEntry(inputEntry.FullName);

						using System.IO.Stream inputEntryStream = inputEntry.Open();
						using System.IO.Stream outputEntryStream = outputEntry.Open();

						if (inputEntry.FullName.Equals("appsettings.json", StringComparison.OrdinalIgnoreCase))
						{
							using MemoryStream memoryStream = new MemoryStream();
							inputEntryStream.CopyTo(memoryStream);

							Dictionary<string, Dictionary<string, object>> document = JsonSerializer.Deserialize<Dictionary<string, Dictionary<string, object>>>(memoryStream.ToArray());
							foreach (KeyValuePair<string, object> pair in settings)
							{
								document["Horde"][pair.Key] = pair.Value;
							}

							using Utf8JsonWriter writer = new Utf8JsonWriter(outputEntryStream, new JsonWriterOptions { Indented = true });
							JsonSerializer.Serialize<Dictionary<string, Dictionary<string, object>>>(writer, document, new JsonSerializerOptions { WriteIndented = true });

							writtenClientId = true;
						}
						else
						{
							inputEntryStream.CopyTo(outputEntryStream);
						}
					}
				}
			}

			if (!writtenClientId)
			{
				throw new InvalidDataException("Missing appsettings.json file from zip archive");
			}

			return outputStream.ToArray();
		}
	}
}
