// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace HordeServer.Utilities
{
	struct LogIndex
	{
		const int NumHashes = 3;
		const int NgramLength = 3;
		const int NgramMask = (1 << (NgramLength * 8)) - 1;

		/// <summary>
		/// Enumerates the hash codes for a given block of text
		/// </summary>
		/// <param name="text">The text to parse</param>
		/// <returns></returns>
		public static IEnumerable<int> GetHashCodes(ReadOnlyMemory<byte> text)
		{
			if (text.Length >= NgramLength)
			{
				int value = 0;
				for (int idx = 0; idx < NgramLength - 1; idx++)
				{
					value = (value << 8) | ToLowerUtf8(text.Span[idx]);
				}
				for (int idx = NgramLength - 1; idx < text.Length; idx++)
				{
					value = ((value << 8) | ToLowerUtf8(text.Span[idx])) & NgramMask;

					int hashValue = value;
					yield return hashValue;

					for (int hashIdx = 1; hashIdx < NumHashes; hashIdx++)
					{
						hashValue = Scramble(hashValue);
						yield return hashValue;
					}
				}
			}
		}

		/// <summary>
		/// Converts a character into a format for hashing
		/// </summary>
		/// <param name="value">The input byte</param>
		/// <returns>The value to include in the trigram</returns>
		static byte ToLowerUtf8(byte value)
		{
			if (value >= 'A' && value <= 'Z')
			{
				return (byte)(value + 'a' - 'A');
			}
			else
			{
				return value;
			}
		}

		/// <summary>
		/// Scramble the given number using a pseudo-RNG
		/// </summary>
		/// <param name="value">Initial value</param>
		/// <returns>The scrambled value</returns>
		private static int Scramble(int value)
		{
			value ^= value << 13;
			value ^= value >> 17;
			value ^= value << 5;
			return value;
		}
	}

	/// <summary>
	/// Read only implementation of a bloom filter
	/// </summary>
	public class ReadOnlyBloomFilter
	{
		/// <summary>
		/// Data for the bloom filter
		/// </summary>
		public ReadOnlyMemory<byte> Data { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="data">Data to construct from</param>
		public ReadOnlyBloomFilter(ReadOnlyMemory<byte> data)
		{
			Data = data;
		}

		/// <summary>
		/// Determines if the filter may contain a value
		/// </summary>
		/// <param name="hashCodes">Sequence of hash codes</param>
		/// <returns>True if the value is in the set</returns>
		public bool Contains(IEnumerable<int> hashCodes)
		{
			ReadOnlySpan<byte> span = Data.Span;
			foreach (int hashCode in hashCodes)
			{
				int index = hashCode & ((span.Length << 3) - 1);
				if ((span[index >> 3] & (1 << (index & 7))) == 0)
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Calculates the load of this filter (ie. the proportion of bits which are set)
		/// </summary>
		/// <returns>Value between 0 and 1</returns>
		public double CalculateLoadFactor()
		{
			int setCount = 0;

			ReadOnlySpan<byte> span = Data.Span;
			for (int idx = 0; idx < span.Length; idx++)
			{
				for (int bit = 0; bit < 8; bit++)
				{
					if ((span[idx] & (1 << bit)) != 0)
					{
						setCount++;
					}
				}
			}

			return (double)setCount / (span.Length * 8);
		}
	}

	/// <summary>
	/// Implementation of a bloom filter
	/// </summary>
	public class BloomFilter
	{
		/// <summary>
		/// The array of bits in the filter
		/// </summary>
		public byte[] Data { get; }

		/// <summary>
		/// Constructs a new filter of the given size
		/// </summary>
		/// <param name="minSize">The minimum size of the filter. Will be rounded up to the next power of two.</param>
		public BloomFilter(int minSize)
		{
			int size = minSize;
			while ((size & (size - 1)) != 0)
			{
				size |= (size - 1);
				size++;
			}
			Data = new byte[size];
		}

		/// <summary>
		/// Constructs a filter from raw data
		/// </summary>
		/// <param name="data">The data to construct from</param>
		public BloomFilter(byte[] data)
		{
			Data = data;
			if ((data.Length & (data.Length - 1)) != 0)
			{
				throw new ArgumentException("Array for bloom filter must be a power of 2 in size.");
			}
		}

		/// <summary>
		/// Add a value to the filter
		/// </summary>
		/// <param name="hashCodes">Sequence of hash codes</param>
		public void Add(IEnumerable<int> hashCodes)
		{
			foreach (int hashCode in hashCodes)
			{
				int index = hashCode & ((Data.Length << 3) - 1);
				Data[index >> 3] = (byte)(Data[index >> 3] | (1 << (index & 7)));
			}
		}

		/// <summary>
		/// Determines if the filter may contain a value
		/// </summary>
		/// <param name="hashCodes">Sequence of hash codes</param>
		/// <returns>True if the value is in the set</returns>
		public bool Contains(IEnumerable<int> hashCodes)
		{
			foreach (int hashCode in hashCodes)
			{
				int index = hashCode & ((Data.Length << 3) - 1);
				if ((Data[index >> 3] & (1 << (index & 7))) == 0)
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Calculates the load of this filter (ie. the proportion of bits which are set)
		/// </summary>
		/// <returns>Value between 0 and 1</returns>
		public double CalculateLoadFactor()
		{
			int setCount = 0;
			int allCount = 0;
			for (int idx = 0; idx < Data.Length; idx++)
			{
				for (int bit = 0; bit < 8; bit++)
				{
					if ((Data[idx] & (1 << bit)) != 0)
					{
						setCount++;
					}
					allCount++;
				}
			}
			return (double)setCount / allCount;
		}
	}
}
