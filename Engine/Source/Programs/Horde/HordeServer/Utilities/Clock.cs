// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using EpicGames.Redis.Utility;
using HordeServer;
using HordeServer.Server;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using StackExchange.Redis;
using TimeZoneConverter;

namespace HordeCommon
{
	/// <summary>
	/// Implementation of <see cref="IClock"/> which returns the current time
	/// </summary>
	public sealed class Clock : IClock, IAsyncDisposable
	{
		sealed class TickerImpl : ITicker, IAsyncDisposable
		{
			readonly string _name;
			readonly CancellationTokenSource _cancellationSource;
			readonly Func<Task> _tickFunc;
			Task? _backgroundTask;

			public TickerImpl(string name, TimeSpan delay, Func<CancellationToken, ValueTask<TimeSpan?>> triggerAsync, ILogger logger)
			{
				_name = name;
				_cancellationSource = new CancellationTokenSource();
				_tickFunc = () => RunAsync(delay, triggerAsync, logger);
			}

			public async Task StartAsync()
			{
				await StopAsync();

				_backgroundTask = Task.Run(_tickFunc);
			}

			public async Task StopAsync()
			{
				if (_backgroundTask != null)
				{
					await _cancellationSource.CancelAsync();
					await _backgroundTask;
					_backgroundTask = null;
				}
			}

			public async ValueTask DisposeAsync()
			{
				await StopAsync();
				_cancellationSource.Dispose();
			}

			public async Task RunAsync(TimeSpan delay, Func<CancellationToken, ValueTask<TimeSpan?>> triggerAsync, ILogger logger)
			{
				while (!_cancellationSource!.IsCancellationRequested)
				{
					try
					{
						if (delay > TimeSpan.Zero)
						{
							await Task.Delay(delay, _cancellationSource.Token);
						}

						TimeSpan? nextDelay = await triggerAsync(_cancellationSource.Token);
						if (nextDelay == null)
						{
							break;
						}

						delay = nextDelay.Value;
					}
					catch (OperationCanceledException) when (_cancellationSource.IsCancellationRequested)
					{
					}
					catch (Exception ex)
					{
						logger.LogError(ex, "Exception while executing scheduled event");
						if (delay < TimeSpan.Zero)
						{
							delay = TimeSpan.FromSeconds(5.0);
							logger.LogWarning("Delaying tick for 5 seconds");
						}
					}
				}
			}

			public override string ToString() => _name;
		}

		readonly IRedisService _redis;
		readonly TimeZoneInfo _timeZone;
		readonly List<TickerImpl> _tickers = new List<TickerImpl>();

		/// <inheritdoc/>
		public DateTime UtcNow => DateTime.UtcNow;

		/// <inheritdoc/>
		public TimeZoneInfo TimeZone => _timeZone;

		/// <summary>
		/// Constructor
		/// </summary>
		public Clock(IRedisService redis, IOptions<ServerSettings> settings)
		{
			_redis = redis;

			string? timeZoneName = settings.Value.ScheduleTimeZone;
			_timeZone = (timeZoneName == null) ? TimeZoneInfo.Local : TZConvert.GetTimeZoneInfo(timeZoneName);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			foreach (TickerImpl ticker in _tickers)
			{
				await ticker.DisposeAsync();
			}
		}

		/// <inheritdoc/>
		public ITicker AddTicker(string name, TimeSpan delay, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger)
		{
			TickerImpl ticker = new TickerImpl(name, delay, tickAsync, logger);
			lock (_tickers)
			{
				_tickers.Add(ticker);
			}
			return ticker;
		}

		/// <inheritdoc/>
		public ITicker AddSharedTicker(string name, TimeSpan delay, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
		{
			RedisKey key = new RedisKey($"tick/{name}");
			return this.AddTicker(name, delay / 4, token => TriggerSharedAsync(key, delay, tickAsync, token), logger);
		}

		async ValueTask TriggerSharedAsync(RedisKey key, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, CancellationToken cancellationToken)
		{
			if (_redis.ReadOnlyMode)
			{
				return;
			}

			using (RedisLock sharedLock = new(_redis.GetDatabase(), key))
			{
				if (await sharedLock.AcquireAsync(interval, false))
				{
					await tickAsync(cancellationToken);
				}
			}
		}
	}

	/// <summary>
	/// Fake clock that doesn't advance by wall block time
	/// Requires manual ticking to progress. Used in tests.
	/// </summary>
	public class FakeClock : IClock
	{
		class TickerImpl : ITicker
		{
			readonly FakeClock _outer;
			readonly string _name;
			readonly TimeSpan _interval;
			public DateTime? NextTime { get; set; }
			public Func<CancellationToken, ValueTask<TimeSpan?>> TickAsync { get; }

			public TickerImpl(FakeClock outer, string name, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync)
			{
				_outer = outer;
				_name = name;
				_interval = interval;
				TickAsync = tickAsync;

				lock (outer._triggers)
				{
					outer._triggers.Add(this);
				}
			}

			public Task StartAsync()
			{
				NextTime = _outer.UtcNow + _interval;
				return Task.CompletedTask;
			}

			public Task StopAsync()
			{
				NextTime = null;
				return Task.CompletedTask;
			}

			public ValueTask DisposeAsync()
			{
				lock (_outer._triggers)
				{
					_outer._triggers.Remove(this);
				}
				return new ValueTask();
			}

			public override string ToString()
			{
				if (NextTime == null)
				{
					return $"{_name} (paused)";
				}
				else
				{
					return $"{_name} ({NextTime.Value})";
				}
			}
		}

		DateTime _utcNowPrivate;
		readonly List<TickerImpl> _triggers = new List<TickerImpl>();

		/// <summary>
		/// Constructor
		/// </summary>
		public FakeClock()
		{
			_utcNowPrivate = DateTime.UtcNow;
			TimeZone = TimeZoneInfo.Utc;
		}

		/// <summary>
		/// Advance time by given amount
		/// Useful for letting time progress during tests
		/// </summary>
		/// <param name="period">Time span to advance</param>
		public async Task AdvanceAsync(TimeSpan period)
		{
			_utcNowPrivate = _utcNowPrivate.Add(period);

			for (int idx = 0; idx < _triggers.Count; idx++)
			{
				TickerImpl trigger = _triggers[idx];
				while (trigger.NextTime != null && _utcNowPrivate > trigger.NextTime)
				{
					TimeSpan? delay = await trigger.TickAsync(CancellationToken.None);
					if (delay == null)
					{
						_triggers.RemoveAt(idx--);
						break;
					}
					trigger.NextTime = _utcNowPrivate + delay.Value;
				}
			}
		}

		/// <inheritdoc/>
		public DateTime UtcNow
		{
			get => _utcNowPrivate;
			set => _utcNowPrivate = value.ToUniversalTime();
		}

		/// <inheritdoc/>
		public TimeZoneInfo TimeZone { get; set; }

		/// <inheritdoc/>
		public ITicker AddTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask<TimeSpan?>> tickAsync, ILogger logger)
		{
			return new TickerImpl(this, name, interval, tickAsync);
		}

		/// <inheritdoc/>
		public ITicker AddSharedTicker(string name, TimeSpan interval, Func<CancellationToken, ValueTask> tickAsync, ILogger logger)
		{
			async ValueTask<TimeSpan?> TickAsync(CancellationToken token)
			{
				await tickAsync(token);
				return interval;
			}
			return new TickerImpl(this, name, interval, TickAsync);
		}
	}
}