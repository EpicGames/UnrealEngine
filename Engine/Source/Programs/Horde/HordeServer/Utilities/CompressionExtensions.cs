// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Buffers.Binary;
using System.IO;
using EpicGames.Core;
using ICSharpCode.SharpZipLib.BZip2;
using OpenTelemetry.Trace;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for compression
	/// </summary>
	static class CompressionExtensions
	{
		/// <summary>
		/// Compress a block of data with bzip2
		/// </summary>
		/// <param name="memory">Memory to compress</param>
		/// <param name="compressionLevel">Compression level, 1 to 9 (where 9 is highest compression)</param>
		/// <returns>The compressed data</returns>
		public static byte[] CompressBzip2(this ReadOnlyMemory<byte> memory, int compressionLevel = 4)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(CompressionExtensions)}.{nameof(CompressBzip2)}");
			span.SetAttribute("decompressedSize", memory.Length);

			byte[] compressedData;
			using (MemoryStream stream = new MemoryStream())
			{
				byte[] decompressedSize = new byte[4];
				BinaryPrimitives.WriteInt32LittleEndian(decompressedSize.AsSpan(), memory.Length);
				stream.Write(decompressedSize.AsSpan());

				using (BZip2OutputStream compressedStream = new(stream, compressionLevel))
				{
					compressedStream.Write(memory.Span);
				}

				compressedData = stream.ToArray();
			}

			span.SetAttribute("compressedSize", compressedData.Length);
			return compressedData;
		}

		/// <summary>
		/// Decompress the data
		/// </summary>
		/// <returns>The decompressed data</returns>
		public static byte[] DecompressBzip2(this ReadOnlyMemory<byte> memory)
		{
			int decompressedSize = BinaryPrimitives.ReadInt32LittleEndian(memory.Span);

			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(CompressionExtensions)}.{nameof(DecompressBzip2)}");
			span.SetAttribute("compressedSize", memory.Length);
			span.SetAttribute("decompressedSize", decompressedSize);

			byte[] data = new byte[decompressedSize];
			using (ReadOnlyMemoryStream stream = new ReadOnlyMemoryStream(memory.Slice(4)))
			{
				using (BZip2InputStream decompressedStream = new BZip2InputStream(stream))
				{
					int readSize = decompressedStream.Read(data.AsSpan());
					if (readSize != data.Length)
					{
						throw new InvalidDataException($"Compressed data is too short (expected {data.Length} bytes, got {readSize} bytes)");
					}
				}
			}
			return data;
		}
	}
}
