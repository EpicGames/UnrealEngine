// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Horde.Common;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Serializer for Condition objects
	/// </summary>
	public sealed class ConditionSerializer : IBsonSerializer<Condition>
	{
		/// <inheritdoc/>
		public Type ValueType => typeof(Condition);

		/// <inheritdoc/>
		void IBsonSerializer.Serialize(BsonSerializationContext context, BsonSerializationArgs args, object value) => Serialize(context, args, (Condition)value);

		/// <inheritdoc/>
		object IBsonSerializer.Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args) => Deserialize(context, args);

		/// <inheritdoc/>
		public Condition Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			if (context.Reader.CurrentBsonType == BsonType.Null)
			{
				context.Reader.ReadNull();
				return null!;
			}
			return Condition.Parse(context.Reader.ReadString());
		}

		/// <inheritdoc/>
		public void Serialize(BsonSerializationContext context, BsonSerializationArgs args, Condition? value)
		{
			if (value == null)
			{
				context.Writer.WriteNull();
			}
			else
			{
				context.Writer.WriteString(value.ToString());
			}
		}
	}
}
