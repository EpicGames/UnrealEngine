// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using MongoDB.Bson.Serialization;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Serializer for ContentHash objects
	/// </summary>
	public sealed class ContentHashSerializer : IBsonSerializer<ContentHash>
	{
		/// <inheritdoc/>
		public Type ValueType => typeof(ContentHash);

		/// <inheritdoc/>
		void IBsonSerializer.Serialize(BsonSerializationContext context, BsonSerializationArgs args, object value)
		{
			Serialize(context, args, (ContentHash)value);
		}

		/// <inheritdoc/>
		object IBsonSerializer.Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			return ((IBsonSerializer<ContentHash>)this).Deserialize(context, args);
		}

		/// <inheritdoc/>
		public ContentHash Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			if (context.Reader.CurrentBsonType == MongoDB.Bson.BsonType.ObjectId)
			{
				return new ContentHash(context.Reader.ReadObjectId().ToByteArray());
			}
			else
			{
				return ContentHash.Parse(context.Reader.ReadString());
			}
		}

		/// <inheritdoc/>
		public void Serialize(BsonSerializationContext context, BsonSerializationArgs args, ContentHash value)
		{
			context.Writer.WriteString(value.ToString());
		}
	}
}
