// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Globalization;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Serializers;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Serializer for DateTimeOffset structs, which writes data as a string
	/// </summary>
	class DateTimeOffsetStringSerializer : StructSerializerBase<DateTimeOffset>
	{
		/// <inheritdoc/>
		public override DateTimeOffset Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			switch (context.Reader.CurrentBsonType)
			{
				case BsonType.String:
					string stringValue = context.Reader.ReadString();
					return DateTimeOffset.Parse(stringValue, DateTimeFormatInfo.InvariantInfo);
				case BsonType.DateTime:
					long ticks = context.Reader.ReadDateTime();
					return DateTimeOffset.FromUnixTimeMilliseconds(ticks);
				default:
					throw new FormatException($"Unable to deserialize a DateTimeOffset from a {context.Reader.CurrentBsonType}");
			}
		}

		/// <inheritdoc/>
		public override void Serialize(BsonSerializationContext context, BsonSerializationArgs args, DateTimeOffset value)
		{
			context.Writer.WriteString(value.ToString(DateTimeFormatInfo.InvariantInfo));
		}
	}
}
