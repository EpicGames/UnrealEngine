// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using MongoDB.Bson;
using MongoDB.Bson.IO;
using MongoDB.Bson.Serialization.Conventions;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Implements a discriminator convention, which defaults to a specific concrete type if no discriminator is specified.
	/// 
	/// This class should also be registered as the discriminator for the concrete type as well as the interface, because the default behavior
	/// of the serializer is to recognize that the concrete type is discriminated and fall back to trying to read the discriminator element,
	/// failing, and falling back to serializing it as the interface recursively until we get a stack overflow.
	/// 
	/// Registering this as the discriminator for the concrete type prevents this, because we can discriminate the concrete type correctly.
	/// </summary>
	class DefaultDiscriminatorConvention : IDiscriminatorConvention
	{
		/// <summary>
		/// The normal discriminator to use
		/// </summary>
		static IDiscriminatorConvention Inner { get; } = StandardDiscriminatorConvention.Hierarchical;

		/// <summary>
		/// The nominal type
		/// </summary>
		readonly Type _baseType;

		/// <summary>
		/// Default type to use if the inner discriminator returns the nominal type
		/// </summary>
		readonly Type _defaultType;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="baseType">The base class</param>
		/// <param name="defaultType">Default type to use if the inner discriminator returns an interface</param>
		public DefaultDiscriminatorConvention(Type baseType, Type defaultType)
		{
			_baseType = baseType;
			_defaultType = defaultType;
		}

		/// <inheritdoc/>
		public string ElementName => Inner.ElementName;

		/// <inheritdoc/>
		public Type GetActualType(IBsonReader bsonReader, Type nominalType)
		{
			Type actualType = Inner.GetActualType(bsonReader, nominalType);
			if (actualType == _baseType)
			{
				actualType = _defaultType;
			}
			return actualType;
		}

		/// <inheritdoc/>
		public BsonValue GetDiscriminator(Type nominalType, Type actualType)
		{
			return Inner.GetDiscriminator(nominalType, actualType);
		}
	}

	/// <summary>
	/// Generic version of <see cref="DefaultDiscriminatorConvention"/>
	/// </summary>
	class DefaultDiscriminatorConvention<TBase, TDerived> : DefaultDiscriminatorConvention
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public DefaultDiscriminatorConvention()
			: base(typeof(TBase), typeof(TDerived))
		{
		}
	}
}
