// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using EpicGames.Core;
using HordeServer.Plugins;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Cache for Json schemas
	/// </summary>
	public class JsonSchemaCache
	{
		readonly JsonSchemaFactory _factory;
		readonly ConcurrentDictionary<Type, JsonSchema> _cachedSchemas = new ConcurrentDictionary<Type, JsonSchema>();

		/// <summary>
		/// Constructor
		/// </summary>
		public JsonSchemaCache(IPluginCollection pluginCollection)
		{
			_factory = new JsonSchemaFactory(new XmlDocReader());

			// Create a dummy class for the global plugin configuration, which has named properties for each plugin
			JsonSchemaObject globalPluginsObj = new JsonSchemaObject();
			globalPluginsObj.Name = "GlobalPluginsConfig";
			foreach (ILoadedPlugin plugin in pluginCollection.LoadedPlugins)
			{
				if (plugin.GlobalConfigType != null)
				{
					JsonSchemaType pluginSchemaType = _factory.CreateSchemaType(plugin.GlobalConfigType);
					globalPluginsObj.Properties.Add(new JsonSchemaProperty(plugin.Name.ToString(), $"Configuration for the {plugin.Name} plugin", pluginSchemaType));
				}
			}
			_factory.TypeCache.Add(typeof(PluginConfigCollection), globalPluginsObj);

			// Same thing for the server config. This is handled by adding a custom property, since config binding for app settings is different to the global config parser.
			JsonSchemaObject serverPluginsObj = new JsonSchemaObject();
			serverPluginsObj.Name = "ServerPluginsConfig";
			foreach (ILoadedPlugin plugin in pluginCollection.LoadedPlugins)
			{
				if (plugin.ServerConfigType != null && plugin.ServerConfigType != typeof(object))
				{
					JsonSchemaType pluginSchemaType = _factory.CreateSchemaType(plugin.ServerConfigType);
					serverPluginsObj.Properties.Add(new JsonSchemaProperty(plugin.Name.ToString(), $"Configuration for the {plugin.Name} plugin", pluginSchemaType));
				}
			}

			JsonSchemaObject serverObj = (JsonSchemaObject)_factory.CreateSchemaType(typeof(ServerSettings));
			serverObj.Properties.Add(new JsonSchemaProperty("Plugins", "Configuration for plugins", serverPluginsObj));
		}

		/// <summary>
		/// Create a Json schema (or retrieve a cached schema)
		/// </summary>
		public JsonSchema CreateSchema(Type type)
		{
			JsonSchema? schema;
			if (!_cachedSchemas.TryGetValue(type, out schema))
			{
				lock (_factory)
				{
					schema = _cachedSchemas.GetOrAdd(type, x => _factory.CreateSchema(x));
				}
			}
			return schema;
		}
	}
}
