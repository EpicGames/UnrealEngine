// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics.CodeAnalysis;
using System.IdentityModel.Tokens.Jwt;
using System.Text.Json;
using Microsoft.AspNetCore.Http;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Helper functions for dealing with JWTs
	/// </summary>
	public static class JwtUtils
	{
		/// <summary>
		/// Gets the bearer token from an HTTP request
		/// </summary>
		/// <param name="request">The request to read from</param>
		/// <param name="bearerPrefix">The bearer token prefix, ex. "Bearer "</param>
		/// <param name="token">On success, receives the bearer token</param>
		/// <returns>True if the bearer token was read</returns>
		public static bool TryGetBearerToken(HttpRequest request, string bearerPrefix, [NotNullWhen(true)] out string? token)
		{
			// Get the authorization header
			string? authorization = request.Headers.Authorization;
			if (String.IsNullOrEmpty(authorization))
			{
				token = null;
				return false;
			}

			// Check if it's a bearer token				
			if (!authorization.StartsWith(bearerPrefix, StringComparison.OrdinalIgnoreCase))
			{
				token = null;
				return false;
			}

			// Get the token
			token = authorization.Substring(bearerPrefix.Length).Trim();
			return true;
		}

		/// <summary>
		/// Tries to parse a JWT and check the issuer matches
		/// </summary>
		/// <param name="token">The token to parse</param>
		/// <param name="jwtToken">On success, receives the parsed JWT</param>
		/// <returns>True if the jwt was parsed</returns>
		public static bool TryParseJwt(string token, [NotNullWhen(true)] out JwtSecurityToken? jwtToken)
		{
			// Check if it's a JWT
			JwtSecurityTokenHandler handler = new JwtSecurityTokenHandler();
			if (!handler.CanReadToken(token))
			{
				jwtToken = null;
				return false;
			}

			// Try to parse the JWT
			try
			{
				jwtToken = handler.ReadJwtToken(token);
				return true;
			}
			catch (JsonException)
			{
				jwtToken = null;
				return false;
			}
		}
	}
}
