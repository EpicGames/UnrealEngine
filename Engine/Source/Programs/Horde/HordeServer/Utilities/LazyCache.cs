// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Diagnostics;
using System.Threading;
using System.Threading.Tasks;

#pragma warning disable VSTHRD110 // Observe the awaitable result of this method call by awaiting it, assigning to a variable, or passing it to another method

namespace HordeServer.Utilities
{
	/// <summary>
	/// Options for <see cref="LazyCache{TKey, TValue}"/>
	/// </summary>
	public class LazyCacheOptions
	{
		/// <summary>
		/// Time after which a value will asynchronously be updated
		/// </summary>
		public TimeSpan? RefreshTime { get; set; } = TimeSpan.FromMinutes(1.0);

		/// <summary>
		/// Maximum age of any returned value. This will prevent a cached value being returned.
		/// </summary>
		public TimeSpan? MaxAge { get; set; } = TimeSpan.FromMinutes(2.0);
	}

	/// <summary>
	/// Implements a cache which starts an asynchronous update of a value after a period of time.
	/// </summary>
	/// <typeparam name="TKey">Key for the cache</typeparam>
	/// <typeparam name="TValue">Value for the cache</typeparam>
	public sealed class LazyCache<TKey, TValue> : IDisposable where TKey : notnull
	{
		class Item
		{
			public Task<TValue>? _currentTask;
			public Stopwatch _timer = Stopwatch.StartNew();
			public Task<TValue>? _updateTask;
		}

		readonly ConcurrentDictionary<TKey, Item> _dictionary = new ConcurrentDictionary<TKey, Item>();
		readonly Func<TKey, Task<TValue>> _getValueAsync;
		readonly LazyCacheOptions _options;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="getValueAsync">Function used to get a value</param>
		/// <param name="options"></param>
		public LazyCache(Func<TKey, Task<TValue>> getValueAsync, LazyCacheOptions options)
		{
			_getValueAsync = getValueAsync;
			_options = options;
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			foreach (Item item in _dictionary.Values)
			{
				item._currentTask?.Wait();
				item._updateTask?.Wait();
			}
		}

		/// <summary>
		/// Gets the value associated with a key
		/// </summary>
		/// <param name="key">The key to query</param>
		/// <param name="maxAge">Maximum age for values to return</param>
		/// <returns></returns>
		public Task<TValue> GetAsync(TKey key, TimeSpan? maxAge = null)
		{
			Item item = _dictionary.GetOrAdd(key, key => new Item());

			// Create the task to get the current value
			Task<TValue> currentTask = InterlockedCreateTaskAsync(ref item._currentTask, () => _getValueAsync(key));

			// If an update has completed, swap it out
			Task<TValue>? updateTask = item._updateTask;
			if (updateTask != null && updateTask.IsCompleted)
			{
				Interlocked.CompareExchange(ref item._currentTask, updateTask, currentTask);
				Interlocked.CompareExchange(ref item._updateTask, null, updateTask);
				item._timer.Restart();
			}

			// Check if we need to update the value
			TimeSpan age = item._timer.Elapsed;
			if (maxAge != null && age > maxAge.Value)
			{
				return InterlockedCreateTaskAsync(ref item._updateTask, () => _getValueAsync(key));
			}
			if (age > _options.RefreshTime)
			{
				InterlockedCreateTaskAsync(ref item._updateTask, () => _getValueAsync(key));
			}

			return currentTask;
		}

		/// <summary>
		/// Creates a task, guaranteeing that only one task will be assigned to the given slot. Creates a cold task and only starts it once the variable is set.
		/// </summary>
		/// <param name="value"></param>
		/// <param name="createTask"></param>
		/// <returns></returns>
		static Task<TValue> InterlockedCreateTaskAsync(ref Task<TValue>? value, Func<Task<TValue>> createTask)
		{
			Task<TValue>? currentTask = value;
			while (currentTask == null)
			{
				Task<Task<TValue>> newTask = new Task<Task<TValue>>(createTask);
				if (Interlocked.CompareExchange(ref value, newTask.Unwrap(), null) == null)
				{
					newTask.Start();
				}
				currentTask = value;
			}
			return currentTask;
		}
	}
}
