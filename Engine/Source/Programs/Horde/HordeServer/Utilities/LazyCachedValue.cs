// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Stores a value that expires after a given time
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class LazyCachedValue<T> where T : class
	{
		/// <summary>
		/// The current value
		/// </summary>
		T? _value;

		/// <summary>
		/// Generator for the new value
		/// </summary>
		readonly Func<T> _generator;

		/// <summary>
		/// Time since the value was updated
		/// </summary>
		readonly Stopwatch _timer = Stopwatch.StartNew();

		/// <summary>
		/// Default expiry time
		/// </summary>
		readonly TimeSpan _defaultMaxAge;

		/// <summary>
		/// Default constructor
		/// </summary>
		public LazyCachedValue(Func<T> generator, TimeSpan maxAge)
		{
			_generator = generator;
			_defaultMaxAge = maxAge;
		}

		/// <summary>
		/// Sets the new value
		/// </summary>
		/// <param name="value">The value to store</param>
		public void Set(T value)
		{
			_value = value;
			_timer.Restart();
		}

		/// <summary>
		/// Tries to get the current value
		/// </summary>
		/// <returns>The cached value, if valid</returns>
		public T GetCached()
		{
			return GetCached(_defaultMaxAge);
		}

		/// <summary>
		/// Tries to get the current value
		/// </summary>
		/// <returns>The cached value, if valid</returns>
		public T GetCached(TimeSpan maxAge)
		{
			T? current = _value;
			if (current == null || _timer.Elapsed > maxAge)
			{
				current = _generator();
				Set(current);
			}
			return current;
		}

		/// <summary>
		/// Gets the latest value, updating the cache
		/// </summary>
		/// <returns>The latest value</returns>
		public T GetLatest()
		{
			T newValue = _generator();
			Set(newValue);
			return newValue;
		}
	}
}
