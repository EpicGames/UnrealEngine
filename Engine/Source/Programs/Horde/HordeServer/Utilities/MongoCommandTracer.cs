// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Driver.Core.Configuration;
using MongoDB.Driver.Core.Events;
using OpenTelemetry.Trace;

namespace HordeServer.Utilities;

/// <summary>
/// OpenTelemetry-based tracer listening for MongoDB command events
/// </summary>
public class MongoCommandTracer
{
	private readonly ConcurrentDictionary<int, TelemetrySpan> _requestToSpans = new();
	private readonly ConcurrentDictionary<long, OperationSpanEntry> _operationToSpans = new();
	private readonly Dictionary<string, MongoCommand> _strToCommand = new();
	private readonly Tracer _tracer;
	private readonly ILogger<MongoCommandTracer> _logger;
	private readonly ConcurrentDictionary<string, byte> _unhandledCommands = new();

	private record OperationSpanEntry(TelemetrySpan Span, int RequestId);
	private record MongoCommand(MongoCommandType Type, string Name, string? StatementFieldName, string? CollectionFieldName = null);

	private enum MongoCommandType
	{
		Aggregate,
		CreateIndexes,
		Delete,
		Distinct,
		Find,
		FindAndModify,
		GetLastError,
		GetMore,
		Insert,
		IsMaster,
		KillCursors,
		ListIndexes,
		Ping,
		SaslContinue,
		SaslStart,
		Update,
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="tracer">Tracer</param>
	/// <param name="logger">Logger</param>
	public MongoCommandTracer(Tracer tracer, ILogger<MongoCommandTracer> logger)
	{
		_tracer = tracer;
		_logger = logger;

		List<MongoCommand> commands = [
			new MongoCommand(MongoCommandType.Aggregate, "aggregate", "aggregate"),
			new MongoCommand(MongoCommandType.CreateIndexes, "createIndexes", "createIndexes"),
			new MongoCommand(MongoCommandType.Delete, "delete", "deletes", "delete"),
			new MongoCommand(MongoCommandType.Distinct, "distinct", null),
			new MongoCommand(MongoCommandType.Find, "find", "filter", "find"),
			new MongoCommand(MongoCommandType.FindAndModify, "findAndModify", "query", "findAndModify"),
			new MongoCommand(MongoCommandType.GetLastError, "getLastError", null),
			new MongoCommand(MongoCommandType.GetMore, "getMore", null, "collection"),
			new MongoCommand(MongoCommandType.Insert, "insert", null, "insert"),
			new MongoCommand(MongoCommandType.IsMaster, "isMaster", null),
			new MongoCommand(MongoCommandType.KillCursors, "killCursors", null),
			new MongoCommand(MongoCommandType.ListIndexes, "listIndexes", "listIndexes"),
			new MongoCommand(MongoCommandType.Ping, "ping", null),
			new MongoCommand(MongoCommandType.SaslContinue, "saslContinue", null),
			new MongoCommand(MongoCommandType.SaslStart, "saslStart", null),
			new MongoCommand(MongoCommandType.Update, "update", "updates", "update"),
		];

		foreach (MongoCommand cmd in commands)
		{
			_strToCommand[cmd.Name] = cmd;
		}
	}

	/// <summary>
	/// Registers event listeners from MongoDB's client
	/// </summary>
	/// <param name="clusterBuilder">A MongoDB cluster builder</param>
	public void Register(ClusterBuilder clusterBuilder)
	{
		clusterBuilder.Subscribe<CommandStartedEvent>(OnEvent);
		clusterBuilder.Subscribe<CommandSucceededEvent>(OnEvent);
		clusterBuilder.Subscribe<CommandFailedEvent>(OnEvent);
	}
	
	internal IReadOnlySet<TelemetrySpan> GetSpans()
	{
		HashSet<TelemetrySpan> spans = [.._requestToSpans.Values];
		return spans.Union(_operationToSpans.Values.Select(x => x.Span)).ToHashSet();
	}

	internal TelemetrySpan GetSpanByRequestId(int requestId)
	{
		return _requestToSpans[requestId];
	}
	
	internal TelemetrySpan GetSpanByOperationId(long operationId)
	{
		return _operationToSpans[operationId].Span;
	}
	
	internal void OnEvent(CommandStartedEvent ev)
	{
		string? GetString(string fieldName)
		{
			return ev.Command.TryGetValue(fieldName, out BsonValue value) ? value.ToString() : null;
		}
		
		try
		{
			MongoCommand? command = ResolveCommand(ev.CommandName);
			if (command == null)
			{
				return;
			}

			string name = command.Name;
			string? collectionName = GetString(command.CollectionFieldName ?? command.Name);
			string? statement = command.StatementFieldName != null ? GetString(command.StatementFieldName) : null;
			
			if (collectionName != null)
			{
				name = $"{collectionName}.{command.Name}";
			}

			OperationSpanEntry? parentSpanEntry = null;
			if (command.Type == MongoCommandType.GetMore && ev.OperationId != null)
			{
				_operationToSpans.TryGetValue(ev.OperationId.Value, out parentSpanEntry);
			}
			
			SpanAttributes sa = new();
			sa.Add("type", "db");
			sa.Add("operation.name", name);
			sa.Add("service.name", OpenTelemetryTracers.MongoDbName);
			
			// OpenTelemetry MongoDB conventions https://opentelemetry.io/docs/specs/semconv/database/mongodb/
			sa.Add("db.system", "mongodb");
			sa.Add("db.name", ev.DatabaseNamespace.ToString());
			sa.Add("db.operationId", ev.OperationId ?? -1);
			sa.Add("db.requestId", ev.RequestId);
			sa.Add("db.serviceId", ev.ServiceId?.ToString());
			sa.Add("db.endpoint", ev.ConnectionId?.ServerId.EndPoint.ToString());
			
			TelemetrySpan span = _tracer.StartActiveSpan(name, SpanKind.Client, parentContext: parentSpanEntry?.Span.Context ?? Tracer.CurrentSpan.Context, sa);
			
			if (collectionName != null)
			{
				span.SetAttribute("db.mongodb.collection", collectionName);
			}
			
			if (statement != null)
			{
				span.SetAttribute("db.statement", statement.Length > 200 ? statement[..200] : statement);
			}

			_requestToSpans.TryAdd(ev.RequestId, span);
			if (command.Type == MongoCommandType.Find && ev.OperationId != null)
			{
				_operationToSpans.TryAdd(ev.OperationId.Value, new OperationSpanEntry(span, ev.RequestId));
			}
		}
		catch (Exception e)
		{
			_logger.LogError(e, "Unhandled exception when capturing MongoDB span");
		}
	}
	
	internal void OnEvent(CommandSucceededEvent ev)
	{
		MongoCommand? command = ResolveCommand(ev.CommandName);
		if (command == null)
		{
			return;
		}

		long? waitedMs = null;
		if (command.Type is MongoCommandType.Find or MongoCommandType.GetMore && ev.OperationId != null)
		{
			waitedMs = GetBsonDocumentLong(ev.Reply, "waitedMS");
			long? cursorId = GetCursorId(ev.Reply);
			bool hasMoreDocuments = cursorId is > 0;
			if (command.Type == MongoCommandType.Find)
			{
				if (hasMoreDocuments)
				{
					// Don't end the span for "find", keep it open and wait for subsequent "getMore" reply
					return;
				}
				else
				{
					// "find" command did not have any more documents
					EndSpan(ev.RequestId, ev.OperationId, Status.Ok, ev.Duration, waitedMs: waitedMs);
				}
			}

			if (command.Type == MongoCommandType.GetMore)
			{
				// Always close the "getMore" span
				EndSpan(ev.RequestId, null, Status.Ok, ev.Duration, waitedMs: waitedMs);
				
				if (!hasMoreDocuments)
				{
					if (_operationToSpans.TryGetValue(ev.OperationId.Value, out OperationSpanEntry? opSpanEntry))
					{
						// End span for entire operation
						EndSpan(opSpanEntry.RequestId, ev.OperationId.Value, Status.Ok);
					}
				}
			}
		}
		else
		{
			EndSpan(ev.RequestId, ev.OperationId, Status.Ok, ev.Duration, waitedMs: waitedMs);
		}
	}
	
	internal void OnEvent(CommandFailedEvent ev)
	{
		EndSpan(ev.RequestId, ev.OperationId, Status.Ok, ev.Duration, ev.Failure);
	}

	private MongoCommand? ResolveCommand(string name)
	{
		if (_strToCommand.TryGetValue(name, out MongoCommand? command))
		{
			return command;
		}

		// Ensure logging is done once per command by storing which type has been logged
		if (_unhandledCommands.TryAdd(name, 0))
		{
			_logger.LogInformation("Trace handling for MongoDB command {Command} is not implemented. Ignoring", name);
		}

		return null;
	}
	
	private void EndSpan(int reqId, long? opId, Status status, TimeSpan? duration = null, Exception? exception = null, long? waitedMs = null)
	{
		if (_requestToSpans.TryRemove(reqId, out TelemetrySpan? span))
		{
			span.SetStatus(status);
			
			if (duration != null)
			{
				span.SetAttribute("db.durationMs", duration.Value.TotalMilliseconds);
			}
			
			if (exception != null)
			{
				span.RecordException(exception);
			}
			
			if (waitedMs != null)
			{
				span.SetAttribute("db.waitedMs", waitedMs.Value);
			}
			
			span.End();
			span.Dispose();
		}

		if (opId != null)
		{
			_operationToSpans.TryRemove(opId.Value, out OperationSpanEntry? _);
		}
	}
	
	private static long? GetBsonDocumentLong(BsonDocument document, string key)
	{
		return document.TryGetPropertyValue(key, BsonType.Int64, out BsonValue? value) ? value.AsInt64 : null;
	}
	
	private static long? GetCursorId(BsonDocument replyDocument)
	{
		return replyDocument.TryGetPropertyValue("cursor", BsonType.Document, out BsonValue? cursorDoc)
			? GetBsonDocumentLong(cursorDoc.AsBsonDocument, "id")
			: null;
	}
}
