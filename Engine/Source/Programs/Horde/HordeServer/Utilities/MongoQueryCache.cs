// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using Microsoft.Extensions.Caching.Memory;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Driver;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Caches queries against a given collection
	/// </summary>
	/// <typeparam name="TDocument">The document type</typeparam>
	class MongoQueryCache<TDocument> : IDisposable
	{
		class QueryState
		{
			public Stopwatch _timer = Stopwatch.StartNew();
			public List<TDocument>? _results;
			public Task? _queryTask;
		}

		readonly IMongoCollection<TDocument> _collection;
		readonly MemoryCache _cache;
		readonly TimeSpan _maxLatency;

		public MongoQueryCache(IMongoCollection<TDocument> collection, TimeSpan maxLatency)
		{
			_collection = collection;

			MemoryCacheOptions options = new MemoryCacheOptions();
			_cache = new MemoryCache(options);

			_maxLatency = maxLatency;
		}

		public void Dispose()
		{
			_cache.Dispose();
		}

		async Task RefreshAsync(QueryState state, FilterDefinition<TDocument> filter, CancellationToken cancellationToken = default)
		{
			state._results = await _collection.Find(filter).ToListAsync(cancellationToken);
			state._timer.Restart();
		}

		public async Task<List<TDocument>> FindAsync(FilterDefinition<TDocument> filter, int index, int count)
		{
			BsonDocument rendered = filter.Render(BsonSerializer.LookupSerializer<TDocument>(), BsonSerializer.SerializerRegistry, MongoDB.Driver.Linq.LinqProvider.V2);
			BsonDocument document = new BsonDocument { new BsonElement("filter", rendered), new BsonElement("index", index), new BsonElement("count", count) };

			string filterKey = document.ToString();

			QueryState? state;
			if (!_cache.TryGetValue(filterKey, out state) || state == null)
			{
				state = new QueryState();
				using (ICacheEntry cacheEntry = _cache.CreateEntry(filterKey))
				{
					cacheEntry.SetSlidingExpiration(TimeSpan.FromMinutes(5.0));
					cacheEntry.SetValue(state);
				}
			}

			if (state._queryTask != null && state._queryTask.IsCompleted)
			{
				await state._queryTask;
				state._queryTask = null;
			}

			if (state._queryTask == null && (state._results == null || state._timer.Elapsed > _maxLatency))
			{
				state._queryTask = Task.Run(() => RefreshAsync(state, filter, CancellationToken.None), CancellationToken.None);
			}
			if (state._queryTask != null && (state._results == null || state._timer.Elapsed > _maxLatency))
			{
				await state._queryTask;
			}

			return state._results!;
		}
	}
}
