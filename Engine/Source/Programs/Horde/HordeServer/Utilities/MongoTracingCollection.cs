// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading;
using System.Threading.Tasks;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Driver;
using MongoDB.Driver.Search;
using OpenTelemetry.Trace;

#pragma warning disable VSTHRD002 // Synchronous waits 

namespace HordeServer.Utilities
{
	/// <summary>
	/// Wrap a IMongoCollection with trace scopes
	///
	/// Will capture the entire invocation of MongoDB calls, including serialization.
	/// The other command logging (not in this file) only deals with queries sent to the server at the protocol level.
	/// </summary>
	/// <typeparam name="T">A MongoDB document</typeparam>
	public class MongoTracingCollection<T> : IMongoCollection<T>
	{
		private readonly IMongoCollection<T> _collection;
		private readonly Task _upgradeTask;
		private readonly Tracer _tracer;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="collection">Collection to wrap with tracing</param>
		/// <param name="indexCreationTask">Task which creates indexes</param>
		/// <param name="tracer">Tracer</param>
		public MongoTracingCollection(IMongoCollection<T> collection, Task indexCreationTask, Tracer tracer)
		{
			_collection = collection;
			_upgradeTask = indexCreationTask;
			_tracer = tracer;
		}

#pragma warning disable CS0618
		/// <inheritdoc />
		public IAsyncCursor<TResult> Aggregate<TResult>(PipelineDefinition<T, TResult> pipeline, AggregateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Aggregate(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TResult> Aggregate<TResult>(IClientSessionHandle session, PipelineDefinition<T, TResult> pipeline, AggregateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Aggregate(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TResult>> AggregateAsync<TResult>(PipelineDefinition<T, TResult> pipeline, AggregateOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(AggregateAsync), _collection);
			return await _collection.AggregateAsync(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TResult>> AggregateAsync<TResult>(IClientSessionHandle session, PipelineDefinition<T, TResult> pipeline, AggregateOptions options = null!,
			CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(AggregateAsync), _collection);
			return await _collection.AggregateAsync(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public void AggregateToCollection<TResult>(PipelineDefinition<T, TResult> pipeline, AggregateOptions? options = null, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.AggregateToCollection(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public void AggregateToCollection<TResult>(IClientSessionHandle session, PipelineDefinition<T, TResult> pipeline, AggregateOptions? options = null, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.AggregateToCollection(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task AggregateToCollectionAsync<TResult>(PipelineDefinition<T, TResult> pipeline, AggregateOptions? options = null, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(AggregateToCollectionAsync), _collection);
			await _collection.AggregateToCollectionAsync(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task AggregateToCollectionAsync<TResult>(IClientSessionHandle session, PipelineDefinition<T, TResult> pipeline, AggregateOptions? options = null, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(AggregateToCollectionAsync), _collection);
			await _collection.AggregateToCollectionAsync(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public BulkWriteResult<T> BulkWrite(IEnumerable<WriteModel<T>> requests, BulkWriteOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.BulkWrite(requests, options, cancellationToken);
		}

		/// <inheritdoc />
		public BulkWriteResult<T> BulkWrite(IClientSessionHandle session, IEnumerable<WriteModel<T>> requests, BulkWriteOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.BulkWrite(session, requests, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<BulkWriteResult<T>> BulkWriteAsync(IEnumerable<WriteModel<T>> requests, BulkWriteOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(BulkWriteAsync), _collection);
			return await _collection.BulkWriteAsync(requests, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<BulkWriteResult<T>> BulkWriteAsync(IClientSessionHandle session, IEnumerable<WriteModel<T>> requests, BulkWriteOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(BulkWriteAsync), _collection);
			return await _collection.BulkWriteAsync(session, requests, options, cancellationToken);
		}

		/// <inheritdoc />
		public long Count(FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Count(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public long Count(IClientSessionHandle session, FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Count(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<long> CountAsync(FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(CountAsync), _collection);
			return await _collection.CountAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<long> CountAsync(IClientSessionHandle session, FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(CountAsync), _collection);
			return await _collection.CountAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public long CountDocuments(FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.CountDocuments(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public long CountDocuments(IClientSessionHandle session, FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.CountDocuments(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<long> CountDocumentsAsync(FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask;
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(CountDocumentsAsync), _collection, filter);
			return await _collection.CountDocumentsAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<long> CountDocumentsAsync(IClientSessionHandle session, FilterDefinition<T> filter, CountOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(CountDocumentsAsync), _collection, filter);
			return await _collection.CountDocumentsAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteMany(FilterDefinition<T> filter, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteMany(filter, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteMany(FilterDefinition<T> filter, DeleteOptions options, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteMany(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteMany(IClientSessionHandle session, FilterDefinition<T> filter, DeleteOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteMany(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteManyAsync(FilterDefinition<T> filter, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteManyAsync), _collection, filter);
			return await _collection.DeleteManyAsync(filter, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteManyAsync(FilterDefinition<T> filter, DeleteOptions options, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteManyAsync), _collection, filter);
			return await _collection.DeleteManyAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteManyAsync(IClientSessionHandle session, FilterDefinition<T> filter, DeleteOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask;
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteManyAsync), _collection, filter);
			return await _collection.DeleteManyAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteOne(FilterDefinition<T> filter, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteOne(filter, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteOne(FilterDefinition<T> filter, DeleteOptions options, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteOne(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public DeleteResult DeleteOne(IClientSessionHandle session, FilterDefinition<T> filter, DeleteOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.DeleteOne(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteOneAsync(FilterDefinition<T> filter, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteOneAsync), _collection, filter);
			return await _collection.DeleteOneAsync(filter, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteOneAsync(FilterDefinition<T> filter, DeleteOptions options, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteOneAsync), _collection, filter);
			return await _collection.DeleteOneAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<DeleteResult> DeleteOneAsync(IClientSessionHandle session, FilterDefinition<T> filter, DeleteOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DeleteOneAsync), _collection, filter);
			return await _collection.DeleteOneAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TField> Distinct<TField>(FieldDefinition<T, TField> field, FilterDefinition<T> filter, DistinctOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Distinct(field, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TField> Distinct<TField>(IClientSessionHandle session, FieldDefinition<T, TField> field, FilterDefinition<T> filter, DistinctOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Distinct(session, field, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TField>> DistinctAsync<TField>(FieldDefinition<T, TField> field, FilterDefinition<T> filter, DistinctOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DistinctAsync), _collection, filter);
			return await _collection.DistinctAsync(field, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TField>> DistinctAsync<TField>(IClientSessionHandle session, FieldDefinition<T, TField> field, FilterDefinition<T> filter, DistinctOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(DistinctAsync), _collection, filter);
			return await _collection.DistinctAsync(session, field, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public long EstimatedDocumentCount(EstimatedDocumentCountOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.EstimatedDocumentCount(options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<long> EstimatedDocumentCountAsync(EstimatedDocumentCountOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(EstimatedDocumentCountAsync), _collection);
			return await _collection.EstimatedDocumentCountAsync(options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TProjection> FindSync<TProjection>(FilterDefinition<T> filter, FindOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindSync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TProjection> FindSync<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, FindOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindSync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TProjection>> FindAsync<TProjection>(FilterDefinition<T> filter, FindOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindAsync), _collection, filter);
			return await _collection.FindAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TProjection>> FindAsync<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, FindOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindAsync), _collection, filter);
			return await _collection.FindAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndDelete<TProjection>(FilterDefinition<T> filter, FindOneAndDeleteOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndDelete(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndDelete<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, FindOneAndDeleteOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndDelete(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndDeleteAsync<TProjection>(FilterDefinition<T> filter, FindOneAndDeleteOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndDeleteAsync), _collection, filter);
			return await _collection.FindOneAndDeleteAsync(filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndDeleteAsync<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, FindOneAndDeleteOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndDeleteAsync), _collection, filter);
			return await _collection.FindOneAndDeleteAsync(session, filter, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndReplace<TProjection>(FilterDefinition<T> filter, T replacement, FindOneAndReplaceOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndReplace(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndReplace<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, FindOneAndReplaceOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndReplace(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndReplaceAsync<TProjection>(FilterDefinition<T> filter, T replacement, FindOneAndReplaceOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndReplaceAsync), _collection, filter, document: replacement);
			return await _collection.FindOneAndReplaceAsync(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndReplaceAsync<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, FindOneAndReplaceOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndReplaceAsync), _collection, filter, document: replacement);
			return await _collection.FindOneAndReplaceAsync(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndUpdate<TProjection>(FilterDefinition<T> filter, UpdateDefinition<T> update,
			FindOneAndUpdateOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndUpdate(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public TProjection FindOneAndUpdate<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, FindOneAndUpdateOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.FindOneAndUpdate(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndUpdateAsync<TProjection>(FilterDefinition<T> filter, UpdateDefinition<T> update, FindOneAndUpdateOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndUpdateAsync), _collection, filter, update);
			return await _collection.FindOneAndUpdateAsync(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<TProjection> FindOneAndUpdateAsync<TProjection>(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, FindOneAndUpdateOptions<T, TProjection> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(FindOneAndUpdateAsync), _collection, filter, update);
			return await _collection.FindOneAndUpdateAsync(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public void InsertOne(T document, InsertOneOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.InsertOne(document, options, cancellationToken);
		}

		/// <inheritdoc />
		public void InsertOne(IClientSessionHandle session, T document, InsertOneOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.InsertOne(session, document, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task InsertOneAsync(T document, CancellationToken cancellationToken)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(InsertOneAsync), _collection, document: document);
			await _collection.InsertOneAsync(document, cancellationToken);
		}

		/// <inheritdoc />
		public async Task InsertOneAsync(T document, InsertOneOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(InsertOneAsync), _collection, document: document);
			await _collection.InsertOneAsync(document, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task InsertOneAsync(IClientSessionHandle session, T document, InsertOneOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(InsertOneAsync), _collection, document: document);
			await _collection.InsertOneAsync(session, document, options, cancellationToken);
		}

		/// <inheritdoc />
		public void InsertMany(IEnumerable<T> documents, InsertManyOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.InsertMany(documents, options, cancellationToken);
		}

		/// <inheritdoc />
		public void InsertMany(IClientSessionHandle session, IEnumerable<T> documents, InsertManyOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			_collection.InsertMany(session, documents, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task InsertManyAsync(IEnumerable<T> documents, InsertManyOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(InsertManyAsync), _collection);
			await _collection.InsertManyAsync(documents, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task InsertManyAsync(IClientSessionHandle session, IEnumerable<T> documents, InsertManyOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(InsertManyAsync), _collection);
			await _collection.InsertManyAsync(session, documents, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TResult> MapReduce<TResult>(BsonJavaScript map, BsonJavaScript reduce, MapReduceOptions<T, TResult> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.MapReduce(map, reduce, options, cancellationToken);
		}

		/// <inheritdoc />
		public IAsyncCursor<TResult> MapReduce<TResult>(IClientSessionHandle session, BsonJavaScript map, BsonJavaScript reduce, MapReduceOptions<T, TResult> options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.MapReduce(session, map, reduce, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TResult>> MapReduceAsync<TResult>(BsonJavaScript map, BsonJavaScript reduce, MapReduceOptions<T, TResult> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(MapReduceAsync), _collection);
			return await _collection.MapReduceAsync(map, reduce, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IAsyncCursor<TResult>> MapReduceAsync<TResult>(IClientSessionHandle session, BsonJavaScript map, BsonJavaScript reduce, MapReduceOptions<T, TResult> options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(MapReduceAsync), _collection);
			return await _collection.MapReduceAsync(session, map, reduce, options, cancellationToken);
		}

		/// <inheritdoc />
		public IFilteredMongoCollection<TDerivedDocument> OfType<TDerivedDocument>() where TDerivedDocument : T
		{
			return _collection.OfType<TDerivedDocument>();
		}

		/// <inheritdoc />
		public ReplaceOneResult ReplaceOne(FilterDefinition<T> filter, T replacement, ReplaceOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.ReplaceOne(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public ReplaceOneResult ReplaceOne(FilterDefinition<T> filter, T replacement, UpdateOptions options, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.ReplaceOne(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public ReplaceOneResult ReplaceOne(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, ReplaceOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.ReplaceOne(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public ReplaceOneResult ReplaceOne(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, UpdateOptions options, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.ReplaceOne(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<ReplaceOneResult> ReplaceOneAsync(FilterDefinition<T> filter, T replacement, ReplaceOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(ReplaceOneAsync), _collection, filter, document: replacement);
			return await _collection.ReplaceOneAsync(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<ReplaceOneResult> ReplaceOneAsync(FilterDefinition<T> filter, T replacement, UpdateOptions options, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(ReplaceOneAsync), _collection, filter, document: replacement);
			return await _collection.ReplaceOneAsync(filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<ReplaceOneResult> ReplaceOneAsync(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, ReplaceOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(ReplaceOneAsync), _collection, filter, document: replacement);
			return await _collection.ReplaceOneAsync(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<ReplaceOneResult> ReplaceOneAsync(IClientSessionHandle session, FilterDefinition<T> filter, T replacement, UpdateOptions options, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(ReplaceOneAsync), _collection, filter, document: replacement);
			return await _collection.ReplaceOneAsync(session, filter, replacement, options, cancellationToken);
		}

		/// <inheritdoc />
		public UpdateResult UpdateMany(FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.UpdateMany(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public UpdateResult UpdateMany(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.UpdateMany(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<UpdateResult> UpdateManyAsync(FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(UpdateManyAsync), _collection, filter, update);
			return await _collection.UpdateManyAsync(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<UpdateResult> UpdateManyAsync(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(UpdateManyAsync), _collection);
			return await _collection.UpdateManyAsync(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public UpdateResult UpdateOne(FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.UpdateOne(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public UpdateResult UpdateOne(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.UpdateOne(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<UpdateResult> UpdateOneAsync(FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(UpdateOneAsync), _collection, filter, update);
			return await _collection.UpdateOneAsync(filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<UpdateResult> UpdateOneAsync(IClientSessionHandle session, FilterDefinition<T> filter, UpdateDefinition<T> update, UpdateOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(UpdateOneAsync), _collection, filter, update);
			return await _collection.UpdateOneAsync(session, filter, update, options, cancellationToken);
		}

		/// <inheritdoc />
		public IChangeStreamCursor<TResult> Watch<TResult>(PipelineDefinition<ChangeStreamDocument<T>, TResult> pipeline, ChangeStreamOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Watch(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public IChangeStreamCursor<TResult> Watch<TResult>(IClientSessionHandle session, PipelineDefinition<ChangeStreamDocument<T>, TResult> pipeline, ChangeStreamOptions options = null!, CancellationToken cancellationToken = default)
		{
			_upgradeTask.Wait(cancellationToken);
			return _collection.Watch(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IChangeStreamCursor<TResult>> WatchAsync<TResult>(PipelineDefinition<ChangeStreamDocument<T>, TResult> pipeline, ChangeStreamOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(WatchAsync), _collection);
			return await _collection.WatchAsync(pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public async Task<IChangeStreamCursor<TResult>> WatchAsync<TResult>(IClientSessionHandle session, PipelineDefinition<ChangeStreamDocument<T>, TResult> pipeline, ChangeStreamOptions options = null!, CancellationToken cancellationToken = default)
		{
			await _upgradeTask.WaitAsync(cancellationToken);
			using TelemetrySpan span = _tracer.StartMongoDbSpan(nameof(WatchAsync), _collection);
			return await _collection.WatchAsync(session, pipeline, options, cancellationToken);
		}

		/// <inheritdoc />
		public IMongoCollection<T> WithReadConcern(ReadConcern readConcern)
		{
			return _collection.WithReadConcern(readConcern);
		}

		/// <inheritdoc />
		public IMongoCollection<T> WithReadPreference(ReadPreference readPreference)
		{
			return _collection.WithReadPreference(readPreference);
		}

		/// <inheritdoc />
		public IMongoCollection<T> WithWriteConcern(WriteConcern writeConcern)
		{
			return _collection.WithWriteConcern(writeConcern);
		}
#pragma warning restore CS0618

		/// <inheritdoc />
		public CollectionNamespace CollectionNamespace => _collection.CollectionNamespace;

		/// <inheritdoc />
		public IMongoDatabase Database => _collection.Database;

		/// <inheritdoc />
		public IBsonSerializer<T> DocumentSerializer => _collection.DocumentSerializer;

		/// <inheritdoc />
		public IMongoIndexManager<T> Indexes => _collection.Indexes;

		/// <inheritdoc />
		public MongoCollectionSettings Settings => _collection.Settings;

		/// <inheritdoc />
		public IMongoSearchIndexManager SearchIndexes => _collection.SearchIndexes;
	}
}