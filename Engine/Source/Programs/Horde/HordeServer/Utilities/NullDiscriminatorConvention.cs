// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using MongoDB.Bson;
using MongoDB.Bson.IO;
using MongoDB.Bson.Serialization.Conventions;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Prevent a discriminator being serialized for a type
	/// </summary>
	public class NullDiscriminatorConvention : IDiscriminatorConvention
	{
		/// <summary>
		/// Instance of the convention
		/// </summary>
		public static NullDiscriminatorConvention Instance { get; } = new NullDiscriminatorConvention();

		/// <inheritdoc/>
		public Type GetActualType(IBsonReader bsonReader, Type nominalType)
		{
			return nominalType;
		}

		/// <inheritdoc/>
		public BsonValue? GetDiscriminator(Type nominalType, Type actualType)
		{
			return null;
		}

		/// <inheritdoc/>
		public string? ElementName => null;
	}
}
