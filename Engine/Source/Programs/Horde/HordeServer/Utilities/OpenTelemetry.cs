// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.Metrics;
using System.Linq;
using System.Net.Http;
using System.Runtime.CompilerServices;
using EpicGames.Horde.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Primitives;
using MongoDB.Bson;
using MongoDB.Driver;
using OpenTelemetry.Exporter;
using OpenTelemetry.Logs;
using OpenTelemetry.Metrics;
using OpenTelemetry.Resources;
using OpenTelemetry.Trace;
using Serilog.Core;

namespace HordeServer.Utilities;

/// <summary>
/// Serilog event enricher attaching trace and span ID for Datadog using current System.Diagnostics.Activity
/// </summary>
public class OpenTelemetryDatadogLogEnricher : ILogEventEnricher
{
	/// <inheritdoc />
	public void Enrich(Serilog.Events.LogEvent logEvent, ILogEventPropertyFactory propertyFactory)
	{
		if (Activity.Current != null)
		{
			string stringTraceId = Activity.Current.TraceId.ToString();
			string stringSpanId = Activity.Current.SpanId.ToString();
			string ddTraceId = Convert.ToUInt64(stringTraceId.Substring(16), 16).ToString();
			string ddSpanId = Convert.ToUInt64(stringSpanId, 16).ToString();

			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.trace_id", ddTraceId));
			logEvent.AddPropertyIfAbsent(propertyFactory.CreateProperty("dd.span_id", ddSpanId));
		}
	}
}

/// <summary>
/// Helper for configuring OpenTelemetry
/// </summary>
public static class OpenTelemetryHelper
{
	private static ResourceBuilder? s_resourceBuilder;

	/// <summary>
	/// Configure OpenTelemetry in Horde and ASP.NET
	/// </summary>
	/// <param name="services">Service collection for DI</param>
	/// <param name="settings">Current server settings</param>
	public static void Configure(IServiceCollection services, OpenTelemetrySettings settings)
	{
		// Always configure tracers/meters as they are used in the codebase even when OpenTelemetry is not configured
		services.AddSingleton(OpenTelemetryTracers.Horde);
		services.AddSingleton(OpenTelemetryMeters.Horde);

		if (!settings.Enabled)
		{
			return;
		}

		services.AddOpenTelemetry()
			.WithTracing(builder => ConfigureTracing(builder, settings))
			.WithMetrics(builder => ConfigureMetrics(builder, settings));
	}

	private static void ConfigureTracing(TracerProviderBuilder builder, OpenTelemetrySettings settings)
	{
		void DatadogHttpRequestEnricher(Activity activity, HttpRequestMessage message)
		{
			activity.SetTag("service.name", settings.ServiceName + "-http-client");
			activity.SetTag("operation.name", "http.request");
			string url = $"{message.Method} {message.Headers.Host}{message.RequestUri?.LocalPath}";
			activity.DisplayName = url;
			activity.SetTag("resource.name", url);
		}

		void DatadogAspNetRequestEnricher(Activity activity, HttpRequest request)
		{
			activity.DisplayName = $"{request.Method} {request.Headers.Host}{request.Path}";

			// From https://docs.datadoghq.com/standard-attributes/
			activity.SetTag("service.name", settings.ServiceName);
			activity.SetTag("operation.name", "http.request");

			// Resolve client's IP via headers or actual TCP/IP remote IP
			string? forwardedForHeader = request.HttpContext.Request.Headers["X-Forwarded-For"].FirstOrDefault();
			string? clientIp = String.IsNullOrEmpty(forwardedForHeader)
				? request.HttpContext.Connection.RemoteIpAddress?.ToString()
				: forwardedForHeader;
			activity.SetTag("http.client_ip", clientIp);

			// Header sent by the dashboard to indicate how long a user has been inactive for a particular browser page (in seconds)
			if (request.Headers.TryGetValue("X-Horde-LastUserActivity", out StringValues values))
			{
				string? value = values.FirstOrDefault();
				if (!String.IsNullOrEmpty(value) && Int32.TryParse(value, out int lastUserActivity))
				{
					activity.SetTag("horde.lastUserActivity", lastUserActivity);
				}
			}
		}

		void DatadogAspNetResponseEnricher(Activity activity, HttpResponse response)
		{
			// The request hook above is executed too early in the middleware chain so user related information must be read here
			// HttpContext can only be accessed after the response has been sent. Both of these tags are standard Datadog attributes.
			activity.SetTag("usr.id", response.HttpContext.User.GetUserId());
			activity.SetTag("usr.name", response.HttpContext.User.GetUser());
		}

		bool FilterHttpRequests(HttpContext context)
		{
			if (context.Request.Path.Value != null && context.Request.Path.Value.Contains("health/", StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}

			return true;
		}

		builder
			.AddSource(OpenTelemetryTracers.SourceNames)
			.AddHttpClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			.AddAspNetCoreInstrumentation(options =>
			{
				options.Filter = FilterHttpRequests;

				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequest = DatadogAspNetRequestEnricher;
					options.EnrichWithHttpResponse = DatadogAspNetResponseEnricher;
				}
			})
			.AddGrpcClientInstrumentation(options =>
			{
				if (settings.EnableDatadogCompatibility)
				{
					options.EnrichWithHttpRequestMessage = DatadogHttpRequestEnricher;
				}
			})
			//.AddRedisInstrumentation()
			.SetResourceBuilder(GetResourceBuilder(settings));

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	private static void ConfigureMetrics(MeterProviderBuilder builder, OpenTelemetrySettings settings)
	{
		builder.AddMeter(OpenTelemetryMeters.MeterNames);
		builder.AddAspNetCoreInstrumentation();
		builder.AddHttpClientInstrumentation();

		if (settings.EnableConsoleExporter)
		{
			builder.AddConsoleExporter();
		}

		foreach ((string name, OpenTelemetryProtocolExporterSettings exporterSettings) in settings.ProtocolExporters)
		{
			builder.AddOtlpExporter(name, exporter =>
			{
				exporter.Endpoint = exporterSettings.Endpoint!;
				exporter.Protocol = Enum.TryParse(exporterSettings.Protocol, true, out OtlpExportProtocol protocol) ? protocol : OtlpExportProtocol.Grpc;
			});
		}
	}

	/// <summary>
	/// Configure .NET logging with OpenTelemetry
	/// </summary>
	/// <param name="builder">Logging builder to modify</param>
	/// <param name="settings">Current server settings</param>
	public static void ConfigureLogging(ILoggingBuilder builder, OpenTelemetrySettings settings)
	{
		if (!settings.Enabled)
		{
			return;
		}

		builder.AddOpenTelemetry(options =>
		{
			options.IncludeScopes = true;
			options.IncludeFormattedMessage = true;
			options.ParseStateValues = true;
			options.SetResourceBuilder(GetResourceBuilder(settings));

			if (settings.EnableConsoleExporter)
			{
				options.AddConsoleExporter();
			}
		});
	}

	private static ResourceBuilder GetResourceBuilder(OpenTelemetrySettings settings)
	{
		if (s_resourceBuilder != null)
		{
			return s_resourceBuilder;
		}

		List<KeyValuePair<string, object>> attributes = settings.Attributes.Select(x => new KeyValuePair<string, object>(x.Key, x.Value)).ToList();
		s_resourceBuilder = ResourceBuilder.CreateDefault()
			.AddService(settings.ServiceName, serviceNamespace: settings.ServiceNamespace, serviceVersion: settings.ServiceVersion)
			.AddAttributes(attributes)
			.AddTelemetrySdk()
			.AddEnvironmentVariableDetector();

		return s_resourceBuilder;
	}
}

/// <summary>
/// Extensions to handle Horde specific data types in the OpenTelemetry library
/// </summary>
public static class OpenTelemetrySpanExtensions
{
	/// <summary>
	/// Start a MongoDB-based tracing span
	/// </summary>
	/// <param name="tracer">Current tracer being extended</param>
	/// <param name="spanName">Name of the span</param>
	/// <param name="collection">An optional MongoDB collection, the name will be used as an attribute</param>
	/// <param name="filter">Optional filter to attach to the trace</param>
	/// <param name="update">Optional update to attach to the trace</param>
	/// <param name="document">Document in the parmaeter</param>
	/// <returns>A new telemetry span</returns>
	[MethodImpl(MethodImplOptions.AggressiveInlining)]
	public static TelemetrySpan StartMongoDbSpan<T>(this Tracer tracer, string spanName, IMongoCollection<T>? collection = null, FilterDefinition<T>? filter = null, UpdateDefinition<T>? update = null, T? document = default)
	{
		_ = tracer;

		TelemetrySpan span = OpenTelemetryTracers.MongoDb
			.StartActiveSpan(spanName, parentContext: Tracer.CurrentSpan.Context)
			.SetAttribute("type", "db")
			.SetAttribute("operation.name", spanName)
			.SetAttribute("service.name", OpenTelemetryTracers.MongoDbName);

		if (collection != null)
		{
			span.SetAttribute("collection", collection.CollectionNamespace.CollectionName);
		}
		if (filter != null)
		{
			span.SetAttribute("filter", filter.Render().ToJson());
		}
		if (update != null)
		{
			span.SetAttribute("update", update.Render().ToJson());
		}
		if (document != null)
		{
			span.SetAttribute("document", document.ToJson());
		}

		return span;
	}
}

/// <summary>
/// Static initialization of all available OpenTelemetry meters
/// </summary>
public static class OpenTelemetryMeters
{
	/// <summary>
	/// Name of default Horde meter
	/// </summary>
	public const string HordeName = "Horde";

	/// <summary>
	/// List of all source names configured in this class.
	/// They are needed at startup when initializing OpenTelemetry
	/// </summary>
	public static string[] MeterNames => new[] { HordeName };

	/// <summary>
	/// Default meter used in Horde
	/// </summary>
	public static readonly Meter Horde = new(HordeName);
}
