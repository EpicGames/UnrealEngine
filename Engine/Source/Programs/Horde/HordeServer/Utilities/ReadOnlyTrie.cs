// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using EpicGames.Core;

namespace HordeServer.Logs
{
	/// <summary>
	/// A read-only trie of long values
	/// </summary>
	public class ReadOnlyTrie : IEnumerable<ulong>
	{
		/// <summary>
		/// Stack item for traversing the tree
		/// </summary>
		[StructLayout(LayoutKind.Auto)]
		struct StackItem
		{
			/// <summary>
			/// The current node index
			/// </summary>
			public int _index;

			/// <summary>
			/// Value in the current node (0-15)
			/// </summary>
			public int _value;
		}

		/// <summary>
		/// Delegate for filtering values during a tree traversal
		/// </summary>
		/// <param name="value">The current value</param>
		/// <param name="mask">Mask for which bits in the value are valid</param>
		/// <returns>True if values matching the given mask should be enumerated</returns>
		public delegate bool VisitorDelegate(ulong value, ulong mask);

		/// <summary>
		/// Height of the tree
		/// </summary>
		const int Height = sizeof(ulong) * 2;

		/// <summary>
		/// Array of bitmasks for each node in the tree
		/// </summary>
		public ushort[] NodeData { get; }

		/// <summary>
		/// Array of child offsets for each node. Excludes the last layer of the tree.
		/// </summary>
		readonly int[] _firstChildIndex;

		/// <summary>
		/// Empty index definition
		/// </summary>
		public static ReadOnlyTrie Empty { get; } = new ReadOnlyTrie(new ushort[1]);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="nodeData">Node data</param>
		public ReadOnlyTrie(ushort[] nodeData)
		{
			NodeData = nodeData;
			_firstChildIndex = CreateChildLookup(nodeData);
		}

		/// <summary>
		/// Tests whether the given value is in the trie
		/// </summary>
		/// <param name="value">The value to check for</param>
		/// <returns>True if the value is in the trie</returns>
		public bool Contains(ulong value)
		{
			int index = 0;
			for (int shift = (sizeof(ulong) * 8) - 4; shift >= 0; shift -= 4)
			{
				int mask = NodeData[index];
				int flag = 1 << (int)((value >> shift) & 15);
				if ((mask & flag) == 0)
				{
					return false;
				}

				index = _firstChildIndex[index];
				for (; ; )
				{
					mask &= (mask - 1);
					if ((mask & flag) == 0)
					{
						break;
					}
					index++;
				}
			}
			return true;
		}

		/// <inheritdoc/>
		public IEnumerator<ulong> GetEnumerator()
		{
			return EnumerateRange(UInt64.MinValue, UInt64.MaxValue).GetEnumerator();
		}

		/// <inheritdoc/>
		IEnumerator IEnumerable.GetEnumerator() => GetEnumerator();

		/// <summary>
		/// Enumerate all values matching a given filter
		/// </summary>
		/// <param name="predicate">Predicate for which values to include</param>
		/// <returns>Values satisfying the given predicate</returns>
		public IEnumerable<ulong> EnumerateValues(VisitorDelegate predicate)
		{
			int depth = 0;
			ulong value = 0;

			StackItem[] stack = new StackItem[Height];
			stack[1]._index = _firstChildIndex[0];

			for (; ; )
			{
				StackItem current = stack[depth];
				if (current._value >= 16)
				{
					// Move up the tree if we've enumerated all the branches at the current level
					depth--;
					if (depth < 0)
					{
						yield break;
					}
					stack[depth]._value++;

					// Increment the child index too. These are stored sequentially for a given parent. The value will be cleared when we recurse into it.
					stack[depth + 1]._index++;
				}
				else if ((NodeData[current._index] & (1 << current._value)) == 0)
				{
					// This branch does not exist. Skip it.
					stack[depth]._value++;
				}
				else
				{
					// Get the value and mask for the current node
					int shift = (stack.Length - depth - 1) * 4;
					ulong mask = ~((1UL << shift) - 1);
					value = (value & (mask << 4)) | ((ulong)(uint)current._value << shift);

					if (!predicate(value, mask))
					{
						// This node is excluded, skip it
						stack[depth]._value++;
						if (depth + 1 < stack.Length)
						{
							stack[depth + 1]._index++;
						}
					}
					else if (depth + 1 < stack.Length)
					{
						// Move down the tree
						depth++;
						stack[depth]._value = 0;
						if (depth + 1 < stack.Length)
						{
							stack[depth + 1]._index = _firstChildIndex[stack[depth]._index];
						}
					}
					else
					{
						// Yield the current value
						yield return value;
						stack[depth]._value++;
					}
				}
			}
		}

		/// <summary>
		/// Enumerates all values in the trie between the given ranges
		/// </summary>
		/// <param name="minValue">Minimum value to enumerate</param>
		/// <param name="maxValue">Maximum value to enumerate</param>
		/// <returns>Sequence of values</returns>
		public IEnumerable<ulong> EnumerateRange(ulong minValue, ulong maxValue)
		{
			return EnumerateValues((value, mask) => (value >= (minValue & mask) && value <= (maxValue & mask)));
		}

		/// <summary>
		/// Creates a lookup for child node offsets from raw node data
		/// </summary>
		/// <param name="nodeData">Array of masks for each node</param>
		/// <returns>Array of offsets</returns>
		static int[] CreateChildLookup(ushort[] nodeData)
		{
			List<int> childOffsets = new List<int>();
			if (nodeData.Length > 0)
			{
				int nodeCount = 1;

				int index = 0;
				int childIndex = nodeCount;

				for (int level = 0; level < Height; level++)
				{
					int nextNodeCount = 0;
					for (int idx = 0; idx < nodeCount; idx++)
					{
						ushort node = nodeData[index++];

						int numChildren = CountBits(node);
						childOffsets.Add(childIndex);
						childIndex += numChildren;

						nextNodeCount += numChildren;
					}
					nodeCount = nextNodeCount;
				}
			}
			return childOffsets.ToArray();
		}

		/// <summary>
		/// Count the number of set bits in the given value
		/// </summary>
		/// <param name="value">Value to test</param>
		/// <returns>Number of set bits</returns>
		static int CountBits(ushort value)
		{
			int count = value;
			count = (count & 0b0101010101010101) + ((count >> 1) & 0b0101010101010101);
			count = (count & 0b0011001100110011) + ((count >> 2) & 0b0011001100110011);
			count = (count & 0b0000111100001111) + ((count >> 4) & 0b0000111100001111);
			count = (count & 0b0000000011111111) + ((count >> 8) & 0b0000000011111111);
			return count;
		}

		/// <summary>
		/// Read a trie from the given buffer
		/// </summary>
		/// <param name="reader">Reader to read from</param>
		/// <returns>New trie</returns>
		public static ReadOnlyTrie Read(MemoryReader reader)
		{
			ReadOnlyMemory<byte> nodes = reader.ReadVariableLengthBytesWithInt32Length();
			ushort[] nodeData = MemoryMarshal.Cast<byte, ushort>(nodes.Span).ToArray();
			return new ReadOnlyTrie(nodeData);
		}

		/// <summary>
		/// Write this trie to the given buffer
		/// </summary>
		/// <param name="writer">Writer to output to</param>
		public void Write(MemoryWriter writer)
		{
			writer.WriteVariableLengthBytesWithInt32Length(MemoryMarshal.AsBytes<ushort>(NodeData));
		}

		/// <summary>
		/// Gets the serialized size of this trie
		/// </summary>
		/// <returns></returns>
		public int GetSerializedSize()
		{
			return (sizeof(int) + NodeData.Length * sizeof(ushort));
		}
	}

	/// <summary>
	/// Extension methods for serializing tries
	/// </summary>
	public static class LogTokenSetExtensions
	{
		/// <summary>
		/// Read a trie from the given buffer
		/// </summary>
		/// <param name="reader">Reader to read from</param>
		/// <returns>New trie</returns>
		public static ReadOnlyTrie ReadTrie(this MemoryReader reader)
		{
			return ReadOnlyTrie.Read(reader);
		}

		/// <summary>
		/// Write this trie to the given buffer
		/// </summary>
		/// <param name="writer">Writer to output to</param>
		/// <param name="trie">Trie to write</param>
		public static void WriteTrie(this MemoryWriter writer, ReadOnlyTrie trie)
		{
			trie.Write(writer);
		}
	}
}
