// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Text.Json.Nodes;
using Serilog.Events;
using Serilog.Formatting;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Wraps an <see cref="ITextFormatter"/> and enforces a maximum byte size on the formatted output.
	/// Uses three-tier truncation:
	/// <list type="bullet">
	///   <item>Tier 1: output fits within the limit — pass through unchanged.</item>
	///   <item>Tier 2: strip redundant enricher properties (ExceptionInfo/ExceptionDetail) and retry.</item>
	///   <item>Tier 3: emit minimal JSON preserving RenderedMessage and full Exception stack trace.</item>
	/// </list>
	/// </summary>
	sealed class SizeLimitedJsonFormatter : ITextFormatter
	{
		static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions { Encoder = JavaScriptEncoder.UnsafeRelaxedJsonEscaping };

		const string ExceptionInfoProperty = "ExceptionInfo";
		const string ExceptionDetailProperty = "ExceptionDetail";

		readonly ITextFormatter _inner;
		readonly int _maxBytes;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inner">The inner formatter to delegate to</param>
		/// <param name="maxBytes">Maximum byte size before truncation (default 256 KB)</param>
		public SizeLimitedJsonFormatter(ITextFormatter inner, int maxBytes = 256 * 1024)
		{
			_inner = inner;
			_maxBytes = maxBytes;
		}

		/// <inheritdoc/>
		public void Format(LogEvent logEvent, TextWriter output)
		{
			using StringWriter buffer = new StringWriter();
			_inner.Format(logEvent, buffer);

			string formatted = buffer.ToString();

			int byteCount = System.Text.Encoding.UTF8.GetByteCount(formatted);
			if (byteCount <= _maxBytes)
			{
				// Tier 1: fits as-is
				output.Write(formatted);
				return;
			}

			// Tier 2: try stripping redundant enricher properties
			if (TryStripRedundantProperties(formatted, byteCount, out string? stripped))
			{
				output.Write(stripped);
				return;
			}

			// Tier 3: minimal fallback with RenderedMessage and Exception
			WriteTier3Fallback(logEvent, byteCount, output);
		}

		/// <summary>
		/// Attempts to strip ExceptionInfo and ExceptionDetail from the Properties object.
		/// Returns true if the stripped output fits within <see cref="_maxBytes"/>.
		/// </summary>
		bool TryStripRedundantProperties(string formatted, int originalByteCount, out string? result)
		{
			result = null;

			JsonNode? root;
			try
			{
				root = JsonNode.Parse(formatted);
			}
			catch (JsonException)
			{
				return false;
			}

			if (root is not JsonObject rootObj)
			{
				return false;
			}

			JsonNode? propertiesNode = rootObj["Properties"];
			if (propertiesNode is not JsonObject properties)
			{
				return false;
			}

			bool removed = false;
			removed |= properties.Remove(ExceptionInfoProperty);
			removed |= properties.Remove(ExceptionDetailProperty);

			if (!removed)
			{
				return false;
			}

			rootObj["_truncated"] = true;
			rootObj["_originalBytes"] = originalByteCount;

			string strippedJson = rootObj.ToJsonString(s_jsonOptions) + "\n";
			int strippedByteCount = System.Text.Encoding.UTF8.GetByteCount(strippedJson);
			if (strippedByteCount > _maxBytes)
			{
				return false;
			}

			result = strippedJson;
			return true;
		}

		/// <summary>
		/// Emits a minimal JSON object preserving RenderedMessage and the full Exception stack trace.
		/// </summary>
		static void WriteTier3Fallback(LogEvent logEvent, int originalByteCount, TextWriter output)
		{
			string timestamp = logEvent.Timestamp.UtcDateTime.ToString("O");
			string level = logEvent.Level.ToString();
			string messageTemplate = EscapeJson(logEvent.MessageTemplate.Text);
			string renderedMessage = EscapeJson(logEvent.RenderMessage());

			output.Write($"{{\"Timestamp\":\"{timestamp}\",\"Level\":\"{level}\",\"MessageTemplate\":\"{messageTemplate}\",\"RenderedMessage\":\"{renderedMessage}\"");
			if (logEvent.Exception != null)
			{
				string exceptionType = EscapeJson(logEvent.Exception.GetType().FullName ?? "");
				string exceptionMessage = EscapeJson(logEvent.Exception.Message);
				string exceptionFull = EscapeJson(logEvent.Exception.ToString());
				output.Write($",\"Exception\":\"{exceptionFull}\",\"ExceptionType\":\"{exceptionType}\",\"ExceptionMessage\":\"{exceptionMessage}\"");
			}
			output.Write($",\"_truncated\":true,\"_originalBytes\":{originalByteCount}}}\n");
		}

		static string EscapeJson(string value)
		{
			return JsonSerializer.Serialize(value, s_jsonOptions)[1..^1];
		}
	}
}
