// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Threading;
using System.Threading.Tasks;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extension methods for working with streams
	/// </summary>
	static class StreamExtensions
	{
		/// <summary>
		/// Reads data of a fixed size into the buffer. Throws an exception if the whole block cannot be read.
		/// </summary>
		/// <param name="stream">The stream to read from</param>
		/// <param name="data">Buffer to receive the read data</param>
		/// <param name="offset">Offset within the buffer to read the new data</param>
		/// <param name="length">Length of the data to read</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public static Task ReadFixedSizeDataAsync(this Stream stream, byte[] data, int offset, int length, CancellationToken cancellationToken = default)
		{
			return ReadFixedSizeDataAsync(stream, data.AsMemory(offset, length), cancellationToken);
		}

		/// <summary>
		/// Reads data of a fixed size into the buffer. Throws an exception if the whole block cannot be read.
		/// </summary>
		/// <param name="stream">The stream to read from</param>
		/// <param name="data">Buffer to receive the read data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public static async Task ReadFixedSizeDataAsync(this Stream stream, Memory<byte> data, CancellationToken cancellationToken = default)
		{
			while (data.Length > 0)
			{
				int count = await stream.ReadAsync(data, cancellationToken);
				if (count == 0)
				{
					throw new EndOfStreamException();
				}
				data = data.Slice(count);
			}
		}
	}
}