// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Exception class designed to allow logging structured log messages
	/// </summary>
	public class StructuredException : Exception
	{
		/// <summary>
		/// Format string for the message
		/// </summary>
		public string Format { get; }

		/// <summary>
		/// Arguments for holes in the format string
		/// </summary>
		public IReadOnlyList<object> Args { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="format">Message template for the log message</param>
		/// <param name="args">Arguments to render</param>
		public StructuredException(string format, params object[] args)
		{
			Format = format;
			Args = args;
		}

		/// <summary>
		/// Creates a log event from this exception
		/// </summary>
		public LogEvent ToLogEvent()
		{
			return LogEvent.Create(LogLevel.Error, default, this, Format, Args);
		}
	}

	/// <summary>
	/// Exception class designed to allow logging structured log messages
	/// </summary>
	public class StructuredHttpException : StructuredException
	{
		/// <summary>
		/// Status code
		/// </summary>
		public int StatusCode { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="statusCode">Status code for the response</param>
		/// <param name="format">Message template for the log message</param>
		/// <param name="args">Arguments to render</param>
		public StructuredHttpException(int statusCode, string format, params object[] args)
			: base(format, args)
		{
			StatusCode = statusCode;
		}
	}
}
