// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Globalization;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Stores a mapping from one set of paths to another
	/// </summary>
	public class ViewMap
	{
		/// <summary>
		/// List of entries making up the view
		/// </summary>
		public List<ViewMapEntry> Entries { get; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public ViewMap()
		{
			Entries = new List<ViewMapEntry>();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="other"></param>
		public ViewMap(ViewMap other)
		{
			Entries = new List<ViewMapEntry>(other.Entries);
		}

		/// <summary>
		/// Determines if a file is included in the view
		/// </summary>
		/// <param name="file">The file to test</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the file is included in the view</returns>
		public bool MatchFile(Utf8String file, Utf8StringComparer comparison)
		{
			bool included = false;
			foreach (ViewMapEntry entry in Entries)
			{
				if (entry.MatchFile(file, comparison))
				{
					included = entry.Include;
				}
			}
			return included;
		}

		/// <summary>
		/// Attempts to convert a source file to its target path
		/// </summary>
		/// <param name="sourceFile"></param>
		/// <param name="comparison">The comparison type</param>
		/// <param name="targetFile"></param>
		/// <returns></returns>
		public bool TryMapFile(Utf8String sourceFile, Utf8StringComparer comparison, out Utf8String targetFile)
		{
			ViewMapEntry? mapEntry = null;
			foreach (ViewMapEntry entry in Entries)
			{
				if (entry.MatchFile(sourceFile, comparison))
				{
					mapEntry = entry;
				}
			}

			if (mapEntry != null && mapEntry.Include)
			{
				targetFile = mapEntry.MapFile(sourceFile);
				return true;
			}
			else
			{
				targetFile = Utf8String.Empty;
				return false;
			}
		}

		/// <summary>
		/// Gets the root paths from the view entries
		/// </summary>
		/// <returns></returns>
		public List<Utf8String> GetRootPaths(Utf8StringComparer comparison)
		{
			List<Utf8String> rootPaths = new List<Utf8String>();
			foreach (ViewMapEntry entry in Entries)
			{
				if (entry.Include)
				{
					int lastSlashIdx = entry.SourcePrefix.LastIndexOf('/');
					Utf8String rootPath = entry.SourcePrefix.Slice(0, lastSlashIdx + 1);

					for (int idx = 0; ; idx++)
					{
						if (idx == rootPaths.Count)
						{
							rootPaths.Add(rootPath);
							break;
						}
						else if (rootPaths[idx].StartsWith(rootPath, comparison))
						{
							rootPaths[idx] = rootPath;
							break;
						}
						else if (rootPath.StartsWith(rootPaths[idx], comparison))
						{
							break;
						}
					}
				}
			}
			return rootPaths;
		}
	}

	/// <summary>
	/// Entry within a ViewMap
	/// </summary>
	public class ViewMapEntry
	{
		/// <summary>
		/// Whether to include files matching this pattern
		/// </summary>
		public bool Include { get; }

		/// <summary>
		/// The wildcard string - either '*' or '...'
		/// </summary>
		public Utf8String Wildcard { get; }

		/// <summary>
		/// The source part of the pattern before the wildcard
		/// </summary>
		public Utf8String SourcePrefix { get; }

		/// <summary>
		/// The source part of the pattern after the wildcard. Perforce does not permit a slash to be in this part of the mapping.
		/// </summary>
		public Utf8String SourceSuffix { get; }

		/// <summary>
		/// The target mapping for the pattern before the wildcard
		/// </summary>
		public Utf8String TargetPrefix { get; }

		/// <summary>
		/// The target mapping for the pattern after the wildcard
		/// </summary>
		public Utf8String TargetSuffix { get; }

		/// <summary>
		/// The full source pattern
		/// </summary>
		public Utf8String Source => new Utf8String($"{SourcePrefix}{Wildcard}{SourceSuffix}");

		/// <summary>
		/// The full target pattern
		/// </summary>
		public Utf8String Target => new Utf8String($"{TargetPrefix}{Wildcard}{TargetSuffix}");

		/// <summary>
		/// Tests if the entry has a file wildcard ('*')
		/// </summary>
		/// <returns>True if the entry has a file wildcard</returns>
		public bool IsFileWildcard() => Wildcard.Length == 1;

		/// <summary>
		/// Tests if the entry has a path wildcard ('...')
		/// </summary>
		/// <returns>True if the entry has a path wildcard</returns>
		public bool IsPathWildcard() => Wildcard.Length == 3;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="other"></param>
		public ViewMapEntry(ViewMapEntry other)
			: this(other.Include, other.Wildcard, other.SourcePrefix, other.SourceSuffix, other.TargetPrefix, other.TargetSuffix)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="source"></param>
		/// <param name="target"></param>
		public ViewMapEntry(bool include, string source, string target)
		{
			Include = include;

			Match match = Regex.Match(source, @"^(.*)(\*|\.\.\.|%%1)(.*)$");
			if (match.Success)
			{
				string wildcardStr = match.Groups[2].Value;

				SourcePrefix = new Utf8String(match.Groups[1].Value);
				SourceSuffix = new Utf8String(match.Groups[3].Value);
				Wildcard = new Utf8String(match.Groups[2].Value);

				int otherIdx = target.IndexOf(wildcardStr, StringComparison.Ordinal);
				TargetPrefix = new Utf8String(target.Substring(0, otherIdx));
				TargetSuffix = new Utf8String(target.Substring(otherIdx + Wildcard.Length));

				if (wildcardStr.Equals("%%1", StringComparison.Ordinal))
				{
					Wildcard = new Utf8String("*");
				}
			}
			else
			{
				SourcePrefix = new Utf8String(source);
				SourceSuffix = Utf8String.Empty;
				TargetPrefix = new Utf8String(target);
				TargetSuffix = Utf8String.Empty;
			}
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="include"></param>
		/// <param name="wildcard"></param>
		/// <param name="sourcePrefix"></param>
		/// <param name="sourceSuffix"></param>
		/// <param name="targetPrefix"></param>
		/// <param name="targetSuffix"></param>
		public ViewMapEntry(bool include, Utf8String wildcard, Utf8String sourcePrefix, Utf8String sourceSuffix, Utf8String targetPrefix, Utf8String targetSuffix)
		{
			Include = include;
			Wildcard = wildcard;
			SourcePrefix = sourcePrefix;
			SourceSuffix = sourceSuffix;
			TargetPrefix = targetPrefix;
			TargetSuffix = targetSuffix;
		}

		/// <summary>
		/// Maps a file to the target path
		/// </summary>
		/// <param name="sourceFile"></param>
		/// <returns></returns>
		public Utf8String MapFile(Utf8String sourceFile)
		{
			int count = sourceFile.Length - SourceSuffix.Length - SourcePrefix.Length;
			return TargetPrefix + sourceFile.Slice(SourcePrefix.Length, count) + TargetSuffix;
		}

		/// <summary>
		/// Determine if a file matches the current entry
		/// </summary>
		/// <param name="path">Path to the file</param>
		/// <param name="comparison">The comparison type</param>
		/// <returns>True if the path matches the entry</returns>
		public bool MatchFile(Utf8String path, Utf8StringComparer comparison)
		{
			if (Wildcard.Length == 0)
			{
				return comparison.Compare(path, SourcePrefix) == 0;
			}
			else
			{
				if (!path.StartsWith(SourcePrefix, comparison) || !path.EndsWith(SourceSuffix, comparison))
				{
					return false;
				}
				if (IsFileWildcard() && path.Slice(SourcePrefix.Length, path.Length - SourceSuffix.Length - SourcePrefix.Length).IndexOf('/') != -1)
				{
					return false;
				}
				return true;
			}
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			StringBuilder builder = new StringBuilder();
			if (!Include)
			{
				builder.Append('-');
			}
			builder.Append(CultureInfo.InvariantCulture, $"{SourcePrefix}{Wildcard}{SourceSuffix} {TargetPrefix}{Wildcard}{TargetSuffix}");
			return builder.ToString();
		}
	}
}
