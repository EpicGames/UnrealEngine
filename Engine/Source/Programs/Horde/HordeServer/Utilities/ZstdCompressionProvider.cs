// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using Microsoft.AspNetCore.ResponseCompression;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Compression provider for zstd responses
	/// </summary>
	class ZstdCompressionProvider : ICompressionProvider
	{
		/// <inheritdoc/>
		public string EncodingName => "zstd";

		/// <inheritdoc/>
		public bool SupportsFlush => true;

		/// <inheritdoc/>
		public Stream CreateStream(Stream outputStream)
		{
			return new ZstdSharp.CompressionStream(outputStream, leaveOpen: false);
		}
	}
}
