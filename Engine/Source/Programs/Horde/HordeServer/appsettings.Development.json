{
	"Horde":
	{
	},
	"Serilog": 
	{
		"MinimumLevel": 
		{
			"Default": "Debug",
			"Override": {
				"HordeServer.Agents.AgentCollection": "Verbose",
				"HordeServer.Agents.AgentScheduler": "Verbose",
				"HordeServer.Perforce.PerforceService": "Verbose"
			}
		}
	}
}
