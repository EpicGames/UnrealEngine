{
	"Horde":
	{
		// Don't use shared system-wide folders for storing data
		"Installed": false,

		// Don't allow agents to try to upgrade; the assembly was built locally
		"EnableUpgradeTasks": false,

		// Don't allow conforms to run
		"EnableConformTasks": false,

		// Allow any local agents to automatically enrol
		"AutoEnrollAgents": true,

		// Apply config changes immediately on startup
		"ForceConfigUpdateOnStartup": true,

		"Commits":
		{
			// Don't replicate metadata from Perforce commits
			"ReplicateMetadata": false,

			// Don't replicate content from Perforce commits
			"ReplicateContent": false
		},

		// Enable the Example/experimental plugin for local development
		"Plugins": {
			"ExamplePlugin": {
				"Enabled": true				
			},
			"Experimental": {
				"Enabled": true
			}
		}
	},
	"Serilog": {
		"MinimumLevel": {
			"Override": {
				"HordeServer.Agents.Fleet": "Warning",
				"HordeServer.Agents.Fleet.Providers.NoOpFleetManager": "Warning"
			}
		}
	}
}
