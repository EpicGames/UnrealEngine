{
	"Horde": {
		"ConsoleLogLevel": "Debug",
		"DashboardUrl": "https://localhost:3000",
		"CorsEnabled": true,
		"CorsOrigin": "https://localhost:5003;http://localhost:3000"
	},
	"Serilog": {
		"MinimumLevel": {
			"Default": "Information",
			"Override": {
				"MongoDB": "Warning", // For bundled MongoDB output
				"Redis": "Warning", // For bundled Redis output
				"Microsoft": "Warning",
				"Microsoft.AspNetCore": "Warning",
				"Microsoft.Hosting": "Information",
				"HordeServer.Authentication": "Warning",
				"System.Net.Http.HttpClient": "Warning",
				"Grpc": "Warning",
				"HordeServer.Issues.IssueService": "Debug",
				"HordeServer.Issues.IssueTagService": "Debug",
				"HordeServer.Storage.ObjectStores.AwsObjectStore": "Information",
				"HordeServer.Storage.StorageService": "Debug",
				"EpicGames.Horde.Storage": "Debug"
			}
		}
	}
}
