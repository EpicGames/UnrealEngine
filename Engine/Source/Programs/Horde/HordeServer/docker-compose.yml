# A basic setup configuring Horde Server with a MongoDB and Redis server. Run with `docker compose up`
# Serves as a starting point to demonstrate what's necessary to start Horde.
# We highly recommend you modify this file as necessary or use it as an inspiration
# for configuring separate containers outside Docker Compose.

services:
  mongodb:
    image: mongo:7.0.32-jammy
    restart: always
    environment:
      # Default username and password, change these!
      MONGO_INITDB_ROOT_USERNAME: horde
      MONGO_INITDB_ROOT_PASSWORD: dbPass123
    command: --quiet --logpath /dev/null
    ports:
      - 27017:27017
    volumes:
      - mongodb:/data/db

  redis:
    image: redis:6.2-alpine
    restart: always
    ports:
      - 30002:30002
    command: redis-server --save 60 1 --port 30002 --loglevel warning
    volumes: 
      - redis:/data

  horde-server:
    # Requires access to Unreal Engine's image repository on GitHub
    image: ghcr.io/epicgames/horde-server:latest
    restart: always
    environment:
      # Horde uses standard configuration from ASP.NET, allowing values to be set through env vars and files.
      # See https://learn.microsoft.com/en-us/aspnet/core/fundamentals/configuration/
      # To configure via config file, see the mounted directory and file ./data/server.json.
      # The env vars below are set as they're configured through Docker Compose
      Horde__HttpPort: 13340
      Horde__Http2Port: 13342
      Horde__RedisConnectionConfig: redis:30002
      Horde__DatabaseConnectionString: mongodb://horde:dbPass123@mongodb:27017/Horde?authSource=admin
    ports:
      - 13340:13340 # HTTP/1
      - 13342:13342 # HTTP/2
    volumes:
      # Mount Horde's data directory to ./data for easy viewing and modifying
      # Once server has started, this directory is populated with default configuration files from the container
      - ./data:/app/Data

# Provide persistence between container restarts for MongoDB and Redis
volumes:
  mongodb:
    driver: local
  redis:
    driver: local