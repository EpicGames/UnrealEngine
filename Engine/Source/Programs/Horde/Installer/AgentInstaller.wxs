<?xml version="1.0" encoding="utf-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<Product Id="1A853492-7FD0-4F88-B120-25292474A829" Name="Unreal Horde Agent" Language="1033" Version="!(bind.fileVersion.HordeAgent.exe)" Manufacturer="Epic Games, Inc." UpgradeCode="35BC60F7-3F82-44C9-A332-9DC857254EC2">
		<Package Description="Unreal Horde Agent" Manufacturer="Epic Games, Inc." InstallerVersion="200" Compressed="yes" InstallPrivileges="elevated" InstallScope="perMachine" />
		<Media Id="1" Cabinet="product.cab" EmbedCab="yes" />

		<!-- For progress messages -->
		<UIRef Id="WixUI_ErrorProgressText" />

		<!-- Customize artwork -->
		<WixVariable Id="WixUIBannerBmp" Value="Banner.bmp" />
		<WixVariable Id="WixUIDialogBmp" Value="Dialog.bmp" />
		<Binary Id="MainImage" SourceFile="Dialog.bmp" />
		<Property Id="SERVER_URL" Value="http://localhost:13340">
			<RegistrySearch Id="DefaultServerUrlMachine" Root="HKLM" Key="Software\Epic Games\Horde" Name="Url" Type="raw" />
			<RegistrySearch Id="DefaultServerUrlUser" Root="HKCU" Key="Software\Epic Games\Horde" Name="Url" Type="raw" />
		</Property>
		<Property Id="OPEN_IN_BROWSER" Value="1" />
		<Property Id="SANDBOX_DIR" Value="C:\HordeAgent\Sandbox" />

		<CustomAction Id="OpenURL" Directory="TARGETDIR" ExeCommand="explorer.exe &quot;[SERVER_URL]/agents/registration&quot;" Impersonate="yes" Return="ignore" Execute="immediate"/>
		<CustomAction Id="AddFirewallRule" Directory="TARGETDIR" Execute="deferred" Return="ignore" ExeCommand="netsh advfirewall firewall add rule name=&quot;Horde Agent&quot; dir=in action=allow protocol=TCP localport=7000-7010" Impersonate="no" />
		<CustomAction Id="RefreshToolbox" Directory="ToolboxDir" Execute="deferred" Return="ignore" ExeCommand='"[ToolboxDir]UnrealToolbox.exe" -Refresh' />
		<!-- Cleanup any server enrollment file from the local system profile, as this will confuse future enrollments -->
		<CustomAction Id="CleanupServerJson" Directory="TARGETDIR" ExeCommand="cmd /C &quot;del &quot;C:\Windows\System32\config\systemprofile\AppData\Local\Epic Games\Horde\Agent\servers.json&quot;&quot;" Execute="deferred" Return="ignore" HideTarget="no" Impersonate="no" />


		<!-- Directories -->
		<Directory Id="TARGETDIR" Name="SourceDir">
			<Directory Id="ProgramFiles64Folder">
				<Directory Id="EpicGames" Name="Epic Games">
					<Directory Id="ToolboxDir" Name="Unreal Toolbox"/>
					<Directory Id="InstallDir" Name="Horde">
						<Directory Id="AgentInstallDir" Name="Agent">
						</Directory>
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="ProgramMenuFolder">
				<Directory Id="ApplicationProgramsFolder" Name="Horde" />
			</Directory>
			<Directory Id="CommonAppDataFolder">
				<Directory Id="EpicProgramDataDir" Name="Epic">
					<Directory Id="HordeProgramDataDir" Name="Horde">
						<Directory Id="HordeProgramDataAgentDir" Name="Agent" />
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="SANDBOX_DIR" />
		</Directory>

		<!-- UI -->
		<UI Id="UserInterface">
			<Property Id="WIXUI_INSTALLDIR" Value="TARGETDIR" />
			<Property Id="WixUI_Mode" Value="Custom" />

			<TextStyle Id="WixUI_Font_Normal" FaceName="Tahoma" Size="8" />
			<TextStyle Id="WixUI_Font_Bigger" FaceName="Tahoma" Size="9" Bold="yes" />
			<TextStyle Id="WixUI_Font_Title" FaceName="Tahoma" Size="9" Bold="yes" />

			<Property Id="DefaultUIFont" Value="WixUI_Font_Normal" />

			<DialogRef Id="ProgressDlg" />
			<DialogRef Id="CustomizeDlg" />
			<DialogRef Id="ErrorDlg" />
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="FatalError" />
			<DialogRef Id="UserExit" />

			<Publish Dialog="WelcomeDlg" Control="Next" Event="NewDialog" Value="LicenseDlg">1</Publish> 
			
			<Publish Dialog="LicenseDlg" Control="Back" Event="NewDialog" Value="WelcomeDlg">1</Publish> 
			<Publish Dialog="LicenseDlg" Control="Next" Event="NewDialog" Value="CustomizeDlg">1</Publish> 

			<Publish Dialog="CustomizeDlg" Control="Back" Event="NewDialog" Value="LicenseDlg">1</Publish>
			<Publish Dialog="CustomizeDlg" Control="Next" Event="NewDialog" Value="ServerUrlDialog">1</Publish>

			<Publish Dialog="AgentExitDialog" Control="Finish" Event="DoAction" Value="OpenURL" Order="1"><![CDATA[OPEN_IN_BROWSER=1]]></Publish>
			<Publish Dialog="AgentExitDialog" Control="Finish" Event="EndDialog" Value="Return" Order="999">1</Publish>

			<Dialog Id="LicenseDlg" Width="370" Height="270" Title="!(loc.LicenseAgreementDlg_Title)">
				<Control Id="BannerBitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="44" TabSkip="no" Text="!(loc.LicenseAgreementDlgBannerBitmap)" />
				<Control Id="BannerLine" Type="Line" X="0" Y="44" Width="370" Height="0" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="25" Y="23" Width="340" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgDescription)" />
				<Control Id="Title" Type="Text" X="15" Y="6" Width="200" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgTitle)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)" />
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="SpawnWaitDialog" Value="WaitForCostingDlg">!(wix.WixUICostingPopupOptOut) OR CostingComplete = 1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<Control Id="LicenseText" Type="ScrollableText" X="20" Y="60" Width="330" Height="160" Sunken="yes" TabSkip="no">
					<Text SourceFile="EULA.rtf" />
				</Control>
			</Dialog>

			<!-- Server Url Dialog -->
			<Dialog Id="ServerUrlDialog" Width="370" Height="270" Title="!(loc.WelcomeDlg_Title)">
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="MainImage" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />

				<Control Id="ServerUrlText" Type="Text" X="145" Y="60" Width="190" Height="15" Transparent="yes" Text="Horde Server URL:" />
				<Control Id="ServerUrlEdit" Type="Edit" X="145" Y="74" Width="190" Height="15" Property="SERVER_URL" Indirect="no" />

				<Control Id="DataDirText" Type="Text" X="145" Y="100" Width="190" Height="24" Transparent="yes" Text="Sandbox directory:" />
				<Control Id="DataDirEdit" Type="PathEdit" X="145" Y="114" Width="165" Height="15" Property="SANDBOX_DIR"/>
				<Control Id="DataDirBrowse" Type="PushButton" X="310" Y="114" Width="25" Height="15" Text="..." >
				  <Publish Property="_BrowseProperty" Value="SANDBOX_DIR" Order="1">1</Publish>
				  <Publish Event="SpawnDialog" Value="BrowseDlg" Order="2">1</Publish>
				</Control>

				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)">
					<Publish Event="NewDialog" Value="CustomizeDlg">1</Publish>
				</Control>
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="EndDialog" Value="Return">1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
			</Dialog>

			<Dialog Id="AgentExitDialog" Width="370" Height="270" Title="!(loc.ExitDialog_Title)">
				<Control Id="Finish" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)" />
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUICancel)" />
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="!(loc.ExitDialogBitmap)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="135" Y="70" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogDescription)" />
				<Control Id="Title" Type="Text" X="135" Y="20" Width="220" Height="60" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogTitle)" />
				<Control Id="OpenInBrowserCheckBox" Type="CheckBox" X="139" Y="120" Width="13" Height="13" Property="OPEN_IN_BROWSER" CheckBoxValue="1" Hidden="yes">
					<Condition Action="show"><![CDATA[NOT REMOVE]]></Condition>
				</Control>
				<Control Type="Text" Id="OpenInBrowserText" Width="190" Height="17" X="154" Y="120" Transparent="yes" Text="Enroll agent with Horde server" Hidden="yes">
					<Condition Action="show"><![CDATA[NOT REMOVE]]></Condition>
				</Control>
			</Dialog>

		</UI>
		<UIRef Id="WixUI_Common" />

		<!-- Agent -->
		<DirectoryRef Id="AgentInstallDir">
			<Component Id="HordeAgentMakeWritable" Guid="E604E294-FDEF-470F-B557-66106304C26B">
				<!-- Make the agent folder writable, so can pull updates from the server -->
				<CreateFolder>
					<util:PermissionEx User="Users" GenericAll="yes" />
				</CreateFolder>
			</Component>			
			<Component Id="HordeAgentExecutable" Guid="2D2756F4-0CA4-4651-8446-4FC32DDB1EEB">				
				<!-- Required as we have 2 components refering to HordeAgent.exe, otherwise breaks reference counting-->
				<Condition>1</Condition>
				<File Id="HordeAgent.exe" KeyPath="yes" Source="$(var.StagingDir)\HordeAgent.exe"/>
			</Component>
			<Component Id="HordeAgentServiceComponent" Guid="3FB0D3C4-027E-4337-907A-9BD1B64243E2">				
				<!-- Required as we have 2 components refering to HordeAgent.exe, otherwise breaks reference counting-->
				<Condition>1</Condition>
				<File Id="HordeAgent.Service.exe" KeyPath="yes" Source="$(var.StagingDir)\HordeAgent.exe"/>
				<ServiceInstall Id="HordeAgentInst" Name="HordeAgent" DisplayName="Horde Agent" Description="Runs compute tasks issued by the Horde server" Start="auto" Type="ownProcess" ErrorControl="normal" Vital="yes" Interactive="no" />
				<ServiceControl Id="HordeAgentCtrl" Name="HordeAgent" Start="install" Stop="both" Remove="uninstall" Wait="yes" />
			</Component>
			<Component Id="HordeAgentInstalledKeyComponent" Guid="5B620296-FCF4-469F-A4D9-5C1FBE1077EA">
				<RegistryKey Root="HKLM" Key="Software\Epic Games\Horde\Agent">
					<RegistryValue Type="integer" Name="Installed" Value="1"/>
				</RegistryKey>
			</Component>
			<Component Id="HordeAgentDefaultServerMachineComponent" Guid="9EAD9116-E10C-42FD-B1F2-BE8599CF3A14" Permanent="yes">
				<RegistryKey Root="HKLM" Key="Software\Epic Games\Horde">
					<RegistryValue Type="string" Name="Url" Value="[SERVER_URL]"/>
				</RegistryKey>
			</Component>
			<Component Id="HordeAgentDefaultServerUserComponent" Guid="B4116253-162C-4876-9861-E70461701FFA" Permanent="yes">
				<RegistryKey Root="HKCU" Key="Software\Epic Games\Horde">
					<RegistryValue Type="string" Name="Url" Value="[SERVER_URL]"/>
				</RegistryKey>
			</Component>
			<Component Id="HordeAgentWorkingDirComponent" Guid="F9F29320-B408-41A0-8F93-1EFB901AFD2A">
				<RegistryKey Root="HKLM" Key="Software\Epic Games\Horde\Agent">
					<RegistryValue Type="string" Name="WorkingDir" Value="[SANDBOX_DIR]" KeyPath="yes"/>
				</RegistryKey>
			</Component>
		</DirectoryRef>
		
		<!-- ProgramData Directory -->
		<DirectoryRef Id="HordeProgramDataAgentDir">
			<Component Id="HordeProgramDataComponent" Guid="6E9D0653-B1E0-49BD-8F55-3B01C723EB2F">
				<CreateFolder>
					<util:PermissionEx User="Everyone" GenericAll="yes" />
				</CreateFolder>
			</Component>
		</DirectoryRef>

		<!-- Agent Sandbox Directory -->
		<DirectoryRef Id="SANDBOX_DIR">
			<Component Id="HordeAgentSandboxDirComponent" Guid="6E9D0653-B1E0-49BD-8F55-3B01C833EB2F">
				<CreateFolder>
					<util:PermissionEx User="Everyone" GenericAll="yes" />
				</CreateFolder>
			</Component>
		</DirectoryRef>

		<!-- Program files setup -->
		<DirectoryRef Id="ApplicationProgramsFolder">
			<Component Id="HordeAgentShortcutComponent" Guid="*">
				<Shortcut Id="HordeAgentShortcut" Name="Horde Agent" Description="Opens the Horde agent folder" Target="[AgentInstallDir]" Arguments="open" WorkingDirectory="AgentInstallDir" Show="minimized" />
				<RemoveFolder Id="CleanUpHordeAgentShortCut" Directory="ApplicationProgramsFolder" On="uninstall" />
				<RegistryValue Root="HKCU" Key="Software\Epic Games\Horde" Name="agent" Type="integer" Value="1" KeyPath="yes" />
			</Component>
		</DirectoryRef>

		<!-- Features -->
		<Feature Id="HordeAgent" Title="Horde Agent" Level="1" AllowAdvertise="no" Absent="disallow">
			<ComponentGroupRef Id="HordeAgentFiles" />
			<ComponentRef Id="HordeAgentExecutable" />
			<ComponentRef Id="HordeAgentMakeWritable" />
			<ComponentRef Id="HordeAgentShortcutComponent" />
			<ComponentRef Id="HordeAgentInstalledKeyComponent" />
			<ComponentRef Id="HordeAgentDefaultServerMachineComponent" />
			<ComponentRef Id="HordeAgentDefaultServerUserComponent" />
			<ComponentRef Id="HordeAgentWorkingDirComponent" />
			<ComponentRef Id="HordeProgramDataComponent" />
			<ComponentRef Id="HordeAgentSandboxDirComponent" />
		</Feature>

		<Feature Id="HordeAgentService" Title="Horde Agent Service" Level="1" AllowAdvertise="no">			
			<ComponentRef Id="HordeAgentServiceComponent" />
		</Feature>

		<InstallUISequence>
			<Show Dialog="AgentExitDialog" OnExit="success">1</Show>
		</InstallUISequence>
		<AdminUISequence>
			<Show Dialog="AgentExitDialog" OnExit="success">1</Show>
		</AdminUISequence>
		<InstallExecuteSequence>
			<Custom Action="AddFirewallRule" Before="InstallFinalize"><![CDATA[NOT Installed]]></Custom>
			<Custom Action="RefreshToolbox" Before="InstallFinalize"/>
			<Custom Action="CleanupServerJson" After="RemoveFiles" > REMOVE="ALL" </Custom>
		</InstallExecuteSequence>

		<Icon Id="Setup.ico" SourceFile="..\HordeAgent\HordeAgent.ico"/>
		<Property Id="ARPPRODUCTICON" Value="Setup.ico" />
	</Product>
</Wix>