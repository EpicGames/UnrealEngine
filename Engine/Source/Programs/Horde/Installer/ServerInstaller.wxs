<?xml version="1.0" encoding="utf-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<Product Id="EC8DD7C9-A32F-42D1-B85D-D8A2CD48D12A" Name="Unreal Horde Server" Language="1033" Version="!(bind.fileVersion.HordeServer.exe)" Manufacturer="Epic Games, Inc." UpgradeCode="78B9853E-0F20-4800-B071-FF31426154C0">
		<Package Description="Unreal Horde Server" Manufacturer="Epic Games, Inc." InstallerVersion="200" Compressed="yes" InstallPrivileges="elevated" InstallScope="perMachine" />
		<Media Id="1" Cabinet="product.cab" EmbedCab="yes" />

		<!-- For progress messages -->
		<UIRef Id="WixUI_ErrorProgressText" />

		<!-- Customize artwork -->
		<WixVariable Id="WixUIBannerBmp" Value="Banner.bmp" />
		<WixVariable Id="WixUIDialogBmp" Value="Dialog.bmp" />
		<Binary Id="MainImage" SourceFile="Dialog.bmp" />
		<Property Id="SERVER_PORT" Value="13340" />
		<Property Id="SERVER_HTTP2_PORT" Value="13342" />
		<Property Id="SERVER_DATA_FOLDER" Value="C:\ProgramData\Epic\Horde\Server" />

		<!-- Open in user's default browser (note difference between explorer.exe and iexplorer.exe) -->
		<Property Id="OPEN_IN_BROWSER" Value="1" />
		<CustomAction Id="OpenURL" Directory="TARGETDIR" ExeCommand="explorer.exe &quot;http://localhost:[SERVER_PORT]/docs/Landing.md&quot;" Impersonate="yes" Return="ignore" Execute="immediate"/>
		<CustomAction Id="AddFirewallRule" Directory="TARGETDIR" Execute="deferred" Return="ignore" ExeCommand="netsh advfirewall firewall add rule name=&quot;Horde Server&quot; dir=in action=allow program=&quot;[#HordeServer.exe]&quot; enable=yes" Impersonate="no" />

		<!-- UI -->
		<UI Id="UserInterface">
			<Property Id="WIXUI_INSTALLDIR" Value="TARGETDIR" />
			<Property Id="WixUI_Mode" Value="Custom" />

			<TextStyle Id="WixUI_Font_Normal" FaceName="Tahoma" Size="8" />
			<TextStyle Id="WixUI_Font_Bigger" FaceName="Tahoma" Size="9" Bold="yes" />
			<TextStyle Id="WixUI_Font_Title" FaceName="Tahoma" Size="9" Bold="yes" />

			<Property Id="DefaultUIFont" Value="WixUI_Font_Normal" />

			<DialogRef Id="ProgressDlg" />
			<DialogRef Id="CustomizeDlg" />
			<DialogRef Id="ErrorDlg" />
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="FatalError" />
			<DialogRef Id="UserExit" />

			<Publish Dialog="WelcomeDlg" Control="Next" Event="NewDialog" Value="LicenseDlg">1</Publish>

			<Publish Dialog="LicenseDlg" Control="Back" Event="NewDialog" Value="WelcomeDlg">1</Publish>
			<Publish Dialog="LicenseDlg" Control="Next" Event="NewDialog" Value="CustomizeDlg">1</Publish>

			<Publish Dialog="CustomizeDlg" Control="Back" Event="NewDialog" Value="LicenseDlg">1</Publish>
			<Publish Dialog="CustomizeDlg" Control="Next" Event="NewDialog" Value="ServerPortDialog">1</Publish>

			<Publish Dialog="ServerExitDialog" Control="Finish" Event="DoAction" Value="OpenURL" Order="1"><![CDATA[OPEN_IN_BROWSER=1 AND (NOT REMOVE AND &HordeTools=3)]]></Publish>
			<Publish Dialog="ServerExitDialog" Control="Finish" Event="EndDialog" Value="Return" Order="999">1</Publish>

			<!-- Custom license notice (do not have to check to agree) -->
			<Dialog Id="LicenseDlg" Width="370" Height="270" Title="!(loc.LicenseAgreementDlg_Title)">
				<Control Id="BannerBitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="44" TabSkip="no" Text="!(loc.LicenseAgreementDlgBannerBitmap)" />
				<Control Id="BannerLine" Type="Line" X="0" Y="44" Width="370" Height="0" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="25" Y="23" Width="340" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgDescription)" />
				<Control Id="Title" Type="Text" X="15" Y="6" Width="200" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgTitle)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)" />
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="SpawnWaitDialog" Value="WaitForCostingDlg">!(wix.WixUICostingPopupOptOut) OR CostingComplete = 1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<Control Id="LicenseText" Type="ScrollableText" X="20" Y="60" Width="330" Height="160" Sunken="yes" TabSkip="no">
					<Text SourceFile="EULA.rtf" />
				</Control>
			</Dialog>

			<!-- Server Port Dialog -->
			<Dialog Id="ServerPortDialog" Width="370" Height="270" Title="!(loc.WelcomeDlg_Title)">
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="MainImage" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />

				<Control Id="ServerPortDesc" Type="Text" X="145" Y="44" Width="180" Height="24" Transparent="yes" Text="Horde Server Configuration" />
				
				<Control Id="ServerPortText" Type="Text" X="145" Y="75" Width="180" Height="24" Transparent="yes" Text="HTTP port" />
				<Control Id="ServerPortEdit" Type="Edit" X="145" Y="90" Height="15" Width="180" Property="SERVER_PORT" Indirect="no" />

				<Control Id="ServerPort2Text" Type="Text" X="145" Y="115" Width="180" Height="24" Transparent="yes" Text="HTTP/2 port (for agents)" />
				<Control Id="ServerPort2Edit" Type="Edit" X="145" Y="130" Height="15" Width="180" Property="SERVER_HTTP2_PORT" Indirect="no" />

				<Control Id="ServerDataFolderText" Type="Text" X="145" Y="155" Width="190" Height="24" Transparent="yes" Text="Server Data Directory:" />
				<Control Id="DataDirEdit" Type="PathEdit" X="145" Y="169" Width="165" Height="15" Property="SERVER_DATA_FOLDER"/>
				<Control Id="DataDirBrowse" Type="PushButton" X="310" Y="169" Width="25" Height="15" Text="..." >
				  <Publish Property="_BrowseProperty" Value="SERVER_DATA_FOLDER" Order="1">1</Publish>
				  <Publish Event="SpawnDialog" Value="BrowseDlg" Order="2">1</Publish>
				</Control>

				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)">
					<Publish Event="NewDialog" Value="CustomizeDlg">1</Publish>
				</Control>
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="EndDialog" Value="Return">1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
			</Dialog>

			<!-- Custom Exit Dialog, there is a way to add a single checkbox to the standard exit dialog, though it looks terrible -->
		    <Dialog Id="ServerExitDialog" Width="370" Height="270" Title="!(loc.ExitDialog_Title)">
				<Control Id="Finish" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)" />
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUICancel)" />
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="!(loc.ExitDialogBitmap)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="135" Y="70" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogDescription)" />
				<Control Id="Title" Type="Text" X="135" Y="20" Width="220" Height="60" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogTitle)" />
				<Control Id="OpenInBrowserCheckBox" Type="CheckBox" X="139" Y="120" Width="13" Height="13" Property="OPEN_IN_BROWSER" CheckBoxValue="1" Hidden="yes">
				 	<Condition Action="show"><![CDATA[NOT REMOVE AND &HordeTools=3]]></Condition>
				</Control>
				<Control Type="Text" Id="OpenInBrowserText" Width="190" Height="17" X="154" Y="120" Transparent="yes" Text="Open in default browser" Hidden="yes">
					<Condition Action="show"><![CDATA[NOT REMOVE AND &HordeTools=3]]></Condition>
				</Control>
    		</Dialog>

		</UI>
		<UIRef Id="WixUI_Common" />

		<InstallUISequence>
			<Show Dialog="ServerExitDialog" OnExit="success">1</Show>
		</InstallUISequence>
		<AdminUISequence>
			<Show Dialog="ServerExitDialog" OnExit="success">1</Show>
		</AdminUISequence>
		<InstallExecuteSequence>
			<Custom Action="AddFirewallRule" Before="InstallFinalize">NOT Installed</Custom>
		</InstallExecuteSequence>

		<!-- Directories -->
		<Directory Id="TARGETDIR" Name="SourceDir">
			<Directory Id="ProgramFiles64Folder">
				<Directory Id="EpicGames" Name="Epic Games">
					<Directory Id="InstallDir" Name="Horde">
						<Directory Id="ServerInstallDir" Name="Server" />						
						<Directory Id="ToolsInstallDir" Name="Tools" />
					</Directory>
				</Directory>
			</Directory>
			<Directory Id="ProgramMenuFolder">
				<Directory Id="ApplicationProgramsFolder" Name="Horde" />
			</Directory>
			<Directory Id="SERVER_DATA_FOLDER"/>
		</Directory>

		<!-- Server -->
		<DirectoryRef Id="ServerInstallDir">
			<Component Id="HordeServerExeComponent" Guid="56B2FE0E-64C9-4A4F-9FAC-BC350BE07C3B">
				<File Id="HordeServer.exe" KeyPath="yes" Source="$(var.StagingDir)\Server\win-x64\HordeServer.exe" />
				<ServiceInstall Id="HordeServerInst" Name="HordeServer" DisplayName="Horde Server" Description="Hosts the UE Horde server" Start="auto" Type="ownProcess" ErrorControl="normal" Vital="yes" Interactive="no" />
				<ServiceControl Id="HordeServerCtrl" Name="HordeServer" Start="install" Stop="both" Remove="uninstall" Wait="yes" />
			</Component>
			<Component Id="HordeServerRegistryComponent" Guid="3ED2E9CF-B4C5-4D5C-80B7-8B943F717A6C">
				<RegistryKey Root="HKLM" Key="Software\Epic Games\Horde\Server">
					<RegistryValue Type="integer" Name="HttpPort" Value="[SERVER_PORT]" KeyPath="yes" />
					<RegistryValue Type="integer" Name="Http2Port" Value="[SERVER_HTTP2_PORT]" />
					<RegistryValue Type="string" Name="DataDir" Value="[SERVER_DATA_FOLDER]" />
					<RegistryValue Type="string" Name="InstalledServerExecutable" Value="[ServerInstallDir]HordeServer.dll" />					
				</RegistryKey>
			</Component>
		</DirectoryRef>

		<!-- ProgramData Directory -->
		<DirectoryRef Id="SERVER_DATA_FOLDER">
			<Component Id="HordeProgramDataComponent" Guid="6618FAC3-5E62-4C23-9706-38E8919616AD">
				<CreateFolder>
					<util:PermissionEx User="Everyone" GenericAll="yes" />
				</CreateFolder>
			</Component>
		</DirectoryRef>

		<!-- Program files setup -->
		<DirectoryRef Id="ApplicationProgramsFolder">
			<Component Id="HordeServerShortcutComponent" Guid="*">
				<Shortcut Id="HordeServerShortcut" Name="Unreal Horde Server" Description="Opens the local Horde server page" Target="[#HordeServer.exe]" Arguments="open" WorkingDirectory="ServerInstallDir" Show="minimized" />
				<RemoveFolder Id="CleanUpHordeServerShortCut" Directory="ApplicationProgramsFolder" On="uninstall" />
				<RegistryValue Root="HKCU" Key="Software\Epic Games\Horde\Server" Name="Installed" Type="integer" Value="1" KeyPath="yes" />
			</Component>
		</DirectoryRef>
		<Feature Id="HordeServer" Title="Horde Server" Level="1" AllowAdvertise="no">
			<ComponentGroupRef Id="HordeServerFiles" />
			<ComponentRef Id="HordeServerExeComponent" />
			<ComponentRef Id="HordeServerRegistryComponent" />
			<ComponentRef Id="HordeServerShortcutComponent" />
			<ComponentRef Id="HordeProgramDataComponent" />
		</Feature>

		<!-- Command line tools -->
		<DirectoryRef Id="ToolsInstallDir">
			<Component Id="ToolsEnvironmentVar" Guid="84E28AF0-EC5C-4595-9E8D-BE4D044BF1FD" KeyPath="yes">
				<Environment Id="ToolsPathEnvVar" Name="PATH" Value="[ToolsInstallDir]" Permanent="yes" Part="last" Action="set" System="yes" />
			</Component>
		</DirectoryRef>
		<Feature Id="HordeTools" Title="Command Line Tools" Level="1" AllowAdvertise="no">
			<ComponentGroupRef Id="HordeToolsFiles" />
			<ComponentRef Id="ToolsEnvironmentVar" />
		</Feature>

		<Icon Id="Setup.ico" SourceFile="..\HordeServer\HordeServer.ico"/>
		<Property Id="ARPPRODUCTICON" Value="Setup.ico" />
	</Product>
</Wix>