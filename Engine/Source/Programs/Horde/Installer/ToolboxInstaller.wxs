<?xml version="1.0" encoding="utf-8"?>
<Wix xmlns="http://schemas.microsoft.com/wix/2006/wi" xmlns:util="http://schemas.microsoft.com/wix/UtilExtension">
	<Product Id="4D59FD3A-A77D-4DD4-87FE-3A55F70F07C3" Name="Unreal Toolbox" Language="1033" Version="!(bind.fileVersion.UnrealToolbox.exe)" Manufacturer="Epic Games, Inc." UpgradeCode="E2A18E8C-C66F-4D6F-89EA-15B063C79F26">
		<Package Description="Unreal Toolbox" Manufacturer="Epic Games, Inc." InstallerVersion="200" Compressed="yes" InstallPrivileges="elevated" InstallScope="perMachine" />
		<Media Id="1" Cabinet="product.cab" EmbedCab="yes" />

		<!-- For progress messages -->
		<UIRef Id="WixUI_ErrorProgressText" />

		<!-- Customize artwork -->
		<WixVariable Id="WixUIBannerBmp" Value="Banner.bmp" />
		<WixVariable Id="WixUIDialogBmp" Value="Dialog.bmp" />
		<Binary Id="MainImage" SourceFile="Dialog.bmp" />
		<Property Id="SERVER_URL" Value="http://localhost:13340">
			<RegistrySearch Id="DefaultServerUrlUser" Root="HKCU" Key="Software\Epic Games\Horde" Name="Url" Type="raw" />
			<RegistrySearch Id="DefaultServerUrlMachine" Root="HKLM" Key="Software\Epic Games\Horde" Name="Url" Type="raw" />			
		</Property>

		<!-- Directories -->
		<Directory Id="TARGETDIR" Name="SourceDir">
			<Directory Id="ProgramFiles64Folder">
				<Directory Id="EpicGames" Name="Epic Games">
					<Directory Id="ToolboxInstallDir" Name="Unreal Toolbox"/>
				</Directory>
			</Directory>
			<Directory Id="ProgramMenuFolder"/>
		</Directory>

		<!-- UI -->
		<UI Id="UserInterface">
			<Property Id="WIXUI_INSTALLDIR" Value="TARGETDIR" />
			<Property Id="WixUI_Mode" Value="Custom" />

			<TextStyle Id="WixUI_Font_Normal" FaceName="Tahoma" Size="8" />
			<TextStyle Id="WixUI_Font_Bigger" FaceName="Tahoma" Size="9" Bold="yes" />
			<TextStyle Id="WixUI_Font_Title" FaceName="Tahoma" Size="9" Bold="yes" />

			<Property Id="DefaultUIFont" Value="WixUI_Font_Normal" />

			<DialogRef Id="ProgressDlg" />
			<DialogRef Id="ErrorDlg" />
			<DialogRef Id="FilesInUse" />
			<DialogRef Id="FatalError" />
			<DialogRef Id="UserExit" />

			<Publish Dialog="WelcomeDlg" Control="Next" Event="NewDialog" Value="LicenseDlg">1</Publish> 
			
			<Publish Dialog="LicenseDlg" Control="Back" Event="NewDialog" Value="WelcomeDlg">1</Publish> 
			<Publish Dialog="LicenseDlg" Control="Next" Event="NewDialog" Value="ServerUrlDialog">1</Publish> 

			<Publish Dialog="AgentExitDialog" Control="Finish" Event="EndDialog" Value="Return" Order="999">1</Publish>

			<Dialog Id="LicenseDlg" Width="370" Height="270" Title="!(loc.LicenseAgreementDlg_Title)">
				<Control Id="BannerBitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="44" TabSkip="no" Text="!(loc.LicenseAgreementDlgBannerBitmap)" />
				<Control Id="BannerLine" Type="Line" X="0" Y="44" Width="370" Height="0" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="25" Y="23" Width="340" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgDescription)" />
				<Control Id="Title" Type="Text" X="15" Y="6" Width="200" Height="15" Transparent="yes" NoPrefix="yes" Text="!(loc.LicenseAgreementDlgTitle)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)" />
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="SpawnWaitDialog" Value="WaitForCostingDlg">!(wix.WixUICostingPopupOptOut) OR CostingComplete = 1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
				<Control Id="LicenseText" Type="ScrollableText" X="20" Y="60" Width="330" Height="160" Sunken="yes" TabSkip="no">
					<Text SourceFile="EULA.rtf" />
				</Control>
			</Dialog>

			<!-- Server Url Dialog -->
			<Dialog Id="ServerUrlDialog" Width="370" Height="270" Title="!(loc.WelcomeDlg_Title)">
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="MainImage" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />

				<Control Id="ServerUrlText" Type="Text" X="138" Y="60" Width="210" Height="15" Transparent="yes" Text="Horde Server URL:" />
				<Control Id="ServerUrlEdit" Type="Edit" X="138" Y="74" Width="210" Height="15" Property="SERVER_URL" Indirect="no" />

				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Text="!(loc.WixUIBack)">
					<Publish Event="NewDialog" Value="LicenseDlg">1</Publish>
				</Control>
				<Control Id="Next" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Text="!(loc.WixUINext)">
					<Publish Event="EndDialog" Value="Return">1</Publish>
				</Control>
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Cancel="yes" Text="!(loc.WixUICancel)">
					<Publish Event="SpawnDialog" Value="CancelDlg">1</Publish>
				</Control>
			</Dialog>

			<Dialog Id="AgentExitDialog" Width="370" Height="270" Title="!(loc.ExitDialog_Title)">
				<Control Id="Finish" Type="PushButton" X="236" Y="243" Width="56" Height="17" Default="yes" Cancel="yes" Text="!(loc.WixUIFinish)" />
				<Control Id="Cancel" Type="PushButton" X="304" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUICancel)" />
				<Control Id="Bitmap" Type="Bitmap" X="0" Y="0" Width="370" Height="234" TabSkip="no" Text="!(loc.ExitDialogBitmap)" />
				<Control Id="Back" Type="PushButton" X="180" Y="243" Width="56" Height="17" Disabled="yes" Text="!(loc.WixUIBack)" />
				<Control Id="BottomLine" Type="Line" X="0" Y="234" Width="370" Height="0" />
				<Control Id="Description" Type="Text" X="135" Y="70" Width="220" Height="40" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogDescription)" />
				<Control Id="Title" Type="Text" X="135" Y="20" Width="220" Height="60" Transparent="yes" NoPrefix="yes" Text="!(loc.ExitDialogTitle)" />
			</Dialog>

		</UI>
		<UIRef Id="WixUI_Common" />

		<!-- Agent -->
		<DirectoryRef Id="ToolboxInstallDir">
			<Component Id="UnrealToolboxMakeWritable" Guid="4A8F5BAB-0BC4-4C72-922E-0449B9E8F5FF">
				<!-- Make the agent folder writable, so can pull updates from the server -->
				<CreateFolder>
					<util:PermissionEx User="Users" GenericAll="yes" />
				</CreateFolder>
			</Component>
		</DirectoryRef>
		
		<!-- Agent Toolbox -->
		<DirectoryRef Id="ToolboxInstallDir">
			<Component Id="UnrealToolboxExeComponent" Guid="0C66C642-4958-4472-956B-180844FE0397">
				<!-- Toolbox application -->
				<File Id="UnrealToolbox.exe" KeyPath="yes" Source="$(var.StagingDir)\UnrealToolbox.exe" />
			</Component>
			<Component Id="UnrealToolboxRunOnStartupComponent" Guid="EC4232DB-44BF-4379-986B-4DBF095713D8">
				<!-- Register Toolbox application to run at startup -->
				<RegistryKey Root="HKLM" Key="Software\Microsoft\Windows\CurrentVersion\Run">
					<RegistryValue Type="string" Name="UnrealToolbox" Value="&quot;[#UnrealToolbox.exe]&quot; -Quiet" KeyPath="yes"/>
				</RegistryKey>
			</Component>
			<Component Id="UnrealToolboxDefaultServerMachineComponent" Guid="9EAD9116-E10C-42FD-B1F2-BE8599CF3A25" Permanent="yes">
				<RegistryKey Root="HKLM" Key="Software\Epic Games\Horde">
					<RegistryValue Type="string" Name="Url" Value="[SERVER_URL]"/>
				</RegistryKey>
			</Component>
			<Component Id="UnrealToolboxDefaultServerUserComponent" Guid="B4116253-162C-4876-9861-E70461701FED" Permanent="yes">
				<RegistryKey Root="HKCU" Key="Software\Epic Games\Horde">
					<RegistryValue Type="string" Name="Url" Value="[SERVER_URL]"/>
				</RegistryKey>
			</Component>
		</DirectoryRef>

		<!-- Program files setup -->
		<DirectoryRef Id="ProgramMenuFolder">
			<Component Id="UnrealToolboxShortcutComponent" Guid="*">
				<Shortcut Id="UnrealToolboxShortcut" Name="Unreal Toolbox" Description="Opens Unreal Toolbox" Target="[#UnrealToolbox.exe]" WorkingDirectory="ToolboxInstallDir" Show="minimized" />
				<RegistryValue Root="HKCU" Key="Software\Epic Games\Unreal Toolbox" Name="Installed" Type="integer" Value="1" KeyPath="yes" />
			</Component>
		</DirectoryRef>

		<!-- Features -->
		<Feature Id="UnrealToolbox" Title="Horde Launcher" Level="1" AllowAdvertise="no">
			<ComponentGroupRef Id="UnrealToolboxFiles" />
			<ComponentRef Id="UnrealToolboxMakeWritable" />
			<ComponentRef Id="UnrealToolboxShortcutComponent" />
			<ComponentRef Id="UnrealToolboxExeComponent" />
			<ComponentRef Id="UnrealToolboxRunOnStartupComponent" />
			<ComponentRef Id="UnrealToolboxDefaultServerMachineComponent" />
			<ComponentRef Id="UnrealToolboxDefaultServerUserComponent" />
		</Feature>

		<CustomAction Id='CloseToolbox' FileKey='UnrealToolbox.exe' ExeCommand="-Close" Return='ignore' Execute="immediate" />
		<CustomAction Id='LaunchToolbox' FileKey='UnrealToolbox.exe' ExeCommand="" Return='asyncNoWait' />

		<InstallUISequence>
			<Show Dialog="AgentExitDialog" OnExit="success">1</Show>
		</InstallUISequence>
		<AdminUISequence>
			<Show Dialog="AgentExitDialog" OnExit="success">1</Show>
		</AdminUISequence>
		<InstallExecuteSequence>
			<Custom Action="CloseToolbox" Before="InstallValidate"><![CDATA[Installed AND !UnrealToolbox=3]]></Custom> 
			<Custom Action='LaunchToolbox' After='InstallFinalize'><![CDATA[NOT REMOVE AND &UnrealToolbox=3]]></Custom>
		</InstallExecuteSequence>

		<Icon Id="Setup.ico" SourceFile="..\HordeAgent\HordeAgent.ico"/>
		<Property Id="ARPPRODUCTICON" Value="Setup.ico" />
	</Product>
</Wix>