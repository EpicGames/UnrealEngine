// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections;
using System.Reflection;
using System.Runtime.Serialization;
using Microsoft.Extensions.Logging;

namespace HordeServer.Analytics.Integration.Tests
{
	public static class AnalyticsIntegrationTestExtensions
	{
		private static readonly ILogger s_logger = LoggerFactory.Create(b => b.AddConsole()).CreateLogger("AnalyticsIntegrationTestExtensions");

		private static object? TryMapAnonymousToTypedObject(object anon, Type targetType)
		{
			if (anon == null)
			{
				return null;
			}

#pragma warning disable SYSLIB0050 // Type or member is obsolete
			object typedObj = FormatterServices.GetUninitializedObject(targetType);
#pragma warning restore SYSLIB0050 // Type or member is obsolete

			foreach (PropertyInfo targetProp in targetType.GetProperties(BindingFlags.Public | BindingFlags.Instance))
			{
				if (!targetProp.CanWrite)
				{
					continue;
				}

				PropertyInfo? anonProp = anon.GetType().GetProperty(targetProp.Name);
				if (anonProp == null)
				{
					continue;
				}

				object? anonValue = anonProp.GetValue(anon);
				if (anonValue == null)
				{
					continue;
				}

				try
				{
					object? converted = Convert.ChangeType(anonValue, Nullable.GetUnderlyingType(targetProp.PropertyType) ?? targetProp.PropertyType);
					targetProp.SetValue(typedObj, converted);
				}
				catch
				{
					if (targetProp.PropertyType.IsInstanceOfType(anonValue))
					{
						targetProp.SetValue(typedObj, anonValue);
					}
				}
			}

			return typedObj;
		}

		/// <summary>
		/// Attempts to map a hashtable to an object.
		/// </summary>
		/// <typeparam name="T">The type of object to return.</typeparam>
		/// <param name="table">The input table to attempt conversion from.</param>
		/// <returns>The resulting object of type if conversion was possible.</returns>
		/// <exception cref="ArgumentNullException">Throws if input table is null.</exception>
		public static T MapToObject<T>(this Hashtable table) where T : class
		{
			ArgumentNullException.ThrowIfNull(table);

			Type targetType = typeof(T);
#pragma warning disable SYSLIB0050 // Type or member is obsolete
			T instance = (T)System.Runtime.Serialization.FormatterServices.GetUninitializedObject(typeof(T));
#pragma warning restore SYSLIB0050 // Type or member is obsolete

			foreach (PropertyInfo prop in targetType.GetProperties(BindingFlags.Public | BindingFlags.Instance))
			{
				if (!prop.CanWrite)
				{
					continue;
				}

				string propName = prop.Name;

				if (!table.ContainsKey(propName))
				{
					continue;
				}

				object? value = table[propName];

				if (value == null)
				{
					prop.SetValue(instance, null);
					continue;
				}

				Type propType = prop.PropertyType;

				try
				{
					Type underlyingType = Nullable.GetUnderlyingType(propType) ?? propType;

					if (underlyingType.IsEnum)
					{
						string? valueString = value.ToString();

						if (String.IsNullOrEmpty(valueString))
						{
							continue;
						}

						object enumValue = Enum.Parse(underlyingType, value: valueString);
						prop.SetValue(instance, enumValue);
					}
					else if (underlyingType == typeof(Guid))
					{
						string? valueString = value.ToString();

						if (String.IsNullOrEmpty(valueString))
						{
							continue;
						}

						prop.SetValue(instance, Guid.Parse(input: value.ToString()!));
					}
					else if (underlyingType == typeof(DateTime))
					{
						prop.SetValue(instance, Convert.ChangeType(value, typeof(DateTime)));
					}
					else if (underlyingType == typeof(bool))
					{
						prop.SetValue(instance, Convert.ToBoolean(value));
					}
					else if (underlyingType.IsArray && value is IEnumerable enumerableValue)
					{
						Type? elemType = underlyingType.GetElementType();

						if (elemType == null)
						{
							continue;
						}

						object[] list = enumerableValue.Cast<object>()
							.Select(v => Convert.ChangeType(v, elemType))
							.ToArray();

						Array array = Array.CreateInstance(elemType, list.Length);
						Array.Copy(list, array, list.Length);

						prop.SetValue(instance, array);
					}
					else if (underlyingType.IsGenericType
							 && typeof(IEnumerable).IsAssignableFrom(underlyingType)
							 && value is IEnumerable enumerable)
					{
						prop.SetValue(instance, value);
					}
					else
					{
						try
						{
							object converted = Convert.ChangeType(value, underlyingType);
							prop.SetValue(instance, converted);
						}
						catch
						{
							// Fallback: try to map anonymous type to expected property type by property name
							object? fallbackValue = TryMapAnonymousToTypedObject(value, underlyingType);
							if (fallbackValue != null)
							{
								prop.SetValue(instance, fallbackValue);
							}
						}
					}
				}
				catch (Exception ex)
				{
					s_logger?.LogDebug("Failed to process prop:{PropName} with value: {Value} due to exception: {Ex}", propName, value, ex);
				}
			}

			return instance;
		}
	}
}
