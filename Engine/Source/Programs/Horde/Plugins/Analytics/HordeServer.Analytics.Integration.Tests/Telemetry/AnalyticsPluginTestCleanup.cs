// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Tests;

namespace HordeServer.Analytics.Integration.Tests.Telemetry
{
	[TestClass]
	public static class AssemblyHooks
	{
		[AssemblyCleanup]
		public static void Cleanup()
		{
			DatabaseIntegrationTest.Cleanup();
		}
	}
}