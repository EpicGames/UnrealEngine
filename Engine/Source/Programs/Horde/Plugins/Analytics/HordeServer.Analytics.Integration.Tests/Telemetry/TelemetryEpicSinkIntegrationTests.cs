// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Telemetry;
using HordeServer.Analytics.Tests;
using HordeServer.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Tests.Issues;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace HordeServer.Analytics.Integration.Tests.Telemetry
{
	/// <summary>
	/// Test Server Info used to prevent pollution of Epic Telemetry data.
	/// </summary>
	public class TestServerInfo : IServerInfo
	{
		#region -- Properties & Members --

		private readonly IServerInfo _serverInfo;

		#endregion -- Properties & Members --

		#region -- Constructor --

		public TestServerInfo(IServerInfo originalServerInfo)
		{
			_serverInfo = originalServerInfo;
		}

		#endregion -- Constructor --

		#region -- Interface API --

		/// <inheritdoc/>
		public SemVer Version => _serverInfo.Version;

		/// <inheritdoc/>
		public string Environment => "Local";

		/// <inheritdoc/>
		public string SessionId => _serverInfo.SessionId;

		/// <inheritdoc/>
		public DirectoryReference AppDir => _serverInfo.AppDir;

		/// <inheritdoc/>
		public DirectoryReference DataDir => _serverInfo.DataDir;

		/// <inheritdoc/>
		public IConfiguration Configuration => _serverInfo.Configuration;

		/// <inheritdoc/>
		public bool ReadOnlyMode => _serverInfo.ReadOnlyMode;

		/// <inheritdoc/>
		public bool EnableDebugEndpoints => _serverInfo.EnableDebugEndpoints;

		/// <inheritdoc/>
		public Uri ServerUrl => _serverInfo.ServerUrl;

		/// <inheritdoc/>
		public Uri DashboardUrl => _serverInfo.DashboardUrl;

		/// <inheritdoc/>
		public bool IsRunModeActive(RunMode mode)
		{
			return _serverInfo.IsRunModeActive(mode);
		}

		#endregion -- Interface API --
	}

	/// <summary>
	/// Test class used to drive data to the Epic Telemetry system in order to verify formats.
	/// </summary>
	/// <remarks>Primarily a data driver.</remarks>
	public class TelemetryEpicSinkIntegrationTests : AbstractIssueServiceTests
	{
		private readonly string _epicTelemetryURI = String.Empty;

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			EpicTelemetryConfig epicTelemetryConfig = new EpicTelemetryConfig();
			epicTelemetryConfig.Enabled = true;
			epicTelemetryConfig.Url = new Uri(_epicTelemetryURI);

			AnalyticsServerConfig analyticsServerConfig = new AnalyticsServerConfig();
			analyticsServerConfig.Sinks.Epic = epicTelemetryConfig;
			ServiceProvider provider = services.BuildServiceProvider();

			IServerInfo initialServerInfo = provider.GetRequiredService<IServerInfo>();

			services.AddSingleton(Options.Create(analyticsServerConfig));
			services.AddSingleton<IServerInfo>(sp =>
			{
				return new TestServerInfo(initialServerInfo);
			});
		}

		#region -- Test Cases --

		public async Task TestEpicSinkAsync()
		{
			// Configure the metric
			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			await UpdateConfigAsync(config =>
			{
				config.Plugins.GetBuildConfig().GetStream(MainStreamId).TelemetryStoreId = TelemetryStoreId.Default;
				config.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			});

			// Generate issue telemetry data
			{
				string[] lines =
				{
					@"WARNING: Engine\Source\Programs\UnrealBuildTool\ProjectFiles\Rider\ToolchainInfo.cs: Missing copyright boilerplate"
				};

				IJob job = CreateJob(MainStreamId, 105, "Test Build", Graph);
				await ParseEventsAsync(job, 0, 0, lines);
				await UpdateCompleteStepAsync(job, 0, 0, JobStepOutcome.Failure);
			}

			TelemetryManager telemetryManager = ServiceProvider.GetRequiredService<TelemetryManager>();
			await telemetryManager.FlushAsync(CancellationToken.None);
		}

		#endregion -- Test Cases --
	}
}
