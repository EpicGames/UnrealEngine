// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections;
using System.Reflection;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using HordeCommon;
using HordeServer.Analytics.Tests;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.Graphs;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Tests;
using HordeServer.Tests.Issues;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Analytics.Integration.Tests.Telemetry
{
	/// <summary>
	/// Test telemetry writer. Used in dependency injection, and will track all received telemetry events for subsequent assertions.
	/// </summary>
	public class TestTelemetryWriter : ITelemetryWriter
	{
		#region -- Static API --

		private static Hashtable ToHashtable(object anon)
		{
			Hashtable resultingTable = new Hashtable();
			foreach (PropertyInfo prop in anon.GetType().GetProperties(BindingFlags.Instance | BindingFlags.Public))
			{
				resultingTable[prop.Name] = prop.GetValue(anon, null);
			}
			return resultingTable;
		}

		#endregion -- Static API --

		#region -- Members & Properties --

		public bool Enabled => true;
		public IReadOnlyDictionary<TelemetryStoreId, IList<Tuple<TelemetryRecordMeta?, Hashtable>>> RecordedTelemetry => _recordedTelemetry;
		private readonly Dictionary<TelemetryStoreId, IList<Tuple<TelemetryRecordMeta?, Hashtable>>> _recordedTelemetry;

		#endregion -- Members & Properties --

		#region -- Constructor --

		public TestTelemetryWriter()
		{
			_recordedTelemetry = new Dictionary<TelemetryStoreId, IList<Tuple<TelemetryRecordMeta?, Hashtable>>>();
		}

		#endregion -- Constructor --

		#region -- Public API --

		internal void ClearStoredTelemetry()
		{
			_recordedTelemetry.Clear();
		}

		#endregion -- Public API --

		#region -- Interface API --

		public void WriteEvent(TelemetryStoreId telemetryStoreId, object payload)
		{
#pragma warning disable CS8625 // Cannot convert null literal to non-nullable reference type.
			WriteEvent(telemetryStoreId, null, payload);
#pragma warning restore CS8625 // Cannot convert null literal to non-nullable reference type.
		}

		public void WriteEvent(TelemetryStoreId telemetryStoreId, TelemetryRecordMeta metadata, object payload)
		{
			IList<Tuple<TelemetryRecordMeta?, Hashtable>>? telemetryStorePayloads;
			Hashtable convertedPayload = ToHashtable(payload);

			if (_recordedTelemetry.TryGetValue(telemetryStoreId, out telemetryStorePayloads))
			{
				telemetryStorePayloads.Add(new Tuple<TelemetryRecordMeta?, Hashtable>(null, convertedPayload));
			}
			else
			{
				ToHashtable(payload);
				telemetryStorePayloads = new List<Tuple<TelemetryRecordMeta?, Hashtable>>();
				telemetryStorePayloads.Add(new Tuple<TelemetryRecordMeta?, Hashtable>(null, convertedPayload));
				_recordedTelemetry.Add(telemetryStoreId, telemetryStorePayloads);
			}
		}

		#endregion -- Interface API --
	}

	/// <summary>
	/// Test class used to test the integration of <see cref="AnalyticsPlugin"/> with the Issue system.
	/// </summary>
	[TestClass]
	public class IssueTelemetryIntegrationTests : AbstractIssueServiceTests
	{
		#region -- Test Constants --

		const string EventNameKeyStr = "EventName";

		#endregion -- Test Constants -- 

		#region -- Interface API --

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			services.AddSingleton<TestTelemetryWriter>();
			services.AddSingleton<ITelemetryWriter>(sp => sp.GetRequiredService<TestTelemetryWriter>());
		}

		#endregion -- Interface API --

		#region -- Test Cases --

		/// <summary>
		/// Test to verify that <see cref="IssueCollection"/> is properly emitting through the <see cref="ITelemetryWriter"/>.
		/// </summary>
		[TestMethod]
		public async Task TestStateIssueAndIssueSpanEmissionAsync()
		{
			// Configure the metric
			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			await UpdateConfigAsync(config =>
			{
				config.Plugins.GetBuildConfig().GetStream(MainStreamId).TelemetryStoreId = TelemetryStoreId.Default;
				config.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			});

			// Test variables
			TestTelemetryWriter concreteTelemetryWriter = ServiceProvider.GetRequiredService<TestTelemetryWriter>();

			// Validate our preconditions
			Assert.IsNotNull(concreteTelemetryWriter);
			Assert.IsFalse(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

			// Generate issue telemetry data
			{
				string[] lines =
				{
					@"WARNING: Engine\Source\Programs\UnrealBuildTool\ProjectFiles\Rider\ToolchainInfo.cs: Missing copyright boilerplate"
				};

				IJob job = CreateJob(MainStreamId, 105, "Test Build", Graph);
				await ParseEventsAsync(job, 0, 0, lines);
				await UpdateCompleteStepAsync(job, 0, 0, JobStepOutcome.Failure);

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of Issue telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Select(x => x).Where(x => x.Item2[EventNameKeyStr] as string == IssueTelemetry.DefaultEventName).Any());
				Assert.IsTrue(actualTelemetryPayloads.Select(x => x).Where(x => x.Item2[EventNameKeyStr] as string == IssueSpanTelemetry.DefaultEventName).Any());
			}

			// Update the issue and make sure we get back external issues
			const string ExpectedExternalIssueKey = "NewIssueKey-1010";
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				IReadOnlyList<IIssue> issues = await IssueCollection.FindIssuesAsync();
				await IssueService.UpdateIssueAsync(issues[0].Id, externalIssueKey: ExpectedExternalIssueKey);

				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Hashtable actualStateIssuePayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == IssueTelemetry.DefaultEventName).First().Item2;
				IssueTelemetry? actualIssueTelemetry = actualStateIssuePayload.MapToObject<IssueTelemetry>();

				// Verify new payloads have been set for external issue
				Assert.AreEqual(ExpectedExternalIssueKey, actualIssueTelemetry.ExternalIssueKey);
			}

			// Generate issues with fingerprints
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				string[] lines =
				{
					FileReference.Combine(WorkspaceDir, "FOO.CPP").FullName + @"(170) : warning C6011: Dereferencing NULL pointer 'CurrentProperty'. : Lines: 159, 162, 163, 169, 170, 174, 176, 159, 162, 163, 169, 170",
					FileReference.Combine(WorkspaceDir, "foo.cpp").FullName + @"(170) : warning C6011: Dereferencing NULL pointer 'CurrentProperty'. : Lines: 159, 162, 163, 169, 170, 174, 176, 159, 162, 163, 169, 170"
				};

				IJob job = CreateJob(MainStreamId, 120, "Compile Test", Graph);
				await ParseEventsAsync(job, 0, 0, lines);
				await UpdateCompleteStepAsync(job, 0, 0, JobStepOutcome.Failure);

				IReadOnlyList<IIssue> expectedIssues = await IssueCollection.FindIssuesAsync();
				IIssue expectedIssue = expectedIssues[0];
				IIssueFingerprint expectedFingerprint = expectedIssue.Fingerprints[0];
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Hashtable actualStateIssueSpanPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == IssueSpanTelemetry.DefaultEventName).First().Item2;
				IssueSpanTelemetry? actualIssueSpanTelemetry = actualStateIssueSpanPayload.MapToObject<IssueSpanTelemetry>();

				// Verify new payloads have been set for fingerprint
				Assert.IsNotNull(actualIssueSpanTelemetry);
				Assert.AreEqual(expectedFingerprint.Type, actualIssueSpanTelemetry.Fingerprint.Type);
				Assert.AreEqual(expectedFingerprint.Keys.Count, actualIssueSpanTelemetry.Fingerprint.Keys.Count);
				Assert.AreEqual(expectedFingerprint.Keys.First(), actualIssueSpanTelemetry.Fingerprint.Keys.First());
			}
		}

		#endregion -- Test Cases --
	}

	/// <summary>
	/// Test class used to test the integration of <see cref="AnalyticsPlugin"/> with the Job system.
	/// </summary>
	[TestClass]
	public class JobTelemetryIntegrationTests : BuildTestSetup
	{
		#region -- Test Helpers --

		#region -- Test Graph Strings --

		private const string Win64ToolNodeName = "Compile UnrealHeaderTool Win64";
		private const string Win64GameNodeName = "Compile ShooterGameEditor Win64";

		private const string MacToolNodeName = "Compile UnrealHeaderTool Mac";
		private const string MacGameNodeName = "Compile ShooterGameEditor Mac";

		private const string AllWin64Label = "Label_AllWin64Nodes";
		private const string AllGameLabel = "Label_AllGameNodes";
		private const string ToolWin64Label = "Label_Win64UHTOnly";
		private const string AllToolLabel = "Label_AllTools";

		private const string Game1Category = "Game_1";
		private const string Win64Category = "Win64";

		private const string Aggregate = "AllNodes";

		#endregion -- Test Graph Strings --

		private async Task<(TemplateId templateRefId1, StreamConfig streamConfig, TestTelemetryWriter concreteTelemetryWriter)> SetupTelemetryTestPreconditionsAsync()
		{
			// Configure the metric
			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			// Setup Project, Stream, Template
			ProjectId projectId = new ProjectId("ue5");
			StreamId streamId = new StreamId("ue5-main");
			TemplateId templateRefId1 = new TemplateId("template1");
			TemplateId templateRefId2 = new TemplateId("template2");

			StreamConfig streamConfig = new StreamConfig { Id = streamId };
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId1, Name = "Test Template" });
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { templateRefId1, templateRefId2 } });

			ProjectConfig projectConfig = new ProjectConfig { Id = projectId };
			projectConfig.Streams.Add(streamConfig);

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);

			await SetConfigAsync(globalConfig);

			buildConfig.GetStream(streamId).TelemetryStoreId = TelemetryStoreId.Default;

			// Initialize the test telemetry writer
			TestTelemetryWriter concreteTelemetryWriter = ServiceProvider.GetRequiredService<TestTelemetryWriter>();
			return (templateRefId1, streamConfig, concreteTelemetryWriter);
		}

		private static List<NewLabel> BasicUncategorized()
		{
			return new List<NewLabel>
			{
				CreateLabel(AllWin64Label, [Win64ToolNodeName, Win64GameNodeName]),
				CreateLabel(ToolWin64Label, [Win64ToolNodeName]),
				CreateLabel(AllToolLabel,  [Win64ToolNodeName, MacToolNodeName])
			};
		}

		private static List<NewLabel> BasicCategorized()
		{
			return new List<NewLabel>
			{
				CreateLabel(AllWin64Label, [Win64ToolNodeName, Win64GameNodeName], Win64Category),
				CreateLabel(ToolWin64Label, [Win64ToolNodeName], Win64Category),
				CreateLabel(AllToolLabel,  [Win64ToolNodeName, MacToolNodeName]),
				CreateLabel(AllGameLabel, [Win64GameNodeName, MacGameNodeName], Game1Category)
			};
		}

		private static NewLabel CreateLabel(string labelName, List<string> nodes, string? category = null)
		{
			NewLabel label = new NewLabel();
			label.DashboardName = labelName;
			label.IncludedNodes = nodes;
			label.RequiredNodes = nodes;

			if (category != null)
			{
				label.DashboardCategory = category;
			}

			return label;
		}

		private static Task<IGraph> UpdateGraph(IGraphCollection graphCollection, IGraph rootGraph, Func<List<NewLabel>> labelGenerator, ITemplate _)
		{
			// Nodes

			List<NewNode> nodes1 = new List<NewNode>
			{
				new NewNode(Win64ToolNodeName),
				new NewNode(Win64GameNodeName),
			};

			List<NewNode> nodes2 = new List<NewNode>
			{
				new NewNode(MacToolNodeName),
				new NewNode(MacGameNodeName),
			};

			// Labels
			List<NewLabel> labels = labelGenerator();

			// Groups
			NewGroup group1 = new NewGroup("DefaultAgent1", nodes1);
			NewGroup group2 = new NewGroup("DefaultAgent2", nodes2);

			NewAggregate aggregate = new NewAggregate(Aggregate, nodes1.Union(nodes2).Select(x => x.Name).ToList());

			return graphCollection.AppendAsync(rootGraph, [group1, group2], [aggregate], labels);
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			services.AddSingleton<TestTelemetryWriter>();
			services.AddSingleton<ITelemetryWriter>(sp => sp.GetRequiredService<TestTelemetryWriter>());
		}

		#endregion -- Test Helpers --

		#region -- Test Constants --

		const string EventNameKeyStr = "EventName";

		#endregion -- Test Constants -- 

		[TestMethod]
		public async Task TestJobSummaryEmissionAsync()
		{
			(TemplateId templateRefId1, StreamConfig streamConfig, TestTelemetryWriter concreteTelemetryWriter) = await SetupTelemetryTestPreconditionsAsync();

			// Validate our preconditions
			Assert.IsNotNull(concreteTelemetryWriter);
			Assert.IsFalse(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

			{
				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);

				IGraph graph = await GraphCollection.AddAsync(template);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);

				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));
			}

			// Validate we have received a telemetry event.
			Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

			// Verify presence of job summary telemetry
			IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
			Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobSummaryTelemetry.DefaultEventName));

			// Assert we actually have the expected telemetry type.
			Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobSummaryTelemetry.DefaultEventName).First().Item2;
			JobSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobSummaryTelemetry>();
			Assert.IsNotNull(actualJobSummaryTelemetry);
		}

		[TestMethod]
		public async Task TestJobLabelSummaryEmissionAsync()
		{
			(TemplateId templateRefId1, StreamConfig streamConfig, TestTelemetryWriter concreteTelemetryWriter) = await SetupTelemetryTestPreconditionsAsync();

			// Validate our preconditions
			Assert.IsNotNull(concreteTelemetryWriter);
			Assert.IsFalse(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

			// Nominal case; all success
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicUncategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetrySuccess", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();
				Assert.IsNotNull(actualJobSummaryTelemetry);

				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(ToolWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(AllWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(AllToolLabel));
				Assert.IsFalse(actualJobSummaryTelemetry.FailedLabels.Any());

				// Assert individualized label telemetry
				IEnumerable<Hashtable> payloads = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelTelemetry.DefaultEventName).Select(x => x.Item2);
				Assert.AreEqual(3, payloads.Count());

				Hashtable actualStateJobLabelPayload = payloads.First();
				JobLabelTelemetry? actualJobLabelPayload = actualStateJobLabelPayload.MapToObject<JobLabelTelemetry>();
				Assert.IsNotNull(actualJobLabelPayload);
			}

			// Label failures - cross batch
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicUncategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(ToolWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(AllWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.FailedLabels.Contains(AllToolLabel));
			}

			// Label warnings - within batch
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicUncategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Warnings)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				Assert.IsTrue(actualJobSummaryTelemetry.WarningLabels.Contains(ToolWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.WarningLabels.Contains(AllWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.WarningLabels.Contains(AllToolLabel));
			}

			// Label failures - within batch
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicUncategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				Assert.IsTrue(actualJobSummaryTelemetry.FailedLabels.Contains(ToolWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.FailedLabels.Contains(AllWin64Label));
				Assert.IsTrue(actualJobSummaryTelemetry.FailedLabels.Contains(AllToolLabel));
			}

			// Label Category success - cross batch
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicCategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				// Assert that the category prefixed label is present.
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(Win64Category, ToolWin64Label)));
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulLabels.Contains(JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(Win64Category, AllWin64Label)));
				Assert.IsTrue(actualJobSummaryTelemetry.SuccessfulCategories.Contains(Win64Category));
			}

			// Label Category failure - cross batch
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicCategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				// Assert that the category prefixed label is present.
				Assert.IsTrue(actualJobSummaryTelemetry.FailedLabels.Contains(JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(Game1Category, AllGameLabel)));
				Assert.IsTrue(actualJobSummaryTelemetry.FailedCategories.Contains(Game1Category));
			}

			// Label Category warning - within categroy - warning label is higher precedence than success label
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicCategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Warnings)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;
				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				// Assert that the category prefixed label is present.
				Assert.IsTrue(actualJobSummaryTelemetry.WarningCategories.Contains(Win64Category));
			}

			// Label Category failurue - within category - failed label is higher precedence than warning label
			{
				// Reset the writer
				concreteTelemetryWriter.ClearStoredTelemetry();

				CreateJobOptions options = new CreateJobOptions();
				options.PreflightCommitId = CommitId.FromPerforceChange(999);

				ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);
				IGraph graph = await GraphCollection.AddAsync(template);
				graph = await UpdateGraph(GraphCollection, graph, new Func<List<NewLabel>>(() => { return BasicCategorized(); }), template);

				options.Arguments.Add(IJob.TargetArgumentPrefix + Aggregate);

				IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "TestJobLabelTelemetryFailure", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));

				// Batch 0
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));

				// Batch 1
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				// Batch 2
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Running)); // Game

				FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

				await clock.AdvanceAsync(TimeSpan.FromSeconds(60.0));

				// Batch 0 - Setup Build
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

				// Batch 1 - Win64
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Warnings)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Failure)); // Game

				// Batch 2 - Mac
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Tool
				job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[1].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success)); // Game

				// Validate we have received a telemetry event.
				Assert.IsTrue(concreteTelemetryWriter.RecordedTelemetry.ContainsKey(TelemetryStoreId.Default));

				// Verify presence of job summary telemetry
				IList<Tuple<TelemetryRecordMeta?, Hashtable>> actualTelemetryPayloads = [.. concreteTelemetryWriter.RecordedTelemetry[TelemetryStoreId.Default]];
				Assert.IsTrue(actualTelemetryPayloads.Any(x => (x.Item2[EventNameKeyStr] as string) == JobLabelSummaryTelemetry.DefaultEventName));

				// Assert we actually have the expected telemetry type.
				Hashtable actualStateJobSummaryPayload = actualTelemetryPayloads.Where(x => x.Item2[EventNameKeyStr] as string == JobLabelSummaryTelemetry.DefaultEventName).First().Item2;

				JobLabelSummaryTelemetry? actualJobSummaryTelemetry = actualStateJobSummaryPayload.MapToObject<JobLabelSummaryTelemetry>();

				Assert.IsNotNull(actualJobSummaryTelemetry);

				// Assert that the category prefixed label is present.
				Assert.IsTrue(actualJobSummaryTelemetry.FailedCategories.Contains(Win64Category));
			}
		}
	}
}