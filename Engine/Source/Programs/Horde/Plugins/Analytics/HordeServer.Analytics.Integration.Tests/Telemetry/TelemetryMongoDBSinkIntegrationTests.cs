// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Telemetry;
using HordeServer.Analytics.Tests;
using HordeServer.Server;
using HordeServer.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Tests.Issues;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Driver;

namespace HordeServer.Analytics.Integration.Tests.Telemetry
{
	/// <summary>
	/// Test class used to drive data to the Mongo DB Telemetry system in order to verify formats.
	/// </summary>
	/// <remarks>Primarily a data driver.</remarks>
	[TestClass]
	public class TelemetryMongoDBSinkIntegrationTests : AbstractIssueServiceTests
	{
		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			MongoTelemetryConfig mongoTelemetryConfig = new MongoTelemetryConfig();
			mongoTelemetryConfig.Enabled = true;
			mongoTelemetryConfig.RetainDays = 1;

			AnalyticsServerConfig analyticsServerConfig = new AnalyticsServerConfig();
			analyticsServerConfig.Sinks.Mongo = mongoTelemetryConfig;
			services.AddSingleton(Options.Create(analyticsServerConfig));
		}

		#region -- Test Cases --

		public async Task TestMongoDBTelemetrySinkAsync()
		{
			// Configure the metric
			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			await UpdateConfigAsync(config =>
			{
				config.Plugins.GetBuildConfig().GetStream(MainStreamId).TelemetryStoreId = TelemetryStoreId.Default;
				config.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			});

			IMongoService mongoService = ServiceProvider.GetRequiredService<IMongoService>();
			IMongoCollection<BsonDocument> collection = mongoService.Database.GetCollection<BsonDocument>("Telemetry");
			long count = await collection.CountDocumentsAsync(FilterDefinition<BsonDocument>.Empty);
			Assert.AreEqual(0, count);
			// Generate issue telemetry data
			{
				string[] lines =
				{
					@"WARNING: Engine\Source\Programs\UnrealBuildTool\ProjectFiles\Rider\ToolchainInfo.cs: Missing copyright boilerplate"
				};

				IJob job = CreateJob(MainStreamId, 105, "Test Build", Graph);
				await ParseEventsAsync(job, 0, 0, lines);
				await UpdateCompleteStepAsync(job, 0, 0, JobStepOutcome.Failure);
			}

			TelemetryManager telemetryManager = ServiceProvider.GetRequiredService<TelemetryManager>();
			await telemetryManager.FlushAsync(CancellationToken.None);

			count = await collection.CountDocumentsAsync(FilterDefinition<BsonDocument>.Empty);
			Assert.IsTrue(count > 0);
		}

		#endregion -- Test Cases --
	}
}
