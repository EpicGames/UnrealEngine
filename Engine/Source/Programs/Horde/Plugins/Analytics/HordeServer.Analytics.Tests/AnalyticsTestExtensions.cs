// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer.Analytics.Tests
{
	public static class AnalyticsTestExtensions
	{
		public static void AddAnalyticsTestConfig(this PluginConfigCollection configCollection, AnalyticsConfig config)
			=> configCollection[new PluginName("Analytics")] = config;
	}
}
