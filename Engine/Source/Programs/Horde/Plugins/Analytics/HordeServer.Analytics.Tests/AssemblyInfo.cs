// Copyright Epic Games, Inc. All Rights Reserved.

[assembly: Parallelize(Workers = 0, Scope = ExecutionScope.MethodLevel)]