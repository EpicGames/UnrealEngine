<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\HordeServer.Analytics\HordeServer.Analytics.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Tests.Shared\HordeServer.Tests.Shared.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>

</Project>
