// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Telemetry.Metrics;
using HordeServer.Server;
using HordeServer.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Telemetry.Sinks;
using HordeServer.Tests;
using HordeServer.Utilities;
using Json.Path;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Analytics.Tests.Telemetry
{
	[TestClass]
	public class TelemetryTests : ServerTestSetup
	{
		static readonly TelemetryRecordMeta s_metadata = new TelemetryRecordMeta();

		public TelemetryTests()
		{
			AddPlugin<AnalyticsPlugin>();
		}

		[TestMethod]
		public async Task FilterAsync()
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric");
			metricConfig.Function = AggregationFunction.Sum;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Filter = JsonPath.Parse("$[?(@.Payload.EventName == 'Included')]");
			metricConfig.Property = JsonPath.Parse("$.Payload.foo");

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			// Test 1
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 1 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 2 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Excluded", foo = 3 }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
				Assert.AreEqual(1, metrics.Count);
				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics[0].Time);
				Assert.AreEqual(3, metrics[0].Value);
				Assert.AreEqual(2, metrics[0].Count);
			}
		}

		[TestMethod]
		public async Task SingleMetricAsync()
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric");
			metricConfig.Function = AggregationFunction.Sum;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Property = JsonPath.Parse("$.Payload.foo");

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			// Test 1
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 1 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 2 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 3 }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
				Assert.AreEqual(1, metrics.Count);
				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics[0].Time);
				Assert.AreEqual(6, metrics[0].Value);
				Assert.AreEqual(3, metrics[0].Count);
			}

			// Test 2
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 3 }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
				Assert.AreEqual(1, metrics.Count);
				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics[0].Time);
				Assert.AreEqual(9, metrics[0].Value);
				Assert.AreEqual(4, metrics[0].Count);
			}

			// Test 3
			{
				await Clock.AdvanceAsync(TimeSpan.FromHours(1.0));

				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 4 }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
				Assert.AreEqual(2, metrics.Count);

				Assert.AreEqual(new DateTime(2023, 6, 8, 5, 0, 0), metrics[0].Time);
				Assert.AreEqual(4, metrics[0].Value);
				Assert.AreEqual(1, metrics[0].Count);

				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics[1].Time);
				Assert.AreEqual(9, metrics[1].Value);
				Assert.AreEqual(4, metrics[1].Count);
			}
		}

		[TestMethod]
		public async Task SeparateMetricsAsync()
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig1 = new MetricConfig();
			metricConfig1.Id = new MetricId("test-metric-1");
			metricConfig1.Function = AggregationFunction.Sum;
			metricConfig1.Interval = TimeSpan.FromHours(1.0);
			metricConfig1.Property = JsonPath.Parse("$.Payload.foo");

			MetricConfig metricConfig2 = new MetricConfig();
			metricConfig2.Id = new MetricId("test-metric-2");
			metricConfig2.Function = AggregationFunction.Sum;
			metricConfig2.Interval = TimeSpan.FromHours(1.0);
			metricConfig2.Property = JsonPath.Parse("$.Payload.bar.baz");

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig1);
			telemetryStoreConfig.Metrics.Add(metricConfig2);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			// Test 1
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 1 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 2, bar = 201 }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 3, bar = new { baz = 101 } }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig1.Id]);
				Assert.AreEqual(1, metrics.Count);
				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics[0].Time);
				Assert.AreEqual(6, metrics[0].Value);
				Assert.AreEqual(3, metrics[0].Count);

				List<IMetric> metrics2 = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig2.Id]);
				Assert.AreEqual(1, metrics2.Count);
				Assert.AreEqual(new DateTime(2023, 6, 8, 4, 0, 0), metrics2[0].Time);
				Assert.AreEqual(101, metrics2[0].Value);
				Assert.AreEqual(1, metrics2[0].Count);
			}
		}
		/*
	[TestMethod]
	public async Task FunctionTestAsync()
	{
		await SingleFunctionTestAsync(AggregationFunction.Count, new double[] { 5, 4, 3, -1, 2 }, 5);
		await SingleFunctionTestAsync(AggregationFunction.Min, new double[] { 5, 4, 3, -1, 2 }, -1);
		await SingleFunctionTestAsync(AggregationFunction.Max, new double[] { 5, 4, 3, -1, 2 }, 5);
		await SingleFunctionTestAsync(AggregationFunction.Sum, new double[] { 5, 4, 3, -1, 2 }, 13);
		await SingleFunctionTestAsync(AggregationFunction.Average, new double[] { 5, 4, 3, -1, 2 }, 2.6);
		await SingleFunctionTestAsync(AggregationFunction.Percentile, 4.25);
	}
		*/
		[TestMethod]
		[DataRow(AggregationFunction.Count, 5)]
		[DataRow(AggregationFunction.Min, -1)]
		[DataRow(AggregationFunction.Max, 5)]
		[DataRow(AggregationFunction.Sum, 13)]
		[DataRow(AggregationFunction.Average, 2.6)]
		[DataRow(AggregationFunction.Percentile, 4.25)]
		public async Task SingleFunctionTestAsync(AggregationFunction function, double result)
		{
			double[] values = new double[] { 5, 4, 3, -1, 2 };

			await Clock.AdvanceAsync(TimeSpan.FromDays(1.0));

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric-2");
			metricConfig.Function = function;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Property = JsonPath.Parse("$.Payload.foo");
			metricConfig.Percentile = 75;

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			foreach (double value in values)
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = value }));
			}

			await sink.FlushAsync(CancellationToken.None);
			await collection.FlushAsync(CancellationToken.None);

			List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id], maxResults: 1);
			Assert.AreEqual(1, metrics.Count);
			Assert.AreEqual(values.Length, metrics[0].Count);
			Assert.AreEqual(result, metrics[0].Value);
		}

		[TestMethod]
		public async Task GroupingTestAsync()
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric-1");
			metricConfig.Function = AggregationFunction.Sum;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Property = JsonPath.Parse("$.Payload.foo");
			metricConfig.GroupBy = "$.Payload.group";

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 1, group = "first" }));
			sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 2, group = "first" }));
			sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 3, group = "first" }));
			sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 4, group = "second" }));
			sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 5 }));
			await sink.FlushAsync(CancellationToken.None);
			await collection.FlushAsync(CancellationToken.None);

			List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
			metrics = metrics.OrderBy(x => x.Group).ToList();
			Assert.AreEqual(3, metrics.Count);

			Assert.AreEqual("", metrics[0].Group);
			Assert.AreEqual(1, metrics[0].Count);
			Assert.AreEqual(5, metrics[0].Value);

			Assert.AreEqual("first", metrics[1].Group);
			Assert.AreEqual(3, metrics[1].Count);
			Assert.AreEqual(6, metrics[1].Value);

			Assert.AreEqual("second", metrics[2].Group);
			Assert.AreEqual(1, metrics[2].Count);
			Assert.AreEqual(4, metrics[2].Value);
		}

		[TestMethod]
		public async Task GroupingEscapeTestAsync()
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric-1");
			metricConfig.Function = AggregationFunction.Sum;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Property = JsonPath.Parse("$.Payload.foo");
			metricConfig.GroupBy = "$.Payload.group";

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { foo = 123, group = "first, second & third" }));
			await sink.FlushAsync(CancellationToken.None);
			await collection.FlushAsync(CancellationToken.None);

			List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
			metrics = metrics.OrderBy(x => x.Group).ToList();
			Assert.AreEqual(1, metrics.Count);

			Assert.AreEqual("\"first, second & third\"", metrics[0].Group);
			Assert.AreEqual(1, metrics[0].Count);
			Assert.AreEqual(123, metrics[0].Value);
		}

		[TestMethod]
		public void GroupConverterTest()
		{
			JsonSerializerOptions serializerOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(serializerOptions);

			MetricConfig? config = JsonSerializer.Deserialize<MetricConfig>("{ \"groupBy\": \"$.foo,$.bar\" }", serializerOptions);
			Assert.IsNotNull(config);
			Assert.AreEqual(2, config.GroupByPaths.Count);
			Assert.AreEqual("$.foo", config.GroupByPaths[0].ToString());
			Assert.AreEqual("$.bar", config.GroupByPaths[1].ToString());

			string text = JsonSerializer.Serialize(config, serializerOptions);
			Assert.IsTrue(text.Contains("\"groupBy\":\"$.foo,$.bar\"", StringComparison.Ordinal));
		}

		[TestMethod]
		[DataRow(AggregationFunction.Count)]
		[DataRow(AggregationFunction.Sum)]
		public async Task MultiGroupingTestAsync(AggregationFunction function)
		{
			Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

			MetricConfig metricConfig = new MetricConfig();
			metricConfig.Id = new MetricId("test-metric");
			metricConfig.Function = function;
			metricConfig.Interval = TimeSpan.FromHours(1.0);
			metricConfig.Filter = JsonPath.Parse("$[?(@.Payload.EventName == 'Included')]");
			metricConfig.GroupBy = "$.Payload.groupFacetA, $.Payload.groupFacetB, $.Payload.groupFacetC";
			if (function == AggregationFunction.Sum)
			{
				metricConfig.Property = JsonPath.Parse("$.Payload.foo");
			}

			TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
			telemetryStoreConfig.Id = TelemetryStoreId.Default;
			telemetryStoreConfig.Metrics.Add(metricConfig);

			AnalyticsConfig analyticsConfig = new AnalyticsConfig();
			analyticsConfig.Stores.Add(telemetryStoreConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
			await SetConfigAsync(globalConfig);

			MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
			IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

			// Test 1
			{
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 1, groupFacetA = "groupA" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 2, groupFacetA = "groupA" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 3, groupFacetA = "groupA", groupFacetB = "groupB" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 4, groupFacetB = "groupB" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 5, groupFacetB = "groupA,groupB" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Included", foo = 6, groupFacetA = "groupA", groupFacetB = "groupB", groupFacetC = "groupC" }));
				sink.SendEvent(telemetryStoreConfig.Id, new TelemetryEvent(s_metadata, new { EventName = "Excluded", foo = 6 }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id]);
				metrics = metrics.OrderBy(x => x.Group).ToList();

				Assert.AreEqual(5, metrics.Count);

				Assert.AreEqual(",\"groupA,groupB\",", metrics[0].Group);
				Assert.AreEqual(",groupB,", metrics[1].Group);
				Assert.AreEqual("groupA,,", metrics[2].Group);
				Assert.AreEqual("groupA,groupB,", metrics[3].Group);
				Assert.AreEqual("groupA,groupB,groupC", metrics[4].Group);

				if (function == AggregationFunction.Sum)
				{
					Assert.AreEqual(1, metrics[0].Count);
					Assert.AreEqual(5, metrics[0].Value);

					Assert.AreEqual(1, metrics[1].Count);
					Assert.AreEqual(4, metrics[1].Value);

					Assert.AreEqual(2, metrics[2].Count);
					Assert.AreEqual(3, metrics[2].Value);

					Assert.AreEqual(1, metrics[3].Count);
					Assert.AreEqual(3, metrics[3].Value);

					Assert.AreEqual(1, metrics[4].Count);
					Assert.AreEqual(6, metrics[4].Value);
				}

				if (function == AggregationFunction.Count)
				{
					for (int i = 0; i < 5; i++)
					{
						Assert.AreEqual(i == 2 ? 2 : 1, metrics[i].Count);
						Assert.AreEqual(i == 2 ? 2 : 1, metrics[i].Value);
					}
				}
			}
		}

		/*
		[TestMethod]
		public async Task TopAndBottomAsync()
		{   
			// TopN
			{
				Clock.UtcNow = new DateTime(2023, 6, 8, 4, 30, 0, DateTimeKind.Utc);

				MetricConfig metricConfig = new MetricConfig();
				metricConfig.Id = new MetricId("test-metric-1");
				metricConfig.Function = AggregationFunction.Sum;
				metricConfig.Interval = TimeSpan.FromHours(1.0);
				metricConfig.Property = JsonPath.Parse("$.Payload.foo");
				metricConfig.GroupBy = "$.Payload.group";
				metricConfig.TopN = 3;

				TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
				telemetryStoreConfig.Id = TelemetryStoreId.Default;
				telemetryStoreConfig.Metrics.Add(metricConfig);

				AnalyticsConfig analyticsConfig = new AnalyticsConfig();
				analyticsConfig.Stores.Add(telemetryStoreConfig);

				GlobalConfig globalConfig = new GlobalConfig();
				globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
				SetConfig(globalConfig);

				MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
				IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 1, group = "first" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 2, group = "first" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 10, group = "first" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 4, group = "second" }));
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);
				
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 5, group = "third" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 6, group = "fourth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 7, group = "fifth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 8, group = "sixth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 9 }));
				
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetricMeta> metricMeta = await collection.FindMetaAsync(telemetryStoreConfig.Id, [metricConfig.Id]);

				Assert.AreEqual(1, metricMeta.Count);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id], null, null, metricMeta[0].Groups.ToArray());
				metrics = metrics.OrderBy(x => x.Group).ToList();
				Assert.AreEqual(3, metrics.Count);

				Assert.AreEqual("", metrics[0].Group);
				Assert.AreEqual(1, metrics[0].Count);
				Assert.AreEqual(9, metrics[0].Value);

				Assert.AreEqual("first", metrics[1].Group);
				Assert.AreEqual(3, metrics[1].Count);
				Assert.AreEqual(13, metrics[1].Value);

				Assert.AreEqual("sixth", metrics[2].Group);
				Assert.AreEqual(1, metrics[2].Count);
				Assert.AreEqual(8, metrics[2].Value);
			}

			// BottomN
			{
				Clock.UtcNow = new DateTime(2024, 6, 8, 4, 30, 0, DateTimeKind.Utc);

				MetricConfig metricConfig = new MetricConfig();
				metricConfig.Id = new MetricId("test-metric-1");
				metricConfig.Function = AggregationFunction.Sum;
				metricConfig.Interval = TimeSpan.FromHours(1.0);
				metricConfig.Property = JsonPath.Parse("$.Payload.foo");
				metricConfig.GroupBy = "$.Payload.group";
				metricConfig.BottomN = 3;

				TelemetryStoreConfig telemetryStoreConfig = new TelemetryStoreConfig();
				telemetryStoreConfig.Id = TelemetryStoreId.Default;
				telemetryStoreConfig.Metrics.Add(metricConfig);

				AnalyticsConfig analyticsConfig = new AnalyticsConfig();
				analyticsConfig.Stores.Add(telemetryStoreConfig);

				GlobalConfig globalConfig = new GlobalConfig();
				globalConfig.Plugins.AddAnalyticsTestConfig(analyticsConfig);
				SetConfig(globalConfig);

				MetricTelemetrySink sink = ServiceProvider.GetRequiredService<MetricTelemetrySink>();
				IMetricCollection collection = ServiceProvider.GetRequiredService<IMetricCollection>();

				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 1, group = "first" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 2, group = "first" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 10, group = "first" }));

				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 4, group = "second" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 5, group = "third" }));
				
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 5, group = "fourth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 1, group = "fourth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 7, group = "fifth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 8, group = "sixth" }));
				sink.SendEvent(TelemetryStoreId.Default, new TelemetryEvent(s_metadata, new { foo = 9 }));
				
				await sink.FlushAsync(CancellationToken.None);
				await collection.FlushAsync(CancellationToken.None);

				List<IMetricMeta> metricMeta = await collection.FindMetaAsync(telemetryStoreConfig.Id, [metricConfig.Id], new DateTime(2024, 1, 1, 1, 1, 1, DateTimeKind.Utc));

				Assert.AreEqual(1, metricMeta.Count);

				List<IMetric> metrics = await collection.FindAsync(telemetryStoreConfig.Id, [metricConfig.Id], new DateTime(2024, 1, 1, 1, 1, 1, DateTimeKind.Utc), null, metricMeta[0].Groups.ToArray());
				metrics = metrics.OrderBy(x => x.Group).ToList();
				Assert.AreEqual(3, metrics.Count);

				Assert.AreEqual("fourth", metrics[0].Group);
				Assert.AreEqual(2, metrics[0].Count);
				Assert.AreEqual(6, metrics[0].Value);

				Assert.AreEqual("second", metrics[1].Group);
				Assert.AreEqual(1, metrics[1].Count);
				Assert.AreEqual(4, metrics[1].Value);

				Assert.AreEqual("third", metrics[2].Group);
				Assert.AreEqual(1, metrics[2].Count);
				Assert.AreEqual(5, metrics[2].Value);
			}			
		}
		*/
	}
}
