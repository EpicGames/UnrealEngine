// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Telemetry;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Telemetry.Metrics;

#pragma warning disable CA2227 // Change X to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Config settings for analytics
	/// </summary>
	public class AnalyticsConfig : IPluginConfig
	{
		/// <summary>
		/// Metrics to aggregate on the Horde server
		/// </summary>
		public List<TelemetryStoreConfig> Stores { get; set; } = new List<TelemetryStoreConfig>();

		private AclConfig _parentAcl = null!;
		private readonly Dictionary<TelemetryStoreId, TelemetryStoreConfig> _telemetryStoreLookup = new Dictionary<TelemetryStoreId, TelemetryStoreConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			_parentAcl = configOptions.ParentAcl;

			_telemetryStoreLookup.Clear();
			foreach (TelemetryStoreConfig telemetryStore in Stores)
			{
				_telemetryStoreLookup.Add(telemetryStore.Id, telemetryStore);
				telemetryStore.PostLoad(configOptions.ParentAcl);
			}
		}

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> _parentAcl.Authorize(action, user);

		/// <summary>
		/// Attempts to get configuration for a pool from this object
		/// </summary>
		/// <param name="telemetryStoreId">The pool identifier</param>
		/// <param name="config">Configuration for the telemetry store</param>
		/// <returns>True if the telemetry configuration was found</returns>
		public bool TryGetTelemetryStore(TelemetryStoreId telemetryStoreId, [NotNullWhen(true)] out TelemetryStoreConfig? config)
			=> _telemetryStoreLookup.TryGetValue(telemetryStoreId, out config);
	}
}
