// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;
using HordeServer.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Telemetry.Sinks;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer
{
	/// <summary>
	/// Main entry point for the analytics plugin
	/// </summary>
	[Plugin("Analytics", GlobalConfigType = typeof(AnalyticsConfig), ServerConfigType = typeof(AnalyticsServerConfig))]
	public class AnalyticsPlugin : IPluginStartup
	{
		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{ }

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<TelemetryManager>();
			services.AddSingleton<ITelemetryWriter>(sp => sp.GetRequiredService<TelemetryManager>());
			services.AddHostedService(sp => sp.GetRequiredService<TelemetryManager>());
			services.AddSingleton<MongoTelemetrySink>();
			services.AddHostedService(sp => sp.GetRequiredService<MongoTelemetrySink>());
			services.AddSingleton<MetricTelemetrySink>();

			services.AddSingleton<MetricCollection>();
			services.AddHostedService(sp => sp.GetRequiredService<MetricCollection>());
			services.AddSingleton<IMetricCollection, MetricCollection>(sp => sp.GetRequiredService<MetricCollection>());

			services.AddHttpClient(EpicTelemetrySink.HttpClientName, client => { });
		}
	}
}
