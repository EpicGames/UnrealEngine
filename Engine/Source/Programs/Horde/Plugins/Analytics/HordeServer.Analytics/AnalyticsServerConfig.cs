// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;
using HordeServer.Telemetry.Sinks;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Server configuration for the analytics system
	/// </summary>
	public class AnalyticsServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Settings for the various telemetry sinks
		/// </summary>
		public TelemetrySinkConfig Sinks { get; } = new TelemetrySinkConfig();
	}

	/// <summary>
	/// Telemetry sinks
	/// </summary>
	public class TelemetrySinkConfig
	{
		/// <summary>
		/// Settings for the Epic telemetry sink
		/// </summary>
		public EpicTelemetryConfig? Epic { get; set; }

		/// <summary>
		/// Settings for the MongoDB telemetry sink
		/// </summary>
		public MongoTelemetryConfig? Mongo { get; set; }
	}

	/// <summary>
	/// Configuration for the telemetry sink
	/// </summary>
	public class BaseTelemetryConfig
	{
		/// <summary>
		/// Whether to enable this sink
		/// </summary>
		public bool Enabled { get; set; } = false;
	}

	/// <summary>
	/// Configuration for the telemetry sink
	/// </summary>
	public class EpicTelemetryConfig : BaseTelemetryConfig, IEpicTelemetrySinkConfig
	{
		/// <summary>
		/// Base URL for the telemetry server
		/// </summary>
		public Uri? Url { get; set; }

		/// <summary>
		/// Application name to send in the event messages
		/// </summary>
		public string AppId { get; set; } = "Horde";

		/// <inheritdoc />
		public override string ToString()
		{
			return $"{nameof(Url)}={Url} {nameof(AppId)}={AppId}";
		}
	}

	/// <summary>
	/// Configuration for the telemetry sink
	/// </summary>
	public class MongoTelemetryConfig : BaseTelemetryConfig
	{
		/// <summary>
		/// Number of days worth of telmetry events to keep
		/// </summary>
		public double RetainDays { get; set; } = 1;
	}
}
