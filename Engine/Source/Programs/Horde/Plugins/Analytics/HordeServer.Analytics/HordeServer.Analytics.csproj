<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="JsonPath.Net" Version="0.6.8" />
    <PackageReference Include="Meziantou.Analyzer" Version="2.0.220">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="T-Digest.NET" Version="1.0.3" />
  </ItemGroup>

</Project>
