// Copyright Epic Games, Inc. All Rights Reserved.

using System.Resources;
using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("HordeServer.Analytics.Tests")]
[assembly: InternalsVisibleTo("HordeServer.Analytics.Integration.Tests")]
[assembly: NeutralResourcesLanguage("en-US")]
