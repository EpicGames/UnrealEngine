// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Telemetry;

namespace HordeServer.Telemetry
{
	/// <summary>
	/// Interface for a telemetry sink
	/// </summary>
	public interface ITelemetrySink : IAsyncDisposable
	{
		/// <summary>
		/// Whether the sink is enabled
		/// </summary>
		bool Enabled { get; }

		/// <summary>
		/// Flush any queued metrics to underlying service
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Completion task</returns>
		ValueTask FlushAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Sends a telemetry event with the given information
		/// </summary>
		/// <param name="storeId">Identifier for the telemetry store</param>
		/// <param name="telemetryEvent">The telemetry event that was received</param>
		void SendEvent(TelemetryStoreId storeId, TelemetryEvent telemetryEvent);
	}
}

