// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Telemetry.Metrics;

namespace HordeServer.Telemetry.Metrics
{
	/// <summary>
	/// Interface for a metric event
	/// </summary>
	public interface IMetric
	{
		/// <summary>
		/// Unique identifier for this event
		/// </summary>
		MetricId MetricId { get; }

		/// <summary>
		/// Start time for this aggregation interval
		/// </summary>
		DateTime Time { get; }

		/// <summary>
		/// Grouping key for this metric
		/// </summary>
		string? Group { get; }

		/// <summary>
		/// Aggregated value for the metric
		/// </summary>
		double Value { get; }

		/// <summary>
		/// Number of samples
		/// </summary>
		int Count { get; }
	}

	/// <summary>
	/// Interface for metric meta data
	/// </summary>
	public interface IMetricMeta
	{
		/// <summary>
		/// Unique identifier for this event
		/// </summary>
		MetricId MetricId { get; }

		/// <summary>
		/// GroupBy for the metric
		/// </summary>
		string GroupBy { get; }

		/// <summary>
		/// Unique groups
		/// </summary>
		List<string> Groups { get; }

		/// <summary>
		/// TopN aggregation
		/// </summary>
		public int TopN { get; }

		/// <summary>
		/// BottomN aggregation
		/// </summary>
		public int BottomN { get; }
	}
}
