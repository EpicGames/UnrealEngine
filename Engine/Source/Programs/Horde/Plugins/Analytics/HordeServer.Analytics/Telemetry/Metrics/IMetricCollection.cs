// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json.Nodes;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Telemetry.Metrics;

namespace HordeServer.Telemetry.Metrics
{
	/// <summary>
	/// Collection of aggregated metrics
	/// </summary>
	public interface IMetricCollection
	{
		/// <summary>
		/// Adds a new event to the collection
		/// </summary>
		/// <param name="storeId">Identifier for a telemetry store</param>
		/// <param name="node">Arbitrary event node</param>
		void AddEvent(TelemetryStoreId storeId, JsonNode node);

		/// <summary>
		/// Finds metrics over a given time period
		/// </summary>
		/// <param name="storeId">Identifier for a telemetry store</param>
		/// <param name="metricIds">Metrics to search for</param>
		/// <param name="minTime">Start of the time period to query</param>
		/// <param name="maxTime">End of the time period to query</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<List<IMetricMeta>> FindMetaAsync(TelemetryStoreId storeId, MetricId[] metricIds, DateTime? minTime = null, DateTime? maxTime = null, int maxResults = 50, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds metrics over a given time period
		/// </summary>
		/// <param name="storeId">Identifier for a telemetry store</param>
		/// <param name="metricIds">Metrics to search for</param>
		/// <param name="minTime">Start of the time period to query</param>
		/// <param name="maxTime">End of the time period to query</param>
		/// <param name="groups">Grouping keys to filter results</param>		
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<List<IMetric>> FindAsync(TelemetryStoreId storeId, MetricId[] metricIds, DateTime? minTime = null, DateTime? maxTime = null, string[]? groups = null, int maxResults = 50, CancellationToken cancellationToken = default);

		/// <summary>
		/// Flush all the pending events to storage
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task FlushAsync(CancellationToken cancellationToken);
	}
}
