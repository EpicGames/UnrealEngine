// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text.Json.Nodes;
using System.Threading.Channels;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Telemetry.Metrics;
using HordeServer.Server;
using HordeServer.Utilities;
using Json.Path;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using TDigestNet;

namespace HordeServer.Telemetry.Metrics
{
	class MetricMeta : IMetricMeta
	{
		/// <summary>
		/// Unique identifier for this event
		/// </summary>
		public MetricId MetricId { get; set; }

		/// <summary>
		/// Metric GroupBy
		/// </summary>
		public string GroupBy { get; set; } = String.Empty;

		/// <summary>
		/// Unique groups
		/// </summary>
		public List<string> Groups { get; set; } = new List<string>();

		/// <summary>
		/// TopN aggregation
		/// </summary>
		public int TopN { get; set; } = 0;

		/// <summary>
		/// BottomN aggregation
		/// </summary>
		public int BottomN { get; set; } = 0;
	}

	class MetricCollection : IMetricCollection, IHostedService, IAsyncDisposable
	{
		class MetricDocument : IMetric
		{
			public ObjectId Id { get; set; }

			[BsonElement("ts")]
			public TelemetryStoreId TelemetryStoreId { get; set; }

			[BsonElement("met")]
			public MetricId MetricId { get; set; }

			[BsonElement("grp")]
			public string Group { get; set; } = String.Empty;

			[BsonElement("time")]
			public DateTime Time { get; set; }

			[BsonElement("value")]
			public double Value { get; set; }

			[BsonElement("count")]
			public int Count { get; set; }

			[BsonElement("state")]
			public byte[]? State { get; set; }
		}

		record class SampleKey(TelemetryStoreId Store, MetricId Metric, string Group, DateTime Time);

		record struct QualifiedMetricId(TelemetryStoreId StoreId, MetricId MetricId)
		{
			public override string ToString()
				=> $"{StoreId}:{MetricId}";
		}

		readonly Lock _lockObject = new Lock();
		readonly IMongoCollection<MetricDocument> _metrics;
		readonly AsyncEvent _newDataEvent = new AsyncEvent();
		readonly AsyncEvent _flushEvent = new AsyncEvent();
		readonly BackgroundTask _processTask;
		readonly BackgroundTask _flushTask;
		readonly IClock _clock;
		readonly IOptionsMonitor<AnalyticsConfig> _analyticsConfig;
		readonly ILogger _logger;

		int _numDroppedItems;
		readonly Channel<(TelemetryStoreId, JsonNode)> _events;
		readonly Dictionary<QualifiedMetricId, TimeSpan> _slowMetrics = new Dictionary<QualifiedMetricId, TimeSpan>();
		Dictionary<SampleKey, List<double>> _queuedSamples = new Dictionary<SampleKey, List<double>>();

		public MetricCollection(IMongoService mongoService, IClock clock, IOptionsMonitor<AnalyticsConfig> analyticsConfig, ILogger<MetricCollection> logger)
		{
			List<MongoIndex<MetricDocument>> indexes = new List<MongoIndex<MetricDocument>>();
			indexes.Add(MongoIndex.Create<MetricDocument>(keys => keys.Ascending(x => x.TelemetryStoreId).Descending(x => x.Time).Ascending(x => x.MetricId).Ascending(x => x.Group)));
			_metrics = mongoService.GetCollection<MetricDocument>("Metrics", indexes);

			_events = Channel.CreateBounded<(TelemetryStoreId, JsonNode)>(new BoundedChannelOptions(10000));

			_processTask = new BackgroundTask(BackgroundProcessAsync);
			_flushTask = new BackgroundTask(BackgroundFlushAsync);
			_clock = clock;
			_analyticsConfig = analyticsConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			_processTask.Start();
			_flushTask.Start();
			return Task.CompletedTask;
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _flushTask.StopAsync(cancellationToken);
			await _processTask.StopAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _flushTask.DisposeAsync();
			await _processTask.DisposeAsync();
		}

		/// <inheritdoc/>
		public void AddEvent(TelemetryStoreId storeId, JsonNode node)
		{
			if (!_events.Writer.TryWrite((storeId, node)))
			{
				Interlocked.Increment(ref _numDroppedItems);
			}
		}

		async Task BackgroundProcessAsync(CancellationToken cancellationToken)
		{
			await foreach ((TelemetryStoreId storeId, JsonNode node) in _events.Reader.ReadAllAsync(cancellationToken))
			{
				try
				{
					ProcessEvent(storeId, node);
				}
				catch (ArgumentException ex)
				{
					// This is currently a narrow exception capture for a common event where malformed payloads are passed the background processor.
					// We will emit this as a logDebug along with the exception while we fix call sites that are malformed.
					_logger.LogDebug(ex, "Skipping telemetry emission for event store {StoreId}; processing failed due to malformed payload.", storeId);
				}
				catch(Exception ex)
				{
					// All other exceptions should be logged as errors.
					_logger.LogError(ex, "Failed to process telemetry event for store {StoreId}", storeId);
				}
			}
		}

		void ProcessEvent(TelemetryStoreId storeId, JsonNode node)
		{
			TelemetryStoreConfig? telemetryStoreConfig;
			if (_analyticsConfig.CurrentValue.TryGetTelemetryStore(storeId, out telemetryStoreConfig))
			{
				JsonArray array = new JsonArray { node };

				Stopwatch timer = Stopwatch.StartNew();
				TimeSpan slowEventTime = TimeSpan.FromMilliseconds(10);

				foreach (MetricConfig metric in telemetryStoreConfig.Metrics)
				{
					TimeSpan startTime = timer.Elapsed;
					ProcessEventInternal(storeId, metric, node, array);

					TimeSpan elapsedTime = timer.Elapsed - startTime;
					if (elapsedTime > slowEventTime)
					{
						lock (_lockObject)
						{
							QualifiedMetricId qualifiedMetricId = new QualifiedMetricId(storeId, metric.Id);

							TimeSpan currentTime;
							_slowMetrics.TryGetValue(qualifiedMetricId, out currentTime);
							_slowMetrics[qualifiedMetricId] = currentTime + elapsedTime;
						}
					}
				}

				array.Remove(node);
			}
		}

		void ProcessEventInternal(TelemetryStoreId telemetryStoreId, MetricConfig metric, JsonNode node, JsonArray array)
		{
			if (metric.Filter != null)
			{
				PathResult filterResult = metric.Filter.Evaluate(array);
				if (filterResult.Error != null)
				{
					_logger.LogWarning("Error evaluating filter for metric {MetricId}: {Message}", metric.Id, filterResult.Error);
					return;
				}
				if (filterResult.Matches == null || filterResult.Matches.Count == 0)
				{
					return;
				}
			}

			List<double> values = new List<double>();

			if (metric.Property != null)
			{
				PathResult result = metric.Property.Evaluate(node);
				if (result.Error != null)
				{
					_logger.LogWarning("Error evaluating filter for metric {MetricId}: {Message}", metric.Id, result.Error);
					return;
				}

				if (result.Matches != null && result.Matches.Count > 0)
				{
					foreach (Json.Path.Node match in result.Matches)
					{
						JsonValue? value = match.Value as JsonValue;
						if (value != null && value.TryGetValue(out double doubleValue))
						{
							values.Add(doubleValue);
						}
						else if (value != null && value.TryGetValue(out string? stringValue))
						{
							if (Double.TryParse(stringValue!, out double numericValue))
							{
								values.Add(numericValue);
							}
							else
							{
								_logger.LogWarning("Unabled to parse string to double for metric {MetricId}", metric.Id);
							}
						}
						else
						{
							_logger.LogWarning("Value for property is not a string or number for metric {MetricId}", metric.Id);
						}
					}
				}
			}
			else
			{
				if (metric.Function != AggregationFunction.Count)
				{
					_logger.LogWarning("Missing property parameter for metric {MetricId}", metric.Id);
					return;
				}

				values.Add(1);
			}

			if (values.Count > 0)
			{
				List<string> groupKeys = new List<string>();
				foreach (JsonPath groupByPath in metric.GroupByPaths)
				{
					PathResult groupResult = groupByPath.Evaluate(node);

					string groupKey;
					if (groupResult.Error != null || groupResult.Matches == null || groupResult.Matches.Count == 0)
					{
						groupKey = "";
					}
					else
					{
						groupKey = EscapeCsv(groupResult.Matches.Select(x => x.Value?.ToString() ?? String.Empty));
					}

					groupKeys.Add(groupKey);
				}

				string group = String.Join(",", groupKeys);

				DateTime utcNow = _clock.UtcNow;
				DateTime sampleTime = new DateTime(utcNow.Ticks - (utcNow.Ticks % metric.Interval.Ticks), DateTimeKind.Utc);

				SampleKey key = new SampleKey(telemetryStoreId, metric.Id, group.ToString(), sampleTime);

				lock (_lockObject)
				{
					QueueSampleValues(key, values);
				}

				_newDataEvent.Set();
			}
		}

		static string EscapeCsv(IEnumerable<string> items)
		{
			return String.Join(",", items.Select(x => EscapeCsv(x)));
		}

		static readonly char[] s_csvEscapeChars = { ',', '\n', '\"' };

		static string? EscapeCsv(string text)
		{
			if (text.IndexOfAny(s_csvEscapeChars) != -1)
			{
				text = text.Replace("\"", "\"\"", StringComparison.Ordinal);
				text = $"\"{text}\"";
			}
			return text;
		}

		void QueueSampleValues(SampleKey key, List<double> values)
		{
			List<double>? queuedValues;
			if (!_queuedSamples.TryGetValue(key, out queuedValues))
			{
				queuedValues = new List<double>();
				_queuedSamples.Add(key, queuedValues);
			}
			queuedValues.AddRange(values);
		}

		/// <inheritdoc/>
		public async Task FlushAsync(CancellationToken cancellationToken)
		{
			// Handle all the events currently buffered for processing
			int count = _events.Reader.Count;
			while (count-- > 0 && _events.Reader.TryRead(out (TelemetryStoreId StoreId, JsonNode Node) item))
			{
				ProcessEvent(item.StoreId, item.Node);
			}

			// Write all the buffered samples
			await WriteQueuedSamplesAsync(cancellationToken);
		}

		async Task BackgroundFlushAsync(CancellationToken cancellationToken)
		{
			Task newDataTask = _newDataEvent.Task;
			Task flushTask = _flushEvent.Task;

			Stopwatch timer = Stopwatch.StartNew();

			while (!cancellationToken.IsCancellationRequested)
			{
				try
				{
					await newDataTask.WaitAsync(cancellationToken);
					await Task.WhenAny(flushTask, Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken));

					newDataTask = _newDataEvent.Task;
					flushTask = _flushEvent.Task;

					try
					{
						await WriteQueuedSamplesAsync(cancellationToken);
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Error flushing telemetry data: {Message}", ex.Message);
					}

					TimeSpan slowMetricReportTime = TimeSpan.FromMinutes(1.0);
					if (timer.Elapsed > slowMetricReportTime)
					{
						int numDroppedItems = Interlocked.Exchange(ref _numDroppedItems, 0);
						if (numDroppedItems > 0)
						{
							lock (_lockObject)
							{
								_logger.LogWarning("{NumEvents} telemetry events were dropped due to queue being full.", numDroppedItems);
								foreach ((QualifiedMetricId qualifiedMetricId, TimeSpan time) in _slowMetrics)
								{
									_logger.LogWarning("Metric {StoreId}:{MetricId} took {TimeSecs:n1}s ({Pct:n1}%) of analysis time in the last minute", qualifiedMetricId.StoreId, qualifiedMetricId.MetricId, time.TotalSeconds, time.TotalSeconds / slowMetricReportTime.TotalSeconds);
								}
								_slowMetrics.Clear();
							}
						}
						timer.Restart();
					}
				}
				catch (OperationCanceledException)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error in telemetry flush loop.");
				}
			}
		}

		async Task WriteQueuedSamplesAsync(CancellationToken cancellationToken)
		{
			// Copy the current sample buffer
			Dictionary<SampleKey, List<double>> samples;
			lock (_lockObject)
			{
				samples = _queuedSamples;
				_queuedSamples = new Dictionary<SampleKey, List<double>>();
			}

			// Add the samples to the database
			foreach ((SampleKey sampleKey, List<double> sampleValues) in samples)
			{
				MetricConfig? metricConfig;
				if (_analyticsConfig.CurrentValue.TryGetTelemetryStore(sampleKey.Store, out TelemetryStoreConfig? telemetryStoreConfig) && telemetryStoreConfig.TryGetMetric(sampleKey.Metric, out metricConfig))
				{
					await CombineValuesAsync(sampleKey.Store, metricConfig, sampleKey.Group, sampleKey.Time, sampleValues, cancellationToken);
				}
			}
		}

		async Task CombineValuesAsync(TelemetryStoreId telemetryStoreId, MetricConfig metricConfig, string group, DateTime time, List<double> values, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				// Find or add the current metric document
				FilterDefinition<MetricDocument> filter = Builders<MetricDocument>.Filter.Expr(x => x.TelemetryStoreId == telemetryStoreId && x.MetricId == metricConfig.Id && x.Group == group && x.Time == time);
				UpdateDefinition<MetricDocument> update = Builders<MetricDocument>.Update.SetOnInsert(x => x.Count, 0);
				MetricDocument metric = await _metrics.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<MetricDocument, MetricDocument> { IsUpsert = true, ReturnDocument = ReturnDocument.After }, cancellationToken);

				// Combine the samples
				TDigest digest = (metric.Count == 0) ? new TDigest() : TDigest.Deserialize(metric.State);
				foreach (double value in values)
				{
					digest.Add(value);
				}
				metric.State = digest.Serialize();

				// Save the previous count of samples to sequence updates to the document
				int prevCount = metric.Count;
				metric.Count += values.Count;

				// Update the current value
				switch (metricConfig.Function)
				{
					case AggregationFunction.Count:
						metric.Value = metric.Count;
						break;
					case AggregationFunction.Min:
						metric.Value = digest.Min;
						break;
					case AggregationFunction.Max:
						metric.Value = digest.Max;
						break;
					case AggregationFunction.Sum:
						metric.Value += values.Sum();
						break;
					case AggregationFunction.Average:
						metric.Value = digest.Average;
						break;
					case AggregationFunction.Percentile:
						metric.Value = digest.Quantile(metricConfig.Percentile / 100.0);
						break;
					default:
						_logger.LogWarning("Unhandled aggregation function '{Function}'", metricConfig.Function);
						break;
				}

				// Update the document
				ReplaceOneResult result = await _metrics.ReplaceOneAsync(x => x.TelemetryStoreId == telemetryStoreId && x.Id == metric.Id && x.Group == group && x.Count == prevCount, metric, cancellationToken: cancellationToken);
				if (result.MatchedCount > 0)
				{
					break;
				}
			}
		}

		/*
		class MetricProjection
		{
			public string _group = String.Empty;
			public IEnumerable<ObjectId>? _ids;
			public double _total;
		}
		*/

		/// <inheritdoc/>
		public async Task<List<IMetricMeta>> FindMetaAsync(TelemetryStoreId telemetryStoreId, MetricId[] metricIds, DateTime? minTime = null, DateTime? maxTime = null, int maxResults = 50, CancellationToken cancellationToken = default)
		{
			List<IMetricMeta> result = new List<IMetricMeta>();

			if (!_analyticsConfig.CurrentValue.TryGetTelemetryStore(telemetryStoreId, out TelemetryStoreConfig? telemetryStoreConfig))
			{
				return result;
			}

			FilterDefinition<MetricDocument> filter = FilterDefinition<MetricDocument>.Empty;
			filter &= Builders<MetricDocument>.Filter.Eq(x => x.TelemetryStoreId, telemetryStoreId);

			if (minTime != null)
			{
				filter &= Builders<MetricDocument>.Filter.Gte(x => x.Time, minTime.Value);
			}
			if (maxTime != null)
			{
				filter &= Builders<MetricDocument>.Filter.Lte(x => x.Time, maxTime.Value);
			}

			for (int i = 0; i < metricIds.Length; i++)
			{
				MetricId metricId = metricIds[i];
				MetricConfig? metricConfig = null;

				if (!telemetryStoreConfig.TryGetMetric(metricId, out metricConfig))
				{
					continue;
				}

				FilterDefinition<MetricDocument> metricFilter = filter & Builders<MetricDocument>.Filter.Eq(x => x.MetricId, metricIds[i]);

				List<string> groups = await (await _metrics.DistinctAsync(x => x.Group, metricFilter, cancellationToken: cancellationToken)).ToListAsync(cancellationToken);

				result.Add(new MetricMeta() { MetricId = metricId, GroupBy = metricConfig!.GroupBy, Groups = groups, TopN = metricConfig.TopN, BottomN = metricConfig.BottomN });
			}

			return result;
		}

		/// <inheritdoc/>
		public async Task<List<IMetric>> FindAsync(TelemetryStoreId telemetryStoreId, MetricId[] metricIds, DateTime? minTime = null, DateTime? maxTime = null, string[]? groups = null, int maxResults = 50, CancellationToken cancellationToken = default)
		{
			if (!_analyticsConfig.CurrentValue.TryGetTelemetryStore(telemetryStoreId, out TelemetryStoreConfig? telemetryStoreConfig))
			{
				return new List<IMetric>();
			}

			List<MetricConfig> metricConfigs = new List<MetricConfig>();
			for (int i = 0; i < metricIds.Length; i++)
			{
				MetricId metricId = metricIds[i];
				MetricConfig? metricConfig = null;

				if (!telemetryStoreConfig.TryGetMetric(metricId, out metricConfig))
				{
					continue;
				}

				metricConfigs.Add(metricConfig);
			}

			FilterDefinition<MetricDocument> filter = FilterDefinition<MetricDocument>.Empty;
			filter &= Builders<MetricDocument>.Filter.Eq(x => x.TelemetryStoreId, telemetryStoreId);

			if (minTime != null)
			{
				filter &= Builders<MetricDocument>.Filter.Gte(x => x.Time, minTime.Value);
			}
			if (maxTime != null)
			{
				filter &= Builders<MetricDocument>.Filter.Lte(x => x.Time, maxTime.Value);
			}
			if (groups != null && groups.Length > 0)
			{
				filter &= Builders<MetricDocument>.Filter.In(x => x.Group, groups);
			}

			/*
			List<MetricDocument> results = new List<MetricDocument>();
			List<MetricConfig> configs = metricConfigs.Where(x => x.TopN == 0 && x.BottomN == 0).ToList();

			// batch into a single query where possible
			if (configs.Count > 0)
			{
				FilterDefinition<MetricDocument> metricFilter = filter & Builders<MetricDocument>.Filter.In(x => x.MetricId, configs.Select(x => x.Id));
				results = await _metrics.Find(metricFilter).SortByDescending(x => x.Time).Limit(maxResults).ToListAsync(cancellationToken);				
			}

			configs = metricConfigs.Where(x => x.TopN > 0 || x.BottomN > 0).ToList();
			foreach (MetricConfig config in configs)
			{
				FilterDefinition<MetricDocument> metricFilter = filter & Builders<MetricDocument>.Filter.Eq(x => x.MetricId, config.Id);

				List<MetricProjection> values;
				if (config.TopN > 0)
				{
					values = await _metrics.Aggregate().Match(metricFilter).Group(x => x.Group, group => new MetricProjection { _group = group.Key, _ids = group.Select(x => x.Id), _total = group.Sum(y => y.Value) }).SortByDescending(x => x._total).Limit(config.TopN).ToListAsync(cancellationToken);
				}
				else
				{
					values = await _metrics.Aggregate().Match(metricFilter).Group(x => x.Group, group => new MetricProjection { _group = group.Key, _ids = group.Select(x => x.Id), _total = group.Sum(y => y.Value) }).SortBy(x => x._total).Limit(config.BottomN).ToListAsync(cancellationToken);
				}

				if (values.Count > 0)
				{
					FilterDefinition<MetricDocument> tfilter = FilterDefinition<MetricDocument>.Empty;
					tfilter &= Builders<MetricDocument>.Filter.In(x => x.Id, values.SelectMany(x => x._ids!));
					results.AddRange(await _metrics.Find(tfilter).SortByDescending(x => x.Time).Limit(maxResults).ToListAsync(cancellationToken));
				}
			}
			*/

			List<MetricDocument> results = new List<MetricDocument>();
			FilterDefinition<MetricDocument> metricFilter = filter & Builders<MetricDocument>.Filter.In(x => x.MetricId, metricIds);
			results = await _metrics.Find(metricFilter).SortByDescending(x => x.Time).Limit(maxResults).ToListAsync(cancellationToken);
			return results.ConvertAll<MetricDocument, IMetric>(x => x);
		}
	}
}
