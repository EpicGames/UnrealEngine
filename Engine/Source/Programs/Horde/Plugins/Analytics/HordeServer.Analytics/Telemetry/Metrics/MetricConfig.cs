// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Telemetry.Metrics;
using EpicGames.Horde.Utilities;
using HordeServer.Acls;
using HordeServer.Configuration;
using HordeServer.Utilities;
using Json.Path;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Telemetry.Metrics
{
	/// <summary>
	/// Method for aggregating samples into a metric
	/// </summary>
	public enum AggregationFunction
	{
		/// <summary>
		/// Count the number of matching elements
		/// </summary>
		Count,

		/// <summary>
		/// Take the minimum value of all samples
		/// </summary>
		Min,

		/// <summary>
		/// Take the maximum value of all samples
		/// </summary>
		Max,

		/// <summary>
		/// Sum all the reported values
		/// </summary>
		Sum,

		/// <summary>
		/// Average all the samples
		/// </summary>
		Average,

		/// <summary>
		/// Estimates the value at a certain percentile
		/// </summary>
		Percentile,
	}

	/// <summary>
	/// Configures a metric to aggregate on the server
	/// </summary>
	public class MetricConfig
	{
		/// <summary>
		/// Identifier for this metric
		/// </summary>
		public MetricId Id { get; set; }

		/// <summary>
		/// Filter expression to evaluate to determine which events to include. This query is evaluated against an array.
		/// </summary>
		[JsonSchemaString]
		public JsonPath? Filter { get; set; }

		/// <summary>
		/// Property to aggregate
		/// </summary>
		[JsonSchemaString]
		public JsonPath? Property { get; set; }

		/// <summary>
		/// Property to group by. Specified as a comma-separated list of JSON path expressions.
		/// </summary>
		public string GroupBy
		{
			get => _groupBy;
			set
			{
				_groupBy = value;

				GroupByPaths.Clear();
				if (!String.IsNullOrWhiteSpace(_groupBy))
				{
					List<string> fields = _groupBy.Split(',').Select(x => x.Trim()).Where(x => x.Length > 0).ToList();
					if (fields.Count > 0)
					{
						GroupByPaths.AddRange(fields.Select(x => JsonPath.Parse(x)));
					}
				}
			}
		}

		[JsonIgnore]
		string _groupBy = String.Empty;

		/// <summary>
		/// Accessor for the <see cref="GroupBy"/> field, parsed as a list of json paths
		/// </summary>
		[JsonIgnore]
		public List<JsonPath> GroupByPaths { get; } = new List<JsonPath>();

		/// <summary>
		/// How to aggregate samples for this metric
		/// </summary>
		public AggregationFunction Function { get; set; }

		/// <summary>
		/// For the percentile function, specifies the percentile to measure
		/// </summary>
		public int Percentile { get; set; } = 95;

		/// <summary>
		/// TopN aggregation
		/// </summary>
		public int TopN { get; set; } = 0;

		/// <summary>
		/// BottomN aggregation
		/// </summary>
		public int BottomN { get; set; } = 0;

		/// <summary>
		/// Interval for each metric. Supports times such as "2d", "1h", "1h30m", "20s".
		/// </summary>
		[JsonConverter(typeof(IntervalJsonConverter))]
		public TimeSpan Interval { get; set; } = TimeSpan.FromHours(1.0);
	}

	/// <summary>
	/// Config for metrics
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/telemetry")]
	[JsonSchemaCatalog("Horde Telemetry", "Horde telemetry configuration file", new[] { "*.telemetry.json", "*.metrics.json", "Metrics/*.json" })]
	[ConfigDoc("*.telemetry.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Telemetry.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class TelemetryStoreConfig
	{
		/// <summary>
		/// Identifier for this store
		/// </summary>
		public TelemetryStoreId Id { get; set; }

		/// <summary>
		/// Permissions for this store
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Metrics to aggregate on the Horde server
		/// </summary>
		public List<MetricConfig> Metrics { get; set; } = new List<MetricConfig>();

		/// <summary>
		/// Configuration for telemetry views
		/// </summary>
		public List<TelemetryViewConfig> Views { get; set; } = new List<TelemetryViewConfig>();

		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		readonly Dictionary<MetricId, MetricConfig> _metricLookup = new Dictionary<MetricId, MetricConfig>();

		/// <summary>
		/// Macros within this configuration
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

		/// <summary>
		/// Called after the store has been deserialized to compute cached values
		/// </summary>
		public void PostLoad(AclConfig parentAcl)
		{
			Acl.PostLoad(parentAcl, $"telemetry-store:{Id}", AclConfig.GetActions([typeof(TelemetryAclAction)])); 

			_metricLookup.Clear();
			foreach (MetricConfig metric in Metrics)
			{
				_metricLookup.Add(metric.Id, metric);
			}
		}

		/// <summary>
		/// Attempt to get config for a metric with the given id
		/// </summary>
		/// <param name="metricId">Metric id</param>
		/// <param name="metricConfig">Receives the config object on success</param>
		/// <returns>True if the metric was found</returns>
		public bool TryGetMetric(MetricId metricId, [NotNullWhen(true)] out MetricConfig? metricConfig)
			=> _metricLookup.TryGetValue(metricId, out metricConfig);
	}
}
