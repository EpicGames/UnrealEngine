// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Net.Http.Headers;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using HordeServer.Utilities;
using Microsoft.AspNetCore.WebUtilities;
using Microsoft.Extensions.Http;
using Microsoft.Extensions.Logging;
using Polly;
using Polly.Extensions.Http;

namespace HordeServer.Telemetry.Sinks
{
	/// <summary>
	/// Options for the Epic telemetry sink
	/// </summary>
	public interface IEpicTelemetrySinkConfig
	{
		/// <summary>
		/// Base URL for sending events
		/// </summary>
		public Uri? Url { get; }

		/// <summary>
		/// The application id
		/// </summary>
		public string AppId { get; }
	}

	/// <summary>
	/// Epic internal telemetry sink using the data router
	/// </summary>
	public sealed class EpicTelemetrySink : ITelemetrySink
	{
		// Converter for datetime formats that Tableau can ingest
		class TableauDateTimeConverter : JsonConverter<DateTime>
		{
			const string Format = "yyyy-MM-dd HH:mm:ss";

			public override void Write(Utf8JsonWriter writer, DateTime date, JsonSerializerOptions options)
			{
				writer.WriteStringValue(date.ToString(Format));
			}

			public override DateTime Read(ref Utf8JsonReader reader, Type typeToConvert, JsonSerializerOptions options)
			{
				return DateTime.ParseExact(reader.GetString() ?? String.Empty, Format, null);
			}
		}

		/// <summary>
		/// Name of the HTTP client for writing telemetry data
		/// </summary>
		public const string HttpClientName = "EpicTelemetrySink";

		static readonly Utf8String s_packetPrefix = new Utf8String("{\"Events\":[");
		static readonly Utf8String s_packetSuffix = new Utf8String("]}");

		readonly Lock _lockObject = new Lock();

		readonly HttpClient _httpClient;
		readonly Uri? _baseUrl;
		readonly JsonSerializerOptions _jsonOptions;

		class Writer : IDisposable
		{
			readonly Uri _uri;
			readonly JsonSerializerOptions _jsonOptions;
			readonly ArrayMemoryWriter _packetMemoryWriter;
			readonly Utf8JsonWriter _packetWriter;

			public Stopwatch Timer { get; } = Stopwatch.StartNew();
			public Uri Uri => _uri;
			public bool HasData => _packetMemoryWriter.WrittenMemory.Length > 0;

			public Writer(Uri uri, JsonSerializerOptions jsonOptions)
			{
				_uri = uri;
				_jsonOptions = jsonOptions;
				_packetMemoryWriter = new ArrayMemoryWriter(65536);
				_packetWriter = new Utf8JsonWriter(_packetMemoryWriter);
			}

			public void Dispose()
			{
				_packetWriter.Dispose();
			}

			public void AddEvent(object payload)
			{
				// Restart the timer
				Timer.Restart();

				// If this is the first event written, write the outer events array
				if (_packetMemoryWriter.Length == 0)
				{
					_packetMemoryWriter.WriteFixedLengthBytes(s_packetPrefix.Span);
				}
				else
				{
					_packetMemoryWriter.WriteUInt8((byte)',');
				}

				// Serialize the event data
				_packetWriter.Reset();
				JsonSerializer.Serialize(_packetWriter, payload, _jsonOptions);
				_packetWriter.Flush();
			}

			public byte[] Flush()
			{
				_packetMemoryWriter.WriteFixedLengthBytes(s_packetSuffix.Span);
				byte[] packet = _packetMemoryWriter.WrittenMemory.ToArray();

				_packetMemoryWriter.Clear();
				_packetWriter.Reset(_packetMemoryWriter);
				return packet;
			}
		}

		readonly IServerInfo _serverInfo;
		readonly Dictionary<TelemetryRecordMeta, Writer> _writers = new Dictionary<TelemetryRecordMeta, Writer>();
		readonly ILogger _logger;

		/// <inheritdoc/>
		public bool Enabled => _baseUrl != null;

		/// <summary>
		/// Constructor
		/// </summary>
		public EpicTelemetrySink(IEpicTelemetrySinkConfig config, IServerInfo serverInfo, ILogger<EpicTelemetrySink> logger)
		{
			_serverInfo = serverInfo;
			_httpClient = CreateHttpClient();

			_jsonOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonOptions);
			_jsonOptions.PropertyNamingPolicy = null;
			_jsonOptions.Converters.Insert(0, new TableauDateTimeConverter());

			_logger = logger;
			_baseUrl = config.Url;
		}

		static HttpClient CreateHttpClient()
		{
			HttpMessageHandler handler = new SocketsHttpHandler();

#pragma warning disable CA2000
			TimeSpan[] retryTimes = new[] { TimeSpan.FromSeconds(1.0), TimeSpan.FromSeconds(5.0), TimeSpan.FromSeconds(10.0) };
			IAsyncPolicy<HttpResponseMessage> handleTransientErrorPolicy = Policy<HttpResponseMessage>.Handle<HttpRequestException>().OrTransientHttpStatusCode().WaitAndRetryAsync(retryTimes);
			handler = new PolicyHttpMessageHandler(handleTransientErrorPolicy) { InnerHandler = handler };
#pragma warning restore CA2000

			return new HttpClient(handler, true);
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
		{
			foreach (Writer writer in _writers.Values)
			{
				writer.Dispose();
			}
			_httpClient.Dispose();
			return new ValueTask();
		}

		/// <inheritdoc/>
		public async ValueTask FlushAsync(CancellationToken cancellationToken)
		{
			// Make sure the settings are valid
			if (_baseUrl == null)
			{
				return;
			}

			// Generate the content for this event
			List<(Uri, byte[])> packets = new List<(Uri, byte[])>();
			lock (_lockObject)
			{
				// Get all the data to write
				List<TelemetryRecordMeta> removeKeys = new List<TelemetryRecordMeta>();
				foreach ((TelemetryRecordMeta key, Writer writer) in _writers)
				{
					if (writer.HasData)
					{
						packets.Add((writer.Uri, writer.Flush()));
					}
					else if (writer.Timer.Elapsed > TimeSpan.FromSeconds(30.0))
					{
						removeKeys.Add(key);
					}
				}

				// Remove any writers that haven't been written to in 30s
				foreach (TelemetryRecordMeta removeKey in removeKeys)
				{
					if (_writers.Remove(removeKey, out Writer? writer))
					{
						writer.Dispose();
					}
				}
			}

			// Post the event data
			foreach ((Uri uri, byte[] packet) in packets)
			{
				using (HttpRequestMessage request = new HttpRequestMessage())
				{
					request.RequestUri = uri;
					request.Method = HttpMethod.Post;
					request.Headers.UserAgent.Add(new ProductInfoHeaderValue("Horde", _serverInfo.Version.ToString()));
					request.Content = new ByteArrayContent(packet);
					request.Content.Headers.ContentType = new MediaTypeHeaderValue("application/json");

					using (HttpResponseMessage response = await _httpClient.SendAsync(request, cancellationToken))
					{
						if (response.IsSuccessStatusCode)
						{
							_logger.LogDebug("Sending {Size} bytes of telemetry data to {Url}", packet.Length, uri);
						}
						else
						{
							string content = await response.Content.ReadAsStringAsync(cancellationToken);
							_logger.LogError("Unable to send telemetry data to server ({Code}): {Message}", response.StatusCode, content);
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public void SendEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{
			if (_baseUrl == null)
			{
				return;
			}

			// Only forward internal Horde events to the data router; other apps (like the engine) will write to the data
			// router by themselves, and we want need duplicates.
			if (!String.Equals(telemetryEvent.RecordMeta.AppId, TelemetryRecordMeta.HordeAppId, StringComparison.OrdinalIgnoreCase))
			{
				return;
			}

			TelemetryRecordMeta recordMeta = telemetryEvent.RecordMeta;
			if (recordMeta.AppId == null || recordMeta.AppVersion == null || recordMeta.AppEnvironment == null)
			{
				_logger.LogDebug("Unable to send telemetry event to Epic data router; missing required fields in record metadata. {@Event}", telemetryEvent);
				return;
			}

			lock (_lockObject)
			{
				Writer? writer;
				if (!_writers.TryGetValue(recordMeta, out writer))
				{
					Dictionary<string, string?> queryParams = new Dictionary<string, string?>();
					queryParams.Add("AppID", telemetryEvent.RecordMeta.AppId);
					queryParams.Add("AppVersion", telemetryEvent.RecordMeta.AppVersion);
					queryParams.Add("AppEnvironment", telemetryEvent.RecordMeta.AppEnvironment);
					queryParams.Add("SessionID", telemetryEvent.RecordMeta.SessionId);
					queryParams.Add("UploadType", "eteventstream");
					queryParams.Add("StoreID", telemetryStoreId.ToString());

					Uri uri = new Uri(QueryHelpers.AddQueryString(_baseUrl.ToString(), queryParams));
					writer = new Writer(uri, _jsonOptions);

					_writers.Add(recordMeta, writer);
				}
				writer.AddEvent(telemetryEvent.Payload);
			}
		}
	}
}
