// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Nodes;
using EpicGames.Horde.Telemetry;
using HordeServer.Telemetry.Metrics;
using HordeServer.Utilities;

namespace HordeServer.Telemetry.Sinks
{
	/// <summary>
	/// Consumes telemetry events and generates metrics
	/// </summary>
	class MetricTelemetrySink : ITelemetrySink
	{
		readonly IMetricCollection _metricCollection;
		readonly JsonSerializerOptions _jsonOptions;

		public MetricTelemetrySink(IMetricCollection metricCollection)
		{
			_metricCollection = metricCollection;

			_jsonOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonOptions);
		}

		/// <inheritdoc/>
		public bool Enabled => true;

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => default;

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken) => default;

		/// <inheritdoc/>
		public void SendEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{
			JsonNode? node = JsonSerializer.SerializeToNode(telemetryEvent, _jsonOptions);
			if (node != null)
			{
				_metricCollection.AddEvent(telemetryStoreId, node);
			}
		}
	}
}
