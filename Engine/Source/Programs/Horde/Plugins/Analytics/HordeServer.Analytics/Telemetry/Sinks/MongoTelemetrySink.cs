// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Telemetry.Sinks
{
	/// <summary>
	/// Telemetry sink which writes data to a MongoDB collection
	/// </summary>
	public sealed class MongoTelemetrySink : ITelemetrySink, IHostedService, IAsyncDisposable
	{
		class EventDocument
		{
			public ObjectId Id { get; set; }

			[BsonElement("ts")]
			public TelemetryStoreId TelemetryStoreId { get; set; }

			[BsonElement("data")]
			public BsonDocument Data { get; set; }

			[BsonConstructor]
			public EventDocument()
			{
				Data = new BsonDocument();
			}

			public EventDocument(TelemetryStoreId telemetryStoreId, BsonDocument data)
			{
				Id = ObjectId.GenerateNewId();
				TelemetryStoreId = telemetryStoreId;
				Data = data;
			}
		}

		readonly MongoTelemetryConfig? _config;
		readonly IMongoCollection<EventDocument> _collection;
#pragma warning disable CA2213 // False positive? _writer is disposed in DisposeAsync.
		readonly MongoBufferedWriter<EventDocument> _writer;
#pragma warning restore CA2213
		readonly ITicker _cleanupTicker;
		readonly JsonSerializerOptions _jsonOptions;
		readonly ILogger _logger;

		/// <inheritdoc/>
		public bool Enabled => _config != null;

		/// <summary>
		/// Constructor
		/// </summary>
		public MongoTelemetrySink(IMongoService mongoService, IClock clock, IOptions<AnalyticsServerConfig> serverSettings, ILogger<MongoTelemetrySink> logger)
		{
			_config = serverSettings.Value.Sinks.Mongo;
			_collection = mongoService.GetCollection<EventDocument>("Telemetry", builder => builder.Ascending(x => x.TelemetryStoreId).Descending(x => x.Id));
			_writer = new MongoBufferedWriter<EventDocument>(_collection, logger);
			_cleanupTicker = clock.AddSharedTicker<MongoTelemetrySink>(TimeSpan.FromHours(4.0), CleanupAsync, logger);
			_logger = logger;

			_jsonOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonOptions);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			if (Enabled)
			{
				await _writer.StartAsync();
				await _cleanupTicker.StartAsync();
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _cleanupTicker.StopAsync();
			await _writer.StopAsync(cancellationToken);
		}

		async ValueTask CleanupAsync(CancellationToken cancellationToken)
		{
			if (_config == null)
			{
				return;
			}

			DateTime baseTime = DateTime.UtcNow - TimeSpan.FromDays(_config.RetainDays);
#pragma warning disable CS0618 // Type or member is obsolete
			ObjectId minObjectId = new ObjectId(baseTime, 0, 0, 0);
#pragma warning restore CS0618
			DeleteResult result = await _collection.DeleteManyAsync(x => x.Id < minObjectId, cancellationToken);
			_logger.LogInformation("Deleted {NumItems} telemetry records", result.DeletedCount);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await FlushAsync(default);
			await _cleanupTicker.DisposeAsync();
			await _writer.DisposeAsync();
		}

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken)
			=> _writer.FlushAsync(cancellationToken);

		/// <inheritdoc/>
		public void SendEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{
			BsonDocument bson = BsonDocument.Parse(JsonSerializer.Serialize(telemetryEvent, _jsonOptions));
			_writer.Write(new EventDocument(telemetryStoreId, bson));
		}
	}
}
