// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Telemetry;

namespace HordeServer.Telemetry.Sinks
{
	/// <summary>
	/// Telemetry sink that discards all events
	/// </summary>
	public sealed class NullTelemetrySink : ITelemetrySink
	{
		/// <inheritdoc/>
		public bool Enabled => false;

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
			=> default;

		/// <inheritdoc/>
		public ValueTask FlushAsync(CancellationToken cancellationToken)
			=> default;

		/// <inheritdoc/>
		public void SendEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{ }
	}
}
