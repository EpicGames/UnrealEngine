// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Text.Json;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Telemetry.Metrics;
using HordeServer.Telemetry.Metrics;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Telemetry
{
	/// <summary>
	/// Controller for the /api/v1/telemetry endpoint
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public class TelemetryController : HordeControllerBase
	{
		readonly TelemetryManager _telemetryManager;
		readonly IMetricCollection _metricCollection;
		readonly IOptionsSnapshot<AnalyticsConfig> _analyticsConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetryController(TelemetryManager telemetryManager, IMetricCollection metricCollection, IOptionsSnapshot<AnalyticsConfig> analyticsConfig)
		{
			_telemetryManager = telemetryManager;
			_metricCollection = metricCollection;
			_analyticsConfig = analyticsConfig;
		}

		/// <summary>
		/// Queries aggregated metrics from the telemetry system
		/// </summary>
		/// <param name="id">The metrics to query</param>
		/// <param name="minTime">Minimum time interval to query</param>
		/// <param name="maxTime">Maximum time interval to query</param>
		/// <param name="group">Grouping key</param>
		/// <param name="results">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/telemetry/metrics")]
		[Obsolete("Pass a telemetry store id in the route instead")]
		public Task<ActionResult<List<GetTelemetryMetricsResponse>>> GetMetricsAsync([FromQuery] MetricId[] id, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] string? group = null, [FromQuery] int results = 50, CancellationToken cancellationToken = default)
		{
			return GetMetricsAsync(TelemetryStoreId.Default, id, minTime, maxTime, results, group == null ? null : [group], cancellationToken);
		}

		/// <summary>
		/// Queries aggregated metrics from the telemetry system
		/// </summary>
		/// <param name="storeId">The telemetry store id</param>
		/// <param name="id">The metrics to query</param>
		/// <param name="minTime">Minimum time interval to query</param>
		/// <param name="maxTime">Maximum time interval to query</param>
		/// <param name="results">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/telemetry/{storeId}/metrics/meta")]
		public async Task<ActionResult<List<GetTelemetryMetricsMetaResponse>>> GetMetricsMetaAsync(TelemetryStoreId storeId, [FromQuery] MetricId[] id, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int results = 50, CancellationToken cancellationToken = default)
		{
			if (!_analyticsConfig.Value.Authorize(TelemetryAclAction.QueryMetrics, User))
			{
				return Forbid(TelemetryAclAction.QueryMetrics);
			}

			TelemetryStoreConfig? telemetryStoreConfig;
			if (!_analyticsConfig.Value.TryGetTelemetryStore(storeId, out telemetryStoreConfig))
			{
				return NotFound(storeId);
			}

			List<GetTelemetryMetricsMetaResponse> result = new List<GetTelemetryMetricsMetaResponse>();

			List<IMetricMeta> metricMetas = await _metricCollection.FindMetaAsync(telemetryStoreConfig.Id, id, minTime, maxTime, results, cancellationToken);

			foreach (IMetricMeta metricMeta in metricMetas)
			{
				result.Add(new GetTelemetryMetricsMetaResponse() { MetricId = metricMeta.MetricId.ToString(), GroupBy = metricMeta.GroupBy, Groups = metricMeta.Groups, TopN = metricMeta.TopN, BottomN = metricMeta.BottomN });
			}

			return result;
		}

		/// <summary>
		/// Queries aggregated metrics from the telemetry system
		/// </summary>
		/// <param name="storeId">The telemetry store id</param>		
		/// <param name="id">The metrics to query</param>
		/// <param name="minTime">Minimum time interval to query</param>
		/// <param name="maxTime">Maximum time interval to query</param>		
		/// <param name="results">Number of results to return</param>
		/// <param name="groups">Metric groups to find</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/telemetry/{storeId}/metrics")]
		public async Task<ActionResult<List<GetTelemetryMetricsResponse>>> GetMetricsAsync(TelemetryStoreId storeId, [FromQuery] MetricId[] id, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int results = 50, [FromQuery(Name = "group")] string[]? groups = null, CancellationToken cancellationToken = default)
		{
			if (!_analyticsConfig.Value.Authorize(TelemetryAclAction.QueryMetrics, User))
			{
				return Forbid(TelemetryAclAction.QueryMetrics);
			}

			TelemetryStoreConfig? telemetryStoreConfig;
			if (!_analyticsConfig.Value.TryGetTelemetryStore(storeId, out telemetryStoreConfig))
			{
				return NotFound(storeId);
			}

			List<GetTelemetryMetricsResponse> result = new List<GetTelemetryMetricsResponse>();

			List<IMetric> metrics = await _metricCollection.FindAsync(storeId, id, minTime, maxTime, groups, results, cancellationToken);

			HashSet<MetricId> unique = new HashSet<MetricId>(metrics.Select(m => m.MetricId));

			foreach (MetricId metricId in unique)
			{
				MetricConfig? metricConfig = telemetryStoreConfig.Metrics.Find(m => m.Id == metricId);

				if (metricConfig == null)
				{
					continue;
				}

				GetTelemetryMetricsResponse response = new GetTelemetryMetricsResponse();
				response.MetricId = metricId.ToString();
				response.GroupBy = metricConfig.GroupBy;
				result.Add(response);

				foreach (IMetric metric in metrics.Where(m => m.MetricId == metricId))
				{
					GetTelemetryMetricResponse rmetric = new GetTelemetryMetricResponse();
					rmetric.Time = metric.Time;
					rmetric.Value = metric.Value;

					if (groups == null || groups.Length > 1)
					{
						rmetric.Group = metric.Group;
					}

					response.Metrics.Add(rmetric);
				}
			}

			return result;
		}

		/// <summary>
		/// Posts a new telemetry event. This API is modeled after Epic's external data router, allowing events in the engine to be sent to Horde using the same mechanism.
		/// </summary>
		/// <param name="request">The event data</param>
		/// <param name="appId">Identifier of the application sending the event</param>
		/// <param name="appVersion">Version number of the application</param>
		/// <param name="appEnvironment">Name of the environment that the sending application is running in</param>
		/// <param name="uploadType">Type of data being uploaded</param>
		/// <param name="sessionId">Identifier used to group telemetry within a single session.</param>
		[HttpPost]
		[Route("/api/v1/telemetry")]
		public ActionResult PostEvent([FromBody] PostTelemetryEventStreamRequest request, [FromQuery][Required] string appId, [FromQuery][Required] string appVersion, [FromQuery][Required] string appEnvironment, [FromQuery][Required] TelemetryUploadType uploadType, [FromQuery] string? sessionId = null)
		{
			return PostEvent(TelemetryStoreId.Default, request, appId, appVersion, appEnvironment, uploadType, sessionId);
		}

		/// <summary>
		/// Posts a new telemetry event. This API is modeled after Epic's external data router, allowing events in the engine to be sent to Horde using the same mechanism.
		/// </summary>
		/// <param name="storeId">The telemetry store</param>
		/// <param name="request">The event data</param>
		/// <param name="appId">Identifier of the application sending the event</param>
		/// <param name="appVersion">Version number of the application</param>
		/// <param name="appEnvironment">Name of the environment that the sending application is running in</param>
		/// <param name="uploadType">Type of data being uploaded</param>
		/// <param name="sessionId">Identifier used to group telemetry within a single session.</param>
		[HttpPost]
		[Route("/api/v1/telemetry/{storeId}")]
		public ActionResult PostEvent(TelemetryStoreId storeId, [FromBody] PostTelemetryEventStreamRequest request, [FromQuery][Required] string appId, [FromQuery][Required] string appVersion, [FromQuery][Required] string appEnvironment, [FromQuery][Required] TelemetryUploadType uploadType, [FromQuery] string? sessionId = null)
		{
			if (uploadType != TelemetryUploadType.EtEventStream)
			{
				return BadRequest("Invalid upload type");
			}

			TelemetryRecordMeta recordMeta = new TelemetryRecordMeta(AppId: appId, AppVersion: appVersion, AppEnvironment: appEnvironment, SessionId: sessionId);
			foreach (JsonElement eventPayload in request.Events)
			{
				_telemetryManager.WriteEvent(storeId, recordMeta, eventPayload);
			}

			return NoContent();
		}

		/// <summary>
		/// Get telemetry views
		/// </summary>
		/// <returns>List of telemetry views</returns>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/telemetry/views")]
		public ActionResult<List<GetTelemetryViewResponse>> GetDashboardViews()
		{
			if (!_analyticsConfig.Value.Authorize(TelemetryAclAction.QueryMetrics, User))
			{
				return Forbid(TelemetryAclAction.QueryMetrics);
			}

			List<GetTelemetryViewResponse> responses = new List<GetTelemetryViewResponse>();

			foreach (TelemetryViewConfig telemetry in _analyticsConfig.Value.Stores.SelectMany(store => store.Views))
			{
				GetTelemetryViewResponse rview = new GetTelemetryViewResponse();
				rview.Id = telemetry.Id.ToString();
				rview.Name = telemetry.Name;
				rview.TelemetryStoreId = telemetry.TelemetryStoreId.ToString();

				foreach (TelemetryVariableConfig variable in telemetry.Variables)
				{
					rview.Variables.Add(new GetTelemetryVariableResponse { Name = variable.Name, Group = variable.Group, Defaults = variable.Defaults });
				}

				foreach (TelemetryCategoryConfig category in telemetry.Categories)
				{
					GetTelemetryCategoryResponse rcategory = new GetTelemetryCategoryResponse { Name = category.Name };

					foreach (TelemetryChartConfig chart in category.Charts)
					{
						GetTelemetryChartResponse rchart = new GetTelemetryChartResponse { Name = chart.Name, Display = chart.Display.ToString(), Graph = chart.Graph.ToString(), Min = chart.Min, Max = chart.Max, Metrics = new List<GetTelemetryChartMetricResponse>() };

						foreach (TelemetryChartMetricConfig metric in chart.Metrics)
						{
							rchart.Metrics.Add(new GetTelemetryChartMetricResponse { MetricId = metric.Id.ToString(), Threshold = metric.Threshold, Alias = metric.Alias });
						}

						rcategory.Charts.Add(rchart);
					}

					rview.Categories.Add(rcategory);
				}

				responses.Add(rview);
			}

			return responses;
		}
	}
}