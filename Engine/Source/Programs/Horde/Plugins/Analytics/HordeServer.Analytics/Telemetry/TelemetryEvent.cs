// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Telemetry
{
	/// <summary>
	/// Wrapper around native telemetry objects
	/// </summary>
	public class TelemetryEvent
	{
		/// <summary>
		/// Record metadata
		/// </summary>
		public TelemetryRecordMeta RecordMeta { get; set; }

		/// <summary>
		/// Accessor for the native object
		/// </summary>
		public object Payload { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetryEvent(TelemetryRecordMeta recordMeta, object payload)
		{
			RecordMeta = recordMeta;
			Payload = payload;
		}
	}
}
