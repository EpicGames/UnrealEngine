// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using HordeServer.Telemetry.Sinks;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Telemetry
{
	/// <summary>
	/// Telemetry sink dispatching incoming events to all registered sinks
	/// </summary>
	public sealed class TelemetryManager : ITelemetryWriter, IHostedService
	{
		/// <inheritdoc/>
		public bool Enabled => _telemetrySinks.Count > 0;

		private readonly List<ITelemetrySink> _telemetrySinks = new();
		private readonly ITicker _ticker;
		private readonly Tracer _tracer;
		private readonly ILogger<TelemetryManager> _logger;
		private readonly TelemetryRecordMeta _serverEventMetadata;

		/// <summary>
		/// Constructor
		/// </summary>
		public TelemetryManager(IServiceProvider serviceProvider, IServerInfo serverInfo, IClock clock, IOptions<AnalyticsServerConfig> serverSettings, Tracer tracer, ILoggerFactory loggerFactory)
		{
			_tracer = tracer;
			_logger = loggerFactory.CreateLogger<TelemetryManager>();
			_ticker = clock.AddTicker<EpicTelemetrySink>(TimeSpan.FromSeconds(30.0), FlushAsync, _logger);
			_serverEventMetadata = new TelemetryRecordMeta(TelemetryRecordMeta.HordeAppId, serverInfo.Version.ToString(), serverInfo.Environment, serverInfo.SessionId);

			TelemetrySinkConfig telemetrySinks = serverSettings.Value.Sinks;
			if (telemetrySinks.Epic?.Enabled ?? false)
			{
				_telemetrySinks.Add(new EpicTelemetrySink(telemetrySinks.Epic, serverInfo, loggerFactory.CreateLogger<EpicTelemetrySink>()));
			}
			if (telemetrySinks.Mongo?.Enabled ?? false)
			{
				_telemetrySinks.Add(serviceProvider.GetRequiredService<MongoTelemetrySink>());
			}

			_telemetrySinks.Add(serviceProvider.GetRequiredService<MetricTelemetrySink>());

			// Add any additional sinks registered by plugins
			foreach (ITelemetrySink sink in serviceProvider.GetServices<ITelemetrySink>())
			{
				if (!_telemetrySinks.Contains(sink))
				{
					_telemetrySinks.Add(sink);
				}
			}
		}

		/// <inheritdoc/>
		public void WriteEvent(TelemetryStoreId telemetryStoreId, object payload)
		{
			WriteEvent(telemetryStoreId, _serverEventMetadata, payload);
		}

		/// <inheritdoc/>
		public void WriteEvent(TelemetryStoreId telemetryStoreId, TelemetryRecordMeta metadata, object payload)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(TelemetryManager)}.{nameof(WriteEvent)}");

			TelemetryEvent telemetryEvent = new TelemetryEvent(metadata, payload);
			foreach (ITelemetrySink sink in _telemetrySinks)
			{
				using TelemetrySpan sinkSpan = _tracer.StartActiveSpan($"SendEvent");
				string fullName = sink.GetType().FullName ?? "Unknown";
				span.SetAttribute("sink", fullName);

				if (sink.Enabled)
				{
					try
					{
						sink.SendEvent(telemetryStoreId, telemetryEvent);
					}
					catch (Exception e)
					{
						_logger.LogWarning(e, "Failed sending event to {Sink}. Message: {Message}. Event: {Event}", fullName, e.Message, GetEventText(telemetryEvent));
					}
				}
			}
		}

		static string GetEventText(TelemetryEvent telemetryEvent)
		{
			try
			{
				JsonSerializerOptions options = new JsonSerializerOptions();
				JsonUtils.ConfigureJsonSerializer(options);

				return JsonSerializer.Serialize(telemetryEvent, options);
			}
			catch
			{
				return "(Unable to serialize)";
			}
		}

		/// <inheritdoc />
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc />
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		/// <inheritdoc />
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
			foreach (ITelemetrySink sink in _telemetrySinks)
			{
				await sink.DisposeAsync();
			}
		}

		/// <inheritdoc />
		public async ValueTask FlushAsync(CancellationToken cancellationToken)
		{
			foreach (ITelemetrySink sink in _telemetrySinks)
			{
				await sink.FlushAsync(cancellationToken);
			}
		}
	}
}
