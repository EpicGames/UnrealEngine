// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Acls;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;

namespace HordeServer.Tests.Acls
{
	[TestClass]
	public class AclTests : BuildTestSetup
	{
		[TestMethod]
		public async Task TestCustomProfileAsync()
		{
			await UpdateConfigAsync(globalConfig =>
			{
				globalConfig.Acl.Profiles = new List<AclProfileConfig>
				{
					new AclProfileConfig
					{
						Id = new AclProfileId("global-profile"),
						Actions = new List<AclAction>{ JobAclAction.CreateJob, JobAclAction.RetryJobStep }
					},
				};

				ProjectConfig projectConfig = new ProjectConfig();
				projectConfig.Id = new ProjectId("ue5");
				projectConfig.Acl.Profiles = new List<AclProfileConfig>
				{
					new AclProfileConfig
					{
						Id = new AclProfileId("project-profile"),
						Actions = new List<AclAction>{ JobAclAction.ViewJob },
						Extends = new List<AclProfileId>{ new AclProfileId("global-profile") },
						ExcludeActions = new List<AclAction>{ JobAclAction.RetryJobStep }
					},
				};
				globalConfig.Plugins.GetBuildConfig().Projects.Add(projectConfig);

				StreamConfig streamConfig = new StreamConfig();
				streamConfig.Id = new StreamId("ue5-main");
				streamConfig.Acl.Profiles = new List<AclProfileConfig>
				{
					new AclProfileConfig
					{
						Id = new AclProfileId("stream-profile"),
						Actions = new List<AclAction>{ JobAclAction.DeleteJob }
					},
				};
				streamConfig.Acl.Entries = new List<AclEntryConfig>
				{
					new AclEntryConfig
					{
						Claim = new AclClaimConfig{ Type = "foo", Value = "bar" },
						Profiles = new List<AclProfileId>{ new AclProfileId("project-profile"), new AclProfileId("stream-profile") }
					}
				};
				projectConfig.Streams.Add(streamConfig);
			});

			GlobalConfig globalConfig = GlobalConfig.CurrentValue;
			globalConfig.Plugins.GetBuildConfig().TryGetStream(new StreamId("ue5-main"), out StreamConfig? streamConfig);

			ClaimsIdentity identity = new ClaimsIdentity(null, new[] { new Claim("foo", "bar") });
			ClaimsPrincipal principal = new ClaimsPrincipal(identity);
			Assert.IsTrue(streamConfig!.Acl.Authorize(JobAclAction.CreateJob, principal));
			Assert.IsTrue(streamConfig!.Acl.Authorize(JobAclAction.ViewJob, principal));
			Assert.IsTrue(streamConfig!.Acl.Authorize(JobAclAction.DeleteJob, principal));
			Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.ExecuteJob, principal));
			Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.RetryJobStep, principal));
		}

		[TestMethod]
		public async Task TestDefaultProfileAsync()
		{
			await UpdateConfigAsync(globalConfig =>
			{
				ProjectConfig projectConfig = new ProjectConfig();
				projectConfig.Id = new ProjectId("ue5");
				globalConfig.Plugins.GetBuildConfig().Projects.Add(projectConfig);

				StreamConfig streamConfig = new StreamConfig();
				streamConfig.Id = new StreamId("ue5-main");
				streamConfig.Acl.Entries = new List<AclEntryConfig>
				{
					new AclEntryConfig
					{
						Claim = new AclClaimConfig{ Type = "foo", Value = "bar" },
						Profiles = new List<AclProfileId>{ new AclProfileId("default-read") }
					}
				};
				projectConfig.Streams.Add(streamConfig);
			});

			GlobalConfig globalConfig = GlobalConfig.CurrentValue;
			globalConfig.Plugins.GetBuildConfig().TryGetStream(new StreamId("ue5-main"), out StreamConfig? streamConfig);

			{
				ClaimsIdentity identity = new ClaimsIdentity(null, new[] { new Claim("foo", "bar") });
				ClaimsPrincipal principal = new ClaimsPrincipal(identity);
				Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.CreateJob, principal));
				Assert.IsTrue(streamConfig!.Acl.Authorize(JobAclAction.ViewJob, principal));
				Assert.IsTrue(streamConfig!.Acl.Authorize(LogAclAction.ViewLog, principal));
				Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.DeleteJob, principal));
			}
			{
				ClaimsIdentity identity = new ClaimsIdentity(null, new[] { new Claim("foo", "bar2") });
				ClaimsPrincipal principal = new ClaimsPrincipal(identity);
				Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.CreateJob, principal));
				Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.ViewJob, principal));
				Assert.IsFalse(streamConfig!.Acl.Authorize(LogAclAction.ViewLog, principal));
				Assert.IsFalse(streamConfig!.Acl.Authorize(JobAclAction.DeleteJob, principal));
			}
		}
	}
}
