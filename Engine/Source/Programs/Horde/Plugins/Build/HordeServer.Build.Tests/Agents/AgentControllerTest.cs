// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using HordeServer.Agents;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Tests.Agents;

[TestClass]
public class AgentControllerDbTest : BuildTestSetup
{
	[TestMethod]
	public async Task UpdateAgentAsync()
	{
		Fixture fixture = await CreateFixtureAsync();
		IAgent fixtureAgent = fixture.Agent1;

		ActionResult<object> obj = await AgentsController.GetAgentAsync(fixtureAgent.Id);
		GetAgentResponse getRes = (obj.Value as GetAgentResponse)!;
		Assert.AreEqual(fixture!.Agent1Name.ToUpper(), getRes.Name);
		Assert.IsNull(getRes.Comment);

		UpdateAgentRequest updateReq = new(Comment: "foo bar baz");
		await AgentsController.UpdateAgentAsync(fixtureAgent.Id, updateReq);

		obj = await AgentsController.GetAgentAsync(fixtureAgent.Id);
		getRes = (obj.Value as GetAgentResponse)!;
		Assert.AreEqual("foo bar baz", getRes.Comment);
	}

	[TestMethod]
	public async Task UpdateAgentFullConformAsync()
	{
		Fixture fixture = await CreateFixtureAsync();
		IAgent agent = fixture.Agent1;

		GetAgentResponse response = Deref((await AgentsController.GetAgentAsync(agent.Id)).Value as GetAgentResponse);
		Assert.IsFalse(response.PendingFullConform);

		await AgentsController.UpdateAgentAsync(agent.Id, new UpdateAgentRequest(RequestFullConform: true));

		response = Deref((await AgentsController.GetAgentAsync(agent.Id)).Value as GetAgentResponse);
		Assert.IsTrue(response.PendingFullConform);
	}

	[TestMethod]
	public async Task UpdateAgentConformAsync()
	{
		Fixture fixture = await CreateFixtureAsync();
		IAgent agent = fixture.Agent1;

		GetAgentResponse response = Deref((await AgentsController.GetAgentAsync(agent.Id)).Value as GetAgentResponse);
		Assert.IsFalse(response.PendingConform);

		await AgentsController.UpdateAgentAsync(agent.Id, new UpdateAgentRequest(RequestConform: true));

		response = Deref((await AgentsController.GetAgentAsync(agent.Id)).Value as GetAgentResponse);
		Assert.IsTrue(response.PendingConform);
	}
}
