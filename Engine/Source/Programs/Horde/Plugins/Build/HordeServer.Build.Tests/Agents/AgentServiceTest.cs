// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Jobs;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Auditing;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Agents;

/// <summary>
/// Testing the agent service
/// </summary>
[TestClass]
public class AgentServiceTest : BuildTestSetup
{
	[TestMethod]
	public async Task GetJobsAsync()
	{
		await CreateFixtureAsync();

		ActionResult<List<object>> res = await JobsController.FindJobsAsync();

		List<GetJobResponse> responses = res.Value!.ConvertAll(x => (GetJobResponse)x);
		responses.SortBy(x => x.CommitId);

		Assert.AreEqual(2, responses.Count);
		Assert.AreEqual("hello1", responses[0].Name);
		Assert.AreEqual("hello2", responses[1].Name);

		res = await JobsController.FindJobsAsync(includePreflight: false);
		Assert.AreEqual(1, res.Value!.Count);
		Assert.AreEqual("hello2", (res.Value[0] as GetJobResponse)!.Name);
	}

	[TestMethod]
	public async Task CreateSessionTestAsync()
	{
		Fixture fixture = await CreateFixtureAsync();

		IAgent agent = await AgentService.CreateSessionAsync(fixture.Agent1, new RpcAgentCapabilities(), 
			"test");

		Assert.IsTrue(AgentService.AuthorizeSession(agent, GetUser(agent), out string _));
		await Clock.AdvanceAsync(TimeSpan.FromMinutes(20));
		Assert.IsFalse(AgentService.AuthorizeSession(agent, GetUser(agent), out string reason));
		Assert.IsTrue(reason.Contains("expired", StringComparison.Ordinal));
	}

	private static long ToUnixTime(DateTime dateTime)
	{
		return new DateTimeOffset(dateTime).ToUnixTimeSeconds();
	}

	[TestMethod]
	public async Task LastOnlineChangeDuringSessionCreateAsync()
	{
		// No session created yet, status change timestamp is empty
		IAgent agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agent1"), AgentMode.Dedicated, false, ""));
		Assert.AreEqual(AgentStatus.Stopped, agent.Status);
		Assert.IsTrue(agent.LastOnlineTime.HasValue);

		// A session has been created, status change timestamp is current time
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(), "v1");
		Assert.AreEqual(AgentStatus.Ok, agent.Status);
		Assert.IsNull(agent.LastOnlineTime);

		await Clock.AdvanceAsync(TimeSpan.FromMinutes(1));
	}

	private static int s_agentId = 1;
	private async Task<IAgent> CreateAgentSessionAsync()
	{
		IAgent agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agentServiceTest-" + s_agentId++), AgentMode.Dedicated, false, ""));
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(), "v1");
		return agent;
	}

	private static void AssertEqual(DateTime expected, DateTime? actual)
	{
		// When saved as MongoDB documents, some precision is lost. This compares only Unix seconds.
		Assert.AreEqual(ToUnixTime(expected), ToUnixTime(actual!.Value));
	}

	[TestMethod]
	[DataRow(AgentStatus.Ok)]
	[DataRow(AgentStatus.Unhealthy)]
	[DataRow(AgentStatus.Stopping)]
	[DataRow(AgentStatus.Stopped)]
	[DataRow(AgentStatus.Unspecified)]
	public async Task LastOnlineChangeAsync(AgentStatus status)
	{
		IAgent agent = await CreateAgentSessionAsync();
		await Clock.AdvanceAsync(TimeSpan.FromMinutes(1));

		agent = (await AgentService.UpdateSessionAsync(agent, agent.SessionId!.Value, status, null, new List<RpcLease>()))!;
		if (agent.Status == AgentStatus.Stopped)
		{
			AssertEqual(Clock.UtcNow, agent.LastOnlineTime);
		}
		else
		{
			Assert.IsNull(agent.LastOnlineTime);
		}
	}

	[TestMethod]
	public async Task AuditLogAwsInstanceTypeChangesAsync()
	{
		Fixture fixture = await CreateFixtureAsync();
		IAuditLogChannel<AgentId> agentLogger = AgentCollection.GetLogger(fixture.Agent1.Id);

		async Task<bool> AuditLogContains(string text)
		{
			await agentLogger.FlushAsync();
			return await agentLogger.FindAsync().AnyAsync(x => x.Data.Contains(text, StringComparison.Ordinal));
		}

		List<string> props = new() { $"{KnownPropertyNames.AwsInstanceType}=m5.large" };
		IAgent agent = await AgentService.CreateSessionAsync(fixture.Agent1, new RpcAgentCapabilities(props), "test");
		Assert.IsFalse(await AuditLogContains("AWS EC2 instance type changed"));

		props = new() { $"{KnownPropertyNames.AwsInstanceType}=c6.xlarge" };
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(props), "test");
		Assert.IsTrue(await AuditLogContains("AWS EC2 instance type changed"));
	}

	[TestMethod]
	public async Task GetAgentRateTestAsync()
	{
		IAgent agent1 = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agent1"), AgentMode.Dedicated, false, ""));
		IAgent agent2 = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agent2"), AgentMode.Dedicated, false, ""));
		await AgentService.CreateSessionAsync(agent1, new RpcAgentCapabilities(new List<string>() { "aws-instance-type=c5.24xlarge", "osfamily=windows" }), "test");
		await AgentService.CreateSessionAsync(agent2, new RpcAgentCapabilities(new List<string>() { "aws-instance-type=c4.4xLARge", "osfamily=WinDowS" }), "test");

		List<AgentRateConfig> agentRateConfigs = new()
		{
			new AgentRateConfig() { Condition = "aws-instance-type == 'c5.24xlarge' && osfamily == 'windows'", Rate = 200, },
			new AgentRateConfig() { Condition = "aws-instance-type == 'c4.4xlarge' && osfamily == 'windows'", Rate = 300 }
		};

		await UpdateConfigAsync(x => ((ComputeConfig)x.Plugins[new PluginName("compute")]).Rates = agentRateConfigs);

		double? rate1 = await AgentService.GetRateAsync(agent1.Id);
		Assert.AreEqual(200, rate1!.Value, 0.1);

		double? rate2 = await AgentService.GetRateAsync(agent2.Id);
		Assert.AreEqual(300, rate2!.Value, 0.1);
	}
	
	[TestMethod]
	[DataRow($"unset=null", true)]
	[DataRow($"{KnownPropertyNames.AutoUpdate}=true", true)]
	[DataRow($"{KnownPropertyNames.AutoUpdate}=TrUE", true)]
	[DataRow($"{KnownPropertyNames.AutoUpdate}=false", false)]
	[DataRow($"{KnownPropertyNames.AutoUpdate}=FaLse", false)]
	public async Task Properties_AutoUpdate_Async(string property, bool expected)
	{
		IAgent newAgent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agent1"), AgentMode.Dedicated, false, ""));
		IAgent agent = await AgentService.CreateSessionAsync(newAgent, new RpcAgentCapabilities([property]), "test");
		Assert.AreEqual(expected, agent.IsAutoUpdateEnabled());
	}
	
	[TestMethod]
	public async Task Properties_ServerDefined_AgentCannotOverride_Async()
	{
		IAgent newAgent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("agent1"), AgentMode.Dedicated, false, ""));
		IAgent agent = await AgentService.CreateSessionAsync(newAgent, new RpcAgentCapabilities(["foo=bar", $"{KnownPropertyNames.Trusted}=true", $"{KnownPropertyNames.Trusted}"]), "test");
		
		Assert.AreEqual(0, newAgent.Properties.Count);
		Assert.AreEqual(1, agent.Properties.Count);
		Assert.AreEqual("foo=bar", agent.Properties[0]);
		Assert.IsTrue(AgentExtensions.IsPropertyServerDefined(KnownPropertyNames.Trusted));
		Assert.IsFalse(AgentExtensions.IsPropertyServerDefined(KnownPropertyNames.OsFamily));
	}
	
	[TestMethod]
	public async Task Properties_ServerDefined_PropagatedToAgent_Async()
	{
		CreateAgentOptions options = new(new AgentId("agent1"), AgentMode.Dedicated, false, "", ["server=foo"]);
		IAgent newAgent = await AgentService.CreateAgentAsync(options);
		IAgent agent = await AgentService.CreateSessionAsync(newAgent, new RpcAgentCapabilities(["agent=bar"]), "test");
		
		Assert.AreEqual(1, newAgent.ServerDefinedProperties.Count);
		Assert.AreEqual(0, newAgent.Properties.Count);
		Assert.AreEqual(2, agent.Properties.Count);
		CollectionAssert.Contains(agent.Properties.ToList(), "server=foo");
		CollectionAssert.Contains(agent.Properties.ToList(), "agent=bar");
	}

	[TestMethod]
	public async Task EphemeralTestAsync()
	{
		IAgent agent = await CreateAgentAsync(new PoolId("pool1"), ephemeral: true);
		Assert.IsTrue(agent.Ephemeral);

		agent = (await AgentService.GetAgentAsync(agent.Id))!;
		Assert.IsTrue(agent.Ephemeral);
		Assert.AreEqual(AgentStatus.Ok, agent.Status);

		// Let background task run for purging outdated sessions, which will terminate session for our agent
		await Clock.AdvanceAsync(TimeSpan.FromHours(1));
		await ServiceProvider.GetRequiredService<AgentCollection>().TickSharedAsync(CancellationToken.None);

		// Ephemeral agent is marked as deleted once its session is terminated
		Assert.IsTrue((await AgentService.GetAgentAsync(agent.Id))!.Deleted);

		await Clock.AdvanceAsync(TimeSpan.FromHours(25));
		await ServiceProvider.GetRequiredService<AgentCollection>().TickSharedAsync(CancellationToken.None);

		// Once more time has passed, the ephemeral agent marked as deleted is removed from database
		Assert.IsNull(await AgentService.GetAgentAsync(agent.Id));
	}
	
	[TestMethod]
	// 5 hour old sessions are deleted
	[DataRow(5, null, true)]
	[DataRow(5, AgentStatus.Stopped, true)]
	[DataRow(5, AgentStatus.Ok, false)]
	[DataRow(5, null, false, false)] // Non-ephemeral agent should never be deleted
	
	// 30 min old sessions are not deleted
	[DataRow(0.5, null, false)]
	[DataRow(0.5, AgentStatus.Stopped, false)]
	[DataRow(0.5, AgentStatus.Ok, false)]
	public async Task Ephemeral_IsDeletedInBackgroundTick_Async(double lastOnlineAgoHours, AgentStatus? status, bool isDeleted, bool isEphemeral = true)
	{
		TimeSpan lastOnlineAgo = TimeSpan.FromHours(lastOnlineAgoHours);
		IAgent? agent = await CreateAgentAsync(new PoolId("pool1"), ephemeral: isEphemeral, adjustClockBy: -lastOnlineAgo, status: status);
		agent = await AgentService.GetAgentAsync(agent.Id);
		Assert.IsNotNull(AgentService.GetAgentAsync(agent!.Id));
		await ServiceProvider.GetRequiredService<AgentCollection>().DeleteExpiredEphemeralAgentsAsync(CancellationToken.None);
		Assert.AreEqual(isDeleted, await AgentService.GetAgentAsync(agent!.Id) == null);
	}

	private static ClaimsPrincipal GetUser(IAgent agent)
	{
		return new ClaimsPrincipal(new ClaimsIdentity(
			new List<Claim> { new(HordeClaimTypes.AgentSessionId, agent.SessionId.ToString()!) }, "TestAuthType"));
	}

	[TestMethod]
	public async Task ConformRemovedAfterUpdateWorkspacesAsync()
	{
		Fixture fixture = await CreateFixtureAsync();
		AgentId agentId = new(fixture.Agent1Name);

		IAgent agent = Deref(await AgentService.GetAgentAsync(agentId));
		agent = Deref(await agent.TryUpdateAsync(new UpdateAgentOptions(RequestFullConform: true)));
		Assert.IsTrue(agent.RequestFullConform);

		await AgentService.TryUpdateWorkspacesAsync(agent, agent.Workspaces.ToList(), false, CancellationToken.None);

		agent = Deref(await AgentService.GetAgentAsync(agentId));
		Assert.IsFalse(agent.RequestFullConform);
	}
}