// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Sessions;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Tests.Agents.Leases
{
	/// <summary>
	/// Database-only integration test for LeasesController
	/// </summary>
	[TestClass]
	public class LeasesControllerDbTest : BuildTestSetup
	{
		[TestMethod]
		public async Task FindLeasesAsync()
		{
			DateTimeOffset minTime = Clock.UtcNow - TimeSpan.FromMinutes(5);
			DateTimeOffset maxTime = Clock.UtcNow;

			ILease lease1 = await CreateLeaseAsync(Clock.UtcNow - TimeSpan.FromMinutes(10), TimeSpan.FromMinutes(6));
			ILease lease2 = await CreateLeaseAsync(Clock.UtcNow - TimeSpan.FromMinutes(7), TimeSpan.FromMinutes(3.1));
			/*ILease outOfTimeWindow = */
			await CreateLeaseAsync(Clock.UtcNow - TimeSpan.FromMinutes(7), TimeSpan.FromMinutes(25));

			ActionResult<List<object>> res = await LeasesController.FindLeasesAsync(null, null, null, null, minTime, maxTime);
			Assert.AreEqual(2, res.Value!.Count);
			Assert.AreEqual(lease2.Id, (res.Value[0] as GetAgentLeaseResponse)!.Id);
			Assert.AreEqual(lease1.Id, (res.Value[1] as GetAgentLeaseResponse)!.Id);
		}

		private async Task<ILease> CreateLeaseAsync(DateTime startTime, TimeSpan duration)
		{
			LeaseId id = new LeaseId(BinaryIdUtils.CreateNew());
			SessionId sessionId = SessionIdUtils.GenerateNewId();
			ILease lease = await LeaseCollection.AddAsync(id, null, "myLease", new AgentId("agent-1"), sessionId, null, null, null, startTime, Array.Empty<byte>());
			await LeaseCollection.TrySetOutcomeAsync(id, startTime + duration, LeaseOutcome.Success, LeaseOutcomeReason.None, null, null);
			return lease;
		}
	}
}