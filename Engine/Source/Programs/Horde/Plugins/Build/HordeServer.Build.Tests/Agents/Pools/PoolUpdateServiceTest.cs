// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Projects;
using HordeServer.Streams;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Agents.Pools;

[TestClass]
public class PoolUpdateServiceTest : BuildTestSetup
{
	private PoolUpdateService _pus = default!;
	private IPool _pool = default!;
	private IAgent _enabledAgent = default!;
	private IAgent _disabledAgent = default!;
	private IAgent _disabledAgentBeyondGracePeriod = default!;

	[TestInitialize]
	public async Task SetupAsync()
	{
		await UpdateConfigAsync(x => x.Plugins.GetComputeConfig().Pools.Clear());
		_pus = new PoolUpdateService(AgentService, PoolService, PoolCollection, Clock, ServiceProvider.GetRequiredService<IOptionsMonitor<BuildConfig>>(), ServiceProvider.GetRequiredService<IOptionsMonitor<ComputeConfig>>(), Tracer, new NullLogger<PoolUpdateService>());
		_pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true });
		_enabledAgent = await CreateAgentAsync(_pool, true);
		_disabledAgent = await CreateAgentAsync(_pool, false, status: AgentStatus.Stopped);
		_disabledAgentBeyondGracePeriod = await CreateAgentAsync(_pool, enabled: false, adjustClockBy: -TimeSpan.FromHours(9), status: AgentStatus.Stopped);
	}

	private async Task RefreshAgentsAsync()
	{
		_enabledAgent = (await AgentService.GetAgentAsync(_enabledAgent.Id))!;
		_disabledAgent = (await AgentService.GetAgentAsync(_disabledAgent.Id))!;
		_disabledAgentBeyondGracePeriod = (await AgentService.GetAgentAsync(_disabledAgentBeyondGracePeriod.Id))!;
	}

	public override async ValueTask DisposeAsync()
	{
		GC.SuppressFinalize(this);
		await base.DisposeAsync();

		await _pus.DisposeAsync();
	}

	[TestMethod]
	public async Task ShutdownDisabledAgents_WithDefaultGlobalGracePeriod_DoesNotRequestShutdownAsync()
	{
		// Act
		await _pus.ShutdownDisabledAgentsAsync(CancellationToken.None);
		await RefreshAgentsAsync();

		// Assert
		Assert.IsFalse(_enabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgentBeyondGracePeriod.RequestShutdown);
	}

	[TestMethod]
	public async Task ShutdownDisabledAgents_WithGlobalGracePeriod_RequestsShutdownAsync()
	{
		// Arrange
		await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().AgentShutdownIfDisabledGracePeriod = TimeSpan.FromHours(3.0));

		// Act
		await _pus.ShutdownDisabledAgentsAsync(CancellationToken.None);
		await RefreshAgentsAsync();

		// Assert
		Assert.IsFalse(_enabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgent.RequestShutdown);
		Assert.IsTrue(_disabledAgentBeyondGracePeriod.RequestShutdown);
	}

	[TestMethod]
	public async Task ShutdownDisabledAgents_WithPerPoolGracePeriod_DoesNotRequestShutdownAsync()
	{
		// Arrange
		// Explicitly set the grace period for the pool to be longer than the default of 8 hours
		await UpdateConfigAsync(config => config.Plugins.GetComputeConfig().Pools[0].ShutdownIfDisabledGracePeriod = TimeSpan.FromHours(24.0));

		// Act
		await _pus.ShutdownDisabledAgentsAsync(CancellationToken.None);
		await RefreshAgentsAsync();

		// Assert
		Assert.IsFalse(_enabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgentBeyondGracePeriod.RequestShutdown);
	}

	[TestMethod]
	public async Task ShutdownDisabledAgents_WithAutoScalingOff_DoesNotRequestShutdownAsync()
	{
		// Arrange
		await UpdateConfigAsync(config => config.Plugins.GetComputeConfig().Pools[0].EnableAutoscaling = false);

		// Act
		await _pus.ShutdownDisabledAgentsAsync(CancellationToken.None);
		await RefreshAgentsAsync();

		// Assert
		Assert.IsFalse(_enabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgent.RequestShutdown);
		Assert.IsFalse(_disabledAgentBeyondGracePeriod.RequestShutdown);
	}

	[TestMethod]
	[DataRow(false, 50)]
	[DataRow(false, null)]
	[DataRow(false, 0)]
	[DataRow(false, 50, null, 0, 2)]
	[DataRow(true, 200)]
	[DataRow(true, 200, 50)]
	[DataRow(true, 200, null)]
	[DataRow(true, null, 0, 50, 300)]
	public async Task AutoConformAgentsAsync(bool conformRequested, params int?[] autoConformThresholdsMb)
	{
		// Arrange
		IAgent agent = await CreateAutoConformAgentAsync(100, autoConformThresholdsMb);

		// Act
		await _pus.UpdatePoolsAsync(CancellationToken.None);
		await _pus.AutoConformAgentsAsync(CancellationToken.None);
		agent = (await AgentService.GetAgentAsync(agent.Id))!;

		// Assert
		Assert.AreEqual(conformRequested, agent.RequestFullConform);
	}

	private async Task<IAgent> CreateAutoConformAgentAsync(int freeDiskSpaceMb, params int?[] autoConformThresholdsMb)
	{
		Dictionary<string, AgentConfig> agentTypes = new();
		Dictionary<string, WorkspaceConfig> workspaceTypes = new();
		for (int i = 0; i < autoConformThresholdsMb.Length; i++)
		{
			string workspaceId = "myWorkspace" + i;
			string agentTypeName = "myAgentType" + i;
			agentTypes[agentTypeName] = new AgentConfig { Pool = _pool.Id, Workspace = workspaceId };
			workspaceTypes[workspaceId] = new WorkspaceConfig { Identifier = workspaceId, ConformDiskFreeSpace = autoConformThresholdsMb[i] };
		}

		await UpdateConfigAsync(x =>
		{
			x.Plugins.GetBuildConfig().Projects.Clear();
			x.Plugins.GetBuildConfig().Projects.Add(new ProjectConfig { Streams = [new StreamConfig { WorkspaceTypes = workspaceTypes, AgentTypes = agentTypes }]});
		});

		IAgent? agent = await CreateAgentAsync(_pool, properties: [$"{KnownPropertyNames.WorkingDiskFreeSpace}={freeDiskSpaceMb * 1024 * 1024}"]);
		agent = await agent.TryTerminateSessionAsync(); // Make agent status = stopped
		Assert.IsNotNull(agent);
		return agent;
	}

	[TestMethod]
	public async Task UpdatePools_V1_UpdatesAllPoolsRegardlessAsync()
	{
		await UpdateConfigAsync(config =>
		{
			config.Version = (int)ConfigVersion.Initial;
			config.Plugins.GetBuildConfig().Projects.Clear();
			config.Plugins.GetBuildConfig().Projects.Add(new ProjectConfig
			{
				Streams =
				[
					new StreamConfig
					{
						WorkspaceTypes = new Dictionary<string, WorkspaceConfig>
						{
							["ws3"] = new WorkspaceConfig { Identifier = "ws3" }
						},
						AgentTypes = new Dictionary<string, AgentConfig>
						{
							["at3"] = new AgentConfig { Pool = _pool.Id, Workspace = "ws3" }
						}
					}
				]
			});
		});

		await _pus.UpdatePoolsAsync(CancellationToken.None);

		IPool? pool = await PoolCollection.GetAsync(_pool.Id);
		Assert.IsNotNull(pool);
		Assert.AreEqual(1, pool.Workspaces.Count);
	}

}
