// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Pools;
using Microsoft.AspNetCore.Mvc;
using MongoDB.Bson;
using MongoDB.Driver;

namespace HordeServer.Tests.Agents.Pools
{
	[TestClass]
	public class PoolsControllerTest : BuildTestSetup
	{
		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			await UpdateConfigAsync(x => x.Plugins.GetComputeConfig().Pools.Clear());
		}

		[TestMethod]
		public async Task GetPoolsTestAsync()
		{
#pragma warning disable CS0618 // Type or member is obsolete
			PoolConfig poolConfig = new PoolConfig
			{
				Name = "Pool1",
			};
#pragma warning restore CS0618

			await UpdateConfigAsync(config =>
			{
				ComputeConfig computeConfig = config.Plugins.GetComputeConfig();
				computeConfig.Pools.Clear();
				computeConfig.Pools.Add(poolConfig);
			});

			ActionResult<List<object>> rawResult = await PoolsController.GetPoolsAsync();
			Assert.AreEqual(1, rawResult.Value!.Count);
			GetPoolResponse response = (rawResult.Value![0] as GetPoolResponse)!;
			Assert.AreEqual(poolConfig.Id.ToString(), response.Id);
			Assert.AreEqual(poolConfig.Name, response.Name);
		}

		[TestMethod]
		public async Task CreatePoolsTestAsync()
		{
#pragma warning disable CS0618 // Type or member is obsolete
			PoolConfig poolConfig = new PoolConfig
			{
				Name = "Pool1",
				ScaleOutCooldown = TimeSpan.FromSeconds(111),
				ScaleInCooldown = TimeSpan.FromSeconds(222),
				SizeStrategies = new List<PoolSizeStrategyInfo> { new(PoolSizeStrategy.JobQueue, "dayOfWeek == 'monday'", @"{""ScaleOutFactor"": 22.0, ""ScaleInFactor"": 33.0}", extraAgentCount: 567) },
				SizeStrategy = PoolSizeStrategy.JobQueue,
				JobQueueSettings = new JobQueueSettings(0.35, 0.85),
				FleetManagers = new List<FleetManagerInfo> { new() { Type = FleetManagerType.AwsReuse, Condition = "dayOfWeek == 'monday'" } },
			};
#pragma warning restore CS0618

			await UpdateConfigAsync(config =>
			{
				ComputeConfig computeConfig = config.Plugins.GetComputeConfig();
				computeConfig.Pools.Clear();
				computeConfig.Pools.Add(poolConfig);
			});

			IPool? pool = await PoolService.GetPoolAsync(poolConfig.Id);
			Assert.IsNotNull(pool);
			Assert.AreEqual(poolConfig.Name, pool.Name);
			Assert.AreEqual(poolConfig.ScaleOutCooldown, pool.ScaleOutCooldown!.Value);
			Assert.AreEqual(poolConfig.ScaleInCooldown, pool.ScaleInCooldown!.Value);
			Assert.AreEqual(poolConfig.JobQueueSettings.ScaleOutFactor, pool.JobQueueSettings!.ScaleOutFactor, 0.0001);
			Assert.AreEqual(poolConfig.JobQueueSettings.ScaleInFactor, pool.JobQueueSettings!.ScaleInFactor, 0.0001);
			Assert.AreEqual(1, pool.SizeStrategies?.Count ?? 0);
			Assert.AreEqual(poolConfig.SizeStrategies[0].Type, pool.SizeStrategies![0].Type);
			Assert.AreEqual(poolConfig.SizeStrategies[0].Condition!.Text, pool.SizeStrategies[0].Condition!.Text);
			Assert.AreEqual(poolConfig.SizeStrategies[0].Config, pool.SizeStrategies[0].Config);
			Assert.AreEqual(poolConfig.SizeStrategies[0].ExtraAgentCount, pool.SizeStrategies[0].ExtraAgentCount);
			Assert.AreEqual(1, pool.FleetManagers?.Count ?? 0);
			Assert.AreEqual(poolConfig.FleetManagers[0].Type, pool.FleetManagers![0].Type);
			Assert.AreEqual(poolConfig.FleetManagers[0].Condition!.Text, pool.FleetManagers[0].Condition!.Text);
			Assert.AreEqual(poolConfig.FleetManagers[0].Config, pool.FleetManagers[0].Config);
		}

		[TestMethod]
		public async Task GetAsync_DbOnlyPoolNotVisibleAtV2Async()
		{
			PoolId dbOnlyId = new PoolId("db-only-test");
			IMongoCollection<BsonDocument> poolsV1 = MongoService.GetCollection<BsonDocument>("Pools");
			await poolsV1.InsertOneAsync(new BsonDocument
			{
				{ "_id", dbOnlyId.ToString() },
				{ "Name", "DbOnly" },
				{ "EnableAutoscaling", true },
				{ "UpdateIndex", 0 },
				{ "Workspaces", new BsonArray() },
				{ "Properties", new BsonDocument() },
			});

			PoolId configPoolId = new PoolId("config-pool");
			await UpdateConfigAsync(config =>
			{
				config.Version = (int)ConfigVersion.PoolsInConfigFiles;
				ComputeConfig computeConfig = config.Plugins.GetComputeConfig();
				computeConfig.Pools.Add(new PoolConfig { Id = configPoolId, Name = "ConfigPool" });
			});

			IPool? configPool = await PoolCollection.GetAsync(configPoolId);
			Assert.IsNotNull(configPool);

			IPool? dbPool = await PoolCollection.GetAsync(dbOnlyId);
			Assert.IsNull(dbPool, "DB-only pools should not be visible at V2+");
		}
	}
}