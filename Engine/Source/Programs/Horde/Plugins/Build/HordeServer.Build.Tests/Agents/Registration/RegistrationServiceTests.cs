// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Cryptography;
using EpicGames.Horde.Agents;
using HordeServer.Agents.Enrollment;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Agents.Registration
{
	[TestClass]
	public class RegistrationServiceTests : BuildTestSetup
	{
		[TestMethod]
		public async Task ApproveAgentAsync()
		{
			EnrollmentService registrationService = ServiceProvider.GetRequiredService<EnrollmentService>();

			string key = RandomNumberGenerator.GetHexString(64);

			const string HostName = "127.0.0.1";
			const string Description = "This is a test";
			await registrationService.AddAsync(key, HostName, Description, CancellationToken.None);

			IReadOnlyList<EnrollmentRequest> requests = await registrationService.FindAsync();
			Assert.AreEqual(1, requests.Count);

			Assert.AreEqual(null, await registrationService.GetApprovalAsync(key, CancellationToken.None));
			await registrationService.ApproveAsync(key, new AgentId("my-agent"));
			Assert.AreEqual(new AgentId("my-agent"), await registrationService.GetApprovalAsync(key, CancellationToken.None));

			using CancellationTokenSource cts = new CancellationTokenSource();
			cts.CancelAfter(TimeSpan.FromSeconds(10.0));

			await registrationService.WaitForApprovalAsync(key, HostName, Description, cts.Token);
		}
	}
}
