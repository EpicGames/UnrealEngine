// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Agents.Relay;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using static HordeServer.Tests.Agents.Relay.NftablesTests;

namespace HordeServer.Tests.Agents.Relay;

public class TestRelayRpcClient : RelayRpc.RelayRpcClient
{
	private static readonly ServerCallContext s_adminContext = new ServerCallContextStub(HordeClaims.AdminClaim.ToClaim());
	public TestAsyncStreamReader<GetPortMappingsResponse> GetPortMappingsResponses { get; } = new(s_adminContext);

	public override AsyncServerStreamingCall<GetPortMappingsResponse> GetPortMappings(
		GetPortMappingsRequest request, Metadata headers = null!,
		DateTime? deadline = null, CancellationToken cancellationToken = default)
	{
		return new AsyncServerStreamingCall<GetPortMappingsResponse>(
			GetPortMappingsResponses, null!, null!, null!, () => { });
	}
}

[TestClass]
public class AgentRelayClientTest
{
	private readonly TestRelayRpcClient _grpcClient = new();
	private readonly AgentRelayClient _relayClient;

	public AgentRelayClientTest()
	{
		using ILoggerFactory loggerFactory = LoggerFactory.Create(builder => builder.AddConsole());
		ILogger<AgentRelayClient> logger = loggerFactory.CreateLogger<AgentRelayClient>();
		_relayClient = new AgentRelayClient("myCluster", "myAgent", new List<string> { "192.168.1.99" }, Nftables.CreateNull(), _grpcClient, new DefaultClock(), logger);
		_relayClient.CooldownOnException = TimeSpan.Zero;
	}

	[TestMethod]
	public async Task GetPortMappingsLongPoll_Results_Async()
	{
		using CancellationTokenSource cts = new(3000);
		_grpcClient.GetPortMappingsResponses.AddMessage(new GetPortMappingsResponse { PortMappings = { LeaseMap1 }, RevisionCount = 1 });
		_grpcClient.GetPortMappingsResponses.AddMessage(new GetPortMappingsResponse { PortMappings = { LeaseMap2, LeaseMap1 }, RevisionCount = 2 });

		List<PortMapping>? pm1 = await _relayClient.GetPortMappingsLongPollAsync(cts.Token);
		Assert.AreEqual(1, pm1!.Count);
		Assert.AreEqual("lease1", pm1[0].LeaseId);
		Assert.AreEqual(1, _relayClient.RevisionNumber);

		List<PortMapping>? pm2 = await _relayClient.GetPortMappingsLongPollAsync(cts.Token);
		Assert.AreEqual(2, pm2!.Count);
		Assert.AreEqual("lease2", pm2[0].LeaseId);
		Assert.AreEqual("lease1", pm2[1].LeaseId);
		Assert.AreEqual(2, _relayClient.RevisionNumber);
	}

	[TestMethod]
	public async Task GetPortMappingsLongPoll_TimesOut_Async()
	{
		using CancellationTokenSource cts = new(3000);
		await Assert.ThrowsExactlyAsync<OperationCanceledException>(() => _relayClient.GetPortMappingsLongPollAsync(cts.Token));
	}

	[TestMethod]
	public async Task ListenForPortMappings_TimesOut_Async()
	{
		using CancellationTokenSource cts = new(3000);

		Task t = _relayClient.ListenForPortMappingsAsync(cts.Token);
		await Task.Delay(100, cts.Token);
		_grpcClient.GetPortMappingsResponses.AddMessage(new GetPortMappingsResponse { PortMappings = { LeaseMap1 } });
		await Task.Delay(100, cts.Token);
		_grpcClient.GetPortMappingsResponses.AddMessage(new GetPortMappingsResponse { PortMappings = { LeaseMap2 } });

		await t;
	}
	
	[TestMethod]
	public void ComparePortMappings()
	{
		IReadOnlySet<PortMapping> added;
		IReadOnlySet<PortMapping> removed;
		
		(added, removed) = AgentRelayClient.ComparePortMappings(null, [LeaseMap1]);
		Assert.AreEqual(1, added.Count);
		Assert.AreEqual(0, removed.Count);
		
		(added, removed) = AgentRelayClient.ComparePortMappings([], [LeaseMap1]);
		Assert.AreEqual(1, added.Count);
		Assert.AreEqual(0, removed.Count);
		
		(added, removed) = AgentRelayClient.ComparePortMappings([LeaseMap1, LeaseMap2], [LeaseMap1, LeaseMap2]);
		Assert.AreEqual(0, added.Count);
		Assert.AreEqual(0, removed.Count);
		
		(added, removed) = AgentRelayClient.ComparePortMappings([LeaseMap1, LeaseMap2], [LeaseMap2, LeaseMap3]);
		Assert.AreEqual(1, added.Count);
		Assert.AreEqual(1, removed.Count);
		Assert.IsTrue(added.Contains(LeaseMap3));
		Assert.IsTrue(removed.Contains(LeaseMap1));
		
		(added, removed) = AgentRelayClient.ComparePortMappings([LeaseMap1, LeaseMap1], [LeaseMap2, LeaseMap2]);
		Assert.AreEqual(1, added.Count);
		Assert.AreEqual(1, removed.Count);
		Assert.IsTrue(added.Contains(LeaseMap2));
		Assert.IsTrue(removed.Contains(LeaseMap1));
	}
}
