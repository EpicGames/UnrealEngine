// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Compute;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Agents.Relay;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Tests.Agents.Relay;

[TestClass]
public class AgentRelayServiceTests : BuildTestSetup
{
	private readonly AgentRelayService _service;
	private readonly GetPortMappingsRequest _request = new() { ClusterId = "cluster1", AgentId = "agent1", IpAddresses = { "192.168.1.1" } };
	private readonly ClusterId _cluster1 = new("cluster1");

	private readonly PortMapping _pm1 = new()
	{
		LeaseId = "eee1",
		AgentIp = "192.168.1.1",
		Ports = { new Port { RelayPort = -1, AgentPort = 1111, Protocol = PortProtocol.Tcp } }
	};

	private readonly PortMapping _pm2 = new()
	{
		LeaseId = "eee2",
		AgentIp = "192.168.1.2",
		Ports =
		{
			new Port { RelayPort = -1, AgentPort = 2222, Protocol = PortProtocol.Tcp },
			new Port { RelayPort = -1, AgentPort = 3333, Protocol = PortProtocol.Udp },
		}
	};

	private readonly PortMapping _pm3 = new()
	{
		LeaseId = "eee3",
		AgentIp = "192.168.1.3",
		Ports =
		{
			new Port { RelayPort = -1, AgentPort = 4444, Protocol = PortProtocol.Tcp },
			new Port { RelayPort = -1, AgentPort = 5555, Protocol = PortProtocol.Udp },
			new Port { RelayPort = -1, AgentPort = 6666, Protocol = PortProtocol.Udp },
		}
	};

	public AgentRelayServiceTests()
	{
		_service = new AgentRelayService(GetRedisServiceSingleton(), Clock, Tracer, NullLogger<AgentRelayService>.Instance);
		_service.SetRandomSeed(1);
		_service.SetRedisChannel($"relay/update/{Guid.NewGuid()}");
	}

	public override async ValueTask DisposeAsync()
	{
		await _service.DisposeAsync();
		await base.DisposeAsync();
		GC.SuppressFinalize(this);
	}

	[TestInitialize]
	public async Task SetupAsync()
	{
		await _service.StartAsync(CancellationToken.None);
	}

	/// <summary>
	/// Helper for testing gRPC calls with streaming responses
	/// </summary>
	/// <param name="call"></param>
	/// <param name="timeout">Max time to wait for test operation to complete</param>
	/// <typeparam name="T">Type streamed back</typeparam>
	/// <returns></returns>
	private static async Task<List<T>> TestStreamingGrpcAsync<T>(Func<TestServerStreamWriter<T>, ServerCallContext, Task> call, TimeSpan timeout) where T : class
	{
		using CancellationTokenSource cts = new(timeout);
		ServerCallContextStub context = new(HordeClaims.AdminClaim.ToClaim());
		context.SetCancellationToken(cts.Token);
		TestServerStreamWriter<T> responseStream = new(context);
		await call(responseStream, context);
		responseStream.Complete();
		return await responseStream.ReadAllAsync().ToListAsync(cts.Token);
	}

	private static GetPortMappingsRequest Request(string cluster = "cluster1", string agent = "agent1", int revision = -1, string ipAddress = "192.168.1.1")
	{
		return new GetPortMappingsRequest { ClusterId = cluster, AgentId = agent, RevisionCount = revision, IpAddresses = { ipAddress } };
	}

	private async Task<GetPortMappingsResponse> GetPortMappingsAsync(GetPortMappingsRequest request, int timeoutMs = 5000)
	{
		List<GetPortMappingsResponse> responses = await TestStreamingGrpcAsync<GetPortMappingsResponse>(async (sw, ctx) =>
		{
			await _service.GetPortMappings(request, sw, ctx);
		}, TimeSpan.FromMilliseconds(timeoutMs));

		Assert.AreEqual(1, responses.Count);
		return responses[0];
	}

	[TestMethod]
	public async Task LongPoll_Simple_Async()
	{
		Task<GetPortMappingsResponse> task = GetPortMappingsAsync(_request);
		await AddPortMappingAsync("cluster1", "eee1", null, "192.168.100.10", _pm1.Ports);
		GetPortMappingsResponse res = await task;
		Assert.AreEqual(1, res.RevisionCount);
		Assert.AreEqual(1, res.PortMappings.Count);
		Assert.AreEqual(LeaseId.Parse("eee1").ToString(), res.PortMappings[0].LeaseId);
	}

	[TestMethod]
	public async Task LongPoll_TooOldRevision_Async()
	{
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		await AddPortMappingAsync("cluster1", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		GetPortMappingsResponse res = await GetPortMappingsAsync(Request(revision: 1));
		Assert.AreEqual(3, res.RevisionCount);
		Assert.AreEqual(3, res.PortMappings.Count);
	}

	[TestMethod]
	public async Task LongPoll_TooNewRevision_Async()
	{
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		await AddPortMappingAsync("cluster1", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		GetPortMappingsResponse res = await GetPortMappingsAsync(Request(revision: 999999));
		Assert.AreEqual(3, res.RevisionCount);
		Assert.AreEqual(3, res.PortMappings.Count);
	}

	[TestMethod]
	public async Task LongPoll_TwoClients_Async()
	{
		Task<GetPortMappingsResponse> task1 = GetPortMappingsAsync(Request());
		Task<GetPortMappingsResponse> task2 = GetPortMappingsAsync(Request());
		await AddPortMappingAsync("cluster1", "eee1", null, "192.168.100.10", _pm1.Ports);

		GetPortMappingsResponse response1 = await task1;
		GetPortMappingsResponse response2 = await task2;
		Assert.AreEqual(1, response1.PortMappings.Count);
		Assert.AreEqual(1, response2.PortMappings.Count);
		Assert.AreEqual(1, response1.RevisionCount);
		Assert.AreEqual(1, response2.RevisionCount);
	}

	[TestMethod]
	public async Task AgentHeartbeat_IsAvailable_Async()
	{
		_service.SetTimeouts(100, 5000);
		await GetPortMappingsAsync(Request(revision: -2));

		List<RelayAgentInfo> agents = await _service.GetAvailableRelayAgentsAsync(_cluster1);
		Assert.AreEqual(1, agents.Count);
		Assert.AreEqual("192.168.1.1", agents[0].IpAddresses[0]);

		Assert.AreEqual(0, (await _service.GetAvailableRelayAgentsAsync(new ClusterId("otherCluster"))).Count);
	}

	[TestMethod]
	public async Task AgentHeartbeat_StaleAgentsAreNotReturned_Async()
	{
		_service.SetTimeouts(100, 5000);
		await GetPortMappingsAsync(Request(revision: -2));
		Assert.AreEqual(1, (await _service.GetAvailableRelayAgentsAsync(new ClusterId(_request.ClusterId))).Count);
		await Clock.AdvanceAsync(TimeSpan.FromMinutes(5));
		Assert.AreEqual(0, (await _service.GetAvailableRelayAgentsAsync(new ClusterId(_request.ClusterId))).Count);
	}

	[TestMethod]
	public async Task GetAndSetPortMappingsAsync()
	{
		(int changeId, List<PortMapping> portMappings) = await _service.GetPortMappingsAsync(_cluster1);
		Assert.AreEqual(0, changeId);
		Assert.AreEqual(0, portMappings.Count);

		PortMapping newPm1 = await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		(changeId, portMappings) = await _service.GetPortMappingsAsync(_cluster1);
		Assert.AreEqual(1, changeId);
		Assert.AreEqual(1, portMappings.Count);
		Assert.AreEqual(newPm1.LeaseId, portMappings[0].LeaseId);
		Assert.AreEqual(newPm1.AgentIp, portMappings[0].AgentIp);
		Assert.AreEqual(LeaseId.Parse("eee1").ToString(), portMappings[0].LeaseId);

		await AddPortMappingAsync("cluster1", _pm2.LeaseId, "100.200.1.1", _pm2.AgentIp, _pm2.Ports);
		(changeId, portMappings) = await _service.GetPortMappingsAsync(_cluster1);
		Assert.AreEqual(2, changeId);
		Assert.AreEqual(2, portMappings.Count);
		portMappings.Sort((a, b) => String.CompareOrdinal(a.LeaseId, b.LeaseId));
		Assert.AreEqual(LeaseId.Parse("eee1").ToString(), portMappings[0].LeaseId);
		Assert.AreEqual(0, portMappings[0].AllowedSourceIps.Count);
		Assert.AreEqual(LeaseId.Parse("eee2").ToString(), portMappings[1].LeaseId);
		CollectionAssert.AreEquivalent(new[] { "100.200.1.1" }, portMappings[1].AllowedSourceIps);
	}

	[TestMethod]
	public async Task PortMapping_Remove_Async()
	{
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		(int changeId, List<PortMapping> portMappings) = await _service.GetPortMappingsAsync(_cluster1);
		Assert.AreEqual(2, changeId);
		Assert.AreEqual(2, portMappings.Count);
		Assert.IsTrue(await _service.RemovePortMappingAsync(_cluster1, LeaseId.Parse(_pm3.LeaseId)));
		(changeId, portMappings) = await _service.GetPortMappingsAsync(_cluster1);
		Assert.AreEqual(3, changeId);
		Assert.AreEqual(1, portMappings.Count);
	}

	[TestMethod]
	public async Task GetAllClustersAsync()
	{
		Assert.AreEqual(0, (await _service.GetClustersAsync()).Count);
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		await AddPortMappingAsync("cluster2", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		Assert.IsTrue(new HashSet<ClusterId> { new("cluster1"), new("cluster2") }.SetEquals(await _service.GetClustersAsync()));
	}

	[TestMethod]
	public async Task PortMapping_RemoveFromAllClusters_Async()
	{
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		await AddPortMappingAsync("cluster2", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		(int revision, List<PortMapping> portMappings) cluster1 = await _service.GetPortMappingsAsync(new ClusterId("cluster1"));
		(int revision, List<PortMapping> portMappings) cluster2 = await _service.GetPortMappingsAsync(new ClusterId("cluster2"));
		Assert.AreEqual(3, cluster1.portMappings.Count + cluster2.portMappings.Count);

		await _service.RemovePortMappingAsync(LeaseId.Parse(_pm1.LeaseId));
		await _service.RemovePortMappingAsync(LeaseId.Parse(_pm2.LeaseId));
		await _service.RemovePortMappingAsync(LeaseId.Parse(_pm3.LeaseId));
		cluster1 = await _service.GetPortMappingsAsync(new ClusterId("cluster1"));
		cluster2 = await _service.GetPortMappingsAsync(new ClusterId("cluster2"));
		Assert.AreEqual(0, cluster1.portMappings.Count + cluster2.portMappings.Count);
	}

	[TestMethod]
	public async Task PortAssignment_Simple_Async()
	{
		PortMapping newPm = await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		Assert.AreEqual(2, newPm.Ports.Count);
		Assert.AreEqual(12214, newPm.Ports[0].RelayPort);
		Assert.AreEqual(14973, newPm.Ports[1].RelayPort);
	}
	
	// TODO: add a test checking for exception "Unable to find an available port range"

	[TestMethod]
	public async Task PortAssignment_TwoMappings_Async()
	{
		_service.SetMinMaxPorts(1000, 1020);
		PortMapping newPm1 = await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		PortMapping newPm2 = await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
		Assert.AreEqual(1005, newPm1.Ports[0].RelayPort);
		Assert.AreEqual(1002, newPm2.Ports[0].RelayPort);
		Assert.AreEqual(1009, newPm2.Ports[1].RelayPort);
	}

	[TestMethod]
	public async Task PortAssignment_OutOfPorts_Async()
	{
		_service.SetMinMaxPorts(1000, 1003);
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		await Assert.ThrowsExactlyAsync<ArgumentException>(() => AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports));
	}

	[TestMethod]
	public async Task PortAssignment_AreReleased_Async()
	{
		_service.SetMinMaxPorts(1000, 1003);
		await AddPortMappingAsync("cluster1", _pm1.LeaseId, null, _pm1.AgentIp, _pm1.Ports);
		await AddPortMappingAsync("cluster1", _pm3.LeaseId, null, _pm3.AgentIp, _pm3.Ports);
		await Assert.ThrowsExactlyAsync<ArgumentException>(() => AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports));
		Assert.IsTrue(await _service.RemovePortMappingAsync(_cluster1, LeaseId.Parse(_pm3.LeaseId)));
		await AddPortMappingAsync("cluster1", _pm2.LeaseId, null, _pm2.AgentIp, _pm2.Ports);
	}

	[TestMethod]
	public void FindAvailablePortRange()
	{
		List<int> Actual(int numPorts, params int[] usedPorts)
		{
			return AgentRelayService.FindAvailablePortRange(usedPorts.ToHashSet(), numPorts, 0, 5).ToList();
		}

		List<int> Expected(params int[] ports)
		{
			return ports.ToHashSet().ToList();
		}

		CollectionAssert.AreEquivalent(Expected(), Actual(0));
		CollectionAssert.AreEquivalent(Expected(0, 1, 2), Actual(3));
		CollectionAssert.AreEquivalent(Expected(0, 1, 2, 3, 4), Actual(5));
		CollectionAssert.AreEquivalent(Expected(0, 1, 2, 3, 4, 5), Actual(6));
		CollectionAssert.AreEquivalent(Expected(), Actual(7));

		CollectionAssert.AreEquivalent(Expected(2, 3), Actual(2, 1));
		CollectionAssert.AreEquivalent(Expected(0, 1), Actual(2, 2));
		CollectionAssert.AreEquivalent(Expected(3, 4), Actual(2, 0, 1, 2));
		CollectionAssert.AreEquivalent(Expected(4, 5), Actual(2, 0, 1, 2, 3));
		CollectionAssert.AreEquivalent(Expected(), Actual(2, 0, 1, 2, 3, 4));
		CollectionAssert.AreEquivalent(Expected(), Actual(2, 0, 1, 2, 3, 4, 5));
		CollectionAssert.AreEquivalent(Expected(1, 2), Actual(2, 0, 3, 4, 5));
	}
	
	[TestMethod]
	public void FindAvailablePorts()
	{
		List<int> Actual(int numPorts, params int[] usedPorts)
		{
			Random random = new (1);
			return AgentRelayService.FindAvailablePorts(random, usedPorts.ToHashSet(), numPorts, 0, 9).ToList();
		}
		
		List<int> Expected(params int[] ports)
		{
			return ports.ToHashSet().ToList();
		}
		
		void AssertPorts(List<int> expected, List<int> actual)
		{
			// Console.WriteLine("{0,-25} {1,-25}", "Expected:", "Actual:");
			// Console.WriteLine("{0,-25} {1,-25}", String.Join(' ', expected), String.Join(' ', actual));
			CollectionAssert.AreEquivalent(expected, actual);
		}
		
		AssertPorts(Expected(), Actual(0));
		AssertPorts(Expected(2, 1, 4), Actual(3));
		AssertPorts(Expected(2, 1, 4, 7, 6, 3, 9, 0, 8, 5), Actual(10));
		AssertPorts(Expected(4, 7, 6, 3), Actual(4, [2, 1, 9]));
		AssertPorts(Expected(4, 7, 6, 3, 0, 8, 5), Actual(7, [2, 1, 9]));
		Assert.ThrowsExactly<ArgumentException>(() => Actual(8, [2, 1, 9]));
		
		Exception e1 = Assert.ThrowsExactly<ArgumentException>(() => Actual(11));
		Assert.IsTrue(e1.Message.Contains("exceeds", StringComparison.InvariantCulture));
	}
	
	private Task<PortMapping> AddPortMappingAsync(string clusterId, string leaseId, string? clientIp, string agentIp, IList<Port> ports, int numRetries = 10)
	{
		IPAddress? clientIpObj = clientIp == null ? null : IPAddress.Parse(clientIp);
		return _service.AddPortMappingAsync(new ClusterId(clusterId), LeaseId.Parse(leaseId), clientIpObj, IPAddress.Parse(agentIp), ports, numRetries);
	}
}