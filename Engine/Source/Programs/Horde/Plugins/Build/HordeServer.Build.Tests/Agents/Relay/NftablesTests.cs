// Copyright Epic Games, Inc. All Rights Reserved.

using Horde.Common.Rpc;
using HordeServer.Agents.Relay;

namespace HordeServer.Tests.Agents.Relay;

[TestClass]
public class NftablesTests
{
	public static readonly PortMapping LeaseMap1 = new()
	{
		LeaseId = "lease1",
		AgentIp = "192.168.1.1",
		Ports = { new Port { RelayPort = 1000, AgentPort = 2000, Protocol = PortProtocol.Tcp } }
	};

	public static readonly PortMapping LeaseMap2 = new()
	{
		LeaseId = "lease2",
		AgentIp = "192.168.1.2",
		AllowedSourceIps = { "10.0.0.22" },
		Ports =
		{
			new Port { RelayPort = 1002, AgentPort = 2002, Protocol = PortProtocol.Tcp },
			new Port { RelayPort = 2222, AgentPort = 5555, Protocol = PortProtocol.Udp }
		}
	};

	public static readonly PortMapping LeaseMap3 = new()
	{
		LeaseId = "lease3",
		AgentIp = "192.168.1.3",
		AllowedSourceIps = { "10.0.0.33", "10.3.3.3" },
		Ports = { new Port { RelayPort = 1003, AgentPort = 2003, Protocol = PortProtocol.Tcp } }
	};

	[TestMethod]
	public void GenerateNftRules_NoPorts()
	{
		Assert.AreEqual(0, Nftables.GenerateNftRules(new List<PortMapping>()).Count);
	}

	[TestMethod]
	public void GenerateNftRules_MultiplePorts()
	{
		List<PortMapping> ports = new() { LeaseMap1, LeaseMap2, LeaseMap3 };
		List<string> rules = Nftables.GenerateNftRules(ports);
		Assert.AreEqual(4, rules.Count);
		Assert.AreEqual("tcp dport 1000 dnat to 192.168.1.1:2000 comment \"leaseId=lease1\"", rules[0]);
		Assert.AreEqual("ip saddr { 10.0.0.22 } tcp dport 1002 dnat to 192.168.1.2:2002 comment \"leaseId=lease2\"", rules[1]);
		Assert.AreEqual("ip saddr { 10.0.0.22 } udp dport 2222 dnat to 192.168.1.2:5555 comment \"leaseId=lease2\"", rules[2]);
		Assert.AreEqual("ip saddr { 10.0.0.33, 10.3.3.3 } tcp dport 1003 dnat to 192.168.1.3:2003 comment \"leaseId=lease3\"", rules[3]);
	}

	[TestMethod]
	public void GenerateNftFile()
	{
		List<PortMapping> ports = new() { LeaseMap1, LeaseMap2 };
		string expected = @"table ip horde
delete table ip horde
table ip horde {
  chain prerouting {
    type nat hook prerouting priority -100; policy accept;
    tcp dport 1000 dnat to 192.168.1.1:2000 comment ""leaseId=lease1""
    ip saddr { 10.0.0.22 } tcp dport 1002 dnat to 192.168.1.2:2002 comment ""leaseId=lease2""
    ip saddr { 10.0.0.22 } udp dport 2222 dnat to 192.168.1.2:5555 comment ""leaseId=lease2""
  }
  chain postrouting {
    type nat hook postrouting priority 100; policy accept;
    masquerade
  }
}
".ReplaceLineEndings("\n");

		Assert.AreEqual(expected, Nftables.GenerateNftFile(ports));
	}
}
