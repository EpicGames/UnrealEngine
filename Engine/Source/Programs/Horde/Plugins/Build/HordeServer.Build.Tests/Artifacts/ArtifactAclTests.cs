// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Acls;
using HordeServer.Artifacts;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Artifacts
{
	[TestClass]
	public class ArtifactAclTests : BuildTestSetup
	{
		const string ClaimType = "artifact-test-claim";

		static StreamId NormalStream { get; } = new StreamId("normal-stream");
		static StreamId SecretStream { get; } = new StreamId("secret-stream");

		static ArtifactType TestArtifactType { get; } = new ArtifactType("normal-build");

		const string NormalReadRole = "normal-read";
		const string SecretReadRole = "secret-read";

		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			await UpdateConfigAsync(SetupFixture);
		}

		void SetupFixture(GlobalConfig globalConfig)
		{
			StreamConfig streamConfig = new StreamConfig { Id = SecretStream };
			{
				ArtifactTypeConfig artifactTypeConfig = new ArtifactTypeConfig();
				artifactTypeConfig.Type = TestArtifactType;
				artifactTypeConfig.Acl = new AclConfig();
				artifactTypeConfig.Acl.Inherit = false;
				artifactTypeConfig.Acl.Entries =
					[
						CreateAclEntry(SecretReadRole, ArtifactAclAction.ReadArtifact)
					];
				streamConfig.ArtifactTypes.Add(artifactTypeConfig);
			}

			ProjectConfig projectConfig = new ProjectConfig();
			projectConfig.Id = new ProjectId("ue5");
			projectConfig.Streams.Add(new StreamConfig { Id = NormalStream });
			projectConfig.Streams.Add(streamConfig);
			{
				ArtifactTypeConfig artifactTypeConfig = new ArtifactTypeConfig();
				artifactTypeConfig.Type = TestArtifactType;
				artifactTypeConfig.Acl = new AclConfig();
				artifactTypeConfig.Acl.Entries =
					[
						CreateAclEntry(NormalReadRole, ArtifactAclAction.ReadArtifact),
						CreateAclEntry(SecretReadRole, ArtifactAclAction.ReadArtifact)
					];
				projectConfig.ArtifactTypes.Add(artifactTypeConfig);
			}

			BuildConfig buildConfig = globalConfig.Plugins.GetBuildConfig();
			buildConfig.Projects.Add(projectConfig);
		}

		static AclEntryConfig CreateAclEntry(string role, AclAction action)
		{
			AclEntryConfig entry = new AclEntryConfig();
			entry.Claim = new AclClaimConfig(ClaimType, role);
			entry.Actions = [action];
			return entry;
		}

		static ClaimsPrincipal CreateUser(string role)
		{
			ClaimsIdentity identity = new ClaimsIdentity([new Claim(ClaimType, role)]);
			return new ClaimsPrincipal(identity);
		}

		[TestMethod]
		public void TestAcls()
		{
			BuildConfig buildConfig = ServiceProvider.GetRequiredService<IOptionsMonitor<BuildConfig>>().CurrentValue;

			// Test that both roles can read from the default stream
			{
				ClaimsPrincipal secretUser = CreateUser(SecretReadRole);
				Assert.IsTrue(buildConfig.AuthorizeArtifact(TestArtifactType, NormalStream, ArtifactAclAction.ReadArtifact, secretUser));

				ClaimsPrincipal normalUser = CreateUser(NormalReadRole);
				Assert.IsTrue(buildConfig.AuthorizeArtifact(TestArtifactType, NormalStream, ArtifactAclAction.ReadArtifact, normalUser));
			}

			// Test that only the secret role can read from the secret stream
			{
				ClaimsPrincipal secretUser = CreateUser(SecretReadRole);
				Assert.IsTrue(buildConfig.AuthorizeArtifact(TestArtifactType, SecretStream, ArtifactAclAction.ReadArtifact, secretUser));

				ClaimsPrincipal normalUser = CreateUser(NormalReadRole);
				Assert.IsFalse(buildConfig.AuthorizeArtifact(TestArtifactType, SecretStream, ArtifactAclAction.ReadArtifact, normalUser));
			}
		}
	}
}
