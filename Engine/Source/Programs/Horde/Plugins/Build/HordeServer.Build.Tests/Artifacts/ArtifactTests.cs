// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using HordeCommon;
using HordeServer.Artifacts;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Artifacts
{
	[TestClass]
	public class ArtifactTests : BuildTestSetup
	{
		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			void SetupFixture(GlobalConfig globalConfig)
			{
				ProjectConfig projectConfig = new ProjectConfig();
				projectConfig.Streams.Add(new StreamConfig { Id = new StreamId("foo") });
				projectConfig.Streams.Add(new StreamConfig { Id = new StreamId("bar") });

				BuildConfig buildConfig = globalConfig.Plugins.GetBuildConfig();
				buildConfig.Projects.Add(projectConfig);
			}

			await UpdateConfigAsync(SetupFixture);
		}

		[TestMethod]
		public async Task CreateArtifactAsync()
		{
			StreamId streamId = new StreamId("foo");

			IArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<IArtifactCollection>();
			IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName("default"), ArtifactType.StepOutput, null, streamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(1, artifacts.Count);
				Assert.AreEqual(artifact.Id, artifacts[0].Id);
			}

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test2" }).ToListAsync();
				Assert.AreEqual(1, artifacts.Count);
				Assert.AreEqual(artifact.Id, artifacts[0].Id);
			}

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test3" }).ToListAsync();
				Assert.AreEqual(0, artifacts.Count);
			}
		}

		[TestMethod]
		public async Task ExpireArtifactAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			ArtifactCollection expirationService = ServiceProvider.GetRequiredService<ArtifactCollection>();

			ArtifactType type = new ArtifactType("my-artifact");
			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepDays = 1 }));

			await expirationService.StartAsync(CancellationToken.None);

			StreamId streamId = new StreamId("foo");

			IArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<IArtifactCollection>();
			IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName("default"), type, null, streamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(1, artifacts.Count);
				Assert.AreEqual(artifact.Id, artifacts[0].Id);
			}

			await clock.AdvanceAsync(TimeSpan.FromDays(2.0));

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(0, artifacts.Count);
			}
		}

		[TestMethod]
		public async Task UpdateExpiryTimesAsync()
		{
			DateTime startTime = Clock.UtcNow;

			ArtifactType type = new ArtifactType("my-artifact");
			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepDays = 1 }));

			ArtifactCollection expirationService = ServiceProvider.GetRequiredService<ArtifactCollection>();

			await expirationService.StartAsync(CancellationToken.None);

			StreamId streamId = new StreamId("foo");

			IArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<IArtifactCollection>();
			IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName("default"), type, null, streamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(1, artifacts.Count);
				Assert.AreEqual(artifact.Id, artifacts[0].Id);
			}

			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes = new List<ArtifactTypeConfig> { new ArtifactTypeConfig { Type = type, KeepDays = 4 } });
			await Clock.AdvanceAsync(TimeSpan.FromDays(2.0));

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(1, artifacts.Count);
			}

			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes = new List<ArtifactTypeConfig> { new ArtifactTypeConfig { Type = type, KeepDays = 1 } });
			await Clock.AdvanceAsync(TimeSpan.FromDays(1.0));

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(0, artifacts.Count);
			}
		}

		[TestMethod]
		public async Task ExpireCountAsync()
		{
			DateTime startTime = Clock.UtcNow;

			ArtifactType type = new ArtifactType("my-artifact");
			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepCount = 4 }));

			ArtifactCollection expirationService = ServiceProvider.GetRequiredService<ArtifactCollection>();
			await expirationService.StartAsync(CancellationToken.None);

			StreamId fooStreamId = new StreamId("foo");
			StreamId barStreamId = new StreamId("bar");

			IArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<IArtifactCollection>();
			for (int idx = 0; idx < 10; idx++)
			{
				await artifactCollection.CreateAsync(new ArtifactName($"default-{idx}"), type, null, fooStreamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());
			}
			for (int idx = 0; idx < 10; idx++)
			{
				await artifactCollection.CreateAsync(new ArtifactName($"default-{idx}"), type, null, barStreamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());
			}

			await Clock.AdvanceAsync(TimeSpan.FromDays(2.0));

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(fooStreamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(4, artifacts.Count);

				for (int idx = 0; idx < 4; idx++)
				{
					Assert.AreEqual(new ArtifactName($"default-{9 - idx}"), artifacts[idx].Name);
				}
			}
			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(barStreamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(4, artifacts.Count);

				for (int idx = 0; idx < 4; idx++)
				{
					Assert.AreEqual(new ArtifactName($"default-{9 - idx}"), artifacts[idx].Name);
				}
			}

			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes = new List<ArtifactTypeConfig> { new ArtifactTypeConfig { Type = type, KeepDays = 1 } });
			await Clock.AdvanceAsync(TimeSpan.FromDays(2.0));

			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(fooStreamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(0, artifacts.Count);
			}
			{
				List<IArtifact> artifacts = await artifactCollection.FindAsync(barStreamId, keys: new[] { "test1" }).ToListAsync();
				Assert.AreEqual(0, artifacts.Count);
			}
		}

		[TestMethod]
		public async Task EnableArtifactExpirationFalseBlocksExpiryAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			ArtifactCollection expirationService = ServiceProvider.GetRequiredService<ArtifactCollection>();

			ArtifactType type = new ArtifactType("my-artifact");
			StreamId streamId = new StreamId("foo");

			// Configure artifact type with 1 day keep and set the safety valve
			await UpdateConfigAsync(config =>
			{
				config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepDays = 1 });
				StreamConfig stream = config.Plugins.GetBuildConfig().Projects[0].Streams[0];
				stream.RetentionOptions.EnableArtifactExpiration = false;
			});

			await expirationService.StartAsync(CancellationToken.None);

			IArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<IArtifactCollection>();
			IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName("default"), type, null, streamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1" }, Array.Empty<string>());

			// Advance well past KeepDays — artifact must survive
			await clock.AdvanceAsync(TimeSpan.FromDays(5.0));

			List<IArtifact> artifacts = await artifactCollection.FindAsync(streamId, keys: new[] { "test1" }).ToListAsync();
			Assert.AreEqual(1, artifacts.Count, "Artifact should not be expired when EnableArtifactExpiration == false");
		}

		[TestMethod]
		public async Task ExpireOrphanedAsync()
		{
			DateTime startTime = Clock.UtcNow;

			ArtifactType type = new ArtifactType("my-artifact");
			StreamId fooStreamId = new StreamId("foo");

			ArtifactCollection artifactCollection = ServiceProvider.GetRequiredService<ArtifactCollection>();
			await artifactCollection.StartAsync(CancellationToken.None);

			// Orphan the artifact type
			{
				await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepCount = 4 }));
				IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName($"default"), type, null, fooStreamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());

				await Clock.AdvanceAsync(TimeSpan.FromDays(30.0));
				IArtifact? artifact2 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNotNull(artifact2);

				await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Clear());
				IArtifact? artifact3 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNotNull(artifact3);

				await Clock.AdvanceAsync(TimeSpan.FromDays(30.0));
				IArtifact? artifact4 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNull(artifact4);
			}

			// Orphan the stream
			{
				await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().ArtifactTypes.Add(new ArtifactTypeConfig { Type = type, KeepCount = 4 }));
				IArtifactBuilder artifact = await artifactCollection.CreateAsync(new ArtifactName($"default"), type, null, fooStreamId, CommitIdWithOrder.FromPerforceChange(1), new string[] { "test1", "test2" }, Array.Empty<string>());

				await Clock.AdvanceAsync(TimeSpan.FromDays(30.0));
				IArtifact? artifact2 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNotNull(artifact2);

				await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().Projects.Clear());
				IArtifact? artifact3 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNotNull(artifact3);

				await Clock.AdvanceAsync(TimeSpan.FromDays(30.0));
				IArtifact? artifact4 = await artifactCollection.GetAsync(artifact.Id);
				Assert.IsNull(artifact4);
			}
		}
	}
}
