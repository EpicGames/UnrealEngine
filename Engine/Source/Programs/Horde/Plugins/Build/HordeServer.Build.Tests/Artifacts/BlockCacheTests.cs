// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Security.Cryptography;
using HordeServer.Artifacts;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tests.Artifacts
{
	[TestClass]
	public class BlockCacheTests
	{
		[TestMethod]
		public void BasicTest()
		{
			using BlockCache blockCache = BlockCache.CreateInMemory(1, 4096, 4096);
			blockCache.Add("hello", new byte[] { 1, 2, 3 });

			using IBlockCacheValue? value = blockCache.Get("hello");
			Assert.IsTrue(value != null && value.Data.ToArray().SequenceEqual(new byte[] { 1, 2, 3 }));
		}

		[TestMethod]
		public void LargeBlockTest()
		{
			byte[] data = RandomNumberGenerator.GetBytes(100000);

			using BlockCache blockCache = BlockCache.CreateInMemory(1);
			blockCache.Add("hello", data);

			using IBlockCacheValue? value = blockCache.Get("hello");
			Assert.IsTrue(value != null && value.Data.ToArray().SequenceEqual(data));
		}

		[TestMethod]
		public void MultiBlockTest()
		{
			string[] keys = new string[1000];
			byte[][] values = new byte[keys.Length][];

			using BlockCache blockCache = BlockCache.CreateInMemory(1);

			Random random = new Random(0);
			for (int idx = 0; idx < keys.Length; idx++)
			{
				keys[idx] = $"key{idx}";

				int length = random.Next(4000) + 1;

				byte[] buffer = new byte[length];
				random.NextBytes(buffer);

				values[idx] = buffer;
				blockCache.Add(keys[idx], values[idx]);
			}

			for (int idx = 0; idx < 2000; idx++)
			{
				int keyIdx = random.Next(keys.Length);

				IBlockCacheValue? value = blockCache.Get(keys[keyIdx]);
				Assert.IsNotNull(value);

				Assert.IsTrue(value.Data.ToArray().SequenceEqual(values[keyIdx]));
			}
		}

		[TestMethod]
		public void RandomBlockTest()
		{
			Random sizeRng = new Random(0);

			List<(string Name, byte[] Data)> items = new List<(string, byte[])>();
			for (int idx = 0; idx < 128; idx++)
			{
				int size = sizeRng.Next(2048, 16384);
				items.Add(($"{idx}", RandomNumberGenerator.GetBytes(size)));
			}

			using BlockCache blockCache = BlockCache.CreateInMemory(5, 16, 4096);
			Parallel.For(0, 16, threadIdx =>
			{
				Random rng = new Random(threadIdx);
				for (int idx = 0; idx < 40000; idx++)
				{
					int itemIdx = rng.Next(items.Count);
					(string name, byte[] data) = items[itemIdx];

					if ((threadIdx & 1) == 0)
					{
						// Writer
						blockCache.Add(name, data);
					}
					else
					{
						// Reader
						using IBlockCacheValue? cacheValue = blockCache.Get(name);
						Assert.IsTrue(cacheValue == null || cacheValue.Data.ToArray().SequenceEqual(data));
					}
				}
			});

			foreach ((string name, byte[] data) in items)
			{
				using IBlockCacheValue? cacheValue = blockCache.Get(name);
				Assert.IsTrue(cacheValue == null || cacheValue.Data.ToArray().SequenceEqual(data));
			}
		}

		class TestLogger : ILogger
		{
			public List<string> Messages { get; } = new List<string>();

			public IDisposable? BeginScope<TState>(TState state) where TState : notnull
				=> null;

			public bool IsEnabled(LogLevel logLevel)
				=> true;

			public void Log<TState>(LogLevel logLevel, EventId eventId, TState state, Exception? exception, Func<TState, Exception?, string> formatter)
				=> Messages.Add(formatter(state, exception));
		}

		[TestMethod]
		public void CorruptionTest()
		{
			TestLogger logger = new TestLogger();

			using BlockCache blockCache = BlockCache.CreateInMemory(1, 4096, 4096, logger);
			blockCache.Add("hello", new byte[] { 1, 2, 3 });

			{
				using IBlockCacheValue? value = blockCache.Get("hello");
				Assert.IsTrue(value != null && value.Data.ToArray().SequenceEqual(new byte[] { 1, 2, 3 }));
				Assert.AreEqual(0, logger.Messages.Count);
			}

			blockCache.DangerousCorruptValue("hello");

			{
				using IBlockCacheValue? value = blockCache.Get("hello");
				Assert.IsNull(value);
				Assert.AreEqual(1, logger.Messages.Count);
			}
		}
	}
}
