// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using HordeServer.Artifacts;
using HordeServer.Storage;
using Microsoft.Extensions.Logging.Abstractions;
using Moq;

namespace HordeServer.Tests.Artifacts
{
	[TestClass]
	public class UnsyncCacheTests
	{
		[TestMethod]
		public async Task TestBlobsAsync()
		{
			NamespaceId ns = new NamespaceId("test");
			byte[] source = Enumerable.Range(0, 1024 * 1024).Select(x => (byte)(x % 257)).ToArray();

			BundleStorageNamespace storageNamespace = BundleStorageNamespace.CreateInMemory(NullLogger.Instance);

			Dictionary<IoHash, ReadOnlyMemory<byte>> chunks = new Dictionary<IoHash, ReadOnlyMemory<byte>>();

			IHashedBlobRef<DirectoryNode> directoryRef;
			await using (IBlobWriter writer = storageNamespace.CreateBlobWriter())
			{
				using ChunkedDataWriter chunkedWriter = new ChunkedDataWriter(writer, new ChunkingOptions());
				await chunkedWriter.AppendAsync(source, CancellationToken.None);

				ChunkedData data = await chunkedWriter.FlushAsync();

				DirectoryNode directory = new DirectoryNode();
				directory.AddFile("hello.txt", FileEntryFlags.None, data);

				directoryRef = await writer.WriteBlobAsync(directory);
			}

			RefName refName = new RefName("test");
			await storageNamespace.AddRefAsync(refName, directoryRef);

			Mock<IStorageService> factory = new Mock<IStorageService>();
			factory.Setup(x => x.TryGetNamespace(ns)).Returns(storageNamespace);

			Mock<IArtifact> artifact = new Mock<IArtifact>();
			artifact.SetupGet(x => x.NamespaceId).Returns(ns);
			artifact.SetupGet(x => x.RefName).Returns(refName);

			using UnsyncCache cache = new UnsyncCache(factory.Object, NullLogger<UnsyncCache>.Instance);

			UnsyncManifest? manifest = await cache.GetManifestAsync(artifact.Object);
			Assert.IsNotNull(manifest);

			UnsyncFile file = manifest.Files[0];
			Assert.AreEqual("hello.txt", file.Name.ToString());

			int offset = 0;
			foreach (UnsyncBlock block in file.Blocks)
			{
				IHashedBlobRef? blobRef = await cache.ReadBlobRefAsync(artifact.Object, block.Blob.Hash);
				Assert.IsNotNull(blobRef);

				using BlobData blobData = await blobRef.ReadBlobDataAsync();

				Assert.IsTrue(blobData.Data.Span.SequenceEqual(source.AsSpan(offset, blobData.Data.Length)));
				offset += (int)block.Length;
			}
		}
	}
}
