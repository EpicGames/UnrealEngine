// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.BuildHealth;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Tests.BuildHealth
{
	[TestClass]
	public class BuildHealthFilterCollectionTests : BuildTestSetup
	{
		private readonly StreamId _streamId = new("ue5-main");

		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			await UpdateConfigAsync(globalConfig =>
			{
				ProjectConfig projectConfig = new() { Id = new ProjectId("ue5") };
				projectConfig.Streams.Add(new StreamConfig { Id = _streamId });
				globalConfig.Plugins.GetBuildConfig().Projects.Add(projectConfig);
			});
		}

		[TestMethod]
		public async Task TestAddBuildHealthFilterAsync()
		{
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			ProjectId expectedProject = new ProjectId("ue5");
			string expectedFilterQuery = "SomeCompletePath?isDebug=true";
			string expectedFilterName = "NewFilter";

			request.FilterProject = expectedProject;
			request.FilterQuery = expectedFilterQuery;
			request.FilterName = expectedFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);

			Assert.IsNotNull(response);
			Assert.IsNotNull(response.Value);
			Assert.AreEqual(expectedProject, response.Value.FilterProject);
			Assert.AreEqual(expectedFilterQuery, response.Value.FilterQuery);
			Assert.AreEqual(expectedFilterName, response.Value.FilterName);
		}

		[TestMethod]
		public async Task TestNonExistentBuildHealthFilterFailureAsync()
		{
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			ProjectId expectedProject = new ProjectId("ue6");
			string expectedFilterQuery = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterQueryLength + 1);
			string expectedFilterName = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterNameLength + 1);

			request.FilterProject = expectedProject;
			request.FilterQuery = expectedFilterQuery;
			request.FilterName = expectedFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);

			Assert.IsNotNull(response);
			Assert.IsNull(response.Value);

			Assert.IsInstanceOfType(response.Result, typeof(BadRequestObjectResult));
		}

		[TestMethod]
		public async Task TestAddBuildHealthFilterFailureAsync()
		{
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			ProjectId expectedProject = new ProjectId("ue5");
			string expectedFilterQuery = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterQueryLength + 1);
			string expectedFilterName = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterNameLength + 1);

			request.FilterProject = expectedProject;
			request.FilterQuery = expectedFilterQuery;
			request.FilterName = expectedFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);

			Assert.IsNotNull(response);
			Assert.IsNull(response.Value);

			Assert.IsInstanceOfType(response.Result, typeof(BadRequestObjectResult));
		}

		[TestMethod]
		public async Task TestUpdatedBuildHealthFilterAsync()
		{
			ProjectId project = new ProjectId("ue5");

			// Create initial filter
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			string initialFilterQuery = "SomeCompletePath?isDebug=true";
			string initialFilterName = "NewFilter";

			request.FilterProject = project;
			request.FilterQuery = initialFilterQuery;
			request.FilterName = initialFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);
			Assert.IsNotNull(response);
			Assert.IsNotNull(response.Value);

			// Update previous filter
			BuildHealthFilterUpdateRequest updateRequest = new BuildHealthFilterUpdateRequest();
			string updatedFilterQuery = "SomeCompletePath?isDebug=false";
			string updatedFilterName = "SomeNewFilter";

			updateRequest.FilterQuery = updatedFilterQuery;
			updateRequest.FilterName = updatedFilterName;

			ActionResult<BuildHealthFilterResponse>? updateResponse = await BuildHealthFilterController.UpdateBuildHealthFilterAsync(response.Value.Id, updateRequest);

			Assert.IsNotNull(updateResponse);
			Assert.IsNotNull(updateResponse.Value);
			Assert.AreEqual(updatedFilterQuery, updateResponse.Value.FilterQuery);
			Assert.AreEqual(updatedFilterName, updateResponse.Value.FilterName);
			Assert.AreEqual(response.Value.Id, updateResponse.Value.Id);

			// Update just filter query
			string updatedFilterQuery2 = "SomeCompletePath?isDebug=true&stream=false";
			updateRequest.FilterName = String.Empty;
			updateRequest.FilterQuery = updatedFilterQuery2;

			ActionResult<BuildHealthFilterResponse>? updateResponse2 = await BuildHealthFilterController.UpdateBuildHealthFilterAsync(response.Value.Id, updateRequest);

			Assert.IsNotNull(updateResponse2);
			Assert.IsNotNull(updateResponse2.Value);
			Assert.AreEqual(updatedFilterQuery2, updateResponse2.Value.FilterQuery);
			Assert.AreEqual(updatedFilterName, updateResponse2.Value.FilterName);

			// Update just filter name
			string updatedFilterName2 = "SomeNewFilter2";
			updateRequest.FilterName = updatedFilterName2;
			updateRequest.FilterQuery = String.Empty;

			ActionResult<BuildHealthFilterResponse>? updateResponse3 = await BuildHealthFilterController.UpdateBuildHealthFilterAsync(updateResponse2.Value.Id, updateRequest);

			Assert.IsNotNull(updateResponse3);
			Assert.IsNotNull(updateResponse3.Value);
			Assert.AreEqual(updatedFilterQuery2, updateResponse3.Value.FilterQuery);
			Assert.AreEqual(updatedFilterName2, updateResponse3.Value.FilterName);
		}

		[TestMethod]
		public async Task TestUpdatedBuildHealthFilterFailureAsync()
		{
			ProjectId project = new ProjectId("ue5");

			// Create initial filter
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			string initialFilterQuery = "SomeCompletePath?isDebug=true";
			string initialFilterName = "NewFilter";

			request.FilterProject = project;
			request.FilterQuery = initialFilterQuery;
			request.FilterName = initialFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);
			Assert.IsNotNull(response);
			Assert.IsNotNull(response.Value);

			// Update previous filter
			BuildHealthFilterUpdateRequest updateRequest = new BuildHealthFilterUpdateRequest();
			string updatedFilterQuery = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterQueryLength + 1);
			string updatedFilterName = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterNameLength + 1);

			updateRequest.FilterQuery = updatedFilterQuery;
			updateRequest.FilterName = updatedFilterName;

			ActionResult<BuildHealthFilterResponse>? updateResponse = await BuildHealthFilterController.UpdateBuildHealthFilterAsync(response.Value.Id, updateRequest);
			Assert.IsNotNull(updateResponse);
			Assert.IsNull(updateResponse.Value);

			Assert.IsInstanceOfType(updateResponse.Result, typeof(BadRequestObjectResult));

			// Empty update should obtain nothing
			updateRequest.FilterQuery = String.Empty;
			updateRequest.FilterName = String.Empty;
			ActionResult<BuildHealthFilterResponse>? emptyUpdateResponse = await BuildHealthFilterController.UpdateBuildHealthFilterAsync(response.Value.Id, updateRequest);
			Assert.IsNotNull(emptyUpdateResponse);
			Assert.IsNull(emptyUpdateResponse.Value);
		}

		[TestMethod]
		public async Task TestGetBuildHealthFilterAsync()
		{
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			ProjectId expectedProject = new ProjectId("ue5");
			string expectedFilterQuery = "SomeFilter";
			string expectedFilterName = "SomeName";

			request.FilterProject = expectedProject;
			request.FilterQuery = expectedFilterQuery;
			request.FilterName = expectedFilterName;

			// Test inidvidiaul get
			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);
			Assert.IsNotNull(response);
			Assert.IsNotNull(response.Value);

			ActionResult<BuildHealthFilterResponse>? getResponse = await BuildHealthFilterController.GetBuildHealthFilterAsync(response!.Value!.Id);

			Assert.IsNotNull(getResponse);
			Assert.IsNotNull(getResponse.Value);
			Assert.AreEqual(expectedFilterName, getResponse.Value.FilterName);
			Assert.AreEqual(expectedFilterQuery, getResponse.Value.FilterQuery);
			Assert.AreEqual(response.Value.Id, getResponse.Value.Id);

			// Test project get
			for (int i = 0; i < 4; i++)
			{
				_ = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);
			}

			ActionResult<List<BuildHealthFilterResponse>> results = await BuildHealthFilterController.GetBuildHealthFiltersAsync(expectedProject);

			// Verify all the results are present
			Assert.IsNotNull(results);
			Assert.IsNotNull(results.Value);
			Assert.AreEqual(5, results.Value.Count);
		}

		[TestMethod]
		public async Task TestGetDeleteBuildHealthFilterAsync()
		{
			ProjectId project = new ProjectId("ue5");

			// Create initial filter
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			string initialFilterQuery = "SomeCompletePath?isDebug=true";
			string initialFilterName = "NewFilter";

			request.FilterProject = project;
			request.FilterQuery = initialFilterQuery;
			request.FilterName = initialFilterName;

			ActionResult<BuildHealthFilterResponse>? response = await BuildHealthFilterController.AddBuildHealthFilterAsync(request);
			Assert.IsNotNull(response);
			Assert.IsNotNull(response.Value);

			// Issue delete
			ActionResult deleteResponse = await BuildHealthFilterController.DeleteBuildHealthFiltersAsync(response.Value.Id);
			Assert.IsNotNull(deleteResponse);

			// Verify delete
			ActionResult<BuildHealthFilterResponse>? reentrantGetResponse = await BuildHealthFilterController.GetBuildHealthFilterAsync(response.Value.Id);
			Assert.IsNotNull(reentrantGetResponse);
			Assert.IsNull(reentrantGetResponse.Value);
		}

		[TestMethod]
		public async Task TestOwnershipSemanticsAsync()
		{
			ProjectId project = new ProjectId("ue5");

			// Create initial filter
			BuildHealthFilterAddRequest request = new BuildHealthFilterAddRequest();
			string initialFilterQuery = "SomeCompletePath?isDebug=true";
			string initialFilterName = "NewFilter";

			request.FilterProject = project;
			request.FilterQuery = initialFilterQuery;
			request.FilterName = initialFilterName;

			UserId originalUser = new UserId(BinaryIdUtils.CreateNew());

			IBuildHealthFilter? result = await BuildHealthFilterCollection.AddBuildHealthFilterAsync(originalUser, request, CancellationToken.None);
			Assert.IsNotNull(result);
			Assert.AreEqual(originalUser, result.Owner);

			// Modify filter of other user
			// Update previous filter
			UserId newUser = new UserId(BinaryIdUtils.CreateNew());
			BuildHealthFilterUpdateRequest updateRequest = new BuildHealthFilterUpdateRequest();
			string updatedFilterQuery = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterQueryLength + 1);
			string updatedFilterName = GenerateFailureString(BuildHealthFilterRestrictions.MaxFilterNameLength + 1);

			updateRequest.FilterQuery = updatedFilterQuery;
			updateRequest.FilterName = updatedFilterName;

			IBuildHealthFilter? failedUpdateResult = await BuildHealthFilterCollection.UpdateBuildHealthFilterAsync(result.Id, newUser, updateRequest, CancellationToken.None);
			Assert.IsNull(failedUpdateResult);

			// Delete filter of other user
			bool failedDeleteResult = await BuildHealthFilterCollection.DeleteBuildHealthFilterAsync(result.Id, newUser, CancellationToken.None);
			Assert.IsFalse(failedDeleteResult);
		}

		private static string GenerateFailureString(int length)
		{
			const string Chunk = "ABCDE";

			if (length <= 0)
			{
				return String.Empty;
			}

			return String.Concat(Enumerable.Repeat(Chunk, (length / Chunk.Length) + 1)).Substring(0, length);
		}
	}
}
