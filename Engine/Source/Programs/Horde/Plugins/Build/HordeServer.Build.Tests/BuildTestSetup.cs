// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using Amazon.AutoScaling;
using Amazon.CloudWatch;
using Amazon.EC2;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.BuildHealth;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Tools;
using HordeCommon.Rpc.Messages;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Enrollment;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Agents.Relay;
using HordeServer.Agents.Telemetry;
using HordeServer.Agents.Utilization;
using HordeServer.Artifacts;
using HordeServer.Build.BuildHealth;
using HordeServer.Commits;
using HordeServer.Compute;
using HordeServer.Dashboard;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Schedules;
using HordeServer.Jobs.Templates;
using HordeServer.Jobs.TestData;
using HordeServer.Jobs.Timing;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.RCA;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Streams;
using HordeServer.Tasks;
using HordeServer.Tests.Stubs.Services;
using HordeServer.Tools;
using HordeServer.Ugs;
using HordeServer.Users;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Moq;

namespace HordeServer.Tests
{
	/// <summary>
	/// Handles set up of collections, services, fixtures etc during testing
	///
	/// Easier to pass all these things around in a single object.
	/// </summary>
	public class BuildTestSetup : ServerTestSetup
	{
		public IGraphCollection GraphCollection => ServiceProvider.GetRequiredService<IGraphCollection>();
		public INotificationTriggerCollection NotificationTriggerCollection => ServiceProvider.GetRequiredService<INotificationTriggerCollection>();
		public IStreamCollection StreamCollection => ServiceProvider.GetRequiredService<IStreamCollection>();
		public IJobCollection JobCollection => ServiceProvider.GetRequiredService<IJobCollection>();
		public IAgentCollection AgentCollection => ServiceProvider.GetRequiredService<IAgentCollection>();
		public IJobStepRefCollection JobStepRefCollection => ServiceProvider.GetRequiredService<IJobStepRefCollection>();
		public IJobTimingCollection JobTimingCollection => ServiceProvider.GetRequiredService<IJobTimingCollection>();
		public IUgsMetadataCollection UgsMetadataCollection => ServiceProvider.GetRequiredService<IUgsMetadataCollection>();
		public IIssueCollection IssueCollection => ServiceProvider.GetRequiredService<IIssueCollection>();
		public IPoolCollection PoolCollection => ServiceProvider.GetRequiredService<IPoolCollection>();
		public ILeaseCollection LeaseCollection => ServiceProvider.GetRequiredService<ILeaseCollection>();
		public ILogCollection LogCollection => ServiceProvider.GetRequiredService<ILogCollection>();
		public ITestDataCollection TestDataCollection => ServiceProvider.GetRequiredService<ITestDataCollection>();
		public ITestDataCollectionV2 TestDataCollectionV2 => ServiceProvider.GetRequiredService<ITestDataCollectionV2>();
		public IUserCollection UserCollection => ServiceProvider.GetRequiredService<IUserCollection>();
		public IDeviceCollection DeviceCollection => ServiceProvider.GetRequiredService<IDeviceCollection>();
		public IDashboardPreviewCollection DashboardPreviewCollection => ServiceProvider.GetRequiredService<IDashboardPreviewCollection>();
		public IBisectTaskCollection BisectTaskCollection => ServiceProvider.GetRequiredService<IBisectTaskCollection>();
		public IBuildHealthFilterCollection BuildHealthFilterCollection => ServiceProvider.GetRequiredService<IBuildHealthFilterCollection>();

		public FleetService FleetService => ServiceProvider.GetRequiredService<FleetService>();
		public AgentService AgentService => ServiceProvider.GetRequiredService<AgentService>();
		public AgentRelayService AgentRelayService => ServiceProvider.GetRequiredService<AgentRelayService>();
		public ICommitService CommitService => ServiceProvider.GetRequiredService<ICommitService>();
		public GlobalsService GlobalsService => ServiceProvider.GetRequiredService<GlobalsService>();
		public ITemplateCollectionInternal TemplateCollection => ServiceProvider.GetRequiredService<ITemplateCollectionInternal>();
		internal PerforceServiceStub PerforceService => (PerforceServiceStub)ServiceProvider.GetRequiredService<IPerforceService>();
		public ISubscriptionCollection SubscriptionCollection => ServiceProvider.GetRequiredService<ISubscriptionCollection>();
		public INotificationService NotificationService => ServiceProvider.GetRequiredService<INotificationService>();
		public IssueService IssueService => ServiceProvider.GetRequiredService<IssueService>();
		public JobTaskSource JobTaskSource => ServiceProvider.GetRequiredService<JobTaskSource>();
		public ConformTaskSource ConformTaskSource => ServiceProvider.GetRequiredService<ConformTaskSource>();
		public JobService JobService => ServiceProvider.GetRequiredService<JobService>();
		public RpcService RpcService => ServiceProvider.GetRequiredService<RpcService>();
		public PoolService PoolService => ServiceProvider.GetRequiredService<PoolService>();
		public ScheduleService ScheduleService => ServiceProvider.GetRequiredService<ScheduleService>();
		public DeviceService DeviceService => ServiceProvider.GetRequiredService<DeviceService>();
		public StorageService StorageService => ServiceProvider.GetRequiredService<StorageService>();
		public TestDataService TestDataService => ServiceProvider.GetRequiredService<TestDataService>();
		public TestHealthService TestHealthService => ServiceProvider.GetRequiredService<TestHealthService>();
		public ComputeService ComputeService => ServiceProvider.GetRequiredService<ComputeService>();

		public StreamsController StreamsController => GetStreamsController();
		public JobsController JobsController => GetJobsController();
		public AgentsController AgentsController => GetAgentsController();
		public PoolsController PoolsController => GetPoolsController();
		public LeasesController LeasesController => GetLeasesController();
		public DevicesController DevicesController => GetDevicesController();
		public DashboardController DashboardController => GetDashboardController();
		public TestDataController TestDataController => GetTestDataController();
		public BisectTasksController BisectTasksController => GetBisectTasksController();
		public BuildHealthFilterController BuildHealthFilterController => GetBuildHealthFilterController();

		public BuildTestSetup()
		{
			AddPlugin<AnalyticsPlugin>();
			AddPlugin<BuildPlugin>();
			AddPlugin<ComputePlugin>();
			AddPlugin<StoragePlugin>();
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			IConfiguration config = new ConfigurationBuilder().Build();
			services.Configure<ServerSettings>(ConfigureSettings);
			services.AddSingleton<IConfiguration>(config);

			services.AddHttpClient<RpcService>();

			services.AddSingleton<IAgentCollection, AgentCollection>();
			services.AddSingleton<IAgentTelemetryCollection, AgentTelemetryCollection>();
			services.AddSingleton<ArtifactCollection>();
			services.AddSingleton<IArtifactCollection>(sp => sp.GetRequiredService<ArtifactCollection>());
			services.AddSingleton<ICommitService, CommitService>();
			services.AddSingleton<IGraphCollection, GraphCollection>();
			services.AddSingleton<IIssueCollection, IssueCollection>();
			services.AddSingleton<IJobCollection, JobCollection>();
			services.AddSingleton<IJobStepRefCollection, JobStepRefCollection>();
			services.AddSingleton<IJobTimingCollection, JobTimingCollection>();
			services.AddSingleton<ILeaseCollection, LeaseCollection>();
			services.AddSingleton<ILogCollection, LogCollection>();
			services.AddSingleton<INotificationTriggerCollection, NotificationTriggerCollection>();
			services.AddSingleton<IPoolCollection, PoolCollection>();
			services.AddSingleton<IBisectTaskCollection, BisectTaskCollection>();
			services.AddSingleton<ISubscriptionCollection, SubscriptionCollection>();
			services.AddSingleton<IStreamCollection, StreamCollection>();
			services.AddSingleton<ITemplateCollection, TemplateCollection>();
			services.AddSingleton<ITestDataCollection, TestDataCollection>();
			services.AddSingleton<ITestDataCollectionV2, TestDataCollectionV2>();
			services.AddSingleton<IUtilizationDataCollection, UtilizationDataCollection>();
			services.AddSingleton<ITemplateCollection, TemplateCollection>();
			services.AddSingleton<IToolCollection, ToolCollection>();
			services.AddSingleton<IUgsMetadataCollection, UgsMetadataCollection>();
			services.AddSingleton<IDeviceCollection, DeviceCollection>();

			// Empty mocked object to satisfy basic test runs
			services.AddSingleton<IAmazonEC2>(sp => new Mock<IAmazonEC2>().Object);
			services.AddSingleton<IAmazonAutoScaling>(sp => new Mock<IAmazonAutoScaling>().Object);
			services.AddSingleton<IAmazonCloudWatch>(sp => new Mock<IAmazonCloudWatch>().Object);
			services.AddSingleton<IFleetManagerFactory, FleetManagerFactory>();

			services.AddSingleton<IPoolSizeStrategyFactory, NoOpPoolSizeStrategyFactory>();
			services.AddSingleton<IPoolSizeStrategyFactory, JobQueueStrategyFactory>();
			services.AddSingleton<IPoolSizeStrategyFactory, LeaseUtilizationStrategyFactory>();
			services.AddSingleton<IPoolSizeStrategyFactory, LeaseUtilizationAwsMetricStrategyFactory>();

			services.AddSingleton<AgentService>();
			services.AddSingleton(provider => new Lazy<AgentService>(provider.GetRequiredService<AgentService>));
			services.AddSingleton(sp => new Lazy<IRcaProcessor?>(() => sp.GetService<IRcaProcessor>()));
			services.AddSingleton<AgentRelayService>();
			services.AddSingleton<AwsAutoScalingLifecycleService>();
			services.AddSingleton<FleetService>();
			services.AddSingleton<RequestTrackerService>();
			services.AddSingleton<GlobalsService>();
			services.AddSingleton<JobTaskSource>();
			services.AddSingleton<IssueService>();
			services.AddSingleton<JobService>();
			services.AddSingleton<JobExpirationService>();
			services.AddSingleton<LogTailService>();
			services.AddSingleton<INotificationService, NotificationService>();
			services.AddSingleton<IPerforceService, PerforceServiceStub>();
			services.AddSingleton<PerforceLoadBalancer>();
			services.AddSingleton<PoolService>();
			services.AddSingleton<BisectService>();
			services.AddSingleton<RpcService>();
			services.AddSingleton<ScheduleService>();
			services.AddSingleton<DeviceService>();
			services.AddSingleton<TestDataService>();
			services.AddSingleton<TestHealthService>();
			services.AddSingleton<ComputeService>();
			services.AddSingleton<EnrollmentService>();

			services.AddSingleton<ConformTaskSource>();
			services.AddSingleton<ICommitService, CommitService>();

			services.AddSingleton<IDefaultAclModifier, BuildAclModifier>();
		}

		public Task<Fixture> CreateFixtureAsync()
		{
			return Fixture.CreateAsync(ConfigService, GraphCollection, TemplateCollection, JobService, AgentService, PluginCollection, ServerSettings);
		}

		private StreamsController GetStreamsController()
		{
			StreamsController streamCtrl = ActivatorUtilities.CreateInstance<StreamsController>(ServiceProvider);
			streamCtrl.ControllerContext = GetControllerContext();
			return streamCtrl;
		}

		private JobsController GetJobsController()
		{
			JobsController jobsCtrl = ActivatorUtilities.CreateInstance<JobsController>(ServiceProvider);
			jobsCtrl.ControllerContext = GetControllerContext();
			return jobsCtrl;
		}

		private DevicesController GetDevicesController()
		{
			DevicesController devicesCtrl = ActivatorUtilities.CreateInstance<DevicesController>(ServiceProvider);
			devicesCtrl.ControllerContext = GetControllerContext();
			return devicesCtrl;
		}

		private DashboardController GetDashboardController()
		{
			DashboardController dashboardCtrl = ActivatorUtilities.CreateInstance<DashboardController>(ServiceProvider);
			dashboardCtrl.ControllerContext = GetControllerContext();
			return dashboardCtrl;
		}

		private TestDataController GetTestDataController()
		{
			TestDataController dataCtrl = ActivatorUtilities.CreateInstance<TestDataController>(ServiceProvider);
			dataCtrl.ControllerContext = GetControllerContext();
			return dataCtrl;
		}

		private BisectTasksController GetBisectTasksController()
		{
			BisectTasksController bisectCtrl = ActivatorUtilities.CreateInstance<BisectTasksController>(ServiceProvider);
			bisectCtrl.ControllerContext = GetControllerContext();
			return bisectCtrl;
		}

		private BuildHealthFilterController GetBuildHealthFilterController()
		{
			BuildHealthFilterController buildHealthFilterCtrl = ActivatorUtilities.CreateInstance<BuildHealthFilterController>(ServiceProvider);
			buildHealthFilterCtrl.ControllerContext = GetControllerContext();
			return buildHealthFilterCtrl;
		}

		private AgentsController GetAgentsController()
		{
			AgentsController agentCtrl = ActivatorUtilities.CreateInstance<AgentsController>(ServiceProvider);
			agentCtrl.ControllerContext = GetControllerContext();
			return agentCtrl;
		}

		private PoolsController GetPoolsController()
		{
			PoolsController controller = ActivatorUtilities.CreateInstance<PoolsController>(ServiceProvider);
			controller.ControllerContext = GetControllerContext();
			return controller;
		}

		private LeasesController GetLeasesController()
		{
			LeasesController controller = ActivatorUtilities.CreateInstance<LeasesController>(ServiceProvider);
			controller.ControllerContext = GetControllerContext();
			return controller;
		}

		private static ControllerContext GetControllerContext()
		{
			ControllerContext controllerContext = new ControllerContext();
			controllerContext.HttpContext = new DefaultHttpContext();
			controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
				new List<Claim> { HordeClaims.AdminClaim.ToClaim() }, "TestAuthType"));
			return controllerContext;
		}

		private static int s_agentIdCounter = 1;
		public Task<IAgent> CreateAgentAsync(IPool pool, bool enabled = true, bool requestShutdown = false, List<string>? properties = null, List<AgentWorkspaceInfo>? workspaces = null, TimeSpan? adjustClockBy = null, AgentStatus status = AgentStatus.Ok)
		{
			return CreateAgentAsync(pool.Id, enabled, requestShutdown, properties, workspaces, adjustClockBy, status: status);
		}

		/// <summary>
		/// Helper function for setting up agents to be used in tests
		/// </summary>
		/// <param name="poolId">Pool ID which the agent should belong to</param>
		/// <param name="enabled">Whether set the agent as enabled</param>
		/// <param name="requestShutdown">Mark it with a request for shutdown</param>
		/// <param name="properties">Any properties to assign</param>
		/// <param name="workspaces">Any workspaces to assign</param>
		/// <param name="adjustClockBy">Time span to temporarily skew the clock when creating the agent</param>
		/// <param name="awsInstanceId">AWS instance ID for the agent (will be set in properties)</param>
		/// <param name="lease">A lease to assign the agent</param>
		/// <param name="ephemeral">Whether the agent is ephemeral</param>
		/// <param name="status">Initial status for the agent</param>
		/// <returns>A new agent</returns>
		public async Task<IAgent> CreateAgentAsync(
			PoolId? poolId,
			bool enabled = true,
			bool requestShutdown = false,
			List<string>? properties = null,
			List<AgentWorkspaceInfo>? workspaces = null,
			TimeSpan? adjustClockBy = null,
			string? awsInstanceId = null,
			CreateLeaseOptions? lease = null,
			bool ephemeral = false,
			AgentStatus? status = AgentStatus.Ok)
		{
			DateTime now = Clock.UtcNow;
			if (adjustClockBy != null)
			{
				Clock.UtcNow = now + adjustClockBy.Value;
			}

			List<string> tempProps = new(properties ?? new List<string>());
			if (awsInstanceId != null)
			{
				tempProps.Add(KnownPropertyNames.AwsInstanceId + "=" + awsInstanceId);
			}

			IAgent? agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("TestAgent" + s_agentIdCounter++), AgentMode.Dedicated, ephemeral, ""));
			Assert.IsNotNull(agent);

			agent = await agent.TryUpdateAsync(new UpdateAgentOptions { Enabled = enabled, ExplicitPools = poolId != null ? [poolId.Value] : [] });
			Assert.IsNotNull(agent);

			if (status != null)
			{
				agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(tempProps), null);
				Assert.IsNotNull(agent);

				if (status != AgentStatus.Ok)
				{
					agent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions { Status = status });
					Assert.IsNotNull(agent);
				}
			}

			if (workspaces is { Count: > 0 })
			{
				await agent.TryUpdateWorkspacesAsync(workspaces, false);
			}

			if (requestShutdown)
			{
				await agent.TryUpdateAsync(new UpdateAgentOptions { RequestShutdown = true });
			}

			if (lease != null)
			{
				await agent.TryCreateLeaseAsync(lease);
			}

			Clock.UtcNow = now;
			return agent;
		}

		protected Task<IPool> CreatePoolAsync(PoolConfig poolConfig, bool viaConfig = true)
		{
			return viaConfig ? CreatePoolViaConfigAsync(poolConfig) : CreatePoolViaDatabaseAsync(poolConfig);
		}

		protected async Task<IPool> CreatePoolViaConfigAsync(PoolConfig poolConfig)
		{
			await UpdateConfigAsync(config => config.Plugins.GetComputeConfig().Pools.Add(poolConfig));
			return await PoolCollection.GetAsync(poolConfig.Id) ?? throw new Exception($"Unable to get pool ID {poolConfig.Id}");
		}

		protected async Task<IPool> CreatePoolViaDatabaseAsync(PoolConfig poolConfig)
		{
#pragma warning disable CS0618 // Type or member is obsolete
			await PoolCollection.CreateConfigAsync(poolConfig.Id, poolConfig.Name, new CreatePoolConfigOptions()
			{
				Condition = poolConfig.Condition,
				EnableAutoscaling = poolConfig.EnableAutoscaling,
				MinAgents = poolConfig.MinAgents,
				NumReserveAgents = poolConfig.NumReserveAgents,
				ConformInterval = poolConfig.ConformInterval,
				ScaleOutCooldown = poolConfig.ScaleOutCooldown,
				ScaleInCooldown = poolConfig.ScaleInCooldown,
				SizeStrategies = poolConfig.SizeStrategies,
				FleetManagers = poolConfig.FleetManagers,
				SizeStrategy = poolConfig.SizeStrategy,
				LeaseUtilizationSettings = poolConfig.LeaseUtilizationSettings,
				JobQueueSettings = poolConfig.JobQueueSettings,
				ComputeQueueAwsMetricSettings = poolConfig.ComputeQueueAwsMetricSettings,
				Properties = poolConfig.Properties,
			});
			return await PoolCollection.GetAsync(poolConfig.Id) ?? throw new Exception($"Unable to get pool ID {poolConfig.Id}");
#pragma warning restore CS0618 // Type or member is obsolete
		}
	}
}