// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using HordeServer.Streams;

namespace HordeServer.Tests.Commits
{
	[TestClass]
	public class CommitTagTests
	{
		[TestMethod]
		public void TestCode()
		{
			StreamConfig config = new StreamConfig();

			FileFilter? filter;
			Assert.IsTrue(config.TryGetCommitTagFilter(CommitTag.Code, out filter));

			Assert.IsTrue(filter.Matches("foo.cpp"));
			Assert.IsTrue(filter.Matches("foo.h"));
			Assert.IsTrue(filter.Matches("foo.cs"));
			Assert.IsFalse(filter.Matches("foo.uasset"));
			Assert.IsFalse(filter.Matches("foo.png"));
		}

		[TestMethod]
		public void TestContent()
		{
			StreamConfig config = new StreamConfig();

			FileFilter? filter;
			Assert.IsTrue(config.TryGetCommitTagFilter(CommitTag.Content, out filter));

			Assert.IsFalse(filter.Matches("foo.cpp"));
			Assert.IsFalse(filter.Matches("foo.h"));
			Assert.IsFalse(filter.Matches("foo.cs"));
			Assert.IsTrue(filter.Matches("foo.uasset"));
			Assert.IsTrue(filter.Matches("foo.png"));
		}

		[TestMethod]
		public void TestCustom()
		{
			StreamConfig config = new StreamConfig();
			config.CommitTags.Add(new CommitTagConfig { Name = new CommitTag("test"), Filter = new List<string> { "*TEST*", "-*TEST2*" } });

			FileFilter? filter;
			Assert.IsTrue(config.TryGetCommitTagFilter(new CommitTag("test"), out filter));

			Assert.IsFalse(filter.Matches("somefile"));
			Assert.IsTrue(filter.Matches("sometestfile"));
			Assert.IsFalse(filter.Matches("sometest2file"));
		}

		[TestMethod]
		public void TestInheritancee()
		{
			StreamConfig config = new StreamConfig();
			config.CommitTags.Add(new CommitTagConfig { Name = new CommitTag("test"), Base = CommitTag.Code, Filter = new List<string> { "*TEST*", "-*TEST2*" } });

			FileFilter? filter;
			Assert.IsTrue(config.TryGetCommitTagFilter(new CommitTag("test"), out filter));

			Assert.IsTrue(filter.Matches("somefile.cpp"));
			Assert.IsFalse(filter.Matches("somefile.uasset"));
			Assert.IsTrue(filter.Matches("sometestfile.uasset"));
			Assert.IsFalse(filter.Matches("sometest2file.uasset"));
		}
	}
}
