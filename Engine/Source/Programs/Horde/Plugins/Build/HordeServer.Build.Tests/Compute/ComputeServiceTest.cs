// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Diagnostics.Metrics;
using System.Net;
using System.Security.Claims;
using System.Text.RegularExpressions;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Compute;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Compute;
using HordeServer.Plugins;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Tests.Compute
{
	[TestClass]
	public class ComputeServiceTest : BuildTestSetup
	{
		private class StubMessageHandler(HttpStatusCode statusCode, string content) : HttpMessageHandler
		{
			private string Content { get; set; } = content;
			protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
			{
				return Task.FromResult(new HttpResponseMessage(statusCode) { Content = new StringContent(Content) });
			}
		}
		
		private readonly ClusterId _cluster1 = new("cluster1");
		private readonly AclClaimConfig _claimConfig = new (HordeClaimTypes.User, "some.user");
		private readonly AclConfig _ubaCacheReadAcl;
		private readonly AclConfig _ubaCacheWriteAcl;
		private readonly AclConfig _ubaCacheReadWriteAcl;
		private readonly ClaimsPrincipal _user;
		private readonly UbaComputeClusterConfig _uccc = new() { CacheEndpoint = "1.2.3.4:3333", CacheHttpEndpoint = "1.2.3.4:4444", CacheHttpCrypto = "aaaaaaaaaaaaaaaaaaaaaaaaaaaaaaaa" };
		
		public ComputeServiceTest()
		{
			_user = new ClaimsPrincipal(new ClaimsIdentity(new List<Claim> { _claimConfig.ToClaim() }, "TestAuthType"));
			_ubaCacheReadAcl = new AclConfig { Entries = [new AclEntryConfig(_claimConfig, [ComputeAclAction.UbaCacheRead])] };
			_ubaCacheWriteAcl = new AclConfig { Entries = [new AclEntryConfig(_claimConfig, [ComputeAclAction.UbaCacheWrite])] };
			_ubaCacheReadWriteAcl = new AclConfig { Entries = [new AclEntryConfig(_claimConfig, [ComputeAclAction.UbaCacheRead, ComputeAclAction.UbaCacheWrite])] };
		}
		
		#region Resource needs
		[TestMethod]
		public async Task ResourceNeedsMetricAsync()
		{
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool1", new() { { "cpu", 101 }, { "ram", 1000 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool1", new() { { "cpu", 102 }, { "ram", 1001 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session2", "pool1", new() { { "cpu", 210 }, { "ram", 1100 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session2", "pool1", new() { { "cpu", 220 }, { "ram", 1200 } });

			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool2", new() { { "cpu", 301 }, { "ram", 1300 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session2", "pool2", new() { { "cpu", 410 }, { "ram", 1400 } });

			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster2"), "session1", "pool1", new() { { "cpu", 20 }, { "ram", 4000 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster2"), "session2", "pool1", new() { { "cpu", 30 }, { "ram", 5500 } });

			List<Measurement<int>> measurements = await ComputeService.CalculateResourceNeedsAsync();
			Assert.AreEqual(6, measurements.Count);
			AssertContainsMeasurement(measurements, 322, "cpu", "cluster1", "pool1");
			AssertContainsMeasurement(measurements, 2201, "ram", "cluster1", "pool1");
			AssertContainsMeasurement(measurements, 711, "cpu", "cluster1", "pool2");
			AssertContainsMeasurement(measurements, 2700, "ram", "cluster1", "pool2");
			AssertContainsMeasurement(measurements, 50, "cpu", "cluster2", "pool1");
			AssertContainsMeasurement(measurements, 9500, "ram", "cluster2", "pool1");
		}

		[TestMethod]
		public async Task ResourceNeedsAreReplacedAsync()
		{
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool1", new() { { "cpu", 1 }, { "ram", 5 } });
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool1", new() { { "cpu", 10 }, { "ram", 50 } });
			List<ComputeService.SessionResourceNeeds> result = await ComputeService.GetResourceNeedsAsync();
			Assert.AreEqual(1, result.Count);
			Assert.AreEqual(10, result[0].ResourceNeeds["cpu"]);
			Assert.AreEqual(50, result[0].ResourceNeeds["ram"]);
		}

		[TestMethod]
		public async Task ResourceNeedsAreRemovedWhenOutdatedAsync()
		{
			await ComputeService.SetResourceNeedsAsync(new ClusterId("cluster1"), "session1", "pool1", new() { { "cpu", 1 }, { "ram", 5 } });
			Assert.AreEqual(1, (await ComputeService.GetResourceNeedsAsync()).Count);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(6));
			Assert.AreEqual(0, (await ComputeService.GetResourceNeedsAsync()).Count);
		}

		[TestMethod]
		public async Task DeniedRequestAsync()
		{
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);

			List<ComputeService.RequestInfo> ids = await ComputeService.GetUnservedRequestsAsync();
			Assert.AreEqual(1, ids.Count);
		}

		[TestMethod]
		public async Task AcceptedRequestAsync()
		{
			await ComputeService.LogRequestAsync(AllocationOutcome.Accepted, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Accepted, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Accepted, "req2", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);

			List<ComputeService.RequestInfo> ids = await ComputeService.GetUnservedRequestsAsync();
			Assert.AreEqual(0, ids.Count);
		}

		[TestMethod]
		public async Task DeniedThenAcceptedRequestAsync()
		{
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Accepted, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);

			List<ComputeService.RequestInfo> ids = await ComputeService.GetUnservedRequestsAsync();
			Assert.AreEqual(0, ids.Count);
		}

		[TestMethod]
		public async Task OnlyIncludeLastMinuteAsync()
		{
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromMinutes(61));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req2", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);

			List<ComputeService.RequestInfo> ids = await ComputeService.GetUnservedRequestsAsync();
			Assert.AreEqual(1, ids.Count);
			Assert.AreEqual("req2", ids[0].RequestId);
		}

		[TestMethod]
		public async Task ComplexAsync()
		{
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req2", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req1", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req2", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Denied, "req3", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(5));
			await ComputeService.LogRequestAsync(AllocationOutcome.Accepted, "req2", new Requirements { Pool = "pool1" }, null, Tracer.CurrentSpan);

			List<ComputeService.RequestInfo> ids = await ComputeService.GetUnservedRequestsAsync();
			CollectionAssert.AreEquivalent(new List<string> { "req1", "req3" }, ids.Select(x => x.RequestId).ToList());
		}

		[TestMethod]
		public void SerializeRequestInfo()
		{
			{
				ComputeService.RequestInfo req = new(DateTimeOffset.UnixEpoch, AllocationOutcome.Accepted, "reqId1", "pool1", "parent1");
				string serialized = req.Serialize();
				ComputeService.RequestInfo deserialized = ComputeService.RequestInfo.Deserialize(serialized)!;
				Assert.AreEqual(DateTimeOffset.UnixEpoch, deserialized!.Timestamp);
				Assert.AreEqual(AllocationOutcome.Accepted, deserialized.Outcome);
				Assert.AreEqual("reqId1", deserialized.RequestId);
				Assert.AreEqual("pool1", deserialized.Pool);
				Assert.AreEqual("parent1", deserialized.ParentLeaseId);
			}

			{
				ComputeService.RequestInfo req = new(DateTimeOffset.UnixEpoch, AllocationOutcome.Accepted, "reqId1", "pool1", null);
				string serialized = req.Serialize();
				ComputeService.RequestInfo deserialized = ComputeService.RequestInfo.Deserialize(serialized)!;
				Assert.IsNull(deserialized.ParentLeaseId);
			}
		}

		[TestMethod]
		public void GroupByPoolAndCount()
		{
			List<ComputeService.RequestInfo> ris = new()
			{
				new ComputeService.RequestInfo(DateTimeOffset.UnixEpoch, AllocationOutcome.Denied, "reqId1", "poolA", null),
				new ComputeService.RequestInfo(DateTimeOffset.UnixEpoch, AllocationOutcome.Denied, "reqId2", "poolA", null),
				new ComputeService.RequestInfo(DateTimeOffset.UnixEpoch, AllocationOutcome.Denied, "reqId3", "poolA", null),
				new ComputeService.RequestInfo(DateTimeOffset.UnixEpoch, AllocationOutcome.Denied, "reqId4", "poolB", null),
				new ComputeService.RequestInfo(DateTimeOffset.UnixEpoch, AllocationOutcome.Denied, "reqId4", "poolB", null),
			};
			Dictionary<string, int> result = ComputeService.GroupByPoolAndCount(ris);
			Assert.AreEqual(3, result["poolA"]);
			Assert.AreEqual(2, result["poolB"]);
		}
		#endregion Resource needs

		[TestMethod]
		public async Task PoolNameTemplatingAsync()
		{
			IOptionsMonitor<ComputeConfig> computeConfig = ServiceProvider.GetRequiredService<IOptionsMonitor<ComputeConfig>>();
			computeConfig.CurrentValue.Networks = new List<NetworkConfig>
			{
				new() { CidrBlock = "12.0.0.0/16", Id = "myNetworkId", ComputeId = "myComputeId" }
			};

			IPAddress ip = IPAddress.Parse("12.0.10.30");
			List<string> props = new() { "ComputeIp=11.0.0.1", "ComputePort=5000" };
			IAgent agent1 = await CreateAgentAsync(new PoolId("foo"), properties: props);
			IAgent agent2 = await CreateAgentAsync(new PoolId("bar-default"), properties: props);
			IAgent agent3 = await CreateAgentAsync(new PoolId("bar-myNetworkId"), properties: props);
			IAgent agent4 = await CreateAgentAsync(new PoolId("qux-myComputeId"), properties: props);
			ClusterId clusterId = new("default");

			AllocateResourceParams arp1 = new(clusterId, _user, ComputeProtocol.Latest, new Requirements { Pool = "foo" }) { RequestId = "req1", RequesterIp = ip, ParentLeaseId = null };
			ComputeResource resource1 = await ComputeService.TryAllocateResourceAsync(arp1, CancellationToken.None);
			Assert.AreEqual(agent1.Id, resource1!.AgentId);

			AllocateResourceParams arp2 = new(clusterId, _user, ComputeProtocol.Latest, new Requirements { Pool = "bar-%REQUESTER_NETWORK_ID%" }) { RequestId = "req2", RequesterIp = IPAddress.Parse("15.0.0.1"), ParentLeaseId = null };
			ComputeResource resource2 = await ComputeService.TryAllocateResourceAsync(arp2, CancellationToken.None);
			Assert.AreEqual(agent2.Id, resource2!.AgentId);

			AllocateResourceParams arp3 = new(clusterId, _user, ComputeProtocol.Latest, new Requirements { Pool = "bar-%REQUESTER_NETWORK_ID%" }) { RequestId = "req3", RequesterIp = ip, ParentLeaseId = null };
			ComputeResource resource3 = await ComputeService.TryAllocateResourceAsync(arp3, CancellationToken.None);
			Assert.AreEqual(agent3.Id, resource3!.AgentId);

			AllocateResourceParams arp4 = new(clusterId, _user, ComputeProtocol.Latest, new Requirements { Pool = "qux-%REQUESTER_COMPUTE_ID%" }) { RequestId = "req4", RequesterIp = ip, ParentLeaseId = null };
			ComputeResource resource4 = await ComputeService.TryAllocateResourceAsync(arp4, CancellationToken.None);
			Assert.AreEqual(agent4.Id, resource4!.AgentId);
		}

		[TestMethod]
		public async Task Assignment_RequesterAlsoRunsAgent_IsNotAssignedAsync()
		{
			AllocateResourceParams CreateParams(string ip)
			{
				return new AllocateResourceParams(new ClusterId("default"), _user, ComputeProtocol.Latest, new Requirements { Pool = "foo" }) { RequesterIp = IPAddress.Parse(ip) };
			}

			IAgent agent = await CreateAgentAsync(new PoolId("foo"), properties: ["ComputeIp=11.0.0.1", "ComputePort=5000"]);

			await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(CreateParams("11.0.0.1"), CancellationToken.None));
			await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(CreateParams("11.0.0.1"), CancellationToken.None));

			ComputeResource resource2 = await ComputeService.TryAllocateResourceAsync(CreateParams("11.0.0.2"), CancellationToken.None);
			Assert.AreEqual(agent.Id, resource2!.AgentId);
		}

		[TestMethod]
		public async Task Assignment_PoolInRequirementsAsync()
		{
			AllocateResourceParams CreateParams(string pool)
			{
				return new AllocateResourceParams(new ClusterId("default"), _user, ComputeProtocol.Latest, new Requirements { Pool = pool });
			}

			IAgent agent1 = await CreateAgentAsync(new PoolId("foo"), properties: ["ComputeIp=11.0.0.1", "ComputePort=5000"]);
			IAgent agent2 = await CreateAgentAsync(new PoolId("bar"), properties: ["ComputeIp=11.0.0.2", "ComputePort=5000"]);

			ComputeResource resource1 = await ComputeService.TryAllocateResourceAsync(CreateParams("foo"), CancellationToken.None);
			Assert.AreEqual(agent1.Id, resource1!.AgentId);

			ComputeResource resource2 = await ComputeService.TryAllocateResourceAsync(CreateParams("bar"), CancellationToken.None);
			Assert.AreEqual(agent2.Id, resource2!.AgentId);

			await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(CreateParams("does-not-exist"), CancellationToken.None));
		}
		
		[TestMethod]
		public async Task Assignment_Protocol_Async()
		{
			await CreateComputeAgentAsync("pool1", protocol: ComputeProtocol.Initial);
			Assert.AreEqual(ComputeProtocol.Initial, (await AllocateAsync("pool1"))!.Protocol);
			
			await CreateComputeAgentAsync("pool2", protocol: ComputeProtocol.Latest);
			Assert.AreEqual(ComputeProtocol.Latest, (await AllocateAsync("pool2"))!.Protocol);
			
			await CreateComputeAgentAsync("pool3", protocol: ComputeProtocol.Latest);
			Assert.AreEqual(ComputeProtocol.Initial, (await AllocateAsync("pool3", protocol: ComputeProtocol.Initial))!.Protocol);
		}
		
		[TestMethod]
		public async Task Assignment_NoResources_Async()
		{
			AllocateResourceParams CreateParams(string pool)
			{
				return new AllocateResourceParams(new ClusterId("default"), _user, ComputeProtocol.Latest, new Requirements { Pool = pool, Exclusive = true });
			}
			
			await CreateAgentAsync(new PoolId("foo"), properties: ["ComputeIp=11.0.0.1", "ComputePort=5000"]);
			await ComputeService.TryAllocateResourceAsync(CreateParams("foo"), CancellationToken.None);
			NoComputeResourcesException ex1 = await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(CreateParams("foo"), CancellationToken.None));
			Assert.AreEqual(NoComputeResourcesException.AllResourcesInUse, ex1.Message);
			
			NoComputeResourcesException ex2 = await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(CreateParams("bar"), CancellationToken.None));
			Assert.AreEqual(NoComputeResourcesException.NoMatchingResources, ex2.Message);
		}

		[TestMethod]
		public async Task Assignment_ComputeClusterMemberConditionAsync()
		{
			ClusterId cluster1 = new("cluster1");
			ClusterId cluster2 = new("cluster2");
			PoolId poolFoo = new("foo");
			PoolId poolBar = new("bar");

			IOptionsMonitor<ComputeConfig> computeConfig = ServiceProvider.GetRequiredService<IOptionsMonitor<ComputeConfig>>();
			computeConfig.CurrentValue.Clusters = new List<ComputeClusterConfig>
			{
				new() { Id = cluster1, Condition = $"pool == 'fOO'" }, // Test case-insensitivity of pool ID matching
				new() { Id = cluster2, Condition = $"pool == '{poolBar.ToString()}'" }
			};
			GlobalConfig.CurrentValue.PostLoad(new ServerSettings(), Array.Empty<ILoadedPlugin>(), Array.Empty<IDefaultAclModifier>());

			List<string> props = ["ComputeIp=11.0.0.1", "ComputePort=5000"];
			IAgent agent1 = await CreateAgentAsync(new PoolId("foo"), properties: props);
			IAgent agent2 = await CreateAgentAsync(new PoolId("bar"), properties: props);

			{
				AllocateResourceParams arpFoo = new(cluster1, _user, ComputeProtocol.Latest, new Requirements { Pool = poolFoo.ToString() });
				Assert.AreEqual(agent1.Id, (await ComputeService.TryAllocateResourceAsync(arpFoo, CancellationToken.None))!.AgentId);

				AllocateResourceParams arpBar = new(cluster1, _user, ComputeProtocol.Latest, new Requirements { Pool = poolBar.ToString() });
				await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(arpBar, CancellationToken.None));
			}

			{
				AllocateResourceParams arpFoo = new(cluster2, _user, ComputeProtocol.Latest, new Requirements { Pool = poolFoo.ToString() });
				await Assert.ThrowsExactlyAsync<NoComputeResourcesException>(() => ComputeService.TryAllocateResourceAsync(arpFoo, CancellationToken.None));

				AllocateResourceParams arpBar = new(cluster2, _user, ComputeProtocol.Latest, new Requirements { Pool = poolBar.ToString() });
				Assert.AreEqual(agent2.Id, (await ComputeService.TryAllocateResourceAsync(arpBar, CancellationToken.None))!.AgentId);
			}
		}

		#region Cluster
		[TestMethod]
		[DataRow("11.0.0.1", "11.0.0.1", null, false)]
		[DataRow("11.0.0.1", "11.0.0.1", "200.20.20.20", false)]
		[DataRow("200.20.20.20", "11.0.0.1", "200.20.20.20", true)]
		[DataRow("11.0.0.1", "11.0.0.1", "bad-ip", true)]
		[DataRow("200.20.20.20", null, "200.20.20.20", true)]
		public void Cluster_ResolveRequesterIp(string expectIp, string? internalIp, string? publicIp, bool? usePublicIp)
		{
			Assert.AreEqual(IPAddress.Parse(expectIp), ComputeService.ResolveRequesterIp(internalIp == null ? null : IPAddress.Parse(internalIp), usePublicIp, publicIp));
		}

		[TestMethod]
		public void Cluster_FindBest()
		{
			ComputeConfig computeConfig = new()
			{
				Networks = [
				new NetworkConfig { CidrBlock = "11.0.0.0/16", Id = "network1", ComputeId = "compute1" },
				new NetworkConfig { CidrBlock = "12.0.0.0/16", Id = "network2", ComputeId = "compute2" },
			]
			};

			PluginConfigOptions configOptions = new PluginConfigOptions(ConfigVersion.Latest, Enumerable.Empty<IPluginConfig>(), new HordeServer.Acls.AclConfig());
			computeConfig.PostLoad(configOptions);

			Assert.AreEqual(new ClusterId("compute1"), ComputeService.FindBestComputeClusterId(computeConfig, IPAddress.Parse("11.0.0.1")));
			Assert.AreEqual(new ClusterId("compute2"), ComputeService.FindBestComputeClusterId(computeConfig, IPAddress.Parse("12.0.1.1")));
			Assert.ThrowsExactly<ComputeServiceException>(() => ComputeService.FindBestComputeClusterId(computeConfig, IPAddress.Parse("123.123.123.123")));

			ComputeConfig globalConfigCatchAll = new()
			{
				Networks = [new NetworkConfig { CidrBlock = "0.0.0.0/0", Id = "catchAll", ComputeId = "catchAll" }]
			};
			globalConfigCatchAll.PostLoad(configOptions);
			Assert.AreEqual(new ClusterId("catchAll"), ComputeService.FindBestComputeClusterId(globalConfigCatchAll, IPAddress.Parse("123.123.123.123")));
		}
		#endregion Cluster
		
		#region Connection
		[TestMethod]
		public async Task Connection_Direct_IpConnection_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Direct);
			Assert.AreEqual(ConnectionMode.Direct, cr!.ConnectionMode);
			Assert.AreEqual("11.0.0.1", cr.Ip.ToString());
			Assert.IsNull(cr.ConnectionAddress);
		}

		[TestMethod]
		public async Task Connection_Direct_PortsAreMapped_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Direct, ports: new Dictionary<string, int> { { "myOtherPort", 13000 }, { "myPort", 12000 } });
			Assert.AreEqual(3, cr!.Ports.Count);
			Assert.AreEqual(new ComputeResourcePort(5000, 5000), cr.Ports[ConnectionMetadataPort.ComputeId]);
			Assert.AreEqual(new ComputeResourcePort(12000, 12000), cr.Ports["myPort"]);
			Assert.AreEqual(new ComputeResourcePort(13000, 13000), cr.Ports["myOtherPort"]);
		}

		[TestMethod]
		public async Task Connection_Tunnel_IpConnection_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Tunnel, tunnelAddress: "localhost:3344");
			Assert.AreEqual(ConnectionMode.Tunnel, cr!.ConnectionMode);
			Assert.AreEqual("11.0.0.1", cr.Ip.ToString());
			Assert.AreEqual("localhost:3344", cr.ConnectionAddress);
		}

		[TestMethod]
		public async Task Connection_Tunnel_PortsAreMapped_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Tunnel, tunnelAddress: "localhost:3344", ports: new Dictionary<string, int> { { "myOtherPort", 13000 }, { "myPort", 12000 } });
			Assert.AreEqual(3, cr!.Ports.Count);
			Assert.AreEqual(new ComputeResourcePort(-1, 5000), cr.Ports[ConnectionMetadataPort.ComputeId]);
			Assert.AreEqual(new ComputeResourcePort(-1, 12000), cr.Ports["myPort"]);
			Assert.AreEqual(new ComputeResourcePort(-1, 13000), cr.Ports["myOtherPort"]);
		}

		[TestMethod]
		public async Task Connection_Relay_IpConnection_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Relay);
			Assert.AreEqual(ConnectionMode.Relay, cr!.ConnectionMode);
			Assert.AreEqual("11.0.0.1", cr.Ip.ToString());
			Assert.AreEqual("192.168.1.1", cr.ConnectionAddress);
		}

		[TestMethod]
		public async Task Connection_Relay_PortsAreMapped_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(ConnectionMode.Relay, ports: new Dictionary<string, int> { { "myOtherPort", 13000 }, { "myPort", 12000 } });
			Assert.AreEqual(ConnectionMode.Relay, cr!.ConnectionMode);
			Assert.AreEqual(3, cr.Ports.Count);
			Assert.AreEqual(new ComputeResourcePort(12214, 5000), cr.Ports[ConnectionMetadataPort.ComputeId]);
			Assert.AreEqual(new ComputeResourcePort(18656, 12000), cr.Ports["myPort"]);
			Assert.AreEqual(new ComputeResourcePort(23151, 13000), cr.Ports["myOtherPort"]);
		}
		
		[TestMethod]
		public async Task Connection_UbaCache_CacheServerDefined_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(useUbaCache: true, ubaConfig: _uccc, acl: _ubaCacheReadAcl);
			Assert.AreEqual("1.2.3.4:3333", cr!.Uba!.CacheEndpoint);
			Assert.IsTrue(Regex.IsMatch(cr.Uba!.CacheSessionKey, "^[0-9a-fA-F]{32}$"));
		}
		
		[TestMethod]
		public async Task Connection_UbaCache_ReadPermission_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(useUbaCache: true, ubaConfig: _uccc, acl: _ubaCacheReadAcl);
			Assert.IsFalse(cr!.Uba!.WriteAccess);
		}
		
		[TestMethod]
		public async Task Connection_UbaCache_WritePermission_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(useUbaCache: true, ubaConfig: _uccc, acl: _ubaCacheWriteAcl);
			Assert.IsTrue(cr!.Uba!.WriteAccess);
		}
		
		[TestMethod]
		public async Task Connection_UbaCache_ReadWritePermission_Async()
		{
			ComputeResource? cr = await AllocateWithAgentAsync(useUbaCache: true, ubaConfig: _uccc, acl: _ubaCacheReadWriteAcl);
			Assert.IsTrue(cr!.Uba!.WriteAccess);
		}
		
		[TestMethod]
		[Ignore] // Uncomment to manually test allocation with a local UBA cache server
		public async Task Connection_UbaCache_RealServer_Async()
		{
			UbaComputeClusterConfig uccc = new()
			{
				CacheEndpoint = "localhost:7009",
				CacheHttpEndpoint = "localhost:7010",
				CacheHttpCrypto = "6C4AD3E4406C08C9C435DE3B4F400FD4"
			};
			using HttpClient httpClient = new ();
			ComputeResource? cr = await AllocateWithAgentAsync(useUbaCache: true, ubaConfig: uccc, httpClient: httpClient, acl: _ubaCacheReadAcl);
			Assert.AreEqual(uccc.CacheEndpoint, cr!.Uba!.CacheEndpoint);
			Console.WriteLine("CacheSessionKey: " + cr.Uba!.CacheSessionKey);
			Assert.IsTrue(Regex.IsMatch(cr.Uba!.CacheSessionKey, "^[0-9a-fA-F]{32}$"));
		}
		
		[TestMethod]
		public async Task Connection_UbaCache_CacheServerNotDefined_Async()
		{
			ComputeServiceException cse1 = await Assert.ThrowsExactlyAsync<ComputeServiceException>(() => AllocateWithAgentAsync(useUbaCache: true, ubaConfig: null));
			Assert.IsTrue(cse1.Message.StartsWith("No UBA cache server defined for compute cluster", StringComparison.Ordinal));
			
			UbaComputeClusterConfig uccc = new() { CacheEndpoint = "localhost:7009", CacheHttpEndpoint = null, CacheHttpCrypto = "aaaaaaaaaaaaaaaaaaaaaaaaaaaaaaaa" };
			ComputeServiceException cse2 = await Assert.ThrowsExactlyAsync<ComputeServiceException>(() => AllocateWithAgentAsync(useUbaCache: true, ubaConfig: uccc));
			Assert.IsTrue(cse2.Message.StartsWith("No UBA cache server defined for compute cluster", StringComparison.Ordinal));
		}
		
		#endregion Connection
		
		[SuppressMessage("Reliability", "CA2000:Dispose objects before losing scope", Justification = "Ignore to simplify test")]
		private async Task<ComputeService> CreateComputeServiceWithAgentAsync(string? tunnelAddress, string pool, ComputeClusterConfig ccc, HttpClient? httpClient = null)
		{
			StubMessageHandler ubaCacheOk = new(HttpStatusCode.OK, "");
			IOptionsMonitor<ComputeConfig> computeConfig = ServiceProvider.GetRequiredService<IOptionsMonitor<ComputeConfig>>();
			ComputeServerConfig ss = new() { ComputeTunnelAddress = tunnelAddress };
			ComputeService cs = new(AgentCollection, ServiceProvider.GetRequiredService<IAgentScheduler>(), LogCollection, AgentService, AgentRelayService, GetRedisServiceSingleton(),
				httpClient ?? new HttpClient(ubaCacheOk), new TestOptionsMonitor<ComputeServerConfig>(ss), computeConfig, Clock, Tracer, Meter,
				NullLogger<ComputeService>.Instance);
			await CreateAgentAsync(new PoolId(pool), properties: ["ComputeIp=11.0.0.1", "ComputePort=5000"]);
			computeConfig.CurrentValue.Clusters = [ccc];
			computeConfig.CurrentValue.PostLoad(new PluginConfigOptions(ConfigVersion.Latest, Enumerable.Empty<IPluginConfig>(), new AclConfig()));
			AgentRelayService.SetRandomSeed(1);
			return cs;
		}

		private async Task<ComputeResource?> AllocateWithAgentAsync(
			ConnectionMode connectionMode = ConnectionMode.Direct,
			Dictionary<string, int>? ports = null,
			bool usePublicIp = false,
			string[]? relayIps = null,
			string? tunnelAddress = null,
			bool useUbaCache = false,
			UbaComputeClusterConfig? ubaConfig = null,
			ComputeProtocol protocol = ComputeProtocol.Latest,
			string pool = "myPool",
			HttpClient? httpClient = null,
			AclConfig? acl = null)
		{
			ComputeClusterConfig ccc = new() { Id = _cluster1, Uba = ubaConfig };
			if (acl != null)
			{
				ccc.Acl = acl;
			}
			await using ComputeService cs = await CreateComputeServiceWithAgentAsync(tunnelAddress, pool, ccc, httpClient);
			
			AllocateResourceParams arp = new(_cluster1, _user, protocol, new Requirements() { Pool = pool })
			{
				ConnectionMode = connectionMode,
				Ports = ports ?? new Dictionary<string, int>(),
				UsePublicIp = usePublicIp,
				UseUbaCache = useUbaCache
			};
			IPAddress[] defaultRelayIps = { IPAddress.Parse("192.168.1.1") };
			await AgentRelayService.UpdateAgentHeartbeatAsync(_cluster1, "myrelay", relayIps?.Select(IPAddress.Parse) ?? defaultRelayIps);
			return await cs.TryAllocateResourceAsync(arp, CancellationToken.None);
		}
		
		private async Task<ComputeResource?> AllocateAsync(
			string pool = "defaultTestPool",
			ConnectionMode connectionMode = ConnectionMode.Direct,
			Dictionary<string, int>? ports = null,
			bool usePublicIp = false,
			ComputeProtocol protocol = ComputeProtocol.Latest)
		{
			AllocateResourceParams arp = new(new ClusterId("default"), _user, protocol, new Requirements() { Pool = pool })
			{
				ConnectionMode = connectionMode,
				Ports = ports ?? new Dictionary<string, int>(),
				UsePublicIp = usePublicIp
			};
			return await ComputeService.TryAllocateResourceAsync(arp, CancellationToken.None);
		}
		
		private Task<IAgent> CreateComputeAgentAsync(
			string poolId = "defaultTestPool",
			string ip = "10.0.0.1",
			int port = 5000,
			ComputeProtocol protocol = ComputeProtocol.Latest)
		{
			List<string> props = [$"{KnownPropertyNames.ComputeIp}={ip}", $"{KnownPropertyNames.ComputePort}={port}", $"{KnownPropertyNames.ComputeProtocol}={(int)protocol}"];
			return CreateAgentAsync(new PoolId(poolId), properties: props);
		}

		private static void AssertContainsMeasurement(List<Measurement<int>> actualMeasurements, int expectedValue, string expectedResource, string expectedClusterId, string expectedPool)
		{
			Dictionary<string, string> expectedTags = new()
			{
				{ "resource", expectedResource },
				{ "cluster", expectedClusterId },
				{ "pool", expectedPool }
			};

			foreach (Measurement<int> measurement in actualMeasurements)
			{
				Dictionary<string, string> actualTags = measurement.Tags.ToArray().ToDictionary(
					kvp => kvp.Key,
					kvp => (string)kvp.Value!);

				bool areTagsEqual = actualTags.Count == expectedTags.Count &&
									actualTags.OrderBy(kvp => kvp.Key)
										.SequenceEqual(expectedTags.OrderBy(kvp => kvp.Key));

				if (areTagsEqual && expectedValue == measurement.Value)
				{
					return;
				}
			}

			Console.WriteLine("Actual:");
			foreach (Measurement<int> m in actualMeasurements)
			{
				Console.WriteLine($"Measurement(value={m.Value} tags={String.Join(',', m.Tags.ToArray())}");
			}
			Assert.Fail("Unable to find measurement");
		}
	}
}