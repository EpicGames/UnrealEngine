// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Dashboard;

namespace HordeServer.Tests.Dashboard
{
	/// <summary>
	/// Tests for the dashboard
	/// </summary>
	[TestClass]
	public class DashboardTest : BuildTestSetup
	{
		[TestMethod]
		public async Task TestPreviewsAsync()
		{
			CreateDashboardPreviewRequest request = new CreateDashboardPreviewRequest();
			request.Summary = "This is a test preview item";

			// create a reservation
			GetDashboardPreviewResponse result = (await DashboardController!.CreateDashbordPreviewAsync(request)).Value!;

			Assert.AreEqual(1, result.Id);
			Assert.AreEqual("This is a test preview item", result.Summary);

			request.Summary = "This is another test preview item";
			result = (await DashboardController!.CreateDashbordPreviewAsync(request)).Value!;

			Assert.AreEqual(2, result.Id);
			Assert.AreEqual("This is another test preview item", result.Summary);

			UpdateDashboardPreviewRequest updateRequest = new UpdateDashboardPreviewRequest();
			updateRequest.Id = 2;
			updateRequest.Summary = "Updated test summary";
			updateRequest.ExampleLink = "http://testexample";
			updateRequest.TrackingLink = "http://testtracking";
			updateRequest.DiscussionLink = "http://testdiscussion";
			updateRequest.DeployedCL = 123;

			result = (await DashboardController!.UpdateDashbordPreviewAsync(updateRequest)).Value!;

			Assert.AreEqual(2, result.Id);
			Assert.AreEqual("http://testexample", result.ExampleLink);
			Assert.AreEqual("http://testtracking", result.TrackingLink);
			Assert.AreEqual("http://testdiscussion", result.DiscussionLink);
			Assert.AreEqual(123, result.DeployedCL);

			List<GetDashboardPreviewResponse> results = (await DashboardController!.GetDashbordPreviewsAsync(true)).Value!;
			results = results.OrderBy(r => r.Id).ToList();
			Assert.AreEqual(2, results.Count);
			Assert.AreEqual(1, results[0].Id);
			Assert.AreEqual(2, results[1].Id);

			UpdateDashboardPreviewRequest updateRequest2 = new UpdateDashboardPreviewRequest();
			updateRequest2.Id = 1;
			updateRequest2.Open = false;
			result = (await DashboardController!.UpdateDashbordPreviewAsync(updateRequest2)).Value!;
			Assert.AreEqual(false, result.Open);

			results = (await DashboardController!.GetDashbordPreviewsAsync(true)).Value!;
			Assert.AreEqual(1, results.Count);
			Assert.AreEqual(2, results[0].Id);
			Assert.AreEqual("Updated test summary", results[0].Summary);

			results = (await DashboardController!.GetDashbordPreviewsAsync(false)).Value!;
			Assert.AreEqual(1, results.Count);
			Assert.AreEqual(1, results[0].Id);
		}
	}
}
