// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Devices;
using HordeServer.Jobs;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using MongoDB.Bson;
using Moq;

namespace HordeServer.Tests.Devices
{
	/// <summary>
	/// Tests for the device service
	/// </summary>
	[TestClass]
	public class DeviceServiceTest : BuildTestSetup
	{
		private DevicesController? _deviceController;

		IGraph? _graph = null;

		// override DeviceController with valid user
		private DevicesController DeviceController
		{
			get
			{
				if (_deviceController == null)
				{
					IUser user = UserCollection.FindOrAddUserByLoginAsync("TestUser").Result;
					_deviceController = base.DevicesController;
					ControllerContext controllerContext = new ControllerContext();
					controllerContext.HttpContext = new DefaultHttpContext();
					controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
						new List<Claim>
						{
							HordeClaims.AdminClaim.ToClaim(),
							new Claim(ClaimTypes.Name, "TestUser"),
							new Claim(HordeClaimTypes.UserId, user.Id.ToString())
						}
						, "TestAuthType"));
					_deviceController.ControllerContext = controllerContext;

				}
				return _deviceController;
			}
		}

		static T ResultToValue<T>(ActionResult<T> result) where T : class
		{
			return ((result.Result! as JsonResult)!.Value! as T)!;
		}

		static NewGroup AddGroup(List<NewGroup> groups)
		{
			NewGroup group = new NewGroup("win64", new List<NewNode>());
			groups.Add(group);
			return group;
		}

		static NewNode AddNode(NewGroup group, string name, string[]? inputDependencies, Action<NewNode>? action = null, IReadOnlyNodeAnnotations? annotations = null)
		{
			NewNode node = new NewNode(name, inputDependencies: inputDependencies?.ToList(), orderDependencies: inputDependencies?.ToList(), annotations: annotations);
			action?.Invoke(node);
			group.Nodes.Add(node);
			return node;
		}

		static async Task<IJob> StartBatchAsync(IJob job, int batchIdx)
		{
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[batchIdx].State);
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[batchIdx].Id, null, JobStepBatchState.Running, null));
			Assert.AreEqual(JobStepBatchState.Running, job.Batches[batchIdx].State);
			return job;
		}

		static async Task<IJob> StartStepAsync(IJob job, int batchIdx, int stepIdx)
		{
			Assert.AreEqual(JobStepState.Ready, job.Batches[batchIdx].Steps[stepIdx].State);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, JobStepState.Running, JobStepOutcome.Success));
			return job;
		}

		static async Task<IJob> FinishStepAsync(IJob job, int batchIdx, int stepIdx, JobStepOutcome outcome)
		{
			job = Deref(await job.TryUpdateStepAsync(job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, JobStepState.Completed, outcome));
			Assert.AreEqual(JobStepState.Completed, job.Batches[batchIdx].Steps[stepIdx].State);
			Assert.AreEqual(outcome, job.Batches[batchIdx].Steps[stepIdx].Outcome);
			return job;
		}

		static async Task<IJob> RunStepAsync(IJob job, int batchIdx, int stepIdx, JobStepOutcome outcome)
		{
			job = Deref(await StartStepAsync(job, batchIdx, stepIdx));
			return Deref(await FinishStepAsync(job, batchIdx, stepIdx, outcome));
		}

		JobStepId GetStepId(IJob job, string nodeName)
		{
			NodeRef? installNode;
			_graph!.TryFindNode(nodeName, out installNode);
			Assert.IsNotNull(installNode);

			IJobStep? step;
			job.TryGetStepForNode(installNode, out step);
			Assert.IsNotNull(step);

			return step.Id;
		}

		async Task<bool> SetupDevicesAsync()
		{

			await CreateFixtureAsync();

			DeviceConfig devices = new DeviceConfig();

			// create 2 pools
			for (int i = 1; i < 3; i++)
			{
				devices.Pools.Add(new DevicePoolConfig() { Id = new DevicePoolId("TestDevicePool" + i), Name = "TestDevicePool" + i, PoolType = DevicePoolType.Automation, ProjectIds = new List<ProjectId>() { new ProjectId("ue5") } });
			}

			// create 3 platforms
			for (int i = 1; i < 4; i++)
			{
				string platformName = "TestDevicePlatform" + i;

				List<string> modelIds = new List<string>();
				for (int j = 2; j < 5; j++)
				{
					modelIds.Add(platformName + "_Model" + j);
				}

				DevicePlatformConfig platform = new DevicePlatformConfig() { Id = new DevicePlatformId(platformName), Name = platformName, Models = modelIds };

				// platform 3 has some name aliases
				if (i == 3)
				{
					platform.LegacyNames = new List<string>() { "TestDevicePlatform3Alias" };
					platform.LegacyPerfSpecHighModel = "TestDevicePlatform3_Model4";
				}

				devices.Platforms.Add(platform);
			}

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Devices = devices);

			for (int i = 1; i < 4; i++)
			{
				for (int j = 0; j < 4; j++)
				{
					for (int k = 1; k < 5; k++)
					{
						// one base model, and 3 other models 
						string? modelId = null;
						if (k > 1)
						{
							modelId = "TestDevicePlatform" + i + "_Model" + k;
						}

						string poolId = (k & 1) != 0 ? "testdevicepool1" : "testdevicepool2";

						await DeviceController.CreateDeviceAsync(new CreateDeviceRequest() { Name = "TestDevice" + (j * 5 + k) + "_Platform" + i + "_" + poolId, Address = "10.0.0.1", Enabled = true, PlatformId = "testdeviceplatform" + i, ModelId = modelId, PoolId = poolId });
					}
				}
			}

			await DeviceService.TickForTestingAsync();

			return true;
		}

		async Task<IJob> SetupJobAsync(string? annotationsIn = null)
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Run Tests");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue5-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await StartBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Update Version Files", null);
			AddNode(initialGroup, "Compile Editor", new[] { "Update Version Files" });

			NewGroup compileGroup = AddGroup(newGroups);
			AddNode(compileGroup, "Compile Client", new[] { "Update Version Files" });

			NewGroup cookGroup = AddGroup(newGroups);
			AddNode(cookGroup, "Cook Client", new[] { "Compile Editor" });

			NewGroup testGroup = AddGroup(newGroups);
			NodeAnnotations? annotations = String.IsNullOrEmpty(annotationsIn) ? null : new NodeAnnotations();
			if (annotationsIn == "DeviceReserveNodes")
			{
				annotations!.Add("DeviceReserveNodes", "Run Test 1,Run Test 2,Run Test 3,Run Test 4");
			}
			else if (annotationsIn == "DeviceReserve")
			{
				annotations!.Add("DeviceReserve", "Begin");
			}

			AddNode(testGroup, "Install Build", new[] { "Cook Client", "Compile Client" }, annotations: annotations);
			AddNode(testGroup, "Run Test 1", new[] { "Install Build" });
			AddNode(testGroup, "Run Test 2", new[] { "Install Build" });

			annotations = annotationsIn == "DeviceReserve" ? new NodeAnnotations() : null;
			if (annotations != null)
			{
				annotations!.Add("DeviceReserve", "End");
			}

			AddNode(testGroup, "Run Test 3", new[] { "Install Build" }, annotations: annotations);
			AddNode(testGroup, "Run Test 4", new[] { "Install Build" });

			AddNode(testGroup, "Run Tests", new[] { "Run Test 1", "Run Test 2", "Run Test 3", "Run Test 4" });

			_graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(_graph));

			return job;
		}

		static LegacyCreateReservationRequest SetupReservationTestAsync(IJob job, string poolId = "TestDevicePool1", string deviceType = "TestDevicePlatform1", JobStepId? stepId = null, string? modelId = null, string? deviceName = null, DeviceTagsRequest? tags = null)
		{
			if (stepId == null)
			{
				stepId = JobStepId.Parse("abcd");
			}

			if (modelId != null)
			{
				deviceType += $":{modelId}";
			}

			// Gauntlet uses the legacy v1 API
			LegacyCreateReservationRequest request = new LegacyCreateReservationRequest();
			request.PoolId = poolId;
			request.DeviceTypes = new string[] { deviceType };
			request.Hostname = "localhost";
			request.Duration = "00:10:00";
			request.JobId = job.Id.ToString();
			request.StepId = stepId.ToString();
			request.DeviceName = deviceName;
			request.Tags = tags;

			return request;
		}

		[TestMethod]
		public async Task TestReservationAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, modelId: "Base");

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);
			Assert.AreEqual(1, reservation.DeviceModels.Length);
			Assert.AreEqual("Base", reservation.DeviceModels[0]);
			Assert.AreEqual("Test job", reservation.JobName);
			Assert.AreEqual("abcd", reservation.StepId);

			// get the device in the reservation, and make sure it is the right platform
			GetLegacyDeviceResponse device = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));
			Assert.AreEqual("TestDevicePlatform1", device.Type);

			// update the reservation
			GetLegacyReservationResponse renewed = ResultToValue(await DeviceController!.UpdateReservationV1Async(reservation.Guid));
			Assert.AreEqual(renewed.Guid, reservation.Guid);

			// delete the reservation, would be nice to also test reservation expiration
			OkResult? deleted = (await DeviceController!.DeleteReservationV1Async(reservation.Guid)) as OkResult;
			Assert.IsNotNull(deleted);

			// check that telemetry was created
			List<GetDeviceTelemetryResponse> telemetry = (await DeviceController!.GetDeviceTelemetryAsync()).Value!;
			Assert.AreEqual(1, telemetry.Count);
			Assert.AreEqual(1, telemetry[0].Telemetry.Count);
			Assert.AreEqual("ue5-main", telemetry[0].Telemetry[0].StreamId);
			Assert.AreEqual("abcd", telemetry[0].Telemetry[0].StepId);
			Assert.AreEqual("Test job", telemetry[0].Telemetry[0].JobName);
		}

		[TestMethod]
		public async Task TestReservationForceDeletionAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, modelId: "Base");

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));

			// verify the reservation exists
			IDeviceReservation? serviceReservation = await DeviceService.TryGetReservationFromLegacyGuidAsync(reservation.Guid);
			Assert.IsNotNull(serviceReservation);

			// delete the reservation
			OkResult? deleted = (await DeviceController!.DeleteReservationV1Async(reservation.Guid, true)) as OkResult;
			Assert.IsNotNull(deleted);

			// Test again to ensure it's now deleted
			serviceReservation = await DeviceService.TryGetReservationFromLegacyGuidAsync(reservation.Guid);
			Assert.IsNull(serviceReservation);
		}

		[TestMethod]
		public async Task TestReservationNodesAsync()
		{
			await SetupDevicesAsync();
			IJob job = await SetupJobAsync("DeviceReserveNodes");

			job = await StartBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Update Version Files
			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Compile Editor

			job = await StartBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // Compile Client

			job = await StartBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // Cook Client

			job = await StartBatchAsync(job, 4);

			// Install  the build
			JobStepId stepId = GetStepId(job, "Install Build");
			job = await StartStepAsync(job, 4, 0); // Install Build														  
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, stepId: stepId);
			GetLegacyReservationResponse installReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			job = await FinishStepAsync(job, 4, 0, JobStepOutcome.Success); // Install Build

			for (int i = 1; i < 5; i++)
			{
				// Run Test 1
				stepId = GetStepId(job, $"Run Test {i}");
				job = await StartStepAsync(job, 4, i);

				request = SetupReservationTestAsync(job, stepId: stepId);

				ActionResult<GetLegacyReservationResponse> result = await DeviceController!.CreateDeviceReservationV1Async(request);

				GetLegacyReservationResponse? reservation;

				if (i == 4)
				{
					// check parallel error conflict
					Assert.AreEqual(409, (result.Result as ConflictObjectResult)!.StatusCode);
					Assert.AreEqual("Reserved nodes must not run in parallel: Run Test 3,Run Test 4", (result.Result as ConflictObjectResult)!.Value);

					// finish the step
					job = await FinishStepAsync(job, 4, 3, JobStepOutcome.Success);

					result = await DeviceController!.CreateDeviceReservationV1Async(request);
					reservation = ResultToValue(result);
				}
				else
				{
					reservation = ResultToValue(result);
				}

				Assert.IsNotNull(reservation);
				Assert.AreEqual(installReservation.Guid, reservation.Guid);

				// Do not finish test 3, to test parallel step error
				if (i != 3)
				{
					await DeviceController!.DeleteReservationV1Async(reservation.Guid);
					job = await FinishStepAsync(job, 4, i, JobStepOutcome.Success);
				}

				await DeviceService.TickForTestingAsync();
			}

			List<IDeviceReservation> reservations = await DeviceService.GetReservationsAsync();
			Assert.AreEqual(0, reservations.Count);

			// check that telemetry was created
			List<GetDeviceTelemetryResponse> telemetry = (await DeviceController!.GetDeviceTelemetryAsync()).Value!;
			Assert.AreEqual(1, telemetry.Count);
			Assert.AreEqual(1, telemetry[0].Telemetry.Count);
			Assert.AreEqual("ue5-main", telemetry[0].Telemetry[0].StreamId);
			Assert.AreEqual(telemetry[0].Telemetry[0].StepId, GetStepId(job, "Install Build").ToString());
			Assert.AreEqual("Install Build", telemetry[0].Telemetry[0].StepName);
			Assert.AreEqual("Test job", telemetry[0].Telemetry[0].JobName);
		}

		[TestMethod]
		public async Task TestReservationNodeDeletionAsync()
		{
			await SetupDevicesAsync();
			IJob job = await SetupJobAsync("DeviceReserveNodes");

			job = await StartBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Update Version Files
			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Compile Editor

			job = await StartBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // Compile Client

			job = await StartBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // Cook Client

			job = await StartBatchAsync(job, 4);

			// Install  the build
			JobStepId stepId = GetStepId(job, "Install Build");
			job = await StartStepAsync(job, 4, 0); // Install Build
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, stepId: stepId);
			GetLegacyReservationResponse installReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			job = await FinishStepAsync(job, 4, 0, JobStepOutcome.Success); // Install Build

			for (int i = 1; i < 3; i++)
			{
				// Run Test 1
				stepId = GetStepId(job, $"Run Test {i}");
				job = await StartStepAsync(job, 4, i);

				request = SetupReservationTestAsync(job, stepId: stepId);

				ActionResult<GetLegacyReservationResponse> result = await DeviceController!.CreateDeviceReservationV1Async(request);

				GetLegacyReservationResponse? reservation = ResultToValue(result);
				Assert.IsNotNull(reservation);
				Assert.AreEqual(installReservation.Guid, reservation.Guid);

				if (i == 1)
				{
					// For the first step, simply delete the reservation at the end of a test like normal
					await DeviceController!.DeleteReservationV1Async(reservation.Guid);
					job = await FinishStepAsync(job, 4, i, JobStepOutcome.Success);
					await DeviceService.TickForTestingAsync();

					// We should find that the reservation still exists because it's part of a reservation block
					IDeviceReservation? serviceReservation = await DeviceService.TryGetReservationFromLegacyGuidAsync(reservation.Guid);
					Assert.IsNotNull(serviceReservation);
				}
				else if (i == 2)
				{
					// For the second step, force delete the reservation at the end of a test to verify the block is cleared despite there being remaining steps
					await DeviceController!.DeleteReservationV1Async(reservation.Guid, true);
					job = await FinishStepAsync(job, 4, i, JobStepOutcome.Success);
					await DeviceService.TickForTestingAsync();

					// We should find that the reservation no longer exists because it was force deleted
					IDeviceReservation? serviceReservation = await DeviceService.TryGetReservationFromLegacyGuidAsync(reservation.Guid);
					Assert.IsNull(serviceReservation);
				}
			}
		}

		[TestMethod]
		public async Task TestReservationMarkersAsync()
		{
			await SetupDevicesAsync();
			IJob job = await SetupJobAsync("DeviceReserve");

			job = await StartBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Update Version Files
			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Compile Editor

			job = await StartBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // Compile Client

			job = await StartBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // Cook Client

			job = await StartBatchAsync(job, 4);

			// Install  the build
			JobStepId stepId = GetStepId(job, "Install Build");
			job = await StartStepAsync(job, 4, 0); // Install Build														  
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, stepId: stepId, modelId: "Base");
			GetLegacyReservationResponse installReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.IsTrue(installReservation.InstallRequired);

			string installProblemDeviceName = installReservation!.DeviceNames[0];

			await DeviceController!.PutDeviceErrorAsync(installProblemDeviceName);
			installReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.IsTrue(installReservation.InstallRequired);
			Assert.AreNotEqual(installProblemDeviceName, installReservation.DeviceNames[0]);

			job = await FinishStepAsync(job, 4, 0, JobStepOutcome.Success); // Install Build

			for (int i = 1; i < 5; i++)
			{
				// Run Test 1
				stepId = GetStepId(job, $"Run Test {i}");
				job = await StartStepAsync(job, 4, i);

				if (i == 2)
				{
					request = SetupReservationTestAsync(job, stepId: stepId, modelId: "TestDevicePlatform1_Model3");
				}
				else
				{
					request = SetupReservationTestAsync(job, stepId: stepId, modelId: "Base");
				}

				ActionResult<GetLegacyReservationResponse> result = await DeviceController!.CreateDeviceReservationV1Async(request);

				GetLegacyReservationResponse? reservation = ResultToValue(result);

				Assert.IsNotNull(reservation);

				if (i == 1)
				{
					Assert.IsFalse(reservation.InstallRequired);
				}

				if (i == 2)
				{
					Assert.AreNotEqual(reservation.DeviceNames[0], installReservation.DeviceNames[0]);
					Assert.AreEqual("TestDevicePlatform1_Model3", reservation.DeviceModels[0]);
					Assert.IsTrue(reservation.InstallRequired);
				}

				if (i == 3)
				{
					string problemDeviceName = reservation.DeviceNames[0];
					await DeviceController!.PutDeviceErrorAsync(problemDeviceName);
					result = await DeviceController!.CreateDeviceReservationV1Async(request);
					reservation = ResultToValue(result);
					Assert.IsNotNull(reservation);
					Assert.IsTrue(reservation.InstallRequired);
					Assert.AreNotEqual(problemDeviceName, reservation.DeviceNames[0]);
					Assert.AreEqual("Base", reservation.DeviceModels[0]);
				}

				if (i != 4)
				{
					Assert.AreEqual(installReservation.Guid, reservation.Guid);
				}
				else
				{
					Assert.IsNull(reservation.InstallRequired);
					Assert.AreNotEqual(installReservation.Guid, reservation.Guid);
				}

				await DeviceController!.DeleteReservationV1Async(reservation.Guid);
				job = await FinishStepAsync(job, 4, i, JobStepOutcome.Success);

				await DeviceService.TickForTestingAsync();
			}

			List<IDeviceReservation> reservations = await DeviceService.GetReservationsAsync();
			Assert.AreEqual(0, reservations.Count);

			// check that telemetry was created
			List<GetDeviceTelemetryResponse> telemetry = (await DeviceController!.GetDeviceTelemetryAsync()).Value!;
			Assert.AreEqual(2, telemetry.Count, 5);
			Assert.AreEqual(1, telemetry[0].Telemetry.Count);
			Assert.AreEqual("ue5-main", telemetry[0].Telemetry[0].StreamId);
			Assert.AreEqual(telemetry[0].Telemetry[0].StepId, GetStepId(job, "Install Build").ToString());
			Assert.AreEqual("Install Build", telemetry[0].Telemetry[0].StepName);
			Assert.AreEqual("Test job", telemetry[0].Telemetry[0].JobName);
		}

		[TestMethod]
		public async Task TestReservationPerfSpecWithAliasAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, "TestDevicePool2", "TestDevicePlatform3Alias:High");

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);
			Assert.AreEqual("Test job", reservation.JobName);
			Assert.AreEqual("abcd", reservation.StepId);

			// get the device in the reservation, and make sure it is the right platform
			GetLegacyDeviceResponse device = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));
			Assert.AreEqual("TestDevicePlatform3Alias", device.Type);
			Assert.AreEqual("TestDevicePlatform3_Model4", device.Model);

			// update the reservation
			GetLegacyReservationResponse renewed = ResultToValue(await DeviceController!.UpdateReservationV1Async(reservation.Guid));
			Assert.AreEqual(renewed.Guid, reservation.Guid);

			// delete the reservation, would be nice to also test reservation expiration
			OkResult? deleted = (await DeviceController!.DeleteReservationV1Async(reservation.Guid)) as OkResult;
			Assert.IsNotNull(deleted);

			// check that telemetry was created
			List<GetDeviceTelemetryResponse> telemetry = (await DeviceController!.GetDeviceTelemetryAsync()).Value!;
			Assert.AreEqual(1, telemetry.Count);
			Assert.AreEqual(1, telemetry[0].Telemetry.Count);
			Assert.AreEqual("ue5-main", telemetry[0].Telemetry[0].StreamId);
			Assert.AreEqual("abcd", telemetry[0].Telemetry[0].StepId);
			Assert.AreEqual("Test job", telemetry[0].Telemetry[0].JobName);
		}

		[TestMethod]
		public async Task TestReservationPerfModelAsync()
		{
			await SetupDevicesAsync();

			List<string> deviceModels = new List<string>() { "TestDevicePlatform1:TestDevicePlatform1_Model2", "TestDevicePlatform1_Model3" };

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, "TestDevicePool1", String.Join(';', deviceModels));

			// update device model
			UpdateDeviceRequest updateRequest = new UpdateDeviceRequest() { ModelId = "TestDevicePlatform1_Model2" };
			await DeviceController!.UpdateDeviceAsync("testdevice1_platform1_testdevicepool1", updateRequest);

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);
			Assert.AreEqual("Test job", reservation.JobName);
			Assert.AreEqual("abcd", reservation.StepId);

			// get the device in the reservation, and make sure it is the right platform and an acceptable model
			GetLegacyDeviceResponse device = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));
			Assert.AreEqual("TestDevicePlatform1", device.Type);

			string? firstModel = device.Model;
			Assert.IsTrue(firstModel == "TestDevicePlatform1_Model2" || firstModel == "TestDevicePlatform1_Model3");

			// get a 2nd reservation
			reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);

			device = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));
			Assert.AreEqual("TestDevicePlatform1", device.Type);

			string? secondModel = device.Model;
			Assert.IsTrue(secondModel != firstModel && (secondModel == "TestDevicePlatform1_Model2" || secondModel == "TestDevicePlatform1_Model3"));
		}

		[TestMethod]
		public async Task TestReservationDeviceNameAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job, "TestDevicePool1", "TestDevicePlatform1", null, null, "testdevice1_platform1_testdevicepool1");

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);
			Assert.AreEqual("testdevice1_platform1_testdevicepool1", reservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);
			Assert.AreEqual("Test job", reservation.JobName);
			Assert.AreEqual("abcd", reservation.StepId);

			// get the device in the reservation, and make sure it is the right device
			GetLegacyDeviceResponse device = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));
			Assert.AreEqual("testdevice1_platform1_testdevicepool1", device.Name, StringComparer.OrdinalIgnoreCase);
		}

		[TestMethod]
		public async Task TestReservationDeviceTagExpectedErrorsAsync()
		{
			await CreateFixtureAsync();

			DeviceConfig devices = new DeviceConfig();

			// Create a pool
			string poolName = "TestDevicePool";
			DevicePoolConfig devicePool = new DevicePoolConfig() { Id = new DevicePoolId(poolName), Name = poolName, PoolType = DevicePoolType.Automation, ProjectIds = new List<ProjectId>() { new ProjectId("ue5") } };
			devices.Pools.Add(devicePool);

			// Create a single platform
			string platformName = "TestDevicePlatform";
			DevicePlatformConfig platform = new DevicePlatformConfig() { Id = new DevicePlatformId(platformName), Name = platformName };
			devices.Platforms.Add(platform);

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Devices = devices);

			// Create a device with a single blocked tag
			CreateDeviceRequest testDevice = new CreateDeviceRequest()
			{
				Name = "TestDevice",
				PoolId = poolName,
				Address = "10.0.0.1",
				PlatformId = platformName,
				Tags = new List<string> { "BlockedTag" }
			};

			await DeviceController.CreateDeviceAsync(testDevice);
			await DeviceService.TickForTestingAsync();

			IJob job = await SetupJobAsync();

			// Now test the three expected failure conditions

			// Bad request - should error because of duplicates in tag request
			DeviceTagsRequest duplicateTags = new DeviceTagsRequest() { RequiredTags = ["DuplicateEntry"], BlockedTags = ["DuplicateEntry"] };
			LegacyCreateReservationRequest duplicateRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, duplicateTags);

			// Missing required - should error because no devices exist with the required tag
			DeviceTagsRequest requiredTag = new DeviceTagsRequest() { RequiredTags = ["RequiredTag"] };
			LegacyCreateReservationRequest requiredRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, requiredTag);

			// Only blocked - should error because no devices exist without the blocked tag
			DeviceTagsRequest blockedTag = new DeviceTagsRequest() { BlockedTags = ["BlockedTag"] };
			LegacyCreateReservationRequest blockedRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, blockedTag);

			ActionResult? duplicateResponse = (await DeviceController!.CreateDeviceReservationV1Async(duplicateRequest)).Result;
			ActionResult? requiredResponse = (await DeviceController!.CreateDeviceReservationV1Async(requiredRequest)).Result;
			ActionResult? blockedResponse = (await DeviceController!.CreateDeviceReservationV1Async(blockedRequest)).Result;

			Assert.IsInstanceOfType(duplicateResponse, typeof(BadRequestObjectResult));
			Assert.IsInstanceOfType(requiredResponse, typeof(ConflictObjectResult));
			Assert.IsInstanceOfType(blockedResponse, typeof(ConflictObjectResult));
		}

		[TestMethod]
		public async Task TestReservationDeviceTagRequirementsAsync()
		{
			await CreateFixtureAsync();

			DeviceConfig devices = new DeviceConfig();

			// Create a pool
			string poolName = "TestDevicePool";
			DevicePoolConfig devicePool = new DevicePoolConfig() { Id = new DevicePoolId(poolName), Name = poolName, PoolType = DevicePoolType.Automation, ProjectIds = new List<ProjectId>() { new ProjectId("ue5") } };
			devices.Pools.Add(devicePool);

			// Create a single platform
			string platformName = "TestDevicePlatform";
			DevicePlatformConfig platform = new DevicePlatformConfig() { Id = new DevicePlatformId(platformName), Name = platformName };
			devices.Platforms.Add(platform);

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Devices = devices);

			// Create 10 devices
			string expectedDeviceName = "ExpectedDevice";
			for (int i = 0; i < 10; ++i)
			{
				CreateDeviceRequest testDevice;
				if (i == 5)
				{
					// 6th device has a required tag
					testDevice = new CreateDeviceRequest()
					{
						Name = expectedDeviceName,
						PoolId = poolName,
						Address = "10.0.0.1",
						PlatformId = platformName,
						Tags = new List<string> { "RequiredTag" }
					};
				}
				else
				{
					// rest have a blocked tag
					testDevice = new CreateDeviceRequest()
					{
						Name = "TestDevice_" + (i + 1),
						PoolId = poolName,
						Address = "10.0.0.1",
						PlatformId = platformName,
						Tags = new List<string> { "BlockedTag" }
					};
				}

				await DeviceController.CreateDeviceAsync(testDevice);
			}
			
			await DeviceService.TickForTestingAsync();

			IJob job = await SetupJobAsync();

			// Request a device with a required tag
			DeviceTagsRequest requiredTag = new DeviceTagsRequest() { RequiredTags = ["RequiredTag"] };
			LegacyCreateReservationRequest requiredRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, requiredTag);

			// Request a device ignoring blocked tags
			DeviceTagsRequest blockedTag = new DeviceTagsRequest() { BlockedTags = ["BlockedTag"] };
			LegacyCreateReservationRequest blockedRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, blockedTag);

			// Ensure we get the required device
			GetLegacyReservationResponse requiredReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(requiredRequest));
			Assert.AreEqual(1, requiredReservation.DeviceNames.Length);
			Assert.AreEqual(expectedDeviceName, requiredReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);

			// Delete the reservation
			await DeviceController.DeleteReservationV1Async(requiredReservation.Guid);
			await DeviceService.TickForTestingAsync();

			// Attempt to reserve a device that is not blocked, which should give us the same device
			GetLegacyReservationResponse blockedReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(blockedRequest));
			Assert.AreEqual(1, blockedReservation.DeviceNames.Length);
			Assert.AreEqual(expectedDeviceName, blockedReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);
		}

		[TestMethod]
		public async Task TestReservationDeviceTagPreferencesAsync()
		{
			await CreateFixtureAsync();

			DeviceConfig devices = new DeviceConfig();

			// Create a pool
			string poolName = "TestDevicePool";
			DevicePoolConfig devicePool = new DevicePoolConfig() { Id = new DevicePoolId(poolName), Name = poolName, PoolType = DevicePoolType.Automation, ProjectIds = new List<ProjectId>() { new ProjectId("ue5") } };
			devices.Pools.Add(devicePool);

			// Create a single platform
			string platformName = "TestDevicePlatform";
			DevicePlatformConfig platform = new DevicePlatformConfig() { Id = new DevicePlatformId(platformName), Name = platformName };
			devices.Platforms.Add(platform);

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Devices = devices);

			// Create 10 devices
			string expectedDeviceName = "ExpectedDevice";
			for (int i = 0; i < 10; ++i)
			{
				CreateDeviceRequest testDevice;
				if (i == 5)
				{
					// 6th device has a preferred tag
					testDevice = new CreateDeviceRequest()
					{
						Name = expectedDeviceName,
						PoolId = poolName,
						Address = "10.0.0.1",
						PlatformId = platformName,
						Tags = new List<string> { "PreferredTag" }
					};
				}
				else
				{
					// rest have an undesired tag
					testDevice = new CreateDeviceRequest()
					{
						Name = "TestDevice_" + (i + 1),
						PoolId = poolName,
						Address = "10.0.0.1",
						PlatformId = platformName,
						Tags = new List<string> { "UndesiredTag" }
					};
				}

				await DeviceController.CreateDeviceAsync(testDevice);
			}

			await DeviceService.TickForTestingAsync();

			IJob job = await SetupJobAsync();

			// Request a device with a preferred tag
			DeviceTagsRequest preferredTag = new DeviceTagsRequest() { PreferredTags = ["PreferredTag"] };
			LegacyCreateReservationRequest preferredRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, preferredTag);

			// Request a device ignoring blocked tags
			DeviceTagsRequest undesiredTag = new DeviceTagsRequest() { UndesiredTags = ["UndesiredTag"] };
			LegacyCreateReservationRequest undesiredRequest = SetupReservationTestAsync(job, poolName, platformName, null, null, null, undesiredTag);

			// Ensure we get the preferred device
			GetLegacyReservationResponse preferredReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(preferredRequest));
			Assert.AreEqual(1, preferredReservation.DeviceNames.Length);
			Assert.AreEqual(expectedDeviceName, preferredReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);

			// Now, while the preferred device is reserved, attempt to reserve it again. 
			// Preferences aren't hard requirements, so we should get a device, just not one that has the preferred tag
			GetLegacyReservationResponse anotherPreferredReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(preferredRequest));
			Assert.AreEqual(1, anotherPreferredReservation.DeviceNames.Length);
			Assert.AreNotEqual(expectedDeviceName, anotherPreferredReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);

			// Delete the reservations
			await DeviceController.DeleteReservationV1Async(preferredReservation.Guid);
			await DeviceController.DeleteReservationV1Async(anotherPreferredReservation.Guid);
			await DeviceService.TickForTestingAsync();

			// Attempt to reserve a device that is not undesired, which should give us the same device
			GetLegacyReservationResponse undesiredReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(undesiredRequest));
			Assert.AreEqual(1, undesiredReservation.DeviceNames.Length);
			Assert.AreEqual(expectedDeviceName, undesiredReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);

			// Now, while the non-undesired device is reserved, attempt to reserve it again. 
			// Preferences aren't hard requirements, so we should get a device, just not one that has is missing the undesired tag
			GetLegacyReservationResponse anotherUndesiredReservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(undesiredRequest));
			Assert.AreEqual(1, anotherUndesiredReservation.DeviceNames.Length);
			Assert.AreNotEqual(expectedDeviceName, anotherUndesiredReservation.DeviceNames[0], StringComparer.OrdinalIgnoreCase);
		}

		[TestMethod]
		public async Task TestProblemDeviceAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest request = SetupReservationTestAsync(job);

			// create a reservation
			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(request));
			Assert.AreEqual(1, reservation.DeviceNames.Length);

			// report a problem with the device 
			OkResult? problem = (await DeviceController!.PutDeviceErrorAsync(reservation.DeviceNames[0])) as OkResult;
			Assert.IsNotNull(problem);

			OkResult? deleted = (await DeviceController!.DeleteReservationV1Async(reservation.Guid)) as OkResult;
			Assert.IsNotNull(deleted);

			// check that telemetry was created
			List<GetDeviceTelemetryResponse> telemetry = (await DeviceController!.GetDeviceTelemetryAsync()).Value!;
			Assert.AreEqual(1, telemetry.Count);
			// check that problem was recorded
			Assert.IsNotNull(telemetry[0].Telemetry[0].ProblemTimeUtc);
			// make sure the reservation time was finished for the problem device
			Assert.IsNotNull(telemetry[0].Telemetry[0].ReservationFinishUtc);

			// report a problem with message
			DeviceErrorRequest requestWithMessage = new();
			requestWithMessage.Message = "Unit Test message";
			problem = (await DeviceController!.PutDeviceErrorAsync(reservation.DeviceNames[0], requestWithMessage)) as OkResult;
			Assert.IsNotNull(problem);
		}

		[TestMethod]
		public async Task TestDevicePoolTelemetryCaptureAsync()
		{
			await SetupDevicesAsync();

			IJob job = await SetupJobAsync();
			LegacyCreateReservationRequest reservationRequest = SetupReservationTestAsync(job);

			// set some device status
			UpdateDeviceRequest request = new UpdateDeviceRequest();

			request.Maintenance = true;
			await DeviceController!.UpdateDeviceAsync("testdevice1_platform1_testdevicepool1", request);

			request.Maintenance = null;
			request.Enabled = false;
			await DeviceController!.UpdateDeviceAsync("testdevice3_platform3_testdevicepool1", request);

			request.Enabled = null;
			request.Problem = true;
			await DeviceController!.UpdateDeviceAsync("testdevice4_platform2_testdevicepool2", request);

			IDevice? maintenanceDevice = await DeviceService.GetDeviceAsync(new DeviceId("testdevice1_platform1_testdevicepool1"));
			Assert.IsNotNull(maintenanceDevice);

			GetLegacyReservationResponse reservation = ResultToValue(await DeviceController!.CreateDeviceReservationV1Async(reservationRequest));
			Assert.AreEqual(1, reservation.DeviceNames.Length);
			Assert.AreNotSame(maintenanceDevice!.Name, reservation.DeviceNames[0]);

			GetLegacyDeviceResponse reservedDevice = ResultToValue(await DeviceController!.GetDeviceV1Async(reservation.DeviceNames[0]));

			await DeviceService.TickForTestingAsync();

			List<GetDevicePoolTelemetryResponse> telemetry = (await DeviceController!.GetDevicePoolTelemetryAsync()).Value!;

			// 2, as generate an initial telemetry tick in setup
			Assert.AreEqual(2, telemetry.Count);
			foreach (GetDevicePlatformTelemetryResponse t in telemetry[1].Telemetry["testdevicepool1"])
			{
				if (t.PlatformId == "testdeviceplatform1")
				{
					Assert.AreEqual(6, t.Available?.Count);
					Assert.AreEqual(1, t.Reserved?.Count);
					Assert.AreEqual(1, t.Maintenance?.Count);

					Assert.AreEqual(true, t.Reserved?.ContainsKey("ue5-main"));
					Assert.AreEqual(1, t.Reserved?["ue5-main"].Count);
					Assert.AreEqual(t.Reserved?["ue5-main"][0].DeviceId, reservedDevice.Id);
					Assert.AreEqual("Test job", t.Reserved?["ue5-main"][0].JobName);
					Assert.AreEqual("abcd", t.Reserved?["ue5-main"][0].StepId);
				}
				else if (t.PlatformId == "testdeviceplatform3")
				{
					Assert.AreEqual(7, t.Available?.Count);
					Assert.AreEqual(1, t.Disabled?.Count);
				}
				else
				{
					Assert.AreEqual(8, t.Available?.Count);
				}
			}

			foreach (GetDevicePlatformTelemetryResponse t in telemetry[1].Telemetry["testdevicepool2"])
			{
				if (t.PlatformId == "testdeviceplatform2")
				{
					Assert.AreEqual(7, t.Available?.Count);
					Assert.AreEqual(1, t.Problem?.Count);
				}
				else
				{
					Assert.AreEqual(8, t.Available?.Count);
				}
			}
		}
		class TestDeviceTelemetry : IDeviceTelemetry
		{
			public DateTime CreateTimeUtc { get; init; }
			public DeviceId DeviceId { get; init; }
			public string? StreamId { get; init; }
			public string? JobId { get; init; }
			public string? JobName { get; init; }
			public string? StepId { get; init; }
			public string? StepName { get; init; }
			public ObjectId? ReservationId { get; init; }
			public DateTime? ReservationStartUtc { get; init; }
			public DateTime? ReservationFinishUtc { get; init; }
			public DateTime? ProblemTimeUtc { get; init; }
		}

		static void AddLoadSpikes(List<IDeviceTelemetry> telemetry, List<IDevice> devices, DevicePoolId pool, DevicePlatformId platform, ref DateTime referenceTime, int nSpikes, TimeSpan duration)
		{
			IEnumerable<DeviceId> selectedDevices = devices.Where(d => d.PoolId == pool && d.PlatformId == platform).Select(d => d.Id);
			for (int i = 0; i < nSpikes; i++)
			{
				foreach (DeviceId id in selectedDevices)
				{
					telemetry.Add(
						new TestDeviceTelemetry()
						{
							CreateTimeUtc = referenceTime,
							DeviceId = id,
							ReservationFinishUtc = referenceTime + duration
						}
					);
					referenceTime = referenceTime.AddSeconds(5); // shift timing slightly every time
				}
				referenceTime = referenceTime.Add(duration + TimeSpan.FromMinutes(5)); // shift timing 5 minutes between spikes
			}
		}

		static void AddProblemSpikes(List<IDeviceTelemetry> telemetry, List<IDevice> devices, DevicePoolId pool, DevicePlatformId platform, ref DateTime referenceTime, int percSpikes, TimeSpan offset)
		{
			IEnumerable<DeviceId> selectedDevices = devices.Where(d => d.PoolId == pool && d.PlatformId == platform).Select(d => d.Id);
			int devicesCount = selectedDevices.Count();
			devicesCount = (int)((double)devicesCount * percSpikes / 100);
			foreach (DeviceId id in selectedDevices.Take(devicesCount))
			{
				telemetry.Add(
					new TestDeviceTelemetry()
					{
						CreateTimeUtc = referenceTime,
						DeviceId = id,
						ReservationFinishUtc = referenceTime + offset,
						ProblemTimeUtc = referenceTime + offset
					}
				);
				referenceTime = referenceTime.AddSeconds(5); // shift timing slightly every time
			}
		}

		[TestMethod]
		public async Task TestPopulateDeviceIssueReportAsync()
		{
			await SetupDevicesAsync();
			List<IDevicePool> pools = DeviceService!.GetPools();
			List<IDevicePlatform> platforms = DeviceService!.GetPlatforms();
			List<IDevice> devices = (await DeviceService!.GetDevicesAsync()).OrderBy(d => d.PoolId.ToString()).ThenBy(d => d.PlatformId.ToString()).ToList();
			List<IDeviceTelemetry> telemetry = new List<IDeviceTelemetry>();

			DateTime referencePoint = DateTime.Now - TimeSpan.FromDays(1);
			AddLoadSpikes(telemetry, devices, pools[0].Id, platforms[0].Id, ref referencePoint, 3, TimeSpan.FromMinutes(3));
			referencePoint = referencePoint.AddHours(1);
			AddProblemSpikes(telemetry, devices, pools[0].Id, platforms[0].Id, ref referencePoint, 40, TimeSpan.FromMinutes(3));

			string deviceToClean = devices.Last().Id.ToString();
			await DeviceController.CleanDeviceAsync(deviceToClean, new CleanDeviceRequest() { Clean = true });
			devices = (await DeviceService!.GetDevicesAsync()).OrderBy(d => d.PoolId.ToString()).ThenBy(d => d.PlatformId.ToString()).ToList();

			DeviceReportData deviceData = new DeviceReportData()
			{
				Pools = pools,
				Platforms = platforms,
				Devices = devices,
				DeviceTelemetry = telemetry
			};
			DeviceReportSettings settings = new DeviceReportSettings()
			{
				Channel = "a-channel",
				ReportMinutes = 3 * 60,
				DeviceProblemCoolDownMinutes = 5,
				DeviceSaturationSpikeThresholdMinutes = 1,
				DeviceCleaningThresholdHours = 0,
				ServerUrl = new Uri("https://testserver")
			};

			DeviceIssueReport issueReport = DeviceReportService.CreateDeviceReport(deviceData, settings);

			Assert.AreEqual(2, issueReport.PlatformReports.Count);
			Assert.AreEqual(3, issueReport.PlatformReports[0].DeviceReports.Count);
			Assert.AreNotEqual(null, issueReport.PlatformReports[0].DeviceReports[0].DevicePoolURL);
			Assert.AreNotEqual(null, issueReport.PlatformReports[0].DeviceReports[1].DevicePoolURL);
			Assert.AreNotEqual(null, issueReport.PlatformReports[0].DeviceReports[2].DevicePoolURL);
			Assert.AreEqual(1, issueReport.PlatformReports[1].DeviceReports.Count);
			Assert.AreNotEqual(null, issueReport.PlatformReports[1].DeviceReports[0].DevicePoolURL);
			Assert.AreNotEqual(null, issueReport.PlatformReports[1].DeviceReports[0].CleaningTime);
			Assert.AreEqual(deviceToClean, issueReport.PlatformReports[1].DeviceReports[0].DeviceId);

			Assert.AreEqual(2, issueReport.PoolReports.Count);
			Assert.AreEqual(3, issueReport.PoolReports[0].Metrics.Count);
			Assert.AreEqual(6, issueReport.PoolReports[0].Metrics[0].AverageLoadPercentage);
			Assert.AreEqual(3, issueReport.PoolReports[0].Metrics[0].SaturationSpikes);
			Assert.AreEqual("00:02", issueReport.PoolReports[0].Metrics[0].SpikeDurationAverage.ToString(@"hh\:mm"));
			Assert.AreEqual(4, issueReport.PoolReports[0].Metrics[0].SpikeDurationPercentage);
			Assert.AreEqual(37, issueReport.PoolReports[0].Metrics[0].MaxConcurrentProblemsPercentage);
			Assert.AreEqual(3, issueReport.PoolReports[0].Metrics[0].MaxConcurrentProblems);
		}

		[TestMethod]
		public async Task TestReservationV2Async()
		{
			await SetupDevicesAsync();
			
			CreateDeviceReservationRequest request = new CreateDeviceReservationRequest { PoolId = "TestDevicePool1", Devices = new List<DeviceReservationRequest> { new DeviceReservationRequest { PlatformId = "testdeviceplatform1" } } };

			// create a reservation
			CreateDeviceReservationResponse reservation = (await DeviceController!.CreateDeviceReservationAsync(request)).Value!;
			Assert.AreEqual(1, reservation.Devices.Count);
			Assert.AreEqual("testdeviceplatform1", reservation.Devices[0].PlatformId);

			// delete the reservation, would be nice to also test reservation expiration
			ActionResult deleted = (await DeviceController!.DeleteReservationAsync(reservation.Id));

			Assert.AreEqual(200, (deleted as OkResult)!.StatusCode);
		}
	}
}
