// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Jobs;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Templates;
using HordeServer.Plugins;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;

namespace HordeServer.Tests
{
	public class Fixture
	{
		public IJob Job1 { get; private set; } = null!;
		public IJob Job2 { get; private set; } = null!;
		public ITemplate Template { get; private set; } = null!;
		public IGraph Graph { get; private set; } = null!;
		public StreamId StreamId { get; private set; }
		public StreamConfig? StreamConfig { get; private set; }
		public TemplateId TemplateRefId1 { get; private set; }
		public TemplateId TemplateRefId2 { get; private set; }
		public IAgent Agent1 { get; private set; } = null!;
		public string Agent1Name { get; private set; } = null!;
		public const string PoolName = "TestingPool";

		public static async Task<Fixture> CreateAsync(ConfigService configService, IGraphCollection graphCollection, ITemplateCollectionInternal templateCollection, JobService jobService, AgentService agentService, IPluginCollection pluginCollection, ServerSettings serverSettings)
		{
			Fixture fixture = new Fixture();
			await fixture.PopulateAsync(configService, graphCollection, templateCollection, jobService, agentService, pluginCollection, serverSettings);

			//			(PerforceService as PerforceServiceStub)?.AddChange("//UE5/Main", 112233, "leet.coder", "Did stuff", new []{"file.cpp"});
			//			(PerforceService as PerforceServiceStub)?.AddChange("//UE5/Main", 1111, "swarm", "A shelved CL here", new []{"renderer.cpp"});

			return fixture;
		}

		private async Task PopulateAsync(ConfigService configService, IGraphCollection graphCollection, ITemplateCollectionInternal templateCollection, JobService jobService, AgentService agentService, IPluginCollection pluginCollection, ServerSettings serverSettings)
		{
			GraphStub fg = new(ContentHash.Empty, 1122);
			Template = await templateCollection.GetOrAddAsync(new TemplateConfig { Name = "Test template" });
			Graph = await graphCollection.AddAsync(Template);

			TemplateRefId1 = new TemplateId("template1");
			TemplateRefId2 = new TemplateId("template2");

			List<TemplateRefConfig> templates = new List<TemplateRefConfig>();
			templates.Add(new TemplateRefConfig { Id = TemplateRefId1, Name = "Test Template" });
			templates.Add(new TemplateRefConfig { Id = TemplateRefId2, Name = "Test Template" });

			List<TabConfig> tabs = new List<TabConfig>();
			tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { TemplateRefId1, TemplateRefId2 } });

			Dictionary<string, AgentConfig> agentTypes = new()
			{
				{ "Win64", new() { Pool = new PoolId(PoolName), Workspace = "Workspace1"} }
			};

			Dictionary<string, WorkspaceConfig> workspaceTypes = new()
			{
				{ "Workspace1", new () { Identifier = "Workspace1", ConformDiskFreeSpace = 500 } }
			};

			StreamId streamId = new StreamId("ue5-main");
			StreamConfig streamConfig = new StreamConfig { Id = streamId, Name = "//UE5/Main", Tabs = tabs, Templates = templates, AgentTypes = agentTypes, WorkspaceTypes = workspaceTypes };

			ProjectId projectId = new ProjectId("ue5");
			ProjectConfig projectConfig = new ProjectConfig { Id = projectId, Name = "UE5", Streams = new List<StreamConfig> { streamConfig } };

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects = new List<ProjectConfig> { projectConfig };

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);
			globalConfig.PostLoad(serverSettings, pluginCollection.LoadedPlugins, Array.Empty<IDefaultAclModifier>());
			configService.OverrideConfig(globalConfig);

			StreamId = streamId;
			StreamConfig = streamConfig;

			Job1 = await jobService.CreateJobAsync(
				jobId: JobId.Parse("5f283932841e7fdbcafb6ab5"),
				streamConfig: streamConfig,
				templateRefId: TemplateRefId1,
				templateHash: Template.Hash,
				graph: Graph,
				name: "hello1",
				commitId: CommitIdWithOrder.FromPerforceChange(1000001),
				codeCommitId: CommitIdWithOrder.FromPerforceChange(1000002),
				new CreateJobOptions { PreflightCommitId = CommitId.FromPerforceChange(1001) }
			);
			Job1 = (await jobService.GetJobAsync(Job1.Id))!;

			Job2 = await jobService.CreateJobAsync(
				jobId: JobId.Parse("5f69ea1b68423e921b035106"),
				streamConfig: streamConfig,
				templateRefId: new TemplateId("template-id-1"),
				templateHash: ContentHash.MD5("made-up-template-hash"),
				graph: fg,
				name: "hello2",
				commitId: CommitIdWithOrder.FromPerforceChange(2000001),
				codeCommitId: CommitIdWithOrder.FromPerforceChange(2000002),
				new CreateJobOptions()
			);
			Job2 = (await jobService.GetJobAsync(Job2.Id))!;

			Agent1Name = "testAgent1";
			Agent1 = await agentService.CreateAgentAsync(new CreateAgentOptions(new AgentId(Agent1Name), AgentMode.Dedicated, false, ""));
		}
	}

	internal class GraphStub(ContentHash id, int schema = 1) : IGraph
	{
		public ContentHash Id { get; set; } = id;
		public int Schema { get; set; } = schema;
		public IReadOnlyList<INodeGroup> Groups { get; set; } = [];
		public IReadOnlyList<IAggregate> Aggregates { get; set; } = [];
		public IReadOnlyList<ILabel> Labels { get; set; } = [];
		public IReadOnlyList<IGraphArtifact> Artifacts { get; set; } = [];
	}

	internal class TemplateStub(ContentHash hash, string name) : ITemplate
	{
		public ContentHash Hash { get; } = hash;
		public string Name { get; } = name;
		public string? Description { get; init; } = null;
		public Priority? Priority { get; init; } = null;
		public bool AllowPreflights { get; init; } = false;
		public bool UpdateIssues { get; init; } = false;
		public bool PromoteIssuesByDefault { get; init; } = false;
		public string? InitialAgentType { get; init; } = null;
		public string? SubmitNewChange { get; init; } = null;
		public string? SubmitDescription { get; init; } = null;
		public IReadOnlyList<string> Arguments { get; init; } = [];
		public IReadOnlyList<ITemplateParameter> Parameters { get; init; } = [];
	}
}