// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json;
using Amazon.AutoScaling;
using Amazon.AutoScaling.Model;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Moq;

namespace HordeServer.Tests.Fleet;

[TestClass]
public class AwsAutoScalingLifecycleServiceTest : BuildTestSetup
{
	private AwsAutoScalingLifecycleService _asgLifecycleService = default!;
	private Mock<IAmazonAutoScaling> _asgMock = default!;
	private readonly List<CompleteLifecycleActionRequest> _lifecycleUpdates = new();
	private readonly FakeAmazonSqs _fakeSqs = new();
	private readonly string _queueUrl1 = "https://sqs.us-east-1.amazonaws.com/123456789/MyQueue-1";
	private readonly string _queueUrl2 = "https://sqs.us-east-1.amazonaws.com/123456789/MyQueue-2";

	public override async ValueTask DisposeAsync()
	{
		GC.SuppressFinalize(this);
		await base.DisposeAsync();

		if (_asgLifecycleService != null)
		{
			await _asgLifecycleService.DisposeAsync();
			_asgLifecycleService = null!;
		}

		_fakeSqs?.Dispose();
	}

	[TestInitialize]
	public async Task SetupAsync()
	{
		ILogger<AwsAutoScalingLifecycleService> logger = ServiceProvider.GetRequiredService<ILogger<AwsAutoScalingLifecycleService>>();
		_asgMock = new Mock<IAmazonAutoScaling>(MockBehavior.Strict);

		Task<CompleteLifecycleActionResponse> OnCompleteLifecycleActionAsync(CompleteLifecycleActionRequest request, CancellationToken cancellationToken)
		{
			_lifecycleUpdates.Add(request);
			return Task.FromResult(new CompleteLifecycleActionResponse() { HttpStatusCode = HttpStatusCode.OK });
		}

		_asgMock
			.Setup(x => x.CompleteLifecycleActionAsync(It.IsAny<CompleteLifecycleActionRequest>(), It.IsAny<CancellationToken>()))
			.Returns(OnCompleteLifecycleActionAsync);

		ComputeServerConfig ss = new() { AwsAutoScalingQueueUrls = new[] { _queueUrl1, _queueUrl2 } };
		_asgLifecycleService = new AwsAutoScalingLifecycleService(AgentService, GetRedisServiceSingleton(), AgentCollection, Clock, new TestOptionsMonitor<ComputeServerConfig>(ss), ServiceProvider, Tracer, logger);
		_asgLifecycleService.SetAmazonClientsTesting(_asgMock.Object, _fakeSqs);
		await _asgLifecycleService.StartAsync(CancellationToken.None);
	}

	[TestMethod]
	public async Task TerminationRequested_WithAgentInService_MarkedForShutdownAsync()
	{
		// Arrange
		LifecycleActionEvent lae = new() { Ec2InstanceId = "i-1234", LifecycleActionToken = "action-token-test", Origin = "AutoScalingGroup" };
		IAgent agent = await CreateAgentAsync(new PoolId("pool1"), properties: new() { KnownPropertyNames.AwsInstanceId + "=" + lae.Ec2InstanceId }, ephemeral: true);
		await AgentService.DeleteAgentAsync(agent);

		// Act
		await _fakeSqs.SendMessageAsync(_queueUrl1, JsonSerializer.Serialize(lae));
		await _asgLifecycleService.ReceiveLifecycleEventsAsync(_queueUrl1, CancellationToken.None);

		// Assert
		agent = (await AgentService.GetAgentAsync(agent.Id))!;
		Assert.IsTrue(agent.RequestShutdown);
	}

	[TestMethod]
	public async Task TerminationRequested_WithAgentOnline_LifecycleIsContinuedAsync()
	{
		// Arrange
		LifecycleActionEvent lae = new() { Ec2InstanceId = "i-1234", LifecycleActionToken = "action-token-test", Origin = "AutoScalingGroup" };
		await CreateAgentAsync(new PoolId("pool1"), properties: new() { KnownPropertyNames.AwsInstanceId + "=" + lae.Ec2InstanceId });
		await _asgLifecycleService.InitiateTerminationAsync(lae, CancellationToken.None);

		// Act
		await Clock.AdvanceAsync(_asgLifecycleService.LifecycleUpdaterInterval + TimeSpan.FromSeconds(10));

		// Assert
		AssertLifecycleUpdate(lae, AwsAutoScalingLifecycleService.ActionContinue, _lifecycleUpdates[0]);
	}

	[TestMethod]
	public async Task TerminationRequested_WithAgentGoingFromOnlineToOfflineAsync()
	{
		// Arrange
		LifecycleActionEvent lae = new() { Ec2InstanceId = "i-1234", LifecycleActionToken = "action-token-test", Origin = "AutoScalingGroup" };
		List<string> props = new() { KnownPropertyNames.AwsInstanceId + "=" + lae.Ec2InstanceId };
		IAgent agent = await CreateAgentAsync(new PoolId("pool1"), properties: props);
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(props), "v1");
		await _asgLifecycleService.InitiateTerminationAsync(lae, CancellationToken.None);
		TimeSpan extraMargin = TimeSpan.FromSeconds(10);

		// Agent is online (e.g still running a job) and the lifecycle update tick is triggered
		await Clock.AdvanceAsync(_asgLifecycleService.LifecycleUpdaterInterval + extraMargin);
		AssertLifecycleUpdate(lae, AwsAutoScalingLifecycleService.ActionContinue, _lifecycleUpdates[0]);

		// Agent still online
		await Clock.AdvanceAsync(_asgLifecycleService.LifecycleUpdaterInterval + extraMargin);
		AssertLifecycleUpdate(lae, AwsAutoScalingLifecycleService.ActionContinue, _lifecycleUpdates[1]);

		// Agent responded to shutdown request, now safe to notify AWS ASG the termination can take place
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(props), "v1");
		agent = (await agent.TryUpdateSessionAsync(new UpdateSessionOptions { Status = AgentStatus.Stopped, Capabilities = new RpcAgentCapabilities(props) }))!;
		await Clock.AdvanceAsync(_asgLifecycleService.LifecycleUpdaterInterval + extraMargin);
		AssertLifecycleUpdate(lae, AwsAutoScalingLifecycleService.ActionAbandon, _lifecycleUpdates[2]);

		// Trigger lifecycle update tick once again and no further updates should have been sent
		await Clock.AdvanceAsync(_asgLifecycleService.LifecycleUpdaterInterval + extraMargin);
		Assert.AreEqual(3, _lifecycleUpdates.Count);

		_ = agent;
	}

	[TestMethod]
	public async Task TerminationRequested_WithAgentInWarmPool_LifecycleIsAbandonedAsync()
	{
		// Arrange
		LifecycleActionEvent lae = new() { Ec2InstanceId = "i-1234", LifecycleActionToken = "action-token-test", Origin = "WarmPool" };

		// Act
		await _asgLifecycleService.InitiateTerminationAsync(lae, CancellationToken.None);

		// Assert
		AssertLifecycleUpdate(lae, AwsAutoScalingLifecycleService.ActionAbandon, _lifecycleUpdates[0]);
	}

	[TestMethod]
	public async Task GetInstancesAvailableForTermination_IdleAgent_ReturnsInstanceIdAsync()
	{
		// Arrange
		_ = await CreateAgentAsync(new PoolId("pool1"), awsInstanceId: "i-1000");
		TerminationPolicyEvent e = CreateTerminationPolicyEvent("i-1000");

		// Act
		List<string> instanceIds = await _asgLifecycleService.GetInstancesAvailableForTerminationAsync(e, CancellationToken.None);

		// Assert
		CollectionAssert.AreEqual(new List<string>() { "i-1000" }, instanceIds);
	}

	[TestMethod]
	public async Task GetInstancesAvailableForTermination_IdleAgentsInMixedAsgs_OnlyReturnInstanceIdFromSameAsgAsync()
	{
		// Arrange
		_ = await CreateAgentAsync(new PoolId("pool1"), awsInstanceId: "i-1000");
		_ = await CreateAgentAsync(new PoolId("pool1"), awsInstanceId: "i-2000");
		TerminationPolicyEvent e = CreateTerminationPolicyEvent("i-1000");

		// Act
		List<string> instanceIds = await _asgLifecycleService.GetInstancesAvailableForTerminationAsync(e, CancellationToken.None);

		// Assert
		CollectionAssert.AreEqual(new List<string>() { "i-1000" }, instanceIds);
	}

	[TestMethod]
	public async Task GetInstancesAvailableForTermination_AgentRunningJob_ReturnsNoInstanceIdAsync()
	{
		// Arrange
		CreateLeaseOptions lease = new(new LeaseId(BinaryIdUtils.CreateNew()), null, "test-lease", null, null, null, null, false, new Empty());
		_ = await CreateAgentAsync(new PoolId("pool1"), awsInstanceId: "i-1000", lease: lease);
		TerminationPolicyEvent e = CreateTerminationPolicyEvent("i-1000");

		// Act
		List<string> instanceIds = await _asgLifecycleService.GetInstancesAvailableForTerminationAsync(e, CancellationToken.None);

		// Assert
		Assert.AreEqual(0, instanceIds.Count);
	}

	[TestMethod]
	public void DeserializeLifecycleActionEvent()
	{
		string rawText = @"
{
	""Origin"": ""AutoScalingGroup"",
	""LifecycleHookName"": ""my-hook-name"",
	""Destination"": ""EC2"",
	""AccountId"": ""123456789"",
	""RequestId"": ""5ab929ac-06d9-11ee-be56-0242ac120002"",
	""LifecycleTransition"": ""autoscaling:EC2_INSTANCE_TERMINATING"",
	""AutoScalingGroupName"": ""my-asg-name"",
	""Service"": ""AWS Auto Scaling"",
	""Time"": ""2023-06-09T11:11:20.345Z"",
	""EC2InstanceId"": ""i-123456789"",
	""LifecycleActionToken"": ""5ec3537e-06d9-11ee-be56-0242ac120002""
}
";

		LifecycleActionEvent? ev = JsonSerializer.Deserialize<LifecycleActionEvent>(rawText);
		Assert.AreEqual("5ec3537e-06d9-11ee-be56-0242ac120002", ev!.LifecycleActionToken);
		Assert.AreEqual("my-asg-name", ev.AutoScalingGroupName);
		Assert.AreEqual("my-hook-name", ev.LifecycleHookName);
		Assert.AreEqual("i-123456789", ev.Ec2InstanceId);
		Assert.AreEqual("autoscaling:EC2_INSTANCE_TERMINATING", ev.LifecycleTransition);
		Assert.AreEqual("AutoScalingGroup", ev.Origin);
	}

	[TestMethod]
	public void DeserializeTerminationPolicyEvent()
	{
		string rawText = @"
{
	""AutoScalingGroupARN"": ""my-asg-arn"",
	""AutoScalingGroupName"": ""my-asg-name"",
	""CapacityToTerminate"": [{
		""AvailabilityZone"": ""us-east-1a"",
		""Capacity"": 5,
		""InstanceMarketOption"": ""on-demand""
	}],
	""Instances"": [{
		""AvailabilityZone"": ""us-east-1b"",
		""InstanceId"": ""i-10001"",
		""InstanceType"": ""m5d.large"",
		""InstanceMarketOption"": ""on-demand""
	}, {
		""AvailabilityZone"": ""us-east-1c"",
		""InstanceId"": ""i-10002"",
		""InstanceType"": ""m6i.large"",
		""InstanceMarketOption"": ""spot""
	}],
	""Cause"": ""SCALE_IN""
}
";

		TerminationPolicyEvent? ev = JsonSerializer.Deserialize<TerminationPolicyEvent>(rawText);
		Assert.AreEqual("my-asg-arn", ev!.AutoScalingGroupArn);
		Assert.AreEqual("my-asg-name", ev.AutoScalingGroupName);
		Assert.AreEqual("SCALE_IN", ev.Cause);

		{
			Assert.AreEqual(1, ev.CapacityToTerminate.Count);
			Assert.AreEqual("us-east-1a", ev.CapacityToTerminate[0].AvailabilityZone);
			Assert.AreEqual(5, ev.CapacityToTerminate[0].Capacity);
			Assert.AreEqual("on-demand", ev.CapacityToTerminate[0].InstanceMarketOption);
		}

		{
			Assert.AreEqual(2, ev.Instances.Count);
			Assert.AreEqual("us-east-1b", ev.Instances[0].AvailabilityZone);
			Assert.AreEqual("i-10001", ev.Instances[0].InstanceId);
			Assert.AreEqual("m5d.large", ev.Instances[0].InstanceType);
			Assert.AreEqual("on-demand", ev.Instances[0].InstanceMarketOption);

			Assert.AreEqual("us-east-1c", ev.Instances[1].AvailabilityZone);
			Assert.AreEqual("i-10002", ev.Instances[1].InstanceId);
			Assert.AreEqual("m6i.large", ev.Instances[1].InstanceType);
			Assert.AreEqual("spot", ev.Instances[1].InstanceMarketOption);
		}
	}

	private static void AssertLifecycleUpdate(LifecycleActionEvent expectedEvent, string expectedResult, CompleteLifecycleActionRequest actual)
	{
		Assert.AreEqual(expectedEvent.Ec2InstanceId, actual.InstanceId);
		Assert.AreEqual(expectedResult, actual.LifecycleActionResult);
		Assert.AreEqual(expectedEvent.LifecycleActionToken, actual.LifecycleActionToken);
		Assert.AreEqual(expectedEvent.LifecycleHookName, actual.LifecycleHookName);
		Assert.AreEqual(expectedEvent.AutoScalingGroupName, actual.AutoScalingGroupName);
	}

	private static TerminationPolicyEvent CreateTerminationPolicyEvent(params string[] instanceIds)
	{
		return new()
		{
			AutoScalingGroupArn = "test-asg-arn",
			AutoScalingGroupName = "test-asg-name",
			Cause = "SCALE_IN",
			CapacityToTerminate = new() { new(1) },
			Instances = new(instanceIds.Select(x => new TerminationPolicyInstance(x)))
		};
	}
}
