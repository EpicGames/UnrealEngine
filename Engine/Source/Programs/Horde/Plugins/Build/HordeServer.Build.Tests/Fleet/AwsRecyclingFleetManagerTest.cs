// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.EC2;
using Amazon.EC2.Model;
using EpicGames.Horde.Agents.Pools;
using HordeCommon;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Fleet.Providers;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.Logging;
using JsonSerializer = System.Text.Json.JsonSerializer;

namespace HordeServer.Tests.Fleet
{
	[TestClass]
	public class AwsRecyclingFleetManagerTest : BuildTestSetup
	{
		private const string PoolId = "test-pool";
		private const string PoolName = "testPool";
		private readonly FakeClock _clock = new();

		[TestMethod]
		public async Task ExpandAmazonEc2ExceptionsArePropagatedAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolId: PoolId);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.OnStartInstances = (_, _) => throw new AmazonEC2Exception("Something bad happened");
			await Assert.ThrowsExactlyAsync<AmazonEC2Exception>(() => ExpandPoolAsync(ec2.Get(), 1, new()));
		}

		[TestMethod]
		public async Task ExpandOneAzPartialCapacityWithFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i1 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName, tagPoolId: PoolId);
			Instance i2 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName, tagPoolId: PoolId);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.C5Large, 2);

			await ExpandPoolAsync(ec2.Get(), 2, new(new List<string> { InstanceType.C5Large }));
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i1.InstanceId)!.InstanceType);
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i2.InstanceId)!.InstanceType);
		}

		[TestMethod]
		public async Task ExpandOneAzPartialCapacityWithMultipleFallbacksAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i1 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			Instance i2 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.C5Large, 2);

			await ExpandPoolAsync(ec2.Get(), 2, new(new List<string> { InstanceType.M5Xlarge, InstanceType.R5Large, InstanceType.C5Large }));
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i1.InstanceId)!.InstanceType);
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i2.InstanceId)!.InstanceType);
		}

		[TestMethod]
		public async Task ExpandOneAzPartialCapacityNoFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);

			await ExpandPoolAsync(ec2.Get(), 2, new());
			Assert.AreEqual(2, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(0, ec2.GetPendingInstanceCount());
		}

		[TestMethod]
		public async Task ExpandTwoAzsPartialCapacityNoFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1B, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1B, InstanceType.M5Large, 0);

			await ExpandPoolAsync(ec2.Get(), 2, new());
			Assert.AreEqual(1, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(1, ec2.GetPendingInstanceCount());
		}

		[TestMethod]
		public async Task ExpandMoreThanNumStoppedInstancesAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1B, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1B, InstanceType.M5Large, 1);

			Assert.AreEqual(2, ec2.GetStoppedInstanceCount());
			await ExpandPoolAsync(ec2.Get(), 10, new());
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());

			// TODO: Verify remainder is reported correctly
		}

		[TestMethod]
		public async Task ExpandTwoAzsFullCapacityNoFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1B, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1B, InstanceType.M5Large, 1);

			await ExpandPoolAsync(ec2.Get(), 2, new());
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
		}

		[TestMethod]
		public async Task ExpandTwoAzsFullCapacityEmptyFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1B, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1B, InstanceType.M5Large, 1);

			await ExpandPoolAsync(ec2.Get(), 2, new(new List<string>()));
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
		}

		[TestMethod]
		public async Task ExpandLaunchesLastUsedInstancesFirstAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i1 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, launchTime: DateTime.Parse("Aug 18, 2018"), tagPoolName: PoolName);
			Instance i2 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, launchTime: DateTime.Parse("Aug 22, 2018"), tagPoolName: PoolName);
			Instance i3 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, launchTime: DateTime.Parse("Aug 5, 2018"), tagPoolName: PoolName);
			Instance i4 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, launchTime: DateTime.Parse("Aug 28, 2018"), tagPoolName: PoolName);
			Instance i5 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, launchTime: DateTime.Parse("Aug 1, 2018"), tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 2);

			_ = i1;
			_ = i3;
			_ = i5;

			await ExpandPoolAsync(ec2.Get(), 2, new(new List<string>()));
			Assert.AreEqual(3, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
			Assert.AreEqual(FakeAmazonEc2.StatePending.Name, ec2.GetInstance(i4.InstanceId)!.State.Name);
			Assert.AreEqual(FakeAmazonEc2.StatePending.Name, ec2.GetInstance(i2.InstanceId)!.State.Name);
		}

		[TestMethod]
		public async Task StopInstancesStuckInPendingAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i1 = ec2.AddInstance(FakeAmazonEc2.StatePending, InstanceType.M5Large, launchTime: _clock.UtcNow.Subtract(TimeSpan.FromMinutes(50)), tagPoolName: PoolName);
			Instance i2 = ec2.AddInstance(FakeAmazonEc2.StatePending, InstanceType.M5Large, launchTime: _clock.UtcNow.Subtract(TimeSpan.FromMinutes(3)), tagPoolName: PoolName);
			Instance i3 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, tagPoolName: PoolName);
			Instance i4 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 2);

			ScaleResult result = await ExpandPoolAsync(ec2.Get(), 2, new(new List<string>()));
			Assert.AreEqual(FleetManagerOutcome.Success, result.Outcome);
			Assert.AreEqual(3, ec2.GetPendingInstanceCount());
			Assert.AreEqual(FakeAmazonEc2.StateStopping.Name, ec2.GetInstance(i1.InstanceId)!.State.Name);
			Assert.AreEqual(FakeAmazonEc2.StatePending.Name, ec2.GetInstance(i2.InstanceId)!.State.Name);
			Assert.AreEqual(FakeAmazonEc2.StatePending.Name, ec2.GetInstance(i3.InstanceId)!.State.Name);
			Assert.AreEqual(FakeAmazonEc2.StatePending.Name, ec2.GetInstance(i4.InstanceId)!.State.Name);
		}

		[TestMethod]
		public async Task ExpandOneAzUnsupportedWithFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i1 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName, tagPoolId: PoolId);
			Instance i2 = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName, tagPoolId: PoolId);
 
			ec2.SetUnsupported(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large.Value);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.C5Large, 2);
 
			await ExpandPoolAsync(ec2.Get(), 2, new(new List<string> { InstanceType.M5Large, InstanceType.C5Large }));
 
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(2, ec2.GetPendingInstanceCount());
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i1.InstanceId)!.InstanceType);
			Assert.AreEqual(InstanceType.C5Large, ec2.GetInstance(i2.InstanceId)!.InstanceType);
		}
 
		[TestMethod]
		public async Task ExpandUnsupportedNoFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
			ec2.SetUnsupported(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large.Value);
 
			await ExpandPoolAsync(ec2.Get(), 1, new());
 
			Assert.AreEqual(1, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(0, ec2.GetPendingInstanceCount());
		}
 
		[TestMethod]
		public async Task ExpandMixedErrorsWithFallbackAsync()
		{
			FakeAmazonEc2 ec2 = new();
			ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, FakeAmazonEc2.AzUsEast1A, tagPoolName: PoolName);
 
			// First: unsupported, Second: no capacity, Third: works
			ec2.SetUnsupported(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large.Value);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.C5Large, 0);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.R5Large, 1);
 
			await ExpandPoolAsync(ec2.Get(), 1, new(new List<string> {
				InstanceType.M5Large,
				InstanceType.C5Large,
				InstanceType.R5Large
			}));
 
			Assert.AreEqual(0, ec2.GetStoppedInstanceCount());
			Assert.AreEqual(1, ec2.GetPendingInstanceCount());
		}
		
		[TestMethod]
		public void DistributesInstanceRequestsEvenly()
		{
			AssertInstanceCounts(0, new() { ["a"] = 3, ["b"] = 4 }, new() { ["a"] = 0, ["b"] = 0 }, 0);
			AssertInstanceCounts(0, new() { }, new() { }, 0);
			AssertInstanceCounts(1, new() { ["a"] = 1, ["b"] = 1, ["c"] = 1 }, new() { ["a"] = 1, ["b"] = 0, ["c"] = 0 }, 0);
			AssertInstanceCounts(2, new() { ["a"] = 1, ["b"] = 1, ["c"] = 1 }, new() { ["a"] = 1, ["b"] = 1, ["c"] = 0 }, 0);
			AssertInstanceCounts(5, new() { ["a"] = 10 }, new() { ["a"] = 5 }, 0);
			AssertInstanceCounts(6, new() { ["a"] = 10, ["b"] = 2 }, new() { ["a"] = 4, ["b"] = 2 }, 0);
			AssertInstanceCounts(3, new() { ["a"] = 1, ["b"] = 1, ["c"] = 1 }, new() { ["a"] = 1, ["b"] = 1, ["c"] = 1 }, 0);

			// Remainder > 0
			AssertInstanceCounts(10, new() { ["a"] = 3, ["b"] = 4 }, new() { ["a"] = 3, ["b"] = 4 }, 3);
			AssertInstanceCounts(10, new() { }, new() { }, 10);
		}

		private static void AssertInstanceCounts(int requestCount, Dictionary<string, int> availableCountPerAz, Dictionary<string, int> expected, int expectedRemainder)
		{
			Dictionary<string, int> actual = AwsRecyclingFleetManager.DistributeInstanceRequestsOverAzs(
				requestCount, availableCountPerAz, out int actualRemainder, false);
			try
			{
				CollectionAssert.AreEquivalent(expected.ToList(), actual.ToList());
				Assert.AreEqual(expectedRemainder, actualRemainder);
			}
			catch (Exception)
			{
				Console.WriteLine("Expected=" + JsonSerializer.Serialize(expected));
				Console.WriteLine("  Actual=" + JsonSerializer.Serialize(actual));
				throw;
			}
		}

		private async Task<ScaleResult> ExpandPoolAsync(IAmazonEC2 ec2, int numRequestedInstances, AwsRecyclingFleetManagerSettings settings)
		{
			using ILoggerFactory loggerFactory = LoggerFactory.Create(builder =>
				{
					builder.SetMinimumLevel(LogLevel.Debug);
					builder.AddSimpleConsole(options => { options.SingleLine = true; });
				});

			ILogger<AwsRecyclingFleetManager> logger = loggerFactory.CreateLogger<AwsRecyclingFleetManager>();
#pragma warning disable CS0618 // Type or member is obsolete
			IPool pool = await CreatePoolAsync(new PoolConfig { Id = new PoolId(PoolId), Name = PoolName, EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });
#pragma warning restore CS0618 // Type or member is obsolete
			AwsRecyclingFleetManager manager = new(ec2, AgentCollection, Meter, _clock, settings, Tracer, logger);
			return await manager.ExpandPoolAsync(pool, new List<IAgent>(), numRequestedInstances, CancellationToken.None);
		}
	}
}