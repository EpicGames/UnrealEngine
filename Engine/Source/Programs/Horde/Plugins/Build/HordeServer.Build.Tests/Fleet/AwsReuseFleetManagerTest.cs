// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.EC2;
using Amazon.EC2.Model;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Fleet.Providers;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tests.Fleet
{
	[TestClass]
	public class AwsReuseFleetManagerTest : BuildTestSetup
	{
		private const string PoolId = "test-pool";
		private const string PoolName = "testPool";
		
		[TestMethod]
		public async Task ExpandOneAgentAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, tagPoolId: PoolId);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M5Large, 1);

			await ExpandPoolAsync(ec2.Get(), 1, new());
			Assert.AreEqual(InstanceType.M5Large, ec2.Instances[i.InstanceId].InstanceType);
			Assert.AreEqual(FakeAmazonEc2.StatePending, ec2.Instances[i.InstanceId].State);
		}

		[TestMethod]
		public async Task ExpandWithInstanceTypeChangeAsync()
		{
			FakeAmazonEc2 ec2 = new();
			Instance i = ec2.AddInstance(FakeAmazonEc2.StateStopped, InstanceType.M5Large, tagPoolName: PoolName);
			ec2.SetCapacity(FakeAmazonEc2.AzUsEast1A, InstanceType.M54xlarge, 1);

			await ExpandPoolAsync(ec2.Get(), 1, new(new List<string> { InstanceType.M54xlarge }));
			Assert.AreEqual(InstanceType.M54xlarge, ec2.Instances[i.InstanceId].InstanceType);
			Assert.AreEqual(FakeAmazonEc2.StatePending, ec2.Instances[i.InstanceId].State);
		}

		private async Task ExpandPoolAsync(IAmazonEC2 ec2, int numRequestedInstances, AwsReuseFleetManagerSettings settings)
		{
			ILogger<AwsReuseFleetManager> logger = ServiceProvider.GetRequiredService<ILogger<AwsReuseFleetManager>>();
#pragma warning disable CS0618 // Type or member is obsolete
			IPool pool = await CreatePoolAsync(new PoolConfig { Id = new PoolId(PoolId), Name = PoolName, EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });
#pragma warning restore CS0618 // Type or member is obsolete
			AwsReuseFleetManager manager = new(ec2, AgentCollection, settings, Tracer, logger);
			await manager.ExpandPoolAsync(pool, new List<IAgent>(), numRequestedInstances, CancellationToken.None);
		}
	}
}