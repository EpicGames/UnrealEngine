// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using Amazon.CloudWatch;
using Amazon.CloudWatch.Model;
using Moq;

namespace HordeServer.Tests.Fleet;

/// <summary>
/// Fake implementation of the IAmazonCloudWatch interface
/// Simplifies testing of external CloudWatch API calls.
/// Uses a mock to avoid implementing the entire interface. Even if shimmed, it becomes many lines of no-op code.
/// </summary>
public class FakeAmazonCloudWatch
{
	private readonly Mock<IAmazonCloudWatch> _mock;
	private readonly Dictionary<string, List<MetricDatum>> _namespaceToMetricData = new();

	public FakeAmazonCloudWatch()
	{
		_mock = new Mock<IAmazonCloudWatch>(MockBehavior.Strict);
		_mock
			.Setup(x => x.PutMetricDataAsync(It.IsAny<PutMetricDataRequest>(), It.IsAny<CancellationToken>()))
			.Returns(PutMetricDataAsync);
	}

	public IAmazonCloudWatch Get()
	{
		return _mock.Object;
	}

	public List<MetricDatum> GetMetricData(string ns)
	{
		return _namespaceToMetricData[ns];
	}

	private Task<PutMetricDataResponse> PutMetricDataAsync(PutMetricDataRequest request, CancellationToken cancellationToken = default)
	{
		if (!_namespaceToMetricData.TryGetValue(request.Namespace, out List<MetricDatum>? datums))
		{
			datums = new List<MetricDatum>();
			_namespaceToMetricData[request.Namespace] = datums;
		}

		datums.AddRange(request.MetricData);
		return Task.FromResult(new PutMetricDataResponse { HttpStatusCode = HttpStatusCode.OK });
	}
}