// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using Amazon.EC2;
using Amazon.EC2.Model;
using HordeServer.Agents.Fleet.Providers;
using Moq;

namespace HordeServer.Tests.Fleet;

/// <summary>
/// Fake implementation of the IAmazonEC2 interface
/// Simplifies testing of external EC2 API calls.
/// Uses a mock to avoid implementing the entire interface. Even if shimmed, it becomes many lines of no-op code.
/// </summary>
public class FakeAmazonEc2
{
	public static readonly InstanceState StatePending = new() { Code = 0, Name = "pending" };
	public static readonly InstanceState StateStopping = new() { Code = 64, Name = "stopping" };
	public static readonly InstanceState StateStopped = new() { Code = 80, Name = "stopped" };
	public const string AzUsEast1A = "us-east-1a";
	public const string AzUsEast1B = "us-east-1b";

	private readonly Mock<IAmazonEC2> _mock;
	private readonly Dictionary<string, Instance> _instances = new();
	private int _instanceIdCounter;

	/// <summary>
	/// Callback when StartInstances method is called. Can be used for mocking exception throwing.
	/// </summary>
	internal Action<StartInstancesRequest, CancellationToken> OnStartInstances { get; set; } = (_, _) => { };

	/// <summary>
	/// Callback when ModifyInstanceAttribute method is called. Can be used for mocking exception throwing.
	/// </summary>
	internal Func<ModifyInstanceAttributeRequest, CancellationToken, Task> OnModifyInstanceAttribute { get; set; } = (_, _) => Task.CompletedTask;

	internal IReadOnlyDictionary<string, Instance> Instances => _instances;

	private readonly Dictionary<string, Dictionary<string, int>> _availabilityZoneCapacity = new()
	{
		{AzUsEast1A, new () },
		{AzUsEast1B, new () }
	};

	private readonly Dictionary<string, HashSet<string>> _unsupported = new()
	{
		{AzUsEast1A, new HashSet<string>()},
		{AzUsEast1B, new HashSet<string>()}
	};

	public FakeAmazonEc2()
	{
		_mock = new Mock<IAmazonEC2>(MockBehavior.Strict);
		_mock
			.Setup(x => x.StartInstancesAsync(It.IsAny<StartInstancesRequest>(), It.IsAny<CancellationToken>()))
			.Returns(StartInstancesAsync);

		_mock
			.Setup(x => x.StopInstancesAsync(It.IsAny<StopInstancesRequest>(), It.IsAny<CancellationToken>()))
			.Returns(StopInstancesAsync);

		_mock
			.Setup(x => x.DescribeInstancesAsync(It.IsAny<DescribeInstancesRequest>(), It.IsAny<CancellationToken>()))
			.Returns(DescribeInstancesAsync);

		_mock
			.Setup(x => x.ModifyInstanceAttributeAsync(It.IsAny<ModifyInstanceAttributeRequest>(), It.IsAny<CancellationToken>()))
			.Returns(ModifyInstanceAttributeAsync);
	}

	public IAmazonEC2 Get()
	{
		return _mock.Object;
	}

	public void SetCapacity(string availabilityZone, string instanceType, int capacity)
	{
		Dictionary<string, int> temp = _availabilityZoneCapacity.GetValueOrDefault(availabilityZone, new());
		temp[instanceType] = capacity;
		_availabilityZoneCapacity[availabilityZone] = temp;
	}

	public void SetUnsupported(string availabilityZone, string instanceType)
	{
		HashSet<string> unsupported = _unsupported.GetValueOrDefault(availabilityZone, new HashSet<string>());
		unsupported.Add(instanceType);
		_unsupported[availabilityZone] = unsupported;
	}

	public Instance AddInstance(
		InstanceState state,
		InstanceType type,
		string availabilityZone = AzUsEast1A,
		string? tagPoolId = null,
		string? tagPoolName = null,
		DateTime? launchTime = null)
	{
		Instance i = new()
		{
			InstanceId = "bogus-instance-" + _instanceIdCounter++,
			State = state,
			InstanceType = type,
			Placement = new Placement(availabilityZone),
			LaunchTime = launchTime ?? DateTime.UnixEpoch
		};
		if (tagPoolId != null)
		{
			i.Tags.Add(new Tag(AwsFleetManager.AwsTagPoolId, tagPoolId));
		}
		if (tagPoolName != null)
		{
			i.Tags.Add(new Tag(AwsFleetManager.AwsTagPoolName, tagPoolName));
		}
		_instances.Add(i.InstanceId, i);
		return i;
	}

	public Instance? GetInstance(string instanceId)
	{
		return _instances[instanceId];
	}

	public static Instance CopyInstance(Instance i)
	{
		return new Instance
		{
			InstanceId = i.InstanceId,
			InstanceType = i.InstanceType,
			State = i.State,
			Placement = i.Placement,
			LaunchTime = i.LaunchTime,
			Tags = [..i.Tags]
		};
	}

	public int GetStoppedInstanceCount() { return _instances.Values.Count(x => x.State == StateStopped); }
	public int GetPendingInstanceCount() { return _instances.Values.Count(x => x.State == StatePending); }

	private static bool MatchesFilter(Instance i, Filter f)
	{
		if (f.Name == "instance-state-name")
		{
			return i.State.Name == f.Values[0];
		}
		
		if (f.Name.StartsWith("tag:", StringComparison.Ordinal))
		{
			string key = f.Name["tag:".Length..];
			Tag? tag = i.Tags.FirstOrDefault(t => String.Equals(t.Key, key, StringComparison.Ordinal));
			if (tag == null)
			{
				return false;
			}
			
			return tag.Value == f.Values[0];
		}
		
		throw new Exception("Unknown filter " + f.Name);
	}
	
	private Task<DescribeInstancesResponse> DescribeInstancesAsync(DescribeInstancesRequest request, CancellationToken cancellationToken)
	{
		 List<Instance> filteredInstances = request.Filters.Count > 0
			? _instances.Values.Where(i => request.Filters.All(f => MatchesFilter(i, f))).ToList()
			: _instances.Values.ToList();

		return Task.FromResult(new DescribeInstancesResponse
		{
			Reservations = [new Reservation { Instances = filteredInstances.Select(CopyInstance).ToList() }],
			HttpStatusCode = HttpStatusCode.OK
		});
	}

	private Task<StartInstancesResponse> StartInstancesAsync(StartInstancesRequest request, CancellationToken cancellationToken)
	{
		List<InstanceStateChange> stateChanges = new();

		// Check if capacity in each AZ can handle this start instance request.
		// If not, abort with an exception prior attempting to start them and modifying in-memory state.
		Dictionary<string, Dictionary<string, int>> azInstanceTypeCapacities = new(_availabilityZoneCapacity);
		foreach (string instanceId in request.InstanceIds)
		{
			if (!_instances.ContainsKey(instanceId))
			{
				throw new ArgumentException("Unknown instance ID " + instanceId);
			}

			string az = _instances[instanceId].Placement.AvailabilityZone;
			string instanceType = _instances[instanceId].InstanceType;

			// Check for unsupported first
			if (_unsupported.GetValueOrDefault(az, new HashSet<string>()).Contains(instanceType))
			{
				throw new AmazonEC2Exception("The requested configuration is currently not supported.")
				{
					ErrorCode = "Unsupported"
				};
			}

			int capacity = azInstanceTypeCapacities[az].GetValueOrDefault(instanceType, 0);

			if (capacity <= 0)
			{
				throw new AmazonEC2Exception("Insufficient capacity.") { ErrorCode = "InsufficientInstanceCapacity" };
			}

			OnStartInstances(request, cancellationToken);

			azInstanceTypeCapacities[az][instanceType] = capacity - 1;
		}

		foreach (string instanceId in request.InstanceIds)
		{
			if (_instances[instanceId].State.Name == StateStopped.Name)
			{
				stateChanges.Add(new()
				{
					InstanceId = instanceId,
					CurrentState = StatePending,
					PreviousState = _instances[instanceId].State
				});
				_instances[instanceId].State = StatePending;
			}
		}

		return Task.FromResult(new StartInstancesResponse
		{
			StartingInstances = stateChanges,
			HttpStatusCode = HttpStatusCode.OK
		});
	}

	private Task<StopInstancesResponse> StopInstancesAsync(StopInstancesRequest request, CancellationToken cancellationToken)
	{
		List<InstanceStateChange> stateChanges = new();

		foreach (string instanceId in request.InstanceIds)
		{
			if (_instances[instanceId].State.Name == StatePending.Name)
			{
				stateChanges.Add(new()
				{
					InstanceId = instanceId,
					CurrentState = StateStopping,
					PreviousState = _instances[instanceId].State
				});
				_instances[instanceId].State = StateStopping;
			}
		}

		return Task.FromResult(new StopInstancesResponse
		{
			StoppingInstances = stateChanges,
			HttpStatusCode = HttpStatusCode.OK
		});
	}

	private async Task<ModifyInstanceAttributeResponse> ModifyInstanceAttributeAsync(ModifyInstanceAttributeRequest request, CancellationToken cancellationToken)
	{
		if (!_instances.ContainsKey(request.InstanceId))
		{
			throw new ArgumentException("Unknown instance ID " + request.InstanceId);
		}

		_instances[request.InstanceId].InstanceType = request.InstanceType;
		await OnModifyInstanceAttribute(request, cancellationToken);

		return new ModifyInstanceAttributeResponse()
		{
			ContentLength = 0,
			HttpStatusCode = HttpStatusCode.OK
		};
	}
}