// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using Amazon.Runtime;
using Amazon.Runtime.Endpoints;
using Amazon.SQS;
using Amazon.SQS.Model;

namespace HordeServer.Tests.Fleet;

#pragma warning disable CA1054

/// <summary>
/// A fake implementation of IAmazonSQS to aid testing
/// Not a fully valid implementation but enough to pass tests in Horde
/// </summary>
public sealed class FakeAmazonSqs : IAmazonSQS
{
	private readonly ConcurrentDictionary<string, List<string>> _queues = new();

	public void Dispose()
	{
	}

	public Task<SendMessageResponse> SendMessageAsync(string queueUrl, string messageBody, CancellationToken cancellationToken = default)
	{
		if (!_queues.TryGetValue(queueUrl, out List<string>? messages))
		{
			messages = new();
			_queues[queueUrl] = messages;
		}

		messages.Add(messageBody);
		return Task.FromResult(new SendMessageResponse() { MessageId = "unset" });
	}

	public Task<ReceiveMessageResponse> ReceiveMessageAsync(ReceiveMessageRequest request, CancellationToken cancellationToken = default)
	{
		if (_queues.TryGetValue(request.QueueUrl, out List<string>? messages))
		{
			return Task.FromResult(new ReceiveMessageResponse
			{
				Messages = messages.Select(body => new Message() { Body = body, MessageId = "unset" }).ToList()
			});
		}

		return Task.FromResult(new ReceiveMessageResponse());
	}

	public Task<DeleteMessageResponse> DeleteMessageAsync(DeleteMessageRequest request, CancellationToken cancellationToken = default)
	{
		return Task.FromResult(new DeleteMessageResponse());
	}

	#region Not implemented

	public ISQSPaginatorFactory Paginators { get; } = null!;
	public IClientConfig Config { get; } = null!;

	public Task<Dictionary<string, string>> GetAttributesAsync(string queueUrl)
	{
		throw new NotImplementedException();
	}

	public Task SetAttributesAsync(string queueUrl, Dictionary<string, string> attributes)
	{
		throw new NotImplementedException();
	}

	public Task<string> AuthorizeS3ToSendMessageAsync(string queueUrl, string bucket)
	{
		throw new NotImplementedException();
	}

	public Task<AddPermissionResponse> AddPermissionAsync(string queueUrl, string label, List<string> awsAccountIds, List<string> actions, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<AddPermissionResponse> AddPermissionAsync(AddPermissionRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ChangeMessageVisibilityResponse> ChangeMessageVisibilityAsync(string queueUrl, string receiptHandle, int visibilityTimeout, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ChangeMessageVisibilityResponse> ChangeMessageVisibilityAsync(ChangeMessageVisibilityRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ChangeMessageVisibilityBatchResponse> ChangeMessageVisibilityBatchAsync(string queueUrl, List<ChangeMessageVisibilityBatchRequestEntry> entries, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ChangeMessageVisibilityBatchResponse> ChangeMessageVisibilityBatchAsync(ChangeMessageVisibilityBatchRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<CreateQueueResponse> CreateQueueAsync(string queueName, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<CreateQueueResponse> CreateQueueAsync(CreateQueueRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<DeleteMessageResponse> DeleteMessageAsync(string queueUrl, string receiptHandle, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<DeleteMessageBatchResponse> DeleteMessageBatchAsync(string queueUrl, List<DeleteMessageBatchRequestEntry> entries, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<DeleteMessageBatchResponse> DeleteMessageBatchAsync(DeleteMessageBatchRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<DeleteQueueResponse> DeleteQueueAsync(string queueUrl, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<DeleteQueueResponse> DeleteQueueAsync(DeleteQueueRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<GetQueueAttributesResponse> GetQueueAttributesAsync(string queueUrl, List<string> attributeNames, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<GetQueueAttributesResponse> GetQueueAttributesAsync(GetQueueAttributesRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<GetQueueUrlResponse> GetQueueUrlAsync(string queueName, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<GetQueueUrlResponse> GetQueueUrlAsync(GetQueueUrlRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ListDeadLetterSourceQueuesResponse> ListDeadLetterSourceQueuesAsync(ListDeadLetterSourceQueuesRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ListQueuesResponse> ListQueuesAsync(string queueNamePrefix, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ListQueuesResponse> ListQueuesAsync(ListQueuesRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ListQueueTagsResponse> ListQueueTagsAsync(ListQueueTagsRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<PurgeQueueResponse> PurgeQueueAsync(string queueUrl, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<PurgeQueueResponse> PurgeQueueAsync(PurgeQueueRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<ReceiveMessageResponse> ReceiveMessageAsync(string queueUrl, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<RemovePermissionResponse> RemovePermissionAsync(string queueUrl, string label, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<RemovePermissionResponse> RemovePermissionAsync(RemovePermissionRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<SendMessageResponse> SendMessageAsync(SendMessageRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<SendMessageBatchResponse> SendMessageBatchAsync(string queueUrl, List<SendMessageBatchRequestEntry> entries, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<SendMessageBatchResponse> SendMessageBatchAsync(SendMessageBatchRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<SetQueueAttributesResponse> SetQueueAttributesAsync(string queueUrl, Dictionary<string, string> attributes, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<SetQueueAttributesResponse> SetQueueAttributesAsync(SetQueueAttributesRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<TagQueueResponse> TagQueueAsync(TagQueueRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<UntagQueueResponse> UntagQueueAsync(UntagQueueRequest request, CancellationToken cancellationToken = default)
	{
		throw new NotImplementedException();
	}

	public Task<CancelMessageMoveTaskResponse> CancelMessageMoveTaskAsync(CancelMessageMoveTaskRequest request, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task<ListMessageMoveTasksResponse> ListMessageMoveTasksAsync(ListMessageMoveTasksRequest request, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Task<StartMessageMoveTaskResponse> StartMessageMoveTaskAsync(StartMessageMoveTaskRequest request, CancellationToken cancellationToken)
	{
		throw new NotImplementedException();
	}

	public Endpoint DetermineServiceOperationEndpoint(AmazonWebServiceRequest request)
	{
		throw new NotImplementedException();
	}
	#endregion Not implemented	
}

#pragma warning restore CA1054