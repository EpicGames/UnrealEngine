// Copyright Epic Games, Inc. All Rights Reserved.

extern alias HordeAgent;
using System.Text.Json;
using System.Text.Json.Nodes;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Fleet.Providers;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

#pragma warning disable CS0618 // Type or member is obsolete

namespace HordeServer.Tests.Fleet
{
	public class FleetManagerSpy : IFleetManager
	{
		public int ExpandPoolAsyncCallCount { get; private set; }
		public int ShrinkPoolAsyncCallCount { get; private set; }
		public IReadOnlyList<IAgent> ExpandAgents => _expandAgents;
		public IReadOnlyList<IAgent> ShrinkAgents => _shrinkAgents;

		private readonly List<IAgent> _expandAgents = new();
		private readonly List<IAgent> _shrinkAgents = new();

		public Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			ExpandPoolAsyncCallCount++;
			_expandAgents.AddRange(agents);
			return Task.FromResult(new ScaleResult(FleetManagerOutcome.Success, count, 0));
		}

		public Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			ShrinkPoolAsyncCallCount++;
			_shrinkAgents.AddRange(agents);
			return Task.FromResult(new ScaleResult(FleetManagerOutcome.Success, 0, count));
		}

		public Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}
	}

	public class FakeFleetManager : IFleetManager
	{
		public const string ScaleOutMessage = "Pool scaled out by fake fleet manager";
		public const string ScaleInMessage = "Pool scaled in by fake fleet manager";

		public int AgentsAddedCount { get; private set; }
		public int AgentsRemovedCount { get; private set; }

		public ScaleResult? ForceResult { get; set; } = null;

		public Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken = default)
		{
			AgentsAddedCount += count;
			ScaleResult scaleResult = new(FleetManagerOutcome.Success, count, 0, ScaleOutMessage);
			return Task.FromResult(ForceResult ?? scaleResult);
		}

		public Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken = default)
		{
			AgentsRemovedCount += count;
			ScaleResult scaleResult = new(FleetManagerOutcome.Success, 0, count, ScaleInMessage);
			return Task.FromResult(ForceResult ?? scaleResult);
		}

		public Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken = default)
		{
			throw new NotImplementedException();
		}
	}

	public class StubFleetManagerFactory : IFleetManagerFactory
	{
		private readonly IFleetManager _fleetManager;

		public StubFleetManagerFactory(IFleetManager fleetManager)
		{
			_fleetManager = fleetManager;
		}

		public IFleetManager CreateFleetManager(FleetManagerType type, JsonObject? config = null)
		{
			return _fleetManager;
		}
	}

	[TestClass]
	public class FleetServiceTest : BuildTestSetup
	{
		readonly FleetManagerSpy _fleetManagerSpy = new();

		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			await UpdateConfigAsync(x => x.Plugins.GetComputeConfig().Pools.Clear());
		}

		[TestMethod]
		public async Task OnlyEnabledAgentsAreAutoScaledAsync()
		{
			await using FleetService service = GetFleetService(_fleetManagerSpy);
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.LeaseUtilization });
			await CreateAgentAsync(pool, true);
			await CreateAgentAsync(pool, true);
			await CreateAgentAsync(pool, false);

			PoolWithAgents poolWithAgents = (await service.GetPoolsWithAgentsAsync())[0];
			await service.ScalePoolAsync(pool, poolWithAgents.Agents, new PoolSizeResult(2, 5), CancellationToken.None);
			Assert.AreEqual(2, _fleetManagerSpy.ExpandAgents.Count);
			Assert.IsTrue(_fleetManagerSpy.ExpandAgents[0].Enabled);
			Assert.IsTrue(_fleetManagerSpy.ExpandAgents[1].Enabled);
		}

		[TestMethod]
		public async Task ScaleOutWithPendingShutdownsAsync()
		{
			await using FleetService service = GetFleetService(new FakeFleetManager());
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.LeaseUtilization });
			IAgent agent1 = await CreateAgentAsync(pool, true);
			IAgent agent2 = await CreateAgentAsync(pool, true, requestShutdown: true);

			PoolWithAgents poolWithAgents = (await service.GetPoolsWithAgentsAsync())[0];
			Assert.AreEqual(1, poolWithAgents.Agents.Count);
			ScaleResult result = await service.ScalePoolAsync(pool, poolWithAgents.Agents, new PoolSizeResult(poolWithAgents.Agents.Count, 2), CancellationToken.None);
			//			Assert.AreEqual(FleetManagerOutcome.Success, result.Outcome);
			Assert.AreEqual("Scaled out by only cancelling shutdowns", result.Message);
			Assert.AreEqual(1, result.AgentsAddedCount);

			Assert.IsFalse((await AgentService.GetAgentAsync(agent2.Id))!.RequestShutdown);
		}

		[TestMethod]
		public async Task ScaleOutWithPendingShutdownsWithStoppedAgentsAsync()
		{
			await using FleetService service = GetFleetService(new FakeFleetManager());
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.LeaseUtilization });
			IAgent agent1 = await CreateAgentAsync(pool, true);
			IAgent agent2 = await CreateAgentAsync(pool, false);
			IAgent agent3 = await CreateAgentAsync(pool, true, requestShutdown: true);

			PoolWithAgents poolWithAgents = (await service.GetPoolsWithAgentsAsync())[0];
			Assert.AreEqual(1, poolWithAgents.Agents.Count);
			ScaleResult result = await service.ScalePoolAsync(pool, poolWithAgents.Agents, new PoolSizeResult(poolWithAgents.Agents.Count, 3), CancellationToken.None);
			Assert.AreEqual(FleetManagerOutcome.Success, result.Outcome);
			Assert.AreEqual(FakeFleetManager.ScaleOutMessage, result.Message);
			Assert.AreEqual(2, result.AgentsAddedCount);

			Assert.IsFalse((await AgentService.GetAgentAsync(agent3.Id))!.RequestShutdown);
		}

		[TestMethod]
		public async Task ScaleOutCooldownAsync()
		{
			await using FleetService service = GetFleetService(_fleetManagerSpy);
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });

			// First scale-out will succeed
			ScaleResult result = await service.ScalePoolAsync(pool, new List<IAgent>(), new PoolSizeResult(0, 1), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.Success, 1, 0), result);
			Assert.AreEqual(1, _fleetManagerSpy.ExpandPoolAsyncCallCount);

			// Cannot scale-out due to cool-down
			result = await service.ScalePoolAsync(pool, new List<IAgent>(), new PoolSizeResult(0, 2), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.NoOp, 0, 0), result);
			Assert.AreEqual(1, _fleetManagerSpy.ExpandPoolAsyncCallCount);

			// Wait some time and then try again
			await Clock.AdvanceAsync(TimeSpan.FromHours(2));
			result = await service.ScalePoolAsync(pool, new List<IAgent>(), new PoolSizeResult(0, 2), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.Success, 2, 0), result);
			Assert.AreEqual(2, _fleetManagerSpy.ExpandPoolAsyncCallCount);
		}

		[TestMethod]
		public async Task ScaleInCooldownAsync()
		{
			await using FleetService service = GetFleetService(_fleetManagerSpy);
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });
			IAgent agent1 = await CreateAgentAsync(pool);
			IAgent agent2 = await CreateAgentAsync(pool);

			// First scale-in will succeed
			ScaleResult result = await service.ScalePoolAsync(pool, new List<IAgent> { agent1, agent2 }, new PoolSizeResult(2, 1), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.Success, 0, 1), result);
			Assert.AreEqual(1, _fleetManagerSpy.ShrinkPoolAsyncCallCount);

			// Cannot scale-in due to cool-down
			result = await service.ScalePoolAsync(pool, new List<IAgent> { agent1 }, new PoolSizeResult(1, 0), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.NoOp, 0, 0), result);
			Assert.AreEqual(1, _fleetManagerSpy.ShrinkPoolAsyncCallCount);

			// Wait some time and then try again
			await Clock.AdvanceAsync(TimeSpan.FromHours(2));
			result = await service.ScalePoolAsync(pool, new List<IAgent> { agent1 }, new PoolSizeResult(1, 0), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.Success, 0, 1), result);
			Assert.AreEqual(2, _fleetManagerSpy.ShrinkPoolAsyncCallCount);
		}

		[TestMethod]
		public async Task ScaleOutDuringDowntimeAsync()
		{
			await using FleetService service = GetFleetService(_fleetManagerSpy, true);
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });
			ScaleResult result = await service.ScalePoolAsync(pool, new List<IAgent>(), new PoolSizeResult(0, 1), CancellationToken.None);
			Assert.AreEqual(new ScaleResult(FleetManagerOutcome.NoOp, 0, 0), result);
			Assert.AreEqual(0, _fleetManagerSpy.ExpandPoolAsyncCallCount);
		}

		[TestMethod]
		public async Task ScaleInDuringDowntimeAsync()
		{
			await using FleetService service = GetFleetService(_fleetManagerSpy, true);
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp });
			IAgent agent1 = await CreateAgentAsync(pool);
			IAgent agent2 = await CreateAgentAsync(pool);

			await service.ScalePoolAsync(pool, new List<IAgent> { agent1, agent2 }, new PoolSizeResult(2, 1), CancellationToken.None);
			Assert.AreEqual(1, _fleetManagerSpy.ShrinkPoolAsyncCallCount);
		}

		private FleetService GetFleetService(IFleetManager fleetManager, bool isDowntimeActive = false)
		{
			ILoggerFactory loggerFactory = ServiceProvider.GetRequiredService<ILoggerFactory>();
			IOptions<ComputeServerConfig> serverSettingsOpt = ServiceProvider.GetRequiredService<IOptions<ComputeServerConfig>>();
			serverSettingsOpt.Value.FleetManagerV2 = FleetManagerType.AwsReuse;

			FleetService service = new(
				AgentService, PoolCollection, new DowntimeServiceStub(isDowntimeActive),
				new StubFleetManagerFactory(fleetManager), Clock, ServiceProvider.GetRequiredService<IEnumerable<IPoolSizeStrategyFactory>>(), serverSettingsOpt, Tracer, loggerFactory.CreateLogger<FleetService>());

			return service;
		}
	}

	[TestClass]
	public class PoolSizeStrategyFactoryTest : BuildTestSetup
	{
		private static int s_poolCount;

		[TestMethod]
		public async Task CreateJobQueueFromLegacySettingsAsync()
		{
			IPool pool1 = await CreatePoolAsync(new PoolConfig { Name = "test1", SizeStrategy = PoolSizeStrategy.JobQueue });
			Assert.AreEqual(typeof(JobQueueStrategy), FleetService.CreatePoolSizeStrategy(pool1).GetType());

			IPool pool2 = await CreatePoolAsync(new PoolConfig { Name = "test2", SizeStrategy = PoolSizeStrategy.JobQueue, JobQueueSettings = new JobQueueSettings(22, 33) });
			IPoolSizeStrategy s = FleetService.CreatePoolSizeStrategy(pool2);
			Assert.AreEqual(typeof(JobQueueStrategy), s.GetType());
			Assert.AreEqual(22.0, ((JobQueueStrategy)s).Settings.ScaleOutFactor);
			Assert.AreEqual(33.0, ((JobQueueStrategy)s).Settings.ScaleInFactor);
		}

		[TestMethod]
		public async Task CreateLeaseUtilizationFromLegacySettingsAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "test1", SizeStrategy = PoolSizeStrategy.LeaseUtilization });
			Assert.AreEqual(typeof(LeaseUtilizationStrategy), FleetService.CreatePoolSizeStrategy(pool).GetType());
		}

		[TestMethod]
		public async Task CreateJobQueueStrategyAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.JobQueue, null, "{\"ScaleOutFactor\": 100, \"ScaleInFactor\": 200}"));
			Assert.AreEqual(typeof(JobQueueStrategy), s.GetType());
			Assert.AreEqual(100.0, ((JobQueueStrategy)s).Settings.ScaleOutFactor);
			Assert.AreEqual(200.0, ((JobQueueStrategy)s).Settings.ScaleInFactor);
		}

		[TestMethod]
		public async Task CreateLeaseUtilizationStrategyAsync()
		{
			string config = "{\"SampleTimeSec\": 10, \"NumSamples\": 20, \"NumSamplesForResult\": 30}";
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, null, config));
			Assert.AreEqual(typeof(LeaseUtilizationStrategy), s.GetType());
			Assert.AreEqual(10, ((LeaseUtilizationStrategy)s).Settings.SampleTimeSec);
			Assert.AreEqual(20, ((LeaseUtilizationStrategy)s).Settings.NumSamples);
			Assert.AreEqual(30, ((LeaseUtilizationStrategy)s).Settings.NumSamplesForResult);
		}

		[TestMethod]
		public async Task CreateLeaseUtilizationAwsMetricStrategyAsync()
		{
			string config = "{\"SamplePeriodSec\": 123, \"CloudWatchNamespace\": \"myNs\"}";
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilizationAwsMetric, null, config));
			Assert.AreEqual(typeof(LeaseUtilizationAwsMetricStrategy), s.GetType());
			Assert.AreEqual(123, ((LeaseUtilizationAwsMetricStrategy)s).Settings.SamplePeriodSec);
			Assert.AreEqual("myNs", ((LeaseUtilizationAwsMetricStrategy)s).Settings.CloudWatchNamespace);
		}

		[TestMethod]
		public async Task CreateStrategyWithExtraAgentCountAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.NoOp, null, "{}", 39));
			PoolSizeResult result = await s.CalculatePoolSizeAsync(null!, new List<IAgent>());
			Assert.AreEqual(39, result.DesiredAgentCount);
		}

		[TestMethod]
		public async Task EmptyOrInvalidJsonConfigAsync()
		{
			await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, null, ""));
			await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, null, "{}"));
			await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, null, "  {} "));

			await ThrowsDerivedExceptionAsync<JsonException>(() => CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, null, "BAD_JSON")));
		}

		static async Task<bool> ThrowsDerivedExceptionAsync<T>(Func<Task> func) where T : Exception
		{
			try
			{
				await func();
				return false;
			}
			catch (T)
			{
				return true;
			}
			catch
			{
				return false;
			}
		}

		[TestMethod]
		public async Task CreateNoOpStrategyAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.NoOp, null, "{}"));
			Assert.AreEqual(typeof(NoOpPoolSizeStrategy), s.GetType());
		}

		[TestMethod]
		public async Task UnknownConfigFieldsAreIgnoredAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.JobQueue, null, "{\"BAD-PROPERTY\": 1337}"));
			Assert.AreEqual(typeof(JobQueueStrategy), s.GetType());
			Assert.AreEqual(0.25, ((JobQueueStrategy)s).Settings.ScaleOutFactor);
			Assert.AreEqual(0.9, ((JobQueueStrategy)s).Settings.ScaleInFactor);
		}

		[TestMethod]
		public async Task ConfigHandlesMixedCaseAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(new PoolSizeStrategyInfo(PoolSizeStrategy.JobQueue, null, "{\"scaleOutFactor\": 100, \"SCALEINFACTOR\": 200}"));
			Assert.AreEqual(typeof(JobQueueStrategy), s.GetType());
			Assert.AreEqual(100.0, ((JobQueueStrategy)s).Settings.ScaleOutFactor);
			Assert.AreEqual(200.0, ((JobQueueStrategy)s).Settings.ScaleInFactor);
		}

		[TestMethod]
		public async Task CreateFromEmptyStrategyListAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync();
			Assert.AreEqual(typeof(NoOpPoolSizeStrategy), s.GetType());
		}

		[TestMethod]
		public async Task ConditionSimpleAsync()
		{
			IPoolSizeStrategy s = await CreateStrategyAsync(
				new PoolSizeStrategyInfo(PoolSizeStrategy.LeaseUtilization, "false", "{}"),
				new PoolSizeStrategyInfo(PoolSizeStrategy.JobQueue, "true", "{\"ScaleOutFactor\": 100, \"ScaleInFactor\": 200}")
			);
			Assert.AreEqual(typeof(JobQueueStrategy), s.GetType());
		}

		[TestMethod]
		public void ConditionYear()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcYear == 2022"));
			Assert.IsTrue(EvalCondition(new DateTime(1900, 1, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcYear == 1900"));
			Assert.IsFalse(EvalCondition(new DateTime(2050, 5, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcYear == 3"));
		}

		[TestMethod]
		public void ConditionMonth()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcMonth == 9"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 1, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcMonth == 1"));
			Assert.IsFalse(EvalCondition(new DateTime(2022, 5, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcMonth == 3"));
		}

		[TestMethod]
		public void ConditionDay()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 1, 12, 0, 0, DateTimeKind.Utc), "timeUtcDay == 1"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 1, 31, 12, 0, 0, DateTimeKind.Utc), "timeUtcDay == 31"));
			Assert.IsFalse(EvalCondition(new DateTime(2022, 5, 5, 12, 0, 0, DateTimeKind.Utc), "timeUtcDay == 3"));
		}

		[TestMethod]
		public void ConditionDayOfWeek()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "dayOfWeek == 'monday'")); // A Monday
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "dayOfWeek == 'Monday'")); // A Monday
			Assert.IsFalse(EvalCondition(new DateTime(2022, 9, 6, 15, 0, 0, DateTimeKind.Utc), "dayOfWeek == 'monday'")); // A Tuesday

			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "timeUtcDayOfWeek == 'monday'")); // A Monday
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "timeUtcDayOfWeek == 'Monday'")); // A Monday
			Assert.IsFalse(EvalCondition(new DateTime(2022, 9, 6, 15, 0, 0, DateTimeKind.Utc), "timeUtcDayOfWeek == 'monday'")); // A Tuesday
		}

		[TestMethod]
		public void ConditionHour()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "timeUtcHour == 15"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 7, 0, 0, DateTimeKind.Utc), "timeUtcHour == 7"));
			Assert.IsFalse(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "timeUtcHour == 3"));
		}

		[TestMethod]
		public void ConditionMin()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 55, 0, DateTimeKind.Utc), "timeUtcMin == 55"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 7, 0, 0, DateTimeKind.Utc), "timeUtcMin == 0"));
			Assert.IsFalse(EvalCondition(new DateTime(2022, 9, 5, 15, 1, 0, DateTimeKind.Utc), "timeUtcMin == 2"));
		}

		[TestMethod]
		public void ConditionSec()
		{
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 0, DateTimeKind.Utc), "timeUtcSec == 0"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 7, 0, 1, DateTimeKind.Utc), "timeUtcSec == 1"));
			Assert.IsTrue(EvalCondition(new DateTime(2022, 9, 5, 7, 0, 15, DateTimeKind.Utc), "timeUtcSec > 5"));
			Assert.IsFalse(EvalCondition(new DateTime(2022, 9, 5, 15, 0, 55, DateTimeKind.Utc), "timeUtcSec == 3"));
		}

		private bool EvalCondition(DateTime now, string condition)
		{
			Clock.UtcNow = now;
			return EpicGames.Horde.Common.Condition.Parse(condition).Evaluate(FleetService.GetPropValues);
		}

		private async Task<IPoolSizeStrategy> CreateStrategyAsync(params PoolSizeStrategyInfo[] infos)
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool-" + s_poolCount++, EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, SizeStrategy = PoolSizeStrategy.NoOp, SizeStrategies = infos.ToList() });
			return FleetService.CreatePoolSizeStrategy(pool);
		}
	}

	[TestClass]
	public class FleetManagerFactoryTest : BuildTestSetup
	{
		private static int s_poolCount;

		[TestMethod]
		public async Task CreateNoOpAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.NoOp, null, "{}"));
			Assert.AreEqual(typeof(NoOpFleetManager), fm.GetType());
		}

		[TestMethod]
		public async Task CreateAwsAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.Aws, null, "{\"ImageId\": \"bogusImageId\"}"));
			Assert.AreEqual(typeof(AwsFleetManager), fm.GetType());
			Assert.AreEqual("bogusImageId", ((AwsFleetManager)fm).Settings.ImageId);
		}

		[TestMethod]
		public async Task CreateAwsReuseAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.AwsReuse, null, "{\"InstanceTypes\": [\"foo\", \"bar\"]}"));
			Assert.AreEqual(typeof(AwsReuseFleetManager), fm.GetType());
			Assert.AreEqual("foo", ((AwsReuseFleetManager)fm).Settings.InstanceTypes![0]);
			Assert.AreEqual("bar", ((AwsReuseFleetManager)fm).Settings.InstanceTypes![1]);
		}

		[TestMethod]
		public async Task CreateAwsAsgAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.AwsAsg, null, "{\"Name\": \"bogusName\"}"));
			Assert.AreEqual(typeof(AwsAsgFleetManager), fm.GetType());
			Assert.AreEqual("bogusName", ((AwsAsgFleetManager)fm).Settings.Name);
		}

		[TestMethod]
		public async Task UnknownConfigFieldsAreIgnoredAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.AwsAsg, null, "{\"Name\": \"bogusName\", \"BAD-PROPERTY\": 1337}"));
			Assert.AreEqual(typeof(AwsAsgFleetManager), fm.GetType());
			Assert.AreEqual("bogusName", ((AwsAsgFleetManager)fm).Settings.Name);
		}

		[TestMethod]
		public async Task EmptyConfigThrowsExceptionAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync(new FleetManagerInfo(FleetManagerType.AwsRecycle, null, ""));
			Assert.AreEqual(typeof(AwsRecyclingFleetManager), fm.GetType());
			Assert.IsNull(((AwsRecyclingFleetManager)fm).Settings.InstanceTypes);
		}

		[TestMethod]
		public async Task CreateFromEmptyListAsync()
		{
			IFleetManager fm = await CreateFleetManagerAsync();
			Assert.AreEqual(typeof(NoOpFleetManager), fm.GetType());
		}

		[TestMethod]
		public async Task ConditionSimpleAsync()
		{
			IFleetManager s = await CreateFleetManagerAsync(
				new FleetManagerInfo(FleetManagerType.Aws, "false", "{}"),
				new FleetManagerInfo(FleetManagerType.AwsReuse, "true", "{\"ScaleOutFactor\": 100, \"ScaleInFactor\": 200}")
			);
			Assert.AreEqual(typeof(AwsReuseFleetManager), s.GetType());
		}

		private async Task<IFleetManager> CreateFleetManagerAsync(params FleetManagerInfo[] infos)
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "testPool-" + s_poolCount++, EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0, FleetManagers = infos.ToList(), SizeStrategy = PoolSizeStrategy.NoOp });
			return FleetService.CreateFleetManager(pool);
		}
	}
}
