// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Pools;
using HordeServer.Jobs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Fleet
{
	[TestClass]
	[DoNotParallelize] // Possibly due to ConfigService being global
	public class JobQueueStrategyTest : BuildTestSetup
	{
		[TestMethod]
		[Ignore("Flaky test when run through CI / Horde")]
		public async Task GetPoolQueueSizesAsync()
		{
			(JobQueueStrategy strategy, _, IPool pool, _) = await SetUpJobsAsync(1, 5);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(strategy.Settings.ReadyTimeThresholdSec) + TimeSpan.FromSeconds(5));
			Dictionary<PoolId, int> poolQueueSizes = await strategy.GetPoolQueueSizesAsync(Clock.UtcNow - TimeSpan.FromHours(2));
			Assert.AreEqual(1, poolQueueSizes.Count);
			Assert.AreEqual(5, poolQueueSizes[pool.Id]);
		}

		[TestMethod]
		[Ignore("Flaky test when run through CI / Horde")]
		public async Task DowntimeActiveAsync()
		{
			(JobQueueStrategy strategy, _, IPool pool, _) = await SetUpJobsAsync(1, 5, isDowntimeActive: true);
			await Clock.AdvanceAsync(TimeSpan.FromSeconds(strategy.Settings.ReadyTimeThresholdSec) + TimeSpan.FromSeconds(5));
			Dictionary<PoolId, int> poolQueueSizes = await strategy.GetPoolQueueSizesAsync(Clock.UtcNow - TimeSpan.FromHours(2));
			Assert.AreEqual(1, poolQueueSizes.Count);
			Assert.AreEqual(0, poolQueueSizes[pool.Id]);
		}

		[TestMethod]
		public async Task EmptyJobQueueAsync()
		{
			await AssertAgentCountAsync(0, -1, false);
		}

		[TestMethod]
		public async Task EmptyJobQueueWithLargePoolAsync()
		{
			await AssertAgentCountAsync(0, -2, false, 20);
		}

		[TestMethod]
		public async Task NoAgentsInPoolAsync()
		{
			await AssertAgentCountAsync(1, 1, true, 0);
		}

		[TestMethod]
		public async Task BatchesNotWaitingLongEnoughAsync()
		{
			await AssertAgentCountAsync(3, -1, false);
		}

		[TestMethod]
		public async Task NumQueuedJobs1Async()
		{
			await AssertAgentCountAsync(1, 1, true);
		}

		[TestMethod]
		public async Task NumQueuedJobs3Async()
		{
			await AssertAgentCountAsync(3, 1, true);
		}

		[TestMethod]
		public async Task NumQueuedJobs6Async()
		{
			await AssertAgentCountAsync(6, 2);
		}

		[TestMethod]
		public async Task NumQueuedJobs25Async()
		{
			await AssertAgentCountAsync(25, 7);
		}

		public async Task AssertAgentCountAsync(int numBatchesReady, int expectedAgentDelta, bool waitedBeyondThreshold = true, int numAgents = 8, bool isDowntimeActive = false)
		{
			(JobQueueStrategy strategy, _, IPool pool, List<IAgent> agents) = await SetUpJobsAsync(1, numBatchesReady, numAgents, isDowntimeActive);
			TimeSpan timeToWait = waitedBeyondThreshold
				? TimeSpan.FromSeconds(strategy.Settings.ReadyTimeThresholdSec) + TimeSpan.FromSeconds(5)
				: TimeSpan.FromSeconds(15);

			await Clock.AdvanceAsync(timeToWait);

			PoolSizeResult result = await strategy.CalculatePoolSizeAsync(pool, agents);
			Assert.AreEqual(agents.Count + expectedAgentDelta, result.DesiredAgentCount);
		}

		private static int s_uniqueId = 0;

		/// <summary>
		/// Set up a fixture for job queue tests, ensuring a certain number of job batches are in running or waiting state
		/// </summary>
		/// <param name="numBatchesRunning">Num of job batches that should be in state running</param>
		/// <param name="numBatchesReady">Num of job batches that should be in state waiting</param>
		/// <param name="numAgents"></param>
		/// <param name="isDowntimeActive"></param>
		private async Task<(JobQueueStrategy, PoolSizeResult, IPool, List<IAgent> agents)> SetUpJobsAsync(int numBatchesRunning, int numBatchesReady, int numAgents = 8, bool isDowntimeActive = false)
		{
			await SetConfigAsync(new GlobalConfig());

			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "bogusPool" + ++s_uniqueId, EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 });
			List<IAgent> agents = new();
			for (int i = 0; i < numAgents; i++)
			{
				agents.Add(await CreateAgentAsync(pool));
			}

			PoolSizeResult poolSize = new(agents.Count, agents.Count, null);

			string agentTypeName1 = "bogusAgentType" + ++s_uniqueId;
			Dictionary<string, AgentConfig> agentTypes = new() { { agentTypeName1, new() { Pool = pool.Id } }, };

			StreamConfig streamConfig = new StreamConfig { Id = new StreamId("ue5"), Name = "//UE5/Main", AgentTypes = agentTypes };

			ProjectConfig projectConfig = new ProjectConfig();
			projectConfig.Id = new ProjectId("ue5");
			projectConfig.Streams.Add(streamConfig);

			await UpdateConfigAsync(config => config.Plugins.GetBuildConfig().Projects = new List<ProjectConfig> { projectConfig });

			string nodeForAgentType1 = "bogusNodeOnAgentType" + ++s_uniqueId;
			IGraph graph = await GraphCollection.AppendAsync(null, new()
			{
				new NewGroup(agentTypeName1, new List<NewNode>
				{
					new (nodeForAgentType1),
				})
			});

			for (int i = 0; i < numBatchesRunning; i++)
			{
				IJob job = await AddPlaceholderJobAsync(graph, streamConfig.Id, nodeForAgentType1);
				await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Running, null);
			}

			for (int i = 0; i < numBatchesReady; i++)
			{
				IJob job = await AddPlaceholderJobAsync(graph, streamConfig.Id, nodeForAgentType1);
				await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Ready, null);
			}

			return (new JobQueueStrategy(JobCollection, GraphCollection, Clock, Cache, isDowntimeActive, ServiceProvider.GetRequiredService<IOptionsMonitor<BuildConfig>>()), poolSize, pool, agents);
		}

		private async Task<IJob> AddPlaceholderJobAsync(IGraph graph, StreamId streamId, string nodeNameToExecute)
		{
			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add($"-Target={nodeNameToExecute}");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), streamId,
				new TemplateId("bogusTemplateRefId"), ContentHash.Empty, graph, "bogusJobName",
				CommitIdWithOrder.FromPerforceChange(1000), CommitIdWithOrder.FromPerforceChange(1000), options);

			return job;
		}
	}
}
