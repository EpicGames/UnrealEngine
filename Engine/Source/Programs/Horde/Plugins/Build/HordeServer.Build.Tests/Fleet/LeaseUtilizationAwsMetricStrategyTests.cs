// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.CloudWatch.Model;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Streams;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Tests.Fleet
{
	[TestClass]
	public class LeaseUtilizationAwsMetricStrategyTest : BuildTestSetup
	{
		private readonly IPool _pool;
		private readonly LeaseUtilizationAwsMetricSettings _settings = new(60, "myNamespace");
		private readonly FakeAmazonCloudWatch _cloudWatch = new();
		private readonly LeaseUtilizationAwsMetricStrategy _strategy;
		private readonly NullLogger<LeaseUtilizationAwsMetricStrategy> _logger = NullLogger<LeaseUtilizationAwsMetricStrategy>.Instance;

		public LeaseUtilizationAwsMetricStrategyTest()
		{
			_pool = CreatePoolAsync(new() { Name = "my-pool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 }).Result;
			_strategy = new(LeaseCollection, _cloudWatch.Get(), _settings, Clock, _logger);
		}

		private async Task<List<IAgent>> CreateAgentsAsync(IPool pool, int numAgents)
		{
			List<IAgent> agents = new(numAgents);
			for (int i = 0; i < numAgents; i++)
			{
				agents.Add(await CreateAgentAsync(pool));
			}

			return agents;
		}

		[TestMethod]
		public async Task MetadataAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			List<MetricDatum> datums = _cloudWatch.GetMetricData("myNamespace");
			Assert.AreEqual(1, datums.Count);
			Assert.AreEqual("LeaseUtilizationPercentage", datums[0].MetricName);
			Assert.AreEqual(1, datums[0].Dimensions.Count);
			Assert.AreEqual("Pool", datums[0].Dimensions[0].Name);
			Assert.AreEqual("my-pool", datums[0].Dimensions[0].Value);
		}

		[TestMethod]
		public async Task UtilizationZeroAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(0.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task NoAgentsAsync()
		{
			// Arrange
			await CreateAgentsAsync(_pool, 4);

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, new List<IAgent>());

			// Assert
			Assert.AreEqual(0.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task UtilizationHalfAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[1], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(50.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task UtilizationFullAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[1], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[2], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(agents[3], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(100.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task AgentMissingInParametersAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[1], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[2], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(agents[3], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));

			// Act
			List<IAgent> oneMissingAgent = agents.Take(3).ToList();
			await _strategy.CalculatePoolSizeAsync(_pool, oneMissingAgent);

			// Assert
			Assert.AreEqual(100.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task IgnoreOtherPoolsAsync()
		{
			// Arrange
			IPool otherPool = await CreatePoolAsync(new() { Name = "other-pool", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 });
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			List<IAgent> otherAgents = await CreateAgentsAsync(otherPool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AddPlaceholderLeaseAsync(agents[1], _pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(otherAgents[0], otherPool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(otherAgents[1], otherPool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(50.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task MoreLeasesThanAgentsAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(5));
			await AddPlaceholderLeaseAsync(agents[0], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(10));
			await AddPlaceholderLeaseAsync(agents[0], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(15));
			await AddPlaceholderLeaseAsync(agents[1], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(5));
			await AddPlaceholderLeaseAsync(agents[1], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(15));
			await AddPlaceholderLeaseAsync(agents[2], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(15));
			await AddPlaceholderLeaseAsync(agents[3], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(15));

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(100.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		[TestMethod]
		public async Task LeasesInProgressAsync()
		{
			// Arrange
			List<IAgent> agents = await CreateAgentsAsync(_pool, 4);
			await AddPlaceholderLeaseAsync(agents[0], _pool, OneMinuteAgo(), null);
			await AddPlaceholderLeaseAsync(agents[1], _pool, OneMinuteAgo(), null);
			await AddPlaceholderLeaseAsync(agents[2], _pool, OneMinuteAgo(), TimeSpan.FromSeconds(15));

			// Act
			await _strategy.CalculatePoolSizeAsync(_pool, agents);

			// Assert
			Assert.AreEqual(75.0, _cloudWatch.GetMetricData("myNamespace")[0].Value, 0.0001);
		}

		private DateTime OneMinuteAgo()
		{
			return Clock.UtcNow - TimeSpan.FromSeconds(60);
		}

		enum LeaseType
		{
			ExecuteJob,
			Compute
		}

		private async Task<ILease> AddPlaceholderLeaseAsync(IAgent agent, IPool pool, DateTime startTime, TimeSpan? duration, LeaseType leaseType = LeaseType.ExecuteJob)
		{
			Assert.IsNotNull(agent.SessionId);

			byte[] payload;
			ExecuteJobTask executeJobTask = new();
			executeJobTask.JobName = "placeholderJobName";
			payload = Any.Pack(executeJobTask).ToByteArray();
			PoolId? poolId = pool.Id;

			if (leaseType == LeaseType.Compute)
			{
				ComputeTask computeTask = new();
				computeTask.Nonce = ByteString.CopyFromUtf8("test-nonce");
				computeTask.Key = ByteString.CopyFromUtf8("test-key");
				payload = Any.Pack(computeTask).ToByteArray();
				poolId = null;
			}

			ILease lease = await LeaseCollection.AddAsync(new LeaseId(BinaryIdUtils.CreateNew()), null, "placeholderLease", agent.Id, agent.SessionId!.Value, new StreamId("placeholderStream"), poolId, null, startTime, payload);
			if (duration != null)
			{
				bool wasModified = await LeaseCollection.TrySetOutcomeAsync(lease.Id, startTime + duration.Value, LeaseOutcome.Success, LeaseOutcomeReason.None, null, null);
				Assert.IsTrue(wasModified);
			}

			return lease;
		}
	}
}