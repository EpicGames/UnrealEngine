// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Streams;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Utilities;

namespace HordeServer.Tests.Fleet
{
	[TestClass]
	public class LeaseUtilizationStrategyTest : BuildTestSetup
	{
		private IAgent _agent1 = null!;
		private IAgent _agent2 = null!;
		private IAgent _agent3 = null!;
		private IAgent _agent4 = null!;
		private List<IAgent> _poolAgents = null!;

		public void CreateAgents(IPool pool)
		{
			_agent1 = CreateAgentAsync(pool).Result;
			_agent2 = CreateAgentAsync(pool).Result;
			_agent3 = CreateAgentAsync(pool).Result;
			_agent4 = CreateAgentAsync(pool).Result;
			_poolAgents = new() { _agent1, _agent2, _agent3, _agent4 };
		}

		[TestMethod]
		public async Task UtilizationFullAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "AutoscalePool1", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 });
			CreateAgents(pool);

			await AddPlaceholderLeaseAsync(_agent1, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent2, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent3, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent4, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120), LeaseType.Compute);
			await AssertPoolSizeAsync(pool, 4);
		}

		[TestMethod]
		public async Task UtilizationHalfAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "AutoscalePool1", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 });
			CreateAgents(pool);

			await AddPlaceholderLeaseAsync(_agent1, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent2, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AssertPoolSizeAsync(pool, 2);
		}

		[TestMethod]
		public async Task UtilizationZeroAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "AutoscalePool1", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 0 });
			CreateAgents(pool);

			await AssertPoolSizeAsync(pool, 0);
		}

		[TestMethod]
		public async Task ReserveAgentsAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "AutoscalePool1", EnableAutoscaling = true, MinAgents = 0, NumReserveAgents = 5 });
			CreateAgents(pool);

			await AddPlaceholderLeaseAsync(_agent1, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent2, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent3, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));
			await AddPlaceholderLeaseAsync(_agent4, pool, Clock.UtcNow - TimeSpan.FromMinutes(120), TimeSpan.FromMinutes(120));

			// Full utilization should mean all agents plus the reserve agents
			await AssertPoolSizeAsync(pool, 9);
		}

		[TestMethod]
		public async Task MinAgentsAsync()
		{
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "AutoscalePool1", EnableAutoscaling = true, MinAgents = 2, NumReserveAgents = 0 });
			CreateAgents(pool);

			// Even with no utilization, expect at least the min number of agents
			await AssertPoolSizeAsync(pool, 2);
		}

		private async Task AssertPoolSizeAsync(IPool pool, int expectedNumAgents)
		{
			IPoolSizeStrategy strategy = FleetService.CreatePoolSizeStrategy(pool);
			PoolSizeResult output = await strategy.CalculatePoolSizeAsync(pool, _poolAgents);
			Assert.AreEqual(expectedNumAgents, output.DesiredAgentCount);
		}

		enum LeaseType
		{
			ExecuteJob,
			Compute
		}

		private async Task<ILease> AddPlaceholderLeaseAsync(IAgent agent, IPool pool, DateTime startTime, TimeSpan duration, LeaseType leaseType = LeaseType.ExecuteJob)
		{
			Assert.IsNotNull(agent.SessionId);

			byte[] payload;
			ExecuteJobTask executeJobTask = new();
			executeJobTask.JobName = "placeholderJobName";
			payload = Any.Pack(executeJobTask).ToByteArray();

			if (leaseType == LeaseType.Compute)
			{
				ComputeTask computeTask = new();
				computeTask.Nonce = ByteString.CopyFromUtf8("test-nonce");
				computeTask.Key = ByteString.CopyFromUtf8("test-key");
				payload = Any.Pack(computeTask).ToByteArray();
			}

			ILease lease = await LeaseCollection.AddAsync(new LeaseId(BinaryIdUtils.CreateNew()), null, "placeholderLease", agent.Id, agent.SessionId!.Value, new StreamId("placeholderStream"), pool.Id, null, startTime, payload);
			bool wasModified = await LeaseCollection.TrySetOutcomeAsync(lease.Id, startTime + duration, LeaseOutcome.Success, LeaseOutcomeReason.None, null, null);
			Assert.IsTrue(wasModified);
			return lease;
		}
	}
}