<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer.Tests</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Microsoft.AspNetCore.Mvc.Testing" Version="10.0.6" />
    <PackageReference Include="Moq" Version="4.20.72" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\HordeServer.Build\HordeServer.Build.csproj" />
    <ProjectReference Include="..\..\Compute\HordeServer.Compute\HordeServer.Compute.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Tests.Shared\HordeServer.Tests.Shared.csproj" />
    <ProjectReference Include="..\..\..\HordeAgent\HordeAgent.csproj">
      <Aliases>HordeAgent</Aliases>
    </ProjectReference>
    <ProjectReference Include="..\..\..\Drivers\JobDriver\JobDriver.csproj">
      <Aliases>JobDriver</Aliases>
    </ProjectReference>
  </ItemGroup>

  <ItemGroup>
    <Using Include="Microsoft.VisualStudio.TestTools.UnitTesting" />
  </ItemGroup>

</Project>
