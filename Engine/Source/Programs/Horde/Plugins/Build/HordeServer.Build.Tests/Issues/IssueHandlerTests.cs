// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Issues.Handlers;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;

namespace HordeServer.Tests.Issues
{
	[TestClass]
	public class IssueHandlerTests
	{
		private class IssueHandlerTestData
		{
			public IssueHandlerTestData(List<LogEvent> logEvents, List<IssueEvent> issueEvents) =>
			(LogEvents_ToDebug, IssueEvents) = (logEvents, issueEvents);

			public List<LogEvent> LogEvents_ToDebug { get; init; } // For debug purposes only
			public List<IssueEvent> IssueEvents { get; init; }
		}

		// Test a single IssueHandler (issue handler priority is irrelevant in such a test)
		static List<IssueEventGroup> TestSingleIssueHandler(IssueHandler issueHandler,string[] jsonStructuredLogs)
		{
			IssueHandlerTestData issueHandlerParsedTestData = ParseJSONStructuredLogs(jsonStructuredLogs);
			foreach (IssueEvent issueEvent in issueHandlerParsedTestData.IssueEvents)
			{
				issueHandler.HandleEvent(issueEvent);
			}
			List<IssueEventGroup> groupedIssues = issueHandler.GetIssues().ToList();
			return groupedIssues;
		}

		// Parse structured logs to create IssueEvents (and LogEvents for debug purposes)
		static IssueHandlerTestData ParseJSONStructuredLogs(string[] jsonStructuredLogs)
		{
			List<LogEvent> logEvents = new List<LogEvent>(); // For debug purposes only
			List<IssueEvent> issueEvents = new List<IssueEvent>();
			foreach (string jsonStructuredLog in jsonStructuredLogs)
			{
				byte[] byteArrayStructuredLog = Encoding.UTF8.GetBytes(jsonStructuredLog);

				logEvents.Add(LogEvent.Read(byteArrayStructuredLog)); // For debug purposes only

				JsonLogEvent jsonLogEventFromStructuredLog;
				Assert.IsTrue(JsonLogEvent.TryParse(byteArrayStructuredLog, out jsonLogEventFromStructuredLog));

				List<JsonLogEvent> listOfJsonLofEvents = new List<JsonLogEvent>();
				listOfJsonLofEvents.Add(jsonLogEventFromStructuredLog);
				issueEvents.Add(new IssueEvent(jsonLogEventFromStructuredLog.LineIndex, jsonLogEventFromStructuredLog.Level, jsonLogEventFromStructuredLog.EventId, listOfJsonLofEvents));
			}

			return new IssueHandlerTestData(logEvents, issueEvents);
		}

		[TestMethod]
		public void LocalizationIssueHandler()
		{
			// Logs
			// 1) Taken from an actual JSON log file
			// 2) Data Obfuscated (modified any data to a plausible yet fake data)
			// 3) Formatted for the test by transforming any " into \"
			string[] jsonStructuredLogs =
			{
				"{ \"time\":\"2025-01-01T01:20:30.990Z\",\"level\":\"Warning\",\"message\":\"LogGatherTextFromAssetsCommandlet: Warning: Package '/Path/To/File1' and '/DifferentPathTo/File2' have the same localization ID (ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEF). Please reset one of these (Asset Localization -> Reset Localization ID) to avoid conflicts.\",\"format\":\"{_channel}: {_severity}: Package '{file}' and '{conflictFile}' have the same localization ID ({locKey}). Please reset one of these (Asset Localization -> Reset Localization ID) to avoid conflicts.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogGatherTextFromAssetsCommandlet\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"id\":304,\"file\":\"/Path/To/File1\",\"conflictFile\":\"/DifferentPathTo/File2\",\"locKey\":\"ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEF\"} }",
				"{ \"time\":\"2025-01-01T01:20:30.991Z\",\"level\":\"Warning\",\"message\":\"LogGatherTextFromAssetsCommandlet: Warning: Package '/Path/To/File3' and '/DifferentPathTo/File4' have the same localization ID (ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEG). Please reset one of these (Asset Localization -> Reset Localization ID) to avoid conflicts.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogGatherTextFromAssetsCommandlet\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"id\":304,\"file\":\"/Path/To/File3\",\"conflictFile\":\"/DifferentPathTo/File4\",\"locKey\":\"ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEG\"},\"format\":\"{_channel}: {_severity}: Package '{file}' and '{conflictFile}' have the same localization ID ({locKey}). Please reset one of these (Asset Localization -> Reset Localization ID) to avoid conflicts.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.992Z\",\"level\":\"Warning\",\"message\":\"LogLocTextHelper: Warning: Text conflict from LOCTEXT macro for namespace 'NamespaceX' and key 'KeyY'. First entry is Plugins/Path/To/File5.cpp(50):'Text from line 50'. Conflicting entry is Plugins/Path/To/File5.cpp(51):'Text from line 51'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"from\":\" from LOCTEXT\",\"locNamespace\":\"NamespaceX\",\"locKey\":\"KeyY\",\"locKeyMetaData\":\"\",\"location\":\"Plugins/Path/To/File5.cpp(50)\",\"text\":\"Text from line 50\",\"textMetaData\":\"\",\"conflictLocation\":\"Plugins/Path/To/File5.cpp(51)\",\"conflictText\":\"Text from line 51\",\"conflictTextMetaData\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Text conflict{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. First entry is {location}:'{text}'{textMetaData}. Conflicting entry is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.993Z\",\"level\":\"Warning\",\"message\":\"LogLocTextHelper: Warning: Text conflict for namespace 'NamespaceA' and key 'KeyA'. Main manifest entry is 'Text from manifest'. Conflict from manifest dependency is Path/To/ManifestFile.manifest(10):'Conflict manifest text'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"locNamespace\":\"NamespaceA\",\"locKey\":\"KeyA\",\"locKeyMetaData\":\"\",\"location\":\"\",\"text\":\"Text from manifest\",\"textMetaData\":\"\",\"conflictLocation\":\"Path/To/ManifestFile.manifest(10)\",\"conflictText\":\"Conflict manifest text\",\"conflictTextMetaData\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Text conflict for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. Main manifest entry is '{text}'{textMetaData}. Conflict from manifest dependency is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.994Z\",\"level\":\"Warning\",\"message\":\"LogLocTextHelper: Warning: Text conflict from LOCTEXT macro for namespace 'NamespaceX' and key 'KeyY'. First entry is Plugins/Path/To/Asset.SubComponent.Whatever.Else:'Text from asset component'. Conflicting entry is Plugins/Path/To/File5.cpp(51):'Text from line 51'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"from\":\" from LOCTEXT\",\"locNamespace\":\"NamespaceX\",\"locKey\":\"KeyY\",\"locKeyMetaData\":\"\",\"location\":\"Plugins/Path/To/Asset.Subcomponent.Whatever.Else\",\"text\":\"Text from asset component\",\"textMetaData\":\"\",\"conflictLocation\":\"Plugins/Path/To/File5.cpp(51)\",\"conflictText\":\"Text from line 51\",\"conflictTextMetaData\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Text conflict{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. First entry is {location}:'{text}'{textMetaData}. Conflicting entry is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.995Z\",\"level\":\"Warning\",\"message\":\"LogLocTextHelper: Warning: Broken Rich Text Tag detected in a translation. An unbalanced tag (a complete/incomplet opening rich text tag (i.e. <TagName>) with an incomplet/complete closing tag(</>)) was detected in a translation but not in its source text. Find the problematic tag in the translation and fix the translation to remove this warning. Translation File:'D:/Path/To/PluginX/Content/Localization/LocTargetX/it/LocTargetX.archive' Namespace And Key:'NamespaceB,KeyB' Translation Text To Fix:'Text with <Broken> Rich Text Tags />.'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"cultudeCode\":\"it\",\"locNamespace\":\"NamespaceB\",\"locKey\":\"KeyB\",\"text\":\"Text with <Correct>Rich Text Tags</>.\",\"conflictFile\":\"D:/Path/To/PluginX/Content/Localization/LocTargetX/it/LocTargetX.archive\",\"conflictText\":\"Text with <Broken>Rich Text Tags</.\",\"id\":304},\"format\":\"{_channel}: {_severity}: Broken Rich Text Tag detected in a translation. An unbalanced tag (a complete/incomplet opening rich text tag (i.e. <TagName>) with an incomplet/complete closing tag(</>)) was detected in the translation but not in the source text. Find the problematic tag in the translation and fix the translation to remove this warning. Translation File:'{conflictFile}' Namespace And Key:'{locNamespace},{locKey}' Translation Text To Fix:'{conflictText}'.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.996Z\",\"level\":\"Error\",\"message\":\"LogLocTextHelper: Error: /Path/To/File6.cpp(60): Failed to add text from LOCTEXT for namespace 'NamespaceC' and key 'KeyC' with source 'Text from File6'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"location\":\"/Path/To/File6.cpp(60)\",\"from\":\" from LOCTEXT\",\"locNamespace\":\"NamespaceC\",\"locKey\":\"KeyC\",\"locKeyMetaData\":\"\",\"text\":\"Text from File6\",\"textMetaData\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: {location}: Failed to add text{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.997Z\",\"level\":\"Error\",\"message\":\"LogLocTextHelper: Error: Failed to add text for namespace 'NamespaceA' and key 'KeyA' with source 'TextA'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"locNamespace\":\"NamespaceA\",\"locKey\":\"KeyA\",\"locKeyMetaData\":\"\",\"text\":\"TextA\",\"textMetaData\":\"\",\"forPlatform\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Failed to add text for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData}{forPlatform}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.998Z\",\"level\":\"Error\",\"message\":\"LogLocTextHelper: Error: Failed to add text for namespace 'NamespaceB' and key 'KeyB' with source 'TextB'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"locNamespace\":\"NamespaceB\",\"locKey\":\"KeyB\",\"locKeyMetaData\":\"\",\"text\":\"TextB\",\"textMetaData\":\"\",\"forPlatform\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Failed to add text for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData} with source '{text}'{textMetaData}{forPlatform}.\"}",
				"{ \"time\":\"2025-01-01T01:20:30.999Z\",\"level\":\"Warning\",\"message\":\"LogLocTextHelper: Warning: Text conflict for namespace '' and key 'ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEH'. First entry is /SamePlugin/Scripts/SubFolder/AssetName.AssetName.Sub.Message.ContextualMessages(0).ContextualMessages.Messages(0).Messages.Message:'Text in first asset.'. Conflicting entry is /SamePlugin/Scripts/SubFolder/CopiedAsset.CopiedAsset.Sub.Message.ContextualMessages(0).ContextualMessages.Messages(0).Messages.Message:'Text in second asset.'.\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogLocTextHelper\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"from\":\"\",\"location\":\"/SamePlugin/Scripts/SubFolder/AssetName.AssetName.Sub.Message.ContextualMessages(0).ContextualMessages.Messages(0).Messages.Message\",\"locNamespace\":\"\",\"locKey\":\"ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEH\",\"locKeyMetaData\":\"\",\"text\":\"Text in first asset.\",\"textMetaData\":\"\",\"conflictLocation\":\"/SamePlugin/Scripts/SubFolder/CopiedAsset.CopiedAsset.Sub.Message.ContextualMessages(0).ContextualMessages.Messages(0).Messages.Message\",\"conflictText\":\"Text in second asset.\",\"conflictTextMetaData\":\"\",\"id\":304},\"format\":\"{_channel}: {_severity}: Text conflict{from} for namespace '{locNamespace}' and key '{locKey}'{locKeyMetaData}. First entry is {location}:'{text}'{textMetaData}. Conflicting entry is {conflictLocation}:'{conflictText}'{conflictTextMetaData}.\"}", // This "location" created an exception in the code at one point
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Localize ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());
			LocalizationIssueHandler localizationIssueHandler = new LocalizationIssueHandler(context);
			List<IssueEventGroup> groupedIssues = TestSingleIssueHandler(localizationIssueHandler, jsonStructuredLogs);

			Assert.IsTrue(groupedIssues.Count == 9);

			Assert.IsTrue(groupedIssues[0].Keys.Count == 3);
			Assert.IsTrue(groupedIssues[0].Keys.Contains(new IssueKey("File1", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[0].Keys.Contains(new IssueKey("File2", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[0].Keys.Contains(new IssueKey(",ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEF", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[0].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[0].Events.Count == 1);
			Assert.IsTrue(groupedIssues[0].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[0].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[0].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[0].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[0].Type == "Localization");

			Assert.IsTrue(groupedIssues[1].Keys.Count == 3);
			Assert.IsTrue(groupedIssues[1].Keys.Contains(new IssueKey("File3", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[1].Keys.Contains(new IssueKey("File4", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[1].Keys.Contains(new IssueKey(",ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEG", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[1].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[1].Events.Count == 1);
			Assert.IsTrue(groupedIssues[1].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[1].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[1].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[1].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[1].Type == "Localization");

			Assert.IsTrue(groupedIssues[2].Keys.Count == 2);
			Assert.IsTrue(groupedIssues[2].Keys.Contains(new IssueKey("File5", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[2].Keys.Contains(new IssueKey("NamespaceX,KeyY", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[2].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[2].Events.Count == 1);
			Assert.IsTrue(groupedIssues[2].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[2].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[2].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[2].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[2].Type == "Localization");

			Assert.IsTrue(groupedIssues[3].Keys.Count == 2);
			Assert.IsTrue(groupedIssues[3].Keys.Contains(new IssueKey("ManifestFile", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[3].Keys.Contains(new IssueKey("NamespaceA,KeyA", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[3].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[3].Events.Count == 1);
			Assert.IsTrue(groupedIssues[3].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[3].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[3].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[3].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[3].Type == "Localization");

			Assert.IsTrue(groupedIssues[4].Keys.Count == 3);
			Assert.IsTrue(groupedIssues[4].Keys.Contains(new IssueKey("Asset", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[4].Keys.Contains(new IssueKey("File5", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[4].Keys.Contains(new IssueKey("NamespaceX,KeyY", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[4].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[4].Events.Count == 1);
			Assert.IsTrue(groupedIssues[4].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[4].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[4].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[4].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[4].Type == "Localization");

			Assert.IsTrue(groupedIssues[5].Keys.Count == 2);
			Assert.IsTrue(groupedIssues[5].Keys.Contains(new IssueKey("LocTargetX", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[5].Keys.Contains(new IssueKey("NamespaceB,KeyB", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[5].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[5].Events.Count == 1);
			Assert.IsTrue(groupedIssues[5].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[5].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[5].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[5].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[5].Type == "Localization");

			Assert.IsTrue(groupedIssues[6].Keys.Count == 2);
			Assert.IsTrue(groupedIssues[6].Keys.Contains(new IssueKey("File6", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[6].Keys.Contains(new IssueKey("NamespaceC,KeyC", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[6].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[6].Events.Count == 1);
			Assert.IsTrue(groupedIssues[6].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[6].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Error);
			Assert.IsTrue(groupedIssues[6].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[6].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[6].Type == "Localization");

			Assert.IsTrue(groupedIssues[7].Keys.Count == 2);
			Assert.IsTrue(groupedIssues[7].Keys.Contains(new IssueKey("NamespaceA,KeyA", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[7].Keys.Contains(new IssueKey("NamespaceB,KeyB", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[7].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[7].Events.Count == 2);
			Assert.IsTrue(groupedIssues[7].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[7].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Error);
			Assert.IsTrue(groupedIssues[7].Events[1].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[7].Events[1].Severity == Microsoft.Extensions.Logging.LogLevel.Error);
			Assert.IsTrue(groupedIssues[7].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[7].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[7].Type == "Localization");

			Assert.IsTrue(groupedIssues[8].Keys.Count == 3);
			Assert.IsTrue(groupedIssues[8].Keys.Contains(new IssueKey("AssetName", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[8].Keys.Contains(new IssueKey("CopiedAsset", IssueKeyType.File)));
			Assert.IsTrue(groupedIssues[8].Keys.Contains(new IssueKey(",ABCDEFGHIJKLMNOPQRSTUVWXYZABCDEH", IssueKeyType.None)));
			Assert.IsTrue(groupedIssues[8].ChangeFilter == "...");
			Assert.IsTrue(groupedIssues[8].Events.Count == 1);
			Assert.IsTrue(groupedIssues[8].Events[0].EventId == KnownLogEvents.Engine_Localization);
			Assert.IsTrue(groupedIssues[8].Events[0].Severity == Microsoft.Extensions.Logging.LogLevel.Warning);
			Assert.IsTrue(groupedIssues[8].Metadata.Count == 2);
			Assert.IsTrue(groupedIssues[8].SummaryTemplate == "{Meta:Node}: Localization {Severity} in {Meta:Stream}");
			Assert.IsTrue(groupedIssues[8].Type == "Localization");
		}

		[TestMethod]
		public void ShaderIssueHandler_GroupsByMaterialWhenPropertyPresent()
		{
			// These are shader warnings for the same material failing to compile on different platforms/shaders.
			// When the "material" property is present (set by ShaderEventMatcher), all warnings for the same
			// material should be keyed by the material path, enabling them to be grouped together.
			string[] jsonStructuredLogs =
			{
				// Shader warning with material property - will be keyed by material path
				"{ \"time\":\"2025-01-01T01:20:30.990Z\",\"level\":\"Warning\",\"message\":\"LogShaderCompilers: Warning: Failed to compile Material /MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial for platform SF_PS5\",\"format\":\"{_channel}: {_severity}: {message}\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogShaderCompilers\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"id\":310,\"file\":\"D:/build/Engine/Shaders/Private/LocalVertexFactory.ush\",\"material\":\"/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial\"} }",
				// Same material, different platform - should have same key
				"{ \"time\":\"2025-01-01T01:20:30.991Z\",\"level\":\"Warning\",\"message\":\"LogShaderCompilers: Warning: Failed to compile Material /MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial for platform SF_VULKAN_SM6\",\"format\":\"{_channel}: {_severity}: {message}\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogShaderCompilers\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"id\":310,\"file\":\"D:/build/Engine/Shaders/Private/MaterialTemplate.ush\",\"material\":\"/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial\"} }",
				// Different material - should have different key
				"{ \"time\":\"2025-01-01T01:20:30.992Z\",\"level\":\"Warning\",\"message\":\"LogShaderCompilers: Warning: Failed to compile Material /Engine/EngineMaterials/LineSetComponentMaterial.LineSetComponentMaterial for platform PCD3D_SM6\",\"format\":\"{_channel}: {_severity}: {message}\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogShaderCompilers\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Warning\"},\"id\":310,\"file\":\"D:/build/Engine/Shaders/Private/LocalVertexFactory.ush\",\"material\":\"/Engine/EngineMaterials/LineSetComponentMaterial.LineSetComponentMaterial\"} }",
			};

			ShaderIssueHandler shaderIssueHandler = new ShaderIssueHandler();
			List<IssueEventGroup> groupedIssues = TestSingleIssueHandler(shaderIssueHandler, jsonStructuredLogs);

			// ShaderIssueHandler creates separate issues but keys them by material path when property is present
			Assert.AreEqual(3, groupedIssues.Count);

			// First two issues have the same material key (PointSetOverlaidComponentMaterial)
			Assert.AreEqual(1, groupedIssues[0].Keys.Count);
			Assert.IsTrue(groupedIssues[0].Keys.Contains(new IssueKey("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial", IssueKeyType.File)));
			Assert.AreEqual("Shader", groupedIssues[0].Type);
			Assert.IsTrue(groupedIssues[0].Metadata.FindValues("Material").Any());
			Assert.AreEqual("PointSetOverlaidComponentMaterial", groupedIssues[0].Metadata.FindValues("Material").First());

			Assert.AreEqual(1, groupedIssues[1].Keys.Count);
			Assert.IsTrue(groupedIssues[1].Keys.Contains(new IssueKey("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial.PointSetOverlaidComponentMaterial", IssueKeyType.File)));

			// Third issue has different material key (LineSetComponentMaterial)
			Assert.AreEqual(1, groupedIssues[2].Keys.Count);
			Assert.IsTrue(groupedIssues[2].Keys.Contains(new IssueKey("/Engine/EngineMaterials/LineSetComponentMaterial.LineSetComponentMaterial", IssueKeyType.File)));
			Assert.AreEqual("LineSetComponentMaterial", groupedIssues[2].Metadata.FindValues("Material").First());
		}

		// Parse structured logs where some entries represent multi-line events (multiple JSON lines sharing lineCount > 1).
		// Lines with lineCount > 1 are grouped into a single IssueEvent, simulating matcher output after grouping.
		static IssueHandlerTestData ParseGroupedJSONStructuredLogs(string[] jsonStructuredLogs)
		{
			List<LogEvent> logEvents = new List<LogEvent>();
			List<IssueEvent> issueEvents = new List<IssueEvent>();
			List<JsonLogEvent> pendingLines = new List<JsonLogEvent>();
			int pendingLineCount = 0;
			int pendingLineIndex = 0;
			Microsoft.Extensions.Logging.LogLevel pendingLevel = Microsoft.Extensions.Logging.LogLevel.None;
			Microsoft.Extensions.Logging.EventId? pendingEventId = null;

			foreach (string jsonStructuredLog in jsonStructuredLogs)
			{
				byte[] byteArrayStructuredLog = Encoding.UTF8.GetBytes(jsonStructuredLog);
				logEvents.Add(LogEvent.Read(byteArrayStructuredLog));

				JsonLogEvent jsonLogEvent;
				Assert.IsTrue(JsonLogEvent.TryParse(byteArrayStructuredLog, out jsonLogEvent));

				if (jsonLogEvent.LineCount > 1)
				{
					// Multi-line event: accumulate lines
					if (pendingLines.Count == 0)
					{
						pendingLineCount = jsonLogEvent.LineCount;
						pendingLineIndex = jsonLogEvent.LineIndex;
						pendingLevel = jsonLogEvent.Level;
						pendingEventId = jsonLogEvent.EventId;
					}
					pendingLines.Add(jsonLogEvent);

					if (pendingLines.Count >= pendingLineCount)
					{
						// All lines collected, create a single IssueEvent
						issueEvents.Add(new IssueEvent(pendingLineIndex, pendingLevel, pendingEventId, new List<JsonLogEvent>(pendingLines)));
						pendingLines.Clear();
					}
				}
				else
				{
					// Single-line event
					List<JsonLogEvent> singleList = new List<JsonLogEvent> { jsonLogEvent };
					issueEvents.Add(new IssueEvent(jsonLogEvent.LineIndex, jsonLogEvent.Level, jsonLogEvent.EventId, singleList));
				}
			}

			Assert.AreEqual(0, pendingLines.Count, "Incomplete multi-line group in test data: lineCount indicated more lines than were provided");
			return new IssueHandlerTestData(logEvents, issueEvents);
		}

		// Test HashedIssueHandler with a single IssueHandler using grouped logs
		static List<IssueEventGroup> TestSingleIssueHandlerGrouped(IssueHandler issueHandler, string[] jsonStructuredLogs)
		{
			IssueHandlerTestData issueHandlerParsedTestData = ParseGroupedJSONStructuredLogs(jsonStructuredLogs);
			foreach (IssueEvent issueEvent in issueHandlerParsedTestData.IssueEvents)
			{
				issueHandler.HandleEvent(issueEvent);
			}
			List<IssueEventGroup> groupedIssues = issueHandler.GetIssues().ToList();
			return groupedIssues;
		}

		[TestMethod]
		public void HashedIssueHandler_CrashCallstackGroupedIntoOneEvent()
		{
			// Simulates a crash sequence after the matcher groups it into a single multi-line event.
			// All lines share lineCount=8 and have incrementing line indices.
			// The first line has the Callstack property set to true.
			string[] jsonStructuredLogs =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: Assertion failed: Pair != nullptr [File:Map.h] [Line: 728]\",\"id\":301,\"line\":1,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":2,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.003Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe589a608c UnrealEditor-Engine.dll!UWorld::Tick() []\",\"id\":301,\"line\":3,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.004Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe589b1234 UnrealEditor-Engine.dll!FEngineLoop::Tick() []\",\"id\":301,\"line\":4,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.005Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe589c5678 UnrealEditor-Cmd.exe!GuardedMain() []\",\"id\":301,\"line\":5,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.006Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe89f44cb0 KERNEL32.DLL!UnknownFunction []\",\"id\":301,\"line\":6,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.007Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe9d14a271 ntdll.dll!UnknownFunction []\",\"id\":301,\"line\":7,\"lineCount\":8,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());
			HashedIssueHandler handler = new HashedIssueHandler(context);
			List<IssueEventGroup> groupedIssues = TestSingleIssueHandlerGrouped(handler, jsonStructuredLogs);

			// All 8 lines form a single multi-line IssueEvent, which the handler hashes as one group
			Assert.AreEqual(1, groupedIssues.Count);
			Assert.AreEqual(1, groupedIssues[0].Events.Count);
		}

		[TestMethod]
		public void HashedIssueHandler_RegularLogChannelEventsRemainSeparate()
		{
			// Regular LogChannel errors without Callstack property should each produce a separate group
			string[] jsonStructuredLogs =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogAudio: Error: Failed to initialize audio device\",\"id\":301,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogAudio\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogRenderer: Error: Shader compilation failed\",\"id\":301,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogRenderer\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());
			HashedIssueHandler handler = new HashedIssueHandler(context);
			List<IssueEventGroup> groupedIssues = TestSingleIssueHandler(handler, jsonStructuredLogs);

			// Each regular event gets its own group with a unique hash
			Assert.AreEqual(2, groupedIssues.Count);
			Assert.AreEqual(1, groupedIssues[0].Events.Count);
			Assert.AreEqual(1, groupedIssues[1].Events.Count);
		}

		[TestMethod]
		public void HashedIssueHandler_DifferentCrashesProduceDifferentHashes()
		{
			// Two different crashes with different callstacks should produce different hash keys
			string[] crashA =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe589a608c UnrealEditor-Engine.dll!UWorld::Tick() []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			string[] crashB =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe1234abcd UnrealEditor-Slate.dll!SWidget::Paint() []\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5678dcba UnrealEditor-SlateCore.dll!FSlateApplication::Tick() []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());

			HashedIssueHandler handlerA = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsA = TestSingleIssueHandlerGrouped(handlerA, crashA);

			HashedIssueHandler handlerB = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsB = TestSingleIssueHandlerGrouped(handlerB, crashB);

			Assert.AreEqual(1, groupsA.Count);
			Assert.AreEqual(1, groupsB.Count);

			// The two crashes should have different keys (different callstack content)
			Assert.IsFalse(groupsA[0].Keys.SetEquals(groupsB[0].Keys), "Different crashes should produce different hash keys");
		}

		[TestMethod]
		public void HashedIssueHandler_SharedGenericFramesDontCauseFalseMatch()
		{
			// Two crashes that share a generic frame (KERNEL32.DLL!UnknownFunction) but differ
			// in other frames should produce different fingerprints
			string[] crashA =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe89f44cb0 KERNEL32.DLL!UnknownFunction []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			string[] crashB =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe1234abcd UnrealEditor-Slate.dll!SWidget::Paint() []\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe89f44cb0 KERNEL32.DLL!UnknownFunction []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());

			HashedIssueHandler handlerA = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsA = TestSingleIssueHandlerGrouped(handlerA, crashA);

			HashedIssueHandler handlerB = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsB = TestSingleIssueHandlerGrouped(handlerB, crashB);

			Assert.AreEqual(1, groupsA.Count);
			Assert.AreEqual(1, groupsB.Count);

			// Despite sharing KERNEL32.DLL!UnknownFunction, different unique frames mean different hashes
			Assert.IsFalse(groupsA[0].Keys.SetEquals(groupsB[0].Keys), "Crashes sharing only generic frames should produce different hash keys");
		}

		[TestMethod]
		public void HashedIssueHandler_CrashSummaryMatchesAcrossPlatforms()
		{
			// Same CrashSummary but different callstack module names (dll vs so) should produce the same hash
			string[] windowsCrash =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Handled ensure: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true,\"CrashSummary\":\"Ensure condition failed: InParentOp [RunIKRigOp.cpp]\"} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007fff5b17b248 UnrealEditor-IKRig.dll!FIKRetargetRunIKRigOp::ValidateIKChain() []\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007fff89f44cb0 KERNEL32.DLL!UnknownFunction []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			string[] linuxCrash =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogOutputDevice: Error: === Handled ensure: ===\",\"id\":301,\"line\":0,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogOutputDevice\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true,\"CrashSummary\":\"Ensure condition failed: InParentOp [RunIKRigOp.cpp]\"} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogOutputDevice: Error: [Callstack] 0x00007c7b053585da libUnrealEditor-IKRig.so!FIKRetargetRunIKRigOp::ValidateIKChain() [/mnt/horde/Sync/Engine/Plugins/IKRig/RunIKRigOp.cpp:436]\",\"id\":301,\"line\":1,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogOutputDevice\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
				"{ \"time\":\"2025-01-01T01:00:00.002Z\",\"level\":\"Error\",\"message\":\"LogOutputDevice: Error: [Callstack] 0x00007c7ce429af1d libUnrealEditor-UnixCommonStartup.so!CommonUnixMain() []\",\"id\":301,\"line\":2,\"lineCount\":3,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogOutputDevice\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());

			HashedIssueHandler handlerWin = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsWin = TestSingleIssueHandlerGrouped(handlerWin, windowsCrash);

			HashedIssueHandler handlerLinux = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsLinux = TestSingleIssueHandlerGrouped(handlerLinux, linuxCrash);

			Assert.AreEqual(1, groupsWin.Count);
			Assert.AreEqual(1, groupsLinux.Count);

			// Same CrashSummary should produce the same hash regardless of platform-specific callstack content
			Assert.IsTrue(groupsWin[0].Keys.SetEquals(groupsLinux[0].Keys), "Same crash on different platforms should produce matching hash keys");
		}

		[TestMethod]
		public void HashedIssueHandler_DifferentCrashSummariesProduceDifferentHashes()
		{
			// Two crashes with different CrashSummary values should not match
			string[] crashA =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Critical error: ===\",\"id\":301,\"line\":0,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true,\"CrashSummary\":\"Assertion failed: Pair != nullptr [Map.h]\"} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":1,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			string[] crashB =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: === Handled ensure: ===\",\"id\":301,\"line\":0,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true,\"CrashSummary\":\"Ensure condition failed: InParentOp [RunIKRigOp.cpp]\"} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":1,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());

			HashedIssueHandler handlerA = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsA = TestSingleIssueHandlerGrouped(handlerA, crashA);

			HashedIssueHandler handlerB = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsB = TestSingleIssueHandlerGrouped(handlerB, crashB);

			Assert.AreEqual(1, groupsA.Count);
			Assert.AreEqual(1, groupsB.Count);
			Assert.IsFalse(groupsA[0].Keys.SetEquals(groupsB[0].Keys), "Different crash summaries should produce different hash keys");
		}

		[TestMethod]
		public void HashedIssueHandler_NoCrashSummaryFallsBackToFullRender()
		{
			// A callstack event WITHOUT CrashSummary should hash on the full render (not match a different callstack)
			string[] crashA =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5b17b248 UnrealEditor-Core.dll!FDebug::CheckVerifyFailedImpl2() []\",\"id\":301,\"line\":0,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe589a608c UnrealEditor-Engine.dll!UWorld::Tick() []\",\"id\":301,\"line\":1,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			string[] crashB =
			{
				"{ \"time\":\"2025-01-01T01:00:00.000Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe1234abcd UnrealEditor-Slate.dll!SWidget::Paint() []\",\"id\":301,\"line\":0,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"Callstack\":true} }",
				"{ \"time\":\"2025-01-01T01:00:00.001Z\",\"level\":\"Error\",\"message\":\"LogWindows: Error: [Callstack] 0x00007ffe5678dcba UnrealEditor-SlateCore.dll!FSlateApplication::Tick() []\",\"id\":301,\"line\":1,\"lineCount\":2,\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogWindows\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"}} }",
			};

			IssueHandlerContext context = new IssueHandlerContext(new StreamId("Release-XX.XX"), new TemplateId(), "Compile ProjectX", new Dictionary<string, string>(), new Dictionary<string, string>());

			HashedIssueHandler handlerA = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsA = TestSingleIssueHandlerGrouped(handlerA, crashA);

			HashedIssueHandler handlerB = new HashedIssueHandler(context);
			List<IssueEventGroup> groupsB = TestSingleIssueHandlerGrouped(handlerB, crashB);

			Assert.AreEqual(1, groupsA.Count);
			Assert.AreEqual(1, groupsB.Count);

			// Without CrashSummary, full render is used — different callstacks produce different hashes
			Assert.IsFalse(groupsA[0].Keys.SetEquals(groupsB[0].Keys), "Callstack events without CrashSummary should hash on full render");
		}

		[TestMethod]
		public void ShaderIssueHandler_GroupsBySourceFileWhenNoMaterialProperty()
		{
			// These are shader errors WITHOUT the material property (e.g., shader syntax errors).
			// They should continue to use the existing behavior: grouped by source file.
			string[] jsonStructuredLogs =
			{
				// Shader error with source file only (no material property)
				"{ \"time\":\"2025-01-01T01:20:30.990Z\",\"level\":\"Error\",\"message\":\"LogShaderCompilers: Error: Shader compilation failed in LocalVertexFactory.ush\",\"format\":\"{_channel}: {_severity}: {message}\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogShaderCompilers\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"id\":310,\"file\":\"D:/build/Engine/Shaders/Private/LocalVertexFactory.ush\"} }",
				// Different source file
				"{ \"time\":\"2025-01-01T01:20:30.991Z\",\"level\":\"Error\",\"message\":\"LogShaderCompilers: Error: Shader compilation failed in MaterialTemplate.ush\",\"format\":\"{_channel}: {_severity}: {message}\",\"properties\":{ \"_channel\":{ \"$type\":\"Channel\",\"$text\":\"LogShaderCompilers\"},\"_severity\":{ \"$type\":\"Severity\",\"$text\":\"Error\"},\"id\":310,\"file\":\"D:/build/Engine/Shaders/Private/MaterialTemplate.ush\"} }",
			};

			ShaderIssueHandler shaderIssueHandler = new ShaderIssueHandler();
			List<IssueEventGroup> groupedIssues = TestSingleIssueHandler(shaderIssueHandler, jsonStructuredLogs);

			// Without material property, issues are keyed by source file
			Assert.AreEqual(2, groupedIssues.Count);

			// First issue keyed by LocalVertexFactory.ush
			Assert.AreEqual(1, groupedIssues[0].Keys.Count);
			Assert.IsTrue(groupedIssues[0].Keys.Contains(new IssueKey("LocalVertexFactory.ush", IssueKeyType.File)));
			Assert.AreEqual("Shader", groupedIssues[0].Type);
			Assert.IsFalse(groupedIssues[0].Metadata.FindValues("Material").Any());

			// Second issue keyed by MaterialTemplate.ush
			Assert.AreEqual(1, groupedIssues[1].Keys.Count);
			Assert.IsTrue(groupedIssues[1].Keys.Contains(new IssueKey("MaterialTemplate.ush", IssueKeyType.File)));
			Assert.IsFalse(groupedIssues[1].Metadata.FindValues("Material").Any());
		}
	}
}
