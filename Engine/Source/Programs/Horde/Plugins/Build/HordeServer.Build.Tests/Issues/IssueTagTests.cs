// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Issues;

namespace HordeServer.Tests.Issues
{
	[TestClass]
	public class IssueTagTests
	{
		[TestMethod]
		public void IssueTag()
		{
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n#horde 123 ").SequenceEqual(new[] { 123 }));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n#horde 123 456").SequenceEqual(new[] { 123, 456 }));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n #horde 123 ").SequenceEqual(new[] { 123 }));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n #horde 123 \n#horde 456").SequenceEqual(new[] { 123, 456 }));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n #horde 123 \n #ROBOMERGE-SOURCE foo").SequenceEqual(Array.Empty<int>()));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n #horde 123, 456").SequenceEqual(new[] { 123, 456 }));
			Assert.IsTrue(IssueTagService.ParseTags("#horde", "hello\n #horde 123 , garbage 456").SequenceEqual(new[] { 123, 456 }));
		}
	}
}
