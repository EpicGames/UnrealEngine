// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Jobs.Bisect;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Moq;

namespace HordeServer.Tests.Jobs.Bisect
{
	[TestClass]
	public class BisectTests : BuildTestSetup
	{
		[TestMethod]
		[Ignore("Bisection is disabled via feature flag (DisableBisection)")]
		public async Task TestStatesAsync()
		{
			(IJob failedJob, IGraph graph) = await SetupBisectionTestAsync();

			// Create the bisection task
			CreateBisectTaskResponse task = Deref(await BisectTasksController!.CreateAsync(new CreateBisectTaskRequest() { JobId = failedJob.Id, NodeName = "CompileEditor" }));

			GetBisectTaskResponse response = Deref(await BisectTasksController!.GetAsync(task!.BisectTaskId));
			Assert.AreEqual(failedJob.TemplateId, response.TemplateId);
			Assert.AreEqual(failedJob.StreamId, response.StreamId);
			Assert.AreEqual("CompileEditor", response.NodeName);
			Assert.AreEqual(JobStepOutcome.Failure, response.Outcome);
			Assert.AreEqual("TestUser", response.Owner?.Name ?? "");
			Assert.AreEqual(failedJob.CommitId, response.InitialCommitId);
			Assert.AreEqual(failedJob.Id.ToString(), response.InitialJobId);
			Assert.AreEqual(failedJob.CommitId, response.CurrentCommitId);
			Assert.AreEqual(failedJob.Id.ToString(), response.CurrentJobId);

			IUser user = await UserCollection.FindOrAddUserByLoginAsync("TestUser");
			IUserSettings settings = await UserCollection.GetSettingsAsync(user.Id);

			Assert.AreEqual(task.BisectTaskId, settings.PinnedBisectTaskIds[0]);

			Assert.AreEqual(task.BisectTaskId, response.Id);
			List<IJob> jobs = await JobCollection.FindBisectTaskJobsAsync(response!.Id, null).ToListAsync();
			Assert.AreEqual(0, jobs.Count);

			// Start bisection service
			BisectService bisectService = ServiceProvider.GetRequiredService<BisectService>();
			await bisectService.StartAsync(CancellationToken.None);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30));

			GetBisectTaskResponse bisectTask = Deref(await BisectTasksController!.GetAsync(task!.BisectTaskId));
			Assert.AreEqual(BisectTaskState.Running, bisectTask.State);
			Assert.AreEqual(10, bisectTask.MinCommitId!.GetPerforceChange());
			jobs = await JobCollection.FindBisectTaskJobsAsync(bisectTask.Id, running: true).ToListAsync();
			Assert.AreEqual(1, jobs.Count);
			Assert.AreEqual(14, jobs[0].CommitId.GetPerforceChange());
			await SetJobOutcomeAsync(jobs[0], graph, JobStepOutcome.Failure);

			List<GetBisectTaskResponse> bisectTasks = Deref(await BisectTasksController!.FindBisectTasksAsync(null, bisectTask.Owner?.Id.ToString() ?? ""));
			Assert.AreEqual(1, bisectTasks.Count);
			Assert.AreEqual(bisectTask.Owner?.Id ?? default, bisectTasks[0].Owner?.Id ?? default);

			List<IBisectTask> running = await BisectTaskCollection.FindActiveAsync().ToListAsync();
			Assert.AreEqual(1, running.Count);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30));

			bisectTask = Deref(await BisectTasksController!.GetAsync(task!.BisectTaskId));
			Assert.AreEqual(BisectTaskState.Running, bisectTask.State);
			jobs = await JobCollection.FindBisectTaskJobsAsync(bisectTask.Id, running: true).ToListAsync();
			Assert.AreEqual(1, jobs.Count);
			Assert.AreEqual(12, jobs[0].CommitId.GetPerforceChange());
			await SetJobOutcomeAsync(jobs[0], graph, JobStepOutcome.Success);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30));

			bisectTask = Deref(await BisectTasksController!.GetAsync(task!.BisectTaskId));
			Assert.AreEqual(BisectTaskState.Running, bisectTask.State);
			jobs = await JobCollection.FindBisectTaskJobsAsync(bisectTask.Id, running: true).ToListAsync();
			Assert.AreEqual(1, jobs.Count);
			Assert.AreEqual(13, jobs[0].CommitId.GetPerforceChange());
			await SetJobOutcomeAsync(jobs[0], graph, JobStepOutcome.Success);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30));

			bisectTask = Deref(await BisectTasksController!.GetAsync(task!.BisectTaskId));
			Assert.AreEqual(BisectTaskState.Succeeded, bisectTask.State);
			Assert.AreEqual(20, bisectTask.InitialCommitId.GetPerforceChange());
			Assert.AreEqual(14, bisectTask.CurrentCommitId.GetPerforceChange());
			jobs = await JobCollection.FindBisectTaskJobsAsync(bisectTask.Id, running: true).ToListAsync();
			Assert.AreEqual(0, jobs.Count);

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30));

			List<GetBisectTaskResponse> jobTasks = Deref(await BisectTasksController!.GetJobBisectTasksAsync(failedJob.Id));
			Assert.AreEqual(1, jobTasks.Count);
			Assert.AreEqual(failedJob.Id.ToString(), jobTasks[0].InitialJobId);
			Assert.AreEqual(5, jobTasks[0].Steps.Count);
			Assert.AreEqual(BisectTaskState.Succeeded, jobTasks[0].State);

			running = await BisectTaskCollection.FindActiveAsync().ToListAsync();
			Assert.AreEqual(0, running.Count);

		}

		StreamId StreamId { get; } = new StreamId("ue4-main");
		TemplateId TemplateId { get; } = new TemplateId("test-build");

		async Task<(IJob failedJob, IGraph graph)> SetupBisectionTestAsync()
		{
			IUser user = await UserCollection.FindOrAddUserByLoginAsync("Bob");

			PerforceService.AddChange(StreamId, 10, user, "", new[] { "Foo.cpp" });

			PerforceService.AddChange(StreamId, 11, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 12, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 13, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 14, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 15, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 16, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 17, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(StreamId, 20, user, "", new[] { "Bar.cpp" });

			ProjectConfig project = new ProjectConfig();
			project.Id = new ProjectId("ue4");
			project.Streams.Add(new StreamConfig { Id = StreamId, Templates = new List<TemplateRefConfig> { new TemplateRefConfig { Id = TemplateId } } });

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects.Add(project));

			// Create the graph
			List<NewNode> nodes = new List<NewNode>();
			nodes.Add(new NewNode("CompileEditor"));

			List<NewGroup> groups = new List<NewGroup>();
			groups.Add(new NewGroup("Foo", nodes));

			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph graph = await GraphCollection.AddAsync(templateMock.Object);
			graph = await GraphCollection.AppendAsync(graph, groups);

			// Create two job runs, one which fails, one that succeeds.
			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add($"{IJob.TargetArgumentPrefix}CompileEditor");

			IJob succeededJob = await CreateJobAsync(10, graph, JobStepOutcome.Success, options);
			// create an interim job at the same change as the one that will be bisected
			await CreateJobAsync(20, graph, JobStepOutcome.Failure, options);
			IJob failedJob = await CreateJobAsync(20, graph, JobStepOutcome.Failure, options);

			return (failedJob, graph);
		}

		async Task<IJob> CreateJobAsync(int change, IGraph graph, JobStepOutcome outcome, CreateJobOptions options)
		{
			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), StreamId, TemplateId, ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(change), CommitIdWithOrder.FromPerforceChange(change), options);
			return await SetJobOutcomeAsync(job, graph, outcome);
		}

		async Task<IJob> SetJobOutcomeAsync(IJob job, IGraph graph, JobStepOutcome outcome)
		{
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Completed, JobStepOutcome.Success));
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, null));

			IJobStepBatch batch = job.Batches[^1];
			IJobStep step = batch.Steps[^1];

			job = Deref(await job.TryUpdateStepAsync(batch.Id, step.Id, JobStepState.Completed, outcome));
			job = Deref(await job.TryUpdateBatchAsync(batch.Id, null, JobStepBatchState.Complete, null));

			INodeGroup group = graph.Groups[batch.GroupIdx];
			INode node = group.Nodes[step.NodeIdx];
			await JobStepRefCollection.InsertOrReplaceAsync(new JobStepRefId(job.Id, batch.Id, step.Id), job.Name, node.Name, job.StreamId, job.TemplateId, job.CommitId, LogIdUtils.GenerateNewId(), null, null, JobStepState.Completed, outcome, false, null, null, 0.0f, 0.0f, DateTime.MinValue, DateTime.MinValue, DateTime.MinValue);
			await BisectTaskCollection.UpdateAsync(job, batch, step, graph);

			return job;
		}

		BisectTasksController? _bisectTasksController;

		// setup BisectTaskController with a test user
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "MSTEST0036:Do not use shadowing", Justification = "<Pending>")]
		private new BisectTasksController BisectTasksController
		{
			get
			{
				if (_bisectTasksController == null)
				{
					IUser user = UserCollection.FindOrAddUserByLoginAsync("TestUser").Result;
					_bisectTasksController = base.BisectTasksController;
					ControllerContext controllerContext = new ControllerContext();
					controllerContext.HttpContext = new DefaultHttpContext();
					controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
						new List<Claim> { HordeClaims.AdminClaim.ToClaim(),
						new Claim(ClaimTypes.Name, "TestUser"),
						new Claim(HordeClaimTypes.UserId, user.Id.ToString()) }
						, "TestAuthType"));
					_bisectTasksController.ControllerContext = controllerContext;

				}
				return _bisectTasksController;
			}
		}
	}
}
