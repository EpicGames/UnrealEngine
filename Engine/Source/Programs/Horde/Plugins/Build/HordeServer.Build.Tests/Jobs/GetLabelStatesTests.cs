// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using Moq;

namespace HordeServer.Tests.Jobs
{
	[TestClass]
	public class GetLabelStatesTests
	{
		public static INode MockNode(string name)
		{
			Mock<INode> node = new Mock<INode>();
			node.SetupGet(x => x.Name).Returns(name);
			return node.Object;
		}

		public static ILabel MockLabel(string labelName, List<NodeRef> nodes)
		{
			Mock<ILabel> label = new Mock<ILabel>();
			label.SetupGet(x => x.DashboardName).Returns(labelName);
			label.SetupGet(x => x.IncludedNodes).Returns(nodes);
			label.SetupGet(x => x.RequiredNodes).Returns(nodes);
			return label.Object;
		}

		public static IGraph CreateGraph()
		{
			List<INode> nodes1 = new List<INode>();
			nodes1.Add(MockNode("Update Version Files"));
			nodes1.Add(MockNode("Compile UnrealHeaderTool Win64"));
			nodes1.Add(MockNode("Compile ShooterGameEditor Win64"));
			nodes1.Add(MockNode("Cook ShooterGame Win64"));

			List<INode> nodes2 = new List<INode>();
			nodes2.Add(MockNode("Compile UnrealHeaderTool Mac"));
			nodes2.Add(MockNode("Compile ShooterGameEditor Mac"));
			nodes2.Add(MockNode("Cook ShooterGame Mac"));

			List<ILabel> labels = new List<ILabel>();
			labels.Add(MockLabel("Label_AllGroup1Nodes", new List<NodeRef>() { new NodeRef(0, 0), new NodeRef(0, 1), new NodeRef(0, 2), new NodeRef(0, 3) }));
			labels.Add(MockLabel("Label_Win64UHTOnly", new List<NodeRef>() { new NodeRef(0, 1) }));
			labels.Add(MockLabel("Label_Group1TwoNodes", new List<NodeRef>() { new NodeRef(0, 0), new NodeRef(0, 3) }));
			labels.Add(MockLabel("Label_BothGroups", new List<NodeRef>() { new NodeRef(0, 0), new NodeRef(1, 0) }));

			Mock<INodeGroup> group1 = new Mock<INodeGroup>(MockBehavior.Strict);
			group1.SetupGet(x => x.Nodes).Returns(nodes1);

			Mock<INodeGroup> group2 = new Mock<INodeGroup>(MockBehavior.Strict);
			group2.SetupGet(x => x.Nodes).Returns(nodes2);

			Mock<IGraph> graphMock = new Mock<IGraph>(MockBehavior.Strict);
			graphMock.SetupGet(x => x.Groups).Returns(new List<INodeGroup> { group1.Object, group2.Object });
			graphMock.SetupGet(x => x.Labels).Returns(labels);
			return graphMock.Object;
		}

		public static IJob CreateGetLabelStatesJob(string name, IGraph graph, List<(JobStepState, JobStepOutcome)> jobStepData)
		{
			JobId jobId = JobId.Parse("5ec16da1774cb4000107c2c1");
			List<IJobStepBatch> batches = new List<IJobStepBatch>();
			int dataIdx = 0;
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup @group = graph.Groups[groupIdx];

				List<IJobStep> steps = new List<IJobStep>();
				for (int nodeIdx = 0; nodeIdx < @group.Nodes.Count; nodeIdx++, dataIdx++)
				{
					JobStepId stepId = new JobStepId((ushort)((groupIdx * 100) + nodeIdx));

					Mock<IJobStep> step = new Mock<IJobStep>(MockBehavior.Strict);
					step.SetupGet(x => x.Id).Returns(stepId);
					step.SetupGet(x => x.NodeIdx).Returns(nodeIdx);
					step.SetupGet(x => x.State).Returns(jobStepData[dataIdx].Item1);
					step.SetupGet(x => x.Outcome).Returns(jobStepData[dataIdx].Item2);
					steps.Add(step.Object);
				}

				JobStepBatchId batchId = new JobStepBatchId((ushort)(groupIdx * 100));

				Mock<IJobStepBatch> batch = new Mock<IJobStepBatch>(MockBehavior.Strict);
				batch.SetupGet(x => x.Id).Returns(batchId);
				batch.SetupGet(x => x.GroupIdx).Returns(groupIdx);
				batch.SetupGet(x => x.Steps).Returns(steps);
				batches.Add(batch.Object);
			}

			Mock<IJob> job = new Mock<IJob>(MockBehavior.Strict);
			job.SetupGet(x => x.Id).Returns(jobId);
			job.SetupGet(x => x.Name).Returns(name);
			job.SetupGet(x => x.Batches).Returns(batches);
			return job.Object;
		}

		public static List<(JobStepState, JobStepOutcome)> GetJobStepDataList(JobStepState state, JobStepOutcome outcome)
		{
			List<(JobStepState, JobStepOutcome)> data = new List<(JobStepState, JobStepOutcome)>();
			for (int i = 0; i < 7; ++i)
			{
				data.Add((state, outcome));
			}
			return data;
		}

		[TestMethod]
		public void GetLabelStatesInitialStateTest()
		{
			// Verify initial state. All job steps are unspecified, so all labels unspecified.
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, GetJobStepDataList(JobStepState.Unspecified, JobStepOutcome.Unspecified));
			IReadOnlyList<(LabelState, LabelOutcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			foreach ((LabelState State, LabelOutcome Outcome) result in labelStates)
			{
				Assert.IsTrue(result.State == LabelState.Running);
				Assert.IsTrue(result.Outcome == LabelOutcome.Success);
			}
		}

		[TestMethod]
		public void GetLabelStatesRunningStateTest()
		{
			// Verify that if any nodes are in the ready or running state that the approrpirate labels switch to running.
			List<(JobStepState, JobStepOutcome)> jobStepData = GetJobStepDataList(JobStepState.Unspecified, JobStepOutcome.Unspecified);
			jobStepData[3] = (JobStepState.Ready, JobStepOutcome.Unspecified);
			jobStepData[4] = (JobStepState.Running, JobStepOutcome.Unspecified);
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, jobStepData);
			IReadOnlyList<(LabelState State, LabelOutcome Outcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			Assert.IsTrue(labelStates[0].State == LabelState.Running);
			Assert.IsTrue(labelStates[0].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[1].State == LabelState.Running);
			Assert.IsTrue(labelStates[1].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[2].State == LabelState.Running);
			Assert.IsTrue(labelStates[2].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[3].State == LabelState.Running);
			Assert.IsTrue(labelStates[3].Outcome == LabelOutcome.Success);
		}

		[TestMethod]
		public void GetLabelStatesWarningOutcomeTest()
		{
			// Verify that if a step has a warning, any applicable labels are set to warning.
			List<(JobStepState, JobStepOutcome)> jobStepData = GetJobStepDataList(JobStepState.Ready, JobStepOutcome.Unspecified);
			jobStepData[3] = (JobStepState.Running, JobStepOutcome.Warnings);
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, jobStepData);
			IReadOnlyList<(LabelState State, LabelOutcome Outcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			Assert.IsTrue(labelStates[0].State == LabelState.Running);
			Assert.IsTrue(labelStates[0].Outcome == LabelOutcome.Warnings);
			Assert.IsTrue(labelStates[1].State == LabelState.Running);
			Assert.IsTrue(labelStates[1].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[2].State == LabelState.Running);
			Assert.IsTrue(labelStates[2].Outcome == LabelOutcome.Warnings);
			Assert.IsTrue(labelStates[3].State == LabelState.Running);
			Assert.IsTrue(labelStates[3].Outcome == LabelOutcome.Success);
		}

		[TestMethod]
		public void GetLabelStatesErrorOutcomeTest()
		{
			// Verify that if a step has an error, any applicable labels are set to failure.
			List<(JobStepState, JobStepOutcome)> jobStepData = GetJobStepDataList(JobStepState.Ready, JobStepOutcome.Unspecified);
			jobStepData[2] = (JobStepState.Running, JobStepOutcome.Warnings);
			jobStepData[4] = (JobStepState.Running, JobStepOutcome.Failure);
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, jobStepData);
			IReadOnlyList<(LabelState State, LabelOutcome Outcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			Assert.IsTrue(labelStates[0].State == LabelState.Running);
			Assert.IsTrue(labelStates[0].Outcome == LabelOutcome.Warnings);
			Assert.IsTrue(labelStates[1].State == LabelState.Running);
			Assert.IsTrue(labelStates[1].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[2].State == LabelState.Running);
			Assert.IsTrue(labelStates[2].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[3].State == LabelState.Running);
			Assert.IsTrue(labelStates[3].Outcome == LabelOutcome.Failure);
		}

		[TestMethod]
		public void GetLabelStatesCompleteStateTest()
		{
			// Verify that if a label's steps are all complete the state is set to complete.
			List<(JobStepState, JobStepOutcome)> jobStepData = GetJobStepDataList(JobStepState.Completed, JobStepOutcome.Success);
			jobStepData[2] = (JobStepState.Running, JobStepOutcome.Failure);
			jobStepData[4] = (JobStepState.Ready, JobStepOutcome.Unspecified);
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, jobStepData);
			IReadOnlyList<(LabelState State, LabelOutcome Outcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			Assert.IsTrue(labelStates[0].State == LabelState.Running);
			Assert.IsTrue(labelStates[0].Outcome == LabelOutcome.Failure);
			Assert.IsTrue(labelStates[1].State == LabelState.Complete);
			Assert.IsTrue(labelStates[1].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[2].State == LabelState.Complete);
			Assert.IsTrue(labelStates[2].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[3].State == LabelState.Running);
			Assert.IsTrue(labelStates[3].Outcome == LabelOutcome.Success);
		}

		[TestMethod]
		public void GetLabelStatesSkippedAbortedStepTest()
		{
			// Verify that if a label's steps are complete and successful but any were skipped or aborted the result is unspecified.
			List<(JobStepState, JobStepOutcome)> jobStepData = GetJobStepDataList(JobStepState.Completed, JobStepOutcome.Success);
			jobStepData[2] = (JobStepState.Aborted, JobStepOutcome.Unspecified);
			jobStepData[4] = (JobStepState.Skipped, JobStepOutcome.Unspecified);
			IGraph graph = CreateGraph();
			IJob job = CreateGetLabelStatesJob("Job", graph, jobStepData);
			IReadOnlyList<(LabelState State, LabelOutcome Outcome)> labelStates = job.GetLabelStates(graph);
			Assert.AreEqual(graph.Labels.Count, labelStates.Count);
			Assert.IsTrue(labelStates[0].State == LabelState.Complete);
			Assert.IsTrue(labelStates[0].Outcome == LabelOutcome.Unspecified);
			Assert.IsTrue(labelStates[1].State == LabelState.Complete);
			Assert.IsTrue(labelStates[1].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[2].State == LabelState.Complete);
			Assert.IsTrue(labelStates[2].Outcome == LabelOutcome.Success);
			Assert.IsTrue(labelStates[3].State == LabelState.Complete);
			Assert.IsTrue(labelStates[3].Outcome == LabelOutcome.Unspecified);
		}
	}
}
