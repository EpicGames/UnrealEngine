// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs.Graphs;
using MongoDB.Bson;
using MongoDB.Driver;

namespace HordeServer.Tests.Jobs
{
	[TestClass]
	public class GraphCollectionTests : BuildTestSetup
	{
		[TestMethod]
		public async Task TestLegacyNodeWithoutOptionalInputsAsync()
		{
			ContentHash hash = ContentHash.MD5("legacy-graph-test");

			// Build a BsonDocument simulating a pre-OptionalInput graph document.
			// The Node deliberately omits nullable fields to mirror what older documents look like in MongoDB.
			BsonDocument nodeDoc = new()
			{
				{ "Name", "TestNode" },
				{ "InputDependencies", new BsonArray() },
				{ "OrderDependencies", new BsonArray() },
				{ "Priority", 0 },
				{ "AllowRetry", true },
				{ "RunEarly", false },
				{ "Warnings", true }
			};

			BsonDocument groupDoc = new()
			{
				{ "AgentType", "Win64" },
				{ "Nodes", new BsonArray { nodeDoc } }
			};

			BsonDocument graphDoc = new()
			{
				{ "_id", hash.ToString() },
				{ "Schema", 0 },
				{ "Groups", new BsonArray { groupDoc } },
				{ "Aggregates", new BsonArray() },
				{ "Labels", new BsonArray() },
				{ "Artifacts", new BsonArray() }
			};

			IMongoCollection<BsonDocument> collection = MongoService.GetCollection<BsonDocument>("Graphs");
			await collection.InsertOneAsync(graphDoc);

			IGraph graph = await GraphCollection.GetAsync(hash, CancellationToken.None);

			Assert.AreEqual(1, graph.Groups.Count);
			Assert.AreEqual(1, graph.Groups[0].Nodes.Count);
			Assert.AreEqual("TestNode", graph.Groups[0].Nodes[0].Name);

			Assert.IsNotNull(graph.Groups[0].Nodes[0].OptionalInputs);
			Assert.AreEqual(0, graph.Groups[0].Nodes[0].OptionalInputs.Count);

			Assert.IsNotNull(graph.Groups[0].Nodes[0].OptionalInputDependencies);
			Assert.AreEqual(0, graph.Groups[0].Nodes[0].OptionalInputDependencies.Length);
		}
	}
}
