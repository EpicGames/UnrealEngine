// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Agents.Sessions;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Moq;

namespace HordeServer.Tests.Jobs
{
	[TestClass]
	public class JobCollectionTests : BuildTestSetup
	{
		private readonly StreamId _streamId = new("ue4-main");
		private readonly TemplateId _templateId1 = new("template1");
		private readonly ITemplate _template1 = new TemplateStub(ContentHash.MD5("graphHash"), "template1");

		// override JobController with valid user, so we can use REST API in tests
		private JobsController? _jobsController;

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "MSTEST0036:Do not use shadowing", Justification = "<Pending>")]
		private new JobsController JobsController
		{
			get
			{
				if (_jobsController == null)
				{
					IUser user = UserCollection.FindOrAddUserByLoginAsync("TestUser").Result;
					_jobsController = base.JobsController;
					ControllerContext controllerContext = new ControllerContext();
					controllerContext.HttpContext = new DefaultHttpContext();
					controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
						new List<Claim>
						{
							HordeClaims.AdminClaim.ToClaim(),
							new Claim(ClaimTypes.Name, "TestUser"),
							new Claim(HordeClaimTypes.UserId, user.Id.ToString())
						}
						, "TestAuthType"));
					_jobsController.ControllerContext = controllerContext;

				}
				return _jobsController;
			}
		}

		static T ResultToValue<T>(ActionResult<T> result) where T : class
		{
			return result.Value! as T;
		}

		[TestInitialize]
		public async Task UpdateConfigAsync()
		{
			await UpdateConfigAsync(globalConfig =>
			{
				ProjectConfig projectConfig = new() { Id = new ProjectId("ue4") };
				projectConfig.Streams.Add(new StreamConfig { Id = _streamId });
				globalConfig.Plugins.GetBuildConfig().Projects.Add(projectConfig);
			});
		}

		static NewGroup AddGroup(List<NewGroup> groups)
		{
			NewGroup group = new NewGroup("win64", new List<NewNode>());
			groups.Add(group);
			return group;
		}

		static NewNode AddNode(NewGroup group, string name, string[]? inputDependencies, Action<NewNode>? action = null)
		{
			return AddNode(group, name, inputDependencies, null, action);
		}

		static NewNode AddNode(NewGroup group, string name, string[]? inputDependencies, string[]? optionalInputDependencies, Action<NewNode>? action = null)
		{
			HashSet<string>? orderDependencies = [];
			if (inputDependencies != null)
			{
				orderDependencies.UnionWith(inputDependencies);
			}
			if (optionalInputDependencies != null)
			{
				orderDependencies.UnionWith(optionalInputDependencies);
			}

			NewNode node = new(name, inputDependencies: inputDependencies?.ToList(), optionalInputDependencies: optionalInputDependencies?.ToList(), orderDependencies: orderDependencies?.ToList());
			action?.Invoke(node);
			group.Nodes.Add(node);
			return node;
		}

		static Task<IJob> RunBatchAsync(IJob job, int batchIdx)
		{
			return UpdateBatchAsync(job, batchIdx, JobStepBatchState.Running);
		}

		static async Task<IJob> UpdateBatchAsync(IJob job, int batchIdx, JobStepBatchState state)
		{
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[batchIdx].State);
			job = Deref(await job.TryAssignLeaseAsync(batchIdx, new PoolId("foo"), new AgentId("agent"), new SessionId(BinaryIdUtils.CreateNew()), new LeaseId(BinaryIdUtils.CreateNew()), new LogId(BinaryIdUtils.CreateNew())));
			if (job.Batches[batchIdx].State != state)
			{
				job = Deref(await job.TryUpdateBatchAsync(job.Batches[batchIdx].Id, null, state, null));
				Assert.AreEqual(state, job.Batches[batchIdx].State);
			}
			return job;
		}

		static async Task<IJob> RunStepAsync(IJob job, int batchIdx, int stepIdx, JobStepOutcome outcome)
		{
			Assert.AreEqual(JobStepState.Ready, job.Batches[batchIdx].Steps[stepIdx].State);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, JobStepState.Running, JobStepOutcome.Success));
			Assert.AreEqual(JobStepState.Running, job.Batches[batchIdx].Steps[stepIdx].State);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, JobStepState.Completed, outcome));
			Assert.AreEqual(JobStepState.Completed, job.Batches[batchIdx].Steps[stepIdx].State);
			Assert.AreEqual(outcome, job.Batches[batchIdx].Steps[stepIdx].Outcome);
			return job;
		}

		[TestMethod]
		public async Task TestStatesAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Publish Client");
			options.Arguments.Add("-Target=Post-Publish Client");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Update Version Files", null);
			AddNode(initialGroup, "Compile Editor", new[] { "Update Version Files" });

			NewGroup compileGroup = AddGroup(newGroups);
			AddNode(compileGroup, "Compile Client", new[] { "Update Version Files" });

			NewGroup publishGroup = AddGroup(newGroups);
			AddNode(publishGroup, "Cook Client", new[] { "Compile Editor" }, x => x.RunEarly = true);
			AddNode(publishGroup, "Publish Client", new[] { "Compile Client", "Cook Client" });
			AddNode(publishGroup, "Post-Publish Client", null, x => x.OrderDependencies = new List<string> { "Publish Client" });

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Update Version Files
			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Compile Editor

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // Compile Client

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Failure); // Cook Client
			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[1].State); // Publish Client
			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[2].State); // Post-Publish Client
		}

		[TestMethod]
		public async Task TestStatesSkippedWhenFailureOfRequiredAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");
			options.Arguments.Add("-Target=Node D");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", ["Node B"]);

			NewGroup groupD = AddGroup(newGroups);
			AddNode(groupD, "Node D", ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // B

			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State);
			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[0].State);
			Assert.AreEqual(JobStepState.Skipped, job.Batches[4].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesCompleteWhenNoFailureAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");
			options.Arguments.Add("-Target=Node D");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", ["Node B"]);

			NewGroup groupD = AddGroup(newGroups);
			AddNode(groupD, "Node D", ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // B

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // C

			job = await RunBatchAsync(job, 4);
			job = await RunStepAsync(job, 4, 0, JobStepOutcome.Success); // D

			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[4].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesCompleteWhenFailureOfOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");
			options.Arguments.Add("-Target=Node D");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", ["Node B"]);

			NewGroup groupD = AddGroup(newGroups);
			AddNode(groupD, "Node D", ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Failure); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // B

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // C

			job = await RunBatchAsync(job, 4);
			job = await RunStepAsync(job, 4, 0, JobStepOutcome.Success); // D

			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State);
			Assert.AreEqual(JobStepState.Completed, job.Batches[4].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesCompleteWithMixOfRequiredAndOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", ["Node A"], ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // B

			Assert.AreEqual(JobStepState.Ready, job.Batches[3].Steps[0].State);

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // C

			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesSkippedWithMixOfRequiredAndOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", ["Node A"], ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Failure); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // B

			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesReadyWhenAgentAbortedOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node B");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			// Node A is started and cancelled by agent
			job = await RunBatchAsync(job, 1);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, JobStepState.Aborted, JobStepOutcome.Failure));

			Assert.AreEqual(JobStepState.Aborted, job.Batches[1].Steps[0].State);
			Assert.IsFalse(job.Batches[1].Steps[0].AbortRequested);

			// Node B should still be ready since A is optional and not aborted by user
			Assert.AreEqual(JobStepState.Ready, job.Batches[2].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesSkippedWhenUserAbortedOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node B");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			// Node A is started and cancelled by user request
			job = await RunBatchAsync(job, 1);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, newAbortRequested: true));

			Assert.AreEqual(JobStepState.Running, job.Batches[1].Steps[0].State);
			Assert.IsTrue(job.Batches[1].Steps[0].AbortRequested);

			// Node B should be skipped since user explicitly aborted A even though A is optional
			Assert.AreEqual(JobStepState.Skipped, job.Batches[2].Steps[0].State);
		}

		[TestMethod]
		public async Task TestStatesCompleteWhenFailureOfChainedOptionalAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", null, ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Failure); // A

			Assert.AreEqual(JobStepState.Ready, job.Batches[2].Steps[0].State);

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // B

			Assert.AreEqual(JobStepState.Ready, job.Batches[3].Steps[0].State);

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // C

			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State);
		}

		[TestMethod]
		public async Task TestRetryWithOptionalDependencyAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node B");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // A

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // B

			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State);
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[2].Steps[0].Outcome);

			Assert.AreEqual(3, job.Batches.Count);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[2].Id, job.Batches[2].Steps[0].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(4, job.Batches.Count);
		}

		[TestMethod]
		public async Task TestRetrySkippedNodeWithAbortedOptionalDependencyAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			// Node B has ONLY optional dependency on Node A
			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node B");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newAbortRequested: true));
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Aborted, JobStepOutcome.Failure));

			Assert.AreEqual(JobStepState.Aborted, job.Batches[0].Steps[0].State); // A
			Assert.AreEqual(JobStepState.Skipped, job.Batches[1].Steps[0].State); // B

			// Retry Node B
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, newRetryByUserId: UserId.Anonymous));

			// Nodes A and B to be retried
			Assert.AreEqual(3, job.Batches.Count);

			Assert.AreEqual(JobStepState.Ready, job.Batches[1].Steps[0].State); // A
			Assert.AreEqual(0, job.Batches[1].GroupIdx);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(0, job.Batches[1].Steps[0].NodeIdx);

			Assert.AreEqual(JobStepState.Waiting, job.Batches[2].Steps[0].State); // B
			Assert.AreEqual(1, job.Batches[2].GroupIdx);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(0, job.Batches[2].Steps[0].NodeIdx);
		}

		[TestMethod]
		public async Task TestRetrySkippedNodeWithSkippedOptionalDependencyAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", ["Node A"]);

			NewGroup groupC = AddGroup(newGroups);
			AddNode(groupC, "Node C", null, ["Node B"]);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node C");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Failure); // A

			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State); // A
			Assert.AreEqual(JobStepState.Skipped, job.Batches[1].Steps[0].State); // B
			Assert.AreEqual(JobStepState.Skipped, job.Batches[2].Steps[0].State); // C

			// Retry Node C
			job = Deref(await job.TryUpdateStepAsync(job.Batches[2].Id, job.Batches[2].Steps[0].Id, newRetryByUserId: UserId.Anonymous));

			Assert.AreEqual(4, job.Batches.Count);

			// Nodes A, B and C to be retried
			Assert.AreEqual(JobStepState.Ready, job.Batches[1].Steps[0].State); // A
			Assert.AreEqual(0, job.Batches[1].GroupIdx);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(0, job.Batches[1].Steps[0].NodeIdx);

			Assert.AreEqual(JobStepState.Waiting, job.Batches[2].Steps[0].State); // B
			Assert.AreEqual(1, job.Batches[2].GroupIdx);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(0, job.Batches[2].Steps[0].NodeIdx);

			Assert.AreEqual(JobStepState.Waiting, job.Batches[3].Steps[0].State); // C
			Assert.AreEqual(2, job.Batches[3].GroupIdx);
			Assert.AreEqual(1, job.Batches[3].Steps.Count);
			Assert.AreEqual(0, job.Batches[3].Steps[0].NodeIdx);
		}

		[TestMethod]
		public async Task TestRetryCompletedNodeDoesNotRetryOptionalDependencyAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = [];

			NewGroup groupA = AddGroup(newGroups);
			AddNode(groupA, "Node A", null);

			// Node B has ONLY optional dependency on Node A
			NewGroup groupB = AddGroup(newGroups);
			AddNode(groupB, "Node B", null, ["Node A"]);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Node B");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Node A
			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Completed, JobStepOutcome.Failure));

			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State);
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[0].Steps[0].Outcome);
			Assert.IsFalse(job.Batches[0].Steps[0].AbortRequested);
			Assert.AreEqual(JobStepState.Ready, job.Batches[1].Steps[0].State);

			// Node B
			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Failure);

			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[1].Steps[0].Outcome);

			// Retry Node B
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(JobStepState.Ready, job.Batches[2].Steps[0].State);
			Assert.AreEqual(1, job.Batches[2].GroupIdx);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(0, job.Batches[2].Steps[0].NodeIdx);
		}

		[TestMethod]
		public async Task TryUpdateGraphAsync()
		{
			// Create the initial graph
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			List<NewGroup> groups = new List<NewGroup>();
			groups.Add(new NewGroup("Test", new List<NewNode> { new NewNode("Initial Node", outputs: new List<string> { "#InitialOutput" }) }));
			groups.Add(new NewGroup("Test", new List<NewNode> { new NewNode("Split Multi-Agent Cook", inputs: new List<string> { "#InitialOutput" }, inputDependencies: new List<string> { "Initial Node" }, outputs: new List<string> { "#SplitOutput" }) }));
			groups.Add(new NewGroup("Test", new List<NewNode> { new NewNode("Gather", inputs: new List<string> { "#SplitOutput" }, inputDependencies: new List<string> { "Split Multi-Agent Cook" }) }));

			IGraph graph1 = await GraphCollection.AppendAsync(baseGraph, groups);

			groups.Insert(2, new NewGroup("Test", new List<NewNode> { new NewNode("Cook Item 1", inputs: new List<string> { "#SplitOutput" }, inputDependencies: new List<string> { "Split Multi-Agent Cook" }, outputs: new List<string> { "#CookOutput1" }) }));
			groups.Insert(3, new NewGroup("Test", new List<NewNode> { new NewNode("Cook Item 2", inputs: new List<string> { "#SplitOutput" }, inputDependencies: new List<string> { "Split Multi-Agent Cook" }, outputs: new List<string> { "#CookOutput2" }) }));
			groups[4].Nodes[0].Inputs = new List<string> { "#CookOutput1", "#CookOutput2" };
			groups[4].Nodes[0].InputDependencies = new List<string> { "Split Multi-Agent Cook", "Cook Item 1", "Cook Item 2" };

			IGraph graph2 = await GraphCollection.AppendAsync(baseGraph, groups);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Gather");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			job = Deref(await job.TryUpdateGraphAsync(graph1));
			Assert.AreEqual(4, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State); // Setup Build
			Assert.AreEqual(JobStepState.Ready, job.Batches[1].Steps[0].State); // Initial Node
			Assert.AreEqual(JobStepState.Waiting, job.Batches[2].Steps[0].State); // Split Multi-Agent Cook
			Assert.AreEqual(JobStepState.Waiting, job.Batches[3].Steps[0].State); // Gather

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Initial Node

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Success); // Split Multi-Agent Cook

			Assert.AreEqual(4, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State); // Setup Build
			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State); // Initial Node
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State); // Split Multi-Agent Cook
			Assert.AreEqual(JobStepState.Ready, job.Batches[3].Steps[0].State); // Gather

			job = Deref(await job.TryUpdateGraphAsync(graph2));

			Assert.AreEqual(6, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State); // Setup Build
			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State); // Initial Node
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State); // Split Multi-Agent Cook
			Assert.AreEqual(JobStepState.Ready, job.Batches[3].Steps[0].State); // Cook Item 1
			Assert.AreEqual(JobStepState.Ready, job.Batches[4].Steps[0].State); // Cook Item 2
			Assert.AreEqual(JobStepState.Waiting, job.Batches[5].Steps[0].State); // Gather

			job = await RunBatchAsync(job, 3);
			job = await RunStepAsync(job, 3, 0, JobStepOutcome.Success); // Split Multi-Agent Cook

			Assert.AreEqual(6, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State); // Cook Item 1
			Assert.AreEqual(JobStepState.Ready, job.Batches[4].Steps[0].State); // Cook Item 2
			Assert.AreEqual(JobStepState.Waiting, job.Batches[5].Steps[0].State); // Gather

			job = await RunBatchAsync(job, 4);
			job = await RunStepAsync(job, 4, 0, JobStepOutcome.Success); // Split Multi-Agent Cook

			Assert.AreEqual(6, job.Batches.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[3].Steps[0].State); // Cook Item 1
			Assert.AreEqual(JobStepState.Completed, job.Batches[4].Steps[0].State); // Cook Item 2
			Assert.AreEqual(JobStepState.Ready, job.Batches[5].Steps[0].State); // Gather
		}

		[TestMethod]
		public async Task UpdateGraphWithModifiedSkippedStepAsync()
		{
			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			List<NewGroup> groups =
			[
				new NewGroup("Test", [new NewNode(name: "Compile Item 1", outputs: ["#CompileOutput1"])]),
				new NewGroup("Test", [new NewNode(name: "Compile Item 2", outputs: ["#CompileOutput1"])]),
				new NewGroup("Test", [new NewNode(name: "Gather", inputDependencies: ["Compile Item 1", "Compile Item 2"])])
			];

			IGraph graph1 = await GraphCollection.AppendAsync(baseGraph, groups);

			groups.Insert(2, new NewGroup("Test", [new NewNode(name: "Generate Metadata", inputDependencies: ["Compile Item 2"])]));
			groups.Insert(3, new NewGroup("Test", [new NewNode(name: "Process Content", outputs: ["#ProcessContent"], inputDependencies: ["Compile Item 2"])]));
			groups.Insert(4, new NewGroup("Test", [new NewNode(name: "Publish Item 2", outputs: ["#PublishItem2"], inputDependencies: ["Process Content", "Generate Metadata", "Compile Item 2"])]));
			groups.Insert(5, new NewGroup("Test", [new NewNode(name: "Store Symbols Item 1", outputs: ["#StoreSymbolsItem1"], inputDependencies: ["Compile Item 1"])]));
			groups.Insert(6, new NewGroup("Test", [new NewNode(name: "Publish Item 1", outputs: ["#PublishItem1"], inputDependencies: ["Store Symbols Item 1"])]));
			// Modify step Gather, now at position 7 
			groups[7].Nodes[0].Inputs = ["#PublishItem1", "#PublishItem2"];
			groups[7].Nodes[0].InputDependencies = ["Publish Item 1", "Publish Item 2"];

			IGraph graph2 = await GraphCollection.AppendAsync(baseGraph, groups);

			CreateJobOptions options = new();
			options.Arguments.Add("-Target=Gather");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			Assert.AreEqual(1, job.Batches.Count);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			job = Deref(await job.TryUpdateGraphAsync(graph1));

			Assert.AreEqual(4, job.Batches.Count);

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Compile Item 1

			// Fail this step to cause dependent steps to be marked as skipped and ignored when update graph is called
			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // Compile Item 2

			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[0].State); // Gather

			job = Deref(await job.TryUpdateGraphAsync(graph2));

			Assert.AreEqual(9, job.Batches.Count);

			// Check the state of the new dependent steps of the failed step are marked as skipped
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State); // Setup Build
			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State); // Compile Item 1
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State); // Compile Item 2
			Assert.AreEqual(JobStepState.Skipped,   job.Batches[3].Steps[0].State); // Generate Metadata
			Assert.AreEqual(JobStepState.Skipped,   job.Batches[4].Steps[0].State); // Process Content
			Assert.AreEqual(JobStepState.Skipped,   job.Batches[5].Steps[0].State); // Publish Item 2
			Assert.AreEqual(JobStepState.Ready,     job.Batches[6].Steps[0].State); // Store Symbols Item 1
			Assert.AreEqual(JobStepState.Waiting,   job.Batches[7].Steps[0].State); // Publish Item 1
			Assert.AreEqual(JobStepState.Skipped,   job.Batches[8].Steps[0].State); // Gather

			// Check the outcome of the new dependent steps of the failed step are marked as failed
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[0].Steps[0].Outcome); // Setup Build
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[1].Steps[0].Outcome); // Compile Item 1
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[2].Steps[0].Outcome); // Compile Item 2
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[3].Steps[0].Outcome); // Generate Metadata
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[4].Steps[0].Outcome); // Process Content
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[5].Steps[0].Outcome); // Publish Item 2
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[6].Steps[0].Outcome); // Store Symbols Item 1
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[7].Steps[0].Outcome); // Publish Item 1
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[8].Steps[0].Outcome); // Gather

			// Check the remaining steps can complete OK
			job = await RunBatchAsync(job, 6);
			job = await RunStepAsync(job, 6, 0, JobStepOutcome.Success); // Store Symbols Item 1
			job = await RunBatchAsync(job, 7);
			job = await RunStepAsync(job, 7, 0, JobStepOutcome.Success); // Publish Item 1

			Assert.AreEqual(JobStepState.Completed, job.Batches[6].Steps[0].State); // Store Symbols Item 1
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[6].Steps[0].Outcome);
			Assert.AreEqual(JobStepState.Completed, job.Batches[7].Steps[0].State); // Publish Item 1
			Assert.AreEqual(JobStepOutcome.Success, job.Batches[7].Steps[0].Outcome);
		}

		[TestMethod]
		public async Task UpdateGraphOnFailedBatchAsync()
		{
			// Create the initial graph
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			List<NewGroup> groups = new List<NewGroup>();
			groups.Add(new NewGroup("Test", new List<NewNode> { new NewNode("Initial Node", outputs: new List<string> { "#InitialOutput" }) }));

			IGraph graph1 = await GraphCollection.AppendAsync(baseGraph, groups);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Gather");
			options.Arguments.Add("-Target=Initial Node");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Try a batch and fail it
			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			// Start the replacement batch and update the graph
			job = await RunBatchAsync(job, 1);
			job = Deref(await job.TryUpdateGraphAsync(graph1));

			// Validate the new job state
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(0, job.Batches[0].Steps.Count);

			Assert.AreEqual(0, job.Batches[1].GroupIdx);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(0, job.Batches[1].Steps[0].NodeIdx);

			Assert.AreEqual(1, job.Batches[2].GroupIdx);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(0, job.Batches[2].Steps[0].NodeIdx);
		}

		[TestMethod]
		public async Task TryAssignLeaseTestAsync()
		{
			Fixture fixture = await CreateFixtureAsync();

			SessionId sessionId1 = SessionIdUtils.GenerateNewId();
			await fixture.Job1.TryAssignLeaseAsync(0, new PoolId("foo"), fixture.Agent1.Id,
				sessionId1, new LeaseId(BinaryIdUtils.CreateNew()), LogIdUtils.GenerateNewId());

			SessionId sessionId2 = SessionIdUtils.GenerateNewId();
			IJob job = (await JobCollection.GetAsync(fixture.Job1.Id))!;
			await job.TryAssignLeaseAsync(0, new PoolId("foo"), fixture.Agent1.Id,
				sessionId2, new LeaseId(BinaryIdUtils.CreateNew()), LogIdUtils.GenerateNewId());

			// Manually verify the log output
		}

		[TestMethod]
		public Task LostLeaseTestWithDependencyAsync()
		{
			return LostLeaseTestInternalAsync(true);
		}

		[TestMethod]
		public Task LostLeaseTestWithoutDependencyAsync()
		{
			return LostLeaseTestInternalAsync(false);
		}

		public async Task LostLeaseTestInternalAsync(bool hasDependency)
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Step 1", null);
			AddNode(initialGroup, "Step 2", hasDependency ? new[] { "Step 1" } : null);
			AddNode(initialGroup, "Step 3", new[] { "Step 2" });

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Step 1
			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Step 2

			// Force an error executing the batch
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[1].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			// Check that it restarted all three nodes
			IJob newJob = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(3, newJob.Batches.Count);
			Assert.AreEqual(1, newJob.Batches[2].GroupIdx);

			if (hasDependency)
			{
				Assert.AreEqual(3, newJob.Batches[2].Steps.Count);

				Assert.AreEqual(0, newJob.Batches[2].Steps[0].NodeIdx);
				Assert.AreEqual(1, newJob.Batches[2].Steps[1].NodeIdx);
				Assert.AreEqual(2, newJob.Batches[2].Steps[2].NodeIdx);
			}
			else
			{
				Assert.AreEqual(2, newJob.Batches[2].Steps.Count);

				Assert.AreEqual(1, newJob.Batches[2].Steps[0].NodeIdx);
				Assert.AreEqual(2, newJob.Batches[2].Steps[1].NodeIdx);
			}
		}

		[TestMethod]
		public async Task IncompleteBatchAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);
			Assert.AreEqual(1, job.Batches.Count);

			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(0, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[1].State);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);

			job = Deref(await job.TryUpdateBatchAsync(job.Batches[1].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(0, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[1].State);
			Assert.AreEqual(0, job.Batches[1].Steps.Count);
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[2].State);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);

			job = Deref(await job.TryUpdateBatchAsync(job.Batches[2].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(0, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[1].State);
			Assert.AreEqual(0, job.Batches[1].Steps.Count);
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[2].State);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(JobStepState.Skipped, job.Batches[2].Steps[0].State);
		}

		[TestMethod]
		public async Task IncompleteBatchRunningAsync()
		{
			// Same as IncompleteBatchAsync, but with steps moved to running state

			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);
			Assert.AreEqual(1, job.Batches.Count);

			// First retry

			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newState: JobStepState.Running));
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(2, job.Batches.Count);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[0].Steps[0].Error);

			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[1].State);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);

			// Second retry

			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, newState: JobStepState.Running));
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[1].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(3, job.Batches.Count);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[0].Steps[0].Error);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[1].State);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[1].Steps[0].Error);

			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[2].State);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);

			// Check it doesn't retry a third time

			job = Deref(await job.TryUpdateStepAsync(job.Batches[2].Id, job.Batches[2].Steps[0].Id, newState: JobStepState.Running));
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[2].Id, null, JobStepBatchState.Complete, JobStepBatchError.Incomplete));

			job = (await JobCollection.GetAsync(job.Id))!;
			Assert.AreEqual(3, job.Batches.Count);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[0].State);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[0].Steps[0].Error);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[1].State);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[1].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[1].Steps[0].Error);

			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[2].State);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[2].Steps[0].State);
			Assert.AreEqual(JobStepError.Incomplete, job.Batches[2].Steps[0].Error);
		}

		[TestMethod]
		public async Task RetryStepSameGroupAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);
			AddNode(group, "Step 2", new[] { "Step 1" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 2");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Fail the first step
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Failure);

			Assert.AreEqual(1, job.Batches.Count);
			Assert.AreEqual(2, job.Batches[0].Steps.Count);
			Assert.AreEqual(JobStepState.Completed, job.Batches[0].Steps[0].State);
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[0].Steps[0].Outcome);
			Assert.AreEqual(JobStepState.Skipped, job.Batches[0].Steps[1].State);
			Assert.AreEqual(JobStepOutcome.Failure, job.Batches[0].Steps[1].Outcome);

			// Retry the failed step (expect: new batch containing the retried step is created, skipped steps from original batch are removed)
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(2, job.Batches[1].Steps.Count);

			IJobStep step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[1].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			// Fail the retried step
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Failure);
			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(2, job.Batches[1].Steps.Count);

			step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Skipped, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			// Retry the failed step for a second time
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(2, job.Batches[2].Steps.Count);

			step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[2].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[2].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);
		}

		[TestMethod]
		public async Task RetryStepDifferentGroupAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);

			NewGroup group2 = AddGroup(newGroups);
			AddNode(group2, "Step 2", new[] { "Step 1" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 2");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Fail the first step
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Failure);
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, null));

			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);

			IJobStep step = job.Batches[0].Steps[0];
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(JobStepState.Skipped, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			// Retry the failed step
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(1, job.Batches[1].Steps.Count);
			Assert.AreEqual(1, job.Batches[2].Steps.Count);

			step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, job.Batches[0].GroupIdx);
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(0, job.Batches[1].GroupIdx);
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[2].Steps[0];
			Assert.AreEqual(1, job.Batches[2].GroupIdx);
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);
		}

		[TestMethod]
		public async Task RetryDownstreamStepAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);
			AddNode(group, "Step 2", new[] { "Step 1" });
			AddNode(group, "Step 3", new[] { "Step 2" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Fail the first step
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Failure);

			Assert.AreEqual(1, job.Batches.Count);
			Assert.AreEqual(3, job.Batches[0].Steps.Count);

			IJobStep step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[0].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Skipped, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[0].Steps[2];
			Assert.AreEqual(2, step.NodeIdx);
			Assert.AreEqual(JobStepState.Skipped, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			// Retry the last failed step. All the failed upstream steps should run again.
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[2].Id, newRetryByUserId: UserId.Anonymous));
			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[0].Steps.Count);
			Assert.AreEqual(3, job.Batches[1].Steps.Count);

			step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Failure, step.Outcome);

			step = job.Batches[1].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[1].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[1].Steps[2];
			Assert.AreEqual(2, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);
		}

		[TestMethod]
		public async Task RetryStepAfterBatchFailAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);
			AddNode(group, "Step 2", new[] { "Step 1" });
			AddNode(group, "Step 3", new[] { "Step 2" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			IJobStepBatch initialBatch = job.Batches[0];

			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateBatchAsync(initialBatch.Id, null, JobStepBatchState.Complete, JobStepBatchError.Cancelled));

			UpdateStepResponse response = ResultToValue(await JobsController.UpdateStepAsync(job.Id, initialBatch.Id, initialBatch.Steps[2].Id, new UpdateStepRequest { Retry = true }));

			job = Deref(await JobCollection.GetAsync(job.Id));

			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(3, job.Batches[1].Steps.Count);
		}

		[TestMethod]
		public async Task UpdateBatchesDuringJobAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);
			AddNode(group, "Step 2", new[] { "Step 1" });
			AddNode(group, "Step 3", new[] { "Step 2" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Pass the first step
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success);

			Assert.AreEqual(1, job.Batches.Count);
			Assert.AreEqual(3, job.Batches[0].Steps.Count);

			IJobStep step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);
			Assert.AreEqual(JobStepOutcome.Success, step.Outcome);

			step = job.Batches[0].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);

			step = job.Batches[0].Steps[2];
			Assert.AreEqual(2, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);

			// Update the priority of the second step. This should trigger a refresh of the graph structure, but NOT cause any new steps to be created.
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[1].Id, newPriority: Priority.High));
			Assert.AreEqual(1, job.Batches.Count);
			Assert.AreEqual(3, job.Batches[0].Steps.Count);

			step = job.Batches[0].Steps[0];
			Assert.AreEqual(0, step.NodeIdx);
			Assert.AreEqual(JobStepState.Completed, step.State);

			step = job.Batches[0].Steps[1];
			Assert.AreEqual(1, step.NodeIdx);
			Assert.AreEqual(JobStepState.Ready, step.State);

			step = job.Batches[0].Steps[2];
			Assert.AreEqual(2, step.NodeIdx);
			Assert.AreEqual(JobStepState.Waiting, step.State);
		}

		[TestMethod]
		public async Task UnknownShelfAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");
			options.Arguments.Add("-Target=Step 3");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);
			Assert.AreEqual(1, job.Batches.Count);

			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Complete, JobStepBatchError.UnknownShelf));

			Assert.AreEqual(1, job.Batches.Count);
		}

		[TestMethod]
		public async Task UpdateDuringBatchStartAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Compile Editor");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Update Version Files", null);
			AddNode(initialGroup, "Compile Editor", new[] { "Update Version Files" });

			NewGroup compileGroup = AddGroup(newGroups);
			AddNode(compileGroup, "Compile Client", new[] { "Update Version Files" });

			// Check that we're only building the editor
			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups, null, null);
			job = Deref(await job.TryUpdateGraphAsync(graph));

			// Move the initial batch to the starting state
			job = await UpdateBatchAsync(job, 1, JobStepBatchState.Starting);
			Assert.AreEqual(2, job.Batches.Count);
			Assert.AreEqual(1, job.Batches[1].GroupIdx);
			Assert.AreEqual(2, job.Batches[1].Steps.Count);
			Assert.AreEqual(0, job.Batches[1].Steps[0].NodeIdx);
			Assert.AreEqual(1, job.Batches[1].Steps[1].NodeIdx);
			JobStepBatchId batchId = job.Batches[1].Id;
			LeaseId leaseId = job.Batches[1].LeaseId!.Value;

			// Force an update on the batches for the job
			options.Arguments.Add("-Target=Compile Client");
			job = Deref(await job.TryUpdateJobAsync(arguments: options.Arguments));

			// Check that we're still using the original batch, and have added one more
			Assert.AreEqual(3, job.Batches.Count);
			Assert.AreEqual(batchId, job.Batches[1].Id);
			Assert.AreEqual(leaseId, job.Batches[1].LeaseId);
			Assert.AreEqual(1, job.Batches[1].GroupIdx);
			Assert.AreEqual(2, job.Batches[2].GroupIdx);
		}

		[TestMethod]
		public async Task AddArtifactsAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			List<NewGraphArtifact> newArtifacts = new List<NewGraphArtifact>();
			newArtifacts.Add(new NewGraphArtifact(new ArtifactName("foo"), new ArtifactType("type"), "hello world", "Engine/Source", new List<string>(), new List<string>(), null, "fileset"));

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newArtifactRequests: newArtifacts);
			Assert.AreEqual(1, graph.Artifacts.Count);
			Assert.AreEqual("foo", graph.Artifacts[0].Name.ToString());
			Assert.AreEqual("type", graph.Artifacts[0].Type.ToString());
			Assert.AreEqual("hello world", graph.Artifacts[0].Description);
			Assert.AreEqual("Engine/Source", graph.Artifacts[0].BasePath);
			Assert.AreEqual("fileset", graph.Artifacts[0].OutputName);
		}

		[TestMethod]
		public async Task FindJobsAsync()
		{
			IGraph graph = await GraphCollection.AddAsync(_template1);

			List<IJob> jobs = [];
			for (int i = 0; i < 10; i++)
			{
				jobs.Add(await AddJobAsync(graph, $"{i}"));
			}

			jobs[2] = await RunBatchAsync(jobs[2], 0);
			await RunStepAsync(jobs[2], 0, 0, JobStepOutcome.Success); // Setup Build

			{
				IReadOnlyList<IJob> jobsFound = await JobCollection.FindAsync(new FindJobOptions());
				Assert.AreEqual(jobs.Count, jobsFound.Count);
			}

			{
				IReadOnlyList<IJob> jobsFound = await JobCollection.FindAsync(new FindJobOptions() { Outcome = [JobStepOutcome.Success] });
				Assert.AreEqual(1, jobsFound.Count);
			}
		}

		[TestMethod]
		public async Task TestParentChildStepsAsync()
		{
			ProjectId projectId = new ProjectId("ue5");
			StreamId streamId = new StreamId("ue5-main");
			TemplateId templateRefId1 = new TemplateId("template1");
			TemplateId templateRefId2 = new TemplateId("template2");

			StreamConfig streamConfig = new StreamConfig { Id = streamId };
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId1, Name = "Test Template 1" });
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId2, Name = "Test Template 2" });
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { templateRefId1, templateRefId2 } });

			ProjectConfig projectConfig = new ProjectConfig { Id = projectId };
			projectConfig.Streams.Add(streamConfig);

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			CreateJobRequest request = new CreateJobRequest(new StreamId("ue5-main"), new TemplateId("template1"));
			request.CommitId = CommitIdWithOrder.FromPerforceChange(123);

			ActionResult<CreateJobResponse>  parentJobResult = await JobsController.CreateJobAsync(request);

			Assert.IsNotNull(parentJobResult.Value);
			JobId parentJobId = JobId.Parse(parentJobResult.Value!.Id);

			IJob parentJob = Deref(await JobCollection.GetAsync(parentJobId));
			
			request.ParentJobId = parentJob.Id.ToString();
			request.ParentJobStepId = parentJob.Batches[0].Steps[0].Id.ToString();

			ActionResult<CreateJobResponse> childJobResult = await JobsController.CreateJobAsync(request);

			Assert.IsNotNull(childJobResult.Value);
			JobId childJobId = JobId.Parse(childJobResult.Value!.Id);

			IJob childJob = Deref(await JobCollection.GetAsync(childJobId));

			Assert.AreEqual(parentJobId, childJob.ParentJobId);
			Assert.AreEqual(parentJob.Batches[0].Steps[0].Id, childJob.ParentJobStepId);

			parentJob = Deref(await JobCollection.GetAsync(parentJobId));

			Assert.AreEqual(1, parentJob.Batches[0].Steps[0].SpawnedJobs?.Count);
			Assert.AreEqual(parentJob.Batches[0].Steps[0].SpawnedJobs![0], childJobId);

		}

		[TestMethod]
		public async Task JobMetaDataAsync()
		{
			ProjectId projectId = new ProjectId("ue5");
			StreamId streamId = new StreamId("ue5-main");
			TemplateId templateRefId1 = new TemplateId("template1");
			TemplateId templateRefId2 = new TemplateId("template2");

			StreamConfig streamConfig = new StreamConfig { Id = streamId };
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId1, Name = "Test Template 1" });
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId2, Name = "Test Template 2" });
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { templateRefId1, templateRefId2 } });

			ProjectConfig projectConfig = new ProjectConfig { Id = projectId };
			projectConfig.Streams.Add(streamConfig);

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			CreateJobRequest request = new CreateJobRequest(new StreamId("ue5-main"), new TemplateId("template1"));
			request.CommitId = CommitIdWithOrder.FromPerforceChange(123);

			ActionResult<CreateJobResponse> jobResult = await JobsController.CreateJobAsync(request);

			Assert.IsNotNull(jobResult.Value);
			JobId jobId = JobId.Parse(jobResult.Value!.Id);

			IJob? job = Deref(await JobCollection.GetAsync(jobId));

			PutJobMetadataRequest metaRequest = new PutJobMetadataRequest() { JobMetaData = new List<string> { "TestMeta=42" }, StepMetaData = new Dictionary<string, List<string>> { { job.Batches[0].Steps[0].Id.ToString(), new List<string> { "TestStepMeta=2025"} } } };

			await JobsController.UpdateJobMetaDataAsync(jobId, metaRequest);

			job = Deref(await JobCollection.GetAsync(jobId));

			Assert.IsNotNull(job);
			Assert.AreEqual(1, job.Metadata.Count);
			Assert.AreEqual("TestMeta=42", job.Metadata[0]);

			Assert.AreEqual(1, job.Batches[0].Steps[0].Metadata.Count);
			Assert.AreEqual("TestStepMeta=2025", job.Batches[0].Steps[0].Metadata[0]);
			
			await JobStepRefCollection.UpdateAsync(job, job.Batches[0], job.Batches[0].Steps[0], job.Graph, null);

			IJobStepRef? jobStepRef = await JobStepRefCollection.FindAsync(jobId, job.Batches[0].Id, job.Batches[0].Steps[0].Id);

			Assert.IsNotNull(jobStepRef);
			Assert.IsNotNull(jobStepRef.Metadata);
			Assert.AreEqual(1, jobStepRef.Metadata.Count);
			Assert.AreEqual("TestStepMeta=2025", jobStepRef.Metadata[0]);
		}

		private async Task<IJob> AddJobAsync(IGraph graph, string commitId)
		{
			return await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), _streamId, _templateId1, _template1.Hash, graph, "jobName", new CommitId(commitId), null, new CreateJobOptions());
		}

		[TestMethod]
		public async Task RetryRunningStepShouldAbortOldStepAsync()
		{
			// Setup: Create a job with a single step
			List<NewGroup> newGroups = new List<NewGroup>();
			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Run the batch and start the step (but don't complete it)
			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));

			Assert.AreEqual(JobStepState.Running, job.Batches[0].Steps[0].State);

			// Retry the RUNNING step
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newRetryByUserId: UserId.Anonymous));

			// Verify: Old step should be marked for abort and a new batch should be created for the retry
			Assert.AreEqual(2, job.Batches.Count, "Should have old batch + new retry batch");
			Assert.IsTrue(job.Batches[0].Steps[0].AbortRequested, "Old running step should be aborted when retried");
		}

		[TestMethod]
		public async Task AbortRunningStepShouldSetBatchErrorAsync()
		{
			// Setup: Create a job with a single step
			List<NewGroup> newGroups = new List<NewGroup>();
			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Run the batch and start the step
			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));

			Assert.AreEqual(JobStepState.Running, job.Batches[0].Steps[0].State);

			// Abort the running step
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newAbortRequested: true, newAbortByUserId: UserId.Anonymous));

			// Verify: Step is aborted and batch error is set
			Assert.IsTrue(job.Batches[0].Steps[0].AbortRequested, "Step should have AbortRequested=true");
			Assert.AreEqual(JobStepBatchError.Cancelled, job.Batches[0].Error, "Batch error should be Cancelled when running step is aborted");
		}

		[TestMethod]
		public async Task RetryRunningStepShouldMarkOldBatchAsNoLongerNeededAsync()
		{
			// Setup: Create a job with a single step
			List<NewGroup> newGroups = new List<NewGroup>();
			NewGroup group = AddGroup(newGroups);
			AddNode(group, "Step 1", null);

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups, null, null);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step 1");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), new StreamId("ue4-main"), new TemplateId("test-build"), ContentHash.SHA1("hello"), graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			// Run the batch and start the step (but don't complete it)
			job = await RunBatchAsync(job, 0);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));

			Assert.AreEqual(JobStepState.Running, job.Batches[0].Steps[0].State);

			// Retry the running step
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, newRetryByUserId: UserId.Anonymous));

			// Verify: Old batch should be marked NoLongerNeeded
			Assert.AreEqual(2, job.Batches.Count, "Should have old batch + new retry batch");
			Assert.AreEqual(JobStepBatchError.NoLongerNeeded, job.Batches[0].Error, "Old batch should be marked NoLongerNeeded when step is retried");
			Assert.AreEqual(JobStepBatchError.None, job.Batches[1].Error, "New batch should have no error");
		}

		[TestMethod]
		public async Task UpdateGraphWhileBatchRunningPreservesDependenciesAsync()
		{
			// Regression test: When UpdateGraphForHorde runs while a batch is already executing
			// (due to RunEarly), nodes added to that batch with dependencies on nodes in new
			// (later) batches must stay in Waiting state. Previously, RefreshDependentJobSteps
			// built the stepForNode dictionary incrementally during iteration, so dependencies
			// in later batches were invisible and the step was incorrectly marked Ready.

			Mock<ITemplate> templateMock = new(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			// Initial graph: single group with a RunEarly node
			List<NewGroup> initialGroups = [];
			NewGroup cookGroup = AddGroup(initialGroups);
			AddNode(cookGroup, "Cook", null, n => { n.RunEarly = true; });

			IGraph graph1 = await GraphCollection.AppendAsync(baseGraph, initialGroups);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Cook");
			options.Arguments.Add("-Target=Download Artifacts");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), _streamId, _templateId1, ContentHash.SHA1("hello"), baseGraph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);

			job = await RunBatchAsync(job, 0); // Setup Build
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build

			job = Deref(await job.TryUpdateGraphAsync(graph1));

			// Cook batch should be Ready due to RunEarly (Setup Build + Cook)
			Assert.HasCount(2, job.Batches);
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[1].State); // Cook batch

			// Start the Cook batch and step (simulating RunEarly agent pickup)
			job = await RunBatchAsync(job, 1);
			job = Deref(await job.TryUpdateStepAsync(job.Batches[1].Id, job.Batches[1].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));
			Assert.AreEqual(JobStepBatchState.Running, job.Batches[1].State);
			Assert.AreEqual(JobStepState.Running, job.Batches[1].Steps[0].State);

			// Now build an updated graph that adds a new node in a new group (Upload Artifacts)
			// and a dependent node in Cook's group. The Upload Artifacts group must be declared
			// first so the graph can resolve the dependency reference; but at runtime the
			// new batch for Upload Artifacts will be appended AFTER the existing Running batch,
			// which is the core of the bug.
			List<NewGroup> updatedGroups = [];

			NewGroup uploadGroup = AddGroup(updatedGroups);
			AddNode(uploadGroup, "Upload Artifacts", null);

			NewGroup cookGroupUpdated = AddGroup(updatedGroups);
			AddNode(cookGroupUpdated, "Cook", null, n => { n.RunEarly = true; });
			AddNode(cookGroupUpdated, "Download Artifacts", ["Upload Artifacts"]);

			IGraph graph2 = Deref(await GraphCollection.AppendAsync(baseGraph, updatedGroups));

			// Update the graph while the Cook batch is Running
			job = Deref(await job.TryUpdateGraphAsync(graph2));

			Assert.HasCount(3, job.Batches);
			// Upload Artifacts has no dependencies and should be Ready.
			Assert.AreEqual(JobStepState.Ready, job.Batches[2].Steps[0].State);
			// Download Artifacts should remain Waiting until Upload Artifacts completes.
			// If this is Ready, the dependency was silently dropped during graph update.
			Assert.AreEqual(JobStepState.Waiting, job.Batches[1].Steps[1].State);
		}
	}
}
