// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Jobs
{
	[TestClass]
	public class JobExpirationTests : BuildTestSetup
	{
		readonly ProjectId _projectId = new ProjectId("ue5");
		readonly StreamId _streamId = new StreamId("ue5-main");
		readonly TemplateId _templateId = new TemplateId("template1");

		async Task<IJob> CreateJobAsync(StreamConfig? streamConfig = null, ProjectConfig? projectConfig = null)
		{
			streamConfig ??= new StreamConfig { Id = _streamId };
			if (streamConfig.JobOptions.ExpireAfterDays == null)
			{
				streamConfig.JobOptions.ExpireAfterDays = 2;
			}
			if (streamConfig.Templates.Count == 0)
			{
				streamConfig.Templates.Add(new TemplateRefConfig { Id = _templateId, Name = "Test Template" });
			}

			projectConfig ??= new ProjectConfig { Id = _projectId };
			if (!projectConfig.Streams.Contains(streamConfig))
			{
				projectConfig.Streams.Add(streamConfig);
			}

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			CreateJobOptions options = new CreateJobOptions();
			options.PreflightCommitId = CommitId.FromPerforceChange(999);

			ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);

			IGraph graph = await GraphCollection.AddAsync(template);

			return await JobService.CreateJobAsync(null, streamConfig, _templateId, template.Hash, graph, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
		}

		[TestMethod]
		public async Task TestJobExpiryAsync()
		{
			await ServiceProvider.GetRequiredService<JobExpirationService>().StartAsync(default);

			IJob job = await CreateJobAsync();

			IJob? newJob = await JobCollection.GetAsync(job.Id);
			Assert.IsNotNull(newJob);

			await Clock.AdvanceAsync(TimeSpan.FromDays(1.0));

			newJob = await JobCollection.GetAsync(job.Id);
			Assert.IsNotNull(newJob);

			await Clock.AdvanceAsync(TimeSpan.FromDays(2.0));

			newJob = await JobCollection.GetAsync(job.Id);
			Assert.IsNull(newJob);
		}

		[TestMethod]
		public async Task EnableJobExpirationFalseBlocksDeletionAsync()
		{
			await ServiceProvider.GetRequiredService<JobExpirationService>().StartAsync(default);

			StreamConfig streamConfig = new StreamConfig { Id = _streamId };
			streamConfig.RetentionOptions.EnableJobExpiration = false;

			IJob job = await CreateJobAsync(streamConfig);

			// Advance well past expiry — job must survive because the safety valve is set
			await Clock.AdvanceAsync(TimeSpan.FromDays(10.0));

			IJob? surviving = await JobCollection.GetAsync(job.Id);
			Assert.IsNotNull(surviving, "Job should not be deleted when EnableJobExpiration == false");
		}

		[TestMethod]
		public async Task MaxJobDeletionsPerTickThrottlesDeletionAsync()
		{
			JobExpirationService svc = ServiceProvider.GetRequiredService<JobExpirationService>();
			await svc.StartAsync(default);

			StreamConfig streamConfig = new StreamConfig { Id = _streamId };
			streamConfig.RetentionOptions.MaxJobDeletionsPerTick = 3;

			ProjectConfig projectConfig = new ProjectConfig { Id = _projectId };

			// Create 5 jobs (all will expire at once)
			List<IJob> jobs = new List<IJob>();
			for (int i = 0; i < 5; i++)
			{
				projectConfig.Streams.Clear();
				jobs.Add(await CreateJobAsync(streamConfig, projectConfig));
			}

			// Advance past expiry — only 3 should be deleted in the first tick
			await Clock.AdvanceAsync(TimeSpan.FromDays(3.0));

			int remaining = 0;
			foreach (IJob job in jobs)
			{
				if (await JobCollection.GetAsync(job.Id) != null)
				{
					remaining++;
				}
			}

			Assert.IsTrue(remaining >= 2, $"Expected at least 2 jobs to survive throttle, but only {remaining} remained");
		}

		[TestMethod]
		public async Task MaxJobDeletionsPerTickZeroMeansUnlimitedAsync()
		{
			JobExpirationService svc = ServiceProvider.GetRequiredService<JobExpirationService>();
			await svc.StartAsync(default);

			StreamConfig streamConfig = new StreamConfig { Id = _streamId };
			streamConfig.RetentionOptions.MaxJobDeletionsPerTick = 0;

			ProjectConfig projectConfig = new ProjectConfig { Id = _projectId };

			// Create 5 jobs
			List<IJob> jobs = new List<IJob>();
			for (int i = 0; i < 5; i++)
			{
				projectConfig.Streams.Clear();
				jobs.Add(await CreateJobAsync(streamConfig, projectConfig));
			}

			// Advance past expiry — all should be deleted (0 = unlimited)
			await Clock.AdvanceAsync(TimeSpan.FromDays(3.0));

			foreach (IJob job in jobs)
			{
				Assert.IsNull(await JobCollection.GetAsync(job.Id), $"Job {job.Id} should have been deleted when MaxJobDeletionsPerTick == 0");
			}
		}

		[TestMethod]
		public async Task RetentionOptionsCascadeFromProjectToStreamAsync()
		{
			await ServiceProvider.GetRequiredService<JobExpirationService>().StartAsync(default);

			// Set safety valve at project level — stream inherits it
			ProjectConfig projectConfig = new ProjectConfig { Id = _projectId };
			projectConfig.RetentionOptions.EnableJobExpiration = false;

			StreamConfig streamConfig = new StreamConfig { Id = _streamId };
			// Stream does NOT set EnableJobExpiration — should inherit false from project

			IJob job = await CreateJobAsync(streamConfig, projectConfig);

			await Clock.AdvanceAsync(TimeSpan.FromDays(10.0));

			IJob? surviving = await JobCollection.GetAsync(job.Id);
			Assert.IsNotNull(surviving, "Job should survive — stream should inherit EnableJobExpiration=false from project");
		}

		[TestMethod]
		[Ignore] // See CL 50304178
		public async Task StuckRunningStepShouldBeTimedOutAsync()
		{
			await ServiceProvider.GetRequiredService<JobExpirationService>().StartAsync(default);

			IJob job = await CreateJobAsync();

			// Assign a lease to the batch and start running
			job = Deref(await job.TryAssignLeaseAsync(0, new PoolId("foo"), new AgentId("agent"), new SessionId(BinaryIdUtils.CreateNew()), new LeaseId(BinaryIdUtils.CreateNew()), new LogId(BinaryIdUtils.CreateNew())));
			job = Deref(await job.TryUpdateBatchAsync(job.Batches[0].Id, null, JobStepBatchState.Running, null));

			// Start the step (put it in running state)
			job = Deref(await job.TryUpdateStepAsync(job.Batches[0].Id, job.Batches[0].Steps[0].Id, JobStepState.Running, JobStepOutcome.Success));

			Assert.AreEqual(JobStepState.Running, job.Batches[0].Steps[0].State);
			Assert.IsFalse(job.Batches[0].Steps[0].AbortRequested, "Step should not be aborted initially");

			// Advance clock by 23 hours - step should still be running
			await Clock.AdvanceAsync(TimeSpan.FromHours(23.0));

			job = Deref(await JobCollection.GetAsync(job.Id));
			Assert.AreEqual(JobStepState.Running, job.Batches[0].Steps[0].State);
			Assert.IsFalse(job.Batches[0].Steps[0].AbortRequested, "Step should not be aborted before 24h");

			// Advance clock past 24 hours total - step should be timed out
			await Clock.AdvanceAsync(TimeSpan.FromHours(2.0));

			job = Deref(await JobCollection.GetAsync(job.Id));
			Assert.IsTrue(job.Batches[0].Steps[0].AbortRequested, "Step should be aborted after 24h timeout");
			Assert.AreEqual(JobStepError.TimedOut, job.Batches[0].Steps[0].Error, "Step error should be TimedOut");
		}
	}
}
