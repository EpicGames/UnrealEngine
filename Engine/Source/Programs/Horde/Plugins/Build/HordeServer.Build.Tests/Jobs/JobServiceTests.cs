// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Jobs;
using HordeServer.Jobs.Templates;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;

namespace HordeServer.Tests.Jobs
{
	using ProjectId = EpicGames.Horde.Projects.ProjectId;

	[TestClass]
	public class JobServiceTests : BuildTestSetup
	{
		[TestMethod]
		public async Task TestChainedJobsAsync()
		{
			ProjectId projectId = new ProjectId("ue5");
			StreamId streamId = new StreamId("ue5-main");
			TemplateId templateRefId1 = new TemplateId("template1");
			TemplateId templateRefId2 = new TemplateId("template2");

			StreamConfig streamConfig = new StreamConfig { Id = streamId };
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId1, Name = "Test Template", ChainedJobs = new List<ChainedJobTemplateConfig> { new ChainedJobTemplateConfig { TemplateId = templateRefId2, Trigger = "Setup Build" } } });
			streamConfig.Templates.Add(new TemplateRefConfig { Id = templateRefId2, Name = "Test Template" });
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { templateRefId1, templateRefId2 } });

			ProjectConfig projectConfig = new ProjectConfig { Id = projectId };
			projectConfig.Streams.Add(streamConfig);

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			CreateJobOptions options = new CreateJobOptions();
			options.PreflightCommitId = CommitId.FromPerforceChange(999);
			options.JobTriggers.AddRange(streamConfig.Templates[0].ChainedJobs!);

			ITemplate template = await TemplateCollection.GetOrAddAsync(streamConfig.Templates[0]);

			IGraph graph = await GraphCollection.AddAsync(template);

			IJob job = await JobService.CreateJobAsync(null, streamConfig, templateRefId1, template.Hash, graph, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);
			Assert.AreEqual(1, job.ChainedJobs.Count);

			job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

			Assert.IsNotNull(job.ChainedJobs[0].JobId);

			IJob? chainedJob = await JobCollection.GetAsync(job.ChainedJobs[0].JobId!.Value);
			Assert.IsNotNull(chainedJob);
			Assert.AreEqual(chainedJob!.Id, job!.ChainedJobs[0].JobId);

			Assert.AreEqual(chainedJob!.CommitId, job!.CommitId);
			Assert.AreEqual(chainedJob!.CodeCommitId, job!.CodeCommitId);
			Assert.AreEqual(chainedJob!.PreflightCommitId, job!.PreflightCommitId);
			Assert.AreEqual(chainedJob!.StartedByUserId, job!.StartedByUserId);
		}

		[TestMethod]
		public async Task ChangeQueryTestAsync()
		{
			StreamId streamId = new StreamId("ue5-main");
			StreamConfig streamConfig = new StreamConfig { Id = streamId };

			ProjectId projectId = new ProjectId("ue5");
			ProjectConfig projectConfig = new ProjectConfig { Id = projectId, Name = "UE4", Streams = new List<StreamConfig> { streamConfig } };

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects = new List<ProjectConfig> { projectConfig };

			GlobalConfig config = new GlobalConfig();
			config.Plugins.AddBuildConfig(buildConfig);
			await SetConfigAsync(config);

			IUser user = await UserCollection.FindOrAddUserByLoginAsync("Bob");

			PerforceService.AddChange(streamId, 1000, user, "", new[] { "Foo.cpp" });
			PerforceService.AddChange(streamId, 1001, user, "", new[] { "Bar.cpp" });
			PerforceService.AddChange(streamId, 1002, user, "", new[] { "Baz.cpp" });
			PerforceService.AddChange(streamId, 1003, user, "", new[] { "Foo.uasset" });
			PerforceService.AddChange(streamId, 1004, user, "", new[] { "Foo.uasset" });

			ICommitCollection commits = PerforceService.GetCommits(streamConfig);

			{
				CommitIdWithOrder? change = await JobService.EvaluateChangeQueryAsync(streamId, new ChangeQueryConfig { CommitTag = CommitTag.Code }, null, commits, CancellationToken.None);
				Assert.AreEqual(1002, change!.GetPerforceChange());
			}
			{
				CommitIdWithOrder? change = await JobService.EvaluateChangeQueryAsync(streamId, new ChangeQueryConfig { CommitTag = CommitTag.Content }, null, commits, CancellationToken.None);
				Assert.AreEqual(1004, change!.GetPerforceChange());
			}
			{
				CommitIdWithOrder? change = await JobService.EvaluateChangeQueryAsync(streamId, new ChangeQueryConfig { CommitTag = CommitTag.Content, Condition = "tag.code == 1" }, new List<CommitTag> { CommitTag.Code }, commits, CancellationToken.None);
				Assert.AreEqual(1004, change!.GetPerforceChange());
			}
			{
				CommitIdWithOrder? change = await JobService.EvaluateChangeQueryAsync(streamId, new ChangeQueryConfig { CommitTag = CommitTag.Content, Condition = "tag.content == 1" }, new List<CommitTag> { CommitTag.Code }, commits, CancellationToken.None);
				Assert.IsNull(change);
			}
		}

		[TestMethod]
		public async Task StopAnyDuplicateJobsByPreflightAsync()
		{
			Fixture fixture = await CreateFixtureAsync();

			string[] args = { "-Target=bogus" };
			IJob orgJob = await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1000, args);
			IJob newJob = await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1000, args);
			IJob differentTplRef = await CreatePreflightJobAsync(fixture, "tpl-ref-other", "tpl-hash-1", "elvis", 1000, args);
			IJob differentTplHash = await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-other", "elvis", 1000, args);
			IJob differentUserName = await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "julia", 1000, args);
			IJob differentArgs = await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1000, new[] { "-Target=other" });

			orgJob = (await JobService.GetJobAsync(orgJob.Id))!;
			newJob = (await JobService.GetJobAsync(newJob.Id))!;
			differentTplRef = (await JobService.GetJobAsync(differentTplRef.Id))!;
			differentTplHash = (await JobService.GetJobAsync(differentTplHash.Id))!;
			differentUserName = (await JobService.GetJobAsync(differentUserName.Id))!;
			differentArgs = (await JobService.GetJobAsync(differentArgs.Id))!;

			Assert.AreEqual(KnownUsers.System, orgJob.AbortedByUserId);
			Assert.IsNull(newJob.AbortedByUserId);
			Assert.IsNull(differentTplRef.AbortedByUserId);
			Assert.IsNull(differentTplHash.AbortedByUserId);
			Assert.IsNull(differentUserName.AbortedByUserId);
			Assert.IsNull(differentArgs.AbortedByUserId);
		}

		[TestMethod]
		public async Task TestJobTemplateSettingsAsync()
		{
			// Scenario: User creates a number of preflights
			// Expected: The users job template settings are populated and updated in place

			Fixture fixture = await CreateFixtureAsync();
			await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1000, new[] { "-Target=targeta" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "julia", 1000, new[] { "-Target=targeta" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-other", "tpl-hash-1", "elvis", 1001, new[] { "-Target=targetb" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1002, new[] { "-Target=targetc" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-1", "tpl-hash-1", "elvis", 1003, new[] { "-Target=targetd" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-other", "tpl-hash-1", "julia", 1004, new[] { "-Target=targete" });
			await CreatePreflightJobAsync(fixture, "tpl-ref-other", "tpl-hash-1", "elvis", 1004, new[] { "-Target=targete" });

			IUser user = await UserCollection.FindOrAddUserByLoginAsync("elvis")!;
			IUserSettings settings = await UserCollection.GetSettingsAsync(user!.Id)!;

			Assert.IsNotNull(settings.JobTemplateSettings);
			Assert.AreEqual(2, settings.JobTemplateSettings!.Count);

			TemplateId templateRef1 = new TemplateId("tpl-ref-1");
			IUserJobTemplateSettings? templateSettings = settings.JobTemplateSettings.FirstOrDefault(x => x.TemplateId == templateRef1);
			Assert.IsNotNull(templateSettings);
			Assert.AreEqual("-Target=targetd", templateSettings.Arguments[0]);

			TemplateId templateRefOther = new TemplateId("tpl-ref-other");
			templateSettings = settings.JobTemplateSettings.FirstOrDefault(x => x.TemplateId == templateRefOther);
			Assert.IsNotNull(templateSettings);
			Assert.AreEqual("-Target=targete", templateSettings.Arguments[0]);
		}

		private async Task<IJob> CreatePreflightJobAsync(Fixture fixture, string templateRefId, string templateHash, string startedByUserName, int preflightChange, string[] arguments)
		{
			IUser user = await UserCollection.FindOrAddUserByLoginAsync(startedByUserName);

			CreateJobOptions options = new CreateJobOptions();
			options.PreflightCommitId = CommitId.FromPerforceChange(preflightChange);
			options.StartedByUserId = user.Id;
			options.Arguments.AddRange(arguments);

			return await JobService.CreateJobAsync(
				jobId: JobIdUtils.GenerateNewId(),
				streamConfig: fixture!.StreamConfig!,
				templateRefId: new TemplateId(templateRefId),
				templateHash: new ContentHash(Encoding.ASCII.GetBytes(templateHash)),
				graph: fixture!.Graph,
				name: "hello1",
				commitId: CommitIdWithOrder.FromPerforceChange(1000001),
				codeCommitId: CommitIdWithOrder.FromPerforceChange(1000002),
				options
			);
		}

		// Only test for cancelled preflights
		// [TestMethod]
		// public async Task StopAnyDuplicateJobsByChange()
		// {
		// 	TestSetup TestSetup = await GetTestSetup();
		// 	Fixture Fix = Fixture!;
		// 	IJob Job1 = Fix.Job1;
		// 	IJob Job2 = Fix.Job2;
		// 	
		// 	Assert.AreEqual(JobState.Waiting, Job1.GetState());
		// 	Assert.IsTrue(await JobService.UpdateBatchAsync(Job1, Job1.Batches[0].Id, ObjectId.GenerateNewId(), JobStepBatchState.Running));
		//
		// 	Job1 = (await JobService.GetJobAsync(Job1.Id))!;
		// 	Job2 = (await JobService.GetJobAsync(Job2.Id))!;
		// 	Assert.AreEqual(JobState.Running, Job1.GetState());
		// 	Assert.AreEqual(JobState.Complete, Job2.GetState());
		// 	
		// 	IJob Job3 = await JobService.CreateJobAsync(null, Fix.Stream!.Id, Fix.TemplateRefId1, Fix.Template.Id, Fix.Graph, "Hello", Job1.Change, Job1.CodeChange, null, "joe", null, null, true, true, null, null, new List<string>());
		// 	await DispatchService.TickOnlyForTestingAsync();
		// 	Job1 = (await JobService.GetJobAsync(Job1.Id))!;
		// 	Assert.AreEqual(JobState.Complete, Job1.GetState());
		// 	Assert.AreEqual(JobState.Waiting, Job3.GetState());
		// }

		[TestMethod]
		public async Task TestRunEarlyAsync()
		{
			StreamId streamId = new StreamId("ue5-main");
			StreamConfig streamConfig = new StreamConfig { Id = streamId };

			ProjectId projectId = new ProjectId("ue5");
			ProjectConfig projectConfig = new ProjectConfig { Id = projectId, Name = "UE5", Streams = new List<StreamConfig> { streamConfig } };

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects = new List<ProjectConfig> { projectConfig };

			GlobalConfig globalConfig = new();
			globalConfig.Plugins.AddBuildConfig(buildConfig);
			await SetConfigAsync(globalConfig);

			// ----

			IAgent? agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("TestAgent"), AgentMode.Dedicated, false, ""));
			Assert.IsNotNull(agent);

			agent = await agent.TryUpdateAsync(new UpdateAgentOptions { Enabled = true, ExplicitPools = new List<PoolId> { new PoolId("win") } });
			Assert.IsNotNull(agent);

			await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(), null);

			ITemplate template = await TemplateCollection.GetOrAddAsync(new TemplateConfig { Name = "Test template" });
			IGraph graph = await GraphCollection.AddAsync(template);

			NewGroup groupA = new NewGroup("win", new List<NewNode>());
			groupA.Nodes.Add(new NewNode("Compile"));

			NewGroup groupB = new NewGroup("win", new List<NewNode>());
			groupB.Nodes.Add(new NewNode("Cook", runEarly: true));
			groupB.Nodes.Add(new NewNode("Middle"));
			groupB.Nodes.Add(new NewNode("Pak", inputDependencies: new List<string> { "Compile", "Cook" }));

			graph = await GraphCollection.AppendAsync(graph, new List<NewGroup> { groupA, groupB });

			CreateJobOptions options = new CreateJobOptions();
			options.PreflightCommitId = CommitId.FromPerforceChange(999);
			options.Arguments.Add("-Target=Pak");

			IJob job = await JobService.CreateJobAsync(null, streamConfig!, new TemplateId("temp"), new ContentHash(new byte[] { 1, 2, 3 }), graph, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options);

			job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[0].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[0].Id, job.Batches[0].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

			job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[1].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[1].Id, job.Batches[1].Steps[0].Id, streamConfig, JobStepState.Running));

			job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[2].Id, streamConfig, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Running));
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[2].Id, job.Batches[2].Steps[0].Id, streamConfig, JobStepState.Completed, JobStepOutcome.Success));

			Assert.AreEqual(JobStepState.Waiting, job.Batches[2].Steps[1].State);
		}
	}
}
