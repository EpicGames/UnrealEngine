// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Jobs;
using HordeServer.Utilities;

namespace HordeServer.Tests.Jobs
{
	[TestClass]
	public class JobTaskSourceTests : BuildTestSetup
	{
		private bool _eventReceived;
		private bool? _eventPoolHasAgentsOnline;

		static NewGroup AddGroup(List<NewGroup> groups)
		{
			NewGroup group = new NewGroup("Win64", new List<NewNode>());
			groups.Add(group);
			return group;
		}

		static NewNode AddNode(NewGroup group, string name, string[]? inputDependencies, Action<NewNode>? action = null)
		{
			NewNode node = new NewNode(name, inputDependencies: inputDependencies?.ToList(), orderDependencies: inputDependencies?.ToList());
			action?.Invoke(node);
			group.Nodes.Add(node);
			return node;
		}

		[TestMethod]
		public async Task UpdateJobQueueNormalAsync()
		{
			Fixture fixture = await SetupPoolWithAgentAsync(isPoolAutoScaled: true, shouldCreateAgent: true, isAgentEnabled: true);

			await JobTaskSource.TickAsync(CancellationToken.None);
			Assert.AreEqual(1, JobTaskSource.GetQueueForTesting().Count);
			Assert.AreEqual(fixture.Job1.Id, JobTaskSource.GetQueueForTesting().Min!.Id.Item1);
			Assert.AreEqual(JobStepBatchState.Ready, JobTaskSource.GetQueueForTesting().Min!.Batch.State);

			Assert.IsTrue(_eventReceived);
			Assert.IsTrue(_eventPoolHasAgentsOnline!.Value);
		}

		[TestMethod]
		public async Task UpdateJobQueueWithNoAgentsInPoolAsync()
		{
			Fixture fixture = await SetupPoolWithAgentAsync(isPoolAutoScaled: true, shouldCreateAgent: false, isAgentEnabled: false);

			await JobTaskSource.TickAsync(CancellationToken.None);
			Assert.AreEqual(0, JobTaskSource.GetQueueForTesting().Count);

			IJob job = (await JobService.GetJobAsync(fixture.Job1.Id))!;
			Assert.AreEqual(JobStepBatchError.NoAgentsInPool, job.Batches[0].Error);

			Assert.IsFalse(_eventReceived);
		}

		[TestMethod]
		public async Task UpdateJobQueueWithNoAgentsOnlineInPoolAsync()
		{
			Fixture fixture = await SetupPoolWithAgentAsync(isPoolAutoScaled: false, shouldCreateAgent: true, isAgentEnabled: false);

			await JobTaskSource.TickAsync(CancellationToken.None);
			Assert.AreEqual(1, JobTaskSource.GetQueueForTesting().Count);

			Assert.AreEqual(fixture.Job1.Id, JobTaskSource.GetQueueForTesting().Min!.Id.Item1);
			Assert.AreEqual(JobStepBatchState.Ready, JobTaskSource.GetQueueForTesting().Min!.Batch.State);

			Assert.IsTrue(_eventReceived);
			Assert.IsFalse(_eventPoolHasAgentsOnline!.Value);
		}

		[TestMethod]
		public async Task UpdateJobQueueWithNoAgentsOnlineInAutoScaledPoolAsync()
		{
			Fixture fixture = await SetupPoolWithAgentAsync(isPoolAutoScaled: true, shouldCreateAgent: true, isAgentEnabled: false);

			await JobTaskSource.TickAsync(CancellationToken.None);
			Assert.AreEqual(1, JobTaskSource.GetQueueForTesting().Count);

			Assert.AreEqual(fixture.Job1.Id, JobTaskSource.GetQueueForTesting().Min!.Id.Item1);
			Assert.AreEqual(JobStepBatchState.Ready, JobTaskSource.GetQueueForTesting().Min!.Batch.State);

			Assert.IsTrue(_eventReceived);
			Assert.IsFalse(_eventPoolHasAgentsOnline!.Value);
		}

		[TestMethod]
		public async Task UpdateJobQueueWithPausedStepAsync()
		{
			Fixture fixture = await SetupPoolWithAgentAsync(isPoolAutoScaled: true, shouldCreateAgent: true, isAgentEnabled: true);

			// update template with some step states
			IStream stream = (await StreamCollection.GetAsync(fixture.StreamConfig!.Id))!;
			stream = Deref(await stream.TryUpdateTemplateRefAsync(fixture.TemplateRefId1, new List<UpdateStepStateRequest>() { new UpdateStepStateRequest() { Name = "Paused Step", PausedByUserId = new UserId(BinaryIdUtils.CreateNew()).ToString() } }));

			// create a new graph with the associated nodes
			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Update Version Files", null);
			AddNode(initialGroup, "Paused Step", new[] { "Update Version Files" });
			AddNode(initialGroup, "Step That Depends on Paused Step", new[] { "Paused Step" });
			AddNode(initialGroup, "Step That Depends on Update Version Files", new[] { "Update Version Files" });

			IGraph graph = await GraphCollection.AppendAsync(null, newGroups);

			// remove the default fixture jobs
			IReadOnlyList<IJob> jobs = await JobCollection.FindAsync(new FindJobOptions());
			for (int i = 0; i < jobs.Count; i++)
			{
				await jobs[i].TryDeleteAsync();
			}

			// create a new job
			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Step That Depends on Paused Step;Step That Depends on Update Version Files");

			IJob job = await JobCollection.AddAsync(JobIdUtils.GenerateNewId(), stream.Id,
				fixture.TemplateRefId1, fixture.Template.Hash, graph, "Test Paused Step Job",
				CommitIdWithOrder.FromPerforceChange(1000), CommitIdWithOrder.FromPerforceChange(1000), options);

			// validate
			await JobTaskSource.TickAsync(CancellationToken.None);
			Assert.AreEqual(1, JobTaskSource.GetQueueForTesting().Count);
			Assert.AreEqual(job.Id, JobTaskSource.GetQueueForTesting().Min!.Id.Item1);
			Assert.AreEqual(JobStepBatchState.Ready, JobTaskSource.GetQueueForTesting().Min!.Batch.State);

			Assert.IsTrue(_eventReceived);
			Assert.IsTrue(_eventPoolHasAgentsOnline!.Value);

			// make sure we get db value
			job = Deref(await JobService.GetJobAsync(job.Id));

			IJobStepBatch batch = job.Batches[0];

			Assert.AreEqual(4, batch.Steps.Count);
			Assert.AreEqual(JobStepState.Ready, batch.Steps[0].State);
			Assert.AreEqual(JobStepState.Skipped, batch.Steps[1].State);
			Assert.AreEqual(JobStepError.Paused, batch.Steps[1].Error);
			Assert.AreEqual(JobStepState.Skipped, batch.Steps[2].State);
			Assert.AreEqual(JobStepState.Waiting, batch.Steps[3].State);

		}

		private async Task<Fixture> SetupPoolWithAgentAsync(bool isPoolAutoScaled, bool shouldCreateAgent, bool isAgentEnabled)
		{
			Fixture fixture = await CreateFixtureAsync();
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = Fixture.PoolName, EnableAutoscaling = isPoolAutoScaled, MinAgents = 0, NumReserveAgents = 0 });

			if (shouldCreateAgent)
			{
				IAgent? agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId("TestAgent"), AgentMode.Dedicated, false, ""));
				Assert.IsNotNull(agent);

				agent = await agent.TryUpdateAsync(new UpdateAgentOptions { Enabled = isAgentEnabled, ExplicitPools = new List<PoolId> { pool.Id } });
				Assert.IsNotNull(agent);

				await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(), null);
			}

			JobTaskSource.OnJobScheduled += (pool, poolHasAgentsOnline, job, graph, batchId) =>
			{
				_eventReceived = true;
				_eventPoolHasAgentsOnline = poolHasAgentsOnline;
			};

			return fixture;
		}

		[TestMethod]
		[DataRow(0, true, DisplayName = "Conform is requested when assigning JobTask and agent has low disk free space")]
		[DataRow(1000, false, DisplayName = "Conform is not requested when assigning JobTask and agent has enough disk free space")]
		public async Task AgentAssignLeaseWithConformCheckAsync(long diskFreeSpaceInMb, bool expectingConform)
		{
			// TODO Make workspace conform size an input parameter rather than be hardcoded in Fixture.PopulateAsync

			Fixture fixture = await CreateFixtureAsync();
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = Fixture.PoolName, EnableAutoscaling = false, MinAgents = 0, NumReserveAgents = 0 });

			IAgent agent = fixture.Agent1;
			agent = Deref(await agent.TryUpdateAsync(new UpdateAgentOptions { Enabled = true, ExplicitPools = [pool.Id] }));

			await UpdateConfigAsync(config =>
			{
				config.Plugins.GetBuildConfig().EnableJobTaskConformCheck = true;
			});

			List<string> properties = [$"{KnownPropertyNames.WorkingDiskFreeSpace}={diskFreeSpaceInMb * 1024 * 1024}"];
			agent = Deref(await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(properties), null));

			await JobTaskSource.TickAsync(CancellationToken.None);

			Task<CreateLeaseOptions?> lease = Deref(await JobTaskSource.AssignLeaseAsync(agent, CancellationToken.None));
			CreateLeaseOptions? leaseResult = await lease.WaitAsync(CancellationToken.None);

			agent = Deref(await AgentService.GetAgentAsync(agent.Id));

			if (expectingConform)
			{
				Assert.IsTrue(agent.RequestFullConform);
				Assert.IsNull(leaseResult);
			}
			else
			{
				Assert.IsFalse(agent.RequestFullConform);
				Assert.IsNotNull(leaseResult);
			}
		}

		[TestMethod]
		[DataRow(true, DisplayName = "A Full Conform lease is assigned")]
		[DataRow(false, DisplayName = "A Non-Full Conform lease assigned")]
		public async Task AgentAssignConformLeaseAsync(bool fullConform)
		{
			Fixture fixture = await CreateFixtureAsync();
			IAgent agent = fixture.Agent1;

			UpdateAgentOptions options = fullConform
				? new() { RequestFullConform = true }
				: new() { RequestConform = true };

			agent = Deref(await agent.TryUpdateAsync(options));

			Task<CreateLeaseOptions?> lease = Deref(await ConformTaskSource.AssignLeaseAsync(agent, CancellationToken.None));
			CreateLeaseOptions leaseResult = Deref(await lease.WaitAsync(CancellationToken.None));

			ConformTask? conformTask = leaseResult.Payload as ConformTask;
			Assert.IsNotNull(conformTask);

			if (fullConform)
			{
				Assert.IsTrue(conformTask.RemoveUntrackedFiles);
			}
			else
			{
				Assert.IsFalse(conformTask.RemoveUntrackedFiles);
			}
		}
	}
}
