// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs;
using HordeServer.Jobs;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.Tests.Jobs
{
	/// <summary>
	/// Database-only test for testing the Job controller. Different from the JobsController test that set up
	/// the entire ASP.NET chain.
	/// </summary>
	[TestClass]
	public class JobsControllerDbTest : BuildTestSetup
	{
		[TestMethod]
		public async Task GetJobsAsync()
		{
			await CreateFixtureAsync();

			ActionResult<List<object>> res = await JobsController.FindJobsAsync();

			List<GetJobResponse> responses = res.Value!.ConvertAll(x => (GetJobResponse)x);
			responses.SortBy(x => x.CommitId);

			Assert.AreEqual(2, responses.Count);
			Assert.AreEqual("hello1", responses[0].Name);
			Assert.AreEqual("hello2", responses[1].Name);

			res = await JobsController.FindJobsAsync(includePreflight: false);
			Assert.AreEqual(1, res.Value!.Count);
			Assert.AreEqual("hello2", (res.Value[0] as GetJobResponse)!.Name);
		}

		[TestMethod]
		public async Task AbortStepTestAsync()
		{
			Fixture fixture = await CreateFixtureAsync();

			IJob job = fixture.Job1;
			JobStepBatchId batchId = job.Batches[0].Id;
			JobStepId stepId = job.Batches[0].Steps[0].Id;

#pragma warning disable CS0618 // Type or member is obsolete
			object obj = (await JobsController.GetStepAsync(job.Id, batchId, stepId)).Value!;
#pragma warning restore CS0618 // Type or member is obsolete
			GetJobStepResponse stepRes = (obj as GetJobStepResponse)!;
			Assert.IsFalse(stepRes.AbortRequested);

			UpdateStepRequest updateReq = new UpdateStepRequest();
			updateReq.AbortRequested = true;
			await JobsController.UpdateStepAsync(job.Id, batchId, stepId, updateReq);
			//	        UpdateStepResponse updateRes = (obj as UpdateStepResponse)!;

#pragma warning disable CS0618 // Type or member is obsolete
			obj = (await JobsController.GetStepAsync(job.Id, batchId, stepId)).Value!;
#pragma warning restore CS0618 // Type or member is obsolete
			stepRes = (obj as GetJobStepResponse)!;
			Assert.IsTrue(stepRes.AbortRequested);
			//	        Assert.AreEqual("Anonymous", StepRes.AbortByUser);
		}

		[TestMethod]
		public async Task FindJobTimingsTestAsync()
		{
			Fixture fixture = await CreateFixtureAsync();
			IJob job = fixture.Job1;
			string[] templates = { job.TemplateId.ToString() };
			object obj = (await JobsController.FindJobTimingsAsync(fixture.StreamConfig!.Id.ToString(), templates)).Value!;
			FindJobTimingsResponse res = (obj as FindJobTimingsResponse)!;
			Assert.AreEqual(1, res.Timings.Count);
			GetJobTimingResponse timingResponse = res.Timings[job.Id.ToString()];
			Assert.AreEqual(0, timingResponse.JobResponse!.Labels!.Count);
			//	        Assert.AreEqual(job.Name, timingResponse.Job!.Name);
		}
	}
}