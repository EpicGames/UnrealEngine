// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Jobs.Schedules;
using HordeServer.Jobs.Templates;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Moq;

namespace HordeServer.Tests.Jobs.Schedules
{
	[TestClass]
	public class SchedulerTests : BuildTestSetup
	{
		ProjectId ProjectId { get; } = new ProjectId("ue5");
		StreamId StreamId { get; } = new StreamId("ue5-main");
		TemplateId TemplateId { get; } = new TemplateId("template1");

		ITemplate _template = default!;
		HashSet<JobId> _initialJobIds = default!;

		[TestInitialize]
		public async Task SetupAsync()
		{
			IUser bob = await UserCollection.FindOrAddUserByLoginAsync("Bob");

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(new ProjectConfig { Id = ProjectId, Name = "UE4" });

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			_template = await TemplateCollection.GetOrAddAsync(new TemplateConfig { Name = "Test template" });

			_initialJobIds = new HashSet<JobId>((await JobCollection.FindAsync(new FindJobOptions())).Select(x => x.Id));

			PerforceService.Changes.Clear();
			PerforceService.AddChange(StreamId, 100, bob, "", new[] { "code.cpp" });
			PerforceService.AddChange(StreamId, 101, bob, "", new[] { "content.uasset" });
			PerforceService.AddChange(StreamId, 102, bob, "", new[] { "content.uasset" });
		}

		async Task<IStream> SetScheduleAsync(ScheduleConfig schedule, List<StreamTag>? streamTags = null)
		{
			await ScheduleService.ResetAsync();

			StreamConfig streamConfig = new StreamConfig();
			streamConfig.Id = StreamId;
			streamConfig.Name = "//UE5/Main";
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { TemplateId } });
			streamConfig.Templates.Add(new TemplateRefConfig { Id = TemplateId, Name = "Test", Schedule = schedule });

			if (streamTags != null)
			{
				streamConfig.StreamTags = streamTags;
			}

			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = new List<StreamConfig> { streamConfig });

			return (await StreamCollection.GetAsync(streamConfig.Id))!;
		}

		public async Task<List<IJob>> FileTestHelperAsync(params string[] files)
		{
			IUser bob = await UserCollection.FindOrAddUserByLoginAsync("Bob", "Bob");

			PerforceService.Changes.Clear();
			PerforceService.AddChange(StreamId, 100, bob, "", new[] { "code.cpp" });
			PerforceService.AddChange(StreamId, 101, bob, "", new[] { "content.uasset" });
			PerforceService.AddChange(StreamId, 102, bob, "", new[] { "content.uasset" });
			PerforceService.AddChange(StreamId, 103, bob, "", new[] { "foo/code.cpp" });
			PerforceService.AddChange(StreamId, 104, bob, "", new[] { "bar/code.cpp" });
			PerforceService.AddChange(StreamId, 105, bob, "", new[] { "foo/bar/content.uasset" });
			PerforceService.AddChange(StreamId, 106, bob, "", new[] { "bar/foo/content.uasset" });

			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.MaxChanges = 10;
			schedule.Patterns.Add(new SchedulePatternConfig { Interval = new ScheduleInterval(1) });
			schedule.Files = files.ToList();
			await SetScheduleAsync(schedule);

			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			return await GetNewJobsAsync();
		}

		[TestMethod]
		public async Task FileTest1Async()
		{
			List<IJob> jobs = await FileTestHelperAsync("....cpp");
			Assert.AreEqual(3, jobs.Count);
			Assert.AreEqual(100, jobs[0].CommitId.GetPerforceChange());
			Assert.AreEqual(103, jobs[1].CommitId.GetPerforceChange());
			Assert.AreEqual(104, jobs[2].CommitId.GetPerforceChange());
		}

		[TestMethod]
		public async Task FileTest2Async()
		{
			List<IJob> jobs = await FileTestHelperAsync("/foo/...");
			Assert.AreEqual(2, jobs.Count);
			Assert.AreEqual(103, jobs[0].CommitId.GetPerforceChange());
			Assert.AreEqual(105, jobs[1].CommitId.GetPerforceChange());
		}

		[TestMethod]
		public async Task FileTest3Async()
		{
			List<IJob> jobs = await FileTestHelperAsync("....uasset", "-/bar/...");
			Assert.AreEqual(3, jobs.Count);
			Assert.AreEqual(101, jobs[0].CommitId.GetPerforceChange());
			Assert.AreEqual(102, jobs[1].CommitId.GetPerforceChange());
			Assert.AreEqual(105, jobs[2].CommitId.GetPerforceChange());
		}

		[TestMethod]
		public void DayScheduleTest()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Utc); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			Mock<ISchedule> scheduleMock = new Mock<ISchedule>(MockBehavior.Strict);
			scheduleMock.SetupGet(x => x.Patterns).Returns([new SchedulePatternConfig(new List<DayOfWeek> { DayOfWeek.Friday, DayOfWeek.Sunday }, 13 * 60, null, null)]);

			ISchedule schedule = scheduleMock.Object;

			DateTime? nextTime = schedule.GetNextTriggerTimeUtc(startTime, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0 + 24.0 * 2.0), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0 + 24.0 * 7.0), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0 + 24.0 * 9.0), nextTime!.Value);
		}

		[TestMethod]
		public void MulticheduleTest()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Utc); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			Mock<ISchedule> scheduleMock = new Mock<ISchedule>(MockBehavior.Strict);
			scheduleMock.SetupGet(x => x.RequireSubmittedChange).Returns(true);
			scheduleMock.SetupGet(x => x.Patterns).Returns([new SchedulePatternConfig(null, 13 * 60, 14 * 60, 15)]);

			ISchedule schedule = scheduleMock.Object;

			DateTime? nextTime = schedule.GetNextTriggerTimeUtc(startTime, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.25), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.5), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.75), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(2.0), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(1.0 + 24.0), nextTime!.Value);
		}

		[TestMethod]
		public void MultiPatternTest()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 0, 0, 0, DateTimeKind.Utc); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			Mock<ISchedule> scheduleMock = new Mock<ISchedule>(MockBehavior.Strict);
			scheduleMock.SetupGet(x => x.RequireSubmittedChange).Returns(false);
			scheduleMock.SetupGet(x => x.Patterns).Returns(
				[
					new SchedulePatternConfig(null, 11 * 60, 0, 0),
					new SchedulePatternConfig(null, 19 * 60, 0, 0)
				]);

			ISchedule schedule = scheduleMock.Object;

			DateTime? nextTime = schedule.GetNextTriggerTimeUtc(startTime, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(11), nextTime!.Value);

			nextTime = schedule.GetNextTriggerTimeUtc(nextTime.Value, TimeZoneInfo.Utc);
			Assert.AreEqual(startTime + TimeSpan.FromHours(19), nextTime!.Value);
		}

		[TestMethod]
		public async Task NoSubmittedChangeTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			//			Schedule.LastTriggerTime = StartTime;
			await SetScheduleAsync(schedule);

			// Initial tick
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs1.Count);

			// Trigger a job
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs2.Count);
			Assert.AreEqual(102, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs2[0].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task RequireSubmittedChangeTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			await SetScheduleAsync(schedule);

			// Initial tick
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs1.Count);

			// Trigger a job
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs2.Count);
			Assert.AreEqual(102, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs2[0].CodeCommitId!.GetPerforceChange());

			StreamConfig? streamConfig;
			GlobalConfig.CurrentValue.Plugins.GetBuildConfig().TryGetStream(StreamId, out streamConfig);

			IStream stream2 = (await StreamCollection.GetAsync(streamConfig!.Id))!;
			ISchedule schedule2 = stream2.Templates.First().Value.Schedule!;
			Assert.AreEqual(102, schedule2.LastTriggerCommitId!.GetPerforceChange());
			Assert.AreEqual(Clock.UtcNow, schedule2.LastTriggerTimeUtc);

			// Trigger another job
			await Clock.AdvanceAsync(TimeSpan.FromHours(0.5));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs3.Count);
		}

		[TestMethod]
		public async Task MultipleJobsTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			schedule.MaxChanges = 2;
			schedule.Commits.Add(CommitTag.Code);
			await SetScheduleAsync(schedule);

			// Initial tick
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs1.Count);

			// Trigger some jobs
			IUser bob = await UserCollection.FindOrAddUserByLoginAsync("Bob");
			PerforceService.AddChange(StreamId, 103, bob, "", new string[] { "foo.cpp" });
			PerforceService.AddChange(StreamId, 104, bob, "", new string[] { "foo.cpp" });
			PerforceService.AddChange(StreamId, 105, bob, "", new string[] { "foo.uasset" });
			PerforceService.AddChange(StreamId, 106, bob, "", new string[] { "foo.cpp" });

			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(2, jobs2.Count);
			Assert.AreEqual(104, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(104, jobs2[0].CodeCommitId!.GetPerforceChange());
			Assert.AreEqual(106, jobs2[1].CommitId.GetPerforceChange());
			Assert.AreEqual(106, jobs2[1].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task SkipCiTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			schedule.MaxChanges = 2;
			schedule.Commits.Add(CommitTag.Code);
			await SetScheduleAsync(schedule);

			// Initial tick
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs1.Count);

			// Trigger some jobs
			IUser bob = await UserCollection.FindOrAddUserByLoginAsync("Bob");
			PerforceService.AddChange(StreamId, 103, bob, "", new string[] { "foo.cpp" });
			PerforceService.AddChange(StreamId, 104, bob, "Don't build this change!\n#skipci", new string[] { "foo.cpp" });
			PerforceService.AddChange(StreamId, 105, bob, "", new string[] { "foo.uasset" });
			PerforceService.AddChange(StreamId, 106, bob, "", new string[] { "foo.cpp" });

			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(2, jobs2.Count);
			Assert.AreEqual(103, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(103, jobs2[0].CodeCommitId!.GetPerforceChange());
			Assert.AreEqual(106, jobs2[1].CommitId.GetPerforceChange());
			Assert.AreEqual(106, jobs2[1].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task MaxActiveTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.RequireSubmittedChange = false;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			schedule.MaxActive = 1;
			await SetScheduleAsync(schedule);

			// Trigger a job
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs2.Count);
			Assert.AreEqual(102, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs2[0].CodeCommitId!.GetPerforceChange());

			// Test that another job does not trigger
			await Clock.AdvanceAsync(TimeSpan.FromHours(0.5));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs3.Count);

			// Mark the original job as complete
			await JobService.UpdateJobAsync(jobs2[0], abortedByUserId: KnownUsers.System);

			// Test that another job does not trigger
			await Clock.AdvanceAsync(TimeSpan.FromHours(0.5));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs4 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs4.Count);
			Assert.AreEqual(102, jobs4[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs4[0].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task CreateNewChangeTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			_ = await SetScheduleAsync(schedule);

			// Trigger a job
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs2.Count);
			Assert.AreEqual(102, jobs2[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs2[0].CodeCommitId!.GetPerforceChange());

			// Check another job does not trigger due to the change above
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs3.Count);
		}

		[TestMethod]
		public async Task GateTestAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			// Create two templates, the second dependent on the first
			ITemplate? newTemplate1 = await TemplateCollection.GetOrAddAsync(new TemplateConfig { Name = "Test template 1" });
			//TemplateRef newTemplateRef1 = new TemplateRef(newTemplate1);
			TemplateId newTemplateRefId1 = new TemplateId("new-template-1");

			ITemplate? newTemplate2 = await TemplateCollection.GetOrAddAsync(new TemplateConfig { Name = "Test template 2" });
			//			TemplateRef newTemplateRef2 = new TemplateRef(newTemplate2);
			//			newTemplateRef2.Schedule = new Schedule(Clock.UtcNow);
			//			newTemplateRef2.Schedule.Gate = new ScheduleGate(newTemplateRefId1, "TriggerNext");
			//			newTemplateRef2.Schedule.Patterns.Add(new SchedulePattern(null, 0, null, 10));
			//			newTemplateRef2.Schedule.LastTriggerTime = startTime;
			TemplateId newTemplateRefId2 = new TemplateId("new-template-2");

			StreamConfig config = new StreamConfig();
			config.Id = StreamId;
			config.Name = "//UE5/Main";
			config.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { newTemplateRefId1, newTemplateRefId2 } });
			config.Templates = new() { new TemplateRefConfig { Id = newTemplateRefId1 }, new TemplateRefConfig { Id = newTemplateRefId2 } };
			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = new List<StreamConfig> { config });

			IStream stream = (await StreamCollection.GetAsync(config.Id))!;

			// Create the TriggerNext step and mark it as complete
			IGraph graphA = await GraphCollection.AddAsync(newTemplate1);
			NewGroup groupA = new NewGroup("win", new List<NewNode> { new NewNode("TriggerNext") });
			graphA = await GraphCollection.AppendAsync(graphA, new List<NewGroup> { groupA });

			// Tick the schedule and make sure it doesn't trigger
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs2.Count);

			// Create a job and fail it
			CreateJobOptions options1 = new CreateJobOptions();
			options1.PreflightCommitId = CommitId.FromPerforceChange(999);
			options1.Arguments.Add("-Target=TriggerNext");

			IJob job1 = await JobService.CreateJobAsync(null, config, newTemplateRefId1, newTemplate1.Hash, graphA, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options1);
			JobStepBatchId batchId1 = job1.Batches[0].Id;
			JobStepId stepId1 = job1.Batches[0].Steps[0].Id;
			job1 = Deref(await JobService.UpdateBatchAsync(job1, batchId1, config, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job1 = Deref(await JobService.UpdateStepAsync(job1, batchId1, stepId1, config, JobStepState.Completed, JobStepOutcome.Failure));
			job1 = Deref(await JobService.UpdateBatchAsync(job1, batchId1, config, LogIdUtils.GenerateNewId(), JobStepBatchState.Complete));
			Assert.IsNotNull(job1);
			await GetNewJobsAsync();

			// Tick the schedule and make sure it doesn't trigger
			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30.0));
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs3.Count);

			// Create a job and make it succeed
			CreateJobOptions options2 = new CreateJobOptions();
			options2.PreflightCommitId = CommitId.FromPerforceChange(999);
			options2.Arguments.Add("-Target=TriggerNext");

			IJob job2 = await JobService.CreateJobAsync(null, config, newTemplateRefId1, newTemplate1.Hash, graphA, "Hello", CommitIdWithOrder.FromPerforceChange(1234), CommitIdWithOrder.FromPerforceChange(1233), options2);
			JobStepBatchId batchId2 = job2.Batches[0].Id;
			JobStepId stepId2 = job2.Batches[0].Steps[0].Id;
			job2 = Deref(await JobService.UpdateBatchAsync(job2, batchId2, config, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
			job2 = Deref(await JobService.UpdateStepAsync(job2, batchId2, stepId2, config, JobStepState.Completed, JobStepOutcome.Success));
			Assert.IsNotNull(job2);

			// Tick the schedule and make sure it does trigger
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs4 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs4.Count);
			Assert.AreEqual(1234, jobs4[0].CommitId.GetPerforceChange());
			Assert.AreEqual(1233, jobs4[0].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task GateTest2Async()
		{
			IUser bob = await UserCollection.FindOrAddUserByLoginAsync("Bob");

			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			PerforceService.Changes.Clear();
			PerforceService.AddChange(StreamId, 1230, bob, "", new[] { "code.cpp" });
			PerforceService.AddChange(StreamId, 1231, bob, "", new[] { "content.uasset" });
			PerforceService.AddChange(StreamId, 1232, bob, "", new[] { "content.uasset" });
			PerforceService.AddChange(StreamId, 1233, bob, "", new[] { "code.cpp" });

			// Create two templates, the second dependent on the first
			TemplateId newTemplateRefId1 = new TemplateId("new-template-1");
			TemplateRefConfig newTemplate1 = new TemplateRefConfig();
			newTemplate1.Id = newTemplateRefId1;

			TemplateId newTemplateRefId2 = new TemplateId("new-template-2");
			TemplateRefConfig newTemplate2 = new TemplateRefConfig();
			newTemplate2.Id = newTemplateRefId2;
			newTemplate2.Name = "Test template 2";
			newTemplate2.Schedule = new ScheduleConfig();
			newTemplate2.Schedule.MaxChanges = 4;
			newTemplate2.Schedule.Commits.Add(CommitTag.Code);
			newTemplate2.Schedule.Gate = new ScheduleGateConfig { TemplateId = newTemplateRefId1, Target = "TriggerNext" };
			newTemplate2.Schedule.Patterns.Add(new SchedulePatternConfig { Interval = ScheduleInterval.Parse("10m") });// (null, 0, null, 10));
																													   //			NewTemplate2.Schedule.LastTriggerTime = StartTime;

			//			IStream? stream = await StreamService.GetStreamAsync(StreamId);

			StreamConfig config = new StreamConfig();
			config.Id = StreamId;
			config.Name = "//UE5/Main";
			config.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { newTemplateRefId1, newTemplateRefId2 } });
			config.Templates.Add(newTemplate1);
			config.Templates.Add(newTemplate2);
			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = new List<StreamConfig> { config });

			IStream stream = (await StreamCollection.GetAsync(config.Id))!;

			//			stream = (await CreateOrReplaceStreamAsync(StreamId, stream, ProjectId, config))!;

			ITemplate template1 = (await TemplateCollection.GetOrAddAsync(config.Templates[0]))!;
			Assert.IsNotNull(template1);
			ITemplate template2 = (await TemplateCollection.GetOrAddAsync(config.Templates[1]))!;
			Assert.IsNotNull(template2);

			// Create the graph
			IGraph graphA = await GraphCollection.AddAsync(template1);
			NewGroup groupA = new NewGroup("win", new List<NewNode> { new NewNode("TriggerNext") });
			graphA = await GraphCollection.AppendAsync(graphA, new List<NewGroup> { groupA });

			// Create successful jobs for all the changes we added above
			for (int change = 1230; change <= 1233; change++)
			{
				int codeChange = (change < 1233) ? 1230 : 1233;

				CreateJobOptions options1 = new CreateJobOptions();
				options1.Arguments.Add("-Target=TriggerNext");

				IJob job1 = await JobService.CreateJobAsync(null, config, newTemplateRefId1, _template.Hash, graphA, "Hello", CommitIdWithOrder.FromPerforceChange(change), CommitIdWithOrder.FromPerforceChange(codeChange), options1);
				for (int batchIdx = 0; batchIdx < job1.Batches.Count; batchIdx++)
				{
					JobStepBatchId batchId1 = job1.Batches[batchIdx].Id;
					job1 = Deref(await JobService.UpdateBatchAsync(job1, batchId1, config, LogIdUtils.GenerateNewId(), JobStepBatchState.Running));
					for (int stepIdx = 0; stepIdx < job1.Batches[batchIdx].Steps.Count; stepIdx++)
					{
						JobStepId stepId1 = job1.Batches[batchIdx].Steps[stepIdx].Id;
						job1 = Deref(await JobService.UpdateStepAsync(job1, batchId1, stepId1, config, JobStepState.Completed, JobStepOutcome.Success, newLogId: LogIdUtils.GenerateNewId()));
					}
					job1 = Deref(await JobService.UpdateBatchAsync(job1, batchId1, config, LogIdUtils.GenerateNewId(), JobStepBatchState.Complete));
				}
			}
			await GetNewJobsAsync();

			// Tick the schedule and make sure it doesn't trigger
			await Clock.AdvanceAsync(TimeSpan.FromMinutes(30.0));
			await ScheduleService.TriggerAsync(StreamId, newTemplateRefId2, Clock.UtcNow, default);
			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(2, jobs3.Count);
			Assert.AreEqual(1230, jobs3[0].CommitId.GetPerforceChange());
			Assert.AreEqual(1233, jobs3[1].CommitId.GetPerforceChange());
		}

		[TestMethod]
		public async Task ClaimsAsync()
		{
			DateTime startTime = new (2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new() {
				Enabled = true,
				Claims = [new ScheduleClaimConfig { Type = "MyType", Value = "MyValue"}]
			};
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			_ = await SetScheduleAsync(schedule);
			
			// Trigger a job
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();
			
			List<IJob> jobs = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs.Count);
			Assert.AreEqual(1, jobs[0].Claims.Count);
			Assert.AreEqual("MyType", jobs[0].Claims[0].Type);
			Assert.AreEqual("MyValue", jobs[0].Claims[0].Value);
		}

		[TestMethod]
		public async Task UpdateConfigAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Local); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;
			Clock.TimeZone = TimeZoneInfo.Local;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.RequireSubmittedChange = false;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			schedule.MaxActive = 2;
			await SetScheduleAsync(schedule);

			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs1.Count);
			Assert.AreEqual(102, jobs1[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs1[0].CodeCommitId!.GetPerforceChange());

			// Make sure the job is registered
			IStream? stream1 = await StreamCollection.GetAsync(GlobalConfig.CurrentValue.Plugins.GetBuildConfig().Streams[0].Id);
			ITemplateRef templateRef1 = stream1!.Templates.First().Value;
			Assert.AreEqual(1, templateRef1.Schedule!.ActiveJobs.Count);
			Assert.AreEqual(jobs1[0].Id, templateRef1.Schedule!.ActiveJobs[0]);

			// Test that another job does not trigger
			await SetScheduleAsync(schedule);

			// Make sure the job is still registered
			IStream? stream2 = await StreamCollection.GetAsync(GlobalConfig.CurrentValue.Plugins.GetBuildConfig().Streams[0].Id);
			ITemplateRef templateRef2 = stream2!.Templates.First().Value;
			Assert.AreEqual(1, templateRef2.Schedule!.ActiveJobs.Count);
			Assert.AreEqual(jobs1[0].Id, templateRef2.Schedule!.ActiveJobs[0]);
		}

		[TestMethod]
		public async Task StreamPausingAsync()
		{
			DateTime startTime = new DateTime(2021, 1, 1, 12, 0, 0, DateTimeKind.Utc); // Friday Jan 1, 2021 
			Clock.UtcNow = startTime;

			ScheduleConfig schedule = new ScheduleConfig();
			schedule.Enabled = true;
			schedule.Patterns.Add(new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") });
			IStream stream = await SetScheduleAsync(schedule);

			await stream.TryUpdatePauseStateAsync(newPausedUntil: startTime.AddHours(5), newPauseComment: "testing");

			// Try trigger a job. No job should be scheduled as the stream is paused
			await Clock.AdvanceAsync(TimeSpan.FromHours(1.25));
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs2.Count);

			// Advance time beyond the pause period. A build should now trigger
			await Clock.AdvanceAsync(TimeSpan.FromHours(5.25));
			await ScheduleService.TickForTestingAsync();

			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs3.Count);
			Assert.AreEqual(102, jobs3[0].CommitId.GetPerforceChange());
			Assert.AreEqual(100, jobs3[0].CodeCommitId!.GetPerforceChange());
		}

		[TestMethod]
		public async Task ScheduleOverrideTestAsync()
		{
			await ScheduleService.ResetAsync();

			TemplateRefConfig templateConfig = new TemplateRefConfig();
			templateConfig.Id = TemplateId;
			templateConfig.Name = "Test";
			templateConfig.Parameters.Add(
				new TemplateTextParameterConfig
				{
					Argument = "-Text=",
					Default = "Default",
					ScheduleOverride = "Scheduled"
				});
			templateConfig.Parameters.Add(
				new TemplateListParameterConfig
				{
					Items = new List<TemplateListParameterItemConfig>
					{
						new TemplateListParameterItemConfig
						{
							ArgumentIfEnabled = "-List=Default",
							ArgumentIfDisabled = "-List=Scheduled",
							Default = true,
							ScheduleOverride = false
						}
					}
				});
			templateConfig.Parameters.Add(
				new TemplateBoolParameterConfig
				{
					ArgumentIfEnabled = "-Bool=Default",
					ArgumentIfDisabled = "-Bool=Scheduled",
					Default = true,
					ScheduleOverride = false
				});
			templateConfig.Schedule = new ScheduleConfig
			{
				Enabled = true,
				Patterns = new List<SchedulePatternConfig> { new SchedulePatternConfig { MinTime = ScheduleTimeOfDay.Parse("13:00"), MaxTime = ScheduleTimeOfDay.Parse("14:00"), Interval = ScheduleInterval.Parse("15m") } }
			};

			StreamConfig streamConfig = new StreamConfig();
			streamConfig.Id = StreamId;
			streamConfig.Name = "//UE5/Main";
			streamConfig.Tabs.Add(new TabConfig { Title = "foo", Templates = new List<TemplateId> { TemplateId } });
			streamConfig.Templates.Add(templateConfig);
			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = new List<StreamConfig> { streamConfig });

			// Make sure we don't have any jobs to start with
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs1 = await GetNewJobsAsync();
			Assert.AreEqual(0, jobs1.Count);

			// Trigger a manual build and check the arguments
			await JobsController.CreateJobAsync(new CreateJobRequest(StreamId, TemplateId));
			List<IJob> jobs2 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs2.Count);
			Assert.AreEqual(3, jobs2[0].Arguments.Count);
			Assert.AreEqual("-Text=Default", jobs2[0].Arguments[0]);
			Assert.AreEqual("-List=Default", jobs2[0].Arguments[1]);
			Assert.AreEqual("-Bool=Default", jobs2[0].Arguments[2]);

			// Trigger a scheduled build and check the arguments
			await Clock.AdvanceAsync(TimeSpan.FromDays(1.0));
			await ScheduleService.TickForTestingAsync();
			List<IJob> jobs3 = await GetNewJobsAsync();
			Assert.AreEqual(1, jobs3.Count);
			Assert.AreEqual(3, jobs3[0].Arguments.Count);
			Assert.AreEqual("-Text=Scheduled", jobs3[0].Arguments[0]);
			Assert.AreEqual("-List=Scheduled", jobs3[0].Arguments[1]);
			Assert.AreEqual("-Bool=Scheduled", jobs3[0].Arguments[2]);
		}

		async Task<List<IJob>> GetNewJobsAsync()
		{
			List<IJob> jobs = (await JobCollection.FindAsync(new FindJobOptions())).ToList();
			jobs.RemoveAll(x => _initialJobIds.Contains(x.Id));
			_initialJobIds.UnionWith(jobs.Select(x => x.Id));
			return jobs.OrderBy(x => x.CommitId).ToList();
		}

		[TestMethod]
		public async Task ScheduleWithConditionEnabledTestAsync()
		{
			List<StreamTag> tags = [
				new() { Name = "A", Enabled = true }
			];
			ScheduleConfig schedule = new()
			{
				Condition = Condition.Parse("A == true")
			};
			await SetScheduleAsync(schedule, tags);
			Assert.IsTrue(schedule.Enabled);
		}

		[TestMethod]
		public async Task ScheduleWithConditionDisabledTestAsync()
		{
			List<StreamTag> tags = [
				new() { Name = "A", Enabled = false }
			];
			ScheduleConfig schedule = new()
			{
				Condition = Condition.Parse("A == true")
			};
			await SetScheduleAsync(schedule, tags);
			Assert.IsFalse(schedule.Enabled);
		}

		[TestMethod]
		public async Task ScheduleWithConditionDisabledFromJsonTestAsync()
		{
			string json = """
			{
				"id": "ue5-main",
				"name": "//UE5/Main",
				"streamTags": [
					{
						"name": "A",
						"enabled": false
					}
				],
				"tabs": [
					{
						"title": "foo",
						"templates": ["template1"]
					}
				],
				"templates": [
					{
						"id": "template1",
						"name": "Test",
						"schedule": {
							"condition": "A == true"
						}
					}
				]
			}
			""";
			StreamConfig streamConfig = JsonSerializer.Deserialize<StreamConfig>(json, JsonUtils.DefaultSerializerOptions)!;
			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = [streamConfig]);
			Assert.IsFalse(streamConfig.Templates[0].Schedule?.Enabled);
		}

		[TestMethod]
		public async Task ScheduleBackwardsCompatibilityDisabledTestAsync()
		{
			ScheduleConfig schedule = new()
			{
				Condition = Condition.Parse("true"),
				// For backwards compatibility the Enabled property remains in use
				Enabled = false
			};
			await SetScheduleAsync(schedule);
			Assert.IsFalse(schedule.Enabled);
		}

		[TestMethod]
		[DataRow(true, DisplayName = "Schedule enabled by default with empty condition")]
		[DataRow(false, DisplayName = "Schedule enabled by default with no condition")]
		public async Task ScheduleEnabledByDefaultValuesTestAsync(bool withCondition)
		{
			ScheduleConfig schedule = new();
			if (withCondition)
			{
				schedule.Condition = Condition.Parse(String.Empty);
			}
			await SetScheduleAsync(schedule);
			Assert.IsTrue(schedule.Enabled);
		}

		[TestMethod]
		public async Task ScheduleWithMissingTagsTestAsync()
		{
			ScheduleConfig schedule = new()
			{
				Condition = Condition.Parse("A == true")
			};
			await Assert.ThrowsExactlyAsync<ConditionException>(async () => await SetScheduleAsync(schedule));
		}

		[TestMethod]
		[DataRow(true, true, DisplayName = "Multiple tags and scheduled is enabled")]
		[DataRow(true, false, DisplayName = "Multiple tags and schedule is disabled")]
		public async Task ScheduleWithMultipleTagsFromJsonTestAsync(bool isShipped, bool isCutOver)
		{
			string json = $$"""
			{
				"id": "game1",
				"name": "//depot/game1",
				"streamTags": [
					{
						"name": "isShipped",
						"enabled": {{isShipped.ToString().ToLower()}}
					},
					{
						"name": "isCutOver",
						"enabled": {{isCutOver.ToString().ToLower()}}
					}
				],
				"tabs": [
					{
						"title": "title",
						"templates": ["template1"]
					}
				],
				"templates": [
					{
						"id": "template1",
						"name": "template1",
						"schedule": {
							"condition": "(isShipped == true) && (isCutOver == true)"
						}
					}
				]
			}
			""";
			StreamConfig streamConfig = JsonSerializer.Deserialize<StreamConfig>(json, JsonUtils.DefaultSerializerOptions)!;
			await UpdateConfigAsync(x => x.Plugins.GetBuildConfig().Projects[0].Streams = [streamConfig]);
			Assert.AreEqual(isShipped && isCutOver, streamConfig.Templates[0].Schedule?.Enabled);
		}
	}
}
