// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.TestData;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using MongoDB.Bson;
using Moq;

namespace HordeServer.Tests.Jobs.TestData
{
	/// <summary>
	/// Tests for the device service
	/// </summary>
	[TestClass]
	public class TestDataTests : BuildTestSetup
	{
		const string MainStreamName = "//UE5/Main";
		readonly StreamId _mainStreamId = new StreamId(StringId.Sanitize(MainStreamName));

		const string ReleaseStreamName = "//UE5/Release";
		readonly StreamId _releaseStreamId = new StreamId(StringId.Sanitize(ReleaseStreamName));

		IGraph _graph = default!;

		TestDataController? _testDataController;

		// override TestDataController with valid user
		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "MSTEST0036:Do not use shadowing", Justification = "<Pending>")]
		private new TestDataController TestDataController
		{
			get
			{
				if (_testDataController == null)
				{
					IUser user = UserCollection.FindOrAddUserByLoginAsync("TestUser").Result;
					_testDataController = base.TestDataController;
					ControllerContext controllerContext = new ControllerContext();
					controllerContext.HttpContext = new DefaultHttpContext();
					controllerContext.HttpContext.User = new ClaimsPrincipal(new ClaimsIdentity(
						new List<Claim> { HordeClaims.AdminClaim.ToClaim(),
						new Claim(ClaimTypes.Name, "TestUser"),
						new Claim(HordeClaimTypes.UserId, user.Id.ToString()) }
						, "TestAuthType"));
					_testDataController.ControllerContext = controllerContext;

				}
				return _testDataController;
			}
		}

		public static INode MockNode(string name, IReadOnlyNodeAnnotations annotations)
		{
			Mock<INode> node = new Mock<INode>(MockBehavior.Strict);
			node.SetupGet(x => x.Name).Returns(name);
			node.SetupGet(x => x.Annotations).Returns(annotations);
			return node.Object;
		}

		static StreamConfig CreateStream(StreamId streamId, string streamName)
		{
			StreamConfig streamConfig = new StreamConfig { Id = streamId, Name = streamName };
			streamConfig.Tabs.Add(new TabConfig { Title = "General", Templates = new List<TemplateId> { new TemplateId("test-template") } });
			streamConfig.Templates.Add(new TemplateRefConfig { Id = new TemplateId("test-template") });
			streamConfig.Workflows.Add(new HordeServer.Issues.WorkflowConfig() { Id = new WorkflowId("test-workflow"), ReportChannel = "some-channel" });
			return streamConfig;
		}

		[TestInitialize]
		public async Task SetupAsync()
		{
			ProjectConfig projectConfig = new ProjectConfig { Id = new ProjectId("ue5"), Name = "UE5" };
			projectConfig.Streams.Add(CreateStream(_mainStreamId, MainStreamName));
			projectConfig.Streams.Add(CreateStream(_releaseStreamId, ReleaseStreamName));

			BuildConfig buildConfig = new BuildConfig();
			buildConfig.Projects.Add(projectConfig);

			GlobalConfig globalConfig = new GlobalConfig();
			globalConfig.Plugins.AddBuildConfig(buildConfig);

			await SetConfigAsync(globalConfig);

			NodeAnnotations workflowAnnotations = new NodeAnnotations();
			workflowAnnotations.WorkflowId = new WorkflowId("test-workflow-id");

			List<INode> nodes = new List<INode>();
			nodes.Add(MockNode("Update Version Files", workflowAnnotations));
			nodes.Add(MockNode("Compile UnrealHeaderTool Win64", workflowAnnotations));
			nodes.Add(MockNode("Compile ShooterGameEditor Win64", workflowAnnotations));
			nodes.Add(MockNode("Cook ShooterGame Win64", workflowAnnotations));

			Mock<INodeGroup> grp = new Mock<INodeGroup>(MockBehavior.Strict);
			grp.SetupGet(x => x.Nodes).Returns(nodes);

			Mock<IGraph> graphMock = new Mock<IGraph>(MockBehavior.Strict);
			graphMock.SetupGet(x => x.Groups).Returns(new List<INodeGroup> { grp.Object });
			_graph = graphMock.Object;
		}

		public IJob CreateJob(StreamId streamId, int change, string name, IGraph graph, TimeSpan time = default)
		{
			JobId jobId = JobIdUtils.GenerateNewId();

			List<IJobStepBatch> batches = new List<IJobStepBatch>();
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup @group = graph.Groups[groupIdx];

				List<IJobStep> steps = new List<IJobStep>();
				for (int nodeIdx = 0; nodeIdx < @group.Nodes.Count; nodeIdx++)
				{
					JobStepId stepId = new JobStepId((ushort)((groupIdx * 100) + nodeIdx));

					ILog logFile = LogCollection.AddAsync(jobId, null, null, LogType.Json).Result;

					Mock<IJobStep> step = new Mock<IJobStep>(MockBehavior.Strict);
					step.SetupGet(x => x.Id).Returns(stepId);
					step.SetupGet(x => x.NodeIdx).Returns(nodeIdx);
					step.SetupGet(x => x.LogId).Returns(logFile.Id);
					step.SetupGet(x => x.StartTimeUtc).Returns(DateTime.UtcNow + time);
					step.SetupGet(x => x.Name).Returns(@group.Nodes[nodeIdx].Name);
					step.SetupGet(x => x.Annotations).Returns(@group.Nodes[nodeIdx].Annotations);

					steps.Add(step.Object);
				}

				JobStepBatchId batchId = new JobStepBatchId((ushort)(groupIdx * 100));

				Mock<IJobStepBatch> batch = new Mock<IJobStepBatch>(MockBehavior.Strict);
				batch.SetupGet(x => x.Id).Returns(batchId);
				batch.SetupGet(x => x.GroupIdx).Returns(groupIdx);
				batch.SetupGet(x => x.Steps).Returns(steps);
				batches.Add(batch.Object);
			}

			Mock<IJob> job = new Mock<IJob>(MockBehavior.Strict);
			job.SetupGet(x => x.Id).Returns(jobId);
			job.SetupGet(x => x.Name).Returns(name);
			job.SetupGet(x => x.StreamId).Returns(streamId);
			job.SetupGet(x => x.TemplateId).Returns(new TemplateId("test-template"));
			job.SetupGet(x => x.CommitId).Returns(CommitIdWithOrder.FromPerforceChange(change));
			job.SetupGet(x => x.PreflightCommitId).Returns(default(CommitId));
			job.SetupGet(x => x.Batches).Returns(batches);
			job.SetupGet(x => x.ShowUgsBadges).Returns(false);
			job.SetupGet(x => x.ShowUgsAlerts).Returns(false);
			job.SetupGet(x => x.PromoteIssuesByDefault).Returns(false);
			job.SetupGet(x => x.UpdateIssues).Returns(false);
			job.SetupGet(x => x.NotificationChannel).Returns("#devtools-horde-slack-testing");
			return job.Object;
		}

		[TestMethod]
		public async Task SimpleReportTestAsync()
		{
			await TestDataService.StartAsync(CancellationToken.None);
			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];

			BsonDocument testData1 = BsonDocument.Parse(String.Join('\n', _simpleTestDataLines));

			List<(string key, BsonDocument)> data1 = new List<(string key, BsonDocument)>();
			BsonArray items = testData1.GetValue("Items").AsBsonArray;
			foreach (BsonValue item in items.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("Data").AsBsonDocument;
				data1.Add(("Simple Report Key", value));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', _simpleTestDataLines2));

			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			items = testData2.GetValue("Items").AsBsonArray;
			foreach (BsonValue item in items.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("Data").AsBsonDocument;
				data2.Add(("Simple Report Key", value));
			}

			await TestDataCollection.AddAsync(job, step, data1.ToArray());
			await TestDataCollection.AddAsync(job2, step2, data1.ToArray());
			await TestDataCollection.AddAsync(job3, step3, data2.ToArray());

			await TestDataService.TickForTestingAsync();

			ActionResult<List<GetTestMetaResponse>> metaResult = await TestDataController.GetTestMetaAsync();
			Assert.IsNotNull(metaResult);
			Assert.IsNotNull(metaResult.Value);
			List<GetTestMetaResponse> meta = metaResult.Value;
			Assert.AreEqual(2, meta.Count);

			ActionResult<List<GetTestStreamResponse>> streamResult = await TestDataController.GetTestStreamsAsync(streamIds);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestStreamResponse> streams = streamResult.Value;

			Assert.AreEqual(2, streams.Count);
			Assert.AreEqual(1, streams[0].Tests.Count);
			Assert.AreEqual(2, streams[0].TestMetadata.Count);
			Assert.AreEqual(0, streams[0].TestSuites.Count);
			Assert.AreEqual(1, streams[1].Tests.Count);
			Assert.AreEqual(2, streams[1].TestMetadata.Count);
			Assert.AreEqual(0, streams[1].TestSuites.Count);

			Assert.AreEqual(streams[0].TestMetadata[0].Id, streams[1].TestMetadata[0].Id);

			ActionResult<List<GetTestDataRefResponse>> refResult = await TestDataController.GetTestDataRefAsync(streamIds, meta.Select(x => x.Id).ToArray(), streams[0].Tests.Select(t => t.Id).ToArray());
			Assert.IsNotNull(refResult);
			Assert.IsNotNull(refResult.Value);
			List<GetTestDataRefResponse> refs = refResult.Value;

			Assert.AreEqual(3, refs.Count);

			GetTestsRequest request = new GetTestsRequest() { TestIds = streams[0].Tests.Select(x => x.Id.ToString()).ToList() };
			ActionResult<List<GetTestResponse>> testResults = await TestDataController.GetTestsAsync(request);
			Assert.IsNotNull(testResults);
			Assert.IsNotNull(testResults.Value);
			Assert.IsNotNull(testResults.Value[0].Id, streams[0].Tests[0].Id);
		}

		internal class TestRequest
		{
			public List<string>? TestIds { get; set; }
		}

		[TestMethod]
		public async Task SessionReportTestAsync()
		{
			await TestDataService.StartAsync(CancellationToken.None);

			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];

			BsonDocument testData = BsonDocument.Parse(String.Join('\n', _testSessionDataLines));

			List<(string key, BsonDocument)> data = new List<(string key, BsonDocument)>();
			BsonArray items = testData.GetValue("Items").AsBsonArray;
			foreach (BsonValue item in items.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("Data").AsBsonDocument;
				data.Add(($"Session Report Key {data.Count}", value));
			}

			List<string> dataLines2 = new List<string>();
			for (int i = 0; i < _testSessionDataLines.Length; i++)
			{
				dataLines2.Add(_testSessionDataLines[i].Replace("Win64", "Android", StringComparison.OrdinalIgnoreCase));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', dataLines2));

			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			items = testData2.GetValue("Items").AsBsonArray;
			foreach (BsonValue item in items.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("Data").AsBsonDocument;
				data2.Add(($"Session Report Key {data2.Count}", value));
			}

			await TestDataCollection.AddAsync(job, step, data.ToArray());
			await TestDataCollection.AddAsync(job2, step2, data.ToArray());
			await TestDataCollection.AddAsync(job3, step3, data2.ToArray());

			await TestDataService.TickForTestingAsync();

			ActionResult<List<GetTestMetaResponse>> metaResult = await TestDataController.GetTestMetaAsync();
			Assert.IsNotNull(metaResult);
			Assert.IsNotNull(metaResult.Value);
			List<GetTestMetaResponse> meta = metaResult.Value;
			Assert.AreEqual(2, meta.Count);
			Assert.AreEqual(1, meta[0].BuildTargets.Count);
			Assert.AreEqual("Client", meta[0].BuildTargets[0]);
			Assert.AreEqual(1, meta[0].Platforms.Count);
			Assert.AreEqual("Win64", meta[0].Platforms[0]);
			Assert.AreEqual("Android", meta[1].Platforms[0]);
			Assert.AreEqual(1, meta[0].Configurations.Count);
			Assert.AreEqual("Development", meta[0].Configurations[0]);
			Assert.AreEqual("EngineTest", meta[0].ProjectName);
			Assert.AreEqual("default", meta[0].RHI);
			Assert.AreEqual("default", meta[0].Variation);

			ActionResult<List<GetTestStreamResponse>> streamResult = await TestDataController.GetTestStreamsAsync(streamIds);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestStreamResponse> streams = streamResult.Value;

			Assert.AreEqual(2, streams.Count);

			Assert.AreEqual(0, streams[0].Tests.Count);
			Assert.AreEqual(2, streams[0].TestMetadata.Count);
			Assert.AreEqual(1, streams[0].TestSuites.Count);

			Assert.AreEqual(0, streams[1].Tests.Count);
			Assert.AreEqual(2, streams[1].TestMetadata.Count);
			Assert.AreEqual(1, streams[1].TestSuites.Count);

			//Assert.AreEqual(streams[0].TestMetadata[0].Id, streams[1].TestMetadata[0].Id);

			ActionResult<List<GetTestDataRefResponse>> refResult = await TestDataController.GetTestDataRefAsync(streamIds, meta.Select(x => x.Id).ToArray());
			Assert.IsNotNull(refResult);
			Assert.IsNotNull(refResult.Value);
			List<GetTestDataRefResponse> refs = refResult.Value;

			Assert.AreEqual(3, refs.Count);

			Assert.AreEqual(1, refs[0].SuiteSkipCount);
			Assert.AreEqual(1, refs[0].SuiteWarningCount);
			Assert.AreEqual(1, refs[0].SuiteErrorCount);
		}

		[TestMethod]
		public async Task SessionReportTestV2Async()
		{
			await TestDataService.StartAsync(CancellationToken.None);

			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_mainStreamId, 106, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];
			IJobStep step4 = job3.Batches[0].Steps[1];

			BsonDocument testData = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines));
			List<(string key, BsonDocument)> data = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines2));
			BsonDocument testData3 = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines3));
			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData2.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data2.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}
			foreach (BsonValue item in testData3.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data2.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			await TestDataCollectionV2.AddAsync(job, step, data.ToArray());
			await TestDataCollectionV2.AddAsync(job2, step2, data.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step3, data.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step3, data2.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step4, data.ToArray());

			await TestDataService.TickForTestingAsync();

			GetStreamTestsRequest streamRequest = new GetStreamTestsRequest() { StreamIds = streamIds.ToList() };
			ActionResult<List<GetTestSessionStreamResponse>> streamResult = await TestDataController.GetTestStreamsV2Async(streamRequest);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestSessionStreamResponse> streams = streamResult.Value;
			Assert.AreEqual(2, streams.Count);
			Assert.AreEqual(2, streams[0].Tests.Count);
			Assert.AreEqual(3, streams[0].TestMetadata.Count);
			Assert.AreEqual(3, streams[0].TestTags.Count);
			Assert.AreEqual(1, streams[1].Tests.Count);
			Assert.AreEqual(1, streams[1].TestMetadata.Count);
			Assert.AreEqual(2, streams[1].TestTags.Count);

			GetTestsRequest testRequest = new GetTestsRequest() { TestIds = streams[0].Tests.Select(t => t.Id).ToList() };
			ActionResult<List<GetTestNameResponse>> testnameResult = await TestDataController.GetTestNameRefV2Async(testRequest);
			Assert.IsNotNull(testnameResult);
			Assert.IsNotNull(testnameResult.Value);
			List<GetTestNameResponse> testnames = testnameResult.Value;
			Assert.AreEqual(2, testnames.Count);
			Assert.IsTrue(testnames.Any(t => t.Key == "UE.Automation(Group:Editor) EngineTest"));
			Assert.IsTrue(testnames.Any(t => t.Key == "UE.Automation(Group:Editor) Lyra"));
			Assert.AreEqual("Editor", testnames[0].Name);
			Assert.IsNull(testnames[0].Team);

			GetMetadataRequest metaRequest = new GetMetadataRequest() { Entries = new() { { "Platform", "Win64" } } };
			ActionResult<List<GetTestMetadataResponse>> metaResult = await TestDataController.GetTestMetadataV2Async(metaRequest);
			Assert.IsNotNull(metaResult);
			Assert.IsNotNull(metaResult.Value);
			List<GetTestMetadataResponse> meta = metaResult.Value;
			Assert.AreEqual(2, meta.Count);
			Assert.IsTrue(meta.All(t => t.Entries.Any(i => i.Key == "BuildTarget" && i.Value == "Editor")));
			Assert.IsTrue(meta.Any(t => t.Entries.Any(i => i.Key == "Project" && i.Value == "EngineTest")));
			Assert.IsTrue(meta.Any(t => t.Entries.Any(i => i.Key == "Project" && i.Value == "Lyra")));

			GetTestTagRequest tagRequest = new GetTestTagRequest() { TagIds = streams[0].TestTags.Select(t => t.Id).ToList() };
			ActionResult<List<GetTestTagResponse>> tagResult = await TestDataController.GetTestTagRefV2Async(tagRequest);
			Assert.IsNotNull(tagResult);
			Assert.IsNotNull(tagResult.Value);
			List<GetTestTagResponse> tags = tagResult.Value;
			Assert.AreEqual(3, tags.Count);
			Assert.IsTrue(tags.Any(t => t.Name == "Engine"));
			Assert.IsTrue(tags.Any(t => t.Name == "Group"));
			Assert.IsTrue(tags.Any(t => t.Name == "Parser"));

			GetTestPhasesRequest phaseRequest = new GetTestPhasesRequest() { TestKeys = ["UE.Automation(Group:Editor) EngineTest"] };
			ActionResult<List<GetTestPhaseResponse>> phaseResult = await TestDataController.GetTestPhaseRefFromTestV2Async(phaseRequest);
			Assert.IsNotNull(phaseResult);
			Assert.IsNotNull(phaseResult.Value);
			List<GetTestPhaseResponse> tests = phaseResult.Value;
			Assert.AreEqual(1, tests.Count);
			Assert.AreEqual("Editor", tests[0].TestName);
			Assert.AreEqual(3, tests[0].Phases.Count);
			Assert.IsTrue(tests[0].Phases.Any(i => i.Name == "EditConditionParser.EvaluateBool"));
			Assert.IsTrue(tests[0].Phases.Any(i => i.Name == "Editor.Failing.Test"));
			GetPhaseResponse phase = tests[0].Phases.First(i => i.Name == "Editor.Failing.Test");
			string phaseId = phase.Id.ToString();

			GetTestSessionsRequest sessionsRequest = new GetTestSessionsRequest() { StreamIds = streamIds.ToList(), TestIds = testnames.Select(t => t.Id.ToString()).ToList() };
			ActionResult<List<GetTestSessionResponse>> testSessionResult = await TestDataController.GetTestSessionsV2Async(sessionsRequest);
			Assert.IsNotNull(testSessionResult);
			Assert.IsNotNull(testSessionResult.Value);
			List<GetTestSessionResponse> sessions = testSessionResult.Value;
			Assert.AreEqual(6, sessions.Count);
			Assert.AreEqual(TestOutcome.Failure, sessions[0].Outcome);
			Assert.IsTrue(sessions[0].TagIds?.All(id => tags.Any(t => t.Id == id)));
			Assert.IsNull(sessions[0].RecipeId);

			Assert.IsNotNull(sessions[0].ErrorFingerprints);
			Assert.AreEqual(1, sessions[0].ErrorFingerprints!.Count);
			GetErrorFingerprintResponse errorFingerprint = sessions[0].ErrorFingerprints![0];
			Assert.AreEqual(1, errorFingerprint.Phases.Count);
			Assert.AreEqual(phase.Key, errorFingerprint.Phases[0]);

			GetPhaseSessionsRequest phaseSessionRequest = new GetPhaseSessionsRequest() { StreamIds = streamIds.ToList(), PhaseIds = tests.SelectMany(t => t.Phases.Select(p => p.Id)).ToList() };
			ActionResult<List<GetTestPhaseSessionResponse>> phaseSessionResult = await TestDataController.GetTestPhaseSessionsV2Async(phaseSessionRequest);
			Assert.IsNotNull(phaseSessionResult);
			Assert.IsNotNull(phaseSessionResult.Value);
			List<GetTestPhaseSessionResponse> phaseSessions = phaseSessionResult.Value;
			Assert.AreEqual(15, phaseSessions.Count);
			Assert.AreEqual(TestPhaseOutcome.Failed, phaseSessions.First(p => p.PhaseRef == phaseId).Outcome);
			Assert.IsTrue(phaseSessions.Any(s => s.TagIds?.All(id => tags.Any(t => t.Id == id)) ?? false));

			Assert.IsTrue(phaseSessions.First(p => p.PhaseRef == phaseId).ErrorFingerprint == errorFingerprint.Key);

			// Get the sessions for one phase in one stream
			phaseSessionRequest = new GetPhaseSessionsRequest() { StreamIds = [streamIds[0]], PhaseIds = [tests[0].Phases[0].Id] };
			phaseSessionResult = await TestDataController.GetTestPhaseSessionsV2Async(phaseSessionRequest);
			Assert.IsNotNull(phaseSessionResult);
			Assert.IsNotNull(phaseSessionResult.Value);
			phaseSessions = phaseSessionResult.Value;
			Assert.AreEqual(4, phaseSessions.Count);

			ActionResult<object> testdata =  await TestDataController.GetTestDataV2Async(sessions[0].TestDataId.ToString());
			Assert.IsNotNull(testdata);
			Assert.IsNotNull(testdata.Value);

			ActionResult<List<GetJobStepTestDataIDResponse>> testDataIds = await TestDataController.GetJobStepTestDataV2Async(job3.Id.ToString(), step3.Id.ToString());
			Assert.IsNotNull(testDataIds);
			Assert.IsNotNull(testDataIds.Value);
			Assert.AreEqual(3, testDataIds.Value.Count);
			Assert.IsTrue(testDataIds.Value.All(item => sessions.Any(s => s.TestDataId == item.TestDataId)));
			Assert.IsTrue(testDataIds.Value.All(item => sessions.First(s => s.TestDataId == item.TestDataId).NameRef == item.TestNameRef));

			testDataIds = await TestDataController.GetJobStepTestDataV2Async(job3.Id.ToString());
			Assert.IsNotNull(testDataIds);
			Assert.IsNotNull(testDataIds.Value);
			Assert.AreEqual(4, testDataIds.Value.Count);
		}

		[TestMethod]
		public async Task SessionReportTestV2RecipeAsync()
		{
			await TestDataService.StartAsync(CancellationToken.None);

			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_mainStreamId, 106, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];

			BsonDocument testData = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines4));
			List<(string key, BsonDocument)> data = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines4));
			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData2.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data2.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			await TestDataCollectionV2.AddAsync(job, step, data.ToArray());
			await TestDataCollectionV2.AddAsync(job2, step2, data.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step3, data2.ToArray());

			await TestDataService.TickForTestingAsync();

			GetStreamTestsRequest streamRequest = new GetStreamTestsRequest() { StreamIds = streamIds.ToList() };
			ActionResult<List<GetTestSessionStreamResponse>> streamResult = await TestDataController.GetTestStreamsV2Async(streamRequest);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestSessionStreamResponse> streams = streamResult.Value;
			Assert.AreNotEqual(0, streams.Count);

			GetTestsRequest testRequest = new GetTestsRequest() { TestIds = streams[0].Tests.Select(t => t.Id).ToList() };
			ActionResult<List<GetTestNameResponse>> testnameResult = await TestDataController.GetTestNameRefV2Async(testRequest);
			Assert.IsNotNull(testnameResult);
			Assert.IsNotNull(testnameResult.Value);
			List<GetTestNameResponse> testnames = testnameResult.Value;
			Assert.AreEqual(1, testnames.Count);
			Assert.IsTrue(testnames.Any(t => t.Key == "UE.Automation(Group:Editor) Lyra"));
			Assert.AreEqual("Editor", testnames[0].Name);
			Assert.AreEqual("UE", testnames[0].Team);
			Assert.AreEqual("UE Functional Test", testnames[0].Harness);
			Assert.AreEqual("test intent", testnames[0].Intent);
			Assert.AreEqual("functional", testnames[0].Profile);

			GetTestRecipesRequest recipeRequest = new GetTestRecipesRequest() { TestIds = testnames.Select(t => t.Id).ToList() };
			ActionResult<List<GetTestRecipeResponse>> recipeResult = await TestDataController.GetTestRecipeRefV2Async(recipeRequest);
			Assert.IsNotNull(recipeResult);
			Assert.IsNotNull(recipeResult.Value);
			List<GetTestRecipeResponse> testRecipes = recipeResult.Value;
			Assert.AreEqual(2, testRecipes.Count);
			Assert.AreEqual("test-workflow-id", testRecipes[0].WorkflowId);
			Assert.AreEqual(_mainStreamId.ToString(), testRecipes[0].StreamId);
			Assert.AreEqual("test-template", testRecipes[0].JobTemplateId);
			Assert.AreEqual("Update Version Files", testRecipes[0].JobStep);
			Assert.AreEqual(10, testRecipes[0].TimeoutMinutes);
			Assert.IsTrue(testRecipes[0].Params?.Any(p => p.Key == "group" && p.Value.Equals("editor")));
			Assert.IsTrue(testRecipes[0].Params?.Any(p => p.Key == "project" && p.Value.Equals("Lyra")));

			Assert.AreEqual(_releaseStreamId.ToString(), testRecipes[1].StreamId);

			GetTestSessionsRequest sessionsRequest = new GetTestSessionsRequest() { StreamIds = streamIds.ToList(), TestIds = testnames.Select(t => t.Id.ToString()).ToList() };
			ActionResult<List<GetTestSessionResponse>> testSessionResult = await TestDataController.GetTestSessionsV2Async(sessionsRequest);
			Assert.IsNotNull(testSessionResult);
			Assert.IsNotNull(testSessionResult.Value);
			List<GetTestSessionResponse> sessions = testSessionResult.Value;
			Assert.AreEqual(3, sessions.Count);
			Assert.AreEqual(testRecipes[0].Id, sessions[0].RecipeId);
		}

		[TestMethod]
		public async Task SessionReportTestV2AuditAsync()
		{
			await TestDataService.StartAsync(CancellationToken.None);

			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_mainStreamId, 106, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];

			BsonDocument testData = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines4));
			List<(string key, BsonDocument)> data = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines2));
			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData2.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data2.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			await TestDataCollectionV2.AddAsync(job, step, data.ToArray());
			await TestDataCollectionV2.AddAsync(job2, step2, data.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step3, data2.ToArray());

			await TestDataService.TickForTestingAsync();

			GetStreamTestsRequest streamRequest = new GetStreamTestsRequest() { StreamIds = streamIds.ToList() };
			ActionResult<List<GetTestSessionStreamResponse>> streamResult = await TestDataController.GetTestStreamsV2Async(streamRequest);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestSessionStreamResponse> streams = streamResult.Value;
			Assert.AreNotEqual(0, streams.Count);
			Assert.AreEqual(2, streams[0].Tests.Count);
			Assert.IsNull(streams[0].Tests[0].Intent);
			Assert.AreEqual("UE", streams[0].Tests[0].Team);

			GetTestAuditRequest testRequestAll = new GetTestAuditRequest() { TestIds = streams[0].Tests.Select(t => t.Id).ToList() };
			ActionResult<List<GetTestAuditResponse>> testauditResult = await TestDataController.GetTestAuditsV2Async(testRequestAll);
			Assert.IsNotNull(testauditResult);
			Assert.IsNotNull(testauditResult.Value);
			List<GetTestAuditResponse> testaudits = testauditResult.Value;
			Assert.AreEqual(0, testaudits.Count);

			string? userIdString = TestDataController.HttpContext.User.GetUserId()?.ToString();

			// Add one entry for a test id
			DateTime auditDate = DateTime.UtcNow;
			string testId1 = streams[0].Tests.FirstOrDefault()!.Id;
			AddTestAuditRequest testAuditPostRequest = new AddTestAuditRequest()
			{
				TestId = testId1,
				Notes = "Some audit notes",
				LastAuditDate = auditDate,
			};
			ActionResult<GetTestAuditResponse> testauditPostResult = await TestDataController.AddTestAuditV2Async(testAuditPostRequest);
			Assert.IsNotNull(testauditPostResult);
			Assert.IsNotNull(testauditPostResult.Value);
			GetTestAuditResponse testauditPost = testauditPostResult.Value;
			Assert.AreEqual(testId1, testauditPost.TestId);
			Assert.AreEqual("Some audit notes", testauditPost.Notes);
			Assert.AreEqual(auditDate, testauditPost.LastAuditDate);
			Assert.AreEqual(true, testauditPost.EnableSchedule);
			Assert.AreEqual(userIdString, testauditPost.UserId);
			Assert.IsTrue(testauditPost.UserInputDate > auditDate);
			Assert.IsNull(testauditPost.Intent);

			// Add a second entry for another test id
			string testId2 = streams[0].Tests[1].Id;
			AddTestAuditRequest testAuditPostRequest2 = new AddTestAuditRequest()
			{
				TestId = testId2,
				Notes = "Some audit notes 2",
				LastAuditDate = auditDate,
			};
			testauditPostResult = await TestDataController.AddTestAuditV2Async(testAuditPostRequest2);
			Assert.IsNotNull(testauditPostResult);
			Assert.IsNotNull(testauditPostResult.Value);
			GetTestAuditResponse testauditPost2 = testauditPostResult.Value;
			Assert.AreEqual(testId2, testauditPost2.TestId);
			Assert.AreEqual("Some audit notes 2", testauditPost2.Notes);
			Assert.AreEqual(auditDate, testauditPost2.LastAuditDate);
			Assert.AreEqual(true, testauditPost2.EnableSchedule);
			Assert.IsNull(testauditPost2.Intent);

			// Check history output
			GetTestAuditRequest testRequestOne = new GetTestAuditRequest() { TestIds = [testId1] };
			testauditResult = await TestDataController.GetTestAuditsV2Async(testRequestOne);
			Assert.IsNotNull(testauditResult);
			Assert.IsNotNull(testauditResult.Value);
			testaudits = testauditResult.Value;
			Assert.AreEqual(1, testaudits.Count);
			testauditPost = testaudits[0];
			Assert.AreEqual(testId1, testauditPost.TestId);
			Assert.AreEqual("Some audit notes", testauditPost.Notes);
			Assert.AreEqual(auditDate.ToString(), testauditPost.LastAuditDate.ToString());
			Assert.AreEqual(true, testauditPost.EnableSchedule);
			Assert.IsNull(testauditPost.Intent);
			Assert.AreEqual(userIdString, testauditPost.UserId);
			Assert.IsTrue(testauditPost.UserInputDate > auditDate);

			// Add one more entry for first test id
			testAuditPostRequest = new AddTestAuditRequest()
			{
				TestId = testId1,
				Notes = "Some audit notes updated",
				Intent = "Test intent",
				Team = "TeamTest",
				Owner = userIdString,
				Customers = [userIdString!],
				Notifications = new Dictionary<string, string>(){ { _mainStreamId.ToString(), "test-workflow1" } , { _releaseStreamId.ToString(), "test-workflow2" } }
			};
			testauditPostResult = await TestDataController.AddTestAuditV2Async(testAuditPostRequest);
			Assert.IsNotNull(testauditPostResult);
			Assert.IsNotNull(testauditPostResult.Value);
			testauditPost = testauditPostResult.Value;
			Assert.AreEqual(testId1, testauditPost.TestId);
			Assert.AreEqual("Some audit notes updated", testauditPost.Notes);
			Assert.AreEqual("Test intent", testauditPost.Intent);
			Assert.AreEqual("TeamTest", testauditPost.Team);

			// Check history output with time range for one test
			testauditResult = await TestDataController.GetTestAuditsV2Async(testRequestOne, DateTimeOffset.UtcNow.AddHours(-1));
			Assert.IsNotNull(testauditResult);
			Assert.IsNotNull(testauditResult.Value);
			testaudits = testauditResult.Value;
			Assert.AreEqual(2, testaudits.Count);
			Assert.IsNull(testaudits[0].LastAuditDate);
			Assert.AreEqual(auditDate.ToString(), testaudits[1].LastAuditDate.ToString());

			// Check history output for latest for 2 tests
			testRequestOne = new GetTestAuditRequest() { TestIds = [testId1, testId2] };
			testauditResult = await TestDataController.GetTestAuditsV2Async(testRequestOne);
			Assert.IsNotNull(testauditResult);
			Assert.IsNotNull(testauditResult.Value);
			testaudits = testauditResult.Value;
			Assert.AreEqual(2, testaudits.Count);
			Assert.IsTrue(testRequestOne.TestIds.All(t => testaudits.Any( a => a.TestId == t)));
			testauditPost = testaudits[0];
			Assert.AreEqual("Some audit notes updated", testauditPost.Notes);
			Assert.AreEqual(userIdString, testauditPost.Owner);
			Assert.IsTrue(testauditPost.Customers?.All(c => c == userIdString));
			Assert.IsTrue(streamIds.All(s => testauditPost.Notifications?.Any(n => n.Key == s) ?? false));
			Assert.IsTrue(new List<string>(["test-workflow1", "test-workflow2"]).All(w => testauditPost.Notifications?.Any(n => n.Value == w) ?? false));
			Assert.IsNull(testauditPost.LastAuditDate);
			Assert.AreEqual("Test intent", testauditPost.Intent);
			Assert.AreEqual("TeamTest", testauditPost.Team);

			// Check test stream update
			streamRequest = new GetStreamTestsRequest() { StreamIds = streamIds.ToList() };
			streamResult = await TestDataController.GetTestStreamsV2Async(streamRequest);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			streams = streamResult.Value;
			Assert.AreEqual(2, streams.Count);
			Assert.AreEqual("TeamTest", streams[0].Tests[0].Team);
		}

		[TestMethod]
		public async Task TestHealthReportAsync()
		{
			await TestDataService.StartAsync(CancellationToken.None);

			string[] streamIds = new string[] { _mainStreamId.ToString(), _releaseStreamId.ToString() };

			IJob job = CreateJob(_mainStreamId, 105, "Test Build", _graph);
			IJobStep step = job.Batches[0].Steps[0];
			IJob job2 = CreateJob(_releaseStreamId, 105, "Test Build", _graph);
			IJobStep step2 = job2.Batches[0].Steps[0];
			IJob job3 = CreateJob(_mainStreamId, 106, "Test Build", _graph);
			IJobStep step3 = job3.Batches[0].Steps[0];
			IJob job4 = CreateJob(_releaseStreamId, 107, "Test Build", _graph);
			IJobStep step4 = job4.Batches[0].Steps[0];

			BsonDocument testData = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines5));
			List<(string key, BsonDocument)> data = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			BsonDocument testData2 = BsonDocument.Parse(String.Join('\n', _sessionTestDataV2Lines4));
			List<(string key, BsonDocument)> data2 = new List<(string key, BsonDocument)>();
			foreach (BsonValue item in testData2.GetValue("items").AsBsonArray.ToList())
			{
				BsonDocument value = item.AsBsonDocument.GetValue("data").AsBsonDocument;
				data2.Add((item.AsBsonDocument.GetValue("key").AsString, value));
			}

			await TestDataCollectionV2.AddAsync(job, step, data.ToArray());
			await TestDataCollectionV2.AddAsync(job2, step2, data.ToArray());
			await TestDataCollectionV2.AddAsync(job3, step3, data2.ToArray());

			await TestDataService.TickForTestingAsync();

			string? userIdString = TestDataController.HttpContext.User.GetUserId()?.ToString();

			GetStreamTestsRequest streamRequest = new GetStreamTestsRequest() { StreamIds = streamIds.ToList() };
			ActionResult<List<GetTestSessionStreamResponse>> streamResult = await TestDataController.GetTestStreamsV2Async(streamRequest);
			Assert.IsNotNull(streamResult);
			Assert.IsNotNull(streamResult.Value);
			List<GetTestSessionStreamResponse> streams = streamResult.Value;
			Assert.AreNotEqual(0, streams.Count);
			string streamId1 = streams[0].StreamId;
			string[] testIds = streams.SelectMany(s => s.Tests.Select(t => t.Id)).Distinct().ToArray();
			Assert.AreEqual(1, testIds.Length);
			string testId1 = testIds[0];

			AddTestAuditRequest testAuditPostRequest = new AddTestAuditRequest()
			{
				TestId = testId1,
				Owner = userIdString,
				Customers = [userIdString!],
				Notifications = new Dictionary<string, string>() { { _mainStreamId.ToString(), "test-workflow" }, { _releaseStreamId.ToString(), "test-workflow" } }
			};
			await TestDataController.AddTestAuditV2Async(testAuditPostRequest);

			GetTestHealthReportRequest testRequestOne = new GetTestHealthReportRequest() { StreamIds = [streamId1], TestIds = [testId1] };
			ActionResult<List<GetTestHealthReportResponse>> testHealthReports = await TestDataController.GetTestHealthReportsV2Async(testRequestOne);

			Assert.IsNotNull(testHealthReports);
			Assert.IsNotNull(testHealthReports.Value);
			List<GetTestHealthReportResponse> reports = testHealthReports.Value;
			Assert.AreEqual(0, reports.Count);

			await TestHealthService.TickForTestingAsync();

			testHealthReports = await TestDataController.GetTestHealthReportsV2Async(testRequestOne);

			Assert.IsNotNull(testHealthReports);
			Assert.IsNotNull(testHealthReports.Value);
			reports = testHealthReports.Value;
			Assert.AreEqual(1, reports.Count);

			GetTestHealthReportRequest testRequestAll = new GetTestHealthReportRequest() { StreamIds = streamIds.ToList() };
			testHealthReports = await TestDataController.GetTestHealthReportsV2Async(testRequestAll);

			Assert.IsNotNull(testHealthReports);
			Assert.IsNotNull(testHealthReports.Value);
			reports = testHealthReports.Value;
			Assert.AreEqual(2, reports.Count);

			Assert.AreEqual(true, reports[0].IsHealthy);
			Assert.AreEqual("Fair", reports[0].State);
			Assert.AreEqual(33, reports[0].SuccessRate);
			Assert.AreEqual(25, reports[0].CatastrophicFailureRate);
			Assert.AreEqual(25, reports[0].RedundantErrorRate);
			Assert.IsNull(reports[0].NotificationLastDate);

			Assert.AreEqual(false, reports[1].IsHealthy);
			Assert.AreEqual("Unstable[3 stars]", reports[1].State);
			Assert.AreEqual(0, reports[1].SuccessRate);
			Assert.AreEqual(33, reports[1].CatastrophicFailureRate);
			Assert.AreEqual(0, reports[1].RedundantErrorRate);
			Assert.IsNotNull(reports[1].NotificationLastDate);

			// Add another run that should improve only the unstable test
			await TestDataCollectionV2.AddAsync(job4, step4, data2.ToArray());
			await TestDataService.TickForTestingAsync();
			await TestHealthService.TickForTestingAsync(true);

			testHealthReports = await TestDataController.GetTestHealthReportsV2Async(testRequestAll);

			Assert.IsNotNull(testHealthReports);
			Assert.IsNotNull(testHealthReports.Value);
			reports = testHealthReports.Value;
			Assert.AreEqual(2, reports.Count);

			Assert.AreEqual(true, reports[0].IsHealthy);
			Assert.AreEqual("Fair", reports[0].State);
			Assert.AreEqual(33, reports[0].SuccessRate);
			Assert.AreEqual(25, reports[0].CatastrophicFailureRate);
			Assert.AreEqual(25, reports[0].RedundantErrorRate);
			Assert.IsNull(reports[0].NotificationLastDate);

			Assert.AreEqual(true, reports[1].IsHealthy);
			Assert.AreEqual("Fair", reports[1].State);
			Assert.AreEqual(33, reports[1].SuccessRate);
			Assert.AreEqual(25, reports[1].CatastrophicFailureRate);
			Assert.AreEqual(25, reports[1].RedundantErrorRate);
			Assert.IsNull(reports[1].NotificationLastDate);
		}

		private readonly string[] _simpleTestDataLines =
		{
			@"{",
			@"  ""Items"": [",
			@"    {",
			@"      ""Key"": ""Simple Report::UE.BootTest EngineTest Editor Win64"",",
			@"      ""Data"": {",
			@"  	  ""Version"" : 1,",
			@"        ""Type"": ""Simple Report"",",
			@"        ""TestName"": ""EditorBootTest"",",
			@"        ""Description"": ""Win64 Development EditorGame"",",
			@"        ""ReportCreatedOn"": ""10/31/2022 11:44:56 AM"",",
			@"        ""TotalDurationSeconds"": 35.33694,",
			@"        ""HasSucceeded"": true,",
			@"        ""Status"": ""Passed"",",
			@"        ""URLLink"": """",",
			@"        ""BuildChangeList"": 22815797,",
			@"        ""MainRole"": {",
			@"          ""Type"": ""Editor"",",
			@"          ""Platform"": ""Win64"",",
			@"          ""Configuration"": ""Development""",
			@"        },",
			@"        ""Roles"": [",
			@"          {",
			@"            ""Type"": ""Editor"",",
			@"            ""Platform"": ""Win64"",",
			@"            ""Configuration"": ""Development""",
			@"          }",
			@"        ],",
			@"        ""TestResult"": ""Passed"",",
			@"        ""Logs"": [],",
			@"        ""Errors"": [],",
			@"        ""Warnings"": [],",
			@"        ""Metadata"": {",
			@"          ""Platform"": ""Win64"",",
			@"          ""BuildTarget"": ""Editor"",",
			@"          ""Configuration"": ""Development"",",
			@"          ""Project"": ""EngineTest""",
			@"        }",
			@"      }",
			@"    }",
			@"  ],",
			@"}"
		};

		private readonly string[] _simpleTestDataLines2 =
		{
			@"{",
			@"  ""Items"": [",
			@"    {",
			@"      ""Key"": ""Simple Report::UE.BootTest EngineTest Editor Win64"",",
			@"      ""Data"": {",
			@"  	  ""Version"" : 1,",
			@"        ""Type"": ""Simple Report"",",
			@"        ""TestName"": ""EditorBootTest"",",
			@"        ""Description"": ""Win64 Development EditorGame"",",
			@"        ""ReportCreatedOn"": ""10/31/2022 11:44:56 AM"",",
			@"        ""TotalDurationSeconds"": 35.33694,",
			@"        ""HasSucceeded"": true,",
			@"        ""Status"": ""Passed"",",
			@"        ""URLLink"": """",",
			@"        ""BuildChangeList"": 22815797,",
			@"        ""MainRole"": {",
			@"          ""Type"": ""Client"",",
			@"          ""Platform"": ""Android"",",
			@"          ""Configuration"": ""Development""",
			@"        },",
			@"        ""Roles"": [",
			@"          {",
			@"            ""Type"": ""Client"",",
			@"            ""Platform"": ""Android"",",
			@"            ""Configuration"": ""Development""",
			@"          }",
			@"        ],",
			@"        ""TestResult"": ""Passed"",",
			@"        ""Logs"": [],",
			@"        ""Errors"": [],",
			@"        ""Warnings"": [],",
			@"        ""Metadata"": {",
			@"          ""Platform"": ""Android"",",
			@"          ""BuildTarget"": ""Client"",",
			@"          ""Configuration"": ""Development"",",
			@"          ""Project"": ""EngineTest""",
			@"        }",
			@"      }",
			@"    }",
			@"  ],",
			@"}"
		};

		private readonly string[] _testSessionDataLines =
		{
			@"{",
			@"    ""Items"": [	",
			@"        {",
			@"            ""Key"": ""Automated Test Session"",",
			@"            ""Data"": {",
			@"  	          ""Version"" : 1,",
			@"                ""Type"": ""Automated Test Session"",",
			@"                ""Name"": ""UE.Automation(Group:UI) EngineTest"",",
			@"                ""PreFlightChange"": """",",
			@"                ""TestSessionInfo"": {",
			@"                    ""DateTime"": ""2022.10.31-01.35.07"",",
			@"                    ""TimeElapseSec"": 17.63286,",
			@"                    ""Tests"": {",
			@"                        ""cbdb55ea"": {",
			@"                            ""Name"": ""Project.Functional Tests.Tests.UI.Clipping.WidgetClipping.Simple UI"",",
			@"                            ""TestUID"": ""cbdb55ea"",",
			@"                            ""Suite"": ""UI"",",
			@"                            ""State"": ""Skipped"",",
			@"                            ""DeviceAppInstanceName"": [",
			@"                                ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042""",
			@"                            ],",
			@"                            ""ErrorCount"": 0,",
			@"                            ""WarningCount"": 0,",
			@"                            ""ErrorHashAggregate"": """",",
			@"                            ""DateTime"": ""2022.10.31-01.34.41"",",
			@"                            ""TimeElapseSec"": 0",
			@"                        },",
			@"                        ""eafa1362"": {",
			@"                            ""Name"": ""Project.Functional Tests.Tests.UI.Effects.UI_Effects.Blur"",",
			@"                            ""TestUID"": ""eafa1362"",",
			@"                            ""Suite"": ""UI"",",
			@"                            ""State"": ""Success"",",
			@"                            ""DeviceAppInstanceName"": [",
			@"                                ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042""",
			@"                            ],",
			@"                            ""ErrorCount"": 0,",
			@"                            ""WarningCount"": 0,",
			@"                            ""ErrorHashAggregate"": """",",
			@"                            ""DateTime"": ""2022.10.31-01.34.49"",",
			@"                            ""TimeElapseSec"": 10.5334",
			@"                        },",
			@"                        ""1521079c"": {",
			@"                            ""Name"": ""Project.Functional Tests.Tests.UI.Fonts.FontOutlineTestUI.FontOutlineTest"",",
			@"                            ""TestUID"": ""1521079c"",",
			@"                            ""Suite"": ""UI"",",
			@"                            ""State"": ""Success"",",
			@"                            ""DeviceAppInstanceName"": [",
			@"                                ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042""",
			@"                            ],",
			@"                            ""ErrorCount"": 0,",
			@"                            ""WarningCount"": 0,",
			@"                            ""ErrorHashAggregate"": """",",
			@"                            ""DateTime"": ""2022.10.31-01.35.00"",",
			@"                            ""TimeElapseSec"": 3.53325",
			@"                        },",
			@"                        ""c2638213"": {",
			@"                            ""Name"": ""Project.Functional Tests.Tests.UI.Fonts.FontRenderingTestUI.FontRenderingTest"",",
			@"                            ""TestUID"": ""c2638213"",",
			@"                            ""Suite"": ""UI"",",
			@"                            ""State"": ""Fail"",",
			@"                            ""DeviceAppInstanceName"": [",
			@"                                ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042""",
			@"                            ],",
			@"                            ""ErrorCount"": 1,",
			@"                            ""WarningCount"": 2,",
			@"                            ""ErrorHashAggregate"": """",",
			@"                            ""DateTime"": ""2022.10.31-01.35.03"",",
			@"                            ""TimeElapseSec"": 3.56621",
			@"                        }",
			@"                    },",
			@"                    ""TestResultsTestDataUID"": ""b420bdde-c030-4add-81d2-3a8404ab3e45""",
			@"                },",
			@"                ""Devices"": [",
			@"                    {",
			@"                        ""Name"": ""RDU-WIN64-12"",",
			@"                        ""AppInstanceName"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""AppInstanceLog"": ""UI_Win64_Game/Client/ClientOutput.log"",",
			@"                        ""Metadata"": {",
			@"                            ""platform"": ""Win64"",",
			@"                            ""os_version"": ""14 3"",",
			@"                            ""model"": ""Default"",",
			@"                            ""gpu"": ""Win64 GPU"",",
			@"                            ""cpumodel"": ""Win64 CPU"",",
			@"                            ""ram_in_gb"": ""5"",",
			@"                            ""render_mode"": ""ES3_1"",",
			@"                            ""rhi"": """"",
			@"                        }",
			@"                    }",
			@"                ],",
			@"                ""IndexedErrors"": {},",
			@"                ""Metadata"": {",
			@"                    ""Platform"": ""Win64"",",
			@"                    ""BuildTarget"": ""Client"",",
			@"                    ""Configuration"": ""Development"",",
			@"                    ""Project"": ""EngineTest"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"            }",
			@"        },",
			@"        {",
			@"            ""Key"": ""Unreal Automated Tests::UE.TargetAutomation(RunTest=UI) Win64"",",
			@"            ""Data"": {",
			@"                ""Type"": ""Unreal Automated Tests"",",
			@"   		      ""Version"" : 1,",
			@"                ""Devices"": [",
			@"                    {",
			@"                        ""DeviceName"": ""RDU-WIN64-12"",",
			@"                        ""Instance"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""Platform"": ""Win64"",",
			@"                        ""OSVersion"": ""14 3"",",
			@"                        ""Model"": ""Default"",",
			@"                        ""GPU"": ""Win64 GPU"",",
			@"                        ""CPUModel"": ""Win64 CPU"",",
			@"                        ""RAMInGB"": 5,",
			@"                        ""RenderMode"": ""ES3_1"",",
			@"                        ""RHI"": """"",
			@"                    }",
			@"                ],",
			@"                ""ReportCreatedOn"": ""2022.10.31-01.35.07"",",
			@"                ""ReportURL"": """",",
			@"                ""SucceededCount"": 3,",
			@"                ""SucceededWithWarningsCount"": 0,",
			@"                ""FailedCount"": 0,",
			@"                ""NotRunCount"": 0,",
			@"                ""InProcessCount"": 0,",
			@"                ""TotalDurationSeconds"": 17.63286,",
			@"                ""Tests"": [",
			@"                    {",
			@"                        ""TestDisplayName"": ""Simple UI"",",
			@"                        ""FullTestPath"": ""Project.Functional Tests.Tests.UI.Clipping.WidgetClipping.Simple UI"",",
			@"                        ""State"": ""Skipped"",",
			@"                        ""DeviceInstance"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""Errors"": 0,",
			@"                        ""Warnings"": 0,",
			@"                        ""ArtifactName"": ""33ec1c33-71b7-41a1-84ba-d5ee2f42af8c.json""",
			@"                    },",
			@"                    {",
			@"                        ""TestDisplayName"": ""Blur"",",
			@"                        ""FullTestPath"": ""Project.Functional Tests.Tests.UI.Effects.UI_Effects.Blur"",",
			@"                        ""State"": ""Success"",",
			@"                        ""DeviceInstance"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""Errors"": 0,",
			@"                        ""Warnings"": 0,",
			@"                        ""ArtifactName"": ""450130ac-18cb-4bee-b529-c1d8ae943a61.json""",
			@"                    },",
			@"                    {",
			@"                        ""TestDisplayName"": ""FontOutlineTest"",",
			@"                        ""FullTestPath"": ""Project.Functional Tests.Tests.UI.Fonts.FontOutlineTestUI.FontOutlineTest"",",
			@"                        ""State"": ""Success"",",
			@"                        ""DeviceInstance"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""Errors"": 0,",
			@"                        ""Warnings"": 0,",
			@"                        ""ArtifactName"": ""12752762-6332-462f-8ef3-254c7502b57a.json""",
			@"                    },",
			@"                    {",
			@"                        ""TestDisplayName"": ""FontRenderingTest"",",
			@"                        ""FullTestPath"": ""Project.Functional Tests.Tests.UI.Fonts.FontRenderingTestUI.FontRenderingTest"",",
			@"                        ""State"": ""Success"",",
			@"                        ""DeviceInstance"": ""3da3633f-c82a-5c25-9a60-1e852705b65e-C0218E40D9AC4B4FBD07326E10179042"",",
			@"                        ""Errors"": 0,",
			@"                        ""Warnings"": 0,",
			@"                        ""ArtifactName"": ""711a5bd9-26dd-470b-a3c6-98f04a522e98.json""",
			@"                    }",
			@"                ],",
			@"                ""Metadata"": {",
			@"                    ""Platform"": ""Win64"",",
			@"                    ""BuildTarget"": ""Client"",",
			@"                    ""Configuration"": ""Development"",",
			@"                    ""Project"": ""EngineTest""",
			@"                }",
			@"            }",
			@"        },",
			@"        {",
			@"            ""Key"": ""Automated Test Session Result Details::b420bdde-c030-4add-81d2-3a8404ab3e45"",",
			@"            ""Data"": {",
			@"                ""cbdb55ea"": {",
			@"                    ""Events"": [",
			@"                        {",
			@"                            ""Message"": ""Skipping test because of exclude list: Sporadic failure with animated UI elements. Jira UE-113396"",",
			@"                            ""Context"": """",",
			@"                            ""Type"": ""Info"",",
			@"                            ""Tag"": ""entry"",",
			@"                            ""Hash"": """",",
			@"                            ""DateTime"": ""0001.01.01-00.00.00"",",
			@"                            ""Artifacts"": []",
			@"                        }",
			@"                    ]",
			@"                },",
			@"                ""eafa1362"": {",
			@"                    ""Events"": [",
			@"                        {",
			@"                            ""Message"": ""LogGauntlet: GauntletHeartbeat: Idle "",",
			@"                            ""Context"": """",",
			@"                            ""Type"": ""Info"",",
			@"                            ""Tag"": ""entry"",",
			@"                            ""Hash"": """",",
			@"                            ""DateTime"": ""0001.01.01-00.00.00"",",
			@"                            ""Artifacts"": []",
			@"                        },",
			@"                        {",
			@"                            ""Message"": ""Screenshot 'Blur' was similar!  Global Difference = 0.000303, Max Local Difference = 0.005257"",",
			@"                            ""Context"": ""UI_Effects/Blur"",",
			@"                            ""Type"": ""Info"",",
			@"                            ""Tag"": ""image comparison"",",
			@"                            ""Hash"": """",",
			@"                            ""DateTime"": ""0001.01.01-00.00.00"",",
			@"                            ""Artifacts"": [",
			@"                                {",
			@"                                    ""Tag"": ""unapproved"",",
			@"                                    ""ReferencePath"": ""imageCompare/UI_Effects/Blur/Win64/ES3_1/Incoming.png""",
			@"                                }",
			@"                            ]",
			@"                        }",
			@"                    ]",
			@"                },",
			@"                ""1521079c"": {",
			@"                    ""Events"": [",
			@"                        {",
			@"                            ""Message"": ""Screenshot 'FontOutlineTest' was similar!  Global Difference = 0.000083, Max Local Difference = 0.003520"",",
			@"                            ""Context"": ""FontOutlineTestUI/FontOutlineTest"",",
			@"                            ""Type"": ""Info"",",
			@"                            ""Tag"": ""image comparison"",",
			@"                            ""Hash"": """",",
			@"                            ""DateTime"": ""0001.01.01-00.00.00"",",
			@"                            ""Artifacts"": [",
			@"                                {",
			@"                                    ""Tag"": ""unapproved"",",
			@"                                    ""ReferencePath"": ""imageCompare/FontOutlineTestUI/FontOutlineTest/Win64/ES3_1/Incoming.png""",
			@"                                }",
			@"                            ]",
			@"                        }",
			@"                    ]",
			@"                },",
			@"                ""c2638213"": {",
			@"                    ""Events"": [",
			@"                        {",
			@"                            ""Message"": ""Screenshot 'FontRenderingTest' was similar!  Global Difference = 0.000000, Max Local Difference = 0.000000"",",
			@"                            ""Context"": ""FontRenderingTestUI/FontRenderingTest"",",
			@"                            ""Type"": ""Info"",",
			@"                            ""Tag"": ""image comparison"",",
			@"                            ""Hash"": """",",
			@"                            ""DateTime"": ""0001.01.01-00.00.00"",",
			@"                            ""Artifacts"": [",
			@"                                {",
			@"                                    ""Tag"": ""unapproved"",",
			@"                                    ""ReferencePath"": ""imageCompare/FontRenderingTestUI/FontRenderingTest/Win64/ES3_1/Incoming.png""",
			@"                                }",
			@"                            ]",
			@"                        }",
			@"                    ]",
			@"                }",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};

		private readonly string[] _sessionTestDataV2Lines =
		{
			@"{",
			@"  ""items"": [",
			@"        {",
			@"            ""key"": ""UE.Automation(Group:Editor) EngineTest"",",
			@"            ""data"": {",
			@"   		      ""version"" : 2,",
			@"                ""summary"": {",
			@"                   ""testName"": ""Editor"",",
			@"        		     ""dateTime"": ""2025-03-03T18:14:04"",",
			@"		             ""timeElapseSec"": 207.02324,",
			@"        		     ""phasesTotalCount"": 1075,",
			@"		             ""phasesSucceededCount"": 1074,",
			@"        		     ""phasesUndefinedCount"": 0,",
			@"		             ""phasesFailedCount"": 1",
			@"        		  },",
			@"                ""phases"": [",
			@"                    {",
			@"                        ""key"": ""874a6711"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBitFlags"",",
			@"                        ""dateTime"": ""2025-03-03T18:07:55"",",
			@"                        ""timeElapseSec"": 0.0879574,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                    },",
			@"                    {",
			@"                        ""key"": ""03f772e3"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBool"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:08"",",
			@"                        ""timeElapseSec"": 0.008163903,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""tags"": [""Group""],",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                   },",
			@"                   {",
			@"                        ""key"": ""62d99b42"",",
			@"                        ""name"": ""Editor.Failing.Test"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:14"",",
			@"                        ""timeElapseSec"": 0.1961886,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc189"",",
			@"                        ""tags"": [""Engine""],",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ],",
			@"                        ""eventStreamPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/EventStreams/62d99b42.json""",
			@"                   }",
			@"                ]",
			@"                ""tags"": [""Group"", ""Engine""]",
			@"                ""metadata"": {",
			@"                    ""Platform"": ""Win64"",",
 			@"                    ""BuildTarget"": ""Editor"",",
			@"                    ""Configuration"": ""Development"",",
 			@"                    ""Project"": ""EngineTest"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"                ""devices"": {",
			@"                  ""9AF6B8CC484096D351EC56ABDFBB919F"": {",
			@"                    ""name"": ""M-SIHLOYQHDOUKO"",",
			@"                    ""appInstanceLogPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/Editor/EditorOutput.log"",",
			@"                    ""metadata"": {",
			@"                      ""platform"": ""WindowsEditor"",",
			@"                      ""os_version"": ""Windows 10 (22H2) [10.0.19045.5487] "",",
			@"                      ""model"": ""Default"",",
			@"                      ""gpu"": ""Parsec Virtual Display Adapter"",",
			@"                      ""cpumodel"": ""AMD EPYC 7R32"",",
			@"                      ""ram_in_gb"": ""127"",",
			@"                      ""render_mode"": ""D3D12_SM6"",",
			@"                      ""rhi"": ""D3D12 (SM6)""",
			@"                    }",
			@"                  }",
			@"                },",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};

		private readonly string[] _sessionTestDataV2Lines2 =
		{
			@"{",
			@"  ""items"": [",
			@"        {",
			@"            ""key"": ""UE.Automation(Group:Editor) EngineTest"",",
			@"            ""data"": {",
			@"   		      ""version"" : 2,",
			@"                ""summary"": {",
			@"                   ""testName"": ""Editor"",",
			@"        		     ""dateTime"": ""2025-03-03T18:14:04"",",
			@"		             ""timeElapseSec"": 207.02324,",
			@"        		     ""phasesTotalCount"": 1075,",
			@"		             ""phasesSucceededCount"": 1074,",
			@"        		     ""phasesUndefinedCount"": 0,",
			@"		             ""phasesFailedCount"": 1",
			@"        		  },",
			@"                ""phases"": [",
			@"                    {",
			@"                        ""key"": ""874a6711"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBitFlags"",",
			@"                        ""dateTime"": ""2025-03-03T18:07:55"",",
			@"                        ""timeElapseSec"": 0.0879574,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                    },",
			@"                    {",
			@"                        ""key"": ""03f772e3"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBool"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:08"",",
			@"                        ""timeElapseSec"": 0.008163903,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""tags"": [""Parser""],",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                   },",
			@"                   {",
			@"                        ""key"": ""62d99b42"",",
			@"                        ""name"": ""Editor.Failing.Test"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:14"",",
			@"                        ""timeElapseSec"": 0.1961886,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc189"",",
			@"                        ""tags"": [""Engine""],",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ],",
			@"                        ""eventStreamPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/EventStreams/62d99b42.json""",
			@"                   }",
			@"                ]",
			@"                ""tags"": [""Parser"", ""Engine""]",
			@"                ""metadata"": {",
			@"                    ""Platform"": ""Mac"",",
 			@"                    ""BuildTarget"": ""Editor"",",
			@"                    ""Configuration"": ""Development"",",
 			@"                    ""Project"": ""EngineTest"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"                ""devices"": {",
			@"                  ""9AF6B8CC484096D351EC56ABDFBB919F"": {",
			@"                    ""name"": ""M-SIHLOYQHDOUKO"",",
			@"                    ""appInstanceLogPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/Editor/EditorOutput.log"",",
			@"                    ""metadata"": {",
			@"                      ""platform"": ""WindowsEditor"",",
			@"                      ""os_version"": ""Windows 10 (22H2) [10.0.19045.5487] "",",
			@"                      ""model"": ""Default"",",
			@"                      ""gpu"": ""Parsec Virtual Display Adapter"",",
			@"                      ""cpumodel"": ""AMD EPYC 7R32"",",
			@"                      ""ram_in_gb"": ""127"",",
			@"                      ""render_mode"": ""D3D12_SM6"",",
			@"                      ""rhi"": ""D3D12 (SM6)""",
			@"                    }",
			@"                  }",
			@"                },",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};

		private readonly string[] _sessionTestDataV2Lines3 =
		{
			@"{",
			@"  ""items"": [",
			@"        {",
			@"            ""key"": ""UE.Automation(Group:Editor) Lyra"",",
			@"            ""data"": {",
			@"   		      ""version"" : 2,",
			@"                ""summary"": {",
			@"                   ""testName"": ""Editor"",",
			@"        		     ""dateTime"": ""2025-03-03T18:14:04"",",
			@"		             ""timeElapseSec"": 207.02324,",
			@"        		     ""phasesTotalCount"": 1075,",
			@"		             ""phasesSucceededCount"": 1074,",
			@"        		     ""phasesUndefinedCount"": 0,",
			@"		             ""phasesFailedCount"": 1",
			@"        		  },",
			@"                ""phases"": [",
			@"                    {",
			@"                        ""key"": ""874a6711"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBitFlags"",",
			@"                        ""dateTime"": ""2025-03-03T18:07:55"",",
			@"                        ""timeElapseSec"": 0.0879574,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                    },",
			@"                    {",
			@"                        ""key"": ""03f772e3"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBool"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:08"",",
			@"                        ""timeElapseSec"": 0.008163903,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                   },",
			@"                   {",
			@"                        ""key"": ""62d99b42"",",
			@"                        ""name"": ""Editor.Failing.Test"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:14"",",
			@"                        ""timeElapseSec"": 0.1961886,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc189"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ],",
			@"                        ""eventStreamPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/EventStreams/62d99b42.json""",
			@"                   }",
			@"                ]",
			@"                ""metadata"": {",
			@"                    ""Platform"": ""Win64"",",
 			@"                    ""BuildTarget"": ""Editor"",",
			@"                    ""Configuration"": ""Development"",",
 			@"                    ""Project"": ""Lyra"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"                ""devices"": {",
			@"                  ""9AF6B8CC484096D351EC56ABDFBB919F"": {",
			@"                    ""name"": ""M-SIHLOYQHDOUKO"",",
			@"                    ""appInstanceLogPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/Editor/EditorOutput.log"",",
			@"                    ""metadata"": {",
			@"                      ""platform"": ""WindowsEditor"",",
			@"                      ""os_version"": ""Windows 10 (22H2) [10.0.19045.5487] "",",
			@"                      ""model"": ""Default"",",
			@"                      ""gpu"": ""Parsec Virtual Display Adapter"",",
			@"                      ""cpumodel"": ""AMD EPYC 7R32"",",
			@"                      ""ram_in_gb"": ""127"",",
			@"                      ""render_mode"": ""D3D12_SM6"",",
			@"                      ""rhi"": ""D3D12 (SM6)""",
			@"                    }",
			@"                  }",
			@"                },",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};

		private readonly string[] _sessionTestDataV2Lines4 =
		{
			@"{",
			@"  ""items"": [",
			@"        {",
			@"            ""key"": ""UE.Automation(Group:Editor) Lyra"",",
			@"            ""data"": {",
			@"   		      ""version"" : 2,",
			@"                ""summary"": {",
			@"                   ""testName"": ""Editor"",",
			@"        		     ""dateTime"": ""2025-03-03T18:14:04"",",
			@"		             ""timeElapseSec"": 207.02324,",
			@"                   ""commandline"": ""run ue.automation -group=editor -project=Lyra"",",
			@"        		     ""phasesTotalCount"": 3,",
			@"		             ""phasesSucceededCount"": 2,",
			@"        		     ""phasesUndefinedCount"": 0,",
			@"		             ""phasesFailedCount"": 1",
			@"        		  },",
			@"                ""phases"": [",
			@"                    {",
			@"                        ""key"": ""874a6711"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBitFlags"",",
			@"                        ""dateTime"": ""2025-03-03T18:07:55"",",
			@"                        ""timeElapseSec"": 0.0879574,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                    },",
			@"                    {",
			@"                        ""key"": ""03f772e3"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBool"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:08"",",
			@"                        ""timeElapseSec"": 0.008163903,",
			@"                        ""outcome"": ""Success"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                   },",
			@"                   {",
			@"                        ""key"": ""62d99b42"",",
			@"                        ""name"": ""Editor.Failing.Test"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:14"",",
			@"                        ""timeElapseSec"": 0.1961886,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc189"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ],",
			@"                        ""eventStreamPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/EventStreams/62d99b42.json""",
			@"                   }",
			@"                ]",
			@"                ""description"": {",
			@"                    ""intent"": ""test intent"",",
 			@"                    ""profile"": ""functional"",",
			@"                    ""harness"": ""UE Functional Test"",",
 			@"                    ""team"": ""UE""",
			@"                }",
			@"                ""recipe"": {",
			@"                    ""nodeClass"": ""testClass"",",
 			@"                    ""version"": ""1.0.0"",",
 			@"                    ""timeoutMinutes"": 10,",
			@"                    ""harness"": ""UE Functional Test"",",
 			@"                    ""params"": {",
			@"                      ""group"": ""editor"",",
			@"                      ""project"": ""Lyra""",
			@"                    }",
			@"                }",
			@"                ""metadata"": {",
			@"                    ""Platform"": ""Win64"",",
 			@"                    ""BuildTarget"": ""Editor"",",
			@"                    ""Configuration"": ""Development"",",
 			@"                    ""Project"": ""Lyra"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"                ""devices"": {",
			@"                  ""9AF6B8CC484096D351EC56ABDFBB919F"": {",
			@"                    ""name"": ""M-SIHLOYQHDOUKO"",",
			@"                    ""appInstanceLogPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/Editor/EditorOutput.log"",",
			@"                    ""metadata"": {",
			@"                      ""platform"": ""WindowsEditor"",",
			@"                      ""os_version"": ""Windows 10 (22H2) [10.0.19045.5487] "",",
			@"                      ""model"": ""Default"",",
			@"                      ""gpu"": ""Parsec Virtual Display Adapter"",",
			@"                      ""cpumodel"": ""AMD EPYC 7R32"",",
			@"                      ""ram_in_gb"": ""127"",",
			@"                      ""render_mode"": ""D3D12_SM6"",",
			@"                      ""rhi"": ""D3D12 (SM6)""",
			@"                    }",
			@"                  }",
			@"                },",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};

		private readonly string[] _sessionTestDataV2Lines5 =
		{
			@"{",
			@"  ""items"": [",
			@"        {",
			@"            ""key"": ""UE.Automation(Group:Editor) Lyra"",",
			@"            ""data"": {",
			@"   		      ""version"" : 2,",
			@"                ""summary"": {",
			@"                   ""testName"": ""Editor"",",
			@"        		     ""dateTime"": ""2025-03-03T18:14:04"",",
			@"		             ""timeElapseSec"": 207.02324,",
			@"                   ""commandline"": ""run ue.automation -group=editor -project=Lyra"",",
			@"        		     ""phasesTotalCount"": 3,",
			@"		             ""phasesSucceededCount"": 0,",
			@"        		     ""phasesUndefinedCount"": 1,",
			@"		             ""phasesFailedCount"": 2",
			@"        		  },",
			@"                ""phases"": [",
			@"                    {",
			@"                        ""key"": ""874a6711"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBitFlags"",",
			@"                        ""dateTime"": ""2025-03-03T18:07:55"",",
			@"                        ""timeElapseSec"": 0.0879574,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc189"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                    },",
			@"                    {",
			@"                        ""key"": ""03f772e3"",",
			@"                        ""name"": ""EditConditionParser.EvaluateBool"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:08"",",
			@"                        ""timeElapseSec"": 0.008163903,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc188"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ]",
			@"                   },",
			@"                   {",
			@"                        ""key"": ""62d99b42"",",
			@"                        ""name"": ""Editor.Failing.Test"",",
			@"                        ""dateTime"": ""2025-03-03T18:08:14"",",
			@"                        ""timeElapseSec"": 0.1961886,",
			@"                        ""outcome"": ""Failed"",",
			@"                        ""errorFingerprint"": ""a6efc180"",",
			@"                        ""deviceKeys"": [",
			@"                            ""9AF6B8CC484096D351EC56ABDFBB919F""",
			@"                        ],",
			@"                   }",
			@"                ]",
			@"                ""description"": {",
			@"                    ""intent"": ""test intent"",",
 			@"                    ""profile"": ""functional"",",
			@"                    ""harness"": ""UE Functional Test"",",
 			@"                    ""team"": ""UE""",
			@"                }",
			@"                ""recipe"": {",
			@"                    ""nodeClass"": ""testClass"",",
 			@"                    ""version"": ""1.0.0"",",
 			@"                    ""timeoutMinutes"": 10,",
			@"                    ""harness"": ""UE Functional Test"",",
 			@"                    ""params"": {",
			@"                      ""group"": ""editor"",",
			@"                      ""project"": ""Lyra""",
			@"                    }",
			@"                }",
			@"                ""metadata"": {",
			@"                    ""Platform"": ""Win64"",",
 			@"                    ""BuildTarget"": ""Editor"",",
			@"                    ""Configuration"": ""Development"",",
 			@"                    ""Project"": ""Lyra"",",
			@"                    ""RHI"": ""default""",
			@"                }",
			@"                ""devices"": {",
			@"                  ""9AF6B8CC484096D351EC56ABDFBB919F"": {",
			@"                    ""name"": ""M-SIHLOYQHDOUKO"",",
			@"                    ""appInstanceLogPath"": ""Engine/Programs/AutomationTool/Saved/Logs/UE.EditorAutomation(RunTest=Group_Editor)_(Win64_Development_Editor)/Editor/EditorOutput.log"",",
			@"                    ""metadata"": {",
			@"                      ""platform"": ""WindowsEditor"",",
			@"                      ""os_version"": ""Windows 10 (22H2) [10.0.19045.5487] "",",
			@"                      ""model"": ""Default"",",
			@"                      ""gpu"": ""Parsec Virtual Display Adapter"",",
			@"                      ""cpumodel"": ""AMD EPYC 7R32"",",
			@"                      ""ram_in_gb"": ""127"",",
			@"                      ""render_mode"": ""D3D12_SM6"",",
			@"                      ""rhi"": ""D3D12 (SM6)""",
			@"                    }",
			@"                  }",
			@"                },",
			@"            }",
			@"        }",
			@"    ]",
			@"}"
		};
	}
}
