// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.DependencyInjection;
using Moq;

namespace HordeServer.Tests.Notifications
{
	public class JobNotificationInformation
	{
		public IJob Job { get; }
		public IJobStepBatch? Batch { get; } = null;
		public IJobStep? Step { get; } = null;
		public INode? Node { get; } = null;
		public List<ILogEventData>? JobStepEventData { get; } = null;
		public IEnumerable<IUser>? UsersToNotify { get; } = null;
		public JobNotificationInformation(IJob job, IJobStepBatch? batch = null, IJobStep? step = null, INode? node = null, List<ILogEventData>? jobStepEventData = null, IEnumerable<IUser>? usersToNotify = null)
		{
			Job = job;
			Batch = batch;
			Step = step;
			Node = node;
			JobStepEventData = jobStepEventData;
			UsersToNotify = usersToNotify;
		}
	}

	public class FakeNotificationSink : INotificationSink
	{
		public JobNotificationInformation? JobNotification { get; set; } = null;
		public List<JobNotificationInformation> JobStepNotifications { get; } = new();

		public List<JobScheduledNotification> JobScheduledNotifications { get; } = new();
		public int JobScheduledCallCount { get; set; }

		public Task NotifyJobScheduledAsync(List<JobScheduledNotification> notifications, CancellationToken cancellationToken)
		{
			JobScheduledNotifications.AddRange(notifications);
			JobScheduledCallCount++;
			return Task.CompletedTask;
		}

		public Task NotifyJobStepCompleteAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			JobNotificationInformation jobNotificationInformation = new JobNotificationInformation(job, batch, step, node, jobStepEventData, usersToNotify);
			JobStepNotifications.Add(jobNotificationInformation);
			return Task.CompletedTask;
		}

		public Task NotifyJobStepAbortedAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			JobNotificationInformation jobNotificationInformation = new JobNotificationInformation(job, batch, step, node, jobStepEventData, usersToNotify);
			JobStepNotifications.Add(jobNotificationInformation);
			return Task.CompletedTask;
		}

		public Task NotifyJobCompleteAsync(IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			JobNotification = new JobNotificationInformation(job);
			return Task.CompletedTask;
		}

		public Task NotifyJobCompleteAsync(IUser user, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken) { throw new NotImplementedException(); }
		public Task NotifyLabelCompleteAsync(IUser user, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> stepData, CancellationToken cancellationToken) { throw new NotImplementedException(); }
		public Task NotifyIssueUpdatedAsync(IIssue issue, CancellationToken cancellationToken) { throw new NotImplementedException(); }
		public Task NotifyConfigUpdateAsync(ConfigUpdateInfo info, CancellationToken cancellationToken) => Task.CompletedTask;
		public Task NotifyConfigUpdateFailureAsync(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
		public Task NotifyDeviceServiceAsync(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? stream = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default) { throw new NotImplementedException(); }
		public Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task SendSessionConflictReportAsync(IReadOnlyList<(AgentId Id, int Count)> conflicts, CancellationToken cancellationToken) => Task.CompletedTask;
		public Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken) => throw new NotImplementedException();
		public Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken = default) => throw new NotImplementedException();
		public Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken = default) => throw new NotImplementedException();
	}

	[TestClass]
	public class NotificationServiceTests : BuildTestSetup
	{
		readonly StreamId _streamId = new("ue4-main");
		readonly StreamConfig _streamConfig;
		readonly TemplateId _templateId1 = new("template1");

		public NotificationServiceTests()
		{
			_streamConfig = new StreamConfig { Id = _streamId };
		}

		protected override void ConfigureServices(IServiceCollection services)
		{
			base.ConfigureServices(services);

			services.AddSingleton<FakeNotificationSink>();
			services.AddSingleton<INotificationSink>(sp => sp.GetRequiredService<FakeNotificationSink>());
		}

		public static IJob CreateJob(StreamId streamId, int change, string name, IGraph graph)
		{
			JobId jobId = JobId.Parse("5ec16da1774cb4000107c2c1");

			List<IJobStepBatch> batches = new List<IJobStepBatch>();
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup @group = graph.Groups[groupIdx];

				List<IJobStep> steps = new List<IJobStep>();
				for (int nodeIdx = 0; nodeIdx < @group.Nodes.Count; nodeIdx++)
				{
					JobStepId stepId = new JobStepId((ushort)((groupIdx * 100) + nodeIdx));

					Mock<IJobStep> step = new Mock<IJobStep>(MockBehavior.Strict);
					step.SetupGet(x => x.Id).Returns(stepId);
					step.SetupGet(x => x.NodeIdx).Returns(nodeIdx);

					steps.Add(step.Object);
				}

				JobStepBatchId batchId = new JobStepBatchId((ushort)(groupIdx * 100));

				Mock<IJobStepBatch> batch = new Mock<IJobStepBatch>(MockBehavior.Strict);
				batch.SetupGet(x => x.Id).Returns(batchId);
				batch.SetupGet(x => x.GroupIdx).Returns(groupIdx);
				batch.SetupGet(x => x.Steps).Returns(steps);
				batches.Add(batch.Object);
			}

			Mock<IJob> job = new Mock<IJob>(MockBehavior.Strict);
			job.SetupGet(x => x.Id).Returns(jobId);
			job.SetupGet(x => x.Name).Returns(name);
			job.SetupGet(x => x.StreamId).Returns(streamId);
			job.SetupGet(x => x.CommitId).Returns(CommitIdWithOrder.FromPerforceChange(change));
			job.SetupGet(x => x.Batches).Returns(batches);
			return job.Object;
		}

		static NewGroup AddGroup(List<NewGroup> groups)
		{
			NewGroup group = new NewGroup("win64", new List<NewNode>());
			groups.Add(group);
			return group;
		}

		static NewNode AddNode(NewGroup group, string name, string[]? inputDependencies, Action<NewNode>? action = null)
		{
			NewNode node = new NewNode(name, inputDependencies: inputDependencies?.ToList(), orderDependencies: inputDependencies?.ToList());
			action?.Invoke(node);
			group.Nodes.Add(node);
			return node;
		}

		async Task UpdateGlobalConfigAsync()
		{
			await UpdateConfigAsync(globalConfig =>
			{
				ProjectConfig projectConfig = new() { Id = new ProjectId("ue4") };
				projectConfig.Streams.Add(new StreamConfig { Id = _streamId });
				globalConfig.Plugins.GetBuildConfig().Projects.Add(projectConfig);
			});
		}

		async Task<IJob> CreateJobAsync()
		{
			Mock<ITemplate> templateMock = new Mock<ITemplate>(MockBehavior.Strict);
			templateMock.SetupGet(x => x.InitialAgentType).Returns((string?)null);

			IGraph baseGraph = await GraphCollection.AddAsync(templateMock.Object);

			CreateJobOptions options = new CreateJobOptions();
			options.Arguments.Add("-Target=Publish Client");
			options.Arguments.Add("-Target=Post-Publish Client");

			List<NewGroup> newGroups = new List<NewGroup>();

			NewGroup initialGroup = AddGroup(newGroups);
			AddNode(initialGroup, "Update Version Files", null);
			AddNode(initialGroup, "Compile Editor", new[] { "Update Version Files" });

			NewGroup compileGroup = AddGroup(newGroups);
			AddNode(compileGroup, "Compile Client", new[] { "Update Version Files" });

			NewGroup publishGroup = AddGroup(newGroups);
			AddNode(publishGroup, "Cook Client", new[] { "Compile Editor" }, x => x.RunEarly = true);
			AddNode(publishGroup, "Publish Client", new[] { "Compile Client", "Cook Client" });
			AddNode(publishGroup, "Post-Publish Client", null, x => x.OrderDependencies = new List<string> { "Publish Client" });

			IGraph graph = await GraphCollection.AppendAsync(baseGraph, newGroups);

			ITemplate template1 = new TemplateStub(ContentHash.MD5("graphHash"), "template1");
			return await JobService.CreateJobAsync(null, _streamConfig!, _templateId1, template1.Hash, graph, "Test job", CommitIdWithOrder.FromPerforceChange(123), CommitIdWithOrder.FromPerforceChange(123), options);
		}

		async Task<IJob> AbortJobAsync(IJob job)
		{
			job = Deref(await JobService.UpdateJobAsync(job, abortedByUserId: UserId.Anonymous));
			Assert.AreEqual(JobState.Complete, job.GetState());
			return job;
		}

		Task<IJob> RunBatchAsync(IJob job, int batchIdx)
		{
			return UpdateBatchAsync(job, batchIdx, JobStepBatchState.Running);
		}

		async Task<IJob> UpdateBatchAsync(IJob job, int batchIdx, JobStepBatchState state)
		{
			Assert.AreEqual(JobStepBatchState.Ready, job.Batches[batchIdx].State);

			LogId logId = LogIdUtils.GenerateNewId();
			job = Deref(await job.TryAssignLeaseAsync(batchIdx, new PoolId("foo"), new AgentId("agent"), new SessionId(BinaryIdUtils.CreateNew()), new LeaseId(BinaryIdUtils.CreateNew()), logId));
			if (job.Batches[batchIdx].State != state)
			{
				job = Deref(await JobService.UpdateBatchAsync(job, job.Batches[batchIdx].Id, _streamConfig, logId, state));
				Assert.AreEqual(state, job.Batches[batchIdx].State);
			}
			return job;
		}

		async Task<IJob> AbortStepAsync(IJob job, int batchIdx, int stepIdx)
		{
			LogId logId = LogIdUtils.GenerateNewId();
			await LogCollection.AddAsync(job.Id, job.Batches[batchIdx].LeaseId, job.Batches[batchIdx].SessionId, LogType.Json, logId);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, _streamConfig, newAbortRequested: true, newAbortByUserId: UserId.Anonymous, newLogId: logId));
			Assert.AreEqual(JobStepState.Aborted, job.Batches[batchIdx].Steps[stepIdx].State);
			return job;
		}

		async Task<IJob> RunStepAsync(IJob job, int batchIdx, int stepIdx, JobStepOutcome outcome)
		{
			Assert.AreEqual(JobStepState.Ready, job.Batches[batchIdx].Steps[stepIdx].State);

			LogId logId = LogIdUtils.GenerateNewId();
			await LogCollection.AddAsync(job.Id, job.Batches[batchIdx].LeaseId, job.Batches[batchIdx].SessionId, LogType.Json, logId);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, _streamConfig, JobStepState.Running, JobStepOutcome.Success, newLogId: logId));
			Assert.AreEqual(JobStepState.Running, job.Batches[batchIdx].Steps[stepIdx].State);
			job = Deref(await JobService.UpdateStepAsync(job, job.Batches[batchIdx].Id, job.Batches[batchIdx].Steps[stepIdx].Id, _streamConfig, JobStepState.Completed, outcome, newLogId: logId));
			Assert.AreEqual(JobStepState.Completed, job.Batches[batchIdx].Steps[stepIdx].State);
			Assert.AreEqual(outcome, job.Batches[batchIdx].Steps[stepIdx].Outcome);
			return job;
		}

		static async Task ValidateJobNotificationAsync(FakeNotificationSink sink, IJob job, JobState state, UserId? abortedByUserId = null, int timeoutMs = 30000)
		{
			DateTime endTime = DateTime.Now + TimeSpan.FromMilliseconds(timeoutMs);

			while (endTime >= DateTime.Now)
			{
				if (sink.JobNotification is not null)
				{
					Assert.IsNotNull(sink.JobNotification);
					Assert.AreEqual(job.Id, sink.JobNotification.Job.Id);
					Assert.AreEqual(state, sink.JobNotification.Job.GetState());
					Assert.AreEqual(abortedByUserId, sink.JobNotification.Job.AbortedByUserId);
					sink.JobNotification = null;
					return;
				}
				await Task.Delay(100);
			}
			Assert.Fail("Unable to validate job notification for job '{JobId}'");
		}

		static async Task ValidateJobStepNotificationAsync(FakeNotificationSink sink, IJob job, int batchIdx, int stepIdx, JobStepState state, JobStepOutcome outcome, int timeoutMs = 30000)
		{
			DateTime endTime = DateTime.Now + TimeSpan.FromMilliseconds(timeoutMs);

			while (endTime >= DateTime.Now)
			{
				if (sink.JobStepNotifications.Any())
				{
					JobNotificationInformation jobNotificationInformation = sink.JobStepNotifications[0];
					Assert.AreEqual(job.Id, jobNotificationInformation.Job.Id);
					Assert.IsNotNull(jobNotificationInformation.Step);

					IJobStep step = job.Batches[batchIdx].Steps[stepIdx];
					Assert.AreEqual(step.Id, jobNotificationInformation.Step.Id);
					Assert.AreEqual(state, jobNotificationInformation.Step.State);
					Assert.AreEqual(outcome, jobNotificationInformation.Step.Outcome);
					sink.JobStepNotifications.RemoveAt(0);
					return;
				}
				await Task.Delay(100);
			}
			Assert.Fail("Unable to validate job step notification for job '{JobId}'");
		}

		[TestMethod]
		public async Task NotifyJobScheduledAsync()
		{
			FakeNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service._ticker.StartAsync();

			Fixture fixture = await CreateFixtureAsync();
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "BogusPool", Properties = new Dictionary<string, string>() });

			Assert.AreEqual(0, fakeSink.JobScheduledNotifications.Count);
			service.NotifyJobScheduled(pool, false, fixture.Job1, fixture.Graph, JobStepBatchId.GenerateNewId());
			service.NotifyJobScheduled(pool, false, fixture.Job2, fixture.Graph, JobStepBatchId.GenerateNewId());

			// Currently no good way to wait for NotifyJobScheduled() to complete as the execution is completely async in background task (see ExecuteAsync)
			await Task.Delay(1000);
			await Clock.AdvanceAsync(service._notificationBatchInterval + TimeSpan.FromMinutes(5));
			Assert.AreEqual(2, fakeSink.JobScheduledNotifications.Count);
			Assert.AreEqual(1, fakeSink.JobScheduledCallCount);
		}

		[TestMethod]
		public async Task JobScheduledNotificationsAreDeduplicatedAsync()
		{
			FakeNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeNotificationSink>();
			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service._ticker.StartAsync();
			Fixture fixture = await CreateFixtureAsync();
			IPool pool = await CreatePoolAsync(new PoolConfig { Name = "BogusPool", Properties = new Dictionary<string, string>() });

			service.NotifyJobScheduled(pool, false, fixture.Job1, fixture.Graph, JobStepBatchId.GenerateNewId());
			service.NotifyJobScheduled(pool, false, fixture.Job1, fixture.Graph, JobStepBatchId.GenerateNewId());
			service.NotifyJobScheduled(pool, false, fixture.Job1, fixture.Graph, JobStepBatchId.GenerateNewId());

			// Currently no good way to wait for NotifyJobScheduled() to complete as the execution is completely async in background task (see ExecuteAsync)
			await Task.Delay(1000);
			await Clock.AdvanceAsync(service._notificationBatchInterval + TimeSpan.FromMinutes(5));

			// Only one job scheduled notification should have been sent, despite queuing three
			Assert.AreEqual(1, fakeSink.JobScheduledNotifications.Count);

			// Clear the cache by compacting it 100%
			MemoryCache cache = (MemoryCache)ServiceProvider.GetRequiredService<IMemoryCache>();
			cache.Compact(1.0);

			// Notify of exactly the same job again
			service.NotifyJobScheduled(pool, false, fixture.Job1, fixture.Graph, JobStepBatchId.GenerateNewId());

			await Task.Delay(1000);
			await Clock.AdvanceAsync(service._notificationBatchInterval + TimeSpan.FromMinutes(5));
			Assert.AreEqual(2, fakeSink.JobScheduledNotifications.Count);
		}

		[TestMethod]
		public async Task NotifyJobCompleteAsync()
		{
			await UpdateGlobalConfigAsync();

			FakeNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service._ticker.StartAsync();

			IJob job = await CreateJobAsync();
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build
			await ValidateJobStepNotificationAsync(fakeSink, job, 0, 0, JobStepState.Completed, JobStepOutcome.Success);

			// Trigger a cancellation of the 'Cook Client' step
			job = await AbortStepAsync(job, 3, 0);
			await ValidateJobStepNotificationAsync(fakeSink, job, 3, 0, JobStepState.Aborted, JobStepOutcome.Failure);

			job = await RunBatchAsync(job, 1);
			job = await RunStepAsync(job, 1, 0, JobStepOutcome.Success); // Update Version Files
			await ValidateJobStepNotificationAsync(fakeSink, job, 1, 0, JobStepState.Completed, JobStepOutcome.Success);

			job = await RunStepAsync(job, 1, 1, JobStepOutcome.Success); // Compile Editor
			await ValidateJobStepNotificationAsync(fakeSink, job, 1, 1, JobStepState.Completed, JobStepOutcome.Success);

			job = await RunBatchAsync(job, 2);
			job = await RunStepAsync(job, 2, 0, JobStepOutcome.Failure); // Compile Client
			await ValidateJobStepNotificationAsync(fakeSink, job, 2, 0, JobStepState.Completed, JobStepOutcome.Failure);

			// Validate that the states did not change after running the other steps
			Assert.AreEqual(JobStepBatchState.Complete, job.Batches[3].State);
			Assert.AreEqual(JobStepState.Aborted, job.Batches[3].Steps[0].State); // Cook Client
			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[1].State); // Publish Client
			Assert.AreEqual(JobStepState.Skipped, job.Batches[3].Steps[2].State); // Post-Publish Client

			// Validate that we received our job completion notification
			await ValidateJobNotificationAsync(fakeSink, job, JobState.Complete);
		}

		[TestMethod]
		public async Task NotifyJobAbortedAsync()
		{
			await UpdateGlobalConfigAsync();

			FakeNotificationSink fakeSink = ServiceProvider.GetRequiredService<FakeNotificationSink>();

			NotificationService service = (NotificationService)ServiceProvider.GetRequiredService<INotificationService>();
			await service._ticker.StartAsync();

			IJob job = await CreateJobAsync();
			job = await RunBatchAsync(job, 0);
			job = await RunStepAsync(job, 0, 0, JobStepOutcome.Success); // Setup Build
			await ValidateJobStepNotificationAsync(fakeSink, job, 0, 0, JobStepState.Completed, JobStepOutcome.Success);

			// Trigger a cancellation of the job
			job = await AbortJobAsync(job);

			// Validate that we received our job completion notification
			await ValidateJobNotificationAsync(fakeSink, job, JobState.Complete, UserId.Anonymous);
		}

		//public void NotifyLabelUpdate(IJob Job, IReadOnlyList<(LabelState, LabelOutcome)> OldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> NewLabelStates)
		//{
		//	// If job has any label trigger IDs, send label complete notifications if needed
		//	if (Job.LabelIdxToTriggerId.Any())
		//	{
		//		EnqueueTask(() => SendAllLabelNotificationsAsync(Job, OldLabelStates, NewLabelStates));
		//	}
		//}

		//private async Task SendAllLabelNotificationsAsync(IJob Job, IReadOnlyList<(LabelState State, LabelOutcome Outcome)> OldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> NewLabelStates)
		//{
		//	IStream? Stream = await StreamService.GetStreamAsync(Job.StreamId);
		//	if (Stream == null)
		//	{
		//		return;
		//	}

		//	IGraph? Graph = await GraphCollection.GetAsync(Job.GraphHash);
		//	if (Graph == null)
		//	{
		//		return;
		//	}

		//	IReadOnlyDictionary<NodeRef, IJobStep> StepForNode = Job.GetStepForNodeMap();
		//	for (int LabelIdx = 0; LabelIdx < Graph.Labels.Count; ++LabelIdx)
		//	{
		//		(LabelState State, LabelOutcome Outcome) OldLabel = OldLabelStates[LabelIdx];
		//		(LabelState State, LabelOutcome Outcome) NewLabel = NewLabelStates[LabelIdx];
		//		if (OldLabel != NewLabel)
		//		{
		//			// If the state transitioned from Unspecified to Running, don't update unless the outcome also changed.
		//			if (OldLabel.State == LabelState.Unspecified && NewLabel.State == LabelState.Running && OldLabel.Outcome == NewLabel.Outcome)
		//			{
		//				continue;
		//			}

		//			// If the label isn't complete, don't report on outcome changing to success, this will be reported when the label state becomes complete.
		//			if (NewLabel.State != LabelState.Complete && NewLabel.Outcome == LabelOutcome.Success)
		//			{
		//				return;
		//			}

		//			bool fireTrigger = NewLabel.State == LabelState.Complete;
		//			INotificationTrigger? Trigger = await GetNotificationTrigger(Job.LabelIdxToTriggerId[LabelIdx], fireTrigger);
		//			if (Trigger == null)
		//			{
		//				continue;
		//			}

		//			await SendLabelUpdateNotificationsAsync(Job, Stream, Graph, StepForNode, Graph.Labels[LabelIdx], NewLabel.State, NewLabel.Outcome, Trigger);
		//		}
		//	}
		//}
	}
}
