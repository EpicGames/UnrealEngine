// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using HordeServer.VersionControl.Perforce;
using HordeServer.Server;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tests.Perforce;

/// <summary>
/// Fake implementation of IHealthMonitor for tests
/// </summary>
/// <typeparam name="T"></typeparam>
public class FakeHealthMonitor<T> : IHealthMonitor<T>
{
	/// <summary>Name set by SetName()</summary>
	public string Name { get; set; } = "<not-set>";

	/// <summary>Updates sent</summary>
	public List<(HealthStatus result, string? message, DateTimeOffset? timestamp)> Updates { get; } = [];

	/// <inheritdoc/>
	public void SetName(string name) { Name = name; }

	/// <inheritdoc/>
	public Task UpdateAsync(HealthStatus result, string? message = null, DateTimeOffset? timestamp = null)
	{
		Updates.Add((result, message, timestamp));
		return Task.CompletedTask;
	}
}

internal class HttpMessageHandlerStub(HttpStatusCode statusCode, string content) : HttpMessageHandler
{
	public string Content { get; set; } = content;
	protected override Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
	{
		return Task.FromResult(new HttpResponseMessage(statusCode) { Content = new StringContent(Content) });
	}
}

[TestClass]
public class PerforceLoadBalancerTests : BuildTestSetup
{
	private const string PerforceHostname = "";

	private readonly ILogger<PerforceLoadBalancer> _logger = CreateConsoleLogger<PerforceLoadBalancer>();

	[TestMethod]
	public async Task StatusHealthyViaInfoAsync()
	{
		CheckPerforceServerIsSet();
		GlobalConfig gc = GetConfig([PerforceHostname]);
		PerforceLoadBalancer plb = Create(gc, "");
		await plb.UpdateHealthTestOnlyAsync("myCluster", PerforceHostname, null, CancellationToken.None);
		IPerforceServer server = (await plb.GetServersAsync(CancellationToken.None)).First();
		Assert.AreEqual(PerforceServerStatus.Healthy, server.Status);
	}

	[TestMethod]
	public async Task StatusHealthyViaHttpAsync()
	{
		CheckPerforceServerIsSet();
		GlobalConfig gc = GetConfig([PerforceHostname]);
		PerforceLoadBalancer plb = Create(gc, GetHealthCheckJsonForStatus(PerforceServerStatus.Healthy));
		await plb.UpdateHealthTestOnlyAsync("myCluster", PerforceHostname, "http://stub-handler-is-used", CancellationToken.None);
		IPerforceServer server = (await plb.GetServersAsync(CancellationToken.None)).First();
		Assert.AreEqual(PerforceServerStatus.Healthy, server.Status);
	}

	[TestMethod]
	public async Task StatusUnhealthyViaHttpAsync()
	{
		CheckPerforceServerIsSet();
		GlobalConfig gc = GetConfig([PerforceHostname]);
		PerforceLoadBalancer plb = Create(gc, GetHealthCheckJsonForStatus(PerforceServerStatus.Unhealthy));
		await plb.UpdateHealthTestOnlyAsync("myCluster", PerforceHostname, "http://stub-handler-is-used", CancellationToken.None);
		IPerforceServer server = (await plb.GetServersAsync(CancellationToken.None)).First();
		Assert.AreEqual(PerforceServerStatus.Unhealthy, server.Status);
	}
	
	[TestMethod]
	[Ignore] // Needs better stubbing of health checks to not run for an excessively long time
	public async Task ServersAddRemoveUpAsync()
	{
		const string Server1 = "server1:1666";
		const string Server2 = "server2:1666";
		GlobalConfig gc = GetConfig([Server1]);
		PerforceLoadBalancer plb = Create(gc, GetHealthCheckJsonForStatus(PerforceServerStatus.Healthy));
		
		// No servers before first tick
		Assert.AreEqual(0, (await plb.GetServersAsync(CancellationToken.None)).Count);
		
		// Servers are updated after first tick
		await plb.TickInternalTestOnlyAsync(CancellationToken.None);
		List<IPerforceServer> servers = await plb.GetServersAsync(CancellationToken.None);
		Assert.AreEqual(1, servers.Count);
		Assert.AreEqual(Server1, servers[0].ServerAndPort);
		
		// Add another server
		gc.Plugins.GetBuildConfig().PerforceClusters = [CreateCluster(Server1, Server2)];
		await plb.TickInternalTestOnlyAsync(CancellationToken.None);
		servers = await plb.GetServersAsync(CancellationToken.None);
		Assert.AreEqual(2, servers.Count);
		Assert.AreEqual(Server1, servers[0].ServerAndPort);
		Assert.AreEqual(Server2, servers[1].ServerAndPort);
		
		// Remove a server
		gc.Plugins.GetBuildConfig().PerforceClusters = [CreateCluster(Server2)];
		await plb.TickInternalTestOnlyAsync(CancellationToken.None);
		servers = await plb.GetServersAsync(CancellationToken.None);
		Assert.AreEqual(1, servers.Count);
		Assert.AreEqual(Server2, servers[0].ServerAndPort);
	}

	internal static GlobalConfig GetConfig(params string[] perforceHosts)
	{
		PerforceCluster perforceCluster = CreateCluster(perforceHosts);
		GlobalConfig gc = new();
		gc.Plugins.AddBuildConfig(new BuildConfig());
		gc.Plugins.GetBuildConfig().PerforceClusters = [perforceCluster];
		return gc;
	}
	
	private static PerforceCluster CreateCluster(params string[] perforceHosts)
	{
		return new PerforceCluster { Servers = perforceHosts.Select(x => new PerforceServer { ServerAndPort = x }).ToList() };
	}

	private static string GetHealthCheckJsonForStatus(PerforceServerStatus status)
	{
		return status switch
		{
			PerforceServerStatus.Healthy => GetHealthCheckJson("green"),
			PerforceServerStatus.Degraded => GetHealthCheckJson("yellow"),
			PerforceServerStatus.Unhealthy => GetHealthCheckJson("red"),
			_ => GetHealthCheckJson("this-is-not-valid"),
		};
	}

	private static string GetHealthCheckJson(string output)
	{
		string result = """{"checker": "edge_traffic_lights", "output": "%OUTPUT%"}""".Replace("%OUTPUT%", output, StringComparison.Ordinal);
		return """{"results": [%RESULT%]}""".Replace("%RESULT%", result, StringComparison.Ordinal);
	}

	private static void CheckPerforceServerIsSet()
	{
		if (String.IsNullOrEmpty(PerforceHostname))
		{
			Assert.Inconclusive($"Set {nameof(PerforceHostname)} to a valid Perforce server to run load balancer tests");
		}
	}

	private PerforceLoadBalancer Create(GlobalConfig gc, string httpCheckResponse)
	{
#pragma warning disable CA2000 // Dispose objects before losing scope
		HttpClient httpClient = new(new HttpMessageHandlerStub(HttpStatusCode.OK, httpCheckResponse));
		return new(MongoService, GetRedisServiceSingleton(), LeaseCollection, Clock, httpClient, new TestOptionsMonitor<BuildConfig>(gc.Plugins.GetBuildConfig()), new FakeHealthMonitor<PerforceLoadBalancer>(), Tracer, _logger);
#pragma warning restore CA2000 // Dispose objects before losing scope		
	}
}

