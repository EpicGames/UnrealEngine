// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Secrets;
using HordeServer.Secrets;
using HordeServer.Server;

namespace HordeServer.Tests.Perforce;

[TestClass]
public class PerforceSecretsTests : BuildTestSetup
{
	public PerforceSecretsTests()
	{
		AddPlugin<SecretsPlugin>();
	}

	[TestInitialize]
	public async Task SetupAsync()
	{
		GlobalConfig globalConfig = new() { ResolveSecrets = true };
		globalConfig.Plugins.AddBuildConfig(new BuildConfig());
		globalConfig.Plugins.AddSecretsConfig(new SecretsConfig());
		await SetConfigAsync(globalConfig);
	}

	private async Task AddCredentialsAsync(PerforceCredentials credentials, SecretConfig? secret)
	{
		await UpdateConfigAsync(globalConfig =>
		{
			if (secret != null)
			{
				globalConfig.Plugins.GetSecretsConfig().Secrets.Add(secret);
			}
			globalConfig.Plugins.GetBuildConfig().PerforceClusters.Add(new PerforceCluster { Credentials = [credentials], Name = "Test" });
		});
	}

	[TestMethod]
	public async Task ResolvePerforcePasswordWithSecretAsync()
	{
		PerforceCredentials credentials = new() { Password = "horde:secret:p4secret.Password" };
		SecretConfig secret = new()
		{
			Id = new SecretId("p4secret"),
			Data = new Dictionary<string, string> { { "Password", "Hello, World!" } }
		};
		await AddCredentialsAsync(credentials, secret);
		Assert.AreEqual("Hello, World!", credentials.Password);
		Assert.AreEqual(credentials.Ticket, String.Empty);
	}

	[TestMethod]
	public async Task ResolvePerforceTicketWithSecretAsync()
	{
		PerforceCredentials credentials = new() { Ticket = "horde:secret:p4secret.Ticket" };
		SecretConfig secret = new()
		{
			Id = new SecretId("p4secret"),
			Data = new Dictionary<string, string> { { "Ticket", "Hello, World!" } }
		};
		await AddCredentialsAsync(credentials, secret);
		Assert.AreEqual("Hello, World!", credentials.Ticket);
		Assert.AreEqual(credentials.Password, String.Empty);
	}

	[TestMethod]
	public async Task ResolvePerforceCredentialsWithoutSecretAsync()
	{
		PerforceCredentials credentials = new() { Password = "password", Ticket = "ticket" };
		await AddCredentialsAsync(credentials, null);
		Assert.AreEqual("password", credentials.Password);
		Assert.AreEqual("ticket", credentials.Ticket);
	}

	[TestMethod]
	[DataRow("horde:secret:notfound.property", DisplayName = "Resolve Perforce Secret, secret not found")]
	[DataRow("horde:secret:p4secret.notfound", DisplayName = "Resolve Perforce Secret, property not found")]
	public async Task ResolvePerforceCredentialsWithSecretNotFoundAsync(string value)
	{
		PerforceCredentials credentials = new() { Password = value, Ticket = value };
		SecretConfig secret = new() { Id = new SecretId("p4secret") };
		await Assert.ThrowsExactlyAsync<KeyNotFoundException>(async () =>
		{
			try
		{
			await AddCredentialsAsync(credentials, secret);
		}
		catch (AggregateException ae)
		{
			throw ae.InnerExceptions[0];
		}
		});
	}

	[TestMethod]
	public async Task ResolvePerforceCredentialsWithSecretMisconfigurationAsync()
	{
		PerforceCredentials credentials = new() { Password = "horde:secret:p4secret.property" };
		SecretConfig secret = new()
		{
			Id = new SecretId("p4secret"),
			Sources = [ new ExternalSecretConfig() { Provider = "SecretProviderKeyNotFound" } ]
		};
		await Assert.ThrowsExactlyAsync<KeyNotFoundException>(async () =>
		{
			try
		{
			await AddCredentialsAsync(credentials, secret);
		}
		catch (AggregateException ae)
		{
			throw ae.InnerExceptions[0];
		}
		});
	}
}
