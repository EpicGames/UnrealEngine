// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using EpicGames.Perforce;
using HordeServer.Acls;
using HordeServer.Plugins;
using HordeServer.Projects;
using HordeServer.VersionControl.Perforce;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tests.Perforce;

public class TelemetryWriterStub : ITelemetryWriter
{
	public bool Enabled { get; } = false;
	public void WriteEvent(TelemetryStoreId telemetryStoreId, object payload) => throw new NotImplementedException();
	public void WriteEvent(TelemetryStoreId telemetryStoreId, TelemetryRecordMeta metadata, object payload) => throw new NotImplementedException();
}

[TestClass]
public class PerforceServiceCacheTests : BuildTestSetup
{
	private readonly ILogger<PerforceLoadBalancer> _logger = CreateConsoleLogger<PerforceLoadBalancer>();

	[TestMethod]
	[Ignore] // Requires configuring the Perforce fixture server
	public async Task CacheCommit_GetFiles_Async()
	{
		ILogger<PerforceServiceCache> pscLogger = CreateConsoleLogger<PerforceServiceCache>();
		BuildServerConfig bsc = new ();
		TestOptions<BuildServerConfig> bscWrapper = new (bsc);

		List<string> servers = ["localhost"];
		StreamConfig sc = new() { Id = new StreamId("FooMain"), Name = "//Foo/Main", ClusterName = "MyCluster" };
		PerforceCluster perforceCluster = new() { Name = sc.ClusterName, Servers = servers.Select(x => new PerforceServer { ServerAndPort = x }).ToList() };
		
		BuildConfig buildConfig = new ();
		buildConfig.Projects.Add(new ProjectConfig { Streams = [sc]});
		buildConfig.PerforceClusters = [perforceCluster];

		GlobalConfig gc = new();
		gc.Plugins.AddBuildConfig(buildConfig);
		gc.Plugins.AddComputeConfig(new ComputeConfig());
		gc.PostLoad(new ServerSettings(), Array.Empty<ILoadedPlugin>(), Array.Empty<IDefaultAclModifier>());
		TestOptionsMonitor<BuildConfig> bcWrapper = new (buildConfig);
		
		PerforceLoadBalancer plb = CreatePlb(gc, "notUsed");
		await using PerforceServiceCache psc = new (
			plb, MongoService, GetRedisServiceSingleton(), DowntimeService, UserCollection, Clock,
			bscWrapper, bcWrapper, new TelemetryWriterStub(), Tracer, pscLogger);
		
		ICommitCollection cc = psc.GetCommits(sc);

		PerforceServiceCache.StreamInfo streamInfo = new (sc, new PerforceViewMap([new PerforceViewMapEntry(true, sc.Name + "/...", "...")]), new PerforceChangeView());
		await psc.UpdateClusterInternalAsync(sc.ClusterName, [streamInfo], null, CancellationToken.None);

		ICommit commit = await cc.GetAsync(CommitIdWithOrder.FromPerforceChange(2));
		IReadOnlyList<string> files = await commit.GetFilesAsync(null, null, CancellationToken.None);
		CollectionAssert.AreEquivalent(new List<string> { "Data/data.txt", "common.h", "main.cpp", "main.h", "unused.cpp" }, files.ToList());
	}
	
	private PerforceLoadBalancer CreatePlb(GlobalConfig gc, string httpCheckResponse)
	{
#pragma warning disable CA2000 // Dispose objects before losing scope
		HttpClient httpClient = new(new HttpMessageHandlerStub(HttpStatusCode.OK, httpCheckResponse));
		return new(MongoService, GetRedisServiceSingleton(), LeaseCollection, Clock, httpClient, new TestOptionsMonitor<BuildConfig>(gc.Plugins.GetBuildConfig()), new FakeHealthMonitor<PerforceLoadBalancer>(), Tracer, _logger);
#pragma warning restore CA2000 // Dispose objects before losing scope
	}
}

