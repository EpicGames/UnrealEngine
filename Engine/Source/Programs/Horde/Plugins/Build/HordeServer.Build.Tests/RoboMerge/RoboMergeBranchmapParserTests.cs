// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using HordeServer.RoboMerge;

namespace HordeServer.Tests.RoboMerge
{
	[TestClass]
	public class RoboMergeBranchmapParserTests
	{
		[TestMethod]
		public void ExtractBotName_StripsSuffix()
		{
			Assert.AreEqual("mygame", RoboMergeBranchmapParser.ExtractBotName("//MyGame/Main/Build/Streams/mygame.branchmap.json"));
			Assert.AreEqual("ue5", RoboMergeBranchmapParser.ExtractBotName("//UE5/Main/Build/Streams/ue5.branchmap.json"));
			Assert.AreEqual("game_dev", RoboMergeBranchmapParser.ExtractBotName("game_dev.branchmap.json"));
		}

		[TestMethod]
		public void ExtractBotName_NoSuffix_ReturnsFilename()
		{
			Assert.AreEqual("somefile.json", RoboMergeBranchmapParser.ExtractBotName("//depot/somefile.json"));
		}

		[TestMethod]
		public void Parse_BasicFields()
		{
			string json = @"{
				""defaultStreamDepot"": ""MyGame"",
				""isDefaultBot"": true,
				""slackChannel"": ""C123"",
				""reportToBuildHealth"": true,
				""excludeAuthors"": [""buildmachine""],
				""visibility"": [""team-a"", ""team-b""],
				""branches"": [],
				""edges"": []
			}";

			MergeGraph graph = Parse("mygame.branchmap.json", json, 100);

			Assert.AreEqual("mygame", graph.BotName);
			Assert.AreEqual("MyGame", graph.DefaultStreamDepot);
			Assert.IsTrue(graph.IsDefaultBot);
			Assert.AreEqual("C123", graph.SlackChannel);
			Assert.IsTrue(graph.ReportToBuildHealth);
			Assert.AreEqual(1, graph.ExcludeAuthors.Count);
			Assert.AreEqual("buildmachine", graph.ExcludeAuthors[0]);
			Assert.AreEqual(2, graph.Visibility.Count);
			Assert.AreEqual(100, graph.HeadChange);
		}

		[TestMethod]
		public void Parse_AliasSingular()
		{
			string json = @"{
				""defaultStreamDepot"": ""MyGame"",
				""alias"": ""game-bot"",
				""branches"": []
			}";

			MergeGraph graph = Parse("mygame.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Aliases.Count);
			Assert.AreEqual("game-bot", graph.Aliases[0]);
		}

		[TestMethod]
		public void Parse_AliasesArray()
		{
			string json = @"{
				""defaultStreamDepot"": ""UE5"",
				""aliases"": [""ue5-bot"", ""ue""],
				""branches"": []
			}";

			MergeGraph graph = Parse("ue5.branchmap.json", json, 1);

			Assert.AreEqual(2, graph.Aliases.Count);
			Assert.AreEqual("ue5-bot", graph.Aliases[0]);
			Assert.AreEqual("ue", graph.Aliases[1]);
		}

		[TestMethod]
		public void Parse_AliasesArrayTakesPrecedence()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""alias"": ""singular"",
				""aliases"": [""arr1"", ""arr2""],
				""branches"": []
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(2, graph.Aliases.Count);
			Assert.AreEqual("arr1", graph.Aliases[0]);
		}

		[TestMethod]
		public void Parse_BranchFields()
		{
			string json = @"{
				""defaultStreamDepot"": ""UE5"",
				""branches"": [
					{
						""name"": ""UE5-Main"",
						""streamName"": ""Main"",
						""aliases"": [""Main""],
						""flowsTo"": [""Dev""],
						""forceFlowTo"": [""Dev""],
						""disallowDeadend"": true,
						""badgeProject"": ""UE5"",
						""graphNodeColor"": ""orange"",
						""resolver"": ""someone""
					}
				]
			}";

			MergeGraph graph = Parse("ue5.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Branches.Count);
			MergeBranch branch = graph.Branches[0];
			Assert.AreEqual("UE5-Main", branch.Name);
			Assert.AreEqual("Main", branch.StreamName);
			Assert.AreEqual(1, branch.Aliases.Count);
			Assert.AreEqual(1, branch.FlowsTo.Count);
			Assert.AreEqual(1, branch.ForceFlowTo.Count);
			Assert.IsTrue(branch.DisallowDeadend);
			Assert.AreEqual("UE5", branch.BadgeProject);
			Assert.AreEqual("orange", branch.GraphNodeColor);
			Assert.AreEqual("someone", branch.Resolver);
		}

		[TestMethod]
		public void Parse_BranchStreamPath()
		{
			string json = @"{
				""defaultStreamDepot"": ""UE5"",
				""branches"": [
					{ ""name"": ""UE5-Main"", ""streamName"": ""Main"" },
					{ ""name"": ""Dev"" },
					{ ""name"": ""Plugins"", ""streamDepot"": ""Plugins"", ""streamName"": ""Main"" }
				]
			}";

			MergeGraph graph = Parse("ue5.branchmap.json", json, 1);

			Assert.AreEqual("//UE5/Main", graph.Branches[0].GetStreamPath("UE5"));
			Assert.AreEqual("//UE5/Dev", graph.Branches[1].GetStreamPath("UE5"));
			Assert.AreEqual("//Plugins/Main", graph.Branches[2].GetStreamPath("UE5"));
		}

		[TestMethod]
		public void Parse_BranchDisabledAndIncognitoMode()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"", ""disabled"": true },
					{ ""name"": ""B"", ""incognitoMode"": true },
					{ ""name"": ""C"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.IsTrue(graph.Branches[0].Disabled);
			Assert.IsFalse(graph.Branches[0].IncognitoMode);
			Assert.IsFalse(graph.Branches[1].Disabled);
			Assert.IsTrue(graph.Branches[1].IncognitoMode);
			Assert.IsFalse(graph.Branches[2].Disabled);
			Assert.IsFalse(graph.Branches[2].IncognitoMode);
		}

		[TestMethod]
		public void Parse_BranchVisibility_ArrayForm()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"", ""visibility"": [""team-a"", ""team-b""] }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.IsNotNull(graph.Branches[0].Visibility);
			Assert.AreEqual(2, graph.Branches[0].Visibility!.Count);
		}

		[TestMethod]
		public void Parse_BranchVisibility_StringForm()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"", ""visibility"": ""team-a"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.IsNotNull(graph.Branches[0].Visibility);
			Assert.AreEqual(1, graph.Branches[0].Visibility!.Count);
			Assert.AreEqual("team-a", graph.Branches[0].Visibility![0]);
		}

		[TestMethod]
		public void Parse_BranchVisibility_Absent()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.IsNull(graph.Branches[0].Visibility);
		}

		[TestMethod]
		public void Parse_Macros_GraphLevel()
		{
			string json = @"{
				""defaultStreamDepot"": ""MyGame"",
				""macros"": {
					""ToMain"": [""#robomerge[mygame] Main""],
					""GameOnly"": [""#robomerge[ue5] -UE5-Main""]
				},
				""branches"": []
			}";

			MergeGraph graph = Parse("mygame.branchmap.json", json, 1);

			Assert.AreEqual(2, graph.Macros.Count);
			Assert.IsTrue(graph.Macros.ContainsKey("ToMain"));
			Assert.AreEqual(1, graph.Macros["ToMain"].Count);
			Assert.AreEqual("#robomerge[mygame] Main", graph.Macros["ToMain"][0]);
		}

		[TestMethod]
		public void Parse_Macros_BranchLevel()
		{
			string json = @"{
				""defaultStreamDepot"": ""UE5"",
				""branches"": [
					{
						""name"": ""UE5-Main"",
						""macros"": {
							""UE5Only"": [""#robomerge[ue5] -Game-Main""]
						}
					}
				]
			}";

			MergeGraph graph = Parse("ue5.branchmap.json", json, 1);

			Assert.IsNotNull(graph.Branches[0].Macros);
			Assert.AreEqual(1, graph.Branches[0].Macros!.Count);
			Assert.IsTrue(graph.Branches[0].Macros!.ContainsKey("UE5Only"));
		}

		[TestMethod]
		public void Parse_Edges()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"", ""flowsTo"": [""B""] },
					{ ""name"": ""B"" }
				],
				""edges"": [
					{
						""from"": ""A"",
						""to"": ""B"",
						""approval"": { ""description"": ""Please submit via tool"", ""block"": true },
						""integrationMethod"": ""normal"",
						""disallowSkip"": true,
						""resolver"": ""someone"",
						""terminal"": true,
						""implicitCommands"": [""#robomerge -A""],
						""branchspec"": ""ROBO:A-To-B""
					}
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Edges.Count);
			MergeEdge edge = graph.Edges[0];
			Assert.AreEqual("A", edge.From);
			Assert.AreEqual("B", edge.To);
			Assert.IsNotNull(edge.Approval);
			Assert.AreEqual("Please submit via tool", edge.Approval!.Description);
			Assert.IsTrue(edge.Approval.Block);
			Assert.AreEqual("normal", edge.IntegrationMethod);
			Assert.IsTrue(edge.DisallowSkip);
			Assert.AreEqual("someone", edge.Resolver);
			Assert.IsTrue(edge.Terminal);
			Assert.AreEqual(1, edge.ImplicitCommands.Count);
			Assert.AreEqual("ROBO:A-To-B", edge.Branchspec);
		}

		[TestMethod]
		public void Parse_ReservedNamesFiltered()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""A"", ""flowsTo"": [""B"", ""NONE"", ""DEFAULT"", ""DEADEND""] },
					{ ""name"": ""B"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Branches[0].FlowsTo.Count);
			Assert.AreEqual("B", graph.Branches[0].FlowsTo[0]);
		}

		[TestMethod]
		public void Parse_SkipsBranchesWithoutName()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": """" },
					{ ""flowsTo"": [""Main""] },
					{ ""name"": ""Main"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Branches.Count);
			Assert.AreEqual("Main", graph.Branches[0].Name);
		}

		[TestMethod]
		public void Parse_SkipsEdgesWithoutFromOrTo()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [{ ""name"": ""A"" }, { ""name"": ""B"" }],
				""edges"": [
					{ ""from"": ""A"" },
					{ ""to"": ""B"" },
					{ ""from"": ""A"", ""to"": ""B"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Edges.Count);
		}

		[TestMethod]
		public void Parse_UnknownFieldsIgnored()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""someNewField"": 42,
				""anotherNewThing"": { ""nested"": true },
				""branches"": [
					{ ""name"": ""Main"", ""futureProperty"": ""hello"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);

			Assert.AreEqual(1, graph.Branches.Count);
			Assert.AreEqual("Main", graph.Branches[0].Name);
		}

		// --- ComputeMergeChain tests ---

		[TestMethod]
		public void ComputeMergeChain_LinearSpine()
		{
			// Release-1 → Release-2 → Dev → Main
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"", ""flowsTo"": [""Dev""] },
					{ ""name"": ""Dev"", ""flowsTo"": [""Main""], ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release-2"", ""flowsTo"": [""Dev""], ""forceFlowTo"": [""Dev""] },
					{ ""name"": ""Release-1"", ""flowsTo"": [""Release-2""], ""forceFlowTo"": [""Release-2""] }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(4, chain!.Entries.Count);
			Assert.AreEqual("Release-1", chain.Entries[0].Branch.Name);
			Assert.AreEqual("Release-2", chain.Entries[1].Branch.Name);
			Assert.AreEqual("Dev", chain.Entries[2].Branch.Name);
			Assert.AreEqual("Main", chain.Entries[3].Branch.Name);
		}

		[TestMethod]
		public void ComputeMergeChain_PreComputesStreamPath()
		{
			string json = @"{
				""defaultStreamDepot"": ""MyProject"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] }
				]
			}";

			MergeGraph graph = Parse("myproject.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual("//MyProject/Dev", chain!.Entries[0].StreamPath);
			Assert.AreEqual("//MyProject/Main", chain.Entries[1].StreamPath);
		}

		[TestMethod]
		public void ComputeMergeChain_DisabledBranchesExcluded()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release"", ""forceFlowTo"": [""Dev""], ""disabled"": true }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(2, chain!.Entries.Count);
			Assert.AreEqual("Dev", chain.Entries[0].Branch.Name);
			Assert.AreEqual("Main", chain.Entries[1].Branch.Name);
		}

		[TestMethod]
		public void ComputeMergeChain_ApprovalGateDetected()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release"", ""forceFlowTo"": [""Dev""] }
				],
				""edges"": [
					{ ""from"": ""Dev"", ""to"": ""Main"", ""approval"": { ""description"": ""Gate"", ""block"": true } }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(3, chain!.Entries.Count);
			Assert.IsFalse(chain.Entries[0].HasApprovalGateBefore); // Release (first)
			Assert.IsFalse(chain.Entries[1].HasApprovalGateBefore); // Dev (no gate from Release→Dev)
			Assert.IsTrue(chain.Entries[2].HasApprovalGateBefore);  // Main (gate from Dev→Main)
		}

		[TestMethod]
		public void ComputeMergeChain_NoForceFlowTo_ReturnsNull()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"", ""flowsTo"": [""Dev""] },
					{ ""name"": ""Dev"", ""flowsTo"": [""Main""] }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNull(chain);
		}

		[TestMethod]
		public void ComputeMergeChain_SingleBranch_ReturnsNull()
		{
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"" }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNull(chain);
		}

		[TestMethod]
		public void ComputeMergeChain_LongReleaseSpine()
		{
			// Tests a deep release chain: Release-1.0 → ... → Release-4.0 → Dev → Main
			string json = @"{
				""defaultStreamDepot"": ""MyGame"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release-4.0"", ""forceFlowTo"": [""Dev""] },
					{ ""name"": ""Release-3.0"", ""forceFlowTo"": [""Release-4.0""] },
					{ ""name"": ""Release-2.0"", ""forceFlowTo"": [""Release-3.0""] },
					{ ""name"": ""Release-1.5"", ""forceFlowTo"": [""Release-2.0""] },
					{ ""name"": ""Release-1.0"", ""forceFlowTo"": [""Release-1.5""] }
				]
			}";

			MergeGraph graph = Parse("mygame.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(7, chain!.Entries.Count);
			Assert.AreEqual("Release-1.0", chain.Entries[0].Branch.Name);
			Assert.AreEqual("Main", chain.Entries[6].Branch.Name);
		}

		[TestMethod]
		public void ComputeMergeChain_SideBranchToMainDoesNotBreakSpine()
		{
			// Spine: Release-1 → Release-2 → Dev → Main
			// Side branch: Dev-Authoring also has forceFlowTo: ["Main"]
			// The side branch should NOT clobber Dev in the reverse map
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release-2"", ""forceFlowTo"": [""Dev""] },
					{ ""name"": ""Release-1"", ""forceFlowTo"": [""Release-2""] },
					{ ""name"": ""Dev-Authoring"", ""forceFlowTo"": [""Main""] }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(4, chain!.Entries.Count);
			Assert.AreEqual("Release-1", chain.Entries[0].Branch.Name);
			Assert.AreEqual("Release-2", chain.Entries[1].Branch.Name);
			Assert.AreEqual("Dev", chain.Entries[2].Branch.Name);
			Assert.AreEqual("Main", chain.Entries[3].Branch.Name);
		}

		[TestMethod]
		public void ComputeMergeChain_MultipleSideBranches()
		{
			// Multiple side branches targeting Main shouldn't break the spine
			string json = @"{
				""defaultStreamDepot"": ""Test"",
				""branches"": [
					{ ""name"": ""Main"" },
					{ ""name"": ""Dev"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Release"", ""forceFlowTo"": [""Dev""] },
					{ ""name"": ""Side-A"", ""forceFlowTo"": [""Main""] },
					{ ""name"": ""Side-B"", ""forceFlowTo"": [""Main""] }
				]
			}";

			MergeGraph graph = Parse("test.branchmap.json", json, 1);
			MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph);

			Assert.IsNotNull(chain);
			Assert.AreEqual(3, chain!.Entries.Count);
			Assert.AreEqual("Release", chain.Entries[0].Branch.Name);
			Assert.AreEqual("Dev", chain.Entries[1].Branch.Name);
			Assert.AreEqual("Main", chain.Entries[2].Branch.Name);
		}

		static MergeGraph Parse(string depotPath, string json, int headChange)
		{
			byte[] data = Encoding.UTF8.GetBytes(json);
			return RoboMergeBranchmapParser.Parse(depotPath, data, headChange);
		}
	}
}
