// Copyright Epic Games, Inc. All Rights Reserved.

using System.IdentityModel.Tokens.Jwt;
using System.IO.Compression;
using System.Security.Claims;
using System.Text;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Tools;
using Grpc.Core;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Tools;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Server
{
	[TestClass]
	public class RpcServiceTest : BuildTestSetup
	{
		// Defined as consts so they can be used in DataRows in tests
		public const string DedicatedAgentTool = "agent-dedicated-tool";
		public const string DedicatedAgentRole = "agent";
		public const string DedicatedAgentRegistrationRole = "agent-registration";
		public const string WorkstationAgentTool = "agent-workstation-tool";
		public const string WorkstationAgentRole = "agent-workstation";
		public const string AdminRole = "admin";
		
		private readonly ServerCallContext _adminContext = new ServerCallContextStub(HordeClaims.AdminClaim.ToClaim());
		private readonly ToolId _dedicatedAgentToolId = new (DedicatedAgentTool);
		private readonly ToolId _workstationAgentToolId = new (WorkstationAgentTool);

		[TestMethod]
		public async Task CreateSessionTestAsync()
		{
			RpcCreateSessionRequest req = new RpcCreateSessionRequest();
			await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => RpcService.CreateSession(req, _adminContext));

			req.Id = new AgentId("MyName").ToString();
			await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => RpcService.CreateSession(req, _adminContext));

			req.Capabilities = new RpcAgentCapabilities();
			RpcCreateSessionResponse res = await RpcService.CreateSession(req, _adminContext);

			Assert.AreEqual("MYNAME", res.AgentId);
			// TODO: Check Token, ExpiryTime, SessionId 
		}
		
		[TestMethod]
		[DataRow(Mode.Dedicated, AdminRole, true)]
		[DataRow(Mode.Dedicated, DedicatedAgentRegistrationRole, true)]
		[DataRow(Mode.Workstation, AdminRole, true)]
		[DataRow(Mode.Workstation, WorkstationAgentRole, true)]
		[DataRow(Mode.Dedicated, DedicatedAgentRole, false)] // Dedicated agent must use agent registration claim
		[DataRow(Mode.Dedicated, WorkstationAgentRole, false)] // Workstation agent cannot create a dedicated agent
		[DataRow(Mode.Workstation, DedicatedAgentRole, false)]
		[DataRow(Mode.Workstation, DedicatedAgentRegistrationRole, false)]
		public async Task CreateAgent_Async(Mode agentMode, string? roleClaim, bool expectSuccess)
		{
			ServerCallContextStub context = CreateServerContext(roleClaim);
			RpcCreateAgentRequest req = new () { Name = "myName", Mode = agentMode };
			if (expectSuccess)
			{
				RpcCreateAgentResponse res = await RpcService.CreateAgent(req, context);
				switch (agentMode)
				{
					case Mode.Dedicated:
						{
							JwtSecurityToken jwtToken = new JwtSecurityTokenHandler().ReadJwtToken(res.Token);
							List<Claim> claims = jwtToken.Claims.ToList();
							Assert.AreEqual("HS256", jwtToken.SignatureAlgorithm);
							Assert.AreEqual(2, claims.Count);
							Assert.AreEqual("http://epicgames.com/ue/horde/agent", claims[0].Type);
							Assert.AreEqual("MYNAME", claims[0].Value);
							break;
						}
					case Mode.Workstation: Assert.AreEqual("", res.Token); break;
					case Mode.Unspecified: default: throw new ArgumentOutOfRangeException(nameof(agentMode), agentMode, null);
				}
			}
			else
			{
				StructuredRpcException ex = await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => RpcService.CreateAgent(req, context));
				Assert.AreEqual(StatusCode.PermissionDenied, ex.StatusCode);
			}
		}
		
		[TestMethod]
		[DataRow(DedicatedAgentTool, "999", DedicatedAgentRole, StatusCode.NotFound)] // Invalid version
		[DataRow("not-valid", "1", DedicatedAgentRole, StatusCode.NotFound)] // Invalid tool ID
		[DataRow(DedicatedAgentTool, "1", DedicatedAgentRole, StatusCode.OK)] // All valid
		[DataRow(DedicatedAgentTool, "1", WorkstationAgentRole, StatusCode.PermissionDenied)] // Workstation role cannot access dedicated agent
		[DataRow(DedicatedAgentTool, "1", null, StatusCode.PermissionDenied)] // Unauthed user cannot access dedicated agent
		[DataRow(WorkstationAgentTool, "1", null, StatusCode.OK)] // Unauthed user can access public workstation agent
		public async Task DownloadSoftware_Async(string toolId, string version, string? roleClaim, StatusCode expectedStatusCode)
		{
			await SetupToolsAsync(CancellationToken.None);
			ServerCallContextStub context = CreateServerContext(roleClaim);
			TestServerStreamWriter<RpcDownloadSoftwareResponse> responseStream = new (context);
			RpcDownloadSoftwareRequest req = new () { Version = $"{toolId}:{version}" };
			if (expectedStatusCode == StatusCode.OK)
			{
				await RpcService.DownloadSoftware(req, responseStream, context);
			}
			else
			{
				StructuredRpcException ex = await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => RpcService.DownloadSoftware(req, responseStream, context));
				Assert.AreEqual(expectedStatusCode, ex.StatusCode);
			}
		}

		[TestMethod]
		public async Task AgentJoinsPoolThroughPropertiesAsync()
		{
			RpcCreateSessionRequest req = new() { Id = new AgentId("bogusAgentName").ToString(), Capabilities = new RpcAgentCapabilities() };
			req.Capabilities.Properties.Add($"{KnownPropertyNames.RequestedPools}=fooPool,barPool");
			RpcCreateSessionResponse res = await RpcService.CreateSession(req, _adminContext);

			IAgent agent = (await AgentService.GetAgentAsync(new AgentId(res.AgentId)))!;
			CollectionAssert.AreEquivalent(new List<PoolId> { new("fooPool"), new("barPool") }, agent.Pools.ToList());

			// Connect a second time, when the agent has already been created
			req = new() { Id = new AgentId("bogusAgentName").ToString(), Capabilities = new RpcAgentCapabilities() };
			req.Capabilities.Properties.Add($"{KnownPropertyNames.RequestedPools}=bazPool");
			res = await RpcService.CreateSession(req, _adminContext);

			agent = (await AgentService.GetAgentAsync(new AgentId(res.AgentId)))!;
			CollectionAssert.AreEquivalent(new List<PoolId> { new("bazPool") }, agent.Pools.ToList());
		}

		[TestMethod]
		public async Task PropertiesFromAgentCapabilitiesAsync()
		{
			RpcCreateSessionRequest req = new() { Id = new AgentId("bogusAgentName").ToString(), Capabilities = new RpcAgentCapabilities() };
			req.Capabilities.Properties.Add("fooKey=barValue");
			RpcCreateSessionResponse res = await RpcService.CreateSession(req, _adminContext);
			IAgent agent = (await AgentService.GetAgentAsync(new AgentId(res.AgentId)))!;
			Assert.IsTrue(agent.Properties.Contains("fooKey=barValue"));
		}

#pragma warning disable CA1041
#pragma warning disable CS0612
		[TestMethod]
		public async Task PropertiesFromDeviceCapabilitiesAsync()
		{
			RpcCreateSessionRequest req = new() { Id = new AgentId("bogusAgentName").ToString(), Capabilities = new RpcAgentCapabilities() };
			req.Capabilities.Devices.Add(new RpcDeviceCapabilities { Handle = "someHandle", Properties = { "foo=bar" } });
			RpcCreateSessionResponse res = await RpcService.CreateSession(req, _adminContext);
			IAgent agent = (await AgentService.GetAgentAsync(new AgentId(res.AgentId)))!;
			Assert.IsTrue(agent.Properties.Contains("foo=bar"));
		}

		[TestMethod]
		public async Task KnownPropertiesAreSetAsResourcesAsync()
		{
			RpcCreateSessionRequest req = new() { Id = new AgentId("bogusAgentName").ToString(), Capabilities = new RpcAgentCapabilities() };
			req.Capabilities.Devices.Add(new RpcDeviceCapabilities { Handle = "someHandle", Properties = { $"{KnownResourceNames.LogicalCores}=10" } });
			RpcCreateSessionResponse res = await RpcService.CreateSession(req, _adminContext);
			IAgent agent = (await AgentService.GetAgentAsync(new AgentId(res.AgentId)))!;
			Assert.AreEqual(10, agent.Resources[KnownResourceNames.LogicalCores]);
		}
#pragma warning restore CS0612
#pragma warning restore CA1041

		[TestMethod]
		public async Task UpdateSessionTestAsync()
		{
			RpcCreateSessionRequest createReq = new RpcCreateSessionRequest
			{
				Id = new AgentId("UpdateSessionTest1").ToString(),
				Capabilities = new RpcAgentCapabilities()
			};
			RpcCreateSessionResponse createRes = await RpcService.CreateSession(createReq, _adminContext);
			string agentId = createRes.AgentId;
			string sessionId = createRes.SessionId;

			TestAsyncStreamReader<RpcUpdateSessionRequest> requestStream = new (_adminContext);
			TestServerStreamWriter<RpcUpdateSessionResponse> responseStream = new (_adminContext);
			Task call = RpcService.UpdateSession(requestStream, responseStream, _adminContext);

			requestStream.AddMessage(new RpcUpdateSessionRequest { AgentId = "does-not-exist", SessionId = sessionId });
			StructuredRpcException re = await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => call);
			Assert.AreEqual(StatusCode.NotFound, re.StatusCode);
			Assert.IsTrue(re.Message.Contains("Invalid agent name", StringComparison.OrdinalIgnoreCase));
		}

		[TestMethod]
		public async Task SessionMismatchWithLowCountThrowsStructuredRpcExceptionAsync()
		{
			// Create first session
			RpcCreateSessionRequest createReq1 = new() { Id = new AgentId("MismatchAgent").ToString(), Capabilities = new RpcAgentCapabilities() };
			RpcCreateSessionResponse createRes1 = await RpcService.CreateSession(createReq1, _adminContext);
			string staleSessionId = createRes1.SessionId;

			// Create a second session (invalidates first)
			RpcCreateSessionRequest createReq2 = new() { Id = new AgentId("MismatchAgent").ToString(), Capabilities = new RpcAgentCapabilities() };
			await RpcService.CreateSession(createReq2, _adminContext);

			// Try to update using stale session → should throw StructuredRpcException (count ≤ threshold)
			ServerCallContext sessionContext = ServerCallContextStub.ForAdminWithAgentSessionId(staleSessionId);
			TestAsyncStreamReader<RpcUpdateSessionRequest> requestStream = new(sessionContext);
			TestServerStreamWriter<RpcUpdateSessionResponse> responseStream = new(sessionContext);
			Task call = RpcService.UpdateSession(requestStream, responseStream, sessionContext);

			requestStream.AddMessage(new RpcUpdateSessionRequest { AgentId = createRes1.AgentId, SessionId = staleSessionId });
			StructuredRpcException ex = await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => call);
			Assert.AreEqual(StatusCode.PermissionDenied, ex.StatusCode);
			Assert.IsTrue(ex.Message.Contains("has completed session", StringComparison.OrdinalIgnoreCase));
		}

		[TestMethod]
		public async Task SessionMismatchWithHighCountThrowsPlainRpcExceptionAsync()
		{
			// Create first session
			RpcCreateSessionRequest createReq1 = new() { Id = new AgentId("MismatchAgent2").ToString(), Capabilities = new RpcAgentCapabilities() };
			RpcCreateSessionResponse createRes1 = await RpcService.CreateSession(createReq1, _adminContext);
			string staleSessionId = createRes1.SessionId;
			AgentId agentId = new(createRes1.AgentId);

			// Create a second session (invalidates first)
			RpcCreateSessionRequest createReq2 = new() { Id = new AgentId("MismatchAgent2").ToString(), Capabilities = new RpcAgentCapabilities() };
			await RpcService.CreateSession(createReq2, _adminContext);

			// Pre-populate conflict count up to MaxDetailedConflictErrors via SessionConflictService
			SessionConflictService conflictService = ServiceProvider.GetRequiredService<SessionConflictService>();
			for (int i = 0; i < HordeServer.Server.RpcService.MaxDetailedConflictErrors; i++)
			{
				await conflictService.RecordConflictAsync(agentId);
			}

			// Try to update using stale session → should throw plain RpcException (count > threshold)
			ServerCallContext sessionContext = ServerCallContextStub.ForAdminWithAgentSessionId(staleSessionId);
			TestAsyncStreamReader<RpcUpdateSessionRequest> requestStream = new(sessionContext);
			TestServerStreamWriter<RpcUpdateSessionResponse> responseStream = new(sessionContext);
			Task call = RpcService.UpdateSession(requestStream, responseStream, sessionContext);

			requestStream.AddMessage(new RpcUpdateSessionRequest { AgentId = createRes1.AgentId, SessionId = staleSessionId });
			RpcException ex = await Assert.ThrowsExactlyAsync<RpcException>(() => call);
			Assert.AreEqual(StatusCode.PermissionDenied, ex.StatusCode);

			// Should NOT be a StructuredRpcException
			Assert.IsFalse(ex is StructuredRpcException);
		}

		[TestMethod]
		public async Task FinishBatchTestAsync()
		{
			RpcCreateSessionRequest createReq = new RpcCreateSessionRequest
			{
				Id = new AgentId("UpdateSessionTest1").ToString(),
				Capabilities = new RpcAgentCapabilities()
			};
			RpcCreateSessionResponse createRes = await RpcService.CreateSession(createReq, _adminContext);
			string agentId = createRes.AgentId;
			string sessionId = createRes.SessionId;

			TestAsyncStreamReader<RpcUpdateSessionRequest> requestStream = new (_adminContext);
			TestServerStreamWriter<RpcUpdateSessionResponse> responseStream =new (_adminContext);
			Task call = RpcService.UpdateSession(requestStream, responseStream, _adminContext);

			requestStream.AddMessage(new RpcUpdateSessionRequest { AgentId = "does-not-exist", SessionId = sessionId });
			StructuredRpcException re = await Assert.ThrowsExactlyAsync<StructuredRpcException>(() => call);
			Assert.AreEqual(StatusCode.NotFound, re.StatusCode);
			Assert.IsTrue(re.Message.Contains("Invalid agent name", StringComparison.OrdinalIgnoreCase));
		}
		
		private async Task SetupToolsAsync(CancellationToken cancellationToken)
		{
			AddPlugin<ToolsPlugin>();
			AclEntryConfig aclEntryConfig = new(HordeClaims.AgentDedicatedRoleClaim, [ToolAclAction.DownloadTool]);
			ToolsConfig toolsConfig = new ();
			toolsConfig.Tools.Add(new ToolConfig(_dedicatedAgentToolId) { Acl = new AclConfig() { Entries = [aclEntryConfig] } });
			toolsConfig.Tools.Add(new ToolConfig(_workstationAgentToolId) { Public = true });
			await UpdateConfigAsync(gc => gc.Plugins.AddToolsConfig(toolsConfig) );
			
			IToolCollection collection = ServiceProvider.GetRequiredService<IToolCollection>();
			await CreateAgentToolDeploymentAsync(collection, _dedicatedAgentToolId, "1", cancellationToken);
			await CreateAgentToolDeploymentAsync(collection, _dedicatedAgentToolId, "2", cancellationToken);
			await CreateAgentToolDeploymentAsync(collection, _workstationAgentToolId, "1", cancellationToken);
		}
		
		private static async Task<ITool?> CreateAgentToolDeploymentAsync(IToolCollection collection, ToolId toolId, string version, CancellationToken cancellationToken)
		{
			ITool? tool = await collection.GetAsync(toolId, cancellationToken);
			Assert.IsNotNull(tool);
			ToolDeploymentConfig tdc = new () { Version = version, Duration = TimeSpan.FromMinutes(5.0), CreatePaused = true };
			
			const string FileName = "version.txt";
			byte[] fileData = Encoding.UTF8.GetBytes(version);
			{
				using MemoryStream stream = new();
				using (ZipArchive archive = new (stream, ZipArchiveMode.Create, leaveOpen: true))
				{
					ZipArchiveEntry entry = archive.CreateEntry(FileName);
					await using (Stream entryStream = await entry.OpenAsync(cancellationToken))
					{
						await entryStream.WriteAsync(fileData, cancellationToken);
					}
				}
				stream.Position = 0;
				return await tool.CreateDeploymentAsync(tdc, stream, cancellationToken);
			}
		}
		
		private static ServerCallContextStub CreateServerContext(string? roleClaim)
		{
			return new ServerCallContextStub(roleClaim != null ? new AclClaimConfig(HordeClaimTypes.Role, roleClaim).ToClaim() : null);
		}
	}
}