// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Streams;

namespace HordeServer.Tests.Streams
{
	[TestClass]
	public class RetentionOptionsTests
	{
		[TestMethod]
		public void MergeDefaults_FillsNullFromParent()
		{
			RetentionOptions parent = new RetentionOptions
			{
				EnableJobExpiration = false,
				EnableArtifactExpiration = false,
				MaxJobDeletionsPerTick = 500
			};

			RetentionOptions child = new RetentionOptions(); // all null
			child.MergeDefaults(parent);

			Assert.AreEqual(false, child.EnableJobExpiration);
			Assert.AreEqual(false, child.EnableArtifactExpiration);
			Assert.AreEqual(500, child.MaxJobDeletionsPerTick);
		}

		[TestMethod]
		public void MergeDefaults_ChildOverridesParent()
		{
			RetentionOptions parent = new RetentionOptions
			{
				EnableJobExpiration = false,
				MaxJobDeletionsPerTick = 500
			};

			RetentionOptions child = new RetentionOptions
			{
				EnableJobExpiration = true, // explicit override
				MaxJobDeletionsPerTick = 100
			};

			child.MergeDefaults(parent);

			// Child values must win
			Assert.AreEqual(true, child.EnableJobExpiration);
			Assert.AreEqual(100, child.MaxJobDeletionsPerTick);
			// Parent fills the gap
			Assert.IsNull(child.EnableArtifactExpiration);
		}

		[TestMethod]
		public void MergeDefaults_NullParentLeavesChildNull()
		{
			RetentionOptions parent = new RetentionOptions(); // all null
			RetentionOptions child = new RetentionOptions();  // all null
			child.MergeDefaults(parent);

			Assert.IsNull(child.EnableJobExpiration);
			Assert.IsNull(child.EnableArtifactExpiration);
			Assert.IsNull(child.MaxJobDeletionsPerTick);
		}
	}
}
