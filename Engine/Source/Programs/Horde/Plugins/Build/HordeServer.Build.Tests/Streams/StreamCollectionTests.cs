// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Primitives;

namespace HordeServer.Tests.Streams
{
	[TestClass]
	public class StreamCollectionTests : BuildTestSetup
	{
		private readonly StreamId _streamId = new("bogusStreamId");

		[TestMethod]
		public void ValidateUndefinedTemplateIdInTabs()
		{
			StreamConfig config = new()
			{
				Tabs = new()
				{
					new TabConfig { Templates = new List<TemplateId> { new ("foo") }},
					new TabConfig { Templates = new List<TemplateId> { new ("bar") }}
				},
				Templates = new() { new TemplateRefConfig { Id = new TemplateId("foo") } }
			};

			Assert.ThrowsExactly<InvalidStreamException>(() => HordeServer.Streams.StreamCollection.Validate(_streamId, config));

			config.Templates.Add(new TemplateRefConfig { Id = new TemplateId("bar") });
			HordeServer.Streams.StreamCollection.Validate(_streamId, config);
		}

		[TestMethod]
		public async Task ValidateGetChangesAsync()
		{
			Fixture fixture = await CreateFixtureAsync();
			IUser jerry = await UserCollection.FindOrAddUserByLoginAsync("Jerry");

			int firstChangelist = 112233;
			int secondChangelist = 112234;
			int thirdChangelist = 112235;
			int invalidChangelist = 112236;

			PerforceService.AddChange(fixture.StreamId, firstChangelist, jerry, "Edited file1.cpp.", new[] { "file1.cpp" });
			PerforceService.AddChange(fixture.StreamId, secondChangelist, jerry, "Added file2.cpp.", new[] { "file2.cpp" });
			PerforceService.AddChange(fixture.StreamId, thirdChangelist, jerry, "Edited file1.cpp.", new[] { "file1.cpp" });

			CommitId firstCommitId = new CommitId(firstChangelist.ToString());
			CommitId secondCommitId = new CommitId(secondChangelist.ToString());
			CommitId thirdCommitId = new CommitId(thirdChangelist.ToString());
			CommitId invalidCommitId = new CommitId(invalidChangelist.ToString());

			// Single element edge case
			{
				ActionResult<List<object>> results = await StreamsController.GetChangesAsync(fixture.StreamId, null, null, [firstCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 1);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
			}

			// Multi element nominal case
			{
				ActionResult<List<object>> results = await StreamsController.GetChangesAsync(fixture.StreamId, null, null, [firstCommitId, secondCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 2);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == secondChangelist.ToString()));
			}

			// Partial success case
			{
				StreamsController httpContextPreservedStreamsController = StreamsController;
				ActionResult<List<object>> results = await httpContextPreservedStreamsController.GetChangesAsync(fixture.StreamId, null, null, [firstCommitId, invalidCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 1);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
				Assert.IsNull(castedValue.SingleOrDefault(c => c.Id.Name == invalidChangelist.ToString()));
				Assert.IsTrue(httpContextPreservedStreamsController.Response.Headers.ContainsKey(StreamsController.MissingCommitsHeader));

				// Assert contents
				StringValues stringValues = httpContextPreservedStreamsController.Response.Headers[StreamsController.MissingCommitsHeader];
				Assert.IsTrue(stringValues.Contains(invalidChangelist.ToString()));
			}

			// Empty edge case
			{
				ActionResult<List<object>> results = await StreamsController.GetChangesAsync(fixture.StreamId, null, null, []);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 0);
			}

			// Bad request test case when providing too many arugments.
			{
				ActionResult<List<object>> results = await StreamsController.GetChangesAsync(fixture.StreamId, firstCommitId, secondCommitId, [thirdCommitId]);
				BadRequestObjectResult? result = results.Result as BadRequestObjectResult;

				Assert.IsNotNull(result);
				Assert.AreEqual(StreamsController.BadRequest().StatusCode, result.StatusCode);
			}
		}

		[TestMethod]
		public async Task ValidateGetBatchedChangesAsync()
		{
			Fixture fixture = await CreateFixtureAsync();
			IUser jerry = await UserCollection.FindOrAddUserByLoginAsync("Jerry");

			int firstChangelist = 112233;
			int secondChangelist = 112234;
			int thirdChangelist = 112235;
			int invalidChangelist = 112236;

			PerforceService.AddChange(fixture.StreamId, firstChangelist, jerry, "Edited file1.cpp.", new[] { "file1.cpp" });
			PerforceService.AddChange(fixture.StreamId, secondChangelist, jerry, "Added file2.cpp.", new[] { "file2.cpp" });
			PerforceService.AddChange(fixture.StreamId, thirdChangelist, jerry, "Edited file1.cpp.", new[] { "file1.cpp" });

			CommitId firstCommitId = new CommitId(firstChangelist.ToString());
			CommitId secondCommitId = new CommitId(secondChangelist.ToString());
			CommitId thirdCommitId = new CommitId(thirdChangelist.ToString());
			CommitId invalidCommitId = new CommitId(invalidChangelist.ToString());

			// Single element edge case
			{
				ActionResult<List<object>> results = await StreamsController.GetBatchedChangesAsync(fixture.StreamId, [firstCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 1);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
			}

			// Multi element nominal case
			{
				ActionResult<List<object>> results = await StreamsController.GetBatchedChangesAsync(fixture.StreamId, [firstCommitId, secondCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 2);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == secondChangelist.ToString()));
			}

			// Partial success case
			{
				StreamsController httpContextPreservedStreamsController = StreamsController;
				ActionResult<List<object>> results = await httpContextPreservedStreamsController.GetBatchedChangesAsync(fixture.StreamId,[firstCommitId, invalidCommitId]);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 1);
				Assert.IsNotNull(castedValue.SingleOrDefault(c => c.Id.Name == firstChangelist.ToString()));
				Assert.IsNull(castedValue.SingleOrDefault(c => c.Id.Name == invalidChangelist.ToString()));
				Assert.IsTrue(httpContextPreservedStreamsController.Response.Headers.ContainsKey(StreamsController.MissingCommitsHeader));

				// Assert contents
				StringValues stringValues = httpContextPreservedStreamsController.Response.Headers[StreamsController.MissingCommitsHeader];
				Assert.IsTrue(stringValues.Contains(invalidChangelist.ToString()));
			}

			// Empty edge case
			{
				ActionResult<List<object>> results = await StreamsController.GetBatchedChangesAsync(fixture.StreamId, []);

				Assert.IsNotNull(results.Value);

				List<GetCommitResponse> castedValue = [.. results.Value.Cast<GetCommitResponse>()];
				Assert.IsTrue(castedValue.Count == 0);
			}
		}
	}
}
