// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Streams;
using HordeServer.Artifacts;
using HordeServer.Projects;
using HordeServer.Streams;

namespace HordeServer.Tests.Streams
{
	[TestClass]
	public class StreamConfigIgnorePatternTests
	{
		[TestMethod]
		public void TestIgnorePatternFiles_BothProjectAndStream()
		{
			ProjectConfig project = new ProjectConfig();
			project.IgnorePatternFiles = new List<string> { "project-patterns.txt", "shared-patterns.txt" };

			StreamConfig stream = new StreamConfig();
			stream.IgnorePatternFiles = new List<string> { "stream-patterns.txt" };

			stream.PostLoad(new StreamId("test-stream"), project, Array.Empty<ArtifactTypeConfig>(), null);

			Assert.IsNotNull(stream.IgnorePatternFiles);
			Assert.AreEqual(3, stream.IgnorePatternFiles.Count);
			// Project-level files should come first
			Assert.AreEqual("project-patterns.txt", stream.IgnorePatternFiles[0]);
			Assert.AreEqual("shared-patterns.txt", stream.IgnorePatternFiles[1]);
			// Stream-level files appended after
			Assert.AreEqual("stream-patterns.txt", stream.IgnorePatternFiles[2]);
		}

		[TestMethod]
		public void TestIgnorePatternFiles_ProjectOnly()
		{
			ProjectConfig project = new ProjectConfig();
			project.IgnorePatternFiles = new List<string> { "project-patterns.txt" };

			StreamConfig stream = new StreamConfig();
			// stream.IgnorePatternFiles is null by default

			stream.PostLoad(new StreamId("test-stream"), project, Array.Empty<ArtifactTypeConfig>(), null);

			Assert.IsNotNull(stream.IgnorePatternFiles);
			Assert.AreEqual(1, stream.IgnorePatternFiles.Count);
			Assert.AreEqual("project-patterns.txt", stream.IgnorePatternFiles[0]);
		}

		[TestMethod]
		public void TestIgnorePatternFiles_StreamOnly()
		{
			ProjectConfig project = new ProjectConfig();
			// project.IgnorePatternFiles is null by default

			StreamConfig stream = new StreamConfig();
			stream.IgnorePatternFiles = new List<string> { "stream-patterns.txt" };

			stream.PostLoad(new StreamId("test-stream"), project, Array.Empty<ArtifactTypeConfig>(), null);

			Assert.IsNotNull(stream.IgnorePatternFiles);
			Assert.AreEqual(1, stream.IgnorePatternFiles.Count);
			Assert.AreEqual("stream-patterns.txt", stream.IgnorePatternFiles[0]);
		}

		[TestMethod]
		public void TestIgnorePatternFiles_NeitherDefined()
		{
			ProjectConfig project = new ProjectConfig();
			StreamConfig stream = new StreamConfig();

			stream.PostLoad(new StreamId("test-stream"), project, Array.Empty<ArtifactTypeConfig>(), null);

			Assert.IsNull(stream.IgnorePatternFiles);
		}

		[TestMethod]
		public void TestIgnorePatternFiles_EmptyProjectList()
		{
			ProjectConfig project = new ProjectConfig();
			project.IgnorePatternFiles = new List<string>();

			StreamConfig stream = new StreamConfig();
			stream.IgnorePatternFiles = new List<string> { "stream-patterns.txt" };

			stream.PostLoad(new StreamId("test-stream"), project, Array.Empty<ArtifactTypeConfig>(), null);

			// Empty project list (Count == 0) should not affect stream's list
			Assert.IsNotNull(stream.IgnorePatternFiles);
			Assert.AreEqual(1, stream.IgnorePatternFiles.Count);
			Assert.AreEqual("stream-patterns.txt", stream.IgnorePatternFiles[0]);
		}
	}
}
