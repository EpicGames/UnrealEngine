// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using EpicGames.Horde.Streams;

namespace HordeServer.Tests.Streams
{
	[TestClass]
	public class StreamServiceTests : BuildTestSetup
	{
		[TestMethod]
		public async Task PausingAsync()
		{
			Fixture fixture = await CreateFixtureAsync();

			IStream stream = (await StreamCollection.GetAsync(fixture!.StreamConfig!.Id))!;
			Assert.IsFalse(stream.IsPaused(DateTime.UtcNow));
			Assert.IsNull(stream.PausedUntil);
			Assert.IsNull(stream.PauseComment);

			DateTime pausedUntil = DateTime.UtcNow.AddHours(1);
			stream = (await stream.TryUpdatePauseStateAsync(newPausedUntil: pausedUntil, newPauseComment: "mycomment"))!;
			// Comparing by string to avoid comparing exact milliseconds as those are not persisted in MongoDB fields
			Assert.IsTrue(stream.IsPaused(DateTime.UtcNow));
			Assert.AreEqual(pausedUntil.ToString(CultureInfo.InvariantCulture), stream.PausedUntil!.Value.ToString(CultureInfo.InvariantCulture));
			Assert.AreEqual("mycomment", stream.PauseComment);

			stream = (await stream.TryUpdatePauseStateAsync(newPausedUntil: null, newPauseComment: null))!;
			Assert.IsFalse(stream.IsPaused(DateTime.UtcNow));
			Assert.IsNull(stream.PausedUntil);
			Assert.IsNull(stream.PauseComment);
		}
	}
}
