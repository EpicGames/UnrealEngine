// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Secrets;
using EpicGames.Horde.Streams;
using HordeServer.Projects;
using HordeServer.Secrets;
using HordeServer.Server;
using HordeServer.Streams;

namespace HordeServer.Tests.Streams;

[TestClass]
public class TokenConfigSecretsTests : BuildTestSetup
{
	public TokenConfigSecretsTests()
	{
		AddPlugin<SecretsPlugin>();
	}

	[TestInitialize]
	public async Task SetupAsync()
	{
		GlobalConfig globalConfig = new() { ResolveSecrets = true };
		globalConfig.Plugins.AddBuildConfig(new BuildConfig());
		globalConfig.Plugins.AddSecretsConfig(new SecretsConfig());
		await SetConfigAsync(globalConfig);
	}

	private async Task AddTokenConfigAsync(TokenConfig tokenConfig, SecretConfig? secret)
	{
		await UpdateConfigAsync(globalConfig =>
		{
			if (secret != null)
			{
				globalConfig.Plugins.GetSecretsConfig().Secrets.Add(secret);
			}
			globalConfig.Plugins.GetBuildConfig().Projects.Add(new ProjectConfig
			{
				Streams =
				[
					new StreamConfig
					{
						Id = new StreamId("test-stream"),
						Tokens = [tokenConfig]
					}
				]
			});
		});
	}

	[TestMethod]
	public async Task ResolveTokenClientSecretWithSecretAsync()
	{
		TokenConfig tokenConfig = new()
		{
			Url = new Uri("https://example.com/token"),
			ClientId = "horde:secret:tokensecret.ClientId",
			ClientSecret = "horde:secret:tokensecret.ClientSecret",
			EnvVar = "TOKEN_ENV"
		};
		SecretConfig secret = new()
		{
			Id = new SecretId("tokensecret"),
			Data = new Dictionary<string, string> { { "ClientId", "ResolvedClientId" }, { "ClientSecret", "SuperSecretValue" } }
		};
		await AddTokenConfigAsync(tokenConfig, secret);
		Assert.AreEqual("ResolvedClientId", tokenConfig.ClientId);
		Assert.AreEqual("SuperSecretValue", tokenConfig.ClientSecret);
	}

	[TestMethod]
	public async Task ResolveTokenClientSecretWithoutSecretAsync()
	{
		TokenConfig tokenConfig = new()
		{
			Url = new Uri("https://example.com/token"),
			ClientId = "my-client",
			ClientSecret = "plain-text-secret",
			EnvVar = "TOKEN_ENV"
		};
		await AddTokenConfigAsync(tokenConfig, null);
		Assert.AreEqual("plain-text-secret", tokenConfig.ClientSecret);
		Assert.AreEqual("my-client", tokenConfig.ClientId);
	}

	[TestMethod]
	[DataRow("horde:secret:notfound.property", DisplayName = "Resolve Token Secret, secret not found")]
	[DataRow("horde:secret:tokensecret.notfound", DisplayName = "Resolve Token Secret, property not found")]
	public async Task ResolveTokenClientSecretNotFoundAsync(string value)
	{
		TokenConfig tokenConfig = new()
		{
			Url = new Uri("https://example.com/token"),
			ClientId = "my-client",
			ClientSecret = value,
			EnvVar = "TOKEN_ENV"
		};
		SecretConfig secret = new() { Id = new SecretId("tokensecret") };
		await Assert.ThrowsExactlyAsync<KeyNotFoundException>(async () =>
		{
			try
			{
				await AddTokenConfigAsync(tokenConfig, secret);
			}
			catch (AggregateException ae)
			{
				throw ae.InnerExceptions[0];
			}
		});
	}
}
