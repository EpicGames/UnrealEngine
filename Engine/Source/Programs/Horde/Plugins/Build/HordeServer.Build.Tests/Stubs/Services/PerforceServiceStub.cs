// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.VersionControl.Perforce;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using EpicGames.Perforce;

namespace HordeServer.Tests.Stubs.Services
{
	public class PerforceServiceStub : IPerforceService
	{
		public string Name { get; } = "Perforce";
		
		class User : IUser
		{
			public UserId Id { get; }
			public string Name { get; set; }
			public string Email { get; set; }
			public string Login { get; set; }

			public User(string login)
			{
				Id = new UserId(BinaryIdUtils.CreateNew());
				Name = login.ToUpperInvariant();
				Email = $"{login}@server";
				Login = login;
			}
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Design", "CA1034")]
		public sealed class Commit : ICommit
		{
			public StreamId StreamId { get; }

			public int Number { get; }
			CommitIdWithOrder ICommit.Id => CommitIdWithOrder.FromPerforceChange(Number);

			public int OriginalChange { get; }
			CommitIdWithOrder ICommit.OriginalCommitId => CommitIdWithOrder.FromPerforceChange(OriginalChange);

			public UserId AuthorId { get; }
			public UserId OwnerId { get; }
			public string Description { get; }
			public string BasePath => throw new NotImplementedException();
			public DateTime DateUtc => DateTime.UtcNow;
			public List<string> Files { get; }

			public Commit(StreamId streamId, int number, int originalChange, UserId authorId, UserId ownerId, string description, List<string> files)
			{
				StreamId = streamId;
				Number = number;
				OriginalChange = originalChange;
				AuthorId = authorId;
				OwnerId = ownerId;
				Description = description;
				Files = files;
			}

			public ValueTask<IReadOnlyList<CommitTag>> GetTagsAsync(CancellationToken cancellationToken)
			{
				StreamConfig config = new StreamConfig();
				Assert.IsTrue(config.TryGetCommitTagFilter(CommitTag.Code, out FileFilter? codeFilter));
				Assert.IsTrue(config.TryGetCommitTagFilter(CommitTag.Content, out FileFilter? contentFilter));

				List<CommitTag> tags = new List<CommitTag>();
				if (codeFilter.ApplyTo(Files).Any())
				{
					tags.Add(CommitTag.Code);
				}
				if (contentFilter.ApplyTo(Files).Any())
				{
					tags.Add(CommitTag.Content);
				}
				return new ValueTask<IReadOnlyList<CommitTag>>(tags);
			}

			public ValueTask<bool> MatchesFilterAsync(FileFilter filter, CancellationToken cancellationToken)
			{
				return new ValueTask<bool>(filter.ApplyTo(Files).Any());
			}

			public ValueTask<IReadOnlyList<string>> GetFilesAsync(int? minFiles, int? maxFiles, CancellationToken cancellationToken)
			{
				return new ValueTask<IReadOnlyList<string>>(Files);
			}
		}

		class ChangeComparer : IComparer<int>
		{
			public int Compare(int x, int y) => y.CompareTo(x);
		}

		public Dictionary<StreamId, SortedDictionary<int, Commit>> Changes { get; } = new Dictionary<StreamId, SortedDictionary<int, Commit>>();

		readonly IUserCollection _userCollection;

		public PerforceServiceStub(IUserCollection userCollection)
		{
			_userCollection = userCollection;
		}

		public Task<IPooledPerforceConnection> ConnectAsync(string? clusterName, string? userName, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		public async ValueTask<IUser> FindOrAddUserAsync(string clusterName, string userName, CancellationToken cancellationToken)
		{
			return await _userCollection.FindOrAddUserByLoginAsync(userName, cancellationToken: cancellationToken);
		}

		public void AddChange(StreamId streamId, int number, IUser author, string description, IEnumerable<string> files)
		{
			AddChange(streamId, number, number, author, author, description, files);
		}

		public void AddChange(StreamId streamId, int number, int originalNumber, IUser author, IUser owner, string description, IEnumerable<string> files)
		{
			SortedDictionary<int, Commit>? streamChanges;
			if (!Changes.TryGetValue(streamId, out streamChanges))
			{
				streamChanges = new SortedDictionary<int, Commit>(new ChangeComparer());
				Changes[streamId] = streamChanges;
			}
			streamChanges.Add(number, new Commit(streamId, number, originalNumber, author.Id, owner.Id, description, files.ToList()));
		}

		public Task<List<ICommit>> GetChangesAsync(StreamConfig stream, int? minChange, bool includeMinChange, int? maxChange, bool includeMaxChange, int? numResults, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			if (minChange != null && !includeMinChange)
			{
				minChange = minChange.Value + 1;
			}
			if (maxChange != null && !includeMaxChange)
			{
				maxChange = maxChange.Value - 1;
			}

			List<ICommit> results = new List<ICommit>();

			SortedDictionary<int, Commit>? streamChanges;
			if (Changes.TryGetValue(stream.Id, out streamChanges) && streamChanges.Count > 0)
			{
				foreach (Commit details in streamChanges.Values)
				{
					if (minChange.HasValue && details.Number < minChange)
					{
						break;
					}
					if (!maxChange.HasValue || details.Number <= maxChange.Value)
					{
						results.Add(details);
					}
					if (numResults != null && numResults > 0 && results.Count >= numResults)
					{
						break;
					}
				}
			}

			return Task.FromResult(results);
		}

		public Task<(CheckShelfResult, ShelfInfo?)> CheckShelfAsync(StreamConfig stream, int changeNumber, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		public Task<List<ICommit>> GetChangeDetailsAsync(StreamConfig stream, IReadOnlyList<int> changeNumbers, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			List<ICommit> results = new List<ICommit>();
			foreach (int changeNumber in changeNumbers)
			{
				if(!Changes.ContainsKey(stream.Id) || !Changes[stream.Id].ContainsKey(changeNumber))
				{
					throw new PerforceException($"PerforceServiceStub is simulating a changelist not found exception {changeNumber}");
				}

				results.Add(Changes[stream.Id][changeNumber]);
			}
			return Task.FromResult(results);
		}

		public static Task<string> CreateTicket()
		{
			return Task.FromResult("bogus-ticket");
		}

		public Task<int> CreateNewChangeAsync(string clusterName, string streamName, string path, string description, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		public Task<(int? Change, string Message)> SubmitShelvedChangeAsync(StreamConfig stream, int shelvedChange, int originalChange, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		public Task<int> DuplicateShelvedChangeAsync(string clusterName, int shelvedChange, CancellationToken cancellationToken)
		{
			return Task.FromResult(shelvedChange);
		}

		public Task DeleteShelvedChangeAsync(string clusterName, int shelvedChange, CancellationToken cancellationToken)
		{
			return Task.CompletedTask;
		}

		public Task UpdateChangelistDescriptionAsync(string clusterName, int change, Func<string, string> description, CancellationToken cancellationToken)
		{
			return Task.CompletedTask;
		}

		public Task<ICommit> GetChangeDetailsAsync(string clusterName, int changeNumber, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}
		/*
				public Task<ICommit> GetChangeDetailsAsync(IStream stream, int changeNumber, CancellationToken cancellationToken)
				{
					return Task.FromResult<ICommit>(Changes[stream.Id][changeNumber]);
				}
		*/
		class CommitCollection : ICommitCollection
		{
			readonly PerforceServiceStub _owner;
			readonly StreamConfig _streamConfig;

			public CommitCollection(PerforceServiceStub owner, StreamConfig stream)
			{
				_owner = owner;
				_streamConfig = stream;
			}

			public Task<CommitIdWithOrder> CreateNewAsync(string path, string description, CancellationToken cancellationToken = default)
			{
				throw new NotImplementedException();
			}

			public async IAsyncEnumerable<ICommit> FindAsync(CommitId? minChange, bool includeMinChange, CommitId? maxChange, bool includeMaxChange, int? maxResults, IReadOnlyList<CommitTag>? tags, CommitSortOrder sortOrder = CommitSortOrder.Descending, [EnumeratorCancellation] CancellationToken cancellationToken = default)
			{
				IEnumerable<ICommit> commits = await _owner.GetChangesAsync(_streamConfig, minChange?.GetPerforceChange(), includeMinChange, maxChange?.GetPerforceChange(), includeMaxChange, null, cancellationToken);

				// Handle sort order - the test stub stores in descending order by default
				if (sortOrder == CommitSortOrder.Ascending)
				{
					commits = commits.Reverse();
				}

				foreach (ICommit commit in commits)
				{
					if (tags != null && tags.Count > 0)
					{
						IReadOnlyList<CommitTag> commitTags = await commit.GetTagsAsync(cancellationToken);
						if (!tags.Any(x => commitTags.Contains(x)))
						{
							continue;
						}
					}
					yield return commit;
				}
			}

			public async Task<ICommit> GetAsync(CommitId changeNumber, CancellationToken cancellationToken = default)
			{
				List<ICommit> commits = await _owner.GetChangeDetailsAsync(_streamConfig, new[] { changeNumber.GetPerforceChange() }, cancellationToken);
				return commits[0];
			}

			public ValueTask<CommitIdWithOrder> GetOrderedAsync(CommitId commit, CancellationToken cancellationToken = default)
				=> ValueTask.FromResult(CommitIdWithOrder.FromPerforceChange(commit.GetPerforceChange()));

			public async Task<CommitIdWithOrder> LatestNumberAsync(CancellationToken cancellationToken = default)
			{
				List<ICommit> commits = await _owner.GetChangesAsync(_streamConfig, null, true, null, true, 1, cancellationToken);
				return commits[0].Id;
			}

			public IAsyncEnumerable<ICommit> SubscribeAsync(CommitId minChange, IReadOnlyList<CommitTag>? tags = null, CancellationToken cancellationToken = default)
			{
				throw new NotImplementedException();
			}
		}

		public ICommitCollection GetCommits(StreamConfig streamConfig)
		{
			return new CommitCollection(this, streamConfig);
		}

		public Task RefreshCachedCommitAsync(string clusterName, int change)
		{
			throw new NotImplementedException();
		}
	}
}
