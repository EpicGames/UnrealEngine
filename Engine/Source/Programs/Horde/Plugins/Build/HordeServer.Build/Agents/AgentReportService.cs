// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using HordeServer.Jobs;
using HordeServer.Notifications;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;

namespace HordeServer.Agents
{
	/// <summary>
	/// Report for the state of agents in the farm
	/// </summary>
	public class AgentReport
	{
		/// <summary>
		/// List of agents stuck in a conform loop, plus the number of times they've attempted to conform
		/// </summary>
		public List<(AgentId, int)> ConformLoop { get; } = new List<(AgentId, int)>();

		/// <summary>
		/// List of agents stuck in a upgrade loop, plus the number of times they've attempted to upgrade
		/// </summary>
		public List<(AgentId, int)> UpgradeLoop { get; } = new List<(AgentId, int)>();
	}

	[SingletonDocument("agent-report-state", "6268871c211d05611b3e4fd8")]
	class AgentReportState : SingletonBase
	{
		public DateTime LastUpdateUtc { get; set; }
	}

	/// <summary>
	/// Posts summaries for all the open issues in different streams to Slack channels
	/// </summary>
	public sealed class AgentReportService : IHostedService, IAsyncDisposable
	{
		readonly SingletonDocument<AgentReportState> _state;
		readonly IAgentCollection _agentCollection;
		readonly INotificationService _notificationService;
		readonly IClock _clock;
		readonly ITicker _ticker;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentReportService(IMongoService mongoService, IAgentCollection agentCollection, INotificationService notificationService, IClock clock, ILogger<AgentReportService> logger)
		{
			_state = new SingletonDocument<AgentReportState>(mongoService);
			_agentCollection = agentCollection;
			_notificationService = notificationService;
			_clock = clock;
			_ticker = clock.AddSharedTicker<AgentReportService>(TimeSpan.FromMinutes(5.0), TickAsync, logger);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			AgentReportState state = await _state.GetAsync(cancellationToken);
			DateTime utcNow = _clock.UtcNow;

			TimeSpan agentReportTime = TimeSpan.FromHours(9.0); // 9am

			DateTime nextUpdateTime = _clock.TimeZone.GetStartOfDayUtc(state.LastUpdateUtc + TimeSpan.FromHours(24.0)) + agentReportTime;
			if (utcNow > nextUpdateTime)
			{
				AgentReport report = new AgentReport();

				await foreach (IAgent agent in  _agentCollection.FindAsync(cancellationToken: cancellationToken))
				{
					if (agent.IsSessionValid(utcNow))
					{
						if (agent.ConformAttemptCount.HasValue && agent.ConformAttemptCount.Value > 3)
						{
							report.ConformLoop.Add((agent.Id, agent.ConformAttemptCount.Value));
						}
						if (agent.UpgradeAttemptCount.HasValue && agent.UpgradeAttemptCount.Value > 3)
						{
							report.UpgradeLoop.Add((agent.Id, agent.UpgradeAttemptCount.Value));
						}
					}
				}

				await _notificationService.SendAgentReportAsync(report, cancellationToken);

				DateTime updateTime = _clock.TimeZone.GetStartOfDayUtc(utcNow) + agentReportTime;
				if (updateTime > utcNow)
				{
					updateTime -= TimeSpan.FromDays(1.0);
				}
				await _state.UpdateAsync(x => x.LastUpdateUtc = updateTime, cancellationToken);
			}
		}
	}
}
