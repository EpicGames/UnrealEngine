// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using HordeServer.Utilities;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents
{
	/// <summary>
	/// Entry to the conform list
	/// </summary>
	public class ConformListEntry
	{
		/// <summary>
		/// The agent id
		/// </summary>
		[BsonElement("a")]
		public AgentId AgentId { get; set; }

		/// <summary>
		/// The lease id
		/// </summary>
		[BsonElement("l")]
		public LeaseId LeaseId { get; set; }

		/// <summary>
		/// Last timestamp that the lease was checked for validity 
		/// </summary>
		[BsonElement("t"), BsonIgnoreIfNull]
		public DateTime? LastCheckTimeUtc { get; set; }
	}

	/// <summary>
	/// List of entries for a particular server
	/// </summary>
	public class ConformListServer
	{
		/// <summary>
		/// The perforce cluster
		/// </summary>
		public string Cluster { get; set; } = String.Empty;

		/// <summary>
		/// The server and port
		/// </summary>
		public string ServerAndPort { get; set; } = String.Empty;

		/// <summary>
		/// List of entries
		/// </summary>
		public List<ConformListEntry> Entries { get; set; } = new List<ConformListEntry>();
	}

	/// <summary>
	/// List of machines that are currently conforming
	/// </summary>
	[SingletonDocument("conform-list", "60afc737f0d2a70754229300")]
	public class ConformList : SingletonBase
	{
		/// <summary>
		/// List of conforming servers [DEPRECATED]
		/// </summary>
		public List<ConformListEntry> Entries { get; set; } = new List<ConformListEntry>();

		/// <summary>
		/// List of conforming servers
		/// </summary>
		public List<ConformListServer> Servers { get; set; } = new List<ConformListServer>();
	}
}
