// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Streams;
using HordeServer.Agents.Pools;
using HordeServer.Jobs;
using HordeServer.Jobs.Graphs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Calculate pool size by observing the number of jobs in waiting state
	///
	/// Allows for more proactive scaling compared to LeaseUtilizationStrategy.
	/// <see cref="LeaseUtilizationStrategy"/> 
	/// </summary>
	public class JobQueueStrategy : IPoolSizeStrategy
	{
		private const string CacheKey = nameof(JobQueueStrategy);
		internal JobQueueSettings Settings { get; }

		private readonly IJobCollection _jobs;
		private readonly IGraphCollection _graphs;
		private readonly IClock _clock;
		private readonly IMemoryCache _cache;
		private readonly bool _isDowntimeActive;
		private readonly IOptionsMonitor<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobQueueStrategy(IJobCollection jobs, IGraphCollection graphs, IClock clock, IMemoryCache cache, bool isDowntimeActive, IOptionsMonitor<BuildConfig> buildConfig, JobQueueSettings? settings = null)
		{
			_jobs = jobs;
			_graphs = graphs;
			_clock = clock;
			_cache = cache;
			_isDowntimeActive = isDowntimeActive;
			_buildConfig = buildConfig;
			Settings = settings ?? new JobQueueSettings();
		}

		/// <inheritdoc/>
		public string Name { get; } = "JobQueue";

		/// <summary>
		/// Extract all job step batches from a job, with their associated pool 
		/// </summary>
		/// <param name="job">Job to extract from</param>
		/// <param name="streams">Cached lookup table of streams</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		private async Task<List<(IJob Job, IJobStepBatch Batch, PoolId PoolId)>> GetJobBatchesWithPoolsAsync(IJob job, Dictionary<StreamId, StreamConfig> streams, CancellationToken cancellationToken)
		{
			IGraph graph = await _graphs.GetAsync(job.GraphHash, cancellationToken);

			List<(IJob Job, IJobStepBatch Batch, PoolId PoolId)> jobBatches = new();
			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.State != JobStepBatchState.Ready)
				{
					continue;
				}

				TimeSpan? waitTime = _clock.UtcNow - batch.ReadyTimeUtc;
				if (waitTime == null)
				{
					continue;
				}
				if (waitTime.Value < TimeSpan.FromSeconds(Settings.ReadyTimeThresholdSec))
				{
					continue;
				}

				if (!streams.TryGetValue(job.StreamId, out StreamConfig? streamConfig))
				{
					continue;
				}

				string batchAgentType = graph.Groups[batch.GroupIdx].AgentType;
				if (!streamConfig.AgentTypes.TryGetValue(batchAgentType, out AgentConfig? agentType))
				{
					continue;
				}

				jobBatches.Add((job, batch, agentType.Pool));
			}

			return jobBatches;
		}

		internal async Task<Dictionary<PoolId, int>> GetPoolQueueSizesAsync(DateTimeOffset jobsCreatedAfter, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(JobQueueStrategy)}.{nameof(GetPoolQueueSizesAsync)}");
			span.SetAttribute("after", jobsCreatedAfter);

			Dictionary<StreamId, StreamConfig> streams = _buildConfig.CurrentValue.Streams.ToDictionary(x => x.Id, x => (StreamConfig)x);
			IReadOnlyList<IJob> recentJobs = await _jobs.FindAsync(new FindJobOptions(MinCreateTime: jobsCreatedAfter, BatchState: JobStepBatchState.Ready), cancellationToken: cancellationToken);
			span.SetAttribute("numJobs", recentJobs.Count);
			span.SetAttribute("numUniqueGraphs", recentJobs.Select(x => x.GraphHash).Distinct().Count());

			List<(IJob Job, IJobStepBatch Batch, PoolId PoolId)> jobBatches = new();
			foreach (IJob job in recentJobs)
			{
				jobBatches.AddRange(await GetJobBatchesWithPoolsAsync(job, streams, cancellationToken));
			}

			List<(PoolId PoolId, int QueueSize)> poolsWithQueueSize = jobBatches.GroupBy(t => t.PoolId).Select(t => (t.Key, t.Count())).ToList();

			span.SetAttribute("numPools", poolsWithQueueSize.Count);

			if (_isDowntimeActive)
			{
				// As an optimization, assume queue size is zero during maintenance windows.
				return poolsWithQueueSize.ToDictionary(x => x.PoolId, x => 0);
			}

			return poolsWithQueueSize.ToDictionary(x => x.PoolId, x => x.QueueSize);
		}

		/// <inheritdoc/>
		public async Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(JobQueueStrategy)}.{nameof(CalculatePoolSizeAsync)}");
			span.SetAttribute(OpenTelemetryTracers.DatadogResourceAttribute, pool.Id.ToString());
			span.SetAttribute("currentAgentCount", agents.Count);
			span.SetAttribute("samplePeriodMin", Settings.SamplePeriodMin);

			DateTimeOffset minCreateTime = _clock.UtcNow - TimeSpan.FromMinutes(Settings.SamplePeriodMin);

			// Cache pool queue sizes for a short while for faster runs when many pools are scaled
			if (!_cache.TryGetValue(CacheKey, out Dictionary<PoolId, int>? poolQueueSizes) || poolQueueSizes == null)
			{
				// Pool sizes haven't been cached, update them (might happen from multiple tasks but that is fine)
				poolQueueSizes = await GetPoolQueueSizesAsync(minCreateTime, cancellationToken);
				_cache.Set(CacheKey, poolQueueSizes, TimeSpan.FromSeconds(60));
			}

			poolQueueSizes.TryGetValue(pool.Id, out int queueSize);

			Dictionary<string, object> status = new()
			{
				["Name"] = GetType().Name,
				["QueueSize"] = queueSize,
				["ScaleOutFactor"] = Settings.ScaleOutFactor,
				["ScaleInFactor"] = Settings.ScaleInFactor,
				["SamplePeriodMin"] = Settings.SamplePeriodMin,
				["ReadyTimeThresholdSec"] = Settings.ReadyTimeThresholdSec,
			};

			if (queueSize > 0)
			{
				int additionalAgentCount = (int)Math.Ceiling(queueSize * Settings.ScaleOutFactor);
				int desiredAgentCount = agents.Count + additionalAgentCount;
				return new PoolSizeResult(agents.Count, desiredAgentCount, status);
			}
			else
			{
				int desiredAgentCount = (int)(agents.Count * Settings.ScaleInFactor);
				return new PoolSizeResult(agents.Count, desiredAgentCount, status);
			}
		}
	}

	class JobQueueStrategyFactory(
		IGraphCollection graphCollection,
		IJobCollection jobCollection,
		IDowntimeService downtimeService,
		IClock clock,
		IMemoryCache memoryCache,
		IOptionsMonitor<BuildConfig> buildConfig)
		: PoolSizeStrategyFactory<JobQueueSettings>(PoolSizeStrategy.JobQueue)
	{
		public override IPoolSizeStrategy Create(JobQueueSettings settings)
			=> new JobQueueStrategy(jobCollection, graphCollection, clock, memoryCache, downtimeService.IsDowntimeActive, buildConfig, settings);
	}
}
