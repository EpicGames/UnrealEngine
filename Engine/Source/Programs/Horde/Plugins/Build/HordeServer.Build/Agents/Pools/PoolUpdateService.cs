// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Streams;
using HordeServer.VersionControl.Perforce;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Pools
{
	/// <summary>
	/// Periodically updates pool documents to contain the correct workspaces
	/// </summary>
	public sealed class PoolUpdateService : IHostedService, IAsyncDisposable
	{
		readonly AgentService _agentService;
		readonly PoolService _poolService;
		readonly IPoolCollection _pools;
		readonly IClock _clock;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly ILogger<PoolUpdateService> _logger;
		readonly Tracer _tracer;
		readonly ITicker _updatePoolsTicker;
		readonly ITicker _shutdownDisabledAgentsTicker;
		readonly ITicker _autoConformAgentsTicker;

		/// <summary>
		/// Constructor
		/// </summary>
		public PoolUpdateService(AgentService agentService, PoolService poolService, IPoolCollection pools, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, IOptionsMonitor<ComputeConfig> computeConfig, Tracer tracer, ILogger<PoolUpdateService> logger)
		{
			_agentService = agentService;
			_poolService = poolService;
			_pools = pools;
			_clock = clock;
			_buildConfig = buildConfig;
			_computeConfig = computeConfig;
			_tracer = tracer;
			_logger = logger;
			_updatePoolsTicker = clock.AddTicker($"{nameof(PoolUpdateService)}.{nameof(UpdatePoolsAsync)}", TimeSpan.FromSeconds(30.0), UpdatePoolsAsync, logger);
			_shutdownDisabledAgentsTicker = clock.AddSharedTicker($"{nameof(PoolUpdateService)}.{nameof(ShutdownDisabledAgentsAsync)}", TimeSpan.FromHours(1), ShutdownDisabledAgentsAsync, logger);
			_autoConformAgentsTicker = clock.AddSharedTicker($"{nameof(PoolUpdateService)}.{nameof(AutoConformAgentsAsync)}", TimeSpan.FromMinutes(15), AutoConformAgentsAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _updatePoolsTicker.StartAsync();
			await _shutdownDisabledAgentsTicker.StartAsync();
			await _autoConformAgentsTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _updatePoolsTicker.StopAsync();
			await _shutdownDisabledAgentsTicker.StopAsync();
			await _autoConformAgentsTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _updatePoolsTicker.DisposeAsync();
			await _shutdownDisabledAgentsTicker.DisposeAsync();
			await _autoConformAgentsTicker.DisposeAsync();
		}

		/// <summary>
		/// Shutdown agents that have been disabled for longer than the configured grace period
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the async task</param>
		/// <returns>Async task</returns>
		internal async ValueTask ShutdownDisabledAgentsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PoolUpdateService)}.{nameof(ShutdownDisabledAgentsAsync)}");

			IReadOnlyList<IPoolConfig> pools = await _pools.GetConfigsAsync(cancellationToken);
			IEnumerable<IAgent> disabledAgents = (await _agentService.GetCachedAgentsAsync(cancellationToken)).Where(x => !x.Enabled && IsAgentAutoScaled(x, pools));

			int c = 0;
			foreach (IAgent agent in disabledAgents)
			{
				_logger.LogInformation("Checking if disabled agent {AgentId} should be shut down", agent.Id.ToString());
				if (HasGracePeriodExpired(agent, pools, _buildConfig.CurrentValue.AgentShutdownIfDisabledGracePeriod))
				{
					await agent.TryUpdateAsync(new UpdateAgentOptions { RequestShutdown = true }, cancellationToken: cancellationToken);
					_logger.LogInformation("Shutting down agent {AgentId} as it has been disabled for longer than grace period", agent.Id.ToString());
					c++;
				}
			}

			span.SetAttribute("numShutdown", c);
		}

		private bool HasGracePeriodExpired(IAgent agent, IReadOnlyList<IPoolConfig> pools, TimeSpan? globalGracePeriod)
		{
			if (agent.LastOnlineTime == null)
			{
				return false;
			}

			TimeSpan? gracePeriod = GetGracePeriod(agent, pools) ?? globalGracePeriod;
			if (gracePeriod == null)
			{
				return false;
			}

			DateTime expirationTime = agent.LastOnlineTime.Value + gracePeriod.Value;
			return _clock.UtcNow > expirationTime;
		}

		private static TimeSpan? GetGracePeriod(IAgent agent, IReadOnlyList<IPoolConfig> pools)
		{
			IEnumerable<PoolId> poolIds = agent.ExplicitPools.Concat(agent.DynamicPools);
			IPoolConfig? pool = pools.FirstOrDefault(x => poolIds.Contains(x.Id) && x.ShutdownIfDisabledGracePeriod != null);
			return pool?.ShutdownIfDisabledGracePeriod;
		}

		private static bool IsAgentAutoScaled(IAgent agent, IReadOnlyList<IPoolConfig> pools)
		{
			IEnumerable<PoolId> poolIds = agent.ExplicitPools.Concat(agent.DynamicPools);
			return pools.Any(x => poolIds.Contains(x.Id) && x.EnableAutoscaling);
		}

		/// <summary>
		/// Find and conform any agents below the disk free conform threshold for a workspace
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the async task</param>
		/// <returns>Async task</returns>
		internal async ValueTask AutoConformAgentsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PoolUpdateService)}.{nameof(AutoConformAgentsAsync)}");
			List<IAgent> agents = (await _agentService.GetCachedAgentsAsync(cancellationToken))
				.Where(x => x is { Status: AgentStatus.Stopped, Deleted: false, Enabled: true, Ephemeral: false }).ToList();

			foreach (IAgent agent in agents)
			{
				await ConformUtilities.AutoConformAgentAsync(agent, _poolService, _buildConfig.CurrentValue, _logger, cancellationToken);
			}
		}

		/// <summary>
		/// Execute the background task
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the async task</param>
		/// <returns>Async task</returns>
		internal async ValueTask UpdatePoolsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PoolUpdateService)}.{nameof(UpdatePoolsAsync)}");

			// Capture the start time for this operation. We use this to attempt to sequence updates to agents, and prevent overriding another server's updates.
			DateTime startTime = DateTime.UtcNow;

			// Update the list
			bool retryUpdate = true;
			while (retryUpdate && !cancellationToken.IsCancellationRequested)
			{
				_logger.LogDebug("Updating pool->workspace map");

				// Assume this will be the last iteration
				retryUpdate = false;

				// Capture the list of pools at the start of this update
				IReadOnlyList<IPoolConfig> currentPools = await _pools.GetConfigsAsync(cancellationToken);

				// Lookup table of pool id to workspaces
				Dictionary<PoolId, AutoSdkConfig> poolToAutoSdkView = new Dictionary<PoolId, AutoSdkConfig>();
				Dictionary<PoolId, List<AgentWorkspaceInfo>> poolToAgentWorkspaces = new Dictionary<PoolId, List<AgentWorkspaceInfo>>();

				// Capture the current config state
				BuildConfig buildConfig = _buildConfig.CurrentValue;

				// Populate the workspace list from the current stream
				foreach (StreamConfig streamConfig in buildConfig.Streams)
				{
					foreach (KeyValuePair<string, AgentConfig> agentTypePair in streamConfig.AgentTypes)
					{
						// Create the new agent workspace
						if (streamConfig.TryGetAgentWorkspace(agentTypePair.Value, out AgentWorkspaceInfo? agentWorkspace, out AutoSdkConfig? autoSdkConfig))
						{
							AgentConfig agentType = agentTypePair.Value;

							// Find or add a list of workspaces for this pool
							List<AgentWorkspaceInfo>? agentWorkspaces;
							if (!poolToAgentWorkspaces.TryGetValue(agentType.Pool, out agentWorkspaces))
							{
								agentWorkspaces = new List<AgentWorkspaceInfo>();
								poolToAgentWorkspaces.Add(agentType.Pool, agentWorkspaces);
							}

							// Add it to the list
							if (!agentWorkspaces.Contains(agentWorkspace))
							{
								agentWorkspaces.Add(agentWorkspace);
							}
							if (autoSdkConfig != null)
							{
								AutoSdkConfig? existingAutoSdkConfig;
								poolToAutoSdkView.TryGetValue(agentType.Pool, out existingAutoSdkConfig);
								poolToAutoSdkView[agentType.Pool] = AutoSdkConfig.Merge(autoSdkConfig, existingAutoSdkConfig);
							}
						}
					}
				}

				if (_computeConfig.CurrentValue.VersionEnum >= ConfigVersion.PoolsInConfigFiles)
				{
					return;
				}

				// Update the list of workspaces for each pool
				foreach (IPoolConfig currentPool in currentPools)
				{

					// Get the new list of workspaces for this pool
					List<AgentWorkspaceInfo>? newWorkspaces;
					if (!poolToAgentWorkspaces.TryGetValue(currentPool.Id, out newWorkspaces))
					{
						newWorkspaces = new List<AgentWorkspaceInfo>();
					}

					// Get the autosdk view
					AutoSdkConfig? newAutoSdkConfig;
					if (!poolToAutoSdkView.TryGetValue(currentPool.Id, out newAutoSdkConfig))
					{
						newAutoSdkConfig = AutoSdkConfig.None;
					}

					// Update the pools document
					if (!AgentWorkspaceInfo.SetEquals(currentPool.Workspaces, newWorkspaces) || currentPool.Workspaces.Count != newWorkspaces.Count || !AutoSdkConfig.Equals(currentPool.AutoSdkConfig, newAutoSdkConfig))
					{
						_logger.LogInformation("New workspaces for pool {Pool}:{Workspaces}", currentPool.Id, String.Join("", newWorkspaces.Select(x => $"\n  Identifier=\"{x.Identifier}\", Stream={x.Stream}")));

						if (newAutoSdkConfig != null)
						{
							_logger.LogInformation("New autosdk view for pool {Pool}:{View}", currentPool.Id, String.Join("", newAutoSdkConfig.View.Select(x => $"\n  {x}")));
						}

#pragma warning disable CS0618 // Type or member is obsolete
						await _pools.UpdateConfigAsync(currentPool.Id, new UpdatePoolConfigOptions { Workspaces = newWorkspaces, AutoSdkConfig = newAutoSdkConfig }, cancellationToken);
#pragma warning restore CS0618 // Type or member is obsolete
					}
				}
			}
		}
	}
}
