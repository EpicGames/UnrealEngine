// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// ACL actions for artifacts
	/// </summary>
	public static class ArtifactAclAction
	{
		/// <summary>
		/// Permission to read from an artifact
		/// </summary>
		public static AclAction ReadArtifact { get; } = new AclAction("ReadArtifact");

		/// <summary>
		/// Permission to write to an artifact
		/// </summary>
		public static AclAction WriteArtifact { get; } = new AclAction("WriteArtifact");

		/// <summary>
		/// Permission to delete to an artifact
		/// </summary>
		public static AclAction DeleteArtifact { get; } = new AclAction("DeleteArtifact");

		/// <summary>
		/// Ability to create an artifact. Typically just for debugging; agents have this access for a particular session.
		/// </summary>
		public static AclAction UploadArtifact { get; } = new AclAction("UploadArtifact");

		/// <summary>
		/// Ability to download an artifact
		/// </summary>
		public static AclAction DownloadArtifact { get; } = new AclAction("DownloadArtifact");
	}
}
