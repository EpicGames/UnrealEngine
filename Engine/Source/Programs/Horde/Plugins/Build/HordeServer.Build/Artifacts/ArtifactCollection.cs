// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Runtime.CompilerServices;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using HordeServer.Commits;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Storage;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Implementation of <see cref="IArtifactCollection"/>
	/// </summary>
	class ArtifactCollection : IArtifactCollection, IHostedService, IAsyncDisposable
	{
		class Artifact : IArtifact
		{
			readonly ArtifactCollection _collection;
			readonly ArtifactDocument _document;

			public ArtifactDocument Document => _document;

			public ArtifactId Id => new ArtifactId(BinaryIdUtils.FromObjectId(_document.Id));
			ArtifactName IArtifact.Name => _document.Name;
			ArtifactType IArtifact.Type => _document.Type;
			string? IArtifact.Description => _document.Description;
			StreamId IArtifact.StreamId => _document.StreamId;
			CommitIdWithOrder IArtifact.CommitId => _document.CommitId;
			IReadOnlyList<string> IArtifact.Keys => _document.Keys;
			IReadOnlyList<string> IArtifact.Metadata => _document.Metadata;
			NamespaceId IArtifact.NamespaceId => _document.NamespaceId;
			RefName IArtifact.RefName => _document.RefName;
			DateTime IArtifact.CreatedAtUtc => (_document.CreatedAtUtc == default) ? _document.Id.CreationTime : _document.CreatedAtUtc;

			IBlobRef<DirectoryNode> IArtifact.Content
				=> _collection.Open(this);

			public Artifact(ArtifactCollection collection, ArtifactDocument document)
			{
				_collection = collection;
				_document = document;
			}

			public Task DeleteAsync(CancellationToken cancellationToken)
				=> _collection.DeleteArtifactAsync(_document, cancellationToken);
		}

		class ArtifactDocument
		{
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonElement("nam")]
			public ArtifactName Name { get; set; }

			[BsonElement("typ")]
			public ArtifactType Type { get; set; }

			[BsonElement("dsc"), BsonIgnoreIfNull]
			public string? Description { get; set; }

			[BsonElement("str")]
			public StreamId StreamId { get; set; }

			[BsonElement("com")]
			public string? CommitName { get; set; }

			[BsonElement("chg")]
			public int CommitOrder { get; set; } // Was P4 changelist number

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("key")]
			public List<string> Keys { get; set; } = new List<string>();

			[BsonElement("met")]
			public List<string> Metadata { get; set; } = new List<string>();

			[BsonElement("ns")]
			public NamespaceId NamespaceId { get; set; }

			[BsonElement("ref")]
			public RefName RefName { get; set; }

			[BsonElement("cre")]
			public DateTime CreatedAtUtc { get; set; }

			[BsonElement("upd")]
			public int UpdateIndex { get; set; }

			[BsonConstructor]
			private ArtifactDocument()
			{
			}

			public ArtifactDocument(ObjectId id, ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitIdWithOrder commitId, IEnumerable<string> keys, IEnumerable<string> metadata, NamespaceId namespaceId, RefName refName, DateTime createdAtUtc)
			{
				Id = id;
				Name = name;
				Type = type;
				Description = description;
				StreamId = streamId;
				CommitId = commitId;
				Keys.AddRange(keys);
				Metadata.AddRange(metadata);
				NamespaceId = namespaceId;
				RefName = refName;
				CreatedAtUtc = createdAtUtc;
			}
		}

		class ArtifactExpiryDocument
		{
			public ObjectId Id { get; set; }

			[BsonElement("str")]
			public StreamId StreamId { get; set; }

			[BsonElement("typ")]
			public ArtifactType Type { get; set; }

			[BsonElement("tim")]
			public DateTime Time { get; set; }
		}

		class ArtifactBuilder : IArtifactBuilder
		{
			record class AliasInfo(string Name, IBlobRef BlobRef, int Rank, ReadOnlyMemory<byte> Data);

			readonly IArtifact _artifact;
			readonly IStorageNamespace _storage;
			readonly List<AliasInfo> _aliases = new List<AliasInfo>();

			public ArtifactId Id => _artifact.Id;
			public ArtifactName Name => _artifact.Name;
			public ArtifactType Type => _artifact.Type;
			public string? Description => _artifact.Description;
			public StreamId StreamId => _artifact.StreamId;
			public CommitIdWithOrder CommitId => _artifact.CommitId;
			public IReadOnlyList<string> Keys => _artifact.Keys;
			public IReadOnlyList<string> Metadata => _artifact.Metadata;
			public NamespaceId NamespaceId => _artifact.NamespaceId;
			public RefName RefName => _artifact.RefName;

			public ArtifactBuilder(Artifact artifact, IStorageNamespace storage)
			{
				_artifact = artifact;
				_storage = storage;
			}

			public Task AddAliasAsync(string name, IBlobRef handle, int rank = 0, ReadOnlyMemory<byte> data = default, CancellationToken cancellationToken = default)
			{
				_aliases.Add(new AliasInfo(name, handle, rank, data.ToArray()));
				return Task.CompletedTask;
			}

			public async Task<IArtifact> CompleteAsync(IHashedBlobRef blobRef, CancellationToken cancellationToken = default)
			{
				await using (IStorageWriter writer = _storage.CreateWriter(cancellationToken))
				{
					foreach (AliasInfo aliasInfo in _aliases)
					{
						await writer.AddAliasAsync(aliasInfo.Name, aliasInfo.BlobRef, aliasInfo.Rank, aliasInfo.Data, cancellationToken);
					}
					await writer.AddRefAsync(_artifact.RefName, blobRef, cancellationToken: cancellationToken);
				}
				return _artifact;
			}

			public IBlobWriter CreateBlobWriter()
				=> _storage.CreateBlobWriter(_artifact.RefName);
		}

		readonly IMongoCollection<ArtifactDocument> _artifactCollection;
		readonly IMongoCollection<ArtifactExpiryDocument> _artifactExpiryCollection;
		readonly IClock _clock;
		readonly ICommitService _commitService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly StorageService _storageService;
		readonly ITicker _ticker;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ArtifactCollection(IMongoService mongoService, IClock clock, ICommitService commitService, StorageService storageService, IOptionsMonitor<BuildConfig> buildConfig, ILogger<ArtifactCollection> logger)
		{
			List<MongoIndex<ArtifactDocument>> indexes = new List<MongoIndex<ArtifactDocument>>();
			indexes.Add(keys => keys.Ascending(x => x.Keys));
			indexes.Add(keys => keys.Ascending(x => x.Type).Descending(x => x.Id));
			indexes.Add(keys => keys.Ascending(x => x.StreamId).Descending(x => x.CommitOrder).Ascending(x => x.Name).Descending(x => x.Id));
			indexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.Type).Descending(x => x.Id).Ascending(x => x.Name));
			_artifactCollection = mongoService.GetCollection<ArtifactDocument>("ArtifactsV2", indexes);

			List<MongoIndex<ArtifactExpiryDocument>> expiryIndexes = new List<MongoIndex<ArtifactExpiryDocument>>();
			expiryIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.Type), unique: true);
			_artifactExpiryCollection = mongoService.GetCollection<ArtifactExpiryDocument>("ArtifactsV2.Expiry", expiryIndexes);

			_clock = clock;
			_commitService = commitService;
			_buildConfig = buildConfig;
			_storageService = storageService;
			_ticker = clock.AddSharedTicker<ArtifactCollection>(TimeSpan.FromHours(1.0), ExpireArtifactsAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
			=> _ticker.DisposeAsync();

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
			=> await _ticker.StartAsync();

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
			=> await _ticker.StopAsync();

		IBlobRef<DirectoryNode> Open(Artifact artifact)
		{
			IStorageNamespace client = _storageService.GetNamespace(artifact.Document.NamespaceId);
			return client.CreateBlobRef<DirectoryNode>(artifact.Document.RefName);
		}

#pragma warning disable CA1308 // Expect ansi-only keys here
		static string NormalizeKey(string key)
			=> key.ToLowerInvariant();
#pragma warning restore CA1308

		/// <summary>
		/// Gets the base path for a set of artifacts
		/// </summary>
		public static string GetArtifactPath(StreamId streamId, ArtifactType type) => $"{type}/{streamId}";

		/// <inheritdoc/>
		public async Task<IArtifactBuilder> CreateAsync(ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitId commitId, IEnumerable<string> keys, IEnumerable<string> metadata, CancellationToken cancellationToken = default)
		{
			if (name.Id.IsEmpty)
			{
				throw new ArgumentException($"Artifact name cannot be empty", nameof(name));
			}
			if (type.Id.IsEmpty)
			{
				throw new ArgumentException($"Artifact type for '{name}' is not valid", nameof(type));
			}

			StreamConfig streamConfig = _buildConfig.CurrentValue.GetStream(streamId);
			if (!streamConfig.TryGetArtifactType(type, out ArtifactTypeConfig? artifactTypeConfig))
			{
				throw new ArtifactTypeNotFoundException(streamId, type);
			}

			// Create an entry in the expiry collection to ensure we can GC any unused items if the stream goes away
			DateTime utcNow = _clock.UtcNow;
			await AddExpiryRecordAsync(streamId, type, utcNow, cancellationToken);

			// Create the artifact
			ObjectId id = ObjectId.GenerateNewId();

			NamespaceId namespaceId = artifactTypeConfig.NamespaceId;
			RefName refName = new RefName($"{GetArtifactPath(streamId, type)}/{commitId}/{name}/{id}");

			CommitIdWithOrder commitIdWithOrder = await _commitService.GetOrderedAsync(streamId, commitId, cancellationToken);

			ArtifactDocument artifactDocument = new ArtifactDocument(id, name, type, description, streamId, commitIdWithOrder, keys.Select(x => NormalizeKey(x)), metadata, namespaceId, refName, _clock.UtcNow);
			await _artifactCollection.InsertOneAsync(artifactDocument, null, cancellationToken);

			Artifact artifact = new Artifact(this, artifactDocument);
			return new ArtifactBuilder(artifact, _storageService.GetNamespace(namespaceId));
		}

		async Task AddExpiryRecordAsync(StreamId streamId, ArtifactType type, DateTime utcNow, CancellationToken cancellationToken = default)
		{
			FilterDefinition<ArtifactExpiryDocument> streamFilter = Builders<ArtifactExpiryDocument>.Filter.Expr(x => x.StreamId == streamId && x.Type == type);
			UpdateDefinition<ArtifactExpiryDocument> streamUpdate = Builders<ArtifactExpiryDocument>.Update.Set(x => x.Time, utcNow);
			await _artifactExpiryCollection.UpdateOneAsync(streamFilter, streamUpdate, new UpdateOptions { IsUpsert = true }, cancellationToken);
		}
		/// <inheritdoc/>
		public async IAsyncEnumerable<IArtifact> FindAsync(ArtifactId[]? ids = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, ArtifactName? name = null, ArtifactType? type = null, IEnumerable<string>? keys = null, int maxResults = 100, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			FilterDefinition<ArtifactDocument> filter = FilterDefinition<ArtifactDocument>.Empty;
			if (ids != null && ids.Length > 0)
			{
				filter &= Builders<ArtifactDocument>.Filter.In(x => x.Id, ids.ConvertAll(x => BinaryIdUtils.ToObjectId(x.Id)));
			}
			if (streamId != null)
			{
				filter &= Builders<ArtifactDocument>.Filter.Eq(x => x.StreamId, streamId.Value);
				if (minCommitId != null)
				{
					CommitIdWithOrder minCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, minCommitId, cancellationToken);
					filter &= Builders<ArtifactDocument>.Filter.Gte(x => x.CommitOrder, minCommitIdWithOrder.Order);
				}
				if (maxCommitId != null)
				{
					CommitIdWithOrder maxCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, maxCommitId, cancellationToken);
					filter &= Builders<ArtifactDocument>.Filter.Lte(x => x.CommitOrder, maxCommitIdWithOrder.Order);
				}
			}
			if (name != null)
			{
				filter &= Builders<ArtifactDocument>.Filter.Eq(x => x.Name, name.Value);
			}
			if (type != null)
			{
				filter &= Builders<ArtifactDocument>.Filter.Eq(x => x.Type, type.Value);
			}
			if (keys != null && keys.Any())
			{
				filter &= Builders<ArtifactDocument>.Filter.All(x => x.Keys, keys.Select(x => NormalizeKey(x)));
			}

			using (IAsyncCursor<ArtifactDocument> cursor = await _artifactCollection.Find(filter).SortByDescending(x => x.CommitOrder).ThenByDescending(x => x.Id).Limit(maxResults).ToCursorAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (ArtifactDocument artifactDocument in cursor.Current)
					{
						yield return new Artifact(this, artifactDocument);
					}
				}
			}
		}

		async ValueTask ExpireArtifactsAsync(CancellationToken cancellationToken)
		{
			_logger.LogInformation("Checking for expired artifacts...");
			Stopwatch timer = Stopwatch.StartNew();

			BuildConfig buildConfig = _buildConfig.CurrentValue;

			DateTime utcNow = _clock.UtcNow;

			// Expire any active streams listed in the config
			foreach (ProjectConfig projectConfig in buildConfig.Projects)
			{
				foreach (StreamConfig streamConfig in projectConfig.Streams)
				{
					if (streamConfig.RetentionOptions.EnableArtifactExpiration == false)
					{
						continue;
					}

					foreach (ArtifactTypeConfig artifactTypeConfig in streamConfig.GetAllArtifactTypes())
					{
						await AddExpiryRecordAsync(streamConfig.Id, artifactTypeConfig.Type, utcNow, cancellationToken);
						await ExpireArtifactsForStreamAsync(streamConfig.Id, artifactTypeConfig, utcNow, cancellationToken);
					}
				}
			}

			// Find any orphaned configurations and expire those too
			DateTime orphanTime = utcNow.AddDays(-5.0);

			List<ArtifactExpiryDocument> expiryDocuments = await _artifactExpiryCollection.Find(x => x.Time < orphanTime).ToListAsync(cancellationToken);
			foreach (ArtifactExpiryDocument expiryDocument in expiryDocuments)
			{
				_logger.LogInformation("Expiring artifacts from orphaned stream {StreamId} (type={Type})", expiryDocument.StreamId, expiryDocument.Type);
				using (IAsyncCursor<ArtifactDocument> cursor = await _artifactCollection.Find(x => x.StreamId == expiryDocument.StreamId && x.Type == expiryDocument.Type).ToCursorAsync(cancellationToken))
				{
					while (await cursor.MoveNextAsync(cancellationToken))
					{
						await DeleteArtifactsAsync(cursor.Current, cancellationToken);
					}
				}
				await _artifactExpiryCollection.DeleteOneAsync(x => x.Id == expiryDocument.Id && x.Time == expiryDocument.Time, cancellationToken);
			}

			_logger.LogInformation("Finished expiring artifacts in {TimeSecs}s.", (long)timer.Elapsed.TotalSeconds);
		}

		async Task ExpireArtifactsForStreamAsync(StreamId streamId, ArtifactTypeConfig artifactTypeConfig, DateTime utcNow, CancellationToken cancellationToken)
		{
			if (artifactTypeConfig.KeepCount == null && artifactTypeConfig.KeepDays == null)
			{
				_logger.LogInformation("No expiration policy set for {StreamId} {ArtifactType}; keeping all artifacts.", streamId, artifactTypeConfig.Type);
				return;
			}

			if (artifactTypeConfig.KeepCount.HasValue)
			{
				_logger.LogInformation("Removing {StreamId} {ArtifactType} artifacts except newest {Count}", streamId, artifactTypeConfig.Type, artifactTypeConfig.KeepCount.Value);
			}

			int count = 0;

			FilterDefinition<ArtifactDocument> filter = Builders<ArtifactDocument>.Filter.Eq(x => x.StreamId, streamId) & Builders<ArtifactDocument>.Filter.Eq(x => x.Type, artifactTypeConfig.Type);
			if (artifactTypeConfig.KeepDays.HasValue)
			{
				DateTime expireAtUtc = utcNow - TimeSpan.FromDays(artifactTypeConfig.KeepDays.Value);
				filter &= Builders<ArtifactDocument>.Filter.Lt(x => x.Id, ObjectId.GenerateNewId(expireAtUtc));
				_logger.LogInformation("Removing {StreamId} {ArtifactType} artifacts except newer than {Time}", streamId, artifactTypeConfig.Type, expireAtUtc);
			}

			IFindFluent<ArtifactDocument, ArtifactDocument> query = _artifactCollection.Find(filter).SortByDescending(x => x.Id);
			using (IAsyncCursor<ArtifactDocument> cursor = await query.ToCursorAsync(cancellationToken))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					List<ArtifactDocument> deleteArtifacts = new List<ArtifactDocument>();
					foreach (ArtifactDocument artifact in cursor.Current)
					{
						if (artifactTypeConfig.KeepCount == null || count >= artifactTypeConfig.KeepCount.Value)
						{
							deleteArtifacts.Add(artifact);
						}
						count++;
					}
					await DeleteArtifactsAsync(deleteArtifacts, cancellationToken);
				}
			}
		}

		async Task DeleteArtifactAsync(ArtifactDocument artifact, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Deleting {StreamId} artifact {ArtifactId}, ref {RefName} (created {CreateTime})", artifact.StreamId, artifact.Id, artifact.RefName, artifact.CreatedAtUtc);

			IStorageNamespace storageNamespace = _storageService.GetNamespace(artifact.NamespaceId);
			await storageNamespace.RemoveRefAsync(artifact.RefName, cancellationToken);

			FilterDefinition<ArtifactDocument> filter = Builders<ArtifactDocument>.Filter.Eq(x => x.Id, artifact.Id);
			await _artifactCollection.DeleteOneAsync(filter, cancellationToken);
		}

		async Task DeleteArtifactsAsync(IEnumerable<ArtifactDocument> deleteArtifacts, CancellationToken cancellationToken)
		{
			foreach (IGrouping<NamespaceId, ArtifactDocument> artifactGroup in deleteArtifacts.GroupBy(x => x.NamespaceId))
			{
				IStorageNamespace storageNamespace = _storageService.GetNamespace(artifactGroup.Key);
				foreach (ArtifactDocument artifact in artifactGroup)
				{
					// Delete the ref allowing the storage service to expire this data
					_logger.LogInformation("Expiring {StreamId} artifact {ArtifactId}, ref {RefName} (created {CreateTime})", artifact.StreamId, artifact.Id, artifact.RefName, artifact.CreatedAtUtc);
					await storageNamespace.RemoveRefAsync(artifact.RefName, cancellationToken);
				}

				FilterDefinition<ArtifactDocument> filter = Builders<ArtifactDocument>.Filter.In(x => x.Id, artifactGroup.Select(x => x.Id));
				await _artifactCollection.DeleteManyAsync(filter, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<IArtifact?> GetAsync(ArtifactId artifactId, CancellationToken cancellationToken = default)
		{
			ObjectId objectId = BinaryIdUtils.ToObjectId(artifactId.Id);

			ArtifactDocument? document = await _artifactCollection.Find(x => x.Id == objectId).FirstOrDefaultAsync(cancellationToken);
			if (document == null)
			{
				return null;
			}

			return new Artifact(this, document);
		}
	}
}
