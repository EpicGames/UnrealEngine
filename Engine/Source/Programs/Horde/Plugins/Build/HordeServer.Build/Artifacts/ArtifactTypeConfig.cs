// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using HordeServer.Acls;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Configuration for an artifact
	/// </summary>
	public class ArtifactTypeConfig
	{
		/// <summary>
		/// Legacy 'Name' property
		/// </summary>
		[Obsolete("Use Type instead")]
		public ArtifactType Name
		{
			get => Type;
			set => Type = value;
		}

		/// <summary>
		/// Name of the artifact type
		/// </summary>
		public ArtifactType Type { get; set; }

		/// <summary>
		/// Acl for the artifact type
		/// </summary>
		public AclConfig? Acl { get; set; }

		/// <summary>
		/// Number of artifacts to retain
		/// </summary>
		public int? KeepCount { get; set; }

		/// <summary>
		/// Number of days to retain artifacts of this type
		/// </summary>
		public int? KeepDays { get; set; }

		/// <summary>
		/// Storage namespace to use for this artifact types
		/// </summary>
		public NamespaceId NamespaceId { get; set; } = new NamespaceId("horde-artifacts");

		/// <summary>
		/// Fixup an artifact type after reading the config
		/// </summary>
		public void PostLoad(AclConfig parentAcl)
		{
			Acl?.PostLoad(parentAcl, $"artifact:{Type}", AclConfig.GetActions([typeof(ArtifactTypeConfig)]));
		}
	}
}
