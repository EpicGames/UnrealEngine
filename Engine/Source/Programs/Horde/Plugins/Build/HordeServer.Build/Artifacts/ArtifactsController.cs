// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Buffers.Binary;
using System.Diagnostics;
using System.Net;
using System.Net.Mime;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Tasks;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Jobs;
using HordeServer.Storage;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Http.Features;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.StaticFiles;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Public interface for artifacts
	/// </summary>
	[Authorize]
	[ApiController]
	public class ArtifactsController : HordeControllerBase
	{
		readonly IArtifactCollection _artifactCollection;
		readonly StorageService _storageService;
		readonly ILeaseCollection _leaseCollection;
		readonly IJobCollection _jobCollection;
		readonly IAclService _aclService;
		readonly UnsyncCache _unsyncCache;
		readonly IBlockCache _blockCache;
		readonly IServerInfo _serverInfo;
		readonly BuildConfig _buildConfig;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ArtifactsController(IArtifactCollection artifactCollection, StorageService storageService, ILeaseCollection leaseCollection, IJobCollection jobCollection, IAclService aclService, UnsyncCache unsyncCache, IBlockCache blockCache, IServerInfo serverInfo, IOptionsSnapshot<BuildConfig> buildConfig, ILogger<ArtifactsController> logger)
		{
			_artifactCollection = artifactCollection;
			_storageService = storageService;
			_leaseCollection = leaseCollection;
			_jobCollection = jobCollection;
			_aclService = aclService;
			_unsyncCache = unsyncCache;
			_blockCache = blockCache;
			_serverInfo = serverInfo;
			_buildConfig = buildConfig.Value;
			_logger = logger;
		}

		/// <summary>
		/// Creates a new artifact. Actual data for the artifact can be uploaded using a storage namespace pointed to the blobs endpoint.
		/// </summary>
		/// <param name="request">Information about the desired artifact</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The created artifact</returns>
		[HttpPost]
		[Route("/api/v2/artifacts")]
		public async Task<ActionResult<CreateArtifactResponse>> CreateArtifactAsync([FromBody] CreateArtifactRequest request, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (request.StreamId != null && request.CommitId != null && _buildConfig.TryGetStream(request.StreamId.Value, out streamConfig))
			{
				if (streamConfig.Authorize(ArtifactAclAction.WriteArtifact, User))
				{
					return await CreateArtifactInternalAsync(request.Name, request.Type, request.Description, request.StreamId.Value, request.CommitId, request.Keys, request.Metadata, cancellationToken);
				}
			}

			LeaseId? leaseId = User.GetLeaseClaim();
			if (leaseId != null)
			{
				ILease? lease = await _leaseCollection.GetAsync(leaseId.Value, cancellationToken);
				if (lease == null)
				{
					_logger.LogInformation("Claim has invalid lease id {LeaseId}", leaseId.Value);
					return Forbid(ArtifactAclAction.WriteArtifact);
				}

				Any payload = Any.Parser.ParseFrom(lease.Payload.ToArray());
				if (!payload.TryUnpack(out ExecuteJobTask jobTask))
				{
					_logger.LogInformation("Lease {LeaseId} is not for a job", leaseId.Value);
					return Forbid(ArtifactAclAction.WriteArtifact);
				}

				IJob? job = await _jobCollection.GetAsync(JobId.Parse(jobTask.JobId), cancellationToken);
				if (job == null)
				{
					_logger.LogInformation("Missing job {JobId} for lease {LeaseId}", JobId.Parse(jobTask.JobId), leaseId.Value);
					return Forbid(ArtifactAclAction.WriteArtifact);
				}

				IJobStepBatch? batch = job.Batches.FirstOrDefault(x => x.LeaseId == leaseId);
				if (batch == null)
				{
					_logger.LogInformation("Unable to find batch in job {JobId} for lease {LeaseId}", job.Id, leaseId.Value);
					return Forbid(ArtifactAclAction.WriteArtifact);
				}

				List<string> keys = new List<string>(request.Keys);
				keys.Add(job.GetArtifactKey());

				IJobStep? step = batch.Steps.FirstOrDefault(x => x.State == JobStepState.Running);
				if (step != null)
				{
					keys.Add(job.GetArtifactKey(step));
				}

				StreamId streamId = request.StreamId ?? job.StreamId;

				AclScopeName scopeName = AclScopeName.Root;
				if (_buildConfig.TryGetTemplate(streamId, job.TemplateId, out TemplateRefConfig? templateRefConfig))
				{
					scopeName = templateRefConfig.Acl.ScopeName;
				}

				return await CreateArtifactInternalAsync(request.Name, request.Type, request.Description, streamId, request.CommitId ?? job.CommitId, keys, request.Metadata, cancellationToken);
			}

			return Forbid(ArtifactAclAction.WriteArtifact);
		}

		async Task<ActionResult<CreateArtifactResponse>> CreateArtifactInternalAsync(ArtifactName name, ArtifactType type, string? description, StreamId streamId, CommitId commitId, List<string> keys, List<string> metadata, CancellationToken cancellationToken)
		{
			IArtifactBuilder artifact = await _artifactCollection.CreateAsync(name, type, description, streamId, commitId, keys, metadata, cancellationToken);
			RefName? prevRefName = await GetPrevRefNameForArtifactAsync(artifact, cancellationToken);

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(HordeClaimTypes.ReadNamespace, $"{artifact.NamespaceId}:{ArtifactCollection.GetArtifactPath(streamId, type)}"));
			claims.Add(new AclClaimConfig(HordeClaimTypes.WriteNamespace, $"{artifact.NamespaceId}:{artifact.RefName}"));

			string token = await _aclService.IssueBearerTokenAsync(claims, TimeSpan.FromHours(8.0), cancellationToken);
			return new CreateArtifactResponse(artifact.Id, artifact.CommitId, artifact.NamespaceId, artifact.RefName, prevRefName, token);
		}

		async Task<RefName?> GetPrevRefNameForArtifactAsync(IArtifactBuilder artifact, CancellationToken cancellationToken)
		{
			IStorageBackend storageBackend = _storageService.CreateBackend(artifact.NamespaceId);
			await foreach (IArtifact prevArtifact in _artifactCollection.FindAsync(streamId: artifact.StreamId, maxCommitId: artifact.CommitId, name: artifact.Name, type: artifact.Type, cancellationToken: cancellationToken))
			{
				if (prevArtifact.CommitId == artifact.CommitId)
				{
					continue;
				}

				if (prevArtifact.NamespaceId != artifact.NamespaceId)
				{
					break;
				}

				HashedBlobRefValue? refValue = await storageBackend.TryReadRefAsync(prevArtifact.RefName, cancellationToken: cancellationToken);
				if (refValue != null)
				{
					return prevArtifact.RefName;
				}
			}
			return null;
		}

		/// <summary>
		/// Finds artifacts matching certain criteria
		/// </summary>
		/// <param name="ids">Artifact identifiers to filter</param>
		/// <param name="streamId">Stream to search</param>
		/// <param name="minChange">Minimum changelist number for artifacts to return</param>
		/// <param name="maxChange">Maximum changelist number for artifacts to return</param>
		/// <param name="name">Artifact name</param>
		/// <param name="type">Type of the artifact</param>
		/// <param name="keys">Keys to find</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="filter">Filter for returned values</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts")]
		[ProducesResponseType(typeof(FindArtifactsResponse), 200)]
		public async Task<ActionResult<object>> FindArtifactsAsync([FromQuery(Name = "id")] ArtifactId[]? ids = null, [FromQuery] StreamId? streamId = null, [FromQuery] CommitId? minChange = null, [FromQuery] CommitId? maxChange = null, [FromQuery(Name = "name")] ArtifactName? name = null, [FromQuery(Name = "type")] ArtifactType? type = null, [FromQuery(Name = "key")] IEnumerable<string>? keys = null, [FromQuery] int maxResults = 100, [FromQuery] PropertyFilter? filter = null)
		{
			FindArtifactsResponse response = new FindArtifactsResponse();
			await foreach (IArtifact artifact in _artifactCollection.FindAsync(ids, streamId, minChange, maxChange, name, type, keys, maxResults, HttpContext.RequestAborted))
			{
				if (_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
				{
					response.Artifacts.Add(new GetArtifactResponse(artifact));
				}
			}

			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Gets metadata about an artifact object
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="filter">Filter for returned properties</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}")]
		[ProducesResponseType(typeof(GetArtifactResponse), 200)]
		public async Task<ActionResult<object>> GetArtifactAsync(ArtifactId id, [FromQuery] PropertyFilter? filter = null)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, HttpContext.RequestAborted);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			return PropertyFilter.Apply(new GetArtifactResponse(artifact), filter);
		}

		/// <summary>
		/// Deletes an artifact object
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpDelete]
		[Route("/api/v2/artifacts/{id}")]
		public async Task<ActionResult> DeleteArtifactAsync(ArtifactId id, CancellationToken cancellationToken)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, HttpContext.RequestAborted);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.DeleteArtifact, User))
			{
				return Forbid(ArtifactAclAction.DeleteArtifact, artifact.StreamId);
			}

			await artifact.DeleteAsync(cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Retrieves bundles for a particular artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="locator">The blob locator</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/blobs/{*locator}")]
		public async Task<ActionResult> ReadArtifactBlobAsync(ArtifactId id, BlobLocator locator, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}
			if (!locator.WithinFolder(new Utf8String(ArtifactCollection.GetArtifactPath(artifact.StreamId, artifact.Type))) && !locator.WithinFolder(artifact.RefName.Text))
			{
				return BadRequest("Invalid blob id for artifact");
			}

			IStorageBackend storageBackend = _storageService.CreateBackend(artifact.NamespaceId);
			return await StorageController.ReadBlobInternalAsync(storageBackend, locator, Request.Headers, cancellationToken);
		}

		/// <summary>
		/// Retrieves the root blob for an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/refs/default")]
		public async Task<ActionResult<ReadRefResponse>> ReadArtifactRefAsync(ArtifactId id, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			return await StorageController.ReadRefInternalAsync(_storageService, $"/api/v2/artifacts/{id}", artifact.NamespaceId, artifact.RefName, Request.Headers, cancellationToken);
		}

		/// <summary>
		/// Gets metadata about an artifact object
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="path">Path to fetch</param>
		/// <param name="search">Optional search parameter</param>
		/// <param name="filter">Filter for returned properties</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/browse")]
		[ProducesResponseType(typeof(GetArtifactDirectoryResponse), 200)]
		public async Task<ActionResult<object>> BrowseArtifactAsync(ArtifactId id, [FromQuery] string? path = null, [FromQuery] string? search = null, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			IStorageNamespace storageNamespace = _storageService.GetNamespace(artifact.NamespaceId);

			DirectoryNode directoryNode;
			try
			{
				directoryNode = await storageNamespace.ReadRefTargetAsync<DirectoryNode>(artifact.RefName, DateTime.UtcNow.AddHours(1.0), cancellationToken: cancellationToken);
			}
			catch (RefNameNotFoundException)
			{
				return NotFound(artifact.NamespaceId, artifact.RefName);
			}

			if (path != null)
			{
				foreach (string fragment in path.Split('/'))
				{
					DirectoryEntry? nextDirectoryEntry;
					if (!directoryNode.TryGetDirectoryEntry(fragment, out nextDirectoryEntry))
					{
						return NotFound();
					}
					directoryNode = await nextDirectoryEntry.Handle.ReadBlobAsync(cancellationToken: cancellationToken);
				}
			}

			GetArtifactDirectoryResponse response = new GetArtifactDirectoryResponse();
			if (String.IsNullOrEmpty(search))
			{
				await ExpandDirectoriesAsync(directoryNode, 0, response, cancellationToken);
			}
			else
			{
				await SearchDirectoriesAsync(directoryNode, search, response, cancellationToken);
			}
			return PropertyFilter.Apply(response, filter);
		}

		static async Task ExpandDirectoriesAsync(DirectoryNode directoryNode, int depth, GetArtifactDirectoryResponse response, CancellationToken cancellationToken)
		{
			foreach (DirectoryEntry subDirectoryEntry in directoryNode.Directories)
			{
				DirectoryNode subDirectoryNode = await subDirectoryEntry.Handle.ReadBlobAsync(cancellationToken: cancellationToken);

				GetArtifactDirectoryEntryResponse subDirectoryEntryResponse = new GetArtifactDirectoryEntryResponse(subDirectoryEntry.Name.ToString(), subDirectoryEntry.Length, subDirectoryEntry.Handle.Hash);

				if (IncludeInlineResponse(depth, subDirectoryNode.Directories.Count, subDirectoryNode.Files.Count))
				{
					await ExpandDirectoriesAsync(subDirectoryNode, depth + 1, subDirectoryEntryResponse, cancellationToken);
				}

				response.Directories ??= new List<GetArtifactDirectoryEntryResponse>();
				response.Directories.Add(subDirectoryEntryResponse);
			}

			foreach (FileEntry fileEntry in directoryNode.Files)
			{
				response.Files ??= new List<GetArtifactFileEntryResponse>();
				response.Files.Add(new GetArtifactFileEntryResponse(fileEntry.Name.ToString(), fileEntry.Length, fileEntry.Hash));
			}
		}

		static async Task SearchDirectoriesAsync(DirectoryNode directoryNode, string search, GetArtifactDirectoryResponse response, CancellationToken cancellationToken)
		{
			await SearchDirectoriesFullAsync(directoryNode, "", search, response, cancellationToken);
			FilterInlineResponses(0, response);
		}

		static async Task SearchDirectoriesFullAsync(DirectoryNode directoryNode, string path, string search, GetArtifactDirectoryResponse response, CancellationToken cancellationToken)
		{
			foreach (DirectoryEntry subDirectoryEntry in directoryNode.Directories)
			{
				DirectoryNode subDirectoryNode = await subDirectoryEntry.Handle.ReadBlobAsync(cancellationToken: cancellationToken);

				GetArtifactDirectoryEntryResponse subDirectoryEntryResponse = new GetArtifactDirectoryEntryResponse(subDirectoryEntry.Name.ToString(), subDirectoryEntry.Length, subDirectoryEntry.Handle.Hash);
				await SearchDirectoriesFullAsync(subDirectoryNode, $"{path}/{subDirectoryEntry.Name}", search, subDirectoryEntryResponse, cancellationToken);

				if ((subDirectoryEntryResponse.Files?.Count ?? 0) > 0 || (subDirectoryEntryResponse.Directories?.Count ?? 0) > 0)
				{
					response.Directories ??= new List<GetArtifactDirectoryEntryResponse>();
					response.Directories.Add(subDirectoryEntryResponse);
				}
			}

			foreach (FileEntry fileEntry in directoryNode.Files)
			{
				string filePath = $"{path}/{fileEntry.Name}";
				if (filePath.Contains(search, StringComparison.OrdinalIgnoreCase))
				{
					response.Files ??= new List<GetArtifactFileEntryResponse>();
					response.Files.Add(new GetArtifactFileEntryResponse(fileEntry.Name.ToString(), fileEntry.Length, fileEntry.Hash));
				}
			}
		}

		static void FilterInlineResponses(int depth, GetArtifactDirectoryResponse response)
		{
			if (response.Directories != null)
			{
				foreach (GetArtifactDirectoryResponse subDirResponse in response.Directories)
				{
					if (IncludeInlineResponse(depth, subDirResponse.Directories?.Count ?? 0, subDirResponse.Files?.Count ?? 0))
					{
						FilterInlineResponses(depth + 1, subDirResponse);
					}
					else
					{
						subDirResponse.Directories = null;
						subDirResponse.Files = null;
					}
				}
			}
		}

		static bool IncludeInlineResponse(int depth, int numDirectories, int numFiles)
		{
			bool result = false;
			if (depth == 0)
			{
				result = (numDirectories + numFiles) < 16;
			}
			else if (depth == 1)
			{
				result = (numDirectories + numFiles) < 8;
			}
			else if (depth < 10)
			{
				result = (numDirectories == 1 && numFiles == 0);
			}
			return result;
		}

		/// <summary>
		/// Browse to an individual file from an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="path">Path to fetch</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/browse/{*path}")]
		public Task<ActionResult<object>> BrowseFileAsync(ArtifactId id, string path, CancellationToken cancellationToken = default)
		{
			return GetFileAsync(id, path, inline: true, cancellationToken);
		}

		/// <summary>
		/// Downloads an individual file from an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="path">Path to fetch</param>
		/// <param name="inline">Whether to request the file be downloaded vs displayed inline</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/file")]
		public async Task<ActionResult<object>> GetFileAsync(ArtifactId id, [FromQuery] string path, [FromQuery] bool inline = false, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			IStorageNamespace storageNamespace = _storageService.GetNamespace(artifact.NamespaceId);
			DirectoryNode directory = await storageNamespace.ReadRefTargetAsync<DirectoryNode>(artifact.RefName, DateTime.UtcNow.AddHours(1.0), cancellationToken: cancellationToken);

			FileEntry? fileEntry = await directory.GetFileEntryByPathAsync(path, cancellationToken: cancellationToken);
			if (fileEntry == null)
			{
				return NotFound($"Unable to find file {path}");
			}

			string? contentType;
			if (!new FileExtensionContentTypeProvider().TryGetContentType(path, out contentType))
			{
				if (path.EndsWith(".log", StringComparison.OrdinalIgnoreCase))
				{
					contentType = MediaTypeNames.Text.Plain;
				}
				else
				{
					contentType = MediaTypeNames.Application.Octet;
				}
			}

			Stream stream = fileEntry.OpenAsStream();
			if (inline)
			{
				return new InlineFileStreamResult(stream, contentType, Path.GetFileName(path));
			}
			else
			{
				return new FileStreamResult(stream, contentType) { FileDownloadName = Path.GetFileName(path) };
			}
		}

		/// <summary>
		/// Downloads the artifact data
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="format">Format for the download type</param>
		/// <param name="filter">Paths to include. The post version of this request allows for more parameters than can fit in a request string.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/download")]
		public async Task<ActionResult<object>> DownloadAsync(ArtifactId id, [FromQuery] DownloadArtifactFormat? format, [FromQuery(Name = "filter")] string[]? filter, CancellationToken cancellationToken = default)
		{
			return await DownloadInternalAsync(id, format, filter, cancellationToken);
		}

		/// <summary>
		/// Downloads an individual file from an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="format">Format for the download type</param>
		/// <param name="request">Filter for the zip file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpPost]
		[Route("/api/v2/artifacts/{id}/download")]
		public async Task<ActionResult<object>> DownloadWithFilterAsync(ArtifactId id, [FromQuery] DownloadArtifactFormat? format, CreateZipRequest request, CancellationToken cancellationToken = default)
		{
			return await DownloadInternalAsync(id, format, request.Filter, cancellationToken);
		}

		async Task<ActionResult> DownloadInternalAsync(ArtifactId id, DownloadArtifactFormat? format, IReadOnlyList<string>? fileFilter, CancellationToken cancellationToken)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			switch (format ?? DownloadArtifactFormat.Zip)
			{
				case DownloadArtifactFormat.Zip:
					return await GetZipInternalAsync(artifact, fileFilter, cancellationToken);
				case DownloadArtifactFormat.Ugs:
					return GetDescriptorInternal(artifact, fileFilter);
				default:
					return BadRequest("Unhandled download format");
			}
		}

		async Task<ActionResult> GetZipInternalAsync(IArtifact artifact, IEnumerable<string>? fileFilter, CancellationToken cancellationToken)
		{
			FileFilter? filter = null;
			if (fileFilter != null && fileFilter.Any())
			{
				filter = new FileFilter(fileFilter);
			}

#pragma warning disable CA2000
			IStorageNamespace storageNamespace = _storageService.GetNamespace(artifact.NamespaceId);
			IHashedBlobRef<DirectoryNode> directory = await storageNamespace.ReadRefAsync<DirectoryNode>(artifact.RefName, DateTime.UtcNow.AddHours(1.0), cancellationToken: cancellationToken);

			Stream stream = directory.AsZipStream(filter);
			return new FileStreamResult(stream, "application/zip") { FileDownloadName = $"{artifact.RefName}.zip" };
#pragma warning restore CA2000
		}

		ActionResult GetDescriptorInternal(IArtifact artifact, IReadOnlyList<string>? fileFilter)
		{
			Uri? baseUri = null;

			// For Zen backend need to pull the build uri from attached metadata
			bool zen = false;
			string? backend = artifact.Metadata.FirstOrDefault(x => x.StartsWith("Backend=", StringComparison.OrdinalIgnoreCase));
			if (!String.IsNullOrEmpty(backend) && backend.EndsWith("=Zen", StringComparison.OrdinalIgnoreCase))
			{
				zen = true;
				string? zenBuildUri = artifact.Metadata.FirstOrDefault(x => x.StartsWith("ZenBuildUri=", StringComparison.OrdinalIgnoreCase));
				if (zenBuildUri != null)
				{
					string[] parts = zenBuildUri.Split('=');
					if (parts.Length == 2 && parts[1].Length > 0)
					{
						baseUri = new Uri(parts[1]);
					}
				}
			}			
			
			if (baseUri == null)
			{ 
				if (zen)
				{
					return NotFound($"Unable to resolve zen uri for artifact {artifact.Id}");
				}

				baseUri = new Uri(_serverInfo.ServerUrl, $"api/v2/artifacts/{artifact.Id}");
			}

			ArtifactDescriptor descriptor = new ArtifactDescriptor(baseUri, new RefName("default"), fileFilter);
			descriptor.Name = artifact.Name.ToString();
			descriptor.Type = artifact.Type.ToString();
			descriptor.Description = artifact.Description;
			descriptor.Keys = new List<string>(artifact.Keys);
			descriptor.Metadata = new List<string>(artifact.Metadata);
			descriptor.Backend = new ArtifactBackend { Type = zen ? ArtifactBackend.BackendTypeEnum.Zen : ArtifactBackend.BackendTypeEnum.Horde };

			foreach (string key in artifact.Keys)
			{
				Match match = Regex.Match(key, @"^job:([0-9a-zA-Z]{24})$");
				if (match.Success)
				{
					descriptor.JobUrl = new Uri(_serverInfo.DashboardUrl, $"job/{match.Groups[1].Value}");
					break;
				}
			}

			byte[] data = descriptor.Serialize();
			return new FileStreamResult(new MemoryStream(data), "application/x-horde-artifact") { FileDownloadName = $"{artifact.Name}.uartifact" };
		}

		class ManifestStreamResult : FileStreamResult
		{
			public ManifestStreamResult(System.IO.Stream stream, string mimeType)
				: base(stream, mimeType)
			{
			}

			/// <inheritdoc/>
			public override Task ExecuteResultAsync(ActionContext context)
			{
				context.HttpContext.Response.Headers["Content-Disposition"] = new ContentDisposition { Inline = true }.ToString();
				return base.ExecuteResultAsync(context);
			}
		}
		/// <summary>
		/// Creates an Unsync manifest for an artifact
		/// </summary>
		/// <param name="id">The artifact id</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>An unsync manifest stream</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/unsync")]
		public async Task<ActionResult> GetUnsyncManifestAsync(ArtifactId id, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			ReadOnlyMemory<byte> manifestData = await _unsyncCache.GetManifestDataAsync(artifact, cancellationToken);
			if (manifestData.IsEmpty)
			{
				return NotFound(id);
			}

#pragma warning disable CA2000
			return new ManifestStreamResult(new ReadOnlyMemoryStream(manifestData), "application/json");
#pragma warning restore CA2000
		}

		/// <summary>
		/// Creates an Unsync manifest for an artifact
		/// </summary>
		/// <param name="id">The artifact id</param>
		/// <param name="request">Information about the blobs to return</param>
		/// <param name="compress">Whether to compress the output data</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>An unsync manifest stream</returns>
		[HttpPost]
		[Route("/api/v2/artifacts/{id}/unsync-blobs")]
		public async Task<ActionResult> GetUnsyncBlobsAsync(ArtifactId id, [FromBody] GetUnsyncDataRequest request, [FromQuery] bool compress = true, CancellationToken cancellationToken = default)
		{
			IArtifact? artifact = await _artifactCollection.GetAsync(id, cancellationToken);
			if (artifact == null)
			{
				return NotFound(id);
			}
			if (!_buildConfig.AuthorizeArtifact(artifact.Type, artifact.StreamId, ArtifactAclAction.ReadArtifact, User))
			{
				return Forbid(ArtifactAclAction.ReadArtifact, artifact.StreamId);
			}

			if (!String.Equals(request.HashStrong, "Blake3.160", StringComparison.OrdinalIgnoreCase))
			{
				return BadRequest($"Unsupported hash algorithm: {request.HashStrong}");
			}

			Stopwatch totalResponseTimer = Stopwatch.StartNew();

			// Disable buffering for the response
			IHttpResponseBodyFeature? responseBodyFeature = HttpContext.Features.Get<IHttpResponseBodyFeature>();
			responseBodyFeature?.DisableBuffering();

			// Merge the legacy block list with the regular one
			foreach (GetUnsyncFileRequest file in request.Files)
			{
				foreach (GetUnsyncBlockRequest block in file.Blocks)
				{
					if (block.Hash != null)
					{
						request.Blocks.Add(block.Hash);
					}
				}
			}

			// Send the response headers
			HttpResponse response = HttpContext.Response;
			response.ContentType = "application/x-horde-unsync-blob";
			response.Headers["x-chunk-content-encoding"] = compress ? "zstd" : "identity";
			response.StatusCode = (int)HttpStatusCode.OK;

			await response.StartAsync(cancellationToken);
			try
			{
				// Create a pipeline for downloading new blocks
				await using BlobPipeline<IoHash> pipeline = new BlobPipeline<IoHash>();

				// Cache of blocks that have already been extracted and compressed
				long cachedResponseLength = 0;
				double cachedResponseTime = 0.0;
				List<(IoHash, IBlockCacheValue)> cachedBlocks = new List<(IoHash, IBlockCacheValue)>();
				try
				{
					// Find all the blob refs that we need to fetch
					foreach (string block in request.Blocks)
					{
						if (!IoHash.TryParse(block, out IoHash hash))
						{
							return BadRequest($"Invalid IoHash value: {block}");
						}

						IHashedBlobRef? blobRef = await _unsyncCache.ReadBlobRefAsync(artifact, hash, cancellationToken);
						if (blobRef == null)
						{
							return NotFound($"Hash '{hash}' is not part of artifact {artifact.Id}");
						}

						IBlockCacheValue? cacheValue = _blockCache.Get(GetUnsyncBlockKey(hash, compress));
						if (cacheValue != null)
						{
							cachedBlocks.Add((hash, cacheValue));
						}
						else
						{
							pipeline.Add(new BlobRequest<IoHash>(blobRef, hash));
						}
					}
					pipeline.FinishAdding();

					// Write all the cached blocks to the response
					Stopwatch cachedResponseTimer = Stopwatch.StartNew();
					foreach ((IoHash hash, IBlockCacheValue value) in cachedBlocks)
					{
						int length = (int)value.Data.Length;

						Memory<byte> buffer = response.BodyWriter.GetMemory(length);
						value.Data.CopyTo(buffer.Span);
						response.BodyWriter.Advance(length);
						cachedResponseLength += length;

						await response.BodyWriter.FlushAsync(cancellationToken);
					}
					cachedResponseTime = cachedResponseTimer.Elapsed.TotalSeconds;
				}
				finally
				{
					foreach ((_, IBlockCacheValue value) in cachedBlocks)
					{
						value.Dispose();
					}
				}

				// Read all the uncached blocks
				long uncachedResponseLength = 0;
				double uncachedResponseTime = 0.0;
				if (cachedBlocks.Count < request.Blocks.Count)
				{
					Stopwatch uncachedResponseTimer = Stopwatch.StartNew();

					ArrayMemoryWriter blockWriter = new ArrayMemoryWriter(128 * 1024);
					await foreach (BlobResponse<IoHash> blobResponse in pipeline.ReadAllAsync(cancellationToken))
					{
						using BlobData blobData = blobResponse.BlobData;
						blockWriter.Clear();

						// Skip the header to start with. We'll write it once we know the compressed size
						blockWriter.Advance(BlockHeaderSize);

						// Write the payload
						if (compress)
						{
							BundleData.Compress(BundleCompressionFormat.Zstd, blobData.Data, blockWriter);
						}
						else
						{
							blockWriter.WriteFixedLengthBytes(blobData.Data.Span);
						}

						// Write the header
						WriteBlockHeader(blockWriter.WrittenSpan, blockWriter.WrittenSpan.Length - BlockHeaderSize, blobResponse.UserData, blobData.Data.Length);

						// Add it to the cache for future requests
						string cacheKey = GetUnsyncBlockKey(blobResponse.UserData, compress);
						_blockCache.Add(cacheKey, blockWriter.WrittenMemory);

						// Copy it to the response output
						await response.BodyWriter.WriteAsync(blockWriter.WrittenMemory, cancellationToken);
						await response.BodyWriter.FlushAsync(cancellationToken);
						uncachedResponseLength += blockWriter.WrittenMemory.Length;
					}

					uncachedResponseTime = uncachedResponseTimer.Elapsed.TotalSeconds;
				}

				long totalResponseLength = cachedResponseLength + uncachedResponseLength;
				double totalResponseTime = totalResponseTimer.Elapsed.TotalSeconds;
				_logger.LogInformation("Unsync request for {NumBlocks} blocks ({Type}, {TotalSize:n1}mb, {TotalSpeed:n1}mb/s); {NumCachedBlocks} cached ({CachedSize:n1}mb, {CachedSpeed:n1}mb/s), {NumUncachedBlocks} uncached ({UncachedSize:n1}mb, {UncachedSpeed:n1}mb/s)",
					request.Blocks.Count,
					compress ? "compressed" : "not compressed",
					totalResponseLength / (1024.0 * 1024.0),
					(totalResponseTime > 0.0) ? (totalResponseLength / (1024 * 1024 * totalResponseTime)) : 0.0,

					cachedBlocks.Count,
					cachedResponseLength / (1024.0 * 1024.0),
					(cachedResponseTime > 0.0) ? (cachedResponseLength / (1024 * 1024 * cachedResponseTime)) : 0.0,

					request.Blocks.Count - cachedBlocks.Count,
					uncachedResponseLength / (1024.0 * 1024.0),
					(uncachedResponseTime > 0.0) ? (uncachedResponseLength / (1024 * 1024 * uncachedResponseTime)) : 0.0
				);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while extracting unsync blobs: {Message}", ex.Message);

				ArrayMemoryWriter writer = new ArrayMemoryWriter(1024);
				WriteErrorHeader(writer, $"Error retrieving blobs: {ex.Message}");

				await response.BodyWriter.WriteAsync(writer.WrittenMemory, cancellationToken);
				await response.BodyWriter.FlushAsync(cancellationToken);
			}

			// Finish the response
			await response.CompleteAsync();

			return Empty;
		}

		static string GetUnsyncBlockKey(IoHash hash, bool compressed)
			=> compressed ? $"unsync-zstd:{hash}" : $"unsync:{hash}";

		const int BlockHeaderSize = (sizeof(long) * 3) + IoHash.NumBytes;

		static void WriteBlockHeader(Span<byte> data, long compressedSize, IoHash decompressedHash, long decompressedSize)
		{
			BinaryPrimitives.WriteUInt64LittleEndian(data, 0x_4C5C_2AAB_A992_610C);
			data = data.Slice(8);

			BinaryPrimitives.WriteUInt64LittleEndian(data, (ulong)compressedSize);
			data = data.Slice(8);

			BinaryPrimitives.WriteUInt64LittleEndian(data, (ulong)decompressedSize);
			data = data.Slice(8);

			decompressedHash.CopyTo(data);
		}

		static void WriteErrorHeader(ArrayMemoryWriter writer, string message)
		{
			Utf8String messageUtf8 = new Utf8String(message);
			writer.WriteUInt64(0x_4C5C_2AAB_A992_DEAD);
			writer.WriteUInt32((uint)(messageUtf8.Length + 1));
			writer.WriteNullTerminatedUtf8String(messageUtf8);
		}

		/// <summary>
		/// Downloads an individual file from an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="filter">Paths to include in the zip file. The post version of this request allows for more parameters than can fit in a request string.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpGet]
		[Route("/api/v2/artifacts/{id}/zip")]
		public async Task<ActionResult> GetZipAsync(ArtifactId id, [FromQuery(Name = "filter")] string[]? filter, CancellationToken cancellationToken = default)
		{
			return await DownloadInternalAsync(id, DownloadArtifactFormat.Zip, filter, cancellationToken);
		}

		/// <summary>
		/// Downloads an individual file from an artifact
		/// </summary>
		/// <param name="id">Identifier of the artifact to retrieve</param>
		/// <param name="request">Filter for the zip file</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the artifacts</returns>
		[HttpPost]
		[Route("/api/v2/artifacts/{id}/zip")]
		public async Task<ActionResult<object>> CreateZipFromFilterAsync(ArtifactId id, CreateZipRequest request, CancellationToken cancellationToken = default)
		{
			return await DownloadInternalAsync(id, DownloadArtifactFormat.Zip, request.Filter, cancellationToken);
		}
	}
}
