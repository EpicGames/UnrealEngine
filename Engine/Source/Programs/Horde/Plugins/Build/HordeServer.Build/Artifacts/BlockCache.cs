// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;
using System.Buffers.Binary;
using System.Collections.Concurrent;
using System.Diagnostics;
using System.IO.Hashing;
using System.IO.MemoryMappedFiles;
using System.Numerics;
using System.Text;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Stats for the cache
	/// </summary>
	public record class BlockCacheStats(int NumItems, int NumBlocks, int NumFreeBlocks, int BlockSize, long TotalSize, long UsedSize, long AllocatedSize);

	/// <summary>
	/// Implements a simple key-value cache using memory mapped files.
	/// </summary>
	public sealed class BlockCache : IBlockCache, IDisposable
	{
		// Size of a page for allocations
		const int PageSizeLog2 = 10;
		const int PageSize = 1 << PageSizeLog2; // 4kb

		// Size of a block; the smallest chunk of data that can be allocated
		const int DefaultBlockSize = 32768;

		// Default number of blocks in each partition
		const int DefaultNumBlocksPerPartition = 32768;

		// When creating free space in the cache, define the number of samples to take, and the number of entries to remove from the examined items.
		const int NumEvictionSamples = 10;
		const int NumEvictionEntries = 4;

		// Header information for a block
		record struct BlockHeader(IoHash Key, uint PackedData, long Digest)
		{
			public const int NumBytes = 32;

			public int Index => (int)((PackedData >> 24) & 0xff);
			public int Count => (int)((PackedData >> 16) & 0xff);
			public int Length => ((int)(PackedData & 0xffff)) + 1;

			public BlockHeader(IoHash key, int index, int count, int length, long digest)
				: this(key, ((uint)index << 24) | ((uint)count << 16) | (uint)(length - 1), digest)
			{ }

			public static BlockHeader Read(ReadOnlySpan<byte> data)
			{
				IoHash hash = new IoHash(data);
				uint packed = BinaryPrimitives.ReadUInt32LittleEndian(data[IoHash.NumBytes..]);
				long digest = BinaryPrimitives.ReadInt64LittleEndian(data[(IoHash.NumBytes + 4)..]);
				return new BlockHeader(hash, packed, digest);
			}

			public void Write(Span<byte> data)
			{
				Key.CopyTo(data);
				BinaryPrimitives.WriteUInt32LittleEndian(data[IoHash.NumBytes..], PackedData);
				BinaryPrimitives.WriteInt64LittleEndian(data[(IoHash.NumBytes + 4)..], Digest);
			}
		}

		class Partition : IDisposable
		{
			public Memory<byte> BlockHeaders { get; }
			public Memory<byte> BlockData { get; }

			public Partition(int numBlocks, Memory<byte> data)
			{
				Memory<byte> remainingData = data;

				BlockHeaders = remainingData.Slice(0, BlockHeader.NumBytes * numBlocks);
				remainingData = remainingData.Slice(PageAlign(BlockHeaders.Length));

				BlockData = remainingData;
			}

			public virtual void Dispose()
			{ }
		}

		class MemoryMappedFilePartition : Partition
		{
			readonly MemoryMappedFile _memoryMappedFile;
			readonly MemoryMappedView _memoryMappedView;

			private MemoryMappedFilePartition(MemoryMappedFile memoryMappedFile, MemoryMappedView memoryMappedView, int numBlocks, Memory<byte> data)
				: base(numBlocks, data)
			{
				_memoryMappedFile = memoryMappedFile;
				_memoryMappedView = memoryMappedView;
			}

			public static MemoryMappedFilePartition Create(FileReference file, int numBlocks, int blockSize)
			{
				int partitionSize = GetPartitionSize(numBlocks, blockSize);

				MemoryMappedFile memoryMappedFile = MemoryMappedFile.CreateFromFile(file.FullName, FileMode.OpenOrCreate, null, partitionSize, MemoryMappedFileAccess.ReadWrite);
				MemoryMappedView memoryMappedView = new MemoryMappedView(memoryMappedFile, 0, partitionSize);

				return new MemoryMappedFilePartition(memoryMappedFile, memoryMappedView, numBlocks, memoryMappedView.GetMemory(0, partitionSize));
			}

			public override void Dispose()
			{
				base.Dispose();

				_memoryMappedFile.Dispose();
				_memoryMappedView.Dispose();
			}
		}

		class BlockCacheValue : IBlockCacheValue
		{
			ReadOnlySequence<byte> _data;
			BlockState? _state;

			public BlockCacheValue(ReadOnlySequence<byte> data, BlockState state)
			{
				_data = data;
				_state = state;
			}

			public ReadOnlySequence<byte> Data => _data;

			public void Dispose()
			{
				_data = ReadOnlySequence<byte>.Empty;

				if (_state != null)
				{
					_state.ReleaseReadLock();
					_state = null;
				}
			}
		}

		class BlockState
		{
			public int[] BlockIdxs { get; }
			public long LastAccessTicks { get; private set; }
			public int LastBlockLength { get; }

			int _readerCount;

			public BlockState(int[] blockIdxs, int lastBlockLength)
			{
				BlockIdxs = blockIdxs;
				LastAccessTicks = Stopwatch.GetTimestamp();
				LastBlockLength = lastBlockLength;
			}

			public void Touch()
				=> LastAccessTicks = Stopwatch.GetTimestamp();

			public bool TryAddReadLock()
			{
				int lockCount = _readerCount;
				while (lockCount >= 0)
				{
					if (Interlocked.CompareExchange(ref _readerCount, lockCount + 1, lockCount) == lockCount)
					{
						return true;
					}
					lockCount = Interlocked.CompareExchange(ref _readerCount, 0, 0);
				}
				return false;
			}

			public bool TryAddWriteLock()
				=> Interlocked.CompareExchange(ref _readerCount, -1, 0) == 0;

			public void ReleaseReadLock()
				=> Interlocked.Decrement(ref _readerCount);
		}

		readonly Random _rng = new Random(0);
		readonly Partition[] _partitions;
		readonly int _numBlocks;
		readonly int _numBlocksPerPartition;
		readonly int _numBlocksPerPartitionLog2;
		readonly int _blockSize;
		readonly int _blockSizeLog2;
		readonly ConcurrentDictionary<IoHash, BlockState> _lookup = new ConcurrentDictionary<IoHash, BlockState>();
		readonly ConcurrentQueue<int> _freeBlocks = new ConcurrentQueue<int>();
		readonly ILogger _logger;

		int _numAllocatedBlocks = 0;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="partitions">Allocated partition data</param>
		/// <param name="numBlocksPerPartition">Number of blocks in each partition</param>
		/// <param name="blockSize">The size of each block. Must be a power of two.</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		BlockCache(Partition[] partitions, int numBlocksPerPartition, int blockSize, ILogger? logger)
		{
			if ((numBlocksPerPartition & (numBlocksPerPartition - 1)) != 0)
			{
				throw new ArgumentException("Number of blocks per partition must be a power of two", nameof(numBlocksPerPartition));
			}
			if ((blockSize & (blockSize - 1)) != 0)
			{
				throw new ArgumentException("Block size must be a power of two", nameof(blockSize));
			}

			_partitions = partitions;
			_numBlocks = partitions.Length * numBlocksPerPartition;
			_numBlocksPerPartition = numBlocksPerPartition;
			_numBlocksPerPartitionLog2 = BitOperations.Log2((uint)numBlocksPerPartition);
			_blockSize = blockSize;
			_blockSizeLog2 = BitOperations.Log2((uint)blockSize);
			_logger = logger ?? NullLogger.Instance;
		}

		/// <summary>
		/// Creates a new cache from memory
		/// </summary>
		/// <param name="numPartitions">Number of partitions in the cache</param>
		/// <param name="numBlocksPerPartition">Number of blocks in each partition</param>
		/// <param name="blockSize">Size of each block</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public static BlockCache CreateInMemory(int numPartitions, int numBlocksPerPartition = DefaultNumBlocksPerPartition, int blockSize = DefaultBlockSize, ILogger? logger = null)
		{
			int partitionSize = GetPartitionSize(numBlocksPerPartition, blockSize);

			Partition[] partitions = new Partition[numPartitions];
			for (int idx = 0; idx < numPartitions; idx++)
			{
				byte[] data = new byte[partitionSize];
				partitions[idx] = new Partition(numBlocksPerPartition, data);
			}

			return new BlockCache(partitions, numBlocksPerPartition, blockSize, logger);
		}

		/// <summary>
		/// Create a new cache backed by files on disk
		/// </summary>
		/// <param name="rootDir">Directory containing the partitions</param>
		/// <param name="numPartitions">Number of partitions to create. Each partition is ~1gb.</param>
		/// <param name="numBlocksPerPartition">Number of blocks in each partition</param>
		/// <param name="blockSize">Size of a block</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		public static BlockCache Create(DirectoryReference rootDir, int numPartitions, int numBlocksPerPartition = DefaultNumBlocksPerPartition, int blockSize = DefaultBlockSize, ILogger? logger = null)
		{
			MemoryMappedFilePartition[] partitions = new MemoryMappedFilePartition[numPartitions];
			try
			{
				DirectoryReference.CreateDirectory(rootDir);
				for (int idx = 0; idx < numPartitions; idx++)
				{
					FileReference file = FileReference.Combine(rootDir, $"partition{idx:0000}.dat");
					partitions[idx] = MemoryMappedFilePartition.Create(file, numBlocksPerPartition, blockSize);
				}
				return new BlockCache(partitions, numBlocksPerPartition, blockSize, logger);
			}
			catch
			{
				for (int idx = 0; idx < numPartitions; idx++)
				{
					partitions[idx]?.Dispose();
				}
				throw;
			}
		}

		static int GetPartitionSize(int numBlocks, int blockSize)
			=> PageAlign(numBlocks * BlockHeader.NumBytes) + (numBlocks * blockSize);

		static int PageAlign(int value)
			=> (value + (PageSize - 1)) & ~(PageSize - 1);

		/// <inheritdoc/>
		public void Dispose()
		{
			foreach (Partition partition in _partitions)
			{
				partition.Dispose();
			}
		}

		/// <summary>
		/// Gets stats for the cache
		/// </summary>
		public BlockCacheStats GetStats()
		{
			long usedSize = 0;
			long allocatedSize = 0;

			int numFreeBlocks = (_numBlocks - _numAllocatedBlocks) + _freeBlocks.Count;
			for (int blockIdx = 0; blockIdx < _numAllocatedBlocks; blockIdx++)
			{
				BlockHeader header = GetBlockHeader(blockIdx);
				if (header.Key == IoHash.Zero)
				{
					numFreeBlocks++;
				}
				else
				{
					usedSize += header.Length;
					allocatedSize += _blockSize;
				}
			}
			return new BlockCacheStats(_lookup.Count, _numBlocks, numFreeBlocks, _blockSize, _numBlocks << _blockSizeLog2, usedSize, allocatedSize);
		}

		int GetNextFreeBlockIdx()
		{
			// Try to allocate a new block
			int numAllocatedBlocks = _numAllocatedBlocks;
			while (numAllocatedBlocks < _numBlocks)
			{
				int initialNumAllocatedBlocks = Interlocked.CompareExchange(ref _numAllocatedBlocks, numAllocatedBlocks + 1, numAllocatedBlocks);
				if (initialNumAllocatedBlocks == numAllocatedBlocks)
				{
					return initialNumAllocatedBlocks;
				}
				numAllocatedBlocks = initialNumAllocatedBlocks;
			}

			// Dequeue a block from the free list
			int blockIdx;
			while (!_freeBlocks.TryDequeue(out blockIdx))
			{
				PriorityQueue<(IoHash, BlockState), long> priorityQueue = new PriorityQueue<(IoHash, BlockState), long>(NumEvictionSamples);
				lock (_rng)
				{
					for (int sampleIdx = 0; sampleIdx < NumEvictionSamples; sampleIdx++)
					{
						blockIdx = _rng.Next(_numBlocks);

						BlockHeader header = GetBlockHeader(blockIdx);
						if (_lookup.TryGetValue(header.Key, out BlockState? state))
						{
							priorityQueue.Enqueue((header.Key, state), state.LastAccessTicks);
						}
					}
				}

				for (int idx = 0; idx < NumEvictionEntries && priorityQueue.Count > 0; idx++)
				{
					(IoHash hash, BlockState evictState) = priorityQueue.Dequeue();
					TryDelete(hash, evictState);
				}
			}
			return blockIdx;
		}

		bool TryDelete(IoHash hash, BlockState blockState)
		{
			if (!blockState.TryAddWriteLock())
			{
				return false;
			}

			_lookup.TryRemove(hash, out _);
			foreach (int evictBlockIdx in blockState.BlockIdxs)
			{
				SetBlockHeader(evictBlockIdx, default);
				_freeBlocks.Enqueue(evictBlockIdx);
			}
			return true;
		}

		BlockHeader GetBlockHeader(int blockIdx)
		{
			Partition partition = _partitions[blockIdx >> _numBlocksPerPartitionLog2];
			ReadOnlySpan<byte> span = partition.BlockHeaders.Span.Slice((blockIdx & (_numBlocksPerPartition - 1)) * BlockHeader.NumBytes, BlockHeader.NumBytes);
			return BlockHeader.Read(span);
		}

		void SetBlockHeader(int blockIdx, BlockHeader header)
		{
			Partition partition = _partitions[blockIdx >> _numBlocksPerPartitionLog2];
			Span<byte> span = partition.BlockHeaders.Span.Slice((blockIdx & (_numBlocksPerPartition - 1)) * BlockHeader.NumBytes, BlockHeader.NumBytes);
			header.Write(span);
		}

		Memory<byte> GetBlockData(int blockIdx, int blockLength)
		{
			Partition partition = _partitions[blockIdx >> _numBlocksPerPartitionLog2];
			int blockIdxInPartition = blockIdx & (_numBlocksPerPartition - 1);
			return partition.BlockData.Slice(blockIdxInPartition << _blockSizeLog2, blockLength);
		}

		/// <inheritdoc/>
		public bool Add(string key, ReadOnlyMemory<byte> value)
		{
			if (value.Length == 0)
			{
				return false;
			}

			IoHash keyHash = IoHash.Compute(Encoding.UTF8.GetBytes(key));

			int numBlocks = (value.Length + (_blockSize - 1)) >> _blockSizeLog2;
			int[] blockIdxs = new int[numBlocks];

			for (int idx = 0; idx < numBlocks; idx++)
			{
				int offset = idx << _blockSizeLog2;
				int length = Math.Min(value.Length - offset, _blockSize);

				int freeBlockIdx = GetNextFreeBlockIdx();
				blockIdxs[idx] = freeBlockIdx;

				Memory<byte> blockData = GetBlockData(freeBlockIdx, length);
				value.Slice(offset, length).CopyTo(blockData);

				long digest = ComputeDigest(blockData.Span);

				BlockHeader header = new BlockHeader(keyHash, idx, numBlocks, length, digest);
				SetBlockHeader(freeBlockIdx, header);
			}

			BlockState state = new BlockState(blockIdxs, value.Length & (_blockSize - 1));
			if (!_lookup.TryAdd(keyHash, state))
			{
				foreach (int blockIdx in blockIdxs)
				{
					_freeBlocks.Enqueue(blockIdx);
				}
			}

			return true;
		}

		static long ComputeDigest(ReadOnlySpan<byte> span)
		{
			Span<byte> result = stackalloc byte[sizeof(long)];
			XxHash64.Hash(span, result);
			return BinaryPrimitives.ReadInt64BigEndian(result);
		}

		/// <summary>
		/// Gets the data associated with a given key
		/// </summary>
		/// <param name="key">Key to search for</param>
		/// <returns>Data corresponding to the given key</returns>
		public IBlockCacheValue? Get(string key)
		{
			IoHash hash = IoHash.Compute(Encoding.UTF8.GetBytes(key));

			BlockState? blockState;
			if (!_lookup.TryGetValue(hash, out blockState) || !blockState.TryAddReadLock())
			{
				return null;
			}

			blockState.Touch();

			ReadOnlySequenceBuilder<byte> builder = new ReadOnlySequenceBuilder<byte>();
			for (int idx = 0; idx < blockState.BlockIdxs.Length; idx++)
			{
				int blockIdx = blockState.BlockIdxs[idx];

				BlockHeader header = GetBlockHeader(blockIdx);
				ReadOnlyMemory<byte> data = GetBlockData(blockIdx, header.Length);

				long digest = ComputeDigest(data.Span);
				if (digest != header.Digest)
				{
					_logger.LogWarning("Corrupt block in cache '{Key}' ({Index}/{Count}) (hash: {Hash}, block {BlockIdx}, expected digest: {Digest:x8}, actual digest: {ActualDigest:x8})", key, idx + 1, blockState.BlockIdxs.Length, hash, blockIdx, header.Digest, digest);
					blockState.ReleaseReadLock();
					TryDelete(hash, blockState);
					return null;
				}

				builder.Append(data);
			}

			return new BlockCacheValue(builder.Construct(), blockState);
		}

		/// <summary>
		/// Helper method for testing corruption handling
		/// </summary>
		internal void DangerousCorruptValue(string key)
		{
			IoHash hash = IoHash.Compute(Encoding.UTF8.GetBytes(key));

			BlockState? blockState;
			if (_lookup.TryGetValue(hash, out blockState) && blockState.TryAddReadLock())
			{
				GetBlockData(blockState.BlockIdxs[0], 1).Span[0]++;
			}
		}
	}
}
