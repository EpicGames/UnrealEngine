// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Implements a key/value cache store.
	/// </summary>
	public interface IBlockCache
	{
		/// <summary>
		/// Adds a new item to the cache
		/// </summary>
		/// <param name="key">Key for finding the item</param>
		/// <param name="value">Data to be stored</param>
		/// <returns>True if the item was added, false if not</returns>
		bool Add(string key, ReadOnlyMemory<byte> value);

		/// <summary>
		/// Retrieves an item from the cache. If successful, the value must be disposed by the caller.
		/// </summary>
		/// <param name="key">Key to search for</param>
		/// <returns>Data corresponding to the given key</returns>
		IBlockCacheValue? Get(string key);
	}

	/// <summary>
	/// Handle to a value in the block cache.
	/// </summary>
	public interface IBlockCacheValue : IDisposable
	{
		/// <summary>
		/// The underlying data
		/// </summary>
		ReadOnlySequence<byte> Data { get; }
	}

	/// <summary>
	/// Extension methods for <see cref="IBlockCache"/>
	/// </summary>
	public static class BlockCacheExtensions
	{
		/// <summary>
		/// Determines if the cache contains a particular key
		/// </summary>
		public static bool Contains(this IBlockCache blockCache, string key)
		{
			using IBlockCacheValue? value = blockCache.Get(key);
			return value != null;
		}
	}
}
