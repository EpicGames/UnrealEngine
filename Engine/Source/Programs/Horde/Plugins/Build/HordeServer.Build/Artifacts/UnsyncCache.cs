// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Frozen;
using System.Diagnostics;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using HordeServer.Storage;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;

namespace HordeServer.Artifacts
{
	/// <summary>
	/// Manifest for Unsync
	/// </summary>
	/// <param name="Files">Files in the manifest</param>
	public record class UnsyncManifest(IReadOnlyList<UnsyncFile> Files);

	/// <summary>
	/// Output file for an unsync manifest
	/// </summary>
	/// <param name="Name">Full path to the file</param>
	/// <param name="ReadOnly">Whether the file is read only</param>
	/// <param name="Executable">Whether to set the executable bit on the file</param>
	/// <param name="Length">Length of the file</param>
	/// <param name="ModTime">Last modified time for the file</param>
	/// <param name="Blocks">List of blocks making up the file</param>
	public record class UnsyncFile(Utf8String Name, bool ReadOnly, bool Executable, long Length, DateTime ModTime, IReadOnlyList<UnsyncBlock> Blocks);

	/// <summary>
	/// Describes a block of data in an unsync manifest
	/// </summary>
	/// <param name="Offset">Offset within the file</param>
	/// <param name="Length">Length of the block</param>
	/// <param name="RollingHash">Rolling hash of the block</param>
	/// <param name="Blob">Handle to the corresponding blob</param>
	public record class UnsyncBlock(long Offset, long Length, uint RollingHash, IHashedBlobRef<LeafChunkedDataNode> Blob);

	/// <summary>
	/// Implements a cache for downloading Unsync blobs
	/// </summary>
	public sealed class UnsyncCache : IDisposable
	{
		class ArtifactInfo
		{
			public UnsyncManifest Manifest { get; }
			public ReadOnlyMemory<byte> ManifestData { get; }
			public FrozenDictionary<IoHash, IHashedBlobRef<LeafChunkedDataNode>> Blobs { get; }

			public ArtifactInfo(UnsyncManifest manifest, ReadOnlyMemory<byte> manifestData, FrozenDictionary<IoHash, IHashedBlobRef<LeafChunkedDataNode>> blobs)
			{
				Manifest = manifest;
				ManifestData = manifestData;
				Blobs = blobs;
			}
		}

		readonly IStorageService _storageService;
		readonly MemoryCache _cache;
		readonly object _lockObject = new object();
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UnsyncCache(IStorageService storageService, ILogger<UnsyncCache> logger)
		{
			_storageService = storageService;
			_cache = new MemoryCache(new MemoryCacheOptions());
			_logger = logger;
		}

		/// <inheritdoc/>
		public void Dispose()
			=> _cache.Dispose();

		/// <summary>
		/// Gets the manifest for an artifact
		/// </summary>
		public async ValueTask<UnsyncManifest?> GetManifestAsync(IArtifact artifact, CancellationToken cancellationToken = default)
		{
			ArtifactInfo? artifactInfo = await GetArtifactInfoAsync(artifact, cancellationToken);
			return artifactInfo?.Manifest;
		}

		/// <summary>
		/// Gets the json manifest data for an artifact
		/// </summary>
		public async ValueTask<ReadOnlyMemory<byte>> GetManifestDataAsync(IArtifact artifact, CancellationToken cancellationToken = default)
		{
			ArtifactInfo? artifactInfo = await GetArtifactInfoAsync(artifact, cancellationToken);
			if (artifactInfo == null)
			{
				return default;
			}

			return artifactInfo.ManifestData;
		}

		async Task<ArtifactInfo?> GetArtifactInfoAsync(IArtifact artifact, CancellationToken cancellationToken = default)
		{
			// Get the artifact info
			BackgroundTask<ArtifactInfo?>? artifactInfoTask;
			if (!_cache.TryGetValue(artifact.Id, out artifactInfoTask) || artifactInfoTask == null)
			{
				lock (_lockObject)
				{
					if (!_cache.TryGetValue(artifact.Id, out artifactInfoTask) || artifactInfoTask == null)
					{
						artifactInfoTask = BackgroundTask.StartNew(ctx => ReadArtifactAsync(artifact, ctx));
						using (ICacheEntry entry = _cache.CreateEntry(artifact.Id))
						{
							entry.SetSize(1);
							entry.SetSlidingExpiration(TimeSpan.FromHours(1.0));
							entry.SetValue(artifactInfoTask);
						}
					}
				}
			}

			// Wait for the read to finish
			Task<ArtifactInfo?> task = artifactInfoTask.Task ?? Task.FromResult<ArtifactInfo?>(null);
			return await task.WaitAsync(cancellationToken);
		}

		static byte[] SerializeManifest(UnsyncManifest manifest)
		{
			using (ChunkedMemoryWriter chunkedMemoryWriter = new ChunkedMemoryWriter(4096))
			{
				using (Utf8JsonWriter writer = new Utf8JsonWriter(chunkedMemoryWriter))
				{
					writer.WriteStartObject();
					writer.WriteString("type", "unsync_manifest");
					writer.WriteString("hash_strong", "Blake3.160");
					writer.WriteString("chunking", "RollingBuzHash");
					writer.WriteNumber("chunking_block_size_min", LeafChunkedDataNodeOptions.Default.MinSize);
					writer.WriteNumber("chunking_block_size_max", LeafChunkedDataNodeOptions.Default.MaxSize);
					writer.WriteNumber("chunking_block_size_target", LeafChunkedDataNodeOptions.Default.TargetSize);

					writer.WriteStartArray("files");
					foreach (UnsyncFile file in manifest.Files)
					{
						writer.WriteStartObject();
						writer.WriteString("name", file.Name);
						if (file.Executable)
						{
							writer.WriteBoolean("exec", file.Executable);
						}
						if (file.ReadOnly)
						{
							writer.WriteBoolean("read_only", file.ReadOnly);
						}
						if (file.ModTime != default)
						{
							writer.WriteNumber("mtime", file.ModTime.ToFileTimeUtc());
						}
						writer.WriteNumber("size", file.Length);
						writer.WriteNumber("block_size", LeafChunkedDataNodeOptions.Default.TargetSize);
						writer.WriteStartArray("blocks");
						foreach (UnsyncBlock block in file.Blocks)
						{
							writer.WriteStartObject();
							writer.WriteNumber("offset", block.Offset);
							writer.WriteNumber("size", block.Length);
							writer.WriteNumber("hash_weak", block.RollingHash);
							writer.WriteString("hash_strong", block.Blob.Hash.ToString());
							writer.WriteEndObject();
						}
						writer.WriteEndArray();
						writer.WriteEndObject();
					}
					writer.WriteEndArray();

					writer.WriteEndObject();
				}
				return chunkedMemoryWriter.ToByteArray();
			}
		}

		/// <summary>
		/// Reads a blob from an artifact
		/// </summary>
		/// <param name="artifact"></param>
		/// <param name="blobHash"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async ValueTask<IHashedBlobRef?> ReadBlobRefAsync(IArtifact artifact, IoHash blobHash, CancellationToken cancellationToken = default)
		{
			ArtifactInfo? artifactInfo = await GetArtifactInfoAsync(artifact, cancellationToken);
			if (artifactInfo == null)
			{
				return null;
			}

			IHashedBlobRef<LeafChunkedDataNode>? blobRef;
			if (!artifactInfo.Blobs.TryGetValue(blobHash, out blobRef))
			{
				return null;
			}

			return blobRef;
		}

		async Task<ArtifactInfo?> ReadArtifactAsync(IArtifact artifact, CancellationToken cancellationToken)
		{
			Stopwatch timer = Stopwatch.StartNew();
			IStorageNamespace storageNamespace = _storageService.GetNamespace(artifact.NamespaceId);

			IHashedBlobRef<DirectoryNode>? target = await storageNamespace.TryReadRefAsync<DirectoryNode>(artifact.RefName, cancellationToken: cancellationToken);
			if (target == null)
			{
				return null;
			}

			List<UnsyncFile> files = new List<UnsyncFile>();
			await FindFilesAsync(new Utf8StringBuilder(), target, files, cancellationToken);

			Dictionary<IoHash, IHashedBlobRef<LeafChunkedDataNode>> blocks = new Dictionary<IoHash, IHashedBlobRef<LeafChunkedDataNode>>();
			foreach (UnsyncBlock block in files.SelectMany(x => x.Blocks))
			{
				blocks[block.Blob.Hash] = block.Blob;
			}

			_logger.LogDebug("Generated Unsync manifest for artifact {ArtifactId} in {Time:n1}ms", artifact.Id, timer.ElapsedMilliseconds);

			UnsyncManifest manifest = new UnsyncManifest(files);
			ReadOnlyMemory<byte> manifestData = SerializeManifest(manifest);

			return new ArtifactInfo(manifest, manifestData, blocks.ToFrozenDictionary());
		}

		static async Task FindFilesAsync(Utf8StringBuilder path, IHashedBlobRef<DirectoryNode> directoryNodeRef, List<UnsyncFile> files, CancellationToken cancellationToken)
		{
			DirectoryNode directoryNode = await directoryNodeRef.ReadBlobAsync(cancellationToken);

			int initialPathLength = path.Length;

			foreach (FileEntry fileEntry in directoryNode.Files)
			{
				List<UnsyncBlock> blocks = new List<UnsyncBlock>();
				if (fileEntry.Length > 0)
				{
					await FindBlocksAsync(0, fileEntry.Target, blocks, cancellationToken);
				}

				bool readOnly = (fileEntry.Flags & FileEntryFlags.ReadOnly) != 0;
				bool executable = (fileEntry.Flags & FileEntryFlags.Executable) != 0;

				path.Append(fileEntry.Name);
				files.Add(new UnsyncFile(path.ToUtf8String().Clone(), readOnly, executable, fileEntry.Length, fileEntry.ModTime, blocks));
				path.Length = initialPathLength;
			}

			foreach (DirectoryEntry directoryEntry in directoryNode.Directories)
			{
				path.Append(directoryEntry.Name);
				path.Append('/');
				await FindFilesAsync(path, directoryEntry.Handle, files, cancellationToken);
				path.Length = initialPathLength;
			}
		}

		static async Task FindBlocksAsync(long offset, ChunkedDataNodeRef nodeRef, List<UnsyncBlock> blocks, CancellationToken cancellationToken)
		{
			if (nodeRef.Type == ChunkedDataNodeType.Leaf)
			{
				blocks.Add(new UnsyncBlock(offset, nodeRef.Length, nodeRef.RollingHash, nodeRef.GetLeafHandle()));
			}
			else
			{
				InteriorChunkedDataNode interiorNode = await nodeRef.ReadBlobAsync<InteriorChunkedDataNode>(cancellationToken: cancellationToken);
				foreach (ChunkedDataNodeRef childRef in interiorNode.Children)
				{
					await FindBlocksAsync(offset, childRef, blocks, cancellationToken);
					offset += childRef.Length;
				}
			}
		}
	}
}
