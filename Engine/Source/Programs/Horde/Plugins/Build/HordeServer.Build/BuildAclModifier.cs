// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Software;
using HordeServer.Artifacts;
using HordeServer.Devices;
using HordeServer.Jobs;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Utilities;

namespace HordeServer
{
	class BuildAclModifier : IDefaultAclModifier
	{
		/// <inheritdoc/>
		public void Apply(DefaultAclBuilder acl)
		{
			acl.AddCustomRole(HordeClaims.AgentRegistrationClaim, new[] { StreamAclAction.ViewStream, ProjectAclAction.ViewProject, JobAclAction.ViewJob });
			acl.AddCustomRole(HordeClaims.AgentDedicatedRoleClaim, new[] { ProjectAclAction.ViewProject, StreamAclAction.ViewStream, LogAclAction.CreateEvent, AgentSoftwareAclAction.DownloadSoftware });
			acl.AddCustomRole(HordeClaims.AgentWorkstationRoleClaim, new[] { AgentAclAction.CreateWorkstationAgent, AgentSoftwareAclAction.DownloadSoftware });
			acl.AddCustomRole(HordeClaims.ConfigureProjectsClaim, new[] { ProjectAclAction.CreateProject, ProjectAclAction.UpdateProject, ProjectAclAction.ViewProject, StreamAclAction.CreateStream, StreamAclAction.UpdateStream, StreamAclAction.ViewStream });
			acl.AddCustomRole(HordeClaims.StartChainedJobClaim, new[] { JobAclAction.CreateJob, JobAclAction.ExecuteJob, JobAclAction.UpdateJob, JobAclAction.ViewJob, StreamAclAction.ViewTemplate, StreamAclAction.ViewStream });

			acl.AddDefaultReadAction(ArtifactAclAction.DownloadArtifact);
			acl.AddDefaultReadAction(ArtifactAclAction.ReadArtifact);
			acl.AddDefaultReadAction(BisectTaskAclAction.ViewBisectTask);
			acl.AddDefaultReadAction(DeviceAclAction.DeviceRead);
			acl.AddDefaultReadAction(TestDataAclAction.TestHealthRead);
			acl.AddDefaultReadAction(JobAclAction.ViewJob);
			acl.AddDefaultReadAction(NotificationAclAction.CreateSubscription);
			acl.AddDefaultReadAction(ProjectAclAction.ViewProject);
			acl.AddDefaultReadAction(StreamAclAction.ViewChanges);
			acl.AddDefaultReadAction(StreamAclAction.ViewStream);
			acl.AddDefaultReadAction(StreamAclAction.ViewTemplate);

			acl.AddDefaultWriteAction(JobAclAction.CreateJob);
			acl.AddDefaultWriteAction(JobAclAction.UpdateJob);
			acl.AddDefaultWriteAction(JobAclAction.RetryJobStep);
			acl.AddDefaultWriteAction(DeviceAclAction.DeviceWrite);
			acl.AddDefaultWriteAction(TestDataAclAction.TestHealthWrite);
			acl.AddDefaultWriteAction(BisectTaskAclAction.CreateBisectTask);
			acl.AddDefaultWriteAction(BisectTaskAclAction.UpdateBisectTask);
		}
	}
}
