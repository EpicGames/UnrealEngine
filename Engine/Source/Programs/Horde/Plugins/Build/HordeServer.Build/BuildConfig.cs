// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using System.Text.Json.Serialization;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Perforce;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Artifacts;
using HordeServer.Commits;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Plugins;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer
{
	/// <summary>
	/// Configuration for the build plugin
	/// </summary>
	public class BuildConfig : IPluginConfig
	{
		/// <summary>
		/// Acl of this scope
		/// </summary>
		[JsonIgnore]
		public AclConfig Acl { get; private set; } = null!;

		/// <summary>
		/// List of Perforce clusters
		/// </summary>
		public List<PerforceCluster> PerforceClusters { get; set; } = new List<PerforceCluster>();

		/// <summary>
		/// Device configuration
		/// </summary>
		public DeviceConfig? Devices { get; set; }

		/// <summary>
		/// Maximum number of conforms to run at once
		/// </summary>
		public int MaxConformCount { get; set; }

		/// <summary>
		/// Time to wait before shutting down an agent that has been disabled
		/// Used if no value is set on the actual pool.
		/// </summary>
		public TimeSpan? AgentShutdownIfDisabledGracePeriod { get; set; } = null;

		/// <summary>
		/// RoboMerge branchmap file configuration for the merge graph service.
		/// When null or empty, the service is inert (no Perforce connections, no API data).
		/// </summary>
		public RoboMergeConfig? Robomerge { get; set; }

		/// <summary>
		/// Configuration for different artifact types
		/// </summary>
		public List<ArtifactTypeConfig> ArtifactTypes { get; set; } = new List<ArtifactTypeConfig>();

		/// <summary>
		/// List of projects
		/// </summary>
		public List<ProjectConfig> Projects { get; set; } = new List<ProjectConfig>();

		/// <summary>
		/// Enumerates all the streams
		/// </summary>
		[JsonIgnore]
		public IReadOnlyList<StreamConfig> Streams { get; private set; } = null!;

		/// <summary>
		/// Whether to allow conform tasks to run
		/// </summary>
		public bool EnableConformTasks { get; set; } = true;

		/// <summary>
		/// Whether to check if a conform is required on an agent before assigning a lease for a job task source.
		/// </summary>
		public bool EnableJobTaskConformCheck { get; set; }

		/// <summary>
		/// Commit tag to use for marking issues as fixed
		/// </summary>
		public string IssueFixedTag { get; set; } = "#horde";

		/// <summary>
		/// Build health issue configurations
		/// </summary>
		public IssueConfig? Issues { get; set; }

		/// <summary>
		/// Whether to allow notifications for Root Cause Analysis to be sent
		/// </summary>
		public bool EnableIssueRootCauseAnalysisNotifications { get; set; }

		/// <summary>
		/// Global kill switch for posting auto-RCA results to Slack triage threads.
		/// When false (default), no auto-RCA Slack notifications are sent regardless of per-workflow AutoRcaSlackEnabled settings.
		/// </summary>
		public bool EnableAutoRcaSlackNotifications { get; set; }

		/// <summary>
		/// Whether to use multi-stage LLM analysis for auto-RCA. Multi-stage is slower but potentially more accurate. Set to false for faster processing with higher throughput.
		/// </summary>
		public bool AutoRcaUseMultiStage { get; set; } = true;

		/// <summary>
		/// Maximum number of concurrent auto-RCA requests. Limits load on the external LLM API.
		/// Set to 0 to disable auto-RCA globally (acts as a kill switch).
		/// </summary>
		public int MaxConcurrentRcaRequests { get; set; }

		/// <summary>
		/// Issue channel groupings
		/// </summary>
		[JsonIgnore]
		public IReadOnlyDictionary<string, string> IssueChannelGroups { get; private set; } = null!;

		/// <summary>
		/// The maximum days since node success to scan for suspects
		/// </summary>
		public int IssueFindSuspectsMaxSuccessDays { get; set; } = 14;

		/// <summary>
		/// The maximum number of seconds to spend scording suspects
		/// </summary>
		public int IssueFindSuspectsMaxScoringSeconds { get; set; } = 60;

		#region Issue Suspect Finding Optimization Configuration

		/// <summary>
		/// Maximum number of commits to examine for suspects.
		/// Previously hardcoded as const MaxChanges = 500.
		/// </summary>
		public int IssueFindSuspectsMaxChanges { get; set; } = 500;

		/// <summary>
		/// Enable parallel scoring of commits for suspect finding.
		/// When enabled, commits are scored concurrently up to MaxParallelism.
		/// </summary>
		public bool IssueFindSuspectsEnableParallelScoring { get; set; } = false;

		/// <summary>
		/// Maximum concurrent commit file lookups when parallel scoring is enabled.
		/// Higher values improve speed but increase Perforce server load.
		/// </summary>
		public int IssueFindSuspectsMaxParallelism { get; set; } = 50;

		/// <summary>
		/// Enable early exit when theoretical maximum score is achieved.
		/// Only exits early if the best possible score has been found.
		/// </summary>
		public bool IssueFindSuspectsEnableEarlyExit { get; set; } = false;

		/// <summary>
		/// Enable bidirectional commit query to capture both recent and old commits.
		/// Queries half the limit from each end of the commit range.
		/// </summary>
		public bool IssueFindSuspectsEnableBidirectionalQuery { get; set; } = false;

		/// <summary>
		/// Enable fingerprint-based fallback when no suspects are found.
		/// Helps prevent duplicate issues when commit range exceeds MaxChanges.
		/// </summary>
		public bool IssueFindSuspectsEnableFingerprintFallback { get; set; } = false;

		/// <summary>
		/// Minimum score required for fingerprint fallback matching.
		/// Higher values are more conservative (fewer false matches).
		/// </summary>
		public int IssueFingerprintFallbackMinScore { get; set; } = 50;

		/// <summary>
		/// Maximum age in days for issues to be considered in fingerprint fallback.
		/// </summary>
		public int IssueFingerprintFallbackMaxAgeDays { get; set; } = 14;

		/// <summary>
		/// Fingerprint types excluded from fallback matching (too generic).
		/// </summary>
		public List<string> IssueFingerprintFallbackExcludedTypes { get; set; } =
			new() { "Default", "ExitCode", "Generic" };

		#endregion

		/// <summary>
		/// Whether to enable expanded IgnorePatterns discovery (workspace-wide scanning + config-specified files).
		/// When false (default), only Engine directory is scanned for ignore patterns.
		/// </summary>
		public bool EnableExpandedIgnorePatterns { get; set; } = true;

		/// <summary>
		/// Fingerprint type prefixes considered "low context" for suspect mention suppression.
		/// When an issue's fingerprint type starts with any of these prefixes AND all suspects are tied
		/// at rank 0, @ mentions are suppressed in Slack. Suspects are still listed without mentions.
		/// Set to an empty array to disable suppression entirely.
		/// </summary>
		public List<string> LowContextFingerprintTypes { get; set; } = new List<string> { "Hashed", "Gauntlet" };

		private readonly Dictionary<ProjectId, ProjectConfig> _projectLookup = new Dictionary<ProjectId, ProjectConfig>();
		private readonly Dictionary<StreamId, StreamConfig> _streamLookup = new Dictionary<StreamId, StreamConfig>();

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
			Acl = configOptions.ParentAcl;
			Streams = Projects.SelectMany(x => x.Streams).ToList();

			Dictionary<string, string> issueChannelGroups = new Dictionary<string, string>();
			IssueChannelGroups = issueChannelGroups;

			// build up the issue channel groupings
			Issues?.PostLoad();

			Issues?.ChannelGroups?.ForEach(group =>
			{
				if (group.Channels == null || group.Channels.Count == 0)
				{
					return;
				}

				string channelGroup = String.Join("_", group.Channels);

				group.Channels.ForEach(channel =>
				{
					if (String.IsNullOrEmpty(channel) || issueChannelGroups.ContainsKey(channel))
					{
						return;
					}

					issueChannelGroups[channel] = channelGroup;
				});
			});

			_projectLookup.Clear();
			_streamLookup.Clear();
			foreach (ProjectConfig project in Projects)
			{
				_projectLookup.Add(project.Id, project);
				project.PostLoad(project.Id, configOptions.ParentAcl, ArtifactTypes, configOptions.Logger);

				foreach (StreamConfig stream in project.Streams)
				{
					_streamLookup.Add(stream.Id, stream);
				}
			}

			foreach (ArtifactTypeConfig artifactTypeConfig in ArtifactTypes)
			{
				artifactTypeConfig.PostLoad(Acl);
			}

			UpdateWorkspacesForPools(configOptions.Plugins);
		}

		void UpdateWorkspacesForPools(IEnumerable<IPluginConfig> plugins)
		{
			// Try to get the compute config, and skip if it isn't configured
			ComputeConfig computeConfig = plugins.OfType<ComputeConfig>().First();

			// Lookup table of pool id to workspaces
			Dictionary<PoolId, AutoSdkConfig> poolToAutoSdkView = new Dictionary<PoolId, AutoSdkConfig>();
			Dictionary<PoolId, List<AgentWorkspaceInfo>> poolToAgentWorkspaces = new Dictionary<PoolId, List<AgentWorkspaceInfo>>();

			// Populate the workspace list from the current stream
			foreach (StreamConfig streamConfig in Streams)
			{
				foreach (KeyValuePair<string, AgentConfig> agentTypePair in streamConfig.AgentTypes)
				{
					// Create the new agent workspace
					if (streamConfig.TryGetAgentWorkspace(agentTypePair.Value, out AgentWorkspaceInfo? agentWorkspace, out AutoSdkConfig? autoSdkConfig))
					{
						AgentConfig agentType = agentTypePair.Value;

						// Find or add a list of workspaces for this pool
						List<AgentWorkspaceInfo>? agentWorkspaces;
						if (!poolToAgentWorkspaces.TryGetValue(agentType.Pool, out agentWorkspaces))
						{
							agentWorkspaces = new List<AgentWorkspaceInfo>();
							poolToAgentWorkspaces.Add(agentType.Pool, agentWorkspaces);
						}

						// Add it to the list
						if (!agentWorkspaces.Contains(agentWorkspace))
						{
							agentWorkspaces.Add(agentWorkspace);
						}
						if (autoSdkConfig != null)
						{
							AutoSdkConfig? existingAutoSdkConfig;
							poolToAutoSdkView.TryGetValue(agentType.Pool, out existingAutoSdkConfig);
							poolToAutoSdkView[agentType.Pool] = AutoSdkConfig.Merge(autoSdkConfig, existingAutoSdkConfig);
						}
					}
				}
			}

			// Update the list of workspaces for each pool
			foreach (PoolConfig pool in computeConfig.Pools)
			{
				// Get the new list of workspaces for this pool
				List<AgentWorkspaceInfo>? newWorkspaces;
				if (!poolToAgentWorkspaces.TryGetValue(pool.Id, out newWorkspaces))
				{
					newWorkspaces = new List<AgentWorkspaceInfo>();
				}

				// Get the autosdk view
				AutoSdkConfig? newAutoSdkConfig;
				if (!poolToAutoSdkView.TryGetValue(pool.Id, out newAutoSdkConfig))
				{
					newAutoSdkConfig = AutoSdkConfig.None;
				}

				pool.Workspaces = newWorkspaces;
				pool.AutoSdkConfig = newAutoSdkConfig;
			}
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular stream
		/// </summary>
		/// <param name="job">Job to authorize for</param>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True if the action is authorized</returns>
		public bool Authorize(IJob? job, AclAction action, ClaimsPrincipal user)
		{
			if (job != null && TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				return streamConfig.Authorize(action, user);
			}
			else
			{
				return Authorize(action, user);
			}
		}

		/// <summary>
		/// Authorize an action to perform on an artifact
		/// </summary>
		public bool AuthorizeArtifact(ArtifactType type, StreamId streamId, AclAction action, ClaimsPrincipal user)
		{
			if (user.HasAdminClaim())
			{
				return true;
			}

			bool? auth = null;
			if (TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				auth = AuthorizeArtifactType(streamConfig.ArtifactTypes, type, action, user);
				auth ??= streamConfig.Acl.AuthorizeSingleScope(action, user);

				ProjectConfig projectConfig = streamConfig.ProjectConfig;

				auth ??= AuthorizeArtifactType(projectConfig.ArtifactTypes, type, action, user);
				auth ??= projectConfig.Acl.AuthorizeSingleScope(action, user);
			}

			auth ??= AuthorizeArtifactType(ArtifactTypes, type, action, user);
			return auth ?? Acl.TryAuthorize(action, user) ?? user.HasClaim(HordeClaimTypes.LeaseStream, streamId.ToString());
		}

		static bool? AuthorizeArtifactType(IReadOnlyList<ArtifactTypeConfig> artifactTypes, ArtifactType artifactType, AclAction action, ClaimsPrincipal user)
		{
			ArtifactTypeConfig? config = artifactTypes.FirstOrDefault(x => x.Type == artifactType);
			return config?.Acl?.AuthorizeSingleScope(action, user);
		}

		/// <summary>
		/// Attempts to get configuration for a project from this object
		/// </summary>
		/// <param name="projectId">The stream identifier</param>
		/// <param name="config">Configuration for the stream</param>
		/// <returns>True if the stream configuration was found</returns>
		public bool TryGetProject(ProjectId projectId, [NotNullWhen(true)] out ProjectConfig? config) => _projectLookup.TryGetValue(projectId, out config);

		/// <summary>
		/// Get configuration for a stream from this object
		/// </summary>
		/// <param name="streamId">The stream identifier</param>
		/// <returns>Configuration for the stream</returns>
		public StreamConfig GetStream(StreamId streamId)
		{
			StreamConfig? streamConfig;
			if (!TryGetStream(streamId, out streamConfig))
			{
				throw new StreamNotFoundException(streamId);
			}
			return streamConfig;
		}

		/// <summary>
		/// Attempts to get configuration for a stream from this object
		/// </summary>
		/// <param name="streamId">The stream identifier</param>
		/// <param name="config">Configuration for the stream</param>
		/// <returns>True if the stream configuration was found</returns>
		public bool TryGetStream(StreamId streamId, [NotNullWhen(true)] out StreamConfig? config) => _streamLookup.TryGetValue(streamId, out config);

		/// <summary>
		/// Attempts to get configuration for a stream from this object
		/// </summary>
		/// <param name="streamId">The stream identifier</param>
		/// <param name="templateId">Template identifier</param>
		/// <param name="config">Configuration for the template</param>
		/// <returns>True if the template configuration was found</returns>
		public bool TryGetTemplate(StreamId streamId, TemplateId templateId, [NotNullWhen(true)] out TemplateRefConfig? config)
		{
			if (!_streamLookup.TryGetValue(streamId, out StreamConfig? streamConfig))
			{
				config = null;
				return false;
			}
			return streamConfig.TryGetTemplate(templateId, out config);
		}

		/// <inheritdoc cref="AclConfig.Authorize"/>
		public bool Authorize(AclAction action, ClaimsPrincipal principal)
			=> Acl.Authorize(action, principal);

		/// <summary>
		/// Finds a perforce cluster with the given name or that contains the provided server
		/// </summary>
		/// <param name="name">Name of the cluster</param>
		/// <param name="serverAndPort">Find cluster which contains server</param>
		/// <returns></returns>
		public PerforceCluster GetPerforceCluster(string? name, string? serverAndPort = null)
		{
			PerforceCluster? cluster = FindPerforceCluster(name, serverAndPort);
			if (cluster == null)
			{
				throw new Exception($"Unknown Perforce cluster '{name}'");
			}
			return cluster;
		}

		/// <summary>
		/// Finds a perforce cluster with the given name or that contains the provided server
		/// </summary>
		/// <param name="name">Name of the cluster</param>
		/// <param name="serverAndPort">Find cluster which contains server</param>
		/// <returns></returns>
		public PerforceCluster? FindPerforceCluster(string? name, string? serverAndPort = null)
		{
			List<PerforceCluster> clusters = PerforceClusters;

			if (serverAndPort != null)
			{
				return clusters.FirstOrDefault(x => x.Servers.FirstOrDefault(server => String.Equals(server.ServerAndPort, serverAndPort, StringComparison.OrdinalIgnoreCase)) != null);
			}

			if (clusters.Count == 0)
			{
				clusters = DefaultClusters;
			}

			if (name == null)
			{
				return clusters.FirstOrDefault();
			}
			else
			{
				return clusters.FirstOrDefault(x => String.Equals(x.Name, name, StringComparison.OrdinalIgnoreCase));
			}
		}

		static List<PerforceCluster> DefaultClusters { get; } = GetDefaultClusters();

		static List<PerforceCluster> GetDefaultClusters()
		{
			PerforceServer server = new PerforceServer();
			server.ServerAndPort = PerforceSettings.Default.ServerAndPort;

			PerforceCluster cluster = new PerforceCluster();
			cluster.Name = "Default";
			cluster.CanImpersonate = false;
			cluster.Servers.Add(server);

			return new List<PerforceCluster> { cluster };
		}
	}

	/// <summary>
	/// Path to a platform and stream to use for syncing AutoSDK
	/// </summary>
	public class AutoSdkWorkspace
	{
		/// <summary>
		/// Name of this workspace
		/// </summary>
		public string? Name { get; set; }

		/// <summary>
		/// The agent properties to check (eg. "OSFamily=Windows")
		/// </summary>
		public List<string> Properties { get; set; } = new List<string>();

		/// <summary>
		/// Username for logging in to the server
		/// </summary>
		public string? UserName { get; set; }

		/// <summary>
		/// Stream to use
		/// </summary>
		[Required]
		public string? Stream { get; set; }
	}

	/// <summary>
	/// Information about an individual Perforce server
	/// </summary>
	public class PerforceServer
	{
		/// <summary>
		/// The server and port. The server may be a DNS entry with multiple records, in which case it will be actively load balanced.
		/// If "ssl:" prefix is used, ensure P4 server's fingerprint/certificate is trusted.
		/// See Horde's documentation on connecting to SSL-enabled Perforce servers.
		/// </summary>
		public string ServerAndPort { get; set; } = "perforce:1666";

		/// <summary>
		/// Whether to query the healthcheck address under each server
		/// </summary>
		public bool HealthCheck { get; set; }

		/// <summary>
		/// Whether to resolve the DNS entries and load balance between different hosts
		/// </summary>
		public bool ResolveDns { get; set; }

		/// <summary>
		/// Maximum number of simultaneous conforms on this server
		/// </summary>
		public int MaxConformCount { get; set; }

		/// <summary>
		/// Optional condition for a machine to be eligible to use this server
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// List of properties for an agent to be eligible to use this server
		/// </summary>
		public List<string>? Properties { get; set; }
	}

	/// <summary>
	/// Credentials for a Perforce user
	/// </summary>
	public class PerforceCredentials
	{
		/// <summary>
		/// The username
		/// </summary>
		public string UserName { get; set; } = String.Empty;

		/// <summary>
		/// Password for the user
		/// </summary>
		[ResolveSecret]
		public string? Password { get; set; } = String.Empty;

		/// <summary>
		/// Login ticket for the user (will be used instead of password if set)
		/// </summary>
		[ResolveSecret]
		public string? Ticket { get; set; } = String.Empty;
	}

	/// <summary>
	/// Information about a cluster of Perforce servers. 
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class PerforceCluster
	{
		/// <summary>
		/// The default cluster name
		/// </summary>
		public const string DefaultName = "Default";

		/// <summary>
		/// Name of the cluster
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// Username for Horde to log in to this server. Will use the first account specified below if not overridden.
		/// </summary>
		public string? ServiceAccount { get; set; }

		/// <summary>
		/// Whether the service account can impersonate other users
		/// </summary>
		public bool CanImpersonate { get; set; } = true;

		/// <summary>
		/// Whether to use partitioned workspaces on this server
		/// </summary>
		public bool SupportsPartitionedWorkspaces { get; set; } = false;

		/// <summary>
		/// List of servers
		/// </summary>
		public List<PerforceServer> Servers { get; set; } = new List<PerforceServer>();

		/// <summary>
		/// List of server credentials
		/// </summary>
		public List<PerforceCredentials> Credentials { get; set; } = new List<PerforceCredentials>();

		/// <summary>
		/// List of autosdk streams
		/// </summary>
		public List<AutoSdkWorkspace> AutoSdk { get; set; } = new List<AutoSdkWorkspace>();
	}

	/// <summary>
	/// Configuration for the RoboMerge branchmap service
	/// </summary>
	public class RoboMergeConfig
	{
		/// <summary>
		/// Perforce depot paths to branchmap JSON files.
		/// Plain depot paths (e.g. "//MyProject/Main/Build/Streams/myproject.branchmap.json").
		/// All files are read from the Perforce cluster specified by <see cref="PerforceCluster"/>.
		/// </summary>
		public List<string> BranchmapFiles { get; set; } = new List<string>();

		/// <summary>
		/// Perforce cluster name to connect to for reading branchmap files.
		/// Maps to a connection configured in BuildServerConfig.Perforce[].
		/// Default: "default" (uses the default Perforce connection).
		/// </summary>
		public string PerforceCluster { get; set; } = "default";

		/// <summary>
		/// How often to poll Perforce for branchmap file changes. Default: 5 minutes.
		/// Changing this requires a server restart (ticker interval is set at construction time).
		/// </summary>
		public TimeSpan TickInterval { get; set; } = TimeSpan.FromMinutes(5);
	}
}
