// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Build.BuildHealth
{
	/// <summary>
	/// ACL Actions used for build health filters.
	/// </summary>
	public static class BuildHealthFilterAclAction
	{
		/// <summary>
		/// Ability to add, update, and delete build health filters.
		/// </summary>
		public static AclAction AddUpdateDeleteBuildHealthFilter { get; } = new AclAction("AddUpdateDeleteBuildHealthFilter");
	}
}
