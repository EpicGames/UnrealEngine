// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.BuildHealth;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Build.BuildHealth
{
	/// <summary>
	/// Build Health Filter Collection implementation.
	/// </summary>
	class BuildHealthFilterCollection : IBuildHealthFilterCollection
	{
		#region -- Private Members --

		readonly IMongoCollection<BuildHealthFilterDocument> _buildHealthFilterDocuments;
		readonly ILogger<BuildHealthFilterCollection> _logger;

		#endregion -- Private Members --

		#region -- Constructor --

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="mongoService">The underlying mongo service to use.</param>
		/// <param name="logger">The logger.</param>
		public BuildHealthFilterCollection(IMongoService mongoService, ILogger<BuildHealthFilterCollection> logger)
		{
			_buildHealthFilterDocuments = mongoService.GetCollection<BuildHealthFilterDocument>("BuildHealthFilters");
			_logger = logger;
		}

		#endregion -- Constructor --

		#region -- Interface API --

		/// <inheritdoc/>
		public async Task<IBuildHealthFilter?> AddBuildHealthFilterAsync(UserId? owner, BuildHealthFilterAddRequest filterRequest, CancellationToken cancellationToken)
		{
			if (!filterRequest.IsValidAddRequest())
			{
				_logger.LogWarning("Invalid add build health filter request. IsValidFilterQuery: {IsValidFilterQuery}; IsValidFilterName: {IsValidFilterQuery}", filterRequest.IsValidFilterQuery(), filterRequest.IsValidFilterName());
				return null;
			}

			BuildHealthFilterDocument newBuildHealthFilter = new BuildHealthFilterDocument() { Id = new BuildHealthFilterId(BinaryIdUtils.CreateNew()), Owner = owner, FilterName = filterRequest.FilterName, FilterDescription = filterRequest.FilterDescription, FilterProject = filterRequest.FilterProject, FilterQuery = filterRequest.FilterQuery, UpdateTimeUtc = DateTime.UtcNow };

			await _buildHealthFilterDocuments.InsertOneAsync(newBuildHealthFilter, null, cancellationToken);
			return newBuildHealthFilter;
		}

		/// <inheritdoc/>
		public async Task<IBuildHealthFilter?> UpdateBuildHealthFilterAsync(BuildHealthFilterId filterId, UserId? owner, BuildHealthFilterUpdateRequest filterRequest, CancellationToken cancellationToken)
		{
			if (!filterRequest.IsValidUpdateRequest())
			{
				_logger.LogWarning("Invalid update build health filter request. IsValidFilterQuery: {IsValidFilterQuery}; IsValidFilterName: {IsValidFilterQuery}", filterRequest.IsValidFilterQuery(), filterRequest.IsValidFilterName());
				return null;
			}

			FilterDefinition<BuildHealthFilterDocument> filter = Builders<BuildHealthFilterDocument>.Filter.Expr(x => x.Id == filterId) & Builders<BuildHealthFilterDocument>.Filter.Expr(x => (x.Owner == null || x.Owner == owner));
			List<UpdateDefinition<BuildHealthFilterDocument>> updates = new List<UpdateDefinition<BuildHealthFilterDocument>>();
			if (!String.IsNullOrEmpty(filterRequest.FilterName))
			{
				updates.Add(Builders<BuildHealthFilterDocument>.Update.Set(x => x.FilterName, filterRequest.FilterName));
			}

			// Check for an explicit null, as Empty/Whitespace is a valid erasure.
			if (filterRequest.FilterDescription != null)
			{
				// If we have passed a whitespace, we will set it to null in the document.
				updates.Add(Builders<BuildHealthFilterDocument>.Update.Set(x => x.FilterDescription, String.IsNullOrWhiteSpace(filterRequest.FilterDescription) ? null : filterRequest.FilterDescription));
			}

			if (!String.IsNullOrEmpty(filterRequest.FilterQuery))
			{
				updates.Add
					(Builders<BuildHealthFilterDocument>.Update.Set(x => x.FilterQuery, filterRequest.FilterQuery));
			}

			// We have no updates, just fetch the item for the requestor.
			if (updates.Count == 0)
			{
				_logger.LogInformation("No updates to apply for filterId ({FilterId}). Returning.", filterId);
				return null;
			}

			updates.Add(Builders<BuildHealthFilterDocument>.Update.Set(x => x.UpdateTimeUtc, DateTime.UtcNow));

			UpdateDefinition<BuildHealthFilterDocument> update = updates.Count > 0 ? Builders<BuildHealthFilterDocument>.Update.Combine(updates) : Builders<BuildHealthFilterDocument>.Update.Combine();
			IBuildHealthFilter? buildHealthFilter = await _buildHealthFilterDocuments.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<BuildHealthFilterDocument, BuildHealthFilterDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);

			if (buildHealthFilter == null)
			{
				_logger.LogInformation("Unable to update Build Health Filter with Id: {BuildHealthFilterId} with Owner: {OwnerId}.", filterId, owner);
			}

			return buildHealthFilter;
		}

		/// <inheritdoc/>
		public async Task<IBuildHealthFilter?> GetBuildHealthFilterAsync(BuildHealthFilterId filterId, CancellationToken cancellationToken)
		{
			FilterDefinition<BuildHealthFilterDocument> filter = Builders<BuildHealthFilterDocument>.Filter.Expr(x => x.Id == filterId);
			IBuildHealthFilter document = await _buildHealthFilterDocuments.Find(filter).FirstOrDefaultAsync<BuildHealthFilterDocument>(cancellationToken);

			return document;
		}

		/// <inheritdoc/>
		public async Task<IEnumerable<IBuildHealthFilter>> GetBuildHealthFiltersAsync(ProjectId projectId, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<BuildHealthFilterDocument> filterBuilder = Builders<BuildHealthFilterDocument>.Filter;
			FilterDefinition<BuildHealthFilterDocument> filter = filterBuilder.Empty;

			filter &= filterBuilder.Eq(x => x.FilterProject, projectId);

			List<BuildHealthFilterDocument> documents = await _buildHealthFilterDocuments.Find(filter).ToListAsync(cancellationToken);

			return documents;
		}

		/// <inheritdoc/>
		public async Task<bool> DeleteBuildHealthFilterAsync(BuildHealthFilterId filterId, UserId? owner, CancellationToken cancellationToken)
		{
			FilterDefinition<BuildHealthFilterDocument> filter = Builders<BuildHealthFilterDocument>.Filter.Expr(x => x.Id == filterId && (x.Owner == null || x.Owner == owner));
			DeleteResult result = await _buildHealthFilterDocuments.DeleteOneAsync(filter, cancellationToken);

			return result.DeletedCount == 1;
		}

		#endregion -- Interface API --

		#region -- Internal Types --

		/// <summary>
		/// Build health filter document.
		/// </summary>
		class BuildHealthFilterDocument : IBuildHealthFilter
		{
			/// <inheritdoc/>
			[BsonRequired, BsonId]
			public BuildHealthFilterId Id { get; set; }

			/// <inheritdoc/>
			[BsonElement("fn")]
			public string FilterName { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("fd")]
			public string? FilterDescription { get; set; }

			/// <inheritdoc/>
			[BsonElement("fp")]
			public ProjectId FilterProject { get; set; }

			/// <inheritdoc/>
			[BsonElement("fq")]
			public string FilterQuery { get; set; } = default!;

			/// <inheritdoc/>
			[BsonElement("o")]
			public UserId? Owner { get; set; }

			/// <inheritdoc/>
			[BsonElement("ut")]
			public DateTime UpdateTimeUtc { get; set; }
		}

		#endregion -- Internal Types --
	}
}
