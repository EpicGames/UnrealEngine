// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.BuildHealth;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;
using HordeServer.Projects;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Build.BuildHealth
{
	/// <summary>
	/// Controller for build health filters.
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class BuildHealthFilterController : HordeControllerBase
	{

		#region -- Private Members --

		readonly IBuildHealthFilterCollection _buildHealthFilterCollection;
		readonly IUserCollection _userCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		#endregion -- Private Members --

		#region -- Constructor --

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="buildHealthFilterCollection">The underyling build health filter collection.</param>
		/// <param name="userCollection">The underlying user collection used for collating ownership details.</param>
		/// <param name="buildConfig">Build config used in accessing relevant ACL configurations.</param>
		public BuildHealthFilterController(IBuildHealthFilterCollection buildHealthFilterCollection, IUserCollection userCollection, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_buildHealthFilterCollection = buildHealthFilterCollection;
			_userCollection = userCollection;
			_buildConfig = buildConfig;
		}

		#endregion -- Constructor --

		#region -- Public API --

		/// <summary>
		/// Adds a new build health filter given the request.
		/// </summary>
		/// <param name="filterUpdateRequest">The add request.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The newly created filter if applicable.</returns>
		/// <remarks>The requesting user will become the implicit owner of the new filter.</remarks>
		[HttpPost]
		[Route("/api/v1/buildhealth/filters")]
		[ProducesResponseType(typeof(BuildHealthFilterResponse), 200)]
		public async Task<ActionResult<BuildHealthFilterResponse>?> AddBuildHealthFilterAsync([FromBody] BuildHealthFilterAddRequest filterUpdateRequest, CancellationToken cancellationToken = default)
		{
			// Validate inputs
			if (String.IsNullOrEmpty(filterUpdateRequest.FilterQuery))
			{
				return BadRequest("No filter query provided. Filter query is a required parameter.");
			}

			if (String.IsNullOrEmpty(filterUpdateRequest.FilterName))
			{
				return BadRequest("No filter name provided. Filter name is a required parameter.");
			}

			if (!filterUpdateRequest.IsValidAddRequest())
			{
				return BadRequest($"UpdateFilterRequest was not valid. Check filter name length (Max:{BuildHealthFilterRestrictions.MaxFilterNameLength}) & filter query length (Max:{BuildHealthFilterRestrictions.MaxFilterQueryLength}) & filter description length (Max:{BuildHealthFilterRestrictions.MaxFilterDescripitonLength}).");
			}

			// Validate user has access for add operation.
			if (!_buildConfig.Value.Authorize(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter, User))
			{
				return Forbid(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter);
			}

			// Validate project exists, and user has access for project
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeProject(filterUpdateRequest.FilterProject);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			UserId? requestingUser = User.GetUserId();
			IBuildHealthFilter? filter = await _buildHealthFilterCollection.AddBuildHealthFilterAsync(requestingUser, filterUpdateRequest, cancellationToken);

			if (filter == null)
			{
				return StatusCode(StatusCodes.Status500InternalServerError);

			}

			return await CreateBuildHealthFilterResponseWithOwnerAsync(filter, cancellationToken);
		}

		/// <summary>
		/// Updates the filter based off the request.
		/// </summary>
		/// <param name="filterId">The id of the filter to update.</param>
		/// <param name="filterUpdateRequest">The update request.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The updated filter if applicable.</returns>
		/// <remarks>Users can only update filters that don't have an owner, ones they are an owner of, or any filter if they are an admin.</remarks>
		[HttpPut]
		[Route("/api/v1/buildhealth/filters/{filterId}")]
		[ProducesResponseType(typeof(BuildHealthFilterResponse), 200)]
		public async Task<ActionResult<BuildHealthFilterResponse>?> UpdateBuildHealthFilterAsync(BuildHealthFilterId filterId, [FromBody] BuildHealthFilterUpdateRequest filterUpdateRequest, CancellationToken cancellationToken = default)
		{
			// Validate inputs
			if (String.IsNullOrEmpty(filterUpdateRequest.FilterName) && String.IsNullOrEmpty(filterUpdateRequest.FilterQuery))
			{
				return BadRequest("Niether filter name or filter query were provided. Please provide at least one in order to update the filter.");
			}

			if (!filterUpdateRequest.IsValidUpdateRequest())
			{
				return BadRequest($"UpdateFilterRequest was not valid. Check filter name length (Max:{BuildHealthFilterRestrictions.MaxFilterNameLength}) & filter query length (Max:{BuildHealthFilterRestrictions.MaxFilterQueryLength}) & filter description length (Max:{BuildHealthFilterRestrictions.MaxFilterDescripitonLength}).");
			}

			// Validate user has access for update operation.
			if (!_buildConfig.Value.Authorize(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter, User))
			{
				return Forbid(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter);
			}

			// Verify filter exists.
			IBuildHealthFilter? existingFilter = await _buildHealthFilterCollection.GetBuildHealthFilterAsync(filterId, cancellationToken);
			if (existingFilter == null)
			{
				return NotFound($"Could not find BuildHealthFilter with Id: {filterId}");
			}

			// Validate project exists, and user has access for project
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeProject(existingFilter.FilterProject);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			// Verify the user is able to edit this filter.
			UserId? requestingUser = User.GetUserId();
			bool canEdit = existingFilter.Owner == requestingUser || User.HasAdminClaim();

			if (!canEdit)
			{
				return Forbid($"User is not the owner of BuildHealthFilter Id: {filterId}.");
			}

			IBuildHealthFilter? filter = await _buildHealthFilterCollection.UpdateBuildHealthFilterAsync(filterId, requestingUser, filterUpdateRequest, cancellationToken);

			if (filter == null)
			{
				return StatusCode(StatusCodes.Status500InternalServerError);

			}

			return await CreateBuildHealthFilterResponseWithOwnerAsync(filter, cancellationToken);
		}

		/// <summary>
		/// Gets filter with specific filter id.
		/// </summary>
		/// <param name="filterId">The filter id to filter on.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The filter if it was found.</returns>
		[HttpGet]
		[Route("/api/v1/buildhealth/filters/{filterId}")]
		[ProducesResponseType(typeof(BuildHealthFilterResponse), 200)]
		public async Task<ActionResult<BuildHealthFilterResponse>?> GetBuildHealthFilterAsync(BuildHealthFilterId filterId, CancellationToken cancellationToken = default)
		{
			IBuildHealthFilter? filter = await _buildHealthFilterCollection.GetBuildHealthFilterAsync(filterId, cancellationToken);
			if (filter == null)
			{
				return NotFound(filterId);
			}

			// Validate project exists, and user has access for project
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeProject(filter.FilterProject);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			return await CreateBuildHealthFilterResponseWithOwnerAsync(filter, cancellationToken);
		}

		/// <summary>
		/// Gets all applicable filters given the filter criteria.
		/// </summary>
		/// <param name="projectId">Project id to filter on.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns>The list of filters that match.</returns>
		[HttpGet]
		[Route("/api/v1/buildhealth/filters")]
		[ProducesResponseType(typeof(List<BuildHealthFilterResponse>), 200)]
		public async Task<ActionResult<List<BuildHealthFilterResponse>>> GetBuildHealthFiltersAsync([FromQuery] ProjectId projectId, CancellationToken cancellationToken = default)
		{
			// Validate project exists, and user has access for project
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeProject(projectId);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			IEnumerable<IBuildHealthFilter> filters = await _buildHealthFilterCollection.GetBuildHealthFiltersAsync(projectId, cancellationToken);
			List<BuildHealthFilterResponse> returnObjects = new List<BuildHealthFilterResponse>(filters.Count());

			foreach (IBuildHealthFilter filter in filters)
			{
				BuildHealthFilterResponse buildHealthFilter = await CreateBuildHealthFilterResponseWithOwnerAsync(filter, cancellationToken);
				returnObjects.Add(buildHealthFilter);
			}

			return returnObjects;
		}

		/// <summary>
		/// Deletes a filter.
		/// </summary>
		/// <param name="filterId">The id of the filter to delete.</param>
		/// <param name="cancellationToken">The cancellation token to use throughout the operation.</param>
		/// <returns></returns>
		/// <remarks>Users can only delete filters that don't have an owner, ones they are an owner of, or any filter if they are an admin.</remarks>
		[HttpDelete]
		[Route("/api/v1/buildhealth/filters/{filterId}")]
		public async Task<ActionResult> DeleteBuildHealthFiltersAsync(BuildHealthFilterId filterId, CancellationToken cancellationToken = default)
		{
			// Validate user has access for delete operation.
			if (!_buildConfig.Value.Authorize(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter, User))
			{
				return Forbid(BuildHealthFilterAclAction.AddUpdateDeleteBuildHealthFilter);
			}

			// Verify filter exists
			IBuildHealthFilter? existingFilter = await _buildHealthFilterCollection.GetBuildHealthFilterAsync(filterId, cancellationToken);

			if (existingFilter == null)
			{
				return NotFound($"Could not find BuildHealthFilter with Id: {filterId}");
			}

			// Validate project exists, and user has access for project
			(bool isValidationSuccessful, ActionResult actionResult) = VerifyAndAuthorizeProject(existingFilter.FilterProject);
			if (!isValidationSuccessful)
			{
				return actionResult;
			}

			// Verify user is able to delete this filter.
			UserId? requestingUser = User.GetUserId();
			bool canEdit = existingFilter.Owner == requestingUser || User.HasAdminClaim();

			if (!canEdit)
			{
				return Forbid("User is not the owner of BuildHealthFilter Id: {BuildHealthFilterId}.", existingFilter.Id);
			}

			if (!await _buildHealthFilterCollection.DeleteBuildHealthFilterAsync(filterId, requestingUser, cancellationToken))
			{
				return BadRequest($"Unable to delete filter ({filterId}).");
			}

			return Ok();
		}

		#endregion -- Public API --

		#region -- Private API --

		private async Task<BuildHealthFilterResponse> CreateBuildHealthFilterResponseWithOwnerAsync(IBuildHealthFilter filter, CancellationToken cancellationToken)
		{
			if (filter.Owner != null)
			{
				GetThinUserInfoResponse? user = (await _userCollection.GetCachedUserAsync(filter.Owner.Value, cancellationToken))?.ToThinApiResponse();

				if (user != null)
				{
					return new BuildHealthFilterResponse(filter, user);
				}
			}

			return new BuildHealthFilterResponse(filter);
		}

		/// <summary>
		/// Helper method to verify and authorize project.
		/// </summary>
		/// <param name="projectId">The projectId to verify the user against.</param>
		/// <returns>Tuple representing whether validation was success, and the action result. Action result is Ok() on success.</returns>
		private (bool isValidationSuccessful, ActionResult actionResult) VerifyAndAuthorizeProject(ProjectId projectId)
		{
			// Verify project.
			ProjectConfig? projectConfig;
			if (!_buildConfig.Value.TryGetProject(projectId, out projectConfig))
			{
				return (isValidationSuccessful: false, actionResult: NotFound(projectId));
			}

			// Verify user can view details around this project.
			if (!projectConfig.Authorize(ProjectAclAction.ViewProject, User))
			{
				return (isValidationSuccessful: false, actionResult: NotFound(ProjectAclAction.ViewProject));
			}

			return (isValidationSuccessful: true, actionResult: Ok());
		}

		#endregion
	}
}
