// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.BuildHealth;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Fleet;
using HordeServer.Agents.Pools;
using HordeServer.Agents.Utilization;
using HordeServer.Artifacts;
using HordeServer.Auditing;
using HordeServer.Build.BuildHealth;
using HordeServer.Commits;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Issues.External;
using HordeServer.Jobs;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Schedules;
using HordeServer.Jobs.Templates;
using HordeServer.Jobs.TestData;
using HordeServer.Jobs.Timing;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.Notifications.Sinks;
using HordeServer.VersionControl.Perforce;
using HordeServer.Plugins;
using HordeServer.RCA;
using HordeServer.Replicators;
using HordeServer.Streams;
using HordeServer.Tasks;
using HordeServer.Ugs;
using HordeServer.Users;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using MongoDB.Bson.Serialization;
using HordeServer.RoboMerge;
using HordeServer.VersionControl;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the build plugin
	/// </summary>
	[Plugin("Build", GlobalConfigType = typeof(BuildConfig), ServerConfigType = typeof(BuildServerConfig))]
	public class BuildPlugin : IPluginStartup
	{
		readonly IServerInfo _serverInfo;
		readonly BuildServerConfig _staticConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public BuildPlugin(IServerInfo serverInfo, BuildServerConfig staticConfig)
		{
			_serverInfo = serverInfo;
			_staticConfig = staticConfig;
		}

		/// <summary>
		/// Static constructor
		/// </summary>
		static BuildPlugin()
		{
			BsonSerializer.RegisterSerializer(new JobStepRefIdSerializer());
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
			app.UseEndpoints(endpoints =>
			{
				endpoints.MapGrpcService<JobRpcService>();
			});
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection services)
		{
			services.AddSingleton<IDefaultAclModifier, BuildAclModifier>();
			services.AddSingleton<IPluginResponseFilter, BuildResponseFilter>();

			services.AddSingleton<ArtifactCollection>();
			services.AddSingleton<IArtifactCollection>(sp => sp.GetRequiredService<ArtifactCollection>());
			services.AddSingleton<IGraphCollection, GraphCollection>();
			services.AddSingleton<IssueCollection>();
			services.AddSingleton<IIssueCollection>(sp => sp.GetRequiredService<IssueCollection>());
			services.AddSingleton<ILogExtIssueProvider>(sp => sp.GetRequiredService<IssueCollection>());
			services.AddSingleton<IJobCollection, JobCollection>();
			services.AddSingleton<IJobStepRefCollection, JobStepRefCollection>();
			services.AddSingleton<IJobTimingCollection, JobTimingCollection>();
			services.AddSingleton<IBisectTaskCollection, BisectTaskCollection>();
			services.AddSingleton<IReplicatorCollection, ReplicatorCollection>();
			services.AddSingleton<IUgsMetadataCollection, UgsMetadataCollection>();
			services.AddSingleton<ISubscriptionCollection, SubscriptionCollection>();
			services.AddSingleton<IStreamCollection, StreamCollection>();
			services.AddSingleton<TemplateCollection>();
			services.AddSingleton<ITemplateCollection>(sp => sp.GetRequiredService<TemplateCollection>());
			services.AddSingleton<ITemplateCollectionInternal>(sp => sp.GetRequiredService<TemplateCollection>());
			services.AddSingleton<ITestDataCollection, TestDataCollection>();
			services.AddSingleton<ITestDataCollectionV2, TestDataCollectionV2>();
			services.AddSingleton<IUtilizationDataCollection, UtilizationDataCollection>();
			services.AddSingleton<ITemplateCollection, TemplateCollection>();
			services.AddSingleton<IBuildHealthFilterCollection, BuildHealthFilterCollection>();

			services.AddSingleton<IPoolSizeStrategyFactory, JobQueueStrategyFactory>();
			services.AddHostedService(sp => sp.GetRequiredService<ArtifactCollection>());
			services.AddSingleton<ICommitService, CommitService>();

			services.AddSingleton<DeviceService>();
			services.AddSingleton<IAuditLog<DeviceId>>(sp => sp.GetRequiredService<IAuditLogFactory<DeviceId>>().Create("Devices.Log", "DeviceId"));
			services.AddSingleton<IBlockCache>(sp => CreateBlockCache());
			services.AddSingleton<TestDataService>();
			services.AddSingleton<IAuditLog<TestId>>(sp => sp.GetRequiredService<IAuditLogFactory<TestId>>().Create("TestHealthAudits.Log", "TestId"));

			services.AddSingleton<IssueService>();
			services.AddSingleton<RcaQueueService>();
			services.AddSingleton<IRcaEnqueuer>(sp => sp.GetRequiredService<RcaQueueService>());
			services.AddSingleton(sp => new Lazy<IRcaProcessor?>(() => sp.GetService<IRcaProcessor>()));
			services.AddSingleton<JobService>();
			services.AddSingleton<IAuditLog<JobId>>(sp => sp.GetRequiredService<IAuditLogFactory<JobId>>().Create("Jobs.Log", "JobId"));
			services.AddSingleton<ILogExtAuthProvider>(sp => sp.GetRequiredService<JobService>());
			services.AddSingleton<INotificationService, NotificationService>();
			services.AddSingleton<UnsyncCache>();

			services.AddSingleton<ScheduleService>();
			services.AddSingleton<IAuditLog<ScheduleId>>(sp => sp.GetRequiredService<IAuditLogFactory<ScheduleId>>().Create("Streams.Log", "ScheduleId"));

			services.AddSingleton<INotificationTriggerCollection, NotificationTriggerCollection>();
			services.AddSingleton<IDeviceCollection, DeviceCollection>();

			services.AddSingleton<IConfigSource, PerforceConfigSource>();

			// Notifications can be triggered from any instance, so always make sure we're ticking the background task.
			services.AddHostedService(provider => (NotificationService)provider.GetRequiredService<INotificationService>());

			// Issues need to be assigned asynchronously on any pod
			services.AddHostedService(provider => provider.GetRequiredService<IssueService>());

			// RCA queue uses shared ticker for multi-instance coordination
			services.AddHostedService(provider => provider.GetRequiredService<RcaQueueService>());

			services.AddHostedService(provider => provider.GetRequiredService<IExternalIssueService>());

			// Task sources. Order of registration is important here; it dictates the priority in which sources are served.
			services.AddSingleton<JobTaskSource>();

			if (!_serverInfo.ReadOnlyMode)
			{
				services.AddHostedService<JobTaskSource>(provider => provider.GetRequiredService<JobTaskSource>());
				services.AddSingleton<ConformTaskSource>();
				services.AddHostedService<ConformTaskSource>(provider => provider.GetRequiredService<ConformTaskSource>());

				services.AddSingleton<ITaskSource, ConformTaskSource>(provider => provider.GetRequiredService<ConformTaskSource>());
				services.AddSingleton<ITaskSource, JobTaskSource>(provider => provider.GetRequiredService<JobTaskSource>());
			}

			if (_staticConfig.Commits.ReplicateMetadata)
			{
				services.AddSingleton<PerforceServiceCache>();
				services.AddSingleton<IPerforceService>(sp => sp.GetRequiredService<PerforceServiceCache>());
			}
			else
			{
				services.AddSingleton<PerforceService>();
				services.AddSingleton<IPerforceService>(sp => sp.GetRequiredService<PerforceService>());
			}

			services.AddSingleton<IVersionControlService>(sp => sp.GetRequiredService<IPerforceService>());
			services.AddSingleton<IChangelistFileReader, ChangelistFileReader>();

			services.AddSingleton<PerforceReplicator>();

			services.AddSingleton<PerforceLoadBalancer>();
			services.AddSingleton<ReplicationService>();

			if (_staticConfig.SlackToken != null)
			{
				services.AddSingleton<SlackNotificationSink>();
				services.AddSingleton<IAvatarService, SlackNotificationSink>(sp => sp.GetRequiredService<SlackNotificationSink>());
				services.AddSingleton<INotificationSink, SlackNotificationSink>(sp => sp.GetRequiredService<SlackNotificationSink>());
				services.AddSingleton<IRcaNotifier, SlackNotificationSink>(sp => sp.GetRequiredService<SlackNotificationSink>());
			}
			else
			{
				services.AddSingleton<IAvatarService, NullAvatarService>();
			}

			if (_staticConfig.JiraUrl != null)
			{
				services.AddSingleton<IExternalIssueService, JiraService>();
			}
			else
			{
				services.AddSingleton<IExternalIssueService, DefaultExternalIssueService>();
			}

			// RoboMerge merge graph service — always registered; inert when unconfigured (tick checks config each time).
			services.AddSingleton<RoboMergeService>();
			services.AddSingleton<IMergeGraphService>(sp => sp.GetRequiredService<RoboMergeService>());
			services.AddHostedService(sp => sp.GetRequiredService<RoboMergeService>());

			if (_serverInfo.IsRunModeActive(RunMode.Worker) && !_serverInfo.ReadOnlyMode)
			{
				services.AddHostedService<AgentReportService>();
				services.AddHostedService<BisectService>();
				services.AddHostedService(provider => provider.GetRequiredService<IssueService>());
				services.AddHostedService<IssueReportService>();
				services.AddHostedService<IssueTagService>();
				services.AddHostedService<JobExpirationService>();
				services.AddHostedService(provider => provider.GetRequiredService<PerforceLoadBalancer>());
				services.AddHostedService<PoolUpdateService>();
				services.AddHostedService<UtilizationDataService>();
				services.AddHostedService(provider => provider.GetRequiredService<DeviceService>());
				services.AddHostedService<DeviceReportService>();
				services.AddHostedService(provider => provider.GetRequiredService<TestDataService>());
				services.AddHostedService<TestHealthService>();

				if (_staticConfig.Commits.ReplicateMetadata)
				{
					services.AddHostedService(provider => provider.GetRequiredService<PerforceServiceCache>());
				}

				if (_staticConfig.Commits.ReplicateContent)
				{
					services.AddHostedService(provider => provider.GetRequiredService<ReplicationService>());
				}

				if (!_staticConfig.DisableSchedules)
				{
					services.AddHostedService(provider => provider.GetRequiredService<ScheduleService>());
				}

				if (_staticConfig.SlackToken != null)
				{
					services.AddHostedService(provider => provider.GetRequiredService<SlackNotificationSink>());
				}
			}
		}

		BlockCache CreateBlockCache()
		{
			DirectoryReference cacheDir = DirectoryReference.Combine(_serverInfo.DataDir, String.IsNullOrEmpty(_staticConfig.BlockCacheDir) ? "BlockCache" : _staticConfig.BlockCacheDir);
			return BlockCache.Create(cacheDir, (int)(_staticConfig.BlockCacheSizeBytes / (1024 * 1024 * 1024)));
		}
	}

	/// <summary>
	/// Helper methods for build config
	/// </summary>
	public static class BuildPluginExtensions
	{
		/// <summary>
		/// Configures the build plugin
		/// </summary>
		public static void AddBuildConfig(this IDictionary<PluginName, IPluginConfig> dictionary, BuildConfig buildConfig)
			=> dictionary[new PluginName("Build")] = buildConfig;

		/// <summary>
		/// Gets configuration for the build plugin
		/// </summary>
		public static BuildConfig GetBuildConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (BuildConfig)dictionary[new PluginName("Build")];

		/// <summary>
		/// Gets configuration for the build plugin
		/// </summary>
		public static bool TryGetBuildConfig(this IDictionary<PluginName, IPluginConfig> dictionary, [NotNullWhen(true)] out BuildConfig? buildConfig)
		{
			IPluginConfig? pluginConfig;
			if (dictionary.TryGetValue(new PluginName("Build"), out pluginConfig) && pluginConfig is BuildConfig newBuildConfig)
			{
				buildConfig = newBuildConfig;
				return true;
			}
			else
			{
				buildConfig = null;
				return false;
			}
		}
	}
}
