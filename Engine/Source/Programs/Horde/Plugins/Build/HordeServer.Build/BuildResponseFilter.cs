// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Dashboard;
using HordeServer.Plugins;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.Options;

namespace HordeServer
{
	class BuildResponseFilter : IPluginResponseFilter
	{
		readonly BuildServerConfig _staticBuildConfig;

		public BuildResponseFilter(IOptions<BuildServerConfig> staticBuildConfig)
		{
			_staticBuildConfig = staticBuildConfig.Value;
		}

		/// <inheritdoc/>
		public void Apply(HttpContext context, object response)
		{
			if (response is GetDashboardConfigResponse dashboardConfigResponse)
			{
				if (_staticBuildConfig.JiraUrl != null)
				{
					dashboardConfigResponse.ExternalIssueServiceName = "Jira";
					dashboardConfigResponse.ExternalIssueServiceUrl = _staticBuildConfig.JiraUrl.ToString().TrimEnd('/');
				}

				if (_staticBuildConfig.P4SwarmUrl != null)
				{
					dashboardConfigResponse.PerforceSwarmUrl = _staticBuildConfig.P4SwarmUrl.ToString().TrimEnd('/');
				}

				if (_staticBuildConfig.RobomergeUrl != null)
				{
					dashboardConfigResponse.RobomergeUrl = _staticBuildConfig.RobomergeUrl.ToString().TrimEnd('/');
				}

				if (_staticBuildConfig.CommitsViewerUrl != null)
				{
					dashboardConfigResponse.CommitsViewerUrl = _staticBuildConfig.CommitsViewerUrl.ToString().TrimEnd('/');
				}

				dashboardConfigResponse.DeviceProblemCooldownMinutes = _staticBuildConfig.DeviceProblemCooldownMinutes;
			}
		}
	}
}
