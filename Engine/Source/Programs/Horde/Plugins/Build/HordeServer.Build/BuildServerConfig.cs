// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage.Bundles;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Perforce;
using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Static configuration for the build plugin
	/// </summary>
	public class BuildServerConfig : PluginServerConfig
	{
		/// <summary>
		/// Perforce connections for use by the Horde server (not agents)
		/// </summary>
		public List<PerforceConnectionSettings> Perforce { get; set; } = new List<PerforceConnectionSettings>();

		/// <summary>
		/// Whether to use the local Perforce environment
		/// </summary>
		public bool UseLocalPerforceEnv { get; set; }

		/// <summary>
		/// Number of pooled perforce connections to keep
		/// </summary>
		public int PerforceConnectionPoolSize { get; set; } = 5;

		/// <summary>
		/// Whether to enable the conform task source.
		/// </summary>
		public bool EnableConformTasks { get; set; } = true;

		/// <summary>
		/// Url of P4 Swarm installation
		/// </summary>
		public Uri? P4SwarmUrl { get; set; }

		/// <summary>
		/// Url of Robomerge installation
		/// </summary>
		public Uri? RobomergeUrl { get; set; }

		/// <summary>
		/// Url of Commits Viewer
		/// </summary>
		public Uri? CommitsViewerUrl { get; set; }

		/// <summary>
		/// The Jira service account user name
		/// </summary>
		public string? JiraUsername { get; set; }

		/// <summary>
		/// The Jira service account API token
		/// </summary>
		public string? JiraApiToken { get; set; }

		/// <summary>
		/// The Uri for the Jira installation
		/// </summary>
		public Uri? JiraUrl { get; set; }

		/// <summary>
		/// The number of days shared device checkouts are held
		/// </summary>
		public int SharedDeviceCheckoutDays { get; set; } = 3;

		/// <summary>
		/// The number of cooldown minutes for device problems
		/// </summary>
		public int DeviceProblemCooldownMinutes { get; set; } = 10;

		/// <summary>
		/// Channel to send device reports to
		/// </summary>
		public string? DeviceReportChannel { get; set; }

		/// <summary>
		/// Whether to disable bisection tasks
		/// </summary>
		public bool DisableBisection { get; set; } = true;

		/// <summary>
		/// Whether to run scheduled jobs.
		/// </summary>
		public bool DisableSchedules { get; set; }

		/// <summary>
		/// Bot token for interacting with Slack (xoxb-*)
		/// </summary>
		public string? SlackToken { get; set; }

		/// <summary>
		/// Token for opening a socket to slack (xapp-*)
		/// </summary>
		public string? SlackSocketToken { get; set; }

		/// <summary>
		/// Admin user token for Slack (xoxp-*). This is only required when using the admin endpoints to invite users.
		/// </summary>
		public string? SlackAdminToken { get; set; }

		/// <summary>
		/// Filtered list of slack users to send notifications to. Should be Slack user ids, separated by commas.
		/// </summary>
		public string? SlackUsers { get; set; }

		/// <summary>
		/// Prefix to use when reporting errors
		/// </summary>
		public string SlackErrorPrefix { get; set; } = ":horde-error: ";

		/// <summary>
		/// Prefix to use when reporting warnings
		/// </summary>
		public string SlackWarningPrefix { get; set; } = ":horde-warning: ";

		/// <summary>
		/// Channel for sending messages related to config update failures
		/// </summary>
		public string? ConfigNotificationChannel { get; set; }

		/// <summary>
		/// Channel to send stream notification update failures to
		/// </summary>
		public string? UpdateStreamsNotificationChannel { get; set; }

		/// <summary>
		/// Slack channel to send job related notifications to. Multiple channels can be specified, separated by ;
		/// </summary>
		public string? JobNotificationChannel { get; set; }

		/// <summary>
		/// Slack channel to send agent related notifications to.
		/// </summary>
		public string? AgentNotificationChannel { get; set; }

		/// <summary>
		/// The number of months to retain test data
		/// </summary>
		public int TestDataRetainMonths { get; set; } = 6;

		/// <summary>
		/// The number of days to go back in historical data to evaluate test health
		/// </summary>
		public int TestHealthHistoryEvaluationDays { get; set; } = 14;

		/// <summary>
		/// The number of days after which a test health report remind for improvement
		/// </summary>
		public int TestHealthReportRemindDays { get; set; } = 14;

		/// <summary>
		/// Directory to store the fine-grained block cache. This caches individual exports embedded in bundles.
		/// </summary>
		public string? BlockCacheDir { get; set; }

		/// <summary>
		/// Maximum size of the block cache. Accepts standard binary suffixes. Currently only allocates in multiples of 1024mb.
		/// </summary>
		public string BlockCacheSize { get; set; } = "4gb";

		/// <summary>
		/// Accessor for the block cache size in bytes
		/// </summary>
		public long BlockCacheSizeBytes => StringUtils.ParseBytesString(BlockCacheSize);

		/// <summary>
		/// Options for the commit service
		/// </summary>
		public CommitSettings Commits { get; set; } = new CommitSettings();
	}

	/// <summary>
	/// Identifier for a pool
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[StringIdConverter(typeof(PerforceConnectionIdConverter))]
	[TypeConverter(typeof(StringIdTypeConverter<PerforceConnectionId, PerforceConnectionIdConverter>))]
	public record struct PerforceConnectionId(StringId Id)
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceConnectionId(string id) : this(new StringId(id))
		{
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="StringId"/> instances.
	/// </summary>
	class PerforceConnectionIdConverter : StringIdConverter<PerforceConnectionId>
	{
		/// <inheritdoc/>
		public override PerforceConnectionId FromStringId(StringId id) => new PerforceConnectionId(id);

		/// <inheritdoc/>
		public override StringId ToStringId(PerforceConnectionId value) => value.Id;
	}

	/// <summary>
	/// Perforce connection information for use by the Horde server (for reading config files, etc...)
	/// </summary>
	public class PerforceConnectionSettings
	{
		/// <summary>
		/// Identifier for the default perforce connection profile
		/// </summary>
		public static PerforceConnectionId Default { get; } = new PerforceConnectionId("default");

		/// <summary>
		/// Identifier for this server
		/// </summary>
		public PerforceConnectionId Id { get; set; } = Default;

		/// <summary>
		/// Server and port
		/// </summary>
		public string? ServerAndPort { get; set; }

		/// <summary>
		/// Credentials for the server
		/// </summary>
		public PerforceCredentials? Credentials { get; set; }

		/// <summary>
		/// Create a <see cref="PerforceSettings"/> object with these settings as overrides
		/// </summary>
		/// <returns>New perforce settings object</returns>
		public PerforceSettings ToPerforceSettings()
		{
			PerforceSettings settings = new PerforceSettings(PerforceSettings.Default);
			settings.PreferNativeClient = true;

			if (!String.IsNullOrEmpty(ServerAndPort))
			{
				settings.ServerAndPort = ServerAndPort;
			}
			if (Credentials != null)
			{
				if (!String.IsNullOrEmpty(Credentials.UserName))
				{
					settings.UserName = Credentials.UserName;
				}

				if (!String.IsNullOrEmpty(Credentials.Ticket))
				{
					settings.Password = Credentials.Ticket;
				}
				else if (!String.IsNullOrEmpty(Credentials.Password))
				{
					settings.Password = Credentials.Password;
				}
			}
			return settings;
		}
	}

	/// <summary>
	/// Options for the commit service
	/// </summary>
	public class CommitSettings
	{
		/// <summary>
		/// Whether to mirror commit metadata to the database
		/// </summary>
		public bool ReplicateMetadata { get; set; } = true;

		/// <summary>
		/// Whether to mirror commit data to storage
		/// </summary>
		public bool ReplicateContent { get; set; } = true;

		/// <summary>
		/// Options for how objects are packed together
		/// </summary>
		public BundleOptions Bundle { get; set; } = new BundleOptions();

		/// <summary>
		/// Options for how objects are sliced
		/// </summary>
		public ChunkingOptions Chunking { get; set; } = new ChunkingOptions();
	}
}
