// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using Grpc.Net.Client;
using Horde.Common.Rpc;
using HordeServer.Agents.Relay;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Commands;

/// <summary>
/// Run server in relay mode
/// </summary>
[Command("relay", "Run server in relay mode")]
class RelayCommand : Command
{
	/// <summary>
	/// Log verbosity level level (use Serilog levels such as debug, warning or information)
	/// </summary>
	[CommandLine("-LogLevel")]
	[Description("Log verbosity level level (use Serilog levels such as debug, warning or information)")]
	public string LogLevelStr { get; set; } = "information";

	/// <summary>
	/// Compute cluster ID this relay server belong to
	/// </summary>
	[CommandLine("-ClusterId=", Required = true)]
	[Description("Compute cluster ID this relay server belong to")]
	public string ClusterId { get; set; } = null!;

	/// <summary>
	/// Unique ID of this relay server instance (arbitrary string)
	/// </summary>
	[CommandLine("-ServerId=", Required = true)]
	[Description("Unique ID of this relay server instance (arbitrary string)")]
	public string AgentId { get; set; } = null!;

	/// <summary>
	/// gRPC URL to Horde server
	/// </summary>
	[CommandLine("-ServerUrl=", Required = true)]
	[Description("gRPC URL to Horde server")]
	public string ServerUrl { get; set; } = null!;

	/// <summary>
	/// IP addresses this relay server can be addressed at. Multiple IPs are separated with comma.
	/// </summary>
	[CommandLine("-ListenIps=", Required = true)]
	[Description("IP addresses this relay server can be addressed at. Multiple IPs are separated with comma.")]
	public string ListenIpsStr { get; set; } = null!;

	/// <summary>
	/// Run 'nft' executable with sudo
	/// </summary>
	[CommandLine("-RunWithSudo=")]
	[Description("Whether to run 'nft' executable with sudo (true/false)")]
	public bool RunWithSudo { get; set; } = true;

	/// <summary>
	/// Constructor
	/// </summary>
	public RelayCommand()
	{
	}

	/// <summary>
	/// Runs the service indefinitely
	/// </summary>
	/// <returns>Exit code</returns>
	public override async Task<int> ExecuteAsync(ILogger logger)
	{
		string[] listenIps = ListenIpsStr.Split(",");

		logger.LogInformation("     Agent ID: {AgentId}", AgentId);
		logger.LogInformation("   Cluster ID: {ClusterId}", ClusterId);
		logger.LogInformation("Run with sudo: {RunWithSudo}", RunWithSudo);
		logger.LogInformation("   Listen IPs: {ListenIps}", String.Join(' ', listenIps));
		logger.LogInformation("   Server URL: {ServerUrl}", ServerUrl);

		Nftables nftables = new(NullLogger<Nftables>.Instance) { RunWithSudo = RunWithSudo };
		await nftables.InitializeAsync(CancellationToken.None);

		AppContext.SetSwitch("System.Net.Http.SocketsHttpHandler.Http2UnencryptedSupport", true);

		using CancellationTokenSource cts = new();
		Console.CancelKeyPress += new((sender, args) =>
		{
			logger.LogInformation("Stopping service due to user request...");
			args.Cancel = true;
			cts.Cancel();
		});

		using GrpcChannel channel = GrpcChannel.ForAddress(ServerUrl);
		RelayRpc.RelayRpcClient relayGrpcClient = new(channel);
		AgentRelayClient relayClient = new(ClusterId, AgentId, listenIps.ToList(), nftables, relayGrpcClient, new DefaultClock(), logger);

		logger.LogInformation("Listening for port mappings...");
		await relayClient.ListenForPortMappingsAsync(cts.Token);
		return 0;
	}
}
