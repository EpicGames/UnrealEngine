// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Replicators;
using EpicGames.Horde.Streams;
using HordeServer.Replicators;
using HordeServer.Streams;
using Microsoft.Extensions.Configuration;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Commands.Test
{
	[Command("test", "replication", "Replicates commits for a particular change of changes from Perforce")]
	class TestReplicationCommand : Command
	{
		[CommandLine("-Stream=", Required = true)]
		public string StreamId { get; set; } = String.Empty;

		[CommandLine("-Replicator=", Required = true)]
		public string ReplicatorId { get; set; } = String.Empty;

		[CommandLine(Required = true)]
		public int Change { get; set; }

		[CommandLine]
		public bool Reset { get; set; }

		[CommandLine]
		public bool Clean { get; set; }

		readonly IConfiguration _configuration;
		readonly IServerStartup _serverStartup;
		readonly ILoggerProvider _loggerProvider;

		public TestReplicationCommand(IConfiguration configuration, IServerStartup serverStartup, ILoggerProvider loggerProvider)
		{
			_configuration = configuration;
			_serverStartup = serverStartup;
			_loggerProvider = loggerProvider;
		}

		public override async Task<int> ExecuteAsync(ILogger logger)
		{
			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.AddSingleton(_configuration);
			serviceCollection.AddSingleton(_loggerProvider);
			_serverStartup.ConfigureServices(serviceCollection);

			await using ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider();

			BuildConfig buildConfig = serviceProvider.GetRequiredService<IOptionsMonitor<BuildConfig>>().CurrentValue;

			PerforceReplicator perforceReplicator = serviceProvider.GetRequiredService<PerforceReplicator>();
			IStreamCollection streamCollection = serviceProvider.GetRequiredService<IStreamCollection>();

			StreamConfig? streamConfig;
			if (!buildConfig.TryGetStream(new StreamId(StreamId), out streamConfig))
			{
				throw new FatalErrorException($"Stream '{StreamId}' not found");
			}

			IReplicatorCollection replicatorCollection = serviceProvider.GetRequiredService<IReplicatorCollection>();

			ReplicatorId id = new ReplicatorId(new StreamId(StreamId), new StreamReplicatorId(ReplicatorId));

			ReplicatorConfig? replicatorConfig;
			if (!streamConfig.TryGetReplicator(new StreamReplicatorId(ReplicatorId), out replicatorConfig))
			{
				throw new FatalErrorException($"Replicator '{ReplicatorId}' not found");
			}

			IReplicator replicator = await replicatorCollection.GetOrAddAsync(id);

			while (replicator.Pause || replicator.Reset != Reset || replicator.Clean != Clean || replicator.NextChange != Change)
			{
				UpdateReplicatorOptions updateOptions = new UpdateReplicatorOptions { Pause = false, Reset = Reset, Clean = Clean, NextChange = Change };

				IReplicator? nextReplicator = await replicator.TryUpdateAsync(updateOptions);
				if (nextReplicator != null)
				{
					replicator = nextReplicator;
					break;
				}

				replicator = await replicatorCollection.GetOrAddAsync(id);
			}

			await perforceReplicator.RunOnceAsync(replicator, buildConfig, streamConfig, replicatorConfig, default);
			return 0;
		}
	}
}
