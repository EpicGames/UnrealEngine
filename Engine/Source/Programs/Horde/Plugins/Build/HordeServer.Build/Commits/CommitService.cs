// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using HordeServer.Streams;
using Microsoft.Extensions.Options;
using HordeServer.VersionControl;

namespace HordeServer.Commits
{
	/// <summary>
	/// Provides commit information for streams
	/// </summary>
	public class CommitService : ICommitService
	{
		readonly IOptionsMonitor<BuildConfig> _buildConfig;

		readonly Dictionary<string, IVersionControlService> _vcsServices = new Dictionary<string, IVersionControlService>(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Constructor
		/// </summary>
		public CommitService(IEnumerable<IVersionControlService> versionControlServices, IOptionsMonitor<BuildConfig> buildConfig)
		{
			_buildConfig = buildConfig;

			foreach (IVersionControlService service in versionControlServices)
			{
				if (!_vcsServices.TryAdd(service.Name, service))
				{
					throw new InvalidOperationException($"Duplicate VCS service registered with name '{service.Name}' (existing: {_vcsServices[service.Name].GetType().Name}, new: {service.GetType().Name})");
				}
			}
		}

		/// <inheritdoc/>
		public ICommitCollection GetCollection(StreamConfig streamConfig)
		{
			if (!_vcsServices.TryGetValue(streamConfig.VCS, out IVersionControlService? vcs))
			{
				throw new KeyNotFoundException($"Unknown VCS Service {streamConfig.VCS}");
			}

			return vcs.GetCommits(streamConfig);
		}

		/// <inheritdoc/>
		public async ValueTask<CommitIdWithOrder> GetOrderedAsync(StreamId streamId, CommitId commitId, CancellationToken cancellationToken = default)
		{
			if (!_buildConfig.CurrentValue.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				throw new StreamNotFoundException(streamId);
			}

			ICommitCollection commitCollection = GetCollection(streamConfig);
			return await commitCollection.GetOrderedAsync(commitId, cancellationToken);
		}
	}
}
