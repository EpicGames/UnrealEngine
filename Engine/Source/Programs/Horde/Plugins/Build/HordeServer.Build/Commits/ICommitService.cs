// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using HordeServer.Streams;

namespace HordeServer.Commits
{
	/// <summary>
	/// Provides information about commits to a stream
	/// </summary>
	public interface ICommitService
	{
		/// <summary>
		/// Gets a commit collection for the given stream
		/// </summary>
		/// <param name="streamConfig">Stream to get commits for</param>
		/// <returns>Collection object</returns>
		ICommitCollection GetCollection(StreamConfig streamConfig);

		/// <summary>
		/// Gets a numbered commit id
		/// </summary>
		/// <param name="streamId">Stream containing the commit</param>
		/// <param name="commitId">The commit to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Numbered commit id</returns>
		ValueTask<CommitIdWithOrder> GetOrderedAsync(StreamId streamId, CommitId commitId, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Exception thrown when a stream can't be found
	/// </summary>
	public sealed class StreamNotFoundException : Exception
	{
		/// <summary>
		/// The stream identifier
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamNotFoundException(StreamId streamId) : base($"Stream {streamId} not found")
			=> StreamId = streamId;
	}
}
