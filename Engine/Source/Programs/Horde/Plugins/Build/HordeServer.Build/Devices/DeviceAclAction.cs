// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Devices
{
	/// <summary>
	/// ACL actions which apply to devices
	/// </summary>
	public static class DeviceAclAction
	{
		/// <summary>
		/// Ability to read devices
		/// </summary>
		public static AclAction DeviceRead { get; } = new AclAction("DeviceRead");

		/// <summary>
		/// Ability to write devices
		/// </summary>
		public static AclAction DeviceWrite { get; } = new AclAction("DeviceWrite");
	}
}
