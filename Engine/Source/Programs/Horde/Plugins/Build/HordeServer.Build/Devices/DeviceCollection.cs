// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq.Expressions;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;

namespace HordeServer.Devices
{
	/// <summary>
	/// Collection of device documents
	/// </summary>
	public class DeviceCollection : IDeviceCollection
	{

		/// <summary>
		/// Document representing a reservation of devices
		/// </summary>
		class DeviceReservationDocument : IDeviceReservation
		{
			/// <summary>
			/// Randomly generated unique id for this reservation.
			/// </summary>
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonRequired]
			public DevicePoolId PoolId { get; set; }

			[BsonIgnoreIfNull]
			public string? StreamId { get; set; }

			[BsonIgnoreIfNull]
			public string? JobId { get; set; }

			/// <summary>
			/// Current Step Id
			/// </summary>
			[BsonIgnoreIfNull]
			public string? StepId { get; set; }

			[BsonIgnoreIfNull]
			public string? JobName { get; set; }

			[BsonIgnoreIfNull]
			public string? StepName { get; set; }

			/// <summary>
			/// Reservations held by a user, requires a token like download code
			/// </summary>
			[BsonIgnoreIfNull]
			public UserId? UserId { get; set; }

			/// <summary>
			/// The hostname of the machine which has made the reservation
			/// </summary>
			[BsonIgnoreIfNull]
			public string? Hostname { get; set; }

			/// <summary>
			/// Optional string holding details about the reservation
			/// </summary>
			[BsonIgnoreIfNull]
			public string? ReservationDetails { get; set; }

			/// <summary>
			/// DeviceIds in the reservation
			/// </summary>
			public List<DeviceId> Devices { get; set; } = new List<DeviceId>();

			public List<string> RequestedDevicePlatforms { get; set; } = new List<string>();

			public DateTime CreateTimeUtc { get; set; }

			public DateTime UpdateTimeUtc { get; set; }

			// Legacy Guid
			public string LegacyGuid { get; set; } = null!;

			[BsonIgnoreIfNull]
			public List<JobStepId>? ReservedStepIds { get; set; } = null;

			[BsonIgnoreIfNull]
			public DeviceId? ProblemDevice { get; set; } = null;

			[BsonConstructor]
			private DeviceReservationDocument()
			{

			}

			public DeviceReservationDocument(ObjectId id, DevicePoolId poolId, List<DeviceId> devices, List<string> requestedDevicePlatforms, DateTime createTimeUtc, string? hostname, string? reservationDetails, string? streamId, string? jobId, string? stepId, string? jobName, string? stepName, List<JobStepId>? reservedStepIds)
			{
				Id = id;
				PoolId = poolId;
				Devices = devices;
				RequestedDevicePlatforms = requestedDevicePlatforms;
				CreateTimeUtc = createTimeUtc;
				UpdateTimeUtc = createTimeUtc;
				Hostname = hostname;
				ReservationDetails = reservationDetails;
				StreamId = streamId;
				JobId = jobId;
				StepId = stepId;
				JobName = jobName;
				StepName = stepName;
				ReservedStepIds = reservedStepIds;
				LegacyGuid = Guid.NewGuid().ToString();
			}
		}

		/// <summary>
		/// Concrete implementation of an device document
		/// </summary>
		class DeviceDocument : IDevice
		{
			public static int CurrentVersion = 1;

			[BsonRequired, BsonId]
			public DeviceId Id { get; set; }

			public DevicePlatformId PlatformId { get; set; }

			public DevicePoolId PoolId { get; set; }

			public string Name { get; set; } = null!;

			public bool Enabled { get; set; }

			[BsonIgnoreIfNull]
			public string? ModelId { get; set; }

			[BsonIgnoreIfNull]
			public string? Address { get; set; }

			[BsonIgnoreIfNull]
			public string? CheckedOutByUser { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? CheckOutTime { get; set; }

			[BsonIgnoreIfNull]
			public bool? CheckoutExpiringNotificationSent { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? ProblemTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? MaintenanceTimeUtc { get; set; }

			/// <summary>
			/// The time the last clean started,
			/// </summary>
			[BsonIgnoreIfNull]
			public DateTime? CleanStartTimeUtc { get; set; }

			/// <summary>
			/// The time the least clean finished
			/// </summary>
			[BsonIgnoreIfNull]
			public DateTime? CleanFinishTimeUtc { get; set; }

			/// <summary>
			/// Whether a clean is required
			/// </summary>
			[BsonIgnoreIfNull]
			public bool? CleanRequired { get; set; }

			/// <summary>
			///  The last time this device was reserved, used to cycle devices for reservations
			/// </summary>
			[BsonIgnoreIfNull]
			public DateTime? ReservationTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public string? ModifiedByUser { get; set; }

			[BsonIgnoreIfNull]
			public string? Notes { get; set; }

			/// <summary>
			/// [DEPRECATED]
			/// </summary>
			[BsonIgnoreIfNull]
			public List<DeviceUtilizationTelemetry>? Utilization { get; set; }

			[BsonIgnoreIfNull]
			public int? Version { get; set; }

			[BsonIgnoreIfNull]
			public List<string>? Tags { get; set; }

			[BsonConstructor]
			private DeviceDocument()
			{
			}

			public DeviceDocument(DeviceId id, DevicePlatformId platformId, DevicePoolId poolId, string name, bool enabled, string? address, string? modelId, UserId? userId, List<string>? tags)
			{
				Id = id;
				PlatformId = platformId;
				PoolId = poolId;
				Name = name;
				Enabled = enabled;
				Address = address;
				ModelId = modelId;
				ModifiedByUser = userId?.ToString();
				Version = CurrentVersion;
				Tags = tags;
			}
		}

		/// <summary>
		/// Device telemetry information for an individual device 
		/// </summary>
		class DeviceTelemetryDocument : IDeviceTelemetry
		{
			/// <summary>
			/// Id of telemetry document
			/// </summary>
			[BsonRequired, BsonId]
			public ObjectId TelemetryId { get; set; }

			/// <summary>
			/// The device id
			/// </summary>
			[BsonRequired, BsonElement("did")]
			public DeviceId DeviceId { get; set; }

			/// <summary>
			/// The time this telemetry data was created
			/// </summary>
			[BsonRequired, BsonElement("c")]
			public DateTime CreateTimeUtc { get; set; }

			/// <summary>
			/// The stream id which utilized device
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("sid")]
			public string? StreamId { get; set; }

			/// <summary>
			/// The job id which utilized device
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("job")]
			public string? JobId { get; set; }

			/// <summary>
			/// The job name
			/// </summary>
			[BsonIgnoreIfNull]
			public string? JobName { get; set; }

			/// <summary>
			/// The job's step id
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("step")]
			public string? StepId { get; set; }

			/// <summary>
			/// The step name
			/// </summary>
			[BsonIgnoreIfNull]
			public string? StepName { get; set; }

			/// <summary>
			/// Reservation Id (transient, reservations are deleted upon expiration)
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("rid")]
			public ObjectId? ReservationId { get; set; }

			/// <summary>
			/// The time device was reserved
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("rs")]
			public DateTime? ReservationStartUtc { get; set; }

			/// <summary>
			/// The time device was freed
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("rf")]
			public DateTime? ReservationFinishUtc { get; set; }

			/// <summary>
			/// If the device reported a problem
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("p")]
			public DateTime? ProblemTimeUtc { get; set; }

			[BsonConstructor]
			private DeviceTelemetryDocument()
			{
			}

			public DeviceTelemetryDocument(DeviceId deviceId, ObjectId? reservationId = null, DateTime? reservationStartTime = null, string? streamId = null, string? jobId = null, string? stepId = null, string? jobName = null, string? stepName = null)
			{
				TelemetryId = ObjectId.GenerateNewId();
				CreateTimeUtc = DateTime.UtcNow;
				DeviceId = deviceId;

				ReservationId = reservationId;
				ReservationStartUtc = reservationStartTime;

				StreamId = streamId;
				JobId = jobId;
				StepId = stepId;
				JobName = jobName;
				StepName = stepName;
			}
		}

		class DeviceReservationPoolTelemetryDocument : IDevicePoolReservationTelemetry
		{
			/// <inheritdoc/>
			[BsonRequired, BsonElement("did")]
			public DeviceId DeviceId { get; set; }

			[BsonIgnoreIfNull, BsonElement("ji")]
			public string? JobId { get; set; }

			[BsonIgnoreIfNull, BsonElement("si")]
			public string? StepId { get; set; }

			[BsonIgnoreIfNull, BsonElement("jn")]
			public string? JobName { get; set; }

			[BsonIgnoreIfNull, BsonElement("sn")]
			public string? StepName { get; set; }

			[BsonConstructor]
			private DeviceReservationPoolTelemetryDocument()
			{
			}

			public DeviceReservationPoolTelemetryDocument(DeviceId deviceId, string? jobId, string? stepId, string? jobName, string? stepName)
			{
				DeviceId = deviceId;
				JobId = jobId;
				StepId = stepId;
				JobName = jobName;
				StepName = stepName;
			}
		}

		class DevicePlatformTelemetryDocument : IDevicePlatformTelemetry
		{
			[BsonRequired, BsonElement("pid")]
			public DevicePlatformId PlatformId { get; set; }

			[BsonIgnoreIfNull, BsonElement("a")]
			public List<DeviceId>? Available { get; set; }
			IReadOnlyList<DeviceId>? IDevicePlatformTelemetry.Available => Available;

			[BsonIgnoreIfNull, BsonElement("m")]
			public List<DeviceId>? Maintenance { get; set; }
			IReadOnlyList<DeviceId>? IDevicePlatformTelemetry.Maintenance => Maintenance;

			[BsonIgnoreIfNull, BsonElement("p")]
			public List<DeviceId>? Problem { get; set; }
			IReadOnlyList<DeviceId>? IDevicePlatformTelemetry.Problem => Problem;

			[BsonIgnoreIfNull, BsonElement("d")]
			public List<DeviceId>? Disabled { get; set; }
			IReadOnlyList<DeviceId>? IDevicePlatformTelemetry.Disabled => Disabled;

			[BsonIgnoreIfNull, BsonElement("r"), BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<StreamId, List<DeviceReservationPoolTelemetryDocument>>? Reserved { get; set; }
			IReadOnlyDictionary<StreamId, IReadOnlyList<IDevicePoolReservationTelemetry>>? IDevicePlatformTelemetry.Reserved => Reserved?.ToDictionary(kvp => kvp.Key, kvp => kvp.Value as IReadOnlyList<IDevicePoolReservationTelemetry>) ?? new Dictionary<StreamId, IReadOnlyList<IDevicePoolReservationTelemetry>>();

			[BsonConstructor]
			private DevicePlatformTelemetryDocument()
			{
			}

			public DevicePlatformTelemetryDocument(DevicePlatformId platformId, List<DeviceId>? available, Dictionary<StreamId, List<DeviceReservationPoolTelemetryDocument>>? reserved, List<DeviceId>? maintenance, List<DeviceId>? problem, List<DeviceId>? disabled)
			{
				PlatformId = platformId;

				if (available != null && available.Count > 0)
				{
					Available = available;
				}

				if (reserved != null && reserved.Count > 0)
				{
					Reserved = reserved;
				}

				if (maintenance != null && maintenance.Count > 0)
				{
					Maintenance = maintenance;
				}

				if (problem != null && problem.Count > 0)
				{
					Problem = problem;
				}

				if (disabled != null && disabled.Count > 0)
				{
					Disabled = disabled;
				}
			}
		}

		/// <summary>
		/// Device telemetry information for pools
		/// </summary>
		class DevicePoolTelemetryDocument : IDevicePoolTelemetry
		{
			/// <summary>
			/// Id of telemetry document
			/// </summary>
			[BsonRequired, BsonId]
			public ObjectId TelemetryId { get; set; }

			/// <summary>
			/// The time this telemetry data was created
			/// </summary>
			[BsonRequired]
			public DateTime CreateTimeUtc { get; set; }

			/// <summary>
			/// Pool platform state
			/// </summary>
			[BsonRequired]
			public Dictionary<DevicePoolId, List<DevicePlatformTelemetryDocument>> Pools { get; set; } = new Dictionary<DevicePoolId, List<DevicePlatformTelemetryDocument>>();

			IReadOnlyDictionary<DevicePoolId, IReadOnlyList<IDevicePlatformTelemetry>> IDevicePoolTelemetry.Pools => Pools.ToDictionary(kvp => kvp.Key, kvp => kvp.Value as IReadOnlyList<IDevicePlatformTelemetry>);

			[BsonConstructor]
			private DevicePoolTelemetryDocument()
			{
			}

			public DevicePoolTelemetryDocument(Dictionary<DevicePoolId, List<DevicePlatformTelemetryDocument>> pools)
			{
				TelemetryId = ObjectId.GenerateNewId();
				CreateTimeUtc = DateTime.UtcNow;
				Pools = pools;
			}
		}

		readonly IMongoCollection<DeviceDocument> _devices;

		readonly IMongoCollection<DeviceReservationDocument> _reservations;

		readonly IMongoCollection<DeviceTelemetryDocument> _deviceTelemetry;

		readonly IMongoCollection<DevicePoolTelemetryDocument> _poolTelemetry;

		readonly IUserCollection _userCollection;

		readonly IAuditLog<DeviceId> _auditLog;
		readonly ILogger<DeviceCollection> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public DeviceCollection(IMongoService mongoService, IUserCollection userCollection, ILogger<DeviceCollection> logger, IAuditLog<DeviceId> auditLog)
		{
			_auditLog = auditLog;
			_logger = logger;
			_userCollection = userCollection;

			_devices = mongoService.GetCollection<DeviceDocument>("Devices", keys => keys.Ascending(x => x.Name), unique: true);
			_reservations = mongoService.GetCollection<DeviceReservationDocument>("Devices.Reservations");

			List<MongoIndex<DeviceTelemetryDocument>> deviceTelemetryIndexes = new List<MongoIndex<DeviceTelemetryDocument>>();
			deviceTelemetryIndexes.Add((keys => keys.Ascending(x => x.CreateTimeUtc)));
			_deviceTelemetry = mongoService.GetCollection<DeviceTelemetryDocument>("Devices.DeviceTelemetryV2", deviceTelemetryIndexes);

			List<MongoIndex<DevicePoolTelemetryDocument>> poolTelemetryIndexes = new List<MongoIndex<DevicePoolTelemetryDocument>>();
			poolTelemetryIndexes.Add((keys => keys.Ascending(x => x.CreateTimeUtc)));
			_poolTelemetry = mongoService.GetCollection<DevicePoolTelemetryDocument>("Devices.PoolTelemetryV2", poolTelemetryIndexes);
		}

		/// <inheritdoc/>
		public async Task<IDevice?> TryAddDeviceAsync(DeviceId id, string name, DevicePlatformId platformId, DevicePoolId poolId, bool? enabled, string? address, string? modelId, UserId? userId, List<string>? tags = null)
		{
			DeviceDocument newDevice = new DeviceDocument(id, platformId, poolId, name, enabled ?? true, address, modelId, userId, tags);
			await _devices.InsertOneAsync(newDevice);
			return newDevice;
		}

		/// <inheritdoc/>
		public async Task<List<IDevice>> FindAllDevicesAsync(List<DeviceId>? deviceIds = null, DevicePoolId? poolId = null, DevicePlatformId? platformId = null)
		{
			FilterDefinition<DeviceDocument> filter = Builders<DeviceDocument>.Filter.Empty;
			if (deviceIds != null)
			{
				filter &= Builders<DeviceDocument>.Filter.In(x => x.Id, deviceIds);
			}
			if (poolId != null)
			{
				filter &= Builders<DeviceDocument>.Filter.Eq(x => x.PoolId, poolId);
			}
			if (platformId != null)
			{
				filter &= Builders<DeviceDocument>.Filter.Eq(x => x.PlatformId, platformId);
			}

			List<DeviceDocument> results = await _devices.Find(filter).ToListAsync();
			return results.OrderBy(x => x.Name).Select<DeviceDocument, IDevice>(x => x).ToList();
		}

		/// <inheritdoc/>
		public async Task<IDevice?> GetDeviceAsync(DeviceId deviceId)
		{
			return await _devices.Find<DeviceDocument>(x => x.Id == deviceId).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<IDevice?> GetDeviceByNameAsync(string deviceName)
		{
			return await _devices.Find<DeviceDocument>(x => x.Name == deviceName).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task CheckoutDeviceAsync(DeviceId deviceId, UserId? checkedOutByUserId)
		{
			UpdateDefinitionBuilder<DeviceDocument> updateBuilder = Builders<DeviceDocument>.Update;

			List<UpdateDefinition<DeviceDocument>> updates = new List<UpdateDefinition<DeviceDocument>>();

			string? userId = checkedOutByUserId?.ToString();

			updates.Add(updateBuilder.Set(x => x.CheckedOutByUser, String.IsNullOrEmpty(userId) ? null : userId));

			if (checkedOutByUserId != null)
			{
				updates.Add(updateBuilder.Set(x => x.CheckOutTime, DateTime.UtcNow));
				updates.Add(updateBuilder.Set(x => x.CheckoutExpiringNotificationSent, false));
				updates.Add(updateBuilder.Set(x => x.CleanRequired, false));
			}
			else
			{
				updates.Add(updateBuilder.Set(x => x.CheckoutExpiringNotificationSent, true));
				// Mark device as needing cleaning
				updates.Add(updateBuilder.Set(x => x.CleanRequired, true));
			}

			await _devices.FindOneAndUpdateAsync<DeviceDocument>(x => x.Id == deviceId, updateBuilder.Combine(updates));
		}

		/// <inheritdoc/>
		public async Task UpdateDeviceAsync(DeviceId deviceId, DevicePoolId? newPoolId, string? newName, string? newAddress, string? newModelId, string? newNotes, bool? newEnabled, bool? newProblem, bool? newMaintenance, UserId? modifiedByUserId = null, string? message = null, List<string>? newTags = null)
		{
			UpdateDefinitionBuilder<DeviceDocument> updateBuilder = Builders<DeviceDocument>.Update;
			List<UpdateDefinition<DeviceDocument>> updates = new List<UpdateDefinition<DeviceDocument>>();

			DateTime utcNow = DateTime.UtcNow;

			IAuditLogChannel<DeviceId> logger = GetDeviceLogger(deviceId);
			IDevice? current = await _devices.Find<DeviceDocument>(x => x.Id == deviceId).FirstOrDefaultAsync();
			IUser? user = null;
			if (modifiedByUserId != null)
			{
				user = await _userCollection.GetCachedUserAsync(modifiedByUserId);
			}

			bool log = false;
			if (current != null)
			{
				log = true;
			}

			if (modifiedByUserId != null)
			{
				updates.Add(updateBuilder.Set(x => x.ModifiedByUser, modifiedByUserId.ToString()));
				if (log && current!.ModifiedByUser != modifiedByUserId.ToString())
				{					
					logger.LogInformation("Modified by user {UserName}", user?.Name ?? modifiedByUserId.ToString());
				}				
			}

			if (newPoolId != null)
			{
				updates.Add(updateBuilder.Set(x => x.PoolId, newPoolId.Value));
				if (log && current!.PoolId != newPoolId.Value)
				{
					logger.LogInformation("Assigned to pool {PoolId}", newPoolId.Value.ToString());
				}				
			}

			if (newName != null)
			{
				updates.Add(updateBuilder.Set(x => x.Name, newName));
				if (log && current!.Name != newName)
				{
					if (user?.Name == null)
					{
						logger.LogInformation("Renamed to {DeviceName}", newName);
					}
					else
					{
						logger.LogInformation("Renamed to {DeviceName} by {UserName}", newName, user.Name);
					}					
				}				
			}

			if (newEnabled != null)
			{
				updates.Add(updateBuilder.Set(x => x.Enabled, newEnabled));

				if (log && current!.Enabled != newEnabled)
				{
					if (newEnabled.Value)
					{
						if (user?.Name == null)
						{
							logger.LogInformation("Device enabled");
						}
						else
						{
							logger.LogInformation("Device enabled by {UserName}", user.Name);
						}							
					}
					else
					{
						if (user?.Name == null)
						{
							logger.LogInformation("Device disabled");
						}
						else
						{
							logger.LogInformation("Device disabled by {UserName}", user.Name);
						}
					}
				}
			}

			if (newAddress != null)
			{
				updates.Add(updateBuilder.Set(x => x.Address, newAddress));
				if (log && current!.Address != newAddress)
				{
					if (user?.Name == null)
					{
						logger.LogInformation("Device address changed to {DeviceAddress}", newAddress);
					}
					else
					{
						logger.LogInformation("Device address changed to {DeviceAddress} by {UserName}", newAddress, user.Name);
					}
				}
			}

			if (!String.IsNullOrEmpty(newModelId))
			{
				if (newModelId == "Base")
				{
					updates.Add(updateBuilder.Set(x => x.ModelId, null));

					if (log && current!.ModelId != null)
					{
						if (user?.Name == null)
						{
							logger.LogInformation("Device model changed to Base");
						}
						else
						{
							logger.LogInformation("Device model changed to Base by {UserName}", user.Name);
						}
					}
				}
				else
				{
					updates.Add(updateBuilder.Set(x => x.ModelId, newModelId));
					if (log && current!.ModelId != newModelId)
					{						
						if (user?.Name == null)
						{
							logger.LogInformation("Device model changed to {DeviceModel}", newModelId);
						}
						else
						{
							logger.LogInformation("Device model changed to {DeviceModel} by {UserName}", newModelId, user.Name);
						}
					}
				}
			}

			if (newNotes != null)
			{
				updates.Add(updateBuilder.Set(x => x.Notes, newNotes));
			}

			if (newProblem.HasValue)
			{
				DateTime? problemTime = null;
				if (newProblem.Value)
				{
					problemTime = utcNow;
				}

				updates.Add(updateBuilder.Set(x => x.ProblemTimeUtc, problemTime));
			}

			if (newMaintenance.HasValue)
			{
				DateTime? maintenanceTime = null;
				if (newMaintenance.Value)
				{
					maintenanceTime = utcNow;
				}

				updates.Add(updateBuilder.Set(x => x.MaintenanceTimeUtc, maintenanceTime));

				if (log && current!.MaintenanceTimeUtc != maintenanceTime)
				{
					if (maintenanceTime != null)
					{
						if (user?.Name == null)
						{
							logger.LogInformation("Device put into maintenance");
						}
						else
						{
							logger.LogInformation("Device put into maintenance by {UserName}", user.Name);
						}
					}
					else
					{
						if (user?.Name == null)
						{
							logger.LogInformation("Device taken out of maintenance");
						}
						else
						{
							logger.LogInformation("Device taken out of maintenance by {UserName}", user.Name);
						}
					}
				}
			}

			if (newTags != null)
			{
				updates.Add(updateBuilder.Set(x => x.Tags, newTags));
			}

			if (updates.Count > 0)
			{
				await _devices.FindOneAndUpdateAsync<DeviceDocument>(x => x.Id == deviceId, updateBuilder.Combine(updates));
			}

			if (newProblem.HasValue && newProblem.Value)
			{
				IDeviceReservation? reservation = await TryGetDeviceReservationAsync(deviceId);
				if (reservation != null)
				{
					if (log)
					{
						 logger.LogInformation("Device reported a problem in step: {StepName} - job {JobId}, step {StepId})", reservation.StepName ?? "Unknown Step", reservation.JobId ?? "Unknown Job Id", reservation.StepId ?? "Unknown Step Id");
					}

					UpdateDefinition<DeviceTelemetryDocument> update = Builders<DeviceTelemetryDocument>.Update.Set(x => x.ProblemTimeUtc, utcNow);
					await _deviceTelemetry.FindOneAndUpdateAsync(t => t.ReservationId == reservation.Id && t.DeviceId == deviceId, update);
				}
				else
				{
					if (log)
					{
						logger.LogInformation("Device reported a problem without reservation information");
					}
				}
			}

			if (message != null)
			{
				if (log)
				{
					logger.LogInformation("{Message}", message);
				}
			}
		}

		/// <inheritdoc/>
		public async Task UpdateDeviceCleaningAsync(DeviceId deviceId, bool? cleaning = null, bool? cleanRequired = null)
		{
			UpdateDefinitionBuilder<DeviceDocument> updateBuilder = Builders<DeviceDocument>.Update;
			List<UpdateDefinition<DeviceDocument>> updates = new List<UpdateDefinition<DeviceDocument>>();
						
			IDevice? current = await _devices.Find<DeviceDocument>(x => x.Id == deviceId).FirstOrDefaultAsync();

			if (current == null)
			{
				_logger.LogInformation("Device cleaning updated for non-existant {DeviceId}", deviceId.ToString());
				return;
			}

			DateTime utcNow = DateTime.UtcNow;

			IAuditLogChannel<DeviceId> logger = GetDeviceLogger(deviceId);

			if (cleaning.HasValue)
			{
				if (cleaning.Value)
				{
					logger.LogInformation("Starting Clean");
					updates.Add(updateBuilder.Set(x => x.CleanStartTimeUtc, utcNow));
					updates.Add(updateBuilder.Set(x => x.CleanFinishTimeUtc, null));
					updates.Add(updateBuilder.Set(x => x.CleanRequired, false));
				}				
				else
				{
					logger.LogInformation("Finished Clean");
					updates.Add(updateBuilder.Set(x => x.CleanFinishTimeUtc, utcNow));
					updates.Add(updateBuilder.Set(x => x.CleanRequired, false));
				}
			}

			if (cleanRequired.HasValue)
			{
				logger.LogInformation("Marking as requiring a clean: {CleanRequired}", cleanRequired.Value);
				updates.Add(updateBuilder.Set(x => x.CleanRequired, cleanRequired.Value));
			}

			if (updates.Count > 0)
			{
				await _devices.FindOneAndUpdateAsync<DeviceDocument>(x => x.Id == deviceId, updateBuilder.Combine(updates));
			}
		}

		/// <inheritdoc/>
		public async Task<bool> DeleteDeviceAsync(DeviceId deviceId)
		{
			FilterDefinition<DeviceDocument> filter = Builders<DeviceDocument>.Filter.Eq(x => x.Id, deviceId);
			DeleteResult result = await _devices.DeleteOneAsync(filter);
			return result.DeletedCount > 0;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IDeviceReservation>> FindAllDeviceReservationsAsync(DevicePoolId? poolId = null, CancellationToken cancellationToken = default)
		{
			return await _reservations.Find(a => poolId == null || a.PoolId == poolId).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IDeviceReservation?> TryFindReserveBlockAsync(JobId? jobId, JobStepId? stepId)
		{
			if (jobId != null && stepId != null)
			{
				FilterDefinitionBuilder<DeviceReservationDocument> filterBuilder = Builders<DeviceReservationDocument>.Filter;
				FilterDefinition<DeviceReservationDocument> filter = filterBuilder.Empty;

				filter &= filterBuilder.Eq(x => x.JobId, jobId.ToString());
				filter &= filterBuilder.AnyEq(x => x.ReservedStepIds, stepId.Value);

				List<DeviceReservationDocument> results = await _reservations.Find(filter).ToListAsync();
				if (results.Count > 0)
				{
					return results[0];
				}
			}

			return null;
		}

		/// <inheritdoc/>
		public async Task<(IDeviceReservation?, bool)> TryAddReservationAsync(DevicePoolId poolId, List<DeviceRequestData> request, int problemCooldown, string? hostname, string? reservationDetails, IJob? job, JobStepId? stepId, string? stepName, List<JobStepId>? stepIds)
		{
			if (request.Count == 0)
			{
				return (null, false);
			}

			IDeviceReservation? existing = await TryFindReserveBlockAsync(job?.Id, stepId);
			if (existing != null && existing.ProblemDevice == null)
			{
				// Check the device models match
				bool modelChange = false;

				List<DeviceDocument> devices = await _devices.Find(x => existing.Devices.Contains(x.Id)).ToListAsync();
				foreach (DeviceRequestData data in request)
				{
					if (modelChange)
					{
						break;
					}

					foreach (DeviceDocument a in devices)
					{
						if (data.IncludeModels.Count > 0 && !data.IncludeModels.Contains(a.ModelId ?? "Base"))
						{
							modelChange = true;
						}

						if (data.ExcludeModels.Count > 0 && (data.ExcludeModels.Contains(a.ModelId ?? "Base")))
						{
							modelChange = true;
						}
					}
				}

				if (!modelChange)
				{
					return (existing, false);
				}
				else
				{
					_logger.LogWarning("Reservation block model changed on Job:{JobId} Step:{StepId}", job?.Id, stepId);
				}
			}

			HashSet<DeviceId> allocated = new HashSet<DeviceId>();
			Dictionary<DeviceId, string> platformRequestMap = new Dictionary<DeviceId, string>();

			List<DeviceReservationDocument> poolReservations = await _reservations.Find(x => x.PoolId == poolId).ToListAsync();

			DateTime reservationTimeUtc = DateTime.UtcNow;

			// Get available devices
			List<DeviceDocument> poolDevices = await _devices.Find(x =>
				x.PoolId == poolId &&
				x.Enabled &&
				x.MaintenanceTimeUtc == null).ToListAsync();

			// filter out problem devices
			poolDevices = poolDevices.FindAll(x => (x.ProblemTimeUtc == null || ((reservationTimeUtc - x.ProblemTimeUtc).Value.TotalMinutes > problemCooldown)));

			// filter out currently reserved devices
			poolDevices = poolDevices.FindAll(x => poolReservations.FirstOrDefault(p => p.Devices.Contains(x.Id)) == null);

			// sort to use last reserved first to cycle devices
			Comparison<DeviceDocument> rankByLastReservationTime = (a, b) =>
			{
				DateTime? aTime = a.ReservationTimeUtc;
				DateTime? bTime = b.ReservationTimeUtc;

				if (aTime == bTime)
				{
					return 0;
				}

				if (aTime == null && bTime != null)
				{
					return -1;
				}
				if (aTime != null && bTime == null)
				{
					return 1;
				}

				return aTime < bTime ? -1 : 1;
			};

			
			poolDevices.Sort(rankByLastReservationTime);

			if (existing?.ProblemDevice != null)
			{
				poolDevices = poolDevices.Where(x => x.Id != existing.ProblemDevice).ToList();
			}

			foreach (DeviceRequestData data in request)
			{
				List<DeviceDocument> rankedDevices = new List<DeviceDocument>(poolDevices);
				
				if (data.Tags != null)
				{
					bool hasPreferences = data.Tags.PreferredTags != null && data.Tags.PreferredTags.Count != 0;
					bool hasUndesired = data.Tags.UndesiredTags != null && data.Tags.UndesiredTags.Count != 0;

					if (hasPreferences || hasUndesired)
					{
						// if there are preferred or undesired tags in the request, sort the devices by ranking preferred highest, undesired lowest.
						// if a device happens to have both a preferred and undesired tag, it's sort order will be ignored
						rankedDevices.Sort((a, b) =>
						{
							int aPreferred = hasPreferences && a.Tags?.Count > 0 ? data.Tags.PreferredTags!.Count(tag => a.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)) : 0;
							int aUndesired = hasUndesired && a.Tags?.Count > 0 ? data.Tags.UndesiredTags!.Count(tag => a.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)) : 0;
							
							int bPreferred = hasPreferences && b.Tags?.Count > 0 ? data.Tags.PreferredTags!.Count(tag => b.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)) : 0;
							int bUndesired = hasUndesired && b.Tags?.Count > 0 ? data.Tags.UndesiredTags!.Count(tag => b.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)) : 0;

							bool aIsPreferred = aPreferred > 0 && aUndesired == 0;
							bool bIsPreferred = bPreferred > 0 && bUndesired == 0;
							
							bool aIsUndesired = aUndesired > 0 && aPreferred == 0;
							bool bIsUndesired = bUndesired > 0 && bPreferred == 0;

							// Sort preferred up
							if (aIsPreferred && !bIsPreferred)
							{
								return -1;
							}
							if (bIsPreferred && !aIsPreferred)
							{
								return 1;
							}
							if (aIsPreferred && bIsPreferred)
							{
								if (aPreferred == bPreferred)
								{
									return rankByLastReservationTime(a, b);
								}
								return aPreferred > bPreferred ? -1 : 1;
							}

							// Sort undesired down
							if (aIsUndesired && !bIsUndesired)
							{
								return 1;
							}
							if (bIsUndesired && !aIsUndesired)
							{
								return -1;
							}
							if (aIsUndesired && bIsUndesired)
							{
								if (aUndesired == bUndesired)
								{
									return rankByLastReservationTime(a, b);
								}
								return aUndesired > bUndesired ? 1 : -1;
							}

							return rankByLastReservationTime(a, b);
						});
					}
				}
				
				DeviceDocument? device = rankedDevices.FirstOrDefault(a =>
				{
					if (allocated.Contains(a.Id) || a.PlatformId != data.PlatformId)
					{
						return false;
					}

					if (data.IncludeModels.Count > 0 && !data.IncludeModels.Contains(a.ModelId ?? "Base"))
					{
						return false;
					}

					if (data.ExcludeModels.Count > 0 && (data.ExcludeModels.Contains(a.ModelId ?? "Base")))
					{
						return false;
					}

					if (!String.IsNullOrEmpty(data.DeviceName) && !a.Name.Equals(data.DeviceName, StringComparison.OrdinalIgnoreCase))
					{
						return false;
					}

					if (data.Tags != null)
					{
						if (data.Tags.RequiredTags != null && data.Tags.RequiredTags.Count > 0)
						{
							if (!data.Tags.RequiredTags.All(tag => a.Tags != null && a.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)))
							{
								return false;
							}
						}

						if (data.Tags.BlockedTags != null && data.Tags.BlockedTags.Count > 0)
						{
							if (data.Tags.BlockedTags.Any(tag => a.Tags != null && a.Tags.Contains(tag, StringComparer.OrdinalIgnoreCase)))
							{
								return false;
							}
						}
					}
					
					return true;
				});

				if (device == null)
				{
					// can't fulfill request
					return (null, false);
				}

				allocated.Add(device.Id);
				platformRequestMap.Add(device.Id, data.RequestedPlatform);
			}

			// update reservation time and utilization for allocated devices 
			foreach (DeviceId id in allocated)
			{
				DeviceDocument device = poolDevices.First((device) => device.Id == id);

				List<DeviceUtilizationTelemetry>? utilization = device.Utilization;

				utilization ??= new List<DeviceUtilizationTelemetry>();

				// keep up to 100, maintaining order
				if (utilization.Count > 99)
				{
					utilization = utilization.GetRange(0, 99);
				}

				utilization.Insert(0, new DeviceUtilizationTelemetry(reservationTimeUtc) { JobId = job?.Id.ToString(), StepId = stepId?.ToString() });

				UpdateDefinitionBuilder<DeviceDocument> deviceBuilder = Builders<DeviceDocument>.Update;
				List<UpdateDefinition<DeviceDocument>> deviceUpdates = new List<UpdateDefinition<DeviceDocument>>();

				deviceUpdates.Add(deviceBuilder.Set(x => x.ReservationTimeUtc, reservationTimeUtc));
				deviceUpdates.Add(deviceBuilder.Set(x => x.Utilization, utilization));

				await _devices.FindOneAndUpdateAsync<DeviceDocument>(x => x.Id == id, deviceBuilder.Combine(deviceUpdates));
			}

			List<DeviceId> deviceIds = allocated.ToList();
			List<string> requestedPlatforms = deviceIds.Select(x => platformRequestMap[x]).ToList();

			IDeviceReservation? returnValue = null;

			// Create new reservation
			if (existing == null)
			{
				DeviceReservationDocument newReservation = new DeviceReservationDocument(ObjectId.GenerateNewId(), poolId, deviceIds, requestedPlatforms, reservationTimeUtc, hostname, reservationDetails, job?.StreamId.ToString(), job?.Id.ToString(), stepId?.ToString(), job?.Name, stepName, stepIds);
				await _reservations.InsertOneAsync(newReservation);
				returnValue = newReservation;
			}
			else
			{
				// update current telemetry telemetry
				await _deviceTelemetry.UpdateManyAsync(x => x.ReservationId == existing.Id, Builders<DeviceTelemetryDocument>.Update.Set(x => x.ReservationFinishUtc, DateTime.UtcNow));

				returnValue = await TryUpdateReservationAsync(existing.Id, deviceIds: deviceIds, clearProblemDevice: true);
			}

			// Create device telemetry data for reservation
			List<DeviceTelemetryDocument> telemetry = new List<DeviceTelemetryDocument>();
			foreach (DeviceId deviceId in deviceIds)
			{
				telemetry.Add(new DeviceTelemetryDocument(deviceId, returnValue!.Id, returnValue!.CreateTimeUtc, job?.StreamId.ToString(), job?.Id.ToString(), stepId?.ToString(), job?.Name, stepName));
			}

			if (telemetry.Count > 0)
			{
				await _deviceTelemetry.InsertManyAsync(telemetry);
			}

			// only return needs install for reserve blocks 
			return (returnValue, existing != null || (stepIds != null && stepIds.Count > 0));
		}

		/// <inheritdoc/>
		public async Task<IDeviceReservation?> TryUpdateReservationAsync(ObjectId id, DeviceId? problemDevice = null, List<DeviceId>? deviceIds = null, bool? clearProblemDevice = null)
		{
			UpdateDefinitionBuilder<DeviceReservationDocument> updateBuilder = Builders<DeviceReservationDocument>.Update;
			List<UpdateDefinition<DeviceReservationDocument>> updates = new List<UpdateDefinition<DeviceReservationDocument>>();

			updates.Add(updateBuilder.Set(x => x.UpdateTimeUtc, DateTime.UtcNow));
			if (problemDevice.HasValue)
			{
				updates.Add(updateBuilder.Set(x => x.ProblemDevice, problemDevice.Value));
			}

			if (deviceIds != null)
			{
				updates.Add(updateBuilder.Set(x => x.Devices, deviceIds));
			}

			if (clearProblemDevice == true)
			{
				updates.Add(updateBuilder.Set(x => x.ProblemDevice, null));
			}

			Expression<Func<DeviceReservationDocument, bool>> filter = x => x.Id == id;
			return await _reservations.FindOneAndUpdateAsync(filter, updateBuilder.Combine(updates), options: new FindOneAndUpdateOptions<DeviceReservationDocument, DeviceReservationDocument> { ReturnDocument = ReturnDocument.After });
		}

		/// <inheritdoc/>
		public async Task<bool> DeleteReservationAsync(ObjectId id)
		{
			FilterDefinition<DeviceTelemetryDocument> telemetryFilter = Builders<DeviceTelemetryDocument>.Filter.Eq(x => x.ReservationId, id);

			// update telemetry
			await _deviceTelemetry.UpdateManyAsync(x => x.ReservationId == id, Builders<DeviceTelemetryDocument>.Update.Set(x => x.ReservationFinishUtc, DateTime.UtcNow));

			FilterDefinition<DeviceReservationDocument> filter = Builders<DeviceReservationDocument>.Filter.Eq(x => x.Id, id);
			DeleteResult result = await _reservations.DeleteOneAsync(filter);
			return result.DeletedCount > 0;
		}

		/// <summary>
		/// Deletes expired user checkouts
		/// </summary>
		public async Task<List<(UserId, IDevice)>?> ExpireCheckedOutAsync(int checkoutDays)
		{
			FilterDefinition<DeviceDocument> filter = Builders<DeviceDocument>.Filter.Empty;
			filter &= Builders<DeviceDocument>.Filter.Where(x => x.CheckedOutByUser != null && x.CheckOutTime != null);

			List<DeviceDocument> checkedOutDevices = await _devices.Find(filter).ToListAsync();

			DateTime utcNow = DateTime.UtcNow;
			List<DeviceDocument> expiredDevices = checkedOutDevices.FindAll(x => (utcNow - x.CheckOutTime!.Value).TotalDays >= checkoutDays).ToList();

			if (expiredDevices.Count > 0)
			{

				FilterDefinition<DeviceDocument> updateFilter = Builders<DeviceDocument>.Filter.In(x => x.Id, expiredDevices.Select(y => y.Id));

				UpdateDefinitionBuilder<DeviceDocument> deviceBuilder = Builders<DeviceDocument>.Update;
				List<UpdateDefinition<DeviceDocument>> deviceUpdates = new List<UpdateDefinition<DeviceDocument>>();

				deviceUpdates.Add(deviceBuilder.Set(x => x.CheckedOutByUser, null));
				deviceUpdates.Add(deviceBuilder.Set(x => x.CheckOutTime, null));
				deviceUpdates.Add(deviceBuilder.Set(x => x.CheckoutExpiringNotificationSent, true));
				deviceUpdates.Add(deviceBuilder.Set(x => x.CleanRequired, true));

				UpdateResult result = await _devices.UpdateManyAsync(Builders<DeviceDocument>.Filter.In(x => x.Id, expiredDevices.Select(y => y.Id)), deviceBuilder.Combine(deviceUpdates));

				if (result.ModifiedCount > 0)
				{
					return expiredDevices.Select(x => (UserId.Parse(x.CheckedOutByUser!), (IDevice)x)).ToList();
				}
			}

			return null;

		}

		/// <summary>
		/// Gets a list of users to notify  that their device is about to expire in the next 24 hours
		/// </summary>
		public async Task<List<(UserId, IDevice)>?> ExpireNotificatonsAsync(int checkoutDays)
		{
			FilterDefinition<DeviceDocument> filter = Builders<DeviceDocument>.Filter.Empty;
			filter &= Builders<DeviceDocument>.Filter.Where(x => x.CheckedOutByUser != null && x.CheckoutExpiringNotificationSent != true);

			List<DeviceDocument> checkedOutDevices = await _devices.Find(filter).ToListAsync();

			DateTime utcNow = DateTime.UtcNow;
			List<DeviceDocument> expiredDevices = checkedOutDevices.FindAll(x => (utcNow - x.CheckOutTime!.Value).TotalDays >= (checkoutDays - 1)).ToList();

			if (expiredDevices.Count > 0)
			{
				FilterDefinition<DeviceDocument> updateFilter = Builders<DeviceDocument>.Filter.In(x => x.Id, expiredDevices.Select(y => y.Id));

				UpdateDefinitionBuilder<DeviceDocument> deviceBuilder = Builders<DeviceDocument>.Update;
				List<UpdateDefinition<DeviceDocument>> deviceUpdates = new List<UpdateDefinition<DeviceDocument>>();

				deviceUpdates.Add(deviceBuilder.Set(x => x.CheckoutExpiringNotificationSent, true));

				UpdateResult result = await _devices.UpdateManyAsync(Builders<DeviceDocument>.Filter.In(x => x.Id, expiredDevices.Select(y => y.Id)), deviceBuilder.Combine(deviceUpdates));

				if (result.ModifiedCount > 0)
				{
					return expiredDevices.Select(x => (UserId.Parse(x.CheckedOutByUser!), (IDevice)x)).ToList();
				}
			}

			return null;
		}

		/// <inheritdoc/>
		public async Task<List<IDeviceReservation>> FindAllReservationsAsync()
		{
			List<DeviceReservationDocument> results = await _reservations.Find(x => true).ToListAsync();
			return results.OrderBy(x => x.CreateTimeUtc.Ticks).Select<DeviceReservationDocument, IDeviceReservation>(x => x).ToList();
		}

		/// <inheritdoc/>
		public async Task<IDeviceReservation?> TryGetReservationAsync(ObjectId reservationId)
		{
			return await _reservations.Find<DeviceReservationDocument>(r => r.Id == reservationId).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<IDeviceReservation?> TryGetReservationFromLegacyGuidAsync(string legacyGuid)
		{
			return await _reservations.Find<DeviceReservationDocument>(r => r.LegacyGuid == legacyGuid).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<IDeviceReservation?> TryGetDeviceReservationAsync(DeviceId id)
		{
			List<DeviceReservationDocument> results = await _reservations.Find(x => true).ToListAsync();
			return results.FirstOrDefault(r => r.Devices.Contains(id));
		}

		/// <inheritdoc/>
		public async Task<List<IDeviceTelemetry>> FindDeviceTelemetryAsync(DeviceId[]? deviceIds = null, DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null)
		{
			FilterDefinitionBuilder<DeviceTelemetryDocument> filterBuilder = Builders<DeviceTelemetryDocument>.Filter;
			FilterDefinition<DeviceTelemetryDocument> filter = filterBuilder.Empty;

			if (deviceIds != null && deviceIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.DeviceId, deviceIds);
			}

			if (minCreateTime != null)
			{
				filter &= filterBuilder.Gte(x => x.CreateTimeUtc!, minCreateTime.Value.UtcDateTime);
			}

			if (maxCreateTime != null)
			{
				filter &= filterBuilder.Lte(x => x.CreateTimeUtc!, maxCreateTime.Value.UtcDateTime);
			}

			List<DeviceTelemetryDocument> results = await _deviceTelemetry.Find(filter).Range(index, count).ToListAsync();
			return results.ConvertAll<IDeviceTelemetry>(x => x);
		}

		struct DevicePoolTelemetryHelper
		{
			public List<DeviceId> _available = new List<DeviceId>();
			public List<DeviceId> _problem = new List<DeviceId>();
			public List<DeviceId> _maintenance = new List<DeviceId>();
			public List<DeviceId> _disabled = new List<DeviceId>();
			public Dictionary<StreamId, List<DeviceReservationPoolTelemetryDocument>> _reserved = new Dictionary<StreamId, List<DeviceReservationPoolTelemetryDocument>>();

			public DevicePoolTelemetryHelper() { }
		}

		/// <summary>
		/// Create a device pool telemetry snapshot
		/// </summary>
		/// <returns></returns>
		public async Task CreatePoolTelemetrySnapshotAsync(List<IDevicePool> pools, int problemCooldown)
		{
			List<IDevice> devices = await FindAllDevicesAsync();
			IReadOnlyList<IDeviceReservation> reservations = await FindAllDeviceReservationsAsync();

			// narrow to automation pools, may want to collect telemetry on other pools in the future
			pools = pools.Where(x => x.PoolType == DevicePoolType.Automation).ToList();
			devices = devices.Where(x => pools.FirstOrDefault(p => p.Id == x.PoolId) != null).ToList();

			if (devices.Count == 0 || pools.Count == 0)
			{
				return;
			}

			DateTime now = DateTime.UtcNow;

			List<IDevice> reservedDevices = devices.Where(x => reservations.FirstOrDefault(r => r.Devices.Contains(x.Id)) != null).ToList();
			List<IDevice> maintenanceDevices = devices.Where(x => x.MaintenanceTimeUtc != null).ToList();
			List<IDevice> disabledDevices = devices.Where(x => !x.Enabled).ToList();
			List<IDevice> problemDevices = devices.Where(x => (x.ProblemTimeUtc != null && ((now - x.ProblemTimeUtc).Value.TotalMinutes < problemCooldown))).ToList();

			Dictionary<DevicePoolId, List<DevicePlatformTelemetryDocument>> poolTelemetry = new Dictionary<DevicePoolId, List<DevicePlatformTelemetryDocument>>();

			foreach (IDevicePool pool in pools)
			{
				List<IDevice> poolDevices = devices.Where(x => x.PoolId == pool.Id).ToList();
				HashSet<DevicePlatformId> platforms = new HashSet<DevicePlatformId>();
				poolDevices.ForEach(d => platforms.Add(d.PlatformId));

				Dictionary<DevicePlatformId, DevicePoolTelemetryHelper> helpers = new Dictionary<DevicePlatformId, DevicePoolTelemetryHelper>();
				foreach (DevicePlatformId platform in platforms)
				{
					helpers[platform] = new DevicePoolTelemetryHelper();
				}

				foreach (IDevice device in poolDevices)
				{
					DevicePoolTelemetryHelper helper = helpers[device.PlatformId];

					if (reservedDevices.Contains(device))
					{
						IDeviceReservation? reservation = reservations.FirstOrDefault(x => x.Devices.Contains(device.Id));
						if (reservation != null && reservation.StreamId != null)
						{
							StreamId streamId = new StreamId(reservation.StreamId);
							List<DeviceReservationPoolTelemetryDocument>? rdevices;
							if (!helper._reserved.TryGetValue(streamId, out rdevices))
							{
								rdevices = new List<DeviceReservationPoolTelemetryDocument>();
								helper._reserved[streamId] = rdevices;
							}
							rdevices.Add(new DeviceReservationPoolTelemetryDocument(device.Id, reservation.JobId, reservation.StepId, reservation.JobName, reservation.StepName));
						}
						continue;
					}

					if (problemDevices.Contains(device))
					{
						helper._problem.Add(device.Id);
						continue;
					}

					if (maintenanceDevices.Contains(device))
					{
						helper._maintenance.Add(device.Id);
						continue;
					}

					if (disabledDevices.Contains(device))
					{
						helper._disabled.Add(device.Id);
						continue;
					}

					helper._available.Add(device.Id);
				}

				List<DevicePlatformTelemetryDocument> platformTelemtry = new List<DevicePlatformTelemetryDocument>();

				foreach (KeyValuePair<DevicePlatformId, DevicePoolTelemetryHelper> platform in helpers)
				{
					DevicePoolTelemetryHelper helper = platform.Value;

					platformTelemtry.Add(new DevicePlatformTelemetryDocument(platform.Key, helper._available, helper._reserved, helper._maintenance, helper._problem, helper._disabled));
				}

				poolTelemetry[pool.Id] = platformTelemtry;
			}

			await _poolTelemetry.InsertOneAsync(new DevicePoolTelemetryDocument(poolTelemetry));
		}

		/// <inheritdoc/>
		public async Task<List<IDevicePoolTelemetry>> FindPoolTelemetryAsync(DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null)
		{
			FilterDefinitionBuilder<DevicePoolTelemetryDocument> filterBuilder = Builders<DevicePoolTelemetryDocument>.Filter;
			FilterDefinition<DevicePoolTelemetryDocument> filter = filterBuilder.Empty;

			if (minCreateTime != null)
			{
				filter &= filterBuilder.Gte(x => x.CreateTimeUtc!, minCreateTime.Value.UtcDateTime);
			}

			if (maxCreateTime != null)
			{
				filter &= filterBuilder.Lte(x => x.CreateTimeUtc!, maxCreateTime.Value.UtcDateTime);
			}

			List<DevicePoolTelemetryDocument> results = await _poolTelemetry.Find(filter).Range(index, count).ToListAsync();
			return results.ConvertAll<IDevicePoolTelemetry>(x => x);
		}

		/// <inheritdoc/>
		public async Task UpgradeAsync()
		{
			FilterDefinition<DeviceDocument> filter = Builders<DeviceDocument>.Filter.Empty;
			filter &= Builders<DeviceDocument>.Filter.Where(x => x.Version == null || x.Version < DeviceDocument.CurrentVersion);

			List<DeviceDocument> results = await _devices.Find(filter).ToListAsync();

			if (results.Count > 0)
			{
				_logger.LogInformation("Found {Count} device documents to upgrade", results.Count);
			}

			foreach (DeviceDocument device in results)
			{
				// unversioned => 1
				if (device.Version == null && DeviceDocument.CurrentVersion == 1)
				{
					UpdateDefinitionBuilder<DeviceDocument> updateBuilder = Builders<DeviceDocument>.Update;
					List<UpdateDefinition<DeviceDocument>> updates = new List<UpdateDefinition<DeviceDocument>>();

					updates.Add(updateBuilder.Set(x => x.Version, DeviceDocument.CurrentVersion));

					if (device.CheckOutTime != null)
					{
						DateTime now = DateTime.UtcNow;
						double days = (now - device.CheckOutTime!.Value).TotalDays;
						if (days > 3)
						{
							updates.Add(updateBuilder.Set(x => x.CheckOutTime, now));
							updates.Add(updateBuilder.Set(x => x.CheckoutExpiringNotificationSent, null));
						}
					}

					await _devices.FindOneAndUpdateAsync(x => x.Id == device.Id, updateBuilder.Combine(updates));
				}
			}
		}

		IAuditLogChannel<DeviceId> GetDeviceLogger(DeviceId deviceId)
		{
			return _auditLog[deviceId];
		}
	}
}
