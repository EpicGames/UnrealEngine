// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Text.Json.Serialization;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Projects;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Devices
{
	/// <summary>
	/// Configuration for devices
	/// </summary>
	public class DeviceConfig
	{
		/// <summary>
		/// List of device platforms
		/// </summary>
		public List<DevicePlatformConfig> Platforms { get; set; } = new List<DevicePlatformConfig>();

		/// <summary>
		/// List of device pools
		/// </summary>
		public List<DevicePoolConfig> Pools { get; set; } = new List<DevicePoolConfig>();
	}

	/// <summary>
	/// Configuration for a device platform 
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class DevicePlatformConfig : IDevicePlatform
	{
		/// <summary>
		/// The id for this platform 
		/// </summary>
		[Required]
		public DevicePlatformId Id { get; set; } = new DevicePlatformId("default");

		/// <summary>
		/// Name of the platform
		/// </summary>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// A list of platform models 
		/// </summary>
		public List<string>? Models { get; set; }
		IReadOnlyList<string>? IDevicePlatform.Models => Models;

		/// <summary>
		/// Legacy names which older versions of Gauntlet may be using
		/// </summary>
		public List<string>? LegacyNames { get; set; }
		IReadOnlyList<string>? IDevicePlatform.LegacyNames => LegacyNames;

		/// <summary>
		/// Model name for the high perf spec, which may be requested by Gauntlet
		/// </summary>
		public string? LegacyPerfSpecHighModel { get; set; }
	}

	/// <summary>
	/// Configuration for a device pool
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class DevicePoolConfig : IDevicePool
	{
		/// <summary>
		/// The id for this platform 
		/// </summary>
		[Required]
		public DevicePoolId Id { get; set; } = new DevicePoolId("default");

		/// <summary>
		/// The name of the pool
		/// </summary>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// The type of the pool
		/// </summary>
		[Required]
		[JsonConverter(typeof(JsonStringEnumConverter))]
		public DevicePoolType PoolType { get; set; } = DevicePoolType.Automation;

		/// <summary>
		/// List of project ids associated with pool
		/// </summary>
		public List<ProjectId>? ProjectIds { get; set; }
	}
}
