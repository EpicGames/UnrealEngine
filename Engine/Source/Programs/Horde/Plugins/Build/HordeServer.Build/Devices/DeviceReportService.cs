// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Devices;
using HordeServer.Jobs;
using HordeServer.Notifications;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Devices
{
#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member
	public class DeviceReport
	{
		public string DeviceId { get; }
		public string DeviceName { get; }
		public string DeviceAddress { get; }

		public string PlatformId { get; }
		public string PlatformName { get; }

		public string PoolId { get; }
		public string PoolName { get; }

		/// <summary>
		/// Problems reported since the last report
		/// </summary>
		public int ProblemDelta { get; set; } = 0;

		/// <summary>
		/// Percent of reported problems against the number of reservation (rounded down)
		/// </summary>
		public int ProblemPercent { get; set; } = 0;

		/// <summary>
		/// Time elapse in cleaning
		/// </summary>
		public TimeSpan? CleaningTime { get; set; }

		/// <summary>
		/// The last problem Description
		/// </summary>
		public string? LastProblemDesc { get; set; }

		/// <summary>
		/// The last problem encountered
		/// </summary>
		public string? LastProblemURL { get; set; }

		/// <summary>
		/// The URL to the device on the pool view
		/// </summary>
		public string? DevicePoolURL { get; set; }

		public List<IDeviceTelemetry> Telemetry { get; }

		public DeviceReport(string platformId, string platformName, string deviceId, string deviceName, string deviceAddress, string poolId, string poolName, List<IDeviceTelemetry> telemetry)
		{
			PlatformId = platformId;
			PlatformName = platformName;
			DeviceId = deviceId;
			DeviceName = deviceName;
			DeviceAddress = deviceAddress;
			PoolId = poolId;
			PoolName = poolName;
			Telemetry = telemetry;
		}
	}

	public class DevicePlatformReport
	{
		public string PlatformId { get; }
		public string PlatformName { get; }
		public List<DeviceReport> DeviceReports { get; set; } = new List<DeviceReport>();

		public DevicePlatformReport(string platformId, string platformName)
		{
			PlatformId = platformId;
			PlatformName = platformName;
		}
	}

	public class DeviceUsageEvent
	{
		/// <summary>
		/// Start of the usage event.
		/// </summary>
		public DateTime Start { get; }
		/// <summary>
		/// Increment(positive value) or decrement(negative value) usage, usually by one. 
		/// </summary>
		public int Increment { get; }
		/// <summary>
		/// Whether a problem was reported during that event.
		/// </summary>
		public bool HasProblem { get; }

		public DeviceUsageEvent(DateTime start, int increment, bool hasProblem = false)
		{
			Start = start;
			Increment = increment;
			HasProblem = hasProblem;
		}
	}

	public class DevicePoolMetrics
	{
		/// <summary>
		/// Platform Id
		/// </summary>
		public string PlatformId { get; }
		/// <summary>
		/// Platform Name
		/// </summary>
		public string PlatformName { get; }
		/// <summary>
		/// The URL to the platform on the pool view
		/// </summary>
		public string? PlatformURL { get; set; }
		/// <summary>
		/// Number total of devices in the pool
		/// </summary>
		public int Total { get; set; } = 0;
		/// <summary>
		/// Number of disabled devices in the pool
		/// </summary>
		public int Disabled { get; set; } = 0;
		/// <summary>
		/// Number of devices marked as Maintenance in the pool
		/// </summary>
		public int Maintenance { get; set; } = 0;
		/// <summary>
		/// Total number of reported problems
		/// </summary>
		public int Problems { get; set; } = 0;
		/// <summary>
		/// Cumulated reservation duration in the pool
		/// </summary>
		public TimeSpan TotalReservationDuration { get; set; } = new TimeSpan();
		/// <summary>
		/// Percentage of the average load per device in the pool
		/// </summary>
		public int AverageLoadPercentage { get; set; } = 0;
		/// <summary>
		/// Total number of pool saturation spikes
		/// </summary>
		public int SaturationSpikes { get; set; } = 0;
		/// <summary>
		/// Cumulated duration of the saturation spikes
		/// </summary>
		public TimeSpan SpikesDuration { get; set; } = new TimeSpan();
		/// <summary>
		/// Average duration a saturation spike
		/// </summary>
		public TimeSpan SpikeDurationAverage { get; set; } = new TimeSpan();
		/// <summary>
		/// Percentage of the cumulated spikes duration against the total reservation duration
		/// </summary>
		public int SpikeDurationPercentage { get; set; } = 0;
		/// <summary>
		/// Maximum concurrent reported problems at any one time
		/// </summary>
		public int MaxConcurrentProblems { get; set; } = 0;
		/// <summary>
		/// Percentage of the maximum concurrent reported problems against the total number of available devices
		/// </summary>
		public int MaxConcurrentProblemsPercentage { get; set; } = 0;
		/// <summary>
		/// Usage time series
		/// </summary>
		public List<DeviceUsageEvent> UsageEvents { get; } = new List<DeviceUsageEvent>();
		/// <summary>
		/// Collection of streams from which device reservation where made
		/// </summary>
		public HashSet<string> Streams { get; } = new HashSet<string>();

		public DevicePoolMetrics(string platformId, string platformName)
		{
			PlatformId = platformId;
			PlatformName = platformName;
		}
	}

	public class DevicePoolReport
	{
		public string PoolId { get; }
		public string PoolName { get; }

		public List<DevicePoolMetrics> Metrics { get; } = new List<DevicePoolMetrics>();

		/// <summary>
		/// The URL to the whole pool view
		/// </summary>
		public string? PoolURL { get; set; }

		public DevicePoolReport(string poolId, string poolName)
		{
			PoolId = poolId;
			PoolName = poolName;
		}
	}

	public class DeviceIssueReport
	{
		public string Channel { get; }

		public List<DevicePlatformReport> PlatformReports { get; set; } = new List<DevicePlatformReport>();

		public List<DevicePoolReport> PoolReports { get; set; } = new List<DevicePoolReport>();

		public DeviceIssueReport(string channel)
		{
			Channel = channel;
		}
	}

	public struct DeviceReportData
	{
		public List<IDevicePool> Pools { get; init; }
		public List<IDevicePlatform> Platforms { get; init; }
		public List<IDevice> Devices { get; init; }
		public List<IDeviceTelemetry> DeviceTelemetry { get; init; }
	}

	public struct DeviceReportSettings
	{
		public string Channel { get; init; }
		public int ReportMinutes { get; init; }
		public int DeviceProblemCoolDownMinutes { get; init; }
		public int DeviceSaturationSpikeThresholdMinutes { get; init; }
		public int DeviceCleaningThresholdHours { get; init; }
		public Uri ServerUrl { get; init; }
	}

#pragma warning restore CS1591 // Missing XML comment for publicly visible type or member

	[SingletonDocument("device-report-state", "6491bc0ce39f8b70b1ef2feb")]
	class DeviceReportState : SingletonBase
	{
		public DateTime ReportTime { get; set; } = DateTime.MinValue;
	}

	/// <summary>
	/// Posts summaries for all the open issues in different streams to Slack channels
	/// </summary>
	public class DeviceReportService : IHostedService
	{
		readonly SingletonDocument<DeviceReportState> _state;
		readonly DeviceService _deviceService;
		readonly INotificationService _notificationService;
		readonly IClock _clock;
		readonly ITicker _ticker;
		readonly IServerInfo _serverInfo;
		readonly IOptions<BuildServerConfig> _staticBuildConfig;
		readonly ILogger<DeviceReportService> _logger;

		readonly int _reportIntervalMinutes = 12 * 60; // every 12 hours

		readonly int _deviceProblemCoolDownMinutes = 10;
		readonly int _deviceSaturationSpikeThresholdMinutes = 10;
		readonly int _deviceCleaningThresholdHours = 6;

		/// <summary>
		/// Constructor
		/// </summary>
		public DeviceReportService(IMongoService mongoService, DeviceService deviceService, INotificationService notificationService, IClock clock, IServerInfo serverInfo, IOptions<BuildServerConfig> staticBuildConfig, ILogger<DeviceReportService> logger)
		{
			_state = new SingletonDocument<DeviceReportState>(mongoService);
			_deviceService = deviceService;
			_notificationService = notificationService;
			_clock = clock;
			_serverInfo = serverInfo;
			_ticker = clock.AddSharedTicker<DeviceReportService>(TimeSpan.FromMinutes(5.0), TickAsync, logger);
			_staticBuildConfig = staticBuildConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			BuildServerConfig staticBuildConfig = _staticBuildConfig.Value;
			if (String.IsNullOrEmpty(staticBuildConfig.DeviceReportChannel))
			{
				return;
			}

			DeviceReportState state = await _state.GetAsync(cancellationToken);
			DateTime currentTime = _clock.UtcNow;
			DateTime lastReportTime = state.ReportTime == DateTime.MinValue ? DateTime.Now.Subtract(TimeSpan.FromMinutes(_reportIntervalMinutes + 1)) : state.ReportTime;

			if ((currentTime - lastReportTime).TotalMinutes <= _reportIntervalMinutes)
			{
				return;
			}

			_logger.LogInformation("Creating device report");

			List<IDevicePool> pools = _deviceService.GetPools();
			List<IDevicePlatform> platforms = _deviceService.GetPlatforms();
			List<IDevice> devices = await _deviceService.GetDevicesAsync();
			List<IDeviceTelemetry> deviceTelemetry = await _deviceService.GetDeviceTelemetryAsync(null, lastReportTime);

			DeviceReportData deviceData = new DeviceReportData()
			{
				Pools = pools,
				Platforms = platforms,
				Devices = devices,
				DeviceTelemetry = deviceTelemetry
			};
			DeviceReportSettings settings = new DeviceReportSettings()
			{
				Channel = staticBuildConfig.DeviceReportChannel,
				ReportMinutes = _reportIntervalMinutes,
				DeviceProblemCoolDownMinutes = _deviceProblemCoolDownMinutes,
				DeviceSaturationSpikeThresholdMinutes = _deviceSaturationSpikeThresholdMinutes,
				DeviceCleaningThresholdHours = _deviceCleaningThresholdHours,
				ServerUrl = _serverInfo.DashboardUrl
			};

			DeviceIssueReport issueReport = CreateDeviceReport(deviceData, settings);

			if (issueReport.PoolReports.Count > 0 || issueReport.PlatformReports.Count > 0)
			{
				try
				{
					_logger.LogInformation("Sending device report notification");

					await _notificationService.SendDeviceIssueReportAsync(issueReport, cancellationToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error sending device health notification");
				}
			}

			state = await _state.UpdateAsync(s => s.ReportTime = currentTime, cancellationToken);
		}

		internal static DeviceIssueReport CreateDeviceReport(DeviceReportData deviceData, DeviceReportSettings settings)
		{
			DeviceIssueReport issueReport = new DeviceIssueReport(settings.Channel);
			deviceData.Devices.OrderBy(d => d.PlatformId.ToString()).ThenBy(d => d.PoolId.ToString()).ToList()
				.ForEach(device =>
			{
				IDevicePool? pool = deviceData.Pools.Find(p => p.Id == device.PoolId);
				IDevicePlatform? platform = deviceData.Platforms.Find(p => p.Id == device.PlatformId);

				if (pool == null || platform == null)
				{
					return;
				}

				if (pool.PoolType != DevicePoolType.Automation)
				{
					return;
				}

				DevicePoolReport? poolReport = issueReport.PoolReports.Find(r => r.PoolId == pool.Id.ToString());
				if (poolReport == null)
				{
					poolReport = new DevicePoolReport(pool.Id.ToString(), pool.Name.ToString());
					poolReport.PoolURL = new Uri($"{settings.ServerUrl}devices?pivotKey=pivot-key-automation&pool={pool.Id}").ToString();
					issueReport.PoolReports.Add(poolReport);
				}

				DevicePoolMetrics? metrics = poolReport.Metrics.Find(m => m.PlatformId == device.PlatformId.ToString());
				if (metrics == null)
				{
					metrics = new DevicePoolMetrics(platform.Id.ToString(), platform.Name.ToString());
					metrics.PlatformURL = new Uri($"{settings.ServerUrl}devices?pivotKey=pivot-key-automation&pool={pool.Id}&platform={platform.Id}").ToString();
					poolReport.Metrics.Add(metrics);
				}

				metrics.Total++;
				if (!device.Enabled)
				{
					metrics.Disabled++;
				}
				else if (device.MaintenanceTimeUtc != null)
				{
					metrics.Maintenance++;
				}

				if (!device.Enabled || device.MaintenanceTimeUtc != null)
				{
					return;
				}

				// Track usage events
				List<DeviceUsageEvent> platformUsage = metrics.UsageEvents;
				DateTime? endTimeUsage = null;
				List<IDeviceTelemetry> telemetry = deviceData.DeviceTelemetry.Where(t => t.DeviceId == device.Id).OrderBy(t => t.CreateTimeUtc).ToList();
				foreach (IDeviceTelemetry item in telemetry)
				{
					// Correct superposed reservations
					if (endTimeUsage != null && endTimeUsage > item.CreateTimeUtc)
					{
						int lastIndex = platformUsage.Count - 1;
						platformUsage[lastIndex] = new DeviceUsageEvent(item.CreateTimeUtc, -1, platformUsage[lastIndex].HasProblem);
						metrics.TotalReservationDuration -= (DateTime)endTimeUsage - platformUsage[lastIndex].Start;
					}

					// Add incremental usage
					bool hasProblem = item.ProblemTimeUtc != null;
					DeviceUsageEvent usage = new DeviceUsageEvent(item.CreateTimeUtc, 1, hasProblem);
					platformUsage.Add(usage);

					// Add decremental usage
					endTimeUsage = !hasProblem? item.ReservationFinishUtc : item.ProblemTimeUtc?.AddMinutes(settings.DeviceProblemCoolDownMinutes);
					if (endTimeUsage != null)
					{
						DeviceUsageEvent endUsage = new DeviceUsageEvent((DateTime)endTimeUsage, -1, hasProblem);
						platformUsage.Add(endUsage);
						metrics.TotalReservationDuration += endUsage.Start - usage.Start;
					}

					// Track stream usage
					if (item.StreamId != null)
					{
						metrics.Streams.Add(item.StreamId);
					}
				}

				Func<DevicePlatformReport> getPlatformReport = () =>
				{
					DevicePlatformReport? platformReport = issueReport.PlatformReports.Find(p => p.PlatformId == device.PlatformId.ToString());
					if (platformReport == null)
					{
						platformReport = new DevicePlatformReport(platform.Id.ToString(), platform.Name);
						issueReport.PlatformReports.Add(platformReport);
					}
					return platformReport;
				};

				Func<string> buildDevicePoolUrl = () => new Uri($"{settings.ServerUrl}devices?pivotKey=pivot-key-automation&pool={pool.Id}&platform={platform.Id}&filter={device.Name}").ToString();

				IEnumerable<IDeviceTelemetry> problems;
				// Check for clean up hang
				TimeSpan? cleaningTime = device.CleanStartTimeUtc != null && device.CleanFinishTimeUtc == null ? (DateTime.UtcNow - device.CleanStartTimeUtc) : null;
				if (cleaningTime != null && cleaningTime.Value.TotalHours >= settings.DeviceCleaningThresholdHours)
				{
					DeviceReport report = new DeviceReport(platform.Id.ToString(), platform.Name, device.Id.ToString(), device.Name, device.Address ?? "Unknown Address", pool.Id.ToString(), pool.Name, telemetry);
					report.CleaningTime = cleaningTime;
					report.DevicePoolURL = buildDevicePoolUrl();

					getPlatformReport().DeviceReports.Add(report);
					
					return;
				}

				// Report device issues, though only if not since disabled or put into maintenance
				problems = telemetry.Where(t => t.ProblemTimeUtc != null).OrderByDescending(t => t.ProblemTimeUtc!);

				if (!problems.Any())
				{
					return;
				}

				// mark that this device reported a problem
				metrics.Problems += 1;

				int problemsCount = problems.Count();
				int problemsPercent = (int)((double)problemsCount / telemetry.Count * 100);

				// don't report low problem rate
				if (problemsPercent < 10)
				{
					return;
				}

				IDeviceTelemetry lastProblem = problems.First();

				DeviceReport deviceReport = new DeviceReport(platform.Id.ToString(), platform.Name, device.Id.ToString(), device.Name, device.Address ?? "Unknown Address", pool.Id.ToString(), pool.Name, telemetry);

				deviceReport.ProblemDelta = problemsCount;
				deviceReport.ProblemPercent = problemsPercent;

				if (lastProblem.JobName != null && lastProblem.StepName != null && lastProblem.JobId != null && lastProblem.StepId != null)
				{
					deviceReport.LastProblemDesc = $"{lastProblem.JobName} - {lastProblem.StepName}";
					deviceReport.LastProblemURL = new Uri($"{settings.ServerUrl}job/{lastProblem.JobId}?step={lastProblem.StepId}").ToString();
				}

				deviceReport.DevicePoolURL = buildDevicePoolUrl();

				getPlatformReport().DeviceReports.Add(deviceReport);
			});

			// Calculate usage and spikes
			issueReport.PoolReports.ForEach(report =>
			{
				report.Metrics.ForEach(metrics =>
				{
					int maxCount = metrics.Total - metrics.Disabled - metrics.Maintenance;
					if (maxCount > 0)
					{
						metrics.UsageEvents.SortBy(usage => usage.Start);
						int usageCount = 0;
						int concurrentProblemsCount = 0;
						int index = 0;
						int lastIndex = metrics.UsageEvents.Count - 1;
						foreach (DeviceUsageEvent usage in metrics.UsageEvents)
						{
							usageCount += usage.Increment;
							if (usageCount == maxCount)
							{
								if (index < lastIndex)
								{
									TimeSpan duration = metrics.UsageEvents[index + 1].Start - usage.Start;
									if (duration.TotalMinutes > settings.DeviceSaturationSpikeThresholdMinutes)
									{
										metrics.SaturationSpikes += 1;
										metrics.SpikesDuration += duration;
									}
								}
								else
								{
									// Can't get the duration, but we still want to count the spike.
									metrics.SaturationSpikes += 1;
								}
							}
							if (usage.HasProblem)
							{
								concurrentProblemsCount += usage.Increment;
								metrics.MaxConcurrentProblems = metrics.MaxConcurrentProblems > concurrentProblemsCount ? metrics.MaxConcurrentProblems : concurrentProblemsCount;
							}
							index++;
						}
						metrics.AverageLoadPercentage = (int)((((double)metrics.TotalReservationDuration.TotalMinutes / maxCount) / settings.ReportMinutes) * 100);
						metrics.MaxConcurrentProblemsPercentage = (int)(((double)metrics.MaxConcurrentProblems / maxCount) * 100);
						if (metrics.SaturationSpikes > 0)
						{
							metrics.SpikeDurationAverage = metrics.SpikesDuration / metrics.SaturationSpikes;
							metrics.SpikeDurationPercentage = (int)((metrics.SpikesDuration.TotalMinutes / settings.ReportMinutes) * 100);
						}
					}
				});
			});

			// order everything nicely
			issueReport.PoolReports = issueReport.PoolReports.OrderBy(r => r.PoolName).ToList();
			issueReport.PlatformReports = issueReport.PlatformReports.OrderBy(r => r.PlatformName).ToList();
			issueReport.PlatformReports.ForEach(r => r.DeviceReports = r.DeviceReports.OrderBy(r => r.PoolName).ThenBy(r => r.DeviceName).ToList());

			return issueReport;
		}
	}
}

// CL 22278596 - Has the device channel notification stuff