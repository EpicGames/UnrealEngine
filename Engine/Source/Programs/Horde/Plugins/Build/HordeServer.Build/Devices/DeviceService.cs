// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Jobs;
using HordeServer.Notifications;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using OpenTelemetry.Trace;

namespace HordeServer.Devices
{
	/// <summary>
	///  Device Pool Authorization (convenience class for pool ACL's)
	/// </summary>
	public class DevicePoolAuthorization
	{
		/// <summary>
		/// The device pool
		/// </summary>
		public IDevicePool Pool { get; private set; }

		/// <summary>
		///  Read access to pool
		/// </summary>
		public bool Read { get; private set; }

		/// <summary>
		///  Write access to pool
		/// </summary>
		public bool Write { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="pool"></param>
		/// <param name="read"></param>
		/// <param name="write"></param>
		public DevicePoolAuthorization(IDevicePool pool, bool read, bool write)
		{
			Pool = pool;
			Read = read;
			Write = write;
		}
	}

	/// <summary>
	/// Device management service
	/// </summary>
	public sealed class DeviceService : IHostedService, IAsyncDisposable
	{
		readonly INotificationService _notificationService;
		readonly JobService _jobService;
		readonly IUserCollection _userCollection;
		readonly Tracer _tracer;
		readonly ILogger<DeviceService> _logger;
		readonly IDeviceCollection _devices;
		readonly ITicker _ticker;
		readonly ITicker _telemetryTicker;
		readonly IServerInfo _serverInfo;
		readonly IAuditLog<DeviceId> _auditLog;
		readonly IOptionsMonitor<BuildServerConfig> _settings;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;

		bool _runUpgrade = true;

		/// <summary>
		/// The number of days shared device sheckouts are held
		/// </summary>
		public int SharedDeviceCheckoutDays => _settings.CurrentValue.SharedDeviceCheckoutDays;

		/// <summary>
		/// Device service constructor
		/// </summary>
		public DeviceService(IDeviceCollection devices, IUserCollection userCollection, JobService jobService, IServerInfo serverInfo, IOptionsMonitor<BuildServerConfig> settings, IOptionsMonitor<BuildConfig> buildConfig, INotificationService notificationService, IClock clock, Tracer tracer, ILogger<DeviceService> logger, IAuditLog<DeviceId> auditLog)
		{
			_userCollection = userCollection;
			_devices = devices;
			_jobService = jobService;
			_notificationService = notificationService;
			_ticker = clock.AddSharedTicker<DeviceService>(TimeSpan.FromMinutes(1.0), TickAsync, logger);
			_telemetryTicker = clock.AddSharedTicker("DeviceService.Telemetry", TimeSpan.FromMinutes(10.0), TickTelemetryAsync, logger);
			_tracer = tracer;
			_logger = logger;
			_serverInfo = serverInfo;
			_settings = settings;
			_buildConfig = buildConfig;
			_auditLog = auditLog;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
			await _telemetryTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
			await _telemetryTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
			await _telemetryTicker.DisposeAsync();
		}

		/// <summary>
		/// Get the logger for the specified device id
		/// </summary>
		/// <param name="deviceId"></param>
		/// <returns></returns>
		public IAuditLogChannel<DeviceId> GetDeviceLogger(DeviceId deviceId)
		{
			return _auditLog[deviceId];
		}

		/// <summary>
		/// Ticks service
		/// </summary>
		async ValueTask TickTelemetryAsync(CancellationToken stoppingToken)
		{
			if (!stoppingToken.IsCancellationRequested)
			{
				using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(DeviceService)}.{nameof(TickTelemetryAsync)}");
				_logger.LogInformation("Updating pool telemetry");
				await _devices.CreatePoolTelemetrySnapshotAsync(GetPools(), _settings.CurrentValue.DeviceProblemCooldownMinutes);
			}
		}

		/// <summary>
		/// Ticks service
		/// </summary>
		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			if (!cancellationToken.IsCancellationRequested)
			{
				using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(DeviceService)}.{nameof(TickAsync)}");

				BuildConfig buildConfig = _buildConfig.CurrentValue;

				if (_runUpgrade)
				{
					try
					{
						await _devices.UpgradeAsync();
						_runUpgrade = false;
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while upgrading device collection: {Message}", ex.Message);
					}
				}

				try
				{
					_logger.LogDebug("Expiring reservations");
					await ExpireReservationsAsync(cancellationToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while expiring reservations: {Message}", ex.Message);
				}

				// expire shared devices and notifications
				try
				{

					_logger.LogDebug("Sending shared device notifications");
					List<(UserId, IDevice)>? expireNotifications = await _devices.ExpireNotificatonsAsync(_settings.CurrentValue.SharedDeviceCheckoutDays);
					if (expireNotifications != null && expireNotifications.Count > 0)
					{
						foreach ((UserId, IDevice) expiredDevice in expireNotifications)
						{
							await NotifyDeviceServiceAsync(buildConfig, $"Device {expiredDevice.Item2.PlatformId.ToString().ToUpperInvariant()} / {expiredDevice.Item2.Name} checkout will expire in 24 hours.  Please visit {_serverInfo.DashboardUrl}devices to check in and back out if needed.", null, null, null, expiredDevice.Item1, cancellationToken);
						}
					}

					_logger.LogDebug("Expiring shared device checkouts");
					List<(UserId, IDevice)>? expireCheckouts = await _devices.ExpireCheckedOutAsync(_settings.CurrentValue.SharedDeviceCheckoutDays);
					if (expireCheckouts != null && expireCheckouts.Count > 0)
					{
						foreach ((UserId, IDevice) expiredDevice in expireCheckouts)
						{
							await NotifyDeviceServiceAsync(buildConfig, $"Device {expiredDevice.Item2.PlatformId.ToString().ToUpperInvariant()} / {expiredDevice.Item2.Name} checkout has expired.  The device has been returned to the shared pool and should no longer be accessed.  Please visit {_serverInfo.DashboardUrl}devices to checkout devices as needed.", null, null, null, expiredDevice.Item1, cancellationToken);
						}
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while expiring device checkouts: {Message}", ex.Message);
				}
			}
		}

		internal async Task TickForTestingAsync()
		{
			await TickAsync(CancellationToken.None);
			await TickTelemetryAsync(CancellationToken.None);
		}

		static bool CheckReservedNodesComplete(IJob job, List<JobStepId> reservedStepIds)
		{
			List<JobStepState> states = new List<JobStepState>() { JobStepState.Ready, JobStepState.Running, JobStepState.Waiting };

			// check batches in case step state has an issue
			bool batchesComplete = true;
			foreach (JobStepId stepId in reservedStepIds)
			{
				foreach (IJobStepBatch batch in job.Batches)
				{
					if (batch.Steps.Any(s => s.Id == stepId))
					{
						if (batch.State != JobStepBatchState.Complete && batch.State != JobStepBatchState.Stopping)
						{
							batchesComplete = false;
						}
					}
				}
			}

			if (batchesComplete)
			{
				return true;
			}

			if (!reservedStepIds.Any(s =>
			{
				IJobStep? step;
				if (!job.TryGetStep(s, out step))
				{
					return false;
				}

				return states.Contains(step.State);
			}))
			{
				return true;
			}

			return false;
		}

		async Task<bool> ExpireReservationsAsync(CancellationToken cancellationToken)
		{
			List<IDeviceReservation> reserves = await _devices.FindAllReservationsAsync();
			List<IDeviceReservation> expired = new List<IDeviceReservation>();

			DateTime utcNow = DateTime.UtcNow;

			List<IDeviceReservation> nodeReserves = reserves.FindAll(r => r.JobId != null && r.ReservedStepIds != null && r.ReservedStepIds.Count > 0).ToList();

			for (int i = 0; i < reserves.Count; i++)
			{
				IDeviceReservation r = reserves[i];

				// timeout
				if (!nodeReserves.Contains(r))
				{
					if ((utcNow - r.UpdateTimeUtc).TotalMinutes > 10)
					{
						expired.Add(r);
					}
				}
				else
				{
					// expire when all reserve steps have completed
					IJob? job = await _jobService.GetJobAsync(JobId.Parse(r.JobId!), cancellationToken);

					if (job == null || CheckReservedNodesComplete(job, r.ReservedStepIds!))
					{
						expired.Add(r);
					}
				}
			}

			bool result = true;
			foreach (IDeviceReservation reservation in expired)
			{
				if (!await DeleteReservationAsync(reservation.Id, true))
				{
					result = false;
				}
			}

			return result;
		}

		/// <summary>
		/// Get a list of existing device platforms
		/// </summary>
		public List<IDevicePlatform> GetPlatforms()
		{
			return _buildConfig.CurrentValue.Devices?.Platforms.ConvertAll(x => (IDevicePlatform)x) ?? new List<IDevicePlatform>();
		}
		/// <summary>
		/// Get a specific device platform
		/// </summary>
		public IDevicePlatform? GetPlatform(DevicePlatformId id)
		{
			return GetPlatforms().FirstOrDefault(p => p.Id == id);
		}

		/// <summary>
		/// Get a device pool by id
		/// </summary>
		public IDevicePool? GetPool(DevicePoolId id)
		{
			return GetPools().FirstOrDefault(p => p.Id == id);
		}

		/// <summary>
		/// Get a list of existing device pools
		/// </summary>
		public List<IDevicePool> GetPools()
		{
			return _buildConfig.CurrentValue.Devices?.Pools.ConvertAll(x => (IDevicePool)x) ?? new List<IDevicePool>();
		}

		/// <summary>
		/// Get a list of devices, optionally filtered to provided ids
		/// </summary>
		public async Task<List<IDevice>> GetDevicesAsync(List<DeviceId>? deviceIds = null, DevicePoolId? poolId = null, DevicePlatformId? platformId = null)
		{
			return await _devices.FindAllDevicesAsync(deviceIds, poolId, platformId);
		}

		/// <summary>
		/// Get device telemetry
		/// </summary>
		public async Task<List<IDeviceTelemetry>> GetDeviceTelemetryAsync(DeviceId[]? deviceIds = null, DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null)
		{
			return await _devices.FindDeviceTelemetryAsync(deviceIds, minCreateTime, maxCreateTime, index, count);
		}

		/// <summary>
		/// Get device pool telemetry
		/// </summary>
		public async Task<List<IDevicePoolTelemetry>> GetDevicePoolTelemetryAsync(DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null)
		{
			return await _devices.FindPoolTelemetryAsync(minCreateTime, maxCreateTime, index, count);
		}

		/// <summary>
		/// Get a specific device
		/// </summary>
		public async Task<IDevice?> GetDeviceAsync(DeviceId id)
		{
			return await _devices.GetDeviceAsync(id);
		}

		/// <summary>
		/// Get a device by name
		/// </summary>
		public async Task<IDevice?> GetDeviceByNameAsync(string deviceName)
		{
			return await _devices.GetDeviceByNameAsync(deviceName);
		}

		/// <summary>
		/// Delete a device
		/// </summary>
		public async Task<bool> DeleteDeviceAsync(DeviceId id)
		{
			return await _devices.DeleteDeviceAsync(id);
		}

		/// <summary>
		/// Create a new device
		/// </summary>
		/// <param name="id">Unique id of the device</param>
		/// <param name="name">Friendly name of the device</param>
		/// <param name="platformId">The device platform</param>
		/// <param name="poolId">Which pool to add the device</param>
		/// <param name="enabled">Whether the device is enabled</param>
		/// <param name="address">Address or hostname of device</param>
		/// <param name="modelId">Vendor model id</param>
		/// <param name="userId">User adding the device</param>
		/// <param name="tags">The metadata tags of the device</param>
		/// <returns></returns>
		public async Task<IDevice?> TryCreateDeviceAsync(DeviceId id, string name, DevicePlatformId platformId, DevicePoolId poolId, bool? enabled, string? address, string? modelId, UserId? userId = null, List<string>? tags = null)
		{
			return await _devices.TryAddDeviceAsync(id, name, platformId, poolId, enabled, address, modelId, userId, tags);
		}

		/// <summary>
		/// Update a device
		/// </summary>
		public async Task UpdateDeviceAsync(DeviceId deviceId, DevicePoolId? newPoolId = null, string? newName = null, string? newAddress = null, string? newModelId = null, string? newNotes = null, bool? newEnabled = null, bool? newProblem = null, bool? newMaintenance = null, UserId? modifiedByUserId = null, string? message = null, List<string>? newTags = null)
		{
			if (newProblem == true)
			{
				IDeviceReservation? reservation = await TryGetDeviceReservationAsync(deviceId);
				if (reservation != null)
				{
					await TryUpdateReservationAsync(reservation.Id, problemDevice: deviceId);
				}
			}
			await _devices.UpdateDeviceAsync(deviceId, newPoolId, newName, newAddress, newModelId, newNotes, newEnabled, newProblem, newMaintenance, modifiedByUserId, message, newTags);
		}

		/// <summary>
		/// Checkout a device
		/// </summary>
		public async Task CheckoutDeviceAsync(DeviceId deviceId, UserId? userId)
		{
			await _devices.CheckoutDeviceAsync(deviceId, userId);
		}

		/// <summary>
		/// Clean a device
		/// </summary>
		public async Task CleanDeviceAsync(DeviceId deviceId, bool cleaning)
		{
			await _devices.UpdateDeviceCleaningAsync(deviceId, cleaning);
		}

		/// <summary>
		/// Try to create a reservation satisfying the specified device platforms and models
		/// </summary>
		public async Task<(IDeviceReservation?, string? errorMessage, bool installRequired)> TryCreateReservationAsync(DevicePoolId poolId, List<DeviceRequestData> request, string? hostname = null, string? reservationDetails = null, JobId? jobId = null, JobStepId? stepId = null, CancellationToken cancellationToken = default)
		{
			IJob? job = null;
			IGraph? graph = null;
			string? stepName = null;

			List<JobStepId>? reserveStepIds = null;

			if (jobId != null)
			{
				IJobStepBatch? stepBatch = null;
				IJobStep? jobStep = null;
				INode? stepNode = null;

				job = await _jobService.GetJobAsync(jobId.Value, cancellationToken);
				if (job != null)
				{
					if (stepId != null)
					{
						graph = await _jobService.GetGraphAsync(job, cancellationToken);
						foreach (IJobStepBatch batch in job.Batches)
						{
							IJobStep? step;
							if (batch.TryGetStep(stepId.Value, out step))
							{
								jobStep = step;
								stepBatch = batch;
								stepNode = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
								stepName = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx].Name;
								break;
							}
						}
					}

					if (stepBatch != null && stepNode != null && stepNode.Annotations != null)
					{
						List<IJobStep> reserveSteps = new List<IJobStep>();

						// List of reserve nodes
						string? reserveNodesValue;
						if (stepNode.Annotations.TryGetValue("DeviceReserveNodes", out reserveNodesValue) && reserveNodesValue.Length > 0)
						{
							List<string> reserveNodes = reserveNodesValue.Split(',').Select(x => x.Trim()).ToList();

							reserveNodes.ForEach(nodeName =>
							{
								NodeRef? nodeRef;

								if (graph!.TryFindNode(nodeName, out nodeRef))
								{
									IJobStep? jobStep = null;
									if (job.TryGetStepForNode(nodeRef, out jobStep))
									{
										reserveSteps.Add(jobStep);
									}
								}
							});

							if (jobStep != null && !reserveSteps.Any(s => s.Id == stepId))
							{
								reserveSteps.Insert(0, jobStep);
							}
						}

						// Reserve begin/end markers
						string? deviceReserve;
						if (stepNode.Annotations.TryGetValue("DeviceReserve", out deviceReserve) && String.Equals(deviceReserve, "Begin", StringComparison.OrdinalIgnoreCase))
						{
							int index = stepBatch.Steps.FindIndex(x => x.Id == jobStep!.Id);
							for (int i = index; i < stepBatch.Steps.Count; i++)
							{
								INode node = graph!.Groups[stepBatch.GroupIdx].Nodes[stepBatch.Steps[i].NodeIdx];
								// An end or begin starts a new reservation block
								if (i != index && node.Annotations.TryGetValue("DeviceReserve", out deviceReserve) && (String.Equals(deviceReserve, "End", StringComparison.OrdinalIgnoreCase) || String.Equals(deviceReserve, "Begin", StringComparison.OrdinalIgnoreCase)))
								{
									if (String.Equals(deviceReserve, "End", StringComparison.OrdinalIgnoreCase))
									{
										reserveSteps.Add(stepBatch.Steps[i]);
									}

									break;
								}

								reserveSteps.Add(stepBatch.Steps[i]);
							}
						}

						reserveStepIds = reserveSteps.Select(s => s.Id).ToList();
					}
				}
				else
				{
					return (null, $"Unable to find job for reservation, {jobId}", false);
				}
			}

			IDevicePool? pool = GetPool(poolId);

			if (pool == null || pool.PoolType != DevicePoolType.Automation)
			{
				string? errorMessage;
				if (pool == null)
				{
					errorMessage = $"Unable to find pool for reservation, {poolId}";
				}
				else
				{
					errorMessage = $"Attempted to reserve a device from a non-automation pool, {poolId}";
				}

				return (null, errorMessage, false);
			}

			(IDeviceReservation? reservation, bool installRequired) = await _devices.TryAddReservationAsync(poolId, request, _settings.CurrentValue.DeviceProblemCooldownMinutes, hostname, reservationDetails, job, stepId, stepName, reserveStepIds);

			// check that only one step is running
			if (job != null && reservation != null && reservation.ReservedStepIds != null)
			{
				List<IJobStep> reserveSteps = new List<IJobStep>();

				foreach (JobStepId id in reservation.ReservedStepIds)
				{
					IJobStep? step;
					if (job!.TryGetStep(id, out step))
					{
						reserveSteps.Add(step);
					}
				}

				if (reserveSteps.Count(s => s.State == JobStepState.Running) > 1)
				{
					List<string> errorSteps = new List<string>();
					foreach (IJobStep step in reserveSteps)
					{
						if (step.State != JobStepState.Running)
						{
							continue;
						}

						foreach (IJobStepBatch batch in job.Batches)
						{
							if (batch.Steps.Any(s => s.Id == step.Id))
							{
								stepName = graph!.Groups[batch.GroupIdx].Nodes[step.NodeIdx].Name;
								errorSteps.Add(stepName);
							}
						}
					}

					return (null, $"Reserved nodes must not run in parallel: {String.Join(',', errorSteps)}", false);
				}
			}

			if (reservation == null)
			{
				return (null, $"Unable to add reservation for {jobId}:{stepId}", false);
			}

			return (reservation, null, installRequired);
		}

		/// <summary>
		/// Update/renew an existing reservation
		/// </summary>
		public Task<IDeviceReservation?> TryUpdateReservationAsync(ObjectId id, DeviceId? problemDevice = null)
		{
			return _devices.TryUpdateReservationAsync(id, problemDevice: problemDevice);
		}

		/// <summary>
		///  Delete an existing reservation
		/// </summary>
		public async Task<bool> DeleteReservationAsync(ObjectId id, bool force = false)
		{
			IDeviceReservation? reservation = await _devices.TryGetReservationAsync(id);

			if (!force)
			{
				if (reservation != null && reservation.JobId != null && reservation.ReservedStepIds != null && reservation.ReservedStepIds.Count > 0)
				{
					// will be deleted in tick
					return false;
				}
			}
			else if (reservation != null)
			{
				_logger.LogInformation("Force deleting reservation {ReservationGuid}", reservation.LegacyGuid);
			}

			return await _devices.DeleteReservationAsync(id);
		}

		/// <summary>
		/// Get a reservation from a legacy guid
		/// </summary>
		public async Task<IDeviceReservation?> TryGetReservationFromLegacyGuidAsync(string legacyGuid)
		{
			return await _devices.TryGetReservationFromLegacyGuidAsync(legacyGuid);
		}

		/// <summary>
		/// Get a reservation from a device id
		/// </summary>
		public async Task<IDeviceReservation?> TryGetDeviceReservationAsync(DeviceId deviceId)
		{
			return await _devices.TryGetDeviceReservationAsync(deviceId);
		}

		/// <summary>
		/// Get a list of existing device reservations
		/// </summary>
		public async Task<List<IDeviceReservation>> GetReservationsAsync()
		{
			return await _devices.FindAllReservationsAsync();
		}

		/// <summary>
		/// 
		/// </summary>
		public async Task NotifyDeviceServiceAsync(BuildConfig buildConfig, string message, DeviceId? deviceId = null, string? jobId = null, string? stepId = null, UserId? userId = null, CancellationToken cancellationToken = default)
		{
			try
			{
				IDevice? device = null;
				IDevicePool? pool = null;
				IJob? job = null;
				IJobStep? step = null;
				INode? node = null;
				StreamConfig? streamConfig = null;
				IUser? user = null;

				if (userId.HasValue)
				{
					user = await _userCollection.GetUserAsync(userId.Value, cancellationToken);
					if (user == null)
					{
						_logger.LogError("Unable to send device notification, can't find User {UserId}", userId.Value);
						return;
					}
				}

				if (deviceId.HasValue)
				{
					device = await GetDeviceAsync(deviceId.Value);
					pool = GetPool(device!.PoolId);
				}

				if (jobId != null)
				{
					job = await _jobService.GetJobAsync(JobId.Parse(jobId), cancellationToken);

					if (job != null)
					{
						buildConfig.TryGetStream(job.StreamId, out streamConfig);

						if (stepId != null)
						{
							IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken)!;

							JobStepId stepIdValue = JobStepId.Parse(stepId);
							IJobStepBatch? batch = job.Batches.FirstOrDefault(b => b.Steps.FirstOrDefault(s => s.Id == stepIdValue) != null);
							if (batch != null)
							{
								step = batch.Steps.FirstOrDefault(s => s.Id == stepIdValue)!;
								INodeGroup group = graph.Groups[batch.GroupIdx];
								node = group.Nodes[step.NodeIdx];
							}
						}
					}
				}

				_notificationService.NotifyDeviceService(message, device, pool, streamConfig, job, step, node, user);

			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error on device notification {Message}", ex.Message);
			}
		}

		/// <summary>
		/// Authorize device action
		/// </summary>
		/// <param name="action"></param>
		/// <param name="user"></param>
		/// <param name="buildConfig"></param>
		/// <returns></returns>
		public static bool Authorize(AclAction action, ClaimsPrincipal user, BuildConfig buildConfig)
		{
			// This is deprecated and device auth should be going through GetUserPoolAuthorizationsAsync

			if (action == DeviceAclAction.DeviceRead)
			{
				return true;
			}

			if (user.IsInRole("Internal-Employees"))
			{
				return true;
			}

			if (buildConfig.Authorize(AdminAclAction.AdminWrite, user))
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Get list of pool authorizations for a user
		/// </summary>
		/// <param name="user"></param>
		/// <param name="buildConfig"></param>
		/// <returns></returns>
		public List<DevicePoolAuthorization> GetUserPoolAuthorizations(ClaimsPrincipal user, BuildConfig buildConfig)
		{
			List<DevicePoolAuthorization> authPools = new List<DevicePoolAuthorization>();

			List<IDevicePool> allPools = GetPools();
			IReadOnlyList<ProjectConfig> projects = buildConfig.Projects;

			// Set of projects associated with device pools
			HashSet<ProjectId> projectIds = new HashSet<ProjectId>(allPools.Where(x => x.ProjectIds != null).SelectMany(x => x.ProjectIds!));

			Dictionary<ProjectId, bool> deviceRead = new Dictionary<ProjectId, bool>();
			Dictionary<ProjectId, bool> deviceWrite = new Dictionary<ProjectId, bool>();

			foreach (ProjectId projectId in projectIds)
			{
				if (projects.Where(x => x.Id == projectId).FirstOrDefault() == null)
				{
					_logger.LogWarning("Device pool authorization references missing project id {ProjectId}", projectId);
					continue;
				}

				if (buildConfig.TryGetProject(projectId, out ProjectConfig? projectConfig))
				{
					deviceRead.Add(projectId, projectConfig.Authorize(DeviceAclAction.DeviceRead, user));
					deviceWrite.Add(projectId, projectConfig.Authorize(DeviceAclAction.DeviceWrite, user));
				}
			}

			// for global pools which aren't associated with a project
			bool globalPoolAccess = user.IsInRole("Internal-Employees");

			if (!globalPoolAccess)
			{
				if (buildConfig.Authorize(AdminAclAction.AdminWrite, user))
				{
					globalPoolAccess = true;
				}
			}

			foreach (IDevicePool pool in allPools)
			{
				if (pool.ProjectIds == null || pool.ProjectIds.Count == 0)
				{
					if (pool.PoolType == DevicePoolType.Shared && !globalPoolAccess)
					{
						authPools.Add(new DevicePoolAuthorization(pool, false, false));
						continue;
					}

					authPools.Add(new DevicePoolAuthorization(pool, true, true));
					continue;
				}

				bool read = false;
				bool write = false;

				foreach (ProjectId projectId in pool.ProjectIds)
				{
					bool value;

					if (!read && deviceRead.TryGetValue(projectId, out value))
					{
						read = value;
					}

					if (!write && deviceWrite.TryGetValue(projectId, out value))
					{
						write = value;
					}
				}

				authPools.Add(new DevicePoolAuthorization(pool, read, write));

			}

			return authPools;
		}

		/// <summary>
		/// Get list of pool authorizations for a user
		/// </summary>
		/// <param name="id"></param>
		/// <param name="user"></param>
		/// <param name="buildConfig"></param>
		/// <returns></returns>
		public DevicePoolAuthorization? GetUserPoolAuthorization(DevicePoolId id, ClaimsPrincipal user, BuildConfig buildConfig)
		{
			List<DevicePoolAuthorization> auth = GetUserPoolAuthorizations(user, buildConfig);
			return auth.Where(x => x.Pool.Id == id).FirstOrDefault();
		}
	}
}
