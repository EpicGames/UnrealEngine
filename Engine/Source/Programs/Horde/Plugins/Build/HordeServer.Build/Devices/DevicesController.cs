// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using HordeServer.Users;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Driver;
using EpicGames.Core;
using HordeServer.Auditing;

namespace HordeServer.Devices
{
	/// <summary>
	/// Controller for device service
	/// </summary>
	public class DevicesController : ControllerBase
	{
		readonly IUserCollection _userCollection;
		readonly DeviceService _deviceService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;

		/// <summary>
		///  Logger for controller
		/// </summary>
		private readonly ILogger<DevicesController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public DevicesController(IUserCollection userCollection, DeviceService deviceService, IOptionsMonitor<BuildConfig> buildConfig, ILogger<DevicesController> logger)
		{
			_userCollection = userCollection;
			_deviceService = deviceService;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		// DEVICES

		/// <summary>
		/// Create a new device
		/// </summary>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/devices")]
		public async Task<ActionResult<CreateDeviceResponse>> CreateDeviceAsync([FromBody] CreateDeviceRequest deviceRequest, CancellationToken cancellationToken = default)
		{
			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(new DevicePoolId(deviceRequest.PoolId!), User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			IUser? internalUser = await _userCollection.GetUserAsync(User, cancellationToken);

			IDevicePlatform? platform = _deviceService.GetPlatform(new DevicePlatformId(deviceRequest.PlatformId!));

			if (platform == null)
			{
				return BadRequest($"Bad platform id {deviceRequest.PlatformId} on request");
			}

			IDevicePool? pool = _deviceService.GetPool(new DevicePoolId(deviceRequest.PoolId!));

			if (pool == null)
			{
				return BadRequest($"Bad pool id {deviceRequest.PoolId} on request");
			}

			string? modelId = null;

			if (!String.IsNullOrEmpty(deviceRequest.ModelId))
			{
				modelId = new string(deviceRequest.ModelId);

				if (platform.Models?.FirstOrDefault(x => x == modelId) == null)
				{
					return BadRequest($"Bad model id {modelId} for platform {platform.Id} on request");
				}
			}

			string name = deviceRequest.Name.Trim();
			string? address = deviceRequest.Address?.Trim();

			IDevice? device = await _deviceService.TryCreateDeviceAsync(DeviceId.Sanitize(name), name, platform.Id, pool.Id, deviceRequest.Enabled, address, modelId, internalUser?.Id, deviceRequest.Tags);

			if (device == null)
			{
				return BadRequest($"Unable to create device");
			}

			return new CreateDeviceResponse(device.Id.ToString());
		}

		/// <summary>
		/// Get list of devices
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices")]
		[ProducesResponseType(typeof(List<GetDeviceResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetDevicesAsync()
		{
			List<DevicePoolAuthorization> poolAuth = _deviceService.GetUserPoolAuthorizations(User, _buildConfig.CurrentValue);

			List<IDevice> devices = await _deviceService.GetDevicesAsync();

			List<object> responses = new List<object>();

			foreach (IDevice device in devices)
			{
				DevicePoolAuthorization? auth = poolAuth.Where(x => x.Pool.Id == device.PoolId).FirstOrDefault();

				if (auth == null || !auth.Read)
				{
					continue;
				}

				DateTime? checkoutExpiration = null;
				if (device.CheckOutTime != null)
				{
					checkoutExpiration = device.CheckOutTime.Value.AddDays(_deviceService.SharedDeviceCheckoutDays);
				}

				List<GetDeviceUtilizationResponse>? utilization = device.Utilization?.Select(u => new GetDeviceUtilizationResponse
				{
					JobId = u.JobId,
					StepId = u.StepId,
					ReservationStartUtc = u.ReservationStartUtc,
					ReservationFinishUtc = u.ReservationFinishUtc
				}).ToList();

				responses.Add(new GetDeviceResponse(device.Id.ToString(), device.PlatformId.ToString(), device.PoolId.ToString(), device.Name, device.Enabled, device.Address, device.ModelId?.ToString(), device.ModifiedByUser, device.Notes, device.ProblemTimeUtc, device.MaintenanceTimeUtc, utilization, device.CheckedOutByUser, device.CheckOutTime, checkoutExpiration, device.CleanRequired, device.CleanStartTimeUtc, device.CleanFinishTimeUtc, device.Tags));
			}

			return responses;
		}

		/// <summary>
		/// Get a specific device
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/{deviceId}")]
		[ProducesResponseType(typeof(GetDeviceResponse), 200)]
		public async Task<ActionResult<object>> GetDeviceAsync(string deviceId)
		{

			DeviceId deviceIdValue = new DeviceId(deviceId);

			IDevice? device = await _deviceService.GetDeviceAsync(deviceIdValue);

			if (device == null)
			{
				return BadRequest($"Unable to find device with id {deviceId}");
			}

			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(device.PoolId, User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			DateTime? checkoutExpiration = null;
			if (device.CheckOutTime != null)
			{
				checkoutExpiration = device.CheckOutTime.Value.AddDays(_deviceService.SharedDeviceCheckoutDays);
			}

			List<GetDeviceUtilizationResponse>? utilization = device.Utilization?.Select(u => new GetDeviceUtilizationResponse
			{
				JobId = u.JobId,
				StepId = u.StepId,
				ReservationStartUtc = u.ReservationStartUtc,
				ReservationFinishUtc = u.ReservationFinishUtc
			}).ToList();

			return new GetDeviceResponse(device.Id.ToString(), device.PlatformId.ToString(), device.PoolId.ToString(), device.Name, device.Enabled, device.Address, device.ModelId?.ToString(), device.ModifiedByUser?.ToString(), device.Notes, device.ProblemTimeUtc, device.MaintenanceTimeUtc, utilization, device.CheckedOutByUser, device.CheckOutTime, checkoutExpiration, device.CleanRequired, device.CleanStartTimeUtc, device.CleanFinishTimeUtc);
		}

		/// <summary>
		/// Update a specific device
		/// </summary>
		[HttpPut]
		[Authorize]
		[Route("/api/v2/devices/{deviceId}")]
		public async Task<ActionResult> UpdateDeviceAsync(string deviceId, [FromBody] UpdateDeviceRequest update, CancellationToken cancellationToken = default)
		{
			IUser? internalUser = await _userCollection.GetUserAsync(User, cancellationToken);

			DeviceId deviceIdValue = new DeviceId(deviceId);

			IDevice? device = await _deviceService.GetDeviceAsync(deviceIdValue);

			if (device == null)
			{
				return BadRequest($"Device with id ${deviceId} does not exist");
			}

			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(device.PoolId, User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			IDevicePlatform? platform = _deviceService.GetPlatform(device.PlatformId);

			if (platform == null)
			{
				return BadRequest($"Platform id ${device.PlatformId} does not exist");
			}

			DevicePoolId? poolIdValue = null;

			if (update.PoolId != null)
			{
				poolIdValue = new DevicePoolId(update.PoolId);
			}

			string? modelIdValue = null;

			if (update.ModelId != null)
			{
				modelIdValue = new string(update.ModelId);

				if (modelIdValue != "Base" && platform.Models?.FirstOrDefault(x => x == modelIdValue) == null)
				{
					return BadRequest($"Bad model id {update.ModelId} for platform {platform.Id} on request");
				}
			}

			string? name = update.Name?.Trim();
			string? address = update.Address?.Trim();

			await _deviceService.UpdateDeviceAsync(deviceIdValue, poolIdValue, name, address, modelIdValue, update.Notes, update.Enabled, update.Problem, update.Maintenance, internalUser?.Id, update.Message, update.Tags);

			return Ok();
		}

		/// <summary>
		/// Checkout a device
		/// </summary>
		[HttpPut]
		[Authorize]
		[Route("/api/v2/devices/{deviceId}/checkout")]
		public async Task<ActionResult> CheckoutDeviceAsync(string deviceId, [FromBody] CheckoutDeviceRequest request, CancellationToken cancellationToken)
		{

			IUser? internalUser = await _userCollection.GetUserAsync(User, cancellationToken);
			if (internalUser == null)
			{
				return NotFound();
			}

			DeviceId deviceIdValue = new DeviceId(deviceId);

			IDevice? device = await _deviceService.GetDeviceAsync(deviceIdValue);

			if (device == null)
			{
				return BadRequest($"Device with id ${deviceId} does not exist");
			}

			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(device.PoolId, User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			if (request.Checkout)
			{
				if (!String.IsNullOrEmpty(device.CheckedOutByUser))
				{
					return BadRequest($"Already checked out by user {device.CheckedOutByUser}");
				}

				await _deviceService.CheckoutDeviceAsync(deviceIdValue, internalUser.Id);

			}
			else
			{
				await _deviceService.CheckoutDeviceAsync(deviceIdValue, null);
			}

			return Ok();
		}

		/// <summary>
		/// Clean a device
		/// </summary>
		[HttpPut]
		[Authorize]
		[Route("/api/v2/devices/{deviceId}/clean")]		
		public async Task<ActionResult> CleanDeviceAsync(string deviceId, [FromBody] CleanDeviceRequest request)
		{

			DeviceId deviceIdValue = new DeviceId(deviceId);

			IDevice? device = await _deviceService.GetDeviceAsync(deviceIdValue);

			if (device == null)
			{
				return BadRequest($"Device with id ${deviceId} does not exist");
			}

			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(device.PoolId, User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			await _deviceService.CleanDeviceAsync(device.Id, request.Clean);

			return Ok();
		}

		/// <summary>
		/// Delete a specific device
		/// </summary>
		[HttpDelete]
		[Authorize]
		[Route("/api/v2/devices/{deviceId}")]
		public async Task<ActionResult> DeleteDeviceAsync(string deviceId)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceWrite, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			DeviceId deviceIdValue = new DeviceId(deviceId);

			IDevice? device = await _deviceService.GetDeviceAsync(deviceIdValue);
			if (device == null)
			{
				return NotFound();
			}

			DevicePoolAuthorization? poolAuth = _deviceService.GetUserPoolAuthorization(device.PoolId, User, _buildConfig.CurrentValue);

			if (poolAuth == null || !poolAuth.Write)
			{
				return Forbid();
			}

			await _deviceService.DeleteDeviceAsync(deviceIdValue);
			return Ok();
		}

		// PLATFORMS

		/// <summary>
		/// Get a list of supported device platforms
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/platforms")]
		[ProducesResponseType(typeof(List<GetDevicePlatformResponse>), 200)]
		public ActionResult<List<object>> GetDevicePlatforms()
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceRead, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			List<IDevicePlatform> platforms = _deviceService.GetPlatforms();

			List<object> responses = new List<object>();

			foreach (IDevicePlatform platform in platforms)
			{
				// @todo: ACL per platform
				responses.Add(new GetDevicePlatformResponse(platform.Id.ToString(), platform.Name, platform.Models?.ToArray() ?? Array.Empty<string>()));
			}

			return responses;
		}

		// POOLS

		/// <summary>
		/// Get a list of existing device pools
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/pools")]
		[ProducesResponseType(typeof(List<GetDevicePoolResponse>), 200)]
		public ActionResult<List<object>> GetDevicePools()
		{
			List<DevicePoolAuthorization> poolAuth = _deviceService.GetUserPoolAuthorizations(User, _buildConfig.CurrentValue);

			List<IDevicePool> pools = _deviceService.GetPools();

			List<object> responses = new List<object>();

			foreach (IDevicePool pool in pools)
			{
				DevicePoolAuthorization? auth = poolAuth.Where(x => x.Pool.Id == pool.Id).FirstOrDefault();

				if (auth == null || !auth.Read)
				{
					continue;
				}

				responses.Add(new GetDevicePoolResponse(pool.Id.ToString(), pool.Name, pool.PoolType, auth.Write));
			}

			return responses;
		}

		/// <summary>
		/// Get device telemetry
		/// </summary>		
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/pools/telemetry")]
		[ProducesResponseType(typeof(List<GetDevicePoolTelemetryResponse>), 200)]
		public async Task<ActionResult<List<GetDevicePoolTelemetryResponse>>> GetDevicePoolTelemetryAsync(
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 1024)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceRead, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			if (minCreateTime == null)
			{
				minCreateTime = new DateTimeOffset(DateTime.UtcNow - TimeSpan.FromHours(24));
			}

			List<IDevicePoolTelemetry> telemetryData = await _deviceService.GetDevicePoolTelemetryAsync(minCreateTime, maxCreateTime, index, count);

			List<GetDevicePoolTelemetryResponse> response = new List<GetDevicePoolTelemetryResponse>();

			foreach (IDevicePoolTelemetry t in telemetryData)
			{
				Dictionary<string, List<GetDevicePlatformTelemetryResponse>> poolData = new Dictionary<string, List<GetDevicePlatformTelemetryResponse>>();

				foreach (KeyValuePair<DevicePoolId, IReadOnlyList<IDevicePlatformTelemetry>> pool in t.Pools)
				{
					List<GetDevicePlatformTelemetryResponse> platformTelemetry = new List<GetDevicePlatformTelemetryResponse>();

					foreach (IDevicePlatformTelemetry telemetry in pool.Value)
					{
						Dictionary<string, List<GetDevicePoolReservationTelemetryResponse>> reservedResponse = new Dictionary<string, List<GetDevicePoolReservationTelemetryResponse>>();

						IReadOnlyDictionary<string, IReadOnlyList<IDevicePoolReservationTelemetry>>? reserved = telemetry.Reserved?.ToDictionary(kvp => kvp.Key.ToString(), kvp => kvp.Value);
						if (reserved != null && reserved.Count > 0)
						{

							foreach (KeyValuePair<string, IReadOnlyList<IDevicePoolReservationTelemetry>> r in reserved)
							{
								reservedResponse[r.Key] = new List<GetDevicePoolReservationTelemetryResponse>();
								foreach (IDevicePoolReservationTelemetry u in r.Value)
								{
									reservedResponse[r.Key].Add(new GetDevicePoolReservationTelemetryResponse(u.DeviceId.ToString(), u.JobId, u.StepId, u.JobName, u.StepName));
								}
							}
						}
						platformTelemetry.Add(new GetDevicePlatformTelemetryResponse(telemetry.PlatformId.ToString(), telemetry.Available?.Select(d => d.ToString()).ToList(), telemetry.Maintenance?.Select(d => d.ToString()).ToList(), telemetry.Problem?.Select(d => d.ToString()).ToList(), telemetry.Disabled?.Select(d => d.ToString()).ToList(), reservedResponse));
					}

					poolData[pool.Key.ToString()] = platformTelemetry;
				}

				response.Add(new GetDevicePoolTelemetryResponse(t.CreateTimeUtc, poolData));
			}

			return response;
		}

		#region Reservations

		/// <summary>
		/// Create a new device reservation
		/// </summary>
		[HttpPost]
		[Authorize]
		[Route("/api/v2/devices/reservations")]
		[ProducesResponseType(typeof(CreateDeviceReservationResponse), 200)]
		public async Task<ActionResult<CreateDeviceReservationResponse>> CreateDeviceReservationAsync([FromBody] CreateDeviceReservationRequest request)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceWrite, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			List<IDevicePool> pools = _deviceService.GetPools();
			List<IDevicePlatform> platforms = _deviceService.GetPlatforms();

			DevicePoolId poolIdValue = new DevicePoolId(request.PoolId);
			IDevicePool? pool = pools.FirstOrDefault(x => x.Id == poolIdValue);
			if (pool == null)
			{
				return BadRequest($"Unknown pool {request.PoolId} on device reservation request");
			}

			List<DeviceRequestData> requestedDevices = new List<DeviceRequestData>();

			foreach (DeviceReservationRequest deviceRequest in request.Devices)
			{
				DevicePlatformId platformIdValue = new DevicePlatformId(deviceRequest.PlatformId);
				IDevicePlatform? platform = platforms.FirstOrDefault(x => x.Id == platformIdValue);

				if (platform == null)
				{
					return BadRequest($"Unknown platform {deviceRequest.PlatformId} on device reservation request");
				}

				if (deviceRequest.IncludeModels != null)
				{
					foreach (string model in deviceRequest.IncludeModels)
					{
						if (platform.Models?.FirstOrDefault(x => x == model) == null)
						{
							return BadRequest($"Unknown model {model} for platform {deviceRequest.PlatformId} on device reservation request");
						}
					}
				}

				if (deviceRequest.ExcludeModels != null)
				{
					foreach (string model in deviceRequest.ExcludeModels)
					{
						if (platform.Models?.FirstOrDefault(x => x == model) == null)
						{
							return BadRequest($"Unknown model {model} for platform {deviceRequest.PlatformId} on device reservation request");
						}
					}
				}

				HashSet<string>? duplicateTags = deviceRequest.Tags?.GetDuplicateEntries();
				if (duplicateTags != null && duplicateTags.Count > 0)
				{
					string duplicateList = String.Join(", ", duplicateTags);
					return BadRequest($"The following device tags were requested more than once: {duplicateList}");
				}

				requestedDevices.Add(new DeviceRequestData(platformIdValue, platformIdValue.ToString(), deviceRequest.IncludeModels, deviceRequest.ExcludeModels, tags: deviceRequest.Tags));
			}

			(IDeviceReservation? reservation, string? errorMessage, bool installRequired) = await _deviceService.TryCreateReservationAsync(poolIdValue, requestedDevices);

			if (reservation == null)
			{
				return Conflict(String.IsNullOrEmpty(errorMessage) ? "Unable to allocated devices for reservation" : errorMessage);
			}

			List<IDevice> devices = await _deviceService.GetDevicesAsync(reservation.Devices);

			CreateDeviceReservationResponse response = new CreateDeviceReservationResponse();

			response.Id = reservation.Id.ToString();
			response.Devices = new List<GetDeviceResponse>();

			foreach (IDevice device in devices)
			{
				List<GetDeviceUtilizationResponse>? utilization = device.Utilization?.Select(u => new GetDeviceUtilizationResponse
				{
					JobId = u.JobId,
					StepId = u.StepId,
					ReservationStartUtc = u.ReservationStartUtc,
					ReservationFinishUtc = u.ReservationFinishUtc
				}).ToList();				
				response.Devices.Add(new GetDeviceResponse(device.Id.ToString(), device.PlatformId.ToString(), device.PoolId.ToString(), device.Name, device.Enabled, device.Address, device.ModelId?.ToString(), device.ModifiedByUser, device.Notes, device.ProblemTimeUtc, device.MaintenanceTimeUtc, utilization));
			}

			return response;
		}

		/// <summary>
		/// Get active device reservations
		/// </summary>		
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/reservations")]
		[ProducesResponseType(typeof(List<GetDeviceReservationResponse>), 200)]
		public async Task<ActionResult<List<GetDeviceReservationResponse>>> GetDeviceReservationsAsync()
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceRead, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			List<IDeviceReservation> reservations = await _deviceService.GetReservationsAsync();

			List<GetDeviceReservationResponse> response = new List<GetDeviceReservationResponse>();

			foreach (IDeviceReservation reservation in reservations)
			{
				GetDeviceReservationResponse reservationResponse = new GetDeviceReservationResponse()
				{
					Id = reservation.Id.ToString(),
					PoolId = reservation.PoolId.ToString(),
					Devices = reservation.Devices.Select(x => x.ToString()).ToList(),
					JobId = reservation.JobId?.ToString(),
					StepId = reservation.StepId?.ToString(),
					JobName = reservation.JobName?.ToString(),
					StepName = reservation.StepName?.ToString(),
					UserId = reservation.UserId?.ToString(),
					Hostname = reservation.Hostname,
					ReservationDetails = reservation.ReservationDetails,
					CreateTimeUtc = reservation.CreateTimeUtc,
					LegacyGuid = reservation.LegacyGuid
				};

				response.Add(reservationResponse);

			}

			return response;
		}

		/// <summary>
		/// Renew an existing reservation
		/// </summary>
		[HttpPut]
		[Authorize]
		[Route("/api/v2/devices/reservations/{reservationId}")]
		public async Task<ActionResult> UpdateReservationAsync(string reservationId)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceWrite, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			ObjectId reservationIdValue = new ObjectId(reservationId);

			IDeviceReservation? reservation = await _deviceService.TryUpdateReservationAsync(reservationIdValue);

			if (reservation == null)
			{
				return BadRequest("Failed to update reservation");
			}

			return Ok();
		}

		/// <summary>
		/// Delete a reservation
		/// </summary>
		[HttpDelete]
		[Authorize]
		[Route("/api/v2/devices/reservations/{reservationId}")]
		public async Task<ActionResult> DeleteReservationAsync(string reservationId)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceWrite, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			ObjectId reservationIdValue = new ObjectId(reservationId);

			await _deviceService.DeleteReservationAsync(reservationIdValue);

			return Ok();
		}

		#endregion

		/// <summary>
		/// Get device telemetry
		/// </summary>		
		[HttpGet]
		[Authorize]
		[Route("/api/v2/devices/telemetry")]
		[ProducesResponseType(typeof(List<GetDeviceTelemetryResponse>), 200)]
		public async Task<ActionResult<List<GetDeviceTelemetryResponse>>> GetDeviceTelemetryAsync(
			[FromQuery(Name = "Id")] string[]? deviceIds = null,
			[FromQuery] string? poolId = null,
			[FromQuery] string? platformId = null,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 1024)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceRead, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			if (minCreateTime == null)
			{
				minCreateTime = new DateTimeOffset(DateTime.UtcNow - TimeSpan.FromHours(24));
			}

			HashSet<DeviceId> deviceIdValues = new HashSet<DeviceId>();

			if (deviceIds != null && deviceIds.Length > 0)
			{
				foreach (string deviceId in deviceIds)
				{
					deviceIdValues.Add(new DeviceId(deviceId));
				}
			}

			DevicePoolId? poolIdValue = poolId == null ? null : new DevicePoolId(poolId);
			DevicePlatformId? platformIdValue = platformId == null ? null : new DevicePlatformId(platformId);

			List<IDevice> devices = await _deviceService.GetDevicesAsync(deviceIdValues.Count > 0 ? deviceIdValues.ToList() : null, poolIdValue, platformIdValue);

			List<GetDeviceTelemetryResponse> response = new List<GetDeviceTelemetryResponse>();

			if (devices.Count == 0)
			{
				return response;
			}

			List<IDeviceTelemetry> telemetryData = await _deviceService.GetDeviceTelemetryAsync(devices.Select(x => x.Id).ToArray(), minCreateTime, maxCreateTime, index, count);

			Dictionary<string, List<GetTelemetryInfoResponse>> results = new Dictionary<string, List<GetTelemetryInfoResponse>>();

			foreach (IDeviceTelemetry t in telemetryData)
			{
				List<GetTelemetryInfoResponse>? info = null;
				string deviceId = t.DeviceId.ToString();
				if (!results.TryGetValue(deviceId, out info))
				{
					info = new List<GetTelemetryInfoResponse>();
					results[deviceId] = info;
				}

				info.Add(new GetTelemetryInfoResponse
				{
					CreateTimeUtc = t.CreateTimeUtc,
					StreamId = t.StreamId,
					JobId = t.JobId,
					StepId = t.StepId,
					JobName = t.JobName,
					StepName = t.StepName,
					ReservationStartUtc = t.ReservationStartUtc,
					ReservationFinishUtc = t.ReservationFinishUtc,
					ProblemTimeUtc = t.ProblemTimeUtc
				});
			}

			foreach (KeyValuePair<string, List<GetTelemetryInfoResponse>> result in results)
			{
				GetDeviceTelemetryResponse deviceTelemetry = new GetDeviceTelemetryResponse(result.Key, result.Value);
				response.Add(deviceTelemetry);
			}

			return response;
		}

		/// <summary>
		/// Retrieve historical information about a specific device
		/// </summary>
		/// <param name="deviceId">Id of the device to get information about</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/devices/{deviceId}/history")]
		public async Task<ActionResult> GetDeviceHistoryAsync(DeviceId deviceId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50)
		{
			if (!DeviceService.Authorize(DeviceAclAction.DeviceRead, User, _buildConfig.CurrentValue))
			{
				return Forbid();
			}

			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync();
			await _deviceService.GetDeviceLogger(deviceId).FindAsync(Response.BodyWriter, minTime, maxTime, index, count);
			await Response.CompleteAsync();

			return Empty;
		}

		// LEGACY V1 API

		enum LegacyPerfSpec
		{
			Unspecified,
			Minimum,
			Recommended,
			High
		};

		/// <summary>
		/// Create a device reservation
		/// </summary>
		[HttpPost]
		[Route("/api/v1/reservations")]
		[ProducesResponseType(typeof(GetLegacyReservationResponse), 200)]
		public async Task<ActionResult<GetLegacyReservationResponse>> CreateDeviceReservationV1Async([FromBody] LegacyCreateReservationRequest request)
		{

			List<IDevicePool> pools = _deviceService.GetPools();
			List<IDevicePlatform> platforms = _deviceService.GetPlatforms();

			_logger.LogInformation("Device reservation Pools:{Pools} Platforms:{Platforms}", String.Join(';', pools.Select(p => p.Name)), String.Join(';', platforms.Select(p => p.Name)));

			string? poolId = request.PoolId;

			// @todo: Remove this once all streams are updated to provide jobid
			string details = "";
			if ((String.IsNullOrEmpty(request.JobId) || String.IsNullOrEmpty(request.StepId)))
			{
				if (!String.IsNullOrEmpty(request.ReservationDetails))
				{
					details = $" - {request.ReservationDetails}";
				}
				else
				{
					details = $" - Host {request.Hostname}";
				}
			}

			if (String.IsNullOrEmpty(poolId))
			{
				// @todo: We default to ue4, though this should be an error, or a configuration setting
				poolId = "ue4";
				//return BadRequest(Message);
			}

			DevicePoolId poolIdValue = DevicePoolId.Sanitize(poolId);
			IDevicePool? pool = pools.FirstOrDefault(x => x.Id == poolIdValue);
			if (pool == null)
			{
				_logger.LogError("Unknown pool {PoolId} {Details}", poolId, details);
				return BadRequest($"Unknown pool {poolId} " + details);
			}

			List<DeviceRequestData> requestedDevices = new List<DeviceRequestData>();

			HashSet<string> legacyPerfSpecs = new HashSet<string> { "Unspecified", "Minimum", "Recommended", "High" };

			List<string> perfSpecs = new List<string>();

			foreach (string deviceType in request.DeviceTypes)
			{

				string platformName = deviceType;
				string? constraint = null;

				if (deviceType.Contains(':', StringComparison.Ordinal))
				{
					string[] tokens = deviceType.Split(":");
					platformName = tokens[0];
					constraint = tokens[1];
				}

				DevicePlatformId platformId = DevicePlatformId.Sanitize(platformName);

				_logger.LogInformation("Device reservation platform {PlatformId}", platformId.ToString());

				IDevicePlatform? platform = platforms.FirstOrDefault(x => x.Id == platformId || (x.LegacyNames != null && x.LegacyNames.Contains(platformName)));

				if (platform == null)
				{
					return BadRequest($"Unknown platform {platformId}" + details);
				}

				List<string> includeModels = new List<string>();
				List<string> excludeModels = new List<string>();

				if (constraint == null)
				{
					perfSpecs.Add("Unspecified");
				}
				else if (legacyPerfSpecs.Contains(constraint))
				{
					perfSpecs.Add(constraint);

					if (constraint == "High")
					{
						if (!String.IsNullOrEmpty(platform.LegacyPerfSpecHighModel))
						{
							includeModels.Add(platform.LegacyPerfSpecHighModel);
						}
					}
					else if (constraint == "Minimum" || constraint == "Recommended")
					{
						if (!String.IsNullOrEmpty(platform.LegacyPerfSpecHighModel))
						{
							excludeModels.Add(platform.LegacyPerfSpecHighModel);
						}
					}
				}
				else
				{
					string[] requestedModels = constraint.Split(';');

					if (requestedModels.Length > 0)
					{
						List<string> platformModels = new List<string>(platform.Models ?? new List<string>());
						platformModels.Add("Base");

						List<string> models = platformModels.Where(x => requestedModels.Contains(x, StringComparer.OrdinalIgnoreCase)).ToList();

						if (models == null || models.Count == 0)
						{
							return NotFound($"Invalid model constraint for platform {platform.Id}: {constraint}");
						}

						includeModels = models;
					}
				}

				requestedDevices.Add(new DeviceRequestData(platform.Id, platformName, includeModels, excludeModels, request.DeviceName, request.Tags));
			}

			HashSet<string>? duplicateTags = request.Tags?.GetDuplicateEntries();
			if (duplicateTags != null && duplicateTags.Count > 0)
			{
				string duplicateList = String.Join(", ", duplicateTags);
				return BadRequest($"The following device tags were requested more than once: {duplicateList}");
			}

			(IDeviceReservation? reservation, string? errorMessage, bool installRequired) = await _deviceService.TryCreateReservationAsync(poolIdValue, requestedDevices, request.Hostname, request.ReservationDetails, request.JobId == null ? null : JobId.Parse(request.JobId), request.StepId == null ? null : JobStepId.Parse(request.StepId));

			if (reservation == null)
			{
				return Conflict(String.IsNullOrEmpty(errorMessage) ? "Unable to allocated devices for reservation" : errorMessage);
			}

			List<IDevice> devices = await _deviceService.GetDevicesAsync(reservation.Devices);

			GetLegacyReservationResponse response = new GetLegacyReservationResponse();

			response.Guid = reservation.LegacyGuid;
			response.DeviceNames = devices.Select(x => x.Name).ToArray();
			response.DevicePerfSpecs = perfSpecs.ToArray();
			response.DeviceModels = devices.Select(x => x.ModelId ?? "Base").ToArray();
			response.HostName = request.Hostname;
			response.StartDateTime = reservation.CreateTimeUtc.ToString("O", System.Globalization.CultureInfo.InvariantCulture);
			response.Duration = $"{request.Duration}";
			response.JobId = reservation.JobId;
			response.StepId = reservation.StepId;
			response.JobName = reservation.JobName;
			response.StepName = reservation.StepName;
			response.InstallRequired = reservation.ReservedStepIds != null && reservation.ReservedStepIds.Count > 0 ? installRequired : null;

			return new JsonResult(response, new JsonSerializerOptions() { PropertyNamingPolicy = null });

		}

		/// <summary>
		/// Renew a reservation
		/// </summary>
		[HttpPut]
		[Route("/api/v1/reservations/{reservationGuid}")]
		[ProducesResponseType(typeof(GetLegacyReservationResponse), 200)]
		public async Task<ActionResult<GetLegacyReservationResponse>> UpdateReservationV1Async(string reservationGuid /* [FromBody] string Duration */)
		{
			IDeviceReservation? reservation = await _deviceService.TryGetReservationFromLegacyGuidAsync(reservationGuid);

			if (reservation == null)
			{
				_logger.LogError("Unable to find reservation for legacy guid {ReservationGuid}", reservationGuid);
				return BadRequest();
			}

			IDeviceReservation? updated = await _deviceService.TryUpdateReservationAsync(reservation.Id);

			if (updated == null)
			{
				_logger.LogError("Unable to find reservation for reservation {ReservationId}", reservation.Id);
				return BadRequest();
			}

			List<IDevice> devices = await _deviceService.GetDevicesAsync(reservation.Devices);

			GetLegacyReservationResponse response = new GetLegacyReservationResponse();

			response.Guid = reservation.LegacyGuid;
			response.DeviceNames = reservation.Devices.Select(deviceId => devices.First(d => d.Id == deviceId).Name).ToArray();
			response.HostName = reservation.Hostname ?? "";
			response.StartDateTime = reservation.CreateTimeUtc.ToString("O", System.Globalization.CultureInfo.InvariantCulture);
			response.Duration = "00:10:00"; // matches gauntlet duration

			return new JsonResult(response, new JsonSerializerOptions() { PropertyNamingPolicy = null });
		}

		/// <summary>
		/// Delete a reservation
		/// </summary>
		[HttpDelete]
		[Route("/api/v1/reservations/{reservationGuid}")]
		public async Task<ActionResult> DeleteReservationV1Async(string reservationGuid, [FromQuery] bool force = false)
		{
			IDeviceReservation? reservation = await _deviceService.TryGetReservationFromLegacyGuidAsync(reservationGuid);

			if (reservation == null)
			{
				return BadRequest($"Unable to find reservation for guid {reservationGuid}");
			}

			await _deviceService.DeleteReservationAsync(reservation.Id, force);

			return Ok();
		}

		/// <summary>
		/// Get device info for a reserved device
		/// </summary>
		[HttpGet]
		[Route("/api/v1/devices/{deviceName}")]
		[ProducesResponseType(typeof(GetLegacyDeviceResponse), 200)]
		public async Task<ActionResult<GetLegacyDeviceResponse>> GetDeviceV1Async(string deviceName)
		{
			IDevice? device = await _deviceService.GetDeviceByNameAsync(deviceName);

			if (device == null)
			{
				return BadRequest($"Unknown device {deviceName}");
			}

			IDevicePlatform? platform = _deviceService.GetPlatform(device.PlatformId);

			if (platform == null)
			{
				return BadRequest($"Unknown device platform {deviceName} {device.PlatformId}");
			}

			IDeviceReservation? reservation = await _deviceService.TryGetDeviceReservationAsync(device.Id);

			string? platformName = null;

			if (reservation != null && reservation.Devices.Count == reservation.RequestedDevicePlatforms.Count)
			{
				for (int i = 0; i < reservation.Devices.Count; i++)
				{
					if (reservation.Devices[i] == device.Id)
					{
						platformName = reservation.RequestedDevicePlatforms[i];
					}
				}
			}

			if (String.IsNullOrEmpty(platformName))
			{
				_logger.LogError("Unable to map platform for {DeviceName} : {PlatformId} from reservation", deviceName, device.PlatformId);
				return BadRequest($"Unable to get platform for {deviceName} from reservation : {device.PlatformId}");
			}

			GetLegacyDeviceResponse response = new GetLegacyDeviceResponse();

			response.Id = device.Id.ToString();
			response.Name = device.Name;
			response.Type = platformName;
			response.IPOrHostName = device.Address ?? "";
			response.AvailableStartTime = "00:00:00";
			response.AvailableEndTime = "00:00:00";
			response.Enabled = true;
			response.Model = device.ModelId ?? "Base";
			response.DeviceData = "";
			response.PerfSpec = "Minimum";
			response.Tags = device.Tags ?? new List<string>();

			if (device.ModelId != null)
			{
				if (!String.IsNullOrEmpty(platform.LegacyPerfSpecHighModel))
				{
					if (platform.LegacyPerfSpecHighModel == device.ModelId)
					{
						response.PerfSpec = "High";
					}
				}
			}

			return new JsonResult(response, new JsonSerializerOptions() { PropertyNamingPolicy = null });
		}

		/// <summary>
		/// Mark a problem device
		/// </summary>
		[HttpPut]
		[Route("/api/v1/deviceerror/{deviceName}")]
		public async Task<ActionResult> PutDeviceErrorAsync(string deviceName, [FromBody] DeviceErrorRequest? request = null)
		{
			IDevice? device = await _deviceService.GetDeviceByNameAsync(deviceName);

			if (device == null)
			{
				return BadRequest($"Unknown device {deviceName}");
			}

			await _deviceService.UpdateDeviceAsync(device.Id, newProblem: true, message: request?.Message);

			return Ok();
		}
	}
}
