// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Devices
{
	/// <summary>
	/// A reservation containing one or more devices
	/// </summary>
	public interface IDeviceReservation
	{
		/// <summary>
		/// Randomly generated unique id for this reservation
		/// </summary>
		public ObjectId Id { get; }

		/// <summary>
		/// Which device pool the reservation is in
		/// </summary>
		public DevicePoolId PoolId { get; }

		/// <summary>
		/// Strwam id holding reservation
		/// </summary>
		public string? StreamId { get; }

		/// <summary>
		/// JobID holding reservation
		/// </summary>
		public string? JobId { get; }

		/// <summary>
		/// Job step id holding reservation
		/// </summary>
		public string? StepId { get; }

		/// <summary>
		/// Job name holding reservation
		/// </summary>
		public string? JobName { get; }

		/// <summary>
		/// Job step name holding reservation
		/// </summary>
		public string? StepName { get; }

		/// <summary>
		/// Reservations held by a user, requires a token
		/// </summary>
		public UserId? UserId { get; }

		/// <summary>
		/// The hostname of machine holding reservation
		/// </summary>
		string? Hostname { get; }

		/// <summary>
		/// The hostname of machine holding reservation
		/// </summary>
		string? ReservationDetails { get; }

		/// <summary>
		/// The UTC time when the reservation was created
		/// </summary>
		DateTime CreateTimeUtc { get; }

		/// <summary>
		/// The last update time for reservation renewal (for expiration)
		/// </summary>
		DateTime UpdateTimeUtc { get; }

		/// <summary>
		/// The reserved devices
		/// </summary>
		public List<DeviceId> Devices { get; }

		/// <summary>
		/// StepIds that use the reservation
		/// </summary>
		public List<JobStepId>? ReservedStepIds { get; }

		/// <summary>
		/// Whether a device problem was reported for the reservation
		/// </summary>
		public DeviceId? ProblemDevice { get; }

		/// <summary>
		/// The requested device platforms for reservation, which may differ from IDevice platform due to devices that support more than one platform, or legacy platforms
		/// </summary>
		public List<string> RequestedDevicePlatforms { get; }

		/// <summary>
		/// The legacy reservation system guid, to be removed once can update Gauntlet client in all streams
		/// </summary>
		public string LegacyGuid { get; }
	}

	/// <summary>
	/// A device platform 
	/// </summary>
	public interface IDevicePlatform
	{
		/// <summary>
		/// Unique identifier of device platform 
		/// </summary>
		public DevicePlatformId Id { get; }

		/// <summary>
		/// Device platform name, for example Android, PS5, etc
		/// </summary>
		string Name { get; }

		/// <summary>
		/// A list of valid models for the platform
		/// </summary>
		public IReadOnlyList<string>? Models { get; }

		/// <summary>
		/// Legacy names which older versions of Gauntlet may be using
		/// </summary>
		public IReadOnlyList<string>? LegacyNames { get; }

		/// <summary>
		/// Model name for the high perf spec, which may be requested by Gauntlet
		/// </summary>
		public string? LegacyPerfSpecHighModel { get; }
	}

	/// <summary>
	/// A logical pool of devices
	/// </summary>
	public interface IDevicePool
	{
		/// <summary>
		/// Unique identifier of pool
		/// </summary>
		public DevicePoolId Id { get; }

		/// <summary>
		/// The type of pool 
		/// </summary>
		public DevicePoolType PoolType { get; }

		/// <summary>
		/// Projects associated with this device pool
		/// </summary>
		public List<ProjectId>? ProjectIds { get; }

		/// <summary>
		/// Friendly name of the pool
		/// </summary>
		string Name { get; }
	}

	/// <summary>
	/// A device utilization snapshot [DEPRECATED]
	/// </summary>
	public class DeviceUtilizationTelemetry
	{
		/// <summary>
		/// The job id which utilized device
		/// </summary>
		public string? JobId { get; set; }

		/// <summary>
		/// The job's step id
		/// </summary>
		public string? StepId { get; set; }

		/// <summary>
		/// The time device was reserved
		/// </summary>
		public DateTime ReservationStartUtc { get; set; }

		/// <summary>
		/// The time device was freed
		/// </summary>
		public DateTime? ReservationFinishUtc { get; set; }

		/// <summary>
		/// Private constructor
		/// </summary>
		[BsonConstructor]
		private DeviceUtilizationTelemetry()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="reservationStartUtc"></param>
		public DeviceUtilizationTelemetry(DateTime reservationStartUtc)
		{
			ReservationStartUtc = reservationStartUtc;
		}
	}

	/// <summary>
	/// Device telemetry information
	/// </summary>
	public interface IDeviceTelemetry
	{
		/// <summary>
		///  The creation time of this telemetry data
		/// </summary>
		public DateTime CreateTimeUtc { get; }

		/// <summary>
		/// The device id for telemetry
		/// </summary>
		public DeviceId DeviceId { get; }

		/// <summary>
		/// The stream id which utilized device
		/// </summary>
		public string? StreamId { get; }

		/// <summary>
		/// The job id which utilized device
		/// </summary>
		public string? JobId { get; }

		/// <summary>
		/// The job name which utilized device
		/// </summary>
		public string? JobName { get; }

		/// <summary>
		/// The job's step id
		/// </summary>
		public string? StepId { get; }

		/// <summary>
		/// The job name which utilized device
		/// </summary>
		public string? StepName { get; }

		/// <summary>
		/// Reservation Id (transient, reservations are deleted upon expiration)
		/// </summary>
		public ObjectId? ReservationId { get; }

		/// <summary>
		/// The time device was reserved
		/// </summary>
		public DateTime? ReservationStartUtc { get; }

		/// <summary>
		/// The time device was freed
		/// </summary>
		public DateTime? ReservationFinishUtc { get; }

		/// <summary>
		/// If the device reported a problem
		/// </summary>
		public DateTime? ProblemTimeUtc { get; }
	}

	/// <summary>
	/// A physical device
	/// </summary>
	public interface IDevice
	{
		/// <summary>
		/// The unique id of the device
		/// </summary>
		public DeviceId Id { get; }

		/// <summary>
		/// The platform of the device
		/// </summary>
		public DevicePlatformId PlatformId { get; }

		/// <summary>
		/// Which pool the device belongs to
		/// </summary>
		public DevicePoolId PoolId { get; }

		/// <summary>
		/// Friendly name of the device
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// The model of the device
		/// </summary>
		public string? ModelId { get; }

		/// <summary>
		/// Address if the device supports shared network connections
		/// </summary>
		public string? Address { get; }

		/// <summary>
		/// Whether the device is currently enabled
		/// </summary>
		public bool Enabled { get; }

		/// <summary>
		/// Id of the user that last modified this device
		/// </summary>
		public string? ModifiedByUser { get; }

		/// <summary>
		/// Id of the user that has this device checked out
		/// </summary>
		public string? CheckedOutByUser { get; }

		/// <summary>
		/// The last time this device was checked out
		/// </summary>
		public DateTime? CheckOutTime { get; }

		/// <summary>
		/// The last time a problem was reported
		/// </summary>
		public DateTime? ProblemTimeUtc { get; }

		/// <summary>
		/// The time device was marked for maintenance
		/// </summary>
		public DateTime? MaintenanceTimeUtc { get; }

		/// <summary>
		/// Markdown notes for device if any 
		/// </summary>
		public string? Notes { get; }

		/// <summary>
		/// Metadata tags on the device
		/// </summary>
		public List<string>? Tags { get; }

		/// <summary>
		/// Whether the device requires a clean
		/// </summary>
		public bool? CleanRequired { get; }

		/// <summary>
		/// The time device the device cleaning started
		/// </summary>
		public DateTime? CleanStartTimeUtc { get; }

		/// <summary>
		/// The time the device cleaning finished
		/// </summary>
		public DateTime? CleanFinishTimeUtc { get; }

		/// <summary>
		/// Device job utilization history 
		/// </summary>
		public List<DeviceUtilizationTelemetry>? Utilization { get; set; }
	}

	/// <summary>
	/// Stream device telemetry for pool snapshot
	/// </summary>
	public interface IDevicePoolReservationTelemetry
	{
		/// <summary>
		/// Device id for reservation
		/// </summary>
		public DeviceId DeviceId { get; }

		/// <summary>
		/// Job id associated with reservation
		/// </summary>
		public string? JobId { get; }

		/// <summary>
		/// The step id of reservation
		/// </summary>
		public string? StepId { get; }

		/// <summary>
		/// The name of the job holding reservation
		/// </summary>
		public string? JobName { get; }

		/// <summary>
		/// The name of the step holding reservation
		/// </summary>
		public string? StepName { get; }
	}

	/// <summary>
	/// Platform telemetry for a device pool
	/// </summary>
	public interface IDevicePlatformTelemetry
	{
		/// <summary>
		/// The corresponding platform id
		/// </summary>
		public DevicePlatformId PlatformId { get; }

		/// <summary>
		/// Available devices of this platform 
		/// </summary>
		public IReadOnlyList<DeviceId>? Available { get; }

		/// <summary>
		/// Number of devices in maintenance state
		/// </summary>
		public IReadOnlyList<DeviceId>? Maintenance { get; }

		/// <summary>
		/// Number of devices in problem state
		/// </summary>
		public IReadOnlyList<DeviceId>? Problem { get; }

		/// <summary>
		/// Number of devices in disabled state
		/// </summary>
		public IReadOnlyList<DeviceId>? Disabled { get; }

		/// <summary>
		/// Number of reserved devices of this platform 
		/// </summary>
		public IReadOnlyDictionary<StreamId, IReadOnlyList<IDevicePoolReservationTelemetry>>? Reserved { get; }
	}

	/// <summary>
	/// A snapshot of device pool telemetry
	/// </summary>
	public interface IDevicePoolTelemetry
	{
		/// <summary>
		/// The creation time of this telemetry data
		/// </summary>
		public DateTime CreateTimeUtc { get; }

		/// <summary>
		/// Pool platform telemetry
		/// </summary>
		public IReadOnlyDictionary<DevicePoolId, IReadOnlyList<IDevicePlatformTelemetry>> Pools { get; }
	}
}
