// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Devices;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Devices
{
	/// <summary>
	/// Device reservation request data
	/// </summary>
	public class DeviceRequestData
	{
		/// <summary>
		/// The platform of the device to reserve
		/// </summary>
		public DevicePlatformId PlatformId { get; set; }

		/// <summary>
		/// The platform client requested, which can differ from the devices platformid due to mapping
		/// </summary>
		public string RequestedPlatform { get; set; }

		/// <summary>
		/// Models to include for this request
		/// </summary>
		public List<string> IncludeModels { get; set; }

		/// <summary>
		/// Models to exclude for this request
		/// </summary>
		public List<string> ExcludeModels { get; set; }

		/// <summary>
		/// Specific device to reserve
		/// </summary>
		public string? DeviceName { get; set; } = null;

		/// <summary>
		/// Desired Tags of the device
		/// </summary>
		public DeviceTagsRequest? Tags { get; set; } = null;

		/// <summary>
		/// Constructor
		/// </summary>
		public DeviceRequestData(DevicePlatformId platformId, string requestedPlatform, List<string>? includeModels = null, List<string>? excludeModels = null,
			string? deviceName = null, DeviceTagsRequest? tags = null)
		{
			PlatformId = platformId;
			IncludeModels = includeModels ?? new List<string>();
			ExcludeModels = excludeModels ?? new List<string>();
			RequestedPlatform = requestedPlatform;
			DeviceName = deviceName;
			Tags = tags;
		}
	}

	/// <summary>
	/// A collection for device management
	/// </summary>
	public interface IDeviceCollection
	{
		// DEVICES

		/// <summary>
		/// Get a device by id
		/// </summary>
		Task<IDevice?> GetDeviceAsync(DeviceId deviceId);

		/// <summary>
		/// Get a device by name
		/// </summary>
		Task<IDevice?> GetDeviceByNameAsync(string deviceName);

		/// <summary>
		/// Get a list of all devices
		/// </summary>
		/// <param name="deviceIds">Optional list of device ids to get</param>
		/// <param name="poolId">Optional id of pool for devices</param>
		/// <param name="platformId">Optional id of platform for devices</param>
		Task<List<IDevice>> FindAllDevicesAsync(List<DeviceId>? deviceIds, DevicePoolId? poolId = null, DevicePlatformId? platformId = null);

		/// <summary>
		/// Adds a new device
		/// </summary>
		/// <param name="id">The device id</param>
		/// <param name="name">The name of the device (unique)</param>
		/// <param name="platformId">The platform of the device</param>
		/// <param name="poolId">Which pool to add it to</param>
		/// <param name="enabled">Whether the device is enabled by default</param>
		/// <param name="address">The network address or hostname the device can be reached at</param>
		/// <param name="modelId">The vendor model id of the device</param>
		/// <param name="userId">The user adding the device</param>
		/// <param name="tags">The metadata tags of the device</param>
		Task<IDevice?> TryAddDeviceAsync(DeviceId id, string name, DevicePlatformId platformId, DevicePoolId poolId, bool? enabled, string? address, string? modelId, UserId? userId, List<string>? tags = null);

		/// <summary>
		/// Update a device
		/// </summary>
		/// <param name="deviceId">The id of the device to update</param>
		/// <param name="newPoolId">The new pool to assign</param>
		/// <param name="newName">The new name to assign</param>		
		/// <param name="newAddress">The new device address or hostname</param>
		/// <param name="newModelId">The new model id</param>
		/// <param name="newNotes">The devices markdown notes</param>
		/// <param name="newEnabled">Whether the device is enabled or not</param>
		/// <param name="newProblem">Whether to set or clear problem state</param>
		/// <param name="newMaintenance">Whether to set or clear maintenance state</param>
		/// <param name="modifiedByUserId">The user who is updating the device</param>
		/// <param name="message">Attached a message to the log of the device</param>
		/// <param name="newTags">The new metadata tags to assign</param>
		Task UpdateDeviceAsync(DeviceId deviceId, DevicePoolId? newPoolId, string? newName, string? newAddress, string? newModelId, string? newNotes, bool? newEnabled, bool? newProblem, bool? newMaintenance, UserId? modifiedByUserId = null, string? message = null, List<string>? newTags = null);

		/// <summary>
		/// Updates a device's cleaning status
		/// </summary>
		/// <param name="deviceId">The device id to update</param>
		/// <param name="cleaning">Whether the device is actively being cleaned</param>
		/// <param name="cleanRequired">Whetner the device requires cleaning, for example after being checked in</param>
		/// <returns></returns>
		Task UpdateDeviceCleaningAsync(DeviceId deviceId, bool? cleaning = null, bool? cleanRequired = null);

		/// <summary>
		/// Delete a device from the collection
		/// </summary>
		/// <param name="deviceId">The id of the device to delete</param>
		Task<bool> DeleteDeviceAsync(DeviceId deviceId);

		/// <summary>
		/// Checkout or checkin the specified device
		/// </summary>
		/// <param name="deviceId"></param>
		/// <param name="checkedOutByUserId"></param>
		/// <returns></returns>
		Task CheckoutDeviceAsync(DeviceId deviceId, UserId? checkedOutByUserId);

		// RESERVATIONS

		/// <summary>
		/// Tries to find an existing reservation block
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="stepId"></param>
		/// <returns></returns>
		Task<IDeviceReservation?> TryFindReserveBlockAsync(JobId? jobId, JobStepId? stepId);

		/// <summary>
		/// Create a new reseveration in the pool with the specified devices
		/// </summary>
		/// <param name="poolId">The pool of devices to use for the new reservation</param>
		/// <param name="request">The requested devices for the reservation</param>
		/// <param name="problemCooldown">The configured problem device cooldown in minutes</param>
		/// <param name="hostname">The hostname of the machine making the reservation</param>
		/// <param name="reservationDetails">The details of the reservation</param>
		/// <param name="job">The job reserving the device</param>
		/// <param name="stepId">The Step Id associated with the job</param>
		/// <param name="stepName">The Step name associated with the job</param>
		/// <param name="stepIds">The step ids of the job to hold sequential reservation</param>		
		Task<(IDeviceReservation?, bool)> TryAddReservationAsync(DevicePoolId poolId, List<DeviceRequestData> request, int problemCooldown, string? hostname, string? reservationDetails, IJob? job, JobStepId? stepId, string? stepName, List<JobStepId>? stepIds);

		/// <summary>
		/// Gets a reservation by guid for legacy clients
		/// </summary>
		/// <param name="legacyGuid">YThe legacy guid of the reservation</param>
		Task<IDeviceReservation?> TryGetReservationFromLegacyGuidAsync(string legacyGuid);

		/// <summary>
		/// Gets a reservation by reservation id
		/// </summary>
		/// <param name="reservationId">The id of the reservation</param>
		Task<IDeviceReservation?> TryGetReservationAsync(ObjectId reservationId);

		/// <summary>
		/// Gets a reservation by device id
		/// </summary>
		/// <param name="id">A device contained in reservation</param>
		Task<IDeviceReservation?> TryGetDeviceReservationAsync(DeviceId id);

		/// <summary>
		/// Get a list of all reservations
		/// </summary>
		Task<List<IDeviceReservation>> FindAllReservationsAsync();

		/// <summary>
		/// Updates a reservation to the current time, for expiration
		/// </summary>
		/// <param name="id">The id of the reservation to update</param>
		/// <param name="problemDevice">Whether the current device has a problem</param>
		/// <param name="deviceIds">New devices assigned to reservation</param>
		/// <param name="clearProblemDevice">Whether to clear problem device</param>
		public Task<IDeviceReservation?> TryUpdateReservationAsync(ObjectId id, DeviceId? problemDevice = null, List<DeviceId>? deviceIds = null, bool? clearProblemDevice = null);

		/// <summary>
		/// Deletes a reservation and releases reserved devices
		/// </summary>
		/// <param name="id">The id of the reservation to delete</param>
		public Task<bool> DeleteReservationAsync(ObjectId id);

		/// <summary>
		/// Deletes user device checkouts
		/// </summary>
		/// <param name="checkoutDays"></param>
		public Task<List<(UserId, IDevice)>?> ExpireCheckedOutAsync(int checkoutDays);

		/// <summary>
		/// Gets a list of users to notify whose device checkout is about to expire
		/// </summary>
		/// <param name="checkoutDays"></param>
		public Task<List<(UserId, IDevice)>?> ExpireNotificatonsAsync(int checkoutDays);

		/// <summary>
		/// Find device telemetry
		/// </summary>
		/// <param name="deviceIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="index"></param>
		/// <param name="count"></param>
		/// <returns></returns>
		public Task<List<IDeviceTelemetry>> FindDeviceTelemetryAsync(DeviceId[]? deviceIds = null, DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null);

		/// <summary>
		/// Creates a device pool telemetry snapshot
		/// </summary>
		/// <returns></returns>
		public Task CreatePoolTelemetrySnapshotAsync(List<IDevicePool> pools, int poolCooldown);

		/// <summary>
		/// Gets pool telemetry for an optional date range
		/// </summary>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="index"></param>
		/// <param name="count"></param>
		/// <returns></returns>
		public Task<List<IDevicePoolTelemetry>> FindPoolTelemetryAsync(DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, int? index = null, int? count = null);

		/// <summary>
		/// Upgrades the collection
		/// </summary>
		public Task UpgradeAsync();

	}
}