<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="Grpc.AspNetCore" Version="2.59.0" />
    <PackageReference Include="Grpc.AspNetCore.Server.Reflection" Version="2.59.0" />
    <PackageReference Include="Grpc.Net.Client" Version="2.59.0" />
    <PackageReference Include="Grpc.Tools" Version="2.59.0">
      <PrivateAssets>all</PrivateAssets>
      <IncludeAssets>runtime; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Polly" Version="8.2.1" />
    <PackageReference Include="TimeZoneConverter" Version="6.1.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Analytics.Telemetry\EpicGames.Analytics.Telemetry.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.IoHash\EpicGames.IoHash.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Perforce.Managed\EpicGames.Perforce.Managed.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Redis\EpicGames.Redis.csproj" />
    <ProjectReference Include="..\..\..\..\Shared\EpicGames.Slack\EpicGames.Slack.csproj" />
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\Compute\HordeServer.Compute\HordeServer.Compute.csproj" />
    <ProjectReference Include="..\..\Secrets\HordeServer.Secrets\HordeServer.Secrets.csproj" />
    <ProjectReference Include="..\..\Tools\HordeServer.Tools\HordeServer.Tools.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Protobuf Include="Protos\horde\job_rpc.proto" ProtoRoot="Protos" GrpcServices="Server" AdditionalImportDirs="..\..\..\..\Shared\EpicGames.Horde\Protos;..\..\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="Protos\horde\job_rpc_messages.proto" ProtoRoot="Protos" AdditionalImportDirs="..\..\Compute\HordeServer.Compute\Protos" />
    <Protobuf Include="Protos\horde\common\label_change.proto" ProtoRoot="Protos" />
  </ItemGroup>

</Project>
