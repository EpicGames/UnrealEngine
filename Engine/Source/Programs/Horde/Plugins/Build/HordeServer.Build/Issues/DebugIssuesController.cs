// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Issues
{
	/// <summary>
	/// Debug functionality for issues
	/// </summary>
	[ApiController]
	[Authorize]
	[DebugEndpoint]
	[Tags("Debug")]
	public class DebugIssuesController : HordeControllerBase
	{
		private readonly IIssueCollection _issueCollection;
		private readonly IssueService _issueService;
		private readonly IUserCollection _userCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public DebugIssuesController(
			IIssueCollection issueCollection,
			IssueService issueService,
			IUserCollection userCollection,
			IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_issueCollection = issueCollection;
			_issueService = issueService;
			_userCollection = userCollection;
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Creates a test issue with synthetic suspects and triggers the notification pipeline.
		/// Useful for testing Slack DM notifications without requiring real Perforce commits and job failures.
		/// </summary>
		[HttpPost]
		[Route("/api/v1/debug/issues/create-test-issue")]
		public async Task<ActionResult> CreateTestIssueAsync(
			[FromQuery] StreamId streamId,
			[FromQuery] int commitId,
			[FromQuery] string[] suspectLogins,
			[FromQuery] int[] suspectCommitIds,
			[FromQuery] TemplateId? templateId = null,
			[FromQuery] string? nodeName = null,
			[FromQuery] string? summary = null,
			[FromQuery] string? fingerprintType = null,
			[FromQuery] string? jobId = null,
			[FromQuery] string? batchId = null,
			[FromQuery] string? stepId = null,
			[FromQuery] string? workflowId = null,
			[FromQuery] int? lastSuccessCommitId = null,
			[FromQuery] int? maxSuspectRank = null,
			[FromQuery] bool promoted = true)
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			if (suspectLogins.Length != suspectCommitIds.Length)
			{
				return BadRequest("suspectLogins and suspectCommitIds must have the same length");
			}

			if (!_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				return NotFound($"Stream '{streamId}' not found");
			}

			// Resolve users
			List<IUser> resolvedUsers = new List<IUser>();
			List<string> notFoundLogins = new List<string>();
			for (int i = 0; i < suspectLogins.Length; i++)
			{
				IUser? user = await _userCollection.FindUserByLoginAsync(suspectLogins[i], HttpContext.RequestAborted);
				if (user == null)
				{
					notFoundLogins.Add(suspectLogins[i]);
				}
				else
				{
					resolvedUsers.Add(user);
				}
			}

			if (notFoundLogins.Count > 0)
			{
				return BadRequest($"Could not resolve users: {String.Join(", ", notFoundLogins)}");
			}

			// Create the issue
			string issueSummary = summary ?? "Debug test issue";
			IIssue issue = await _issueCollection.AddIssueAsync(issueSummary, HttpContext.RequestAborted);

			// Promote the issue if requested
			if (promoted)
			{
				issue = await _issueCollection.TryUpdateIssueAsync(issue, null, new UpdateIssueOptions(Promoted: true), HttpContext.RequestAborted) ?? issue;
			}

			// Build fingerprint — use the summary text directly as the template so UpdateDerivedDataAsync doesn't overwrite it with "{Summary}"
			NewIssueFingerprint fingerprint = new NewIssueFingerprint(fingerprintType ?? "Compile", issueSummary, "Code");

			// Use real job/batch/step IDs when provided, otherwise generate synthetic ones
			JobId effectiveJobId = jobId != null ? JobId.Parse(jobId) : JobIdUtils.GenerateNewId();
			JobStepBatchId effectiveBatchId = batchId != null ? JobStepBatchId.Parse(batchId) : JobStepBatchId.GenerateNewId();
			JobStepId effectiveStepId = stepId != null ? JobStepId.Parse(stepId) : JobStepId.GenerateNewId();

			// Build annotations (for workflow routing)
			NodeAnnotations? annotations = null;
			if (workflowId != null)
			{
				annotations = new NodeAnnotations();
				annotations.WorkflowId = new WorkflowId(workflowId);
			}

			// Build step data
			TemplateId effectiveTemplateId = templateId ?? new TemplateId("test-executor");
			string effectiveNodeName = nodeName ?? "Test Node";
			NewIssueStepData stepData = new NewIssueStepData(
				CommitIdWithOrder.FromPerforceChange(commitId),
				IssueSeverity.Error,
				"Debug Test Job",
				effectiveJobId,
				effectiveBatchId,
				effectiveStepId,
				DateTime.UtcNow,
				null,
				annotations,
				promoted
			);

			// Build suspects
			List<NewIssueSpanSuspectData> suspects = new List<NewIssueSpanSuspectData>();
			for (int i = 0; i < resolvedUsers.Count; i++)
			{
				suspects.Add(new NewIssueSpanSuspectData(CommitIdWithOrder.FromPerforceChange(suspectCommitIds[i]), resolvedUsers[i].Id));
			}

			// Build and add span
			NewIssueSpanData spanData = new NewIssueSpanData(streamId, streamConfig.Name, effectiveTemplateId, effectiveNodeName, fingerprint, stepData);
			spanData.Suspects = suspects;
			spanData.MaxSuspectRank = maxSuspectRank;

			// Set LastSuccess if provided (enables CL range display in Slack instead of "No previous successes")
			if (lastSuccessCommitId != null)
			{
				spanData.LastSuccess = new NewIssueStepData(
					CommitIdWithOrder.FromPerforceChange(lastSuccessCommitId.Value),
					IssueSeverity.Error,
					"Debug Test Job",
					effectiveJobId,
					effectiveBatchId,
					effectiveStepId,
					DateTime.UtcNow.AddMinutes(-10),
					null,
					null,
					false
				);
			}
			IIssueSpan span = await _issueCollection.AddSpanAsync(issue.Id, spanData, HttpContext.RequestAborted);

			// Add step to span
			await _issueCollection.AddStepAsync(span.Id, stepData, HttpContext.RequestAborted);

			// Trigger derived data update + notification pipeline
			await _issueService.UpdateDerivedDataAsync(issue.Id, HttpContext.RequestAborted);

			return Ok(new
			{
				IssueId = issue.Id,
				Suspects = resolvedUsers.Select((u, i) => new { Login = suspectLogins[i], UserId = u.Id.ToString(), CommitId = suspectCommitIds[i] })
			});
		}
	}
}
