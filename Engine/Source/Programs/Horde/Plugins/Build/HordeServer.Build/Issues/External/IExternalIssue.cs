// Copyright Epic Games, Inc. All Rights Reserved.	

namespace HordeServer.Issues.External
{
	/// <summary>
	/// 
	/// </summary>
	public interface IExternalIssue
	{
		/// <summary>
		/// The issue key or tracking id
		/// </summary>
		public string Key { get; set; }

		/// <summary>
		/// The issue link on external tracking site
		/// </summary>
		public string? Link { get; set; }

		/// <summary>
		/// The issue status name, "To Do", "In Progress", etc
		/// </summary>
		public string? StatusName { get; set; }

		/// <summary>
		/// The issue resolution name, "Fixed", "Closed", etc
		/// </summary>
		public string? ResolutionName { get; set; }

		/// <summary>
		/// The issue priority name, "1 - Critical", "2 - Major", etc
		/// </summary>
		public string? PriorityName { get; set; }

		/// <summary>
		/// The current assignee's user name
		/// </summary>
		public string? AssigneeName { get; set; }

		/// <summary>
		/// The current assignee's display name
		/// </summary>
		public string? AssigneeDisplayName { get; set; }

		/// <summary>
		/// The current assignee's email address
		/// </summary>
		public string? AssigneeEmailAddress { get; set; }
	}
}

