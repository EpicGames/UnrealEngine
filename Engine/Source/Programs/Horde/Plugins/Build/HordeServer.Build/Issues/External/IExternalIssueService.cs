// Copyright Epic Games, Inc. All Rights Reserved.	

using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.Hosting;

namespace HordeServer.Issues.External
{
	/// <summary>
	/// External issue tracking project
	/// </summary>
	public interface IExternalIssueProject
	{
		/// <summary>
		/// Project key
		/// </summary>
		string Key { get; }

		/// <summary>
		/// Project name
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Project id
		/// </summary>
		string Id { get; }

		/// <summary>
		/// component id => name
		/// </summary>
		Dictionary<string, string> Components { get; }

		/// <summary>
		/// // IssueType id => name
		/// </summary>
		Dictionary<string, string> IssueTypes { get; }
	}

	/// <summary>
	/// Interface for an external issue tracking service
	/// </summary>
	public interface IExternalIssueService : IHostedService
	{
		/// <summary>
		/// Gets the URL for an issue from a key
		/// </summary>
		/// <param name="key">Key for the issue</param>
		/// <returns>Url to the issue</returns>
		string? GetIssueUrl(string key);

		/// <summary>
		/// Get issues associated with provided keys
		/// </summary>
		/// <param name="keys"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<List<IExternalIssue>> GetIssuesAsync(string[] keys, CancellationToken cancellationToken);

		/// <summary>
		/// Create and link an external issue
		/// </summary>		
		/// <param name="user"></param>
		/// <param name="externalIssueUser"></param>
		/// <param name="issueId"></param>
		/// <param name="summary"></param>
		/// <param name="projectId"></param>
		/// <param name="componentId"></param>
		/// <param name="issueType"></param>
		/// <param name="description"></param>
		/// <param name="hordeIssueLink"></param>
		/// <param name="cancellationToken"></param>
		Task<(string? key, string? url)> CreateIssueAsync(IUser user, string? externalIssueUser, int issueId, string summary, string projectId, string componentId, string issueType, string? description, string? hordeIssueLink, CancellationToken cancellationToken);

		/// <summary>
		///  Get projects for provided keys
		/// </summary>
		/// <param name="streamConfig"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>;
		Task<List<IExternalIssueProject>> GetProjectsAsync(StreamConfig streamConfig, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Default external issue service
	/// </summary>
	public class DefaultExternalIssueService : IExternalIssueService
	{
		/// <inheritdoc/>
		public string? GetIssueUrl(string key)
		{
			return null;
		}

		/// <inheritdoc/>
		public Task<List<IExternalIssue>> GetIssuesAsync(string[] keys, CancellationToken cancellationToken)
		{
			return Task.FromResult(new List<IExternalIssue>());
		}

		/// <inheritdoc/>
		public Task<(string? key, string? url)> CreateIssueAsync(IUser user, string? externalIssueUser, int issueId, string summary, string projectId, string componentId, string issueType, string? description, string? hordeIssueLink, CancellationToken cancellationToken)
		{
			throw new NotImplementedException();
		}

		/// <inheritdoc/>
		public Task<List<IExternalIssueProject>> GetProjectsAsync(StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			return Task.FromResult(new List<IExternalIssueProject>());
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken token) => Task.CompletedTask;

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken token) => Task.CompletedTask;
	}
}