// Copyright Epic Games, Inc. All Rights Reserved.	

using System.Collections.Concurrent;
using System.Net.Http.Headers;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Polly;
using Polly.Extensions.Http;

namespace HordeServer.Issues.External
{
	/// <summary>
	/// Jira issue information
	/// </summary>
	internal class JiraIssue : IExternalIssue
	{
		/// <summary>
		/// The Jira issue key
		/// </summary>
		public string Key { get; set; } = String.Empty;

		/// <summary>
		/// The Jira issue link
		/// </summary>
		public string? Link { get; set; }

		/// <summary>
		/// The issue status name, "To Do", "In Progress", etc
		/// </summary>
		public string? StatusName { get; set; }

		/// <summary>
		/// The issue resolution name, "Fixed", "Closed", etc
		/// </summary>
		public string? ResolutionName { get; set; }

		/// <summary>
		/// The issue priority name, "1 - Critical", "2 - Major", etc
		/// </summary>
		public string? PriorityName { get; set; }

		/// <summary>
		/// The current assignee's user name
		/// </summary>
		public string? AssigneeName { get; set; }

		/// <summary>
		/// The current assignee's display name
		/// </summary>
		public string? AssigneeDisplayName { get; set; }

		/// <summary>
		/// The current assignee's email address
		/// </summary>
		public string? AssigneeEmailAddress { get; set; }
	}

	internal class JiraCreateResponse
	{
		/// <summary>
		/// Key of external issue
		/// </summary>
		public string Key { get; set; } = String.Empty;

	}

	internal class JiraProject : IExternalIssueProject
	{
		public string Key { get; set; } = String.Empty;

		public string Name { get; set; } = String.Empty;

		public string Id { get; set; } = String.Empty;

		// component id => name
		public Dictionary<string, string> Components { get; set; } = new Dictionary<string, string>();

		// IssueType id => name
		public Dictionary<string, string> IssueTypes { get; set; } = new Dictionary<string, string>();

	}

	/// <summary>
	/// Jira service functionality
	/// </summary>
	internal sealed class JiraService : IExternalIssueService, IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Cached JsonSerializerOptions for case-insensitive deserialization
		/// </summary>
		static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions { PropertyNameCaseInsensitive = true };

		/// <summary>
		/// Normalizes a URL by ensuring it has a trailing slash
		/// </summary>
		static Uri NormalizeUrl(Uri url)
		{
			string urlString = url.ToString();
			return urlString.EndsWith('/') ? url : new Uri(urlString + '/');
		}

		readonly ILogger _logger;

		/// <summary>
		/// The server settings
		/// </summary>
		readonly BuildServerConfig _settings;
		readonly HttpClient _client;
		readonly AsyncPolicy<HttpResponseMessage> _retryPolicy;
		readonly ConcurrentDictionary<string, JiraCacheValue> _issueCache = new ConcurrentDictionary<string, JiraCacheValue>();

		/// <summary>
		/// Singleton instance of the project service
		/// </summary>
		readonly IssueService _issueService;

		readonly IUserCollection _userCollection;

		readonly ITicker _ticker;
		readonly ITicker _updateTicker;

		readonly Uri _jiraUrl;

		/// <summary>
		/// Base URL returned from Jira's serverInfo endpoint, used for all REST calls.
		/// Initialized to configured URL, updated after serverInfo detection.
		/// </summary>
		Uri _jiraBaseUrl;

		// Ticker timing configuration
		readonly TimeSpan _tickInitialDelay = TimeSpan.FromSeconds(15);
		readonly TimeSpan _tickInterval = TimeSpan.FromMinutes(2);
		readonly TimeSpan _updateTickInterval = TimeSpan.FromMinutes(1);

		/// <summary>
		/// Detected Jira deployment type (Cloud vs Server/Data Center)
		/// </summary>
		JiraDeploymentType _deploymentType = JiraDeploymentType.Unknown;

		/// <summary>
		/// Returns true if connected to Jira Cloud, false for Server/Data Center
		/// </summary>
		bool IsCloud => _deploymentType == JiraDeploymentType.Cloud;

		readonly Dictionary<string, JiraProject> _cachedJiraProjects = new Dictionary<string, JiraProject>();

		readonly IOptionsMonitor<BuildConfig> _buildConfig;

		/// <summary>
		/// Jira service constructor
		/// </summary>
		/// <param name="settings"></param>
		/// <param name="issueService"></param>
		/// <param name="userCollection"></param>
		/// <param name="clock"></param>
		/// <param name="buildConfig"></param>
		/// <param name="logger"></param>
		public JiraService(IOptions<BuildServerConfig> settings, IssueService issueService, IUserCollection userCollection, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<JiraService> logger)
		{
			_settings = settings.Value;
			_logger = logger;
			_issueService = issueService;
			_userCollection = userCollection;
			_ticker = clock.AddTicker<JiraService>(_tickInitialDelay, TickAsync, logger);
			_updateTicker = clock.AddSharedTicker("JiraService.Update", _updateTickInterval, TickUpdateAsync, logger);
			_jiraUrl = NormalizeUrl(_settings.JiraUrl!);
			_jiraBaseUrl = _jiraUrl;
			_buildConfig = buildConfig;

			// setup http client for Jira rest api queries
			_client = new HttpClient();
			byte[] authBytes = Encoding.ASCII.GetBytes($"{_settings.JiraUsername}:{_settings.JiraApiToken}");
			_client.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Basic", Convert.ToBase64String(authBytes));
			_client.Timeout = TimeSpan.FromSeconds(20.0);
			_retryPolicy = HttpPolicyExtensions.HandleTransientHttpError().WaitAndRetryAsync(3, attempt => TimeSpan.FromSeconds(Math.Pow(2.0, attempt)));
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await DetectDeploymentTypeAsync(cancellationToken);
			await _ticker.StartAsync();
			await _updateTicker.StartAsync();
		}

		/// <summary>
		/// Detects whether the Jira instance is Cloud or Server/Data Center by querying the serverInfo endpoint
		/// </summary>
		async Task DetectDeploymentTypeAsync(CancellationToken cancellationToken)
		{
			try
			{
				Uri uri = new Uri(_jiraUrl, "rest/api/2/serverInfo");
				HttpResponseMessage response = await _retryPolicy.ExecuteAsync(ctx => _client.GetAsync(uri, ctx), cancellationToken);
				response.EnsureSuccessStatusCode();

				string responseBody = await response.Content.ReadAsStringAsync(cancellationToken);
				ServerInfoResponse? serverInfo = JsonSerializer.Deserialize<ServerInfoResponse>(responseBody, s_jsonOptions);

				bool isCloud = String.Equals(serverInfo?.DeploymentType, "Cloud", StringComparison.OrdinalIgnoreCase);
				_deploymentType = isCloud ? JiraDeploymentType.Cloud : JiraDeploymentType.OnPremise;

				if (!String.IsNullOrEmpty(serverInfo?.BaseUrl))
				{
					_jiraBaseUrl = NormalizeUrl(new Uri(serverInfo.BaseUrl));
				}

				_logger.LogInformation("Detected Jira {DeploymentType} (version: {Version}, baseUrl: {BaseUrl})", serverInfo?.DeploymentType ?? "Unknown", serverInfo?.Version, _jiraBaseUrl);
			}
			catch (Exception ex) when (ex is not OperationCanceledException)
			{
				_logger.LogWarning(ex, "Unable to detect Jira deployment type, defaulting to on-premise");
				_deploymentType = JiraDeploymentType.OnPremise;
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
			await _updateTicker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
			await _updateTicker.DisposeAsync();
			_client.Dispose();
		}

		/// <inheritdoc/>
		public string? GetIssueUrl(string key) => $"{_jiraBaseUrl}browse/{key}";

		public Task<List<IExternalIssueProject>> GetProjectsAsync(StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			HashSet<string> projectKeys = new HashSet<string>();
			List<IExternalIssueProject> result = new List<IExternalIssueProject>();

			if (streamConfig == null || streamConfig.Workflows == null)
			{
				return Task.FromResult(result);
			}

			streamConfig.Workflows.ForEach(workflow =>
			{
				if (workflow.ExternalIssues != null)
				{
					projectKeys.Add(workflow.ExternalIssues.ProjectKey);
				}
			});

			foreach (string key in projectKeys)
			{
				if (_cachedJiraProjects.TryGetValue(key, out JiraProject? project))
				{
					result.Add(project);
				}
			}

			return Task.FromResult(result);
		}

		async Task UpdateJiraProjectsAsync(string[] jiraProjectKeys, CancellationToken cancellationToken)
		{
			if (jiraProjectKeys.Length == 0)
			{
				return;
			}

			for (int i = 0; i < jiraProjectKeys.Length; i++)
			{
				string projectKey = jiraProjectKeys[i];

				Uri uri = new Uri(_jiraBaseUrl, $"rest/api/2/project/{projectKey}");

				HttpResponseMessage response;

				try
				{
					response = await _retryPolicy.ExecuteAsync(ctx => _client.GetAsync(uri, ctx), cancellationToken);
					response.EnsureSuccessStatusCode();
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception)
				{
					_logger.LogWarning("Unable to get project info for {ProjectKey}", projectKey);
					continue;
				}

				string responseBody = await response.Content.ReadAsStringAsync(cancellationToken);

				JsonElement jsonProject = JsonSerializer.Deserialize<JsonElement>(responseBody);

				JiraProject jiraProject = new JiraProject() { Key = projectKey, Id = jsonProject.GetProperty("id")!.GetString()!, Name = jsonProject.GetProperty("name")!.GetString()! };

				// Issue Types
				if (jsonProject.TryGetProperty("issueTypes", out JsonElement issueTypes))
				{
					foreach (JsonElement issueType in issueTypes.EnumerateArray())
					{
						// filter out subtasks issue types
						if (issueType.TryGetProperty("subtask", out JsonElement subtask))
						{
							if (subtask.GetBoolean())
							{
								continue;
							}
						}

						// filter out certain issue types
						string name = issueType.GetProperty("name").GetString()!;
						if (name == "Epic")
						{
							continue;
						}

						jiraProject.IssueTypes[issueType.GetProperty("id").GetString()!] = name;
					}
				}

				// Components
				// We do get components from the projects endpoint, though these don't contain the archived property :/
				try
				{
					response = await _retryPolicy.ExecuteAsync(ctx => _client.GetAsync(new Uri(_jiraBaseUrl, $"rest/api/2/project/{projectKey}/components"), ctx), cancellationToken);
					response.EnsureSuccessStatusCode();
				}
				catch (Exception)
				{
					_logger.LogWarning("Unable to get components info for {ProjectKey}", projectKey);
					continue;
				}

				responseBody = await response.Content.ReadAsStringAsync(cancellationToken);

				JsonElement jiraComponents = JsonSerializer.Deserialize<JsonElement>(responseBody);

				for (int j = 0; j < jiraComponents.GetArrayLength(); j++)
				{
					JsonElement component = jiraComponents[j];

					// skip archived components
					if (component.TryGetProperty("archived", out JsonElement archived) && archived.GetBoolean())
					{
						continue;
					}

					if (component.TryGetStringProperty("id", out string? id))
					{
						if (component.TryGetStringProperty("name", out string? name))
						{
							jiraProject.Components[id] = name;
						}
					}
				}

				_cachedJiraProjects[projectKey] = jiraProject;
			}
		}

		/// <summary>
		/// Periodically syncs Horde issue ownership with Jira assignees by matching email addresses
		/// </summary>
		async ValueTask TickUpdateAsync(CancellationToken stoppingToken)
		{
			if (!stoppingToken.IsCancellationRequested)
			{
				// Refresh issues
				IReadOnlyList<IIssue> openIssues = await _issueService.Collection.FindIssuesAsync(resolved: false, cancellationToken: stoppingToken);
			
				// Auto assign any issues where possible
				List<JiraCacheValue> externalIssues = _issueCache.Values.ToList(); 
				const int RateLimit = 4;
				int assigned = 0;

				foreach (IIssue issue in openIssues)
				{
					if (String.IsNullOrEmpty(issue.ExternalIssueKey))
					{
						continue;
					}

					JiraCacheValue? externalIssue = externalIssues.Where(x => (issue.ExternalIssueKey != null) && (x._issue.Key == issue.ExternalIssueKey)).FirstOrDefault();

					if (externalIssue.HasValue)
					{
						JiraIssue jiraIssue = externalIssue.Value._issue;

						if (jiraIssue != null && !String.IsNullOrEmpty(jiraIssue.AssigneeEmailAddress))
						{

							IUser? user = await _userCollection.FindUserByEmailAsync(jiraIssue.AssigneeEmailAddress, cancellationToken: stoppingToken);

							if (user != null && user.Id != issue.OwnerId)
							{

								await _issueService.UpdateIssueAsync(issue.Id, ownerId: user.Id, cancellationToken: stoppingToken);

								_logger.LogInformation("Assigning issue {IssueId} to user {UserId} by matching {EmailAddress}", issue.Id, user.Id, jiraIssue.AssigneeEmailAddress);

								assigned++;

								if (assigned == RateLimit)
								{
									break;
								}
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Periodically refreshes Jira project metadata and issue status for all open Horde issues with linked Jira keys
		/// </summary>
		async ValueTask<TimeSpan?> TickAsync(CancellationToken cancellationToken)
		{
			HashSet<string> jiraProjectKeys = new HashSet<string>();

			BuildConfig buildConfig = _buildConfig.CurrentValue;

			foreach (StreamConfig streamConfig in buildConfig.Streams)
			{
				foreach (WorkflowConfig workflow in streamConfig.Workflows)
				{
					if (workflow.ExternalIssues == null || String.IsNullOrEmpty(workflow.ExternalIssues.ProjectKey))
					{
						continue;
					}

					jiraProjectKeys.Add(workflow.ExternalIssues.ProjectKey);
				}
			}

			// update projects
			await UpdateJiraProjectsAsync(jiraProjectKeys.ToArray(), cancellationToken);

			HashSet<string> jiraKeys = new HashSet<string>();

			// Refresh issues
			IReadOnlyList<IIssue> openIssues = await _issueService.Collection.FindIssuesAsync(resolved: false, cancellationToken: cancellationToken);
			for (int idx = 0; idx < openIssues.Count; idx++)
			{
				IIssue openIssue = openIssues[idx];
				if (!String.IsNullOrEmpty(openIssue.ExternalIssueKey))
				{
					jiraKeys.Add(openIssue.ExternalIssueKey);
				}
			}
			
			await GetIssuesAsync(jiraKeys.ToArray(), cancellationToken);

			return _tickInterval;
		}

		public async Task<(string? key, string? url)> CreateIssueAsync(IUser user, string? externalIssueUser, int issueId, string summary, string projectId, string componentId, string issueType, string? description, string? hordeIssueLink, CancellationToken cancellationToken)
		{

			IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
			if (issue == null)
			{
				throw new Exception($"Issue not found: {issueId}");
			}

			string issueDesc = description ?? "";

			if (hordeIssueLink != null)
			{
				issueDesc = $"\n{issueDesc}\n\nJira issue created by {user.Name} for *[Horde Issue {issueId}|{hordeIssueLink}]*\n\n";
			}
			else
			{
				issueDesc = $"\n{issueDesc}\n\nJira Issue created by {user.Name} from Horde\n\n";
			}

			Uri uri = new Uri(_jiraBaseUrl, "rest/api/2/issue");

			string bodyJson = JsonSerializer.Serialize(new
			{
				fields = new
				{
					project = new
					{
						id = projectId
					},
					issuetype = new
					{
						id = issueType
					},
					summary,
					description = issueDesc,
					components = new object[]
					{
						new
						{
							id = componentId
						}
					}
				}
			});

			HttpResponseMessage response = await _retryPolicy.ExecuteAsync(() => _client.PostAsync(uri, new StringContent(bodyJson, Encoding.UTF8, "application/json")));

			try
			{
				response.EnsureSuccessStatusCode();
			}
			catch 
			{
				_logger.LogWarning("JiraService create issue: {Uri} returned {Code} Payload: {BodyJson}", uri, response.StatusCode, bodyJson);
				throw;
			}
			

			string responseBody = await response.Content.ReadAsStringAsync(cancellationToken);

			JiraCreateResponse? jiraResponse = JsonSerializer.Deserialize<JiraCreateResponse>(responseBody, s_jsonOptions);

			if (jiraResponse == null || String.IsNullOrEmpty(jiraResponse.Key))
			{
				throw new Exception($"Unable to parse returned jira json: {responseBody}");
			}

			await _issueService.UpdateIssueAsync(issueId, externalIssueKey: jiraResponse.Key, initiatedById: user.Id, cancellationToken: cancellationToken);

			// add the user as a watcher to the newly created issue
			if (externalIssueUser != null)
			{
				uri = new Uri(_jiraBaseUrl, $"rest/api/2/issue/{jiraResponse.Key}/watchers");

				try
				{
					response = await _retryPolicy.ExecuteAsync(ctx => _client.PostAsync(uri, new StringContent(JsonSerializer.Serialize(externalIssueUser), Encoding.UTF8, "application/json"), ctx), cancellationToken);
					response.EnsureSuccessStatusCode();
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception ex)
				{
					_logger.LogWarning(ex, "Unable to add {ExternalUser} as watcher to issue {IssueKey}", externalIssueUser, jiraResponse.Key);
				}
			}

			return (jiraResponse.Key, $"{_jiraBaseUrl}browse/{jiraResponse.Key}");
		}

		/// <inheritdoc/>
		public async Task<List<IExternalIssue>> GetIssuesAsync(string[] jiraKeys, CancellationToken cancellationToken)
		{
			List<IExternalIssue> result = new List<IExternalIssue>();

			if (jiraKeys.Length == 0)
			{
				return result;
			}

			List<string> queryJiras = new List<string>();
			for (int i = 0; i < jiraKeys.Length; i++)
			{
				JiraCacheValue value;
				if (_issueCache.TryGetValue(jiraKeys[i], out value))
				{
					if (DateTime.UtcNow.Subtract(value._cacheTime).TotalMinutes >= 2)
					{
						queryJiras.Add(jiraKeys[i]);
					}
				}
				else
				{
					queryJiras.Add(jiraKeys[i]);
				}
			}

			if (queryJiras.Count > 0)
			{
				HttpResponseMessage response;
				Uri uri;
				if (IsCloud)
				{
					// Jira Cloud uses the newer /rest/api/3/search/jql endpoint
					uri = new Uri(_jiraBaseUrl, "rest/api/3/search/jql");
					object searchRequest = new
					{
						jql = $"issueKey in ({String.Join(",", queryJiras)})",
						fields = new[] { "assignee", "status", "resolution", "priority" },
						maxResults = queryJiras.Count
					};
					using StringContent content = new StringContent(JsonSerializer.Serialize(searchRequest), Encoding.UTF8, "application/json");
					response = await _client.PostAsync(uri, content, cancellationToken);
				}
				else
				{
					// Jira Server/Data Center still supports GET on /rest/api/2/search
					uri = new Uri(_jiraBaseUrl, $"rest/api/2/search?jql=issueKey%20in%20({String.Join(",", queryJiras)})&fields=assignee,status,resolution,priority&maxResults={queryJiras.Count}");
					response = await _client.GetAsync(uri, cancellationToken);
				}

				if (!response.IsSuccessStatusCode)
				{
					_logger.LogWarning("{Method} to {Uri} returned {Code} ({Response})", IsCloud ? "POST" : "GET", uri, response.StatusCode, await response.Content.ReadAsStringAsync(cancellationToken));
					return result;
				}

				byte[] data = await response.Content.ReadAsByteArrayAsync(cancellationToken);
				IssueQueryResponse? jiras = JsonSerializer.Deserialize<IssueQueryResponse>(data.AsSpan(), s_jsonOptions);

				if (jiras != null)
				{
					for (int i = 0; i < jiras.Issues.Count; i++)
					{
						IssueResponse issue = jiras.Issues[i];

						JiraIssue jiraIssue = new JiraIssue();
						jiraIssue.Key = issue.Key;
						jiraIssue.Link = $"{_jiraBaseUrl}browse/{issue.Key}";
						// v2 (Server/DC) uses Name, v3 (Cloud) falls back to DisplayName then EmailAddress
						jiraIssue.AssigneeName = issue.Fields?.Assignee?.Name ?? issue.Fields?.Assignee?.DisplayName ?? issue.Fields?.Assignee?.EmailAddress;
						jiraIssue.AssigneeDisplayName = issue.Fields?.Assignee?.DisplayName;
						jiraIssue.AssigneeEmailAddress = issue.Fields?.Assignee?.EmailAddress;
						jiraIssue.PriorityName = issue.Fields?.Priority?.Name;
						jiraIssue.ResolutionName = issue.Fields?.Resolution?.Name;
						jiraIssue.StatusName = issue.Fields?.Status?.Name;

						_issueCache[issue.Key] = new JiraCacheValue() { _issue = jiraIssue, _cacheTime = DateTime.UtcNow };
					}
				}
			}

			for (int i = 0; i < jiraKeys.Length; i++)
			{
				JiraCacheValue value;
				if (_issueCache.TryGetValue(jiraKeys[i], out value))
				{
					result.Add(value._issue);
				}
			}

			return result;

		}

		// Jira rest api mapping and caching

		struct JiraCacheValue
		{
			public JiraIssue _issue;
			public DateTime _cacheTime;
		};

		class IssueQueryResponse
		{
			public int Total { get; set; } = 0;

			public List<IssueResponse> Issues { get; set; } = new List<IssueResponse>();
		}

		class IssueResponse
		{
			public string Key { get; set; } = String.Empty;

			public IssueFields? Fields { get; set; }
		}

		class IssueFields
		{
			public AssigneeField? Assignee { get; set; }

			public StatusField? Status { get; set; }

			public ResolutionField? Resolution { get; set; }

			public PriorityField? Priority { get; set; }
		}

		class AssigneeField
		{
			/// <summary>
			/// User name (Server/Data Center only, not present in Cloud v3)
			/// </summary>
			public string? Name { get; set; }

			/// <summary>
			/// Account ID (Cloud only, not present in Server/Data Center)
			/// </summary>
			public string? AccountId { get; set; }

			public string? DisplayName { get; set; }

			public string? EmailAddress { get; set; }
		}

		class StatusField
		{
			public string? Name { get; set; }
		}

		class PriorityField
		{
			public string? Name { get; set; }
		}

		class ResolutionField
		{
			public string? Name { get; set; }
		}

		/// <summary>
		/// Jira deployment type
		/// </summary>
		enum JiraDeploymentType
		{
			/// <summary>
			/// Unknown deployment type (not yet detected)
			/// </summary>
			Unknown,

			/// <summary>
			/// Jira Cloud deployment
			/// </summary>
			Cloud,

			/// <summary>
			/// Jira Server or Data Center (on-premise) deployment
			/// </summary>
			OnPremise
		}

		/// <summary>
		/// Response from /rest/api/2/serverInfo endpoint
		/// </summary>
		class ServerInfoResponse
		{
			public string? BaseUrl { get; set; }
			public string? Version { get; set; }
			public string? DeploymentType { get; set; }
			public int[]? VersionNumbers { get; set; }
			public string? ServerTitle { get; set; }
		}
	}
}
