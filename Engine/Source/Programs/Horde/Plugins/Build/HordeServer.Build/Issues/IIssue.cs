// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Issues
{
	/// <summary>
	/// Stores information about a build health issue
	/// </summary>
	public interface IIssue
	{
		/// <summary>
		/// Id for resolved by system
		/// </summary>
		static UserId ResolvedByTimeoutId { get; } = UserId.Parse("609592712b5c90b5bcf88c48");

		/// <summary>
		/// Id for resolved by unknown user
		/// </summary>
		static UserId ResolvedByUnknownId { get; } = UserId.Parse("609593b83e9b0b6dde620cf3");

		/// <summary>
		/// The unique object id
		/// </summary>
		public int Id { get; }

		/// <summary>
		/// Summary for this issue
		/// </summary>
		public string Summary { get; }

		/// <summary>
		/// Summary set by users
		/// </summary>
		public string? UserSummary { get; }

		/// <summary>
		/// Notes set by the user
		/// </summary>
		public string? Description { get; }

		/// <summary>
		/// Fingerprint for this issue
		/// </summary>
		public IReadOnlyList<IIssueFingerprint> Fingerprints { get; }

		/// <summary>
		/// Severity of this issue
		/// </summary>
		public IssueSeverity Severity { get; }

		/// <summary>
		/// Whether this issue is promoted to be publicly visible
		/// </summary>
		public bool Promoted { get; }

		/// <summary>
		/// User id of the owner
		/// </summary>
		public UserId? OwnerId { get; }

		/// <summary>
		/// User id of the person that nominated the owner
		/// </summary>
		public UserId? NominatedById { get; }

		/// <summary>
		/// Time at which the issue was created
		/// </summary>
		public DateTime CreatedAt { get; }

		/// <summary>
		/// Time that the current owner was nominated
		/// </summary>
		public DateTime? NominatedAt { get; }

		/// <summary>
		/// Time at which the issue was acknowledged
		/// </summary>
		public DateTime? AcknowledgedAt { get; }

		/// <summary>
		/// Time at which the issue was resolved
		/// </summary>
		public DateTime? ResolvedAt { get; }

		/// <summary>
		/// User that resolved the issue
		/// </summary>
		public UserId? ResolvedById { get; }

		/// <summary>
		/// Time at which the issue was verified fixed
		/// </summary>
		public DateTime? VerifiedAt { get; }

		/// <summary>
		/// Time at which the issue was last seen.
		/// </summary>
		public DateTime LastSeenAt { get; }

		/// <summary>
		/// Fix commit for this issue
		/// </summary>
		public CommitId? FixCommitId { get; }

		/// <summary>
		/// Root commit for this issue
		/// </summary>
		public CommitId? RootCommitId { get; }

		/// <summary>
		/// User to provide the root cause analysis for this issue
		/// </summary>
		public UserId? RootCauseOwnerId { get; }

		/// <summary>
		/// Root cause category for this issue
		/// </summary>
		public string? RootCauseCategory { get; }

		/// <summary>
		/// Root cause summary for this issue
		/// </summary>
		public string? RootCauseSummary { get; }

		/// <summary>
		/// An issue id that is marked as a duplicate
		/// </summary>
		public int? DuplicateIssueId { get; }

		/// <summary>
		/// Whether the issue is marked fixed as systemic
		/// </summary>
		public bool FixSystemic { get; }

		//		/// <summary>
		//		/// The first stream that encountered the error. The fix will be considered failed if an error after FixChange occurs in this stream. 
		//		/// </summary>
		//		public StreamId? OriginStreamId { get; }

		/// <summary>
		/// List of streams affected by this issue
		/// </summary>
		public IReadOnlyList<IIssueStream> Streams { get; }

		/// <summary>
		/// Spans which should be excluded from this issue
		/// </summary>
		public List<ObjectId>? ExcludeSpans { get; }

		/// <summary>
		/// Update index for this instance
		/// </summary>
		public int UpdateIndex { get; }

		/// <summary>
		/// External issue linked to this issue
		/// </summary>
		public string? ExternalIssueKey { get; }

		/// <summary>
		/// User who quarantined the issue
		/// </summary>
		public UserId? QuarantinedByUserId { get; set; }

		/// <summary>
		/// The UTC time when the issue was quarantined
		/// </summary>
		public DateTime? QuarantineTimeUtc { get; set; }

		/// <summary>
		/// User who force closed the issue
		/// </summary>
		public UserId? ForceClosedByUserId { get; set; }

		/// <summary>
		/// The workflow thread url created for this issue
		/// </summary>
		public Uri? WorkflowThreadUrl { get; set; }
	}

	/// <summary>
	/// Information about a stream affected by an issue
	/// </summary>
	public interface IIssueStream
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// Whether this stream is the merge origin for other streams
		/// </summary>
		bool? MergeOrigin { get; }

		/// <summary>
		/// Whether this stream contains the fix change
		/// </summary>
		bool? ContainsFix { get; }

		/// <summary>
		/// Whether the fix failed in this stream
		/// </summary>
		bool? FixFailed { get; }
	}

	/// <summary>
	/// Suspect for an issue
	/// </summary>
	public interface IIssueSuspect
	{
		/// <summary>
		/// Unique id for this suspect
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// Issue that this suspect belongs to
		/// </summary>
		int IssueId { get; }

		/// <summary>
		/// The user id of the change's author
		/// </summary>
		UserId AuthorId { get; }

		/// <summary>
		/// The commit suspected of causing this issue (in the origin stream)
		/// </summary>
		CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// Time at which the author declined the issue
		/// </summary>
		DateTime? DeclinedAt { get; }
	}

	/// <summary>
	/// Extension methods for issues
	/// </summary>
	static class IssueExtensions
	{
		/// <summary>
		/// Test whether an issue is marked as fixed
		/// </summary>
		public static bool IsMarkedFixed(this IIssue issue)
			=> issue.FixCommitId != null || issue.FixSystemic;

		/// <summary>
		/// Creates a lookup from stream id to whether it's fixed
		/// </summary>
		/// <param name="issue"></param>
		/// <returns></returns>
		public static Dictionary<StreamId, bool> GetFixStreamIds(this IIssue issue)
		{
			Dictionary<StreamId, bool> fixStreamIds = new Dictionary<StreamId, bool>();
			foreach (IIssueStream stream in issue.Streams)
			{
				if (stream.ContainsFix.HasValue)
				{
					fixStreamIds[stream.StreamId] = stream.ContainsFix.Value;
				}
			}
			return fixStreamIds;
		}
	}
}