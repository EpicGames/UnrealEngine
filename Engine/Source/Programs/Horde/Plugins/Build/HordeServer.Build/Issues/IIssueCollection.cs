// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Jobs;
using MongoDB.Bson;
using MongoDB.Driver;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Issues
{

	/// <summary>
	/// Interface for an issue collection
	/// </summary>
	public interface IIssueCollection
	{
		/// <summary>
		/// Attempts to enters a critical section over the issue collection. Used for performing operations that require coordination of several documents (eg. attaching spans to issues).
		/// </summary>
		/// <param name="purpose">Descriptive name for what purpose lock is held. Solely for logging and debugging</param>
		/// <returns>The disposable critical section</returns>
		Task<IAsyncDisposable> EnterCriticalSectionAsync(string purpose);

		#region Issues

		/// <summary>
		/// Creates a new issue
		/// </summary>
		/// <param name="summary">Summary text for the issue</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new issue instance</returns>
		Task<IIssue> AddIssueAsync(string summary, CancellationToken cancellationToken = default);

		/// <summary>
		/// Retrieves and issue by id
		/// </summary>
		/// <param name="issueId">Unique id of the issue</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The issue matching the given id, or null</returns>
		Task<IIssue?> GetIssueAsync(int issueId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds the suspects for an issue
		/// </summary>
		/// <param name="issueId">The issue to retrieve suspects for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of suspects</returns>
		Task<IReadOnlyList<IIssueSuspect>> FindSuspectsAsync(int issueId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for open issues
		/// </summary>
		/// <param name="ids">Set of issue ids to find</param>
		/// <param name="ownerId">The user to find issues for</param>
		/// <param name="streamId">The stream affected by the issue</param>
		/// <param name="minCommitId">Minimum changelist affected by the issue</param>
		/// <param name="maxCommitId">Maximum changelist affected by the issue</param>
		/// <param name="resolved">Include issues that are now resolved</param>
		/// <param name="promoted">Include only promoted issues</param>
		/// <param name="index">Index within the results to return</param>
		/// <param name="count">Number of results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of streams open in the given stream at the given changelist</returns>
		Task<IReadOnlyList<IIssue>> FindIssuesAsync(IEnumerable<int>? ids = null, UserId? ownerId = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, bool? resolved = null, bool? promoted = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for open issues
		/// </summary>
		/// <param name="commits">List of suspect commits</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of issues that are affected by the given changes</returns>
		Task<IReadOnlyList<IIssue>> FindIssuesForChangesAsync(List<CommitIdWithOrder> commits, CancellationToken cancellationToken = default);

		/// <summary>
		/// Try to update the state of an issue
		/// </summary>
		/// <param name="issue">The issue to update</param>
		/// <param name="initiatedByUserId">User that is making the change</param>
		/// <param name="options">Options for updating an issue</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the issue was updated</returns>
		Task<IIssue?> TryUpdateIssueAsync(IIssue issue, UserId? initiatedByUserId, UpdateIssueOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates derived data for an issue (ie. data computed from the spans attached to it). Also clears the issue's 'modified' state.
		/// </summary>
		/// <param name="issue">Issue to update</param>
		/// <param name="newSummary">New summary for the issue</param>
		/// <param name="newSeverity">New severity for the issue</param>
		/// <param name="newFingerprints">New fingerprints for the issue</param>
		/// <param name="newStreams">New streams for the issue</param>
		/// <param name="newSuspects">New suspects for the issue</param>
		/// <param name="newResolvedAt">Time for the last resolved change</param>
		/// <param name="newVerifiedAt">Time that the issue was resolved</param>
		/// <param name="newLastSeenAt">Last time the issue was seen</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated issue, or null if the issue is modified in the interim</returns>
		Task<IIssue?> TryUpdateIssueDerivedDataAsync(IIssue issue, string newSummary, IssueSeverity newSeverity, List<NewIssueFingerprint> newFingerprints, List<NewIssueStream> newStreams, List<NewIssueSuspectData> newSuspects, DateTime? newResolvedAt, DateTime? newVerifiedAt, DateTime newLastSeenAt, CancellationToken cancellationToken = default);

		#endregion

		#region Spans

		/// <summary>
		/// Creates a new span from the given failure
		/// </summary>
		/// <param name="issueId">The issue that the span belongs to</param>
		/// <param name="newSpan">Information about the new span</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New span, or null if the sequence token is not valid</returns>
		Task<IIssueSpan> AddSpanAsync(int issueId, NewIssueSpanData newSpan, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a particular span
		/// </summary>
		/// <param name="spanId">Unique id of the span</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New span, or null if the sequence token is not valid</returns>
		Task<IIssueSpan?> GetSpanAsync(ObjectId spanId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates the given span. Note that data in the span's issue may be derived from this, and the issue should be updated afterwards.
		/// </summary>
		/// <param name="span">Span to update</param>
		/// <param name="newLastSuccess">New last successful step</param>
		/// <param name="newFailure">New failed step</param>
		/// <param name="newNextSuccess">New next successful step</param>
		/// <param name="newSuspects">New suspects for the span</param>
		/// <param name="newMaxSuspectRank">New maximum suspect rank for the span</param>
		/// <param name="newIssueId">The new issue id for this span</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated span, or null on failure</returns>
		Task<IIssueSpan?> TryUpdateSpanAsync(IIssueSpan span, NewIssueStepData? newLastSuccess = null, NewIssueStepData? newFailure = null, NewIssueStepData? newNextSuccess = null, List<NewIssueSpanSuspectData>? newSuspects = null, int? newMaxSuspectRank = null, int? newIssueId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets all the spans for a particular issue
		/// </summary>
		/// <param name="issueId">Issue id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of spans</returns>
		Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(int issueId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Retrieves multiple spans
		/// </summary>
		/// <param name="spanIds">The span ids</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of spans</returns>
		Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(IEnumerable<ObjectId> spanIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds the open issues for a given stream
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="templateId">The template id</param>
		/// <param name="name">Name of the node</param>
		/// <param name="commitId">Changelist number to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of open issues</returns>
		Task<IReadOnlyList<IIssueSpan>> FindOpenSpansAsync(StreamId streamId, TemplateId templateId, string name, CommitId commitId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for open issues
		/// </summary>
		/// <param name="ids">Set of issue ids to find</param>
		/// <param name="issueIds">The issue ids to retrieve spans for</param>
		/// <param name="streamId">The stream affected by the issue</param>
		/// <param name="minCommitId">Minimum commit affected by the issue</param>
		/// <param name="maxCommitId">Maximum commit affected by the issue</param>
		/// <param name="resolved">Include issues that are now resolved</param>
		/// <param name="index">Index within the results to return</param>
		/// <param name="count">Number of results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of streams open in the given stream at the given changelist</returns>
		Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(IEnumerable<ObjectId>? ids = null, IEnumerable<int>? issueIds = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, bool? resolved = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets distinct issue ids from spans matching the given filter, using an aggregate pipeline to avoid transferring full span documents.
		/// </summary>
		/// <param name="streamId">The stream to query</param>
		/// <param name="minCommitId">Minimum commit id filter</param>
		/// <param name="maxCommitId">Maximum commit id filter</param>
		/// <param name="resolved">Include resolved or unresolved spans</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of distinct issue ids</returns>
		Task<IReadOnlyList<int>> FindSpanIssueIdsAsync(StreamId streamId, CommitId? minCommitId = null, CommitId? maxCommitId = null, bool? resolved = null, CancellationToken cancellationToken = default);

		#endregion

		#region Steps

		/// <summary>
		/// Adds a new step
		/// </summary>
		/// <param name="spanId">Initial span for the step</param>
		/// <param name="newStep">Information about the new step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New step object</returns>
		Task<IIssueStep> AddStepAsync(ObjectId spanId, NewIssueStepData newStep, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find steps for the given spans
		/// </summary>
		/// <param name="spanIds">Span ids</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of steps</returns>
		Task<IReadOnlyList<IIssueStep>> FindStepsAsync(IEnumerable<ObjectId> spanIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find steps for the given spans
		/// </summary>
		/// <param name="jobId">The job id</param>
		/// <param name="batchId">The batch id</param>
		/// <param name="stepId">The step id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of steps</returns>
		Task<IReadOnlyList<IIssueStep>> FindStepsAsync(JobId jobId, JobStepBatchId? batchId, JobStepId? stepId, CancellationToken cancellationToken = default);

		#endregion

		/// <summary>
		/// Gets the logger for a particular issue
		/// </summary>
		/// <param name="issueId">The issue id</param>
		/// <returns>Logger for this issue</returns>
		IAuditLogChannel<int> GetLogger(int issueId);
	}

	/// <summary>
	/// Options for updating an issue
	/// </summary>
	/// <param name="Severity">New severity for the issue</param>
	/// <param name="Summary">New summary for the issue</param>
	/// <param name="UserSummary">New user summary for the issue</param>
	/// <param name="Description">New description for the issue</param>
	/// <param name="Promoted">New promoted state of the issue</param>
	/// <param name="OwnerId">New owner of the issue</param>
	/// <param name="NominatedById">Person that nominated the new owner</param>
	/// <param name="Acknowledged">Whether the issue has been acknowledged</param>
	/// <param name="DeclinedById">Name of a user that has declined the issue</param>
	/// <param name="FixCommitId">Fix changelist for the issue. Pass a default value to clear the fix changelist.</param>
	/// <param name="FixSystemic">Whether the issue should be marked fixed as a systemic issue</param>
	/// <param name="RootCommitId">Root changelist for the issue. Pass an empty value to clear the root changelist.</param>
	/// <param name="RootCauseOwnerId">New owner of the root cause</param>
	/// <param name="RootCauseCategory">Root cause category</param>
	/// <param name="RootCauseSummary">Root cause summary.</param>
	/// <param name="DuplicateIssueId">Duplicate issue id</param>
	/// <param name="ResolvedById">User that resolved the issue (may be ObjectId.Empty to clear)</param>
	/// <param name="ExcludeSpanIds">List of span ids to exclude from this issue</param>
	/// <param name="LastSeenAt"></param>
	/// <param name="ExternalIssueKey">Issue key for external issue tracking</param>
	/// <param name="QuarantinedById">The user that quarantined the issue</param>
	/// <param name="ForceClosedById">The user that force closed the issue</param>
	/// <param name="WorkflowThreadUrl">The workflow thread url associated with the issue</param>
	public record class UpdateIssueOptions
	(
		IssueSeverity? Severity = null,
		string? Summary = null,
		string? UserSummary = null,
		string? Description = null,
		bool? Promoted = null,
		UserId? OwnerId = null,
		UserId? NominatedById = null,
		bool? Acknowledged = null,
		UserId? DeclinedById = null,
		CommitId? FixCommitId = null,
		bool? FixSystemic = null,
		CommitId? RootCommitId = null,
		UserId? RootCauseOwnerId = null,
		string? RootCauseCategory = null,
		string? RootCauseSummary = null,
		int? DuplicateIssueId = null,
		UserId? ResolvedById = null,
		List<ObjectId>? ExcludeSpanIds = null,
		DateTime? LastSeenAt = null,
		string? ExternalIssueKey = null,
		UserId? QuarantinedById = null,
		UserId? ForceClosedById = null,
		Uri? WorkflowThreadUrl = null
	);

	/// <summary>
	/// Extension methods for issues
	/// </summary>
	public static class IssueCollectionExtensions
	{
		/// <summary>
		/// Find suspects for the given issue
		/// </summary>
		/// <param name="issueCollection"></param>
		/// <param name="issue"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static Task<IReadOnlyList<IIssueSuspect>> FindSuspectsAsync(this IIssueCollection issueCollection, IIssue issue, CancellationToken cancellationToken = default)
		{
			return issueCollection.FindSuspectsAsync(issue.Id, cancellationToken);
		}

		/// <summary>
		/// Find steps for the given spans
		/// </summary>
		/// <param name="issueCollection"></param>
		/// <param name="spanId">Span ids</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of steps</returns>
		public static Task<IReadOnlyList<IIssueStep>> FindStepsAsync(this IIssueCollection issueCollection, ObjectId spanId, CancellationToken cancellationToken = default)
		{
			return issueCollection.FindStepsAsync(new[] { spanId }, cancellationToken);
		}

		/// <summary>
		/// Searches for open issues affecting a job
		/// </summary>
		/// <param name="issueCollection"></param>
		/// <param name="job"></param>
		/// <param name="graph"></param>
		/// <param name="stepId"></param>
		/// <param name="batchId"></param>
		/// <param name="labelIdx"></param>
		/// <param name="ownerId"></param>
		/// <param name="resolved">Whether to include results that are resolved</param>
		/// <param name="promoted">Whether to filter by promoted issues</param>
		/// <param name="index">Index within the results to return</param>
		/// <param name="count">Number of results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<IReadOnlyList<IIssue>> FindIssuesForJobAsync(this IIssueCollection issueCollection, IJob job, IGraph graph, JobStepId? stepId = null, JobStepBatchId? batchId = null, int? labelIdx = null, UserId? ownerId = null, bool? resolved = null, bool? promoted = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<IIssueStep> steps = await issueCollection.FindStepsAsync(job.Id, batchId, stepId, cancellationToken);
			IReadOnlyList<IIssueSpan> spans = await issueCollection.FindSpansAsync(steps.Select(x => x.SpanId), cancellationToken);

			if (labelIdx != null)
			{
				HashSet<string> nodeNames = new HashSet<string>(job.GetNodesForLabel(graph, labelIdx.Value).Select(x => graph.GetNode(x).Name));
				spans = spans.Where(x => nodeNames.Contains(x.NodeName)).ToList();
			}

			List<int> issueIds = new List<int>(spans.Select(x => x.IssueId));
			if (issueIds.Count == 0)
			{
				return new List<IIssue>();
			}

			return await issueCollection.FindIssuesAsync(ids: issueIds, ownerId: ownerId, resolved: resolved, promoted: promoted, index: index, count: count, cancellationToken: cancellationToken);
		}
	}

	/// <summary>
	/// Fingerprint for an issue
	/// </summary>
	public class NewIssueFingerprint : IIssueFingerprint, IEquatable<IIssueFingerprint>
	{
		/// <inheritdoc/>
		public string Type { get; set; }

		/// <inheritdoc/>
		public string SummaryTemplate { get; set; }

		/// <inheritdoc cref="IIssueFingerprint.Keys"/>
		public HashSet<IssueKey> Keys { get; set; } = new HashSet<IssueKey>();

		/// <inheritdoc/>
		IReadOnlySet<IssueKey> IIssueFingerprint.Keys => Keys;

		/// <inheritdoc cref="IIssueFingerprint.RejectKeys"/>
		public HashSet<IssueKey> RejectKeys { get; set; } = new HashSet<IssueKey>();

		/// <inheritdoc/>
		IReadOnlySet<IssueKey>? IIssueFingerprint.RejectKeys => (RejectKeys.Count > 0) ? RejectKeys : null;

		/// <inheritdoc/>
		public HashSet<IssueMetadata> Metadata { get; set; } = new HashSet<IssueMetadata>();

		/// <inheritdoc/>
		IReadOnlySet<IssueMetadata>? IIssueFingerprint.Metadata => (Metadata.Count > 0) ? Metadata : null;

		/// <inheritdoc/>
		public string ChangeFilter { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The type of issue</param>
		/// <param name="summaryTemplate">Template for the summary string to display for the issue</param>
		/// <param name="changeFilter">Filter for changes covered by this issue</param>
		public NewIssueFingerprint(string type, string summaryTemplate, string changeFilter)
		{
			Type = type;
			SummaryTemplate = summaryTemplate;
			ChangeFilter = changeFilter;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="type">The type of issue</param>
		/// <param name="summaryTemplate">Template for the summary string to display for the issue</param>
		/// <param name="keys">Keys which uniquely identify this issue</param>
		/// <param name="rejectKeys">Keys which should not match with this issue</param>
		/// <param name="metadata">Additional metadata added by the issue handler</param>
		/// <param name="changeFilter">Filter for changes covered by this issue</param>
		public NewIssueFingerprint(string type, string summaryTemplate, IEnumerable<IssueKey> keys, IEnumerable<IssueKey>? rejectKeys, IEnumerable<IssueMetadata>? metadata, string changeFilter)
		{
			Type = type;
			SummaryTemplate = summaryTemplate;
			Keys = new HashSet<IssueKey>(keys);

			if (rejectKeys != null && rejectKeys.Any())
			{
				RejectKeys = new HashSet<IssueKey>(rejectKeys);
			}
			if (metadata != null && metadata.Any())
			{
				Metadata = new HashSet<IssueMetadata>(metadata);
			}

			ChangeFilter = changeFilter;
		}

		/// <summary>
		/// Copy constructor
		/// </summary>
		/// <param name="other">The fingerprint to copy from</param>
		public NewIssueFingerprint(IIssueFingerprint other)
			: this(other.Type, other.SummaryTemplate, other.Keys, other.RejectKeys, other.Metadata, other.ChangeFilter)
		{
		}

		/// <summary>
		/// Merges another fingerprint into this one
		/// </summary>
		/// <param name="other">The other fingerprint to merge with</param>
		public void MergeWith(IIssueFingerprint other)
		{
			Keys.UnionWith(other.Keys);
			if (other.RejectKeys != null)
			{
				RejectKeys ??= new HashSet<IssueKey>();
				RejectKeys.UnionWith(other.RejectKeys);
			}
			if (other.Metadata != null)
			{
				Metadata ??= new HashSet<IssueMetadata>();
				Metadata.UnionWith(other.Metadata);
			}
		}

		/// <inheritdoc/>
		public override bool Equals(object? other) => Equals(other as IIssueFingerprint);

		/// <inheritdoc/>
		public bool Equals(IIssueFingerprint? otherFingerprint)
		{
			if (!ReferenceEquals(this, otherFingerprint))
			{
				if (otherFingerprint == null || !Type.Equals(otherFingerprint.Type, StringComparison.Ordinal) || !Keys.SetEquals(otherFingerprint.Keys))
				{
					return false;
				}
				if (!ContentsEqual(RejectKeys, otherFingerprint.RejectKeys))
				{
					return false;
				}
				if (!ContentsEqual(Metadata, otherFingerprint.Metadata))
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Gets a hashcode for this fingerprint
		/// </summary>
		/// <returns>The hashcode value</returns>
		public override int GetHashCode()
		{
			int result = StringComparer.Ordinal.GetHashCode(Type);
			return HashCode.Combine(result, GetContentsHash(Keys), GetContentsHash(RejectKeys), GetContentsHash(Metadata));
		}

		/// <summary>
		/// Checks if the contents of two sets are equal
		/// </summary>
		/// <param name="setA"></param>
		/// <param name="setB"></param>
		/// <returns></returns>
		static bool ContentsEqual<T>(IReadOnlySet<T>? setA, IReadOnlySet<T>? setB)
		{
			if (setA == null || setA.Count == 0)
			{
				return setB == null || setB.Count == 0;
			}
			else
			{
				return setB != null && setA.SetEquals(setB);
			}
		}

		/// <summary>
		/// Gets the hash of the contents of a case insensitive set
		/// </summary>
		/// <param name="set"></param>
		/// <returns></returns>
		static int GetContentsHash<T>(IEnumerable<T>? set)
		{
			int value = 0;
			if (set != null)
			{
				foreach (T element in set)
				{
					value = HashCode.Combine(value, element);
				}
			}
			return value;
		}

		/// <summary>
		/// Merges two fingerprints togetherthis issue fingerprint with another
		/// </summary>
		/// <param name="a">The first fingerprint</param>
		/// <param name="b">The second fingerprint to merge with</param>
		/// <returns>Merged fingerprint</returns>
		public static NewIssueFingerprint Merge(IIssueFingerprint a, IIssueFingerprint b)
		{
			NewIssueFingerprint newFingerprint = new NewIssueFingerprint(a);
			newFingerprint.MergeWith(b);
			return newFingerprint;
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"{Type}: \"{String.Join("\", \"", Keys)}\"";
		}
	}

	/// <summary>
	/// Information about a new span
	/// </summary>
	public class NewIssueSpanData
	{
		/// <inheritdoc cref="IIssueSpan.StreamId"/>
		public StreamId StreamId { get; set; }

		/// <inheritdoc cref="IIssueSpan.StreamName"/>
		public string StreamName { get; set; }

		/// <inheritdoc cref="IIssueSpan.TemplateRefId"/>
		public TemplateId TemplateRefId { get; set; }

		/// <inheritdoc cref="IIssueSpan.NodeName"/>
		public string NodeName { get; set; }

		/// <inheritdoc cref="IIssueSpan.Fingerprint"/>
		public NewIssueFingerprint Fingerprint { get; set; }

		/// <inheritdoc cref="IIssueSpan.LastSuccess"/>
		public NewIssueStepData? LastSuccess { get; set; }

		/// <inheritdoc cref="IIssueSpan.FirstFailure"/>
		public NewIssueStepData FirstFailure { get; set; }

		/// <inheritdoc cref="IIssueSpan.NextSuccess"/>
		public NewIssueStepData? NextSuccess { get; set; }

		/// <inheritdoc cref="IIssueSpan.Suspects"/>
		public List<NewIssueSpanSuspectData> Suspects { get; set; } = new List<NewIssueSpanSuspectData>();

		/// <inheritdoc cref="IIssueSpan.MaxSuspectRank"/>
		public int? MaxSuspectRank { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public NewIssueSpanData(StreamId streamId, string streamName, TemplateId templateRefId, string nodeName, NewIssueFingerprint fingerprint, NewIssueStepData firstFailure)
		{
			StreamId = streamId;
			StreamName = streamName;
			TemplateRefId = templateRefId;
			NodeName = nodeName;
			Fingerprint = fingerprint;
			FirstFailure = firstFailure;
		}
	}

	/// <summary>
	/// Identifies a particular changelist and job
	/// </summary>
	public class NewIssueStepData
	{
		/// <inheritdoc cref="IIssueStep.CommitId"/>
		public CommitIdWithOrder CommitId { get; set; }

		/// <inheritdoc cref="IIssueStep.Severity"/>
		public IssueSeverity Severity { get; set; }

		/// <inheritdoc cref="IIssueStep.JobName"/>
		public string JobName { get; set; }

		/// <inheritdoc cref="IIssueStep.JobId"/>
		public JobId JobId { get; set; }

		/// <inheritdoc cref="IIssueStep.BatchId"/>
		public JobStepBatchId BatchId { get; set; }

		/// <inheritdoc cref="IIssueStep.StepId"/>
		public JobStepId StepId { get; set; }

		/// <inheritdoc cref="IIssueStep.StepTime"/>
		public DateTime StepTime { get; set; }

		/// <inheritdoc cref="IIssueStep.LogId"/>
		public LogId? LogId { get; set; }

		/// <inheritdoc cref="IIssueStep.Annotations"/>
		public NodeAnnotations? Annotations { get; set; }

		/// <inheritdoc cref="IIssueStep.PromoteByDefault"/>
		public bool PromoteByDefault { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="commitId">The commit for this job</param>
		/// <param name="severity">Severity of the issue in this step</param>
		/// <param name="jobName">The job name</param>
		/// <param name="jobId">The unique job id</param>
		/// <param name="batchId">The batch id</param>
		/// <param name="stepId">The step id</param>
		/// <param name="stepTime">Time that the step started</param>
		/// <param name="logId">Unique id of the log file for this step</param>
		/// <param name="annotations">Annotations for this step</param>
		/// <param name="promoted">Whether this step is promoted</param>
		public NewIssueStepData(CommitIdWithOrder commitId, IssueSeverity severity, string jobName, JobId jobId, JobStepBatchId batchId, JobStepId stepId, DateTime stepTime, LogId? logId, IReadOnlyNodeAnnotations? annotations, bool promoted)
		{
			CommitId = commitId;
			Severity = severity;
			JobName = jobName;
			JobId = jobId;
			BatchId = batchId;
			StepId = stepId;
			StepTime = stepTime;
			LogId = logId;
			if (annotations != null)
			{
				Annotations = new NodeAnnotations(annotations);
			}
			PromoteByDefault = promoted;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="job">The job being built</param>
		/// <param name="batch">Batch of the job for the step</param>
		/// <param name="step">The step being built</param>
		/// <param name="severity">Severity of the issue in this step</param>
		/// <param name="annotations">Annotations for this step</param>
		/// <param name="promoted">Whether this step is promoted</param>
		public NewIssueStepData(IJob job, IJobStepBatch batch, IJobStep step, IssueSeverity severity, IReadOnlyNodeAnnotations? annotations, bool promoted)
			: this(job.CommitId, severity, job.Name, job.Id, batch.Id, step.Id, step.StartTimeUtc ?? default, step.LogId, annotations, promoted)
		{
		}

		/// <summary>
		/// Constructor for sentinel steps
		/// </summary>
		/// <param name="jobStepRef">The jobstep reference</param>
		public NewIssueStepData(IJobStepRef jobStepRef)
			: this(jobStepRef.CommitId, IssueSeverity.Unspecified, jobStepRef.JobName, jobStepRef.Id.JobId, jobStepRef.Id.BatchId, jobStepRef.Id.StepId, jobStepRef.StartTimeUtc, jobStepRef.LogId, null, false)
		{
		}
	}

	/// <summary>
	/// Information about a suspect changelist that may have caused an issue
	/// </summary>
	public class NewIssueSuspectData
	{
		/// <inheritdoc cref="IIssueSuspect.AuthorId"/>
		public UserId AuthorId { get; set; }

		/// <inheritdoc cref="IIssueSuspect.CommitId"/>
		public CommitIdWithOrder CommitId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="authorId">Author of the change</param>
		/// <param name="commitId">The changelist number</param>
		public NewIssueSuspectData(UserId authorId, CommitIdWithOrder commitId)
		{
			AuthorId = authorId;
			CommitId = commitId;
		}
	}

	/// <summary>
	/// Information about a suspect changelist that may have caused an issue
	/// </summary>
	public class NewIssueSpanSuspectData
	{
		/// <inheritdoc cref="IIssueSpanSuspect.CommitId"/>
		public CommitIdWithOrder CommitId { get; set; }

		/// <inheritdoc cref="IIssueSpanSuspect.AuthorId"/>
		public UserId AuthorId { get; set; }

		/// <inheritdoc cref="IIssueSpanSuspect.SourceCommitId"/>
		public CommitIdWithOrder? SourceCommitId { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="change">The changelist number</param>
		/// <param name="authorId">Author of the change</param>
		public NewIssueSpanSuspectData(CommitIdWithOrder change, UserId authorId)
		{
			CommitId = change;
			AuthorId = authorId;
		}
	}

	/// <summary>
	/// Information about a stream containing an issue
	/// </summary>
	public class NewIssueStream : IIssueStream
	{
		/// <inheritdoc cref="IIssueStream.StreamId"/>
		public StreamId StreamId { get; set; }

		/// <inheritdoc cref="IIssueStream.MergeOrigin"/>
		public bool? MergeOrigin { get; set; }

		/// <inheritdoc cref="IIssueStream.ContainsFix"/>
		public bool? ContainsFix { get; set; }

		/// <inheritdoc cref="IIssueStream.FixFailed"/>
		public bool? FixFailed { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="streamId"></param>
		public NewIssueStream(StreamId streamId)
		{
			StreamId = streamId;
		}
	}
}
