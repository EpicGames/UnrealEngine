// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using HordeServer.Users;

namespace HordeServer.Issues
{
	/// <summary>
	/// Cached issue information
	/// </summary>
	public interface IIssueDetails
	{
		/// <summary>
		/// The issue instance
		/// </summary>
		IIssue Issue { get; }

		/// <summary>
		/// The user that currently owns this issue
		/// </summary>
		IUser? Owner { get; }

		/// <summary>
		/// The user that nominated the current owner
		/// </summary>
		IUser? NominatedBy { get; }

		/// <summary>
		/// The user that resolved the issue
		/// </summary>
		IUser? ResolvedBy { get; }

		/// <summary>
		/// The user that provided the root cause analysis
		/// </summary>
		IUser? RootCauseOwner { get; }

		/// <summary>
		/// List of spans for the issue
		/// </summary>
		IReadOnlyList<IIssueSpan> Spans { get; }

		/// <summary>
		/// List of steps for the issue
		/// </summary>
		IReadOnlyList<IIssueStep> Steps { get; }

		/// <summary>
		/// List of suspects for the issue
		/// </summary>
		IReadOnlyList<IIssueSuspect> Suspects { get; }

		/// <summary>
		/// List of users that are suspects for this issue
		/// </summary>
		IReadOnlyList<IUser> SuspectUsers { get; }

		/// <summary>
		/// Issue key in external issue tracker
		/// </summary>
		string? ExternalIssueKey { get; }

		/// <summary>
		/// User who quarantined the issue
		/// </summary>
		IUser? QuarantinedBy { get; }

		/// <summary>
		/// The UTC time when the issue was quarantined
		/// </summary>
		DateTime? QuarantineTimeUtc { get; }

		/// <summary>
		/// User who force closed the issue
		/// </summary>
		IUser? ForceClosedBy { get; }

		/// <summary>
		/// Determines whether the given user should be notified about the given issue
		/// </summary>
		/// <returns>True if the user should be notified for this change</returns>
		bool ShowNotifications();

		/// <summary>
		/// Determines if the issue is relevant to the given user
		/// </summary>
		/// <param name="userId">The user to query</param>
		/// <returns>True if the issue is relevant to the given user</returns>
		bool IncludeForUser(UserId userId);
	}

	/// <summary>
	/// Extension methods for IIssueService implementations
	/// </summary>
	public static class IssueDetailsExtensions
	{
		/// <summary>
		/// Gets an issue details object for a specific issue id
		/// </summary>
		/// <param name="issueService">The issue service</param>
		/// <param name="issueId">Issue id to query </param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<IIssueDetails?> GetIssueDetailsAsync(this IssueService issueService, int issueId, CancellationToken cancellationToken)
		{
			IIssue? issue = await issueService.Collection.GetIssueAsync(issueId, cancellationToken);
			if (issue == null)
			{
				return null;
			}
			return await issueService.GetIssueDetailsAsync(issue, cancellationToken);
		}
	}
}
