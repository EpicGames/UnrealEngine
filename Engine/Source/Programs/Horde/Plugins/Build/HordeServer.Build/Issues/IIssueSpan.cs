// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Issues
{
	/// <summary>
	/// Trace of a set of errors in a single step across multiple jobs
	/// </summary>
	public interface IIssueSpan
	{
		/// <summary>
		/// Unique id for the span
		/// </summary>
		public ObjectId Id { get; }

		/// <summary>
		/// The unique stream id
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// The stream name
		/// </summary>
		public string StreamName { get; }

		/// <summary>
		/// The template containing this node
		/// </summary>
		public TemplateId TemplateRefId { get; }

		/// <summary>
		/// Name of the node
		/// </summary>
		public string NodeName { get; }

		/// <summary>
		/// Fingerprint for this issue
		/// </summary>
		public IIssueFingerprint Fingerprint { get; }

		/// <summary>
		/// The previous build 
		/// </summary>
		public IIssueStep? LastSuccess { get; }

		/// <summary>
		/// Maximum changelist number that failed
		/// </summary>
		public IIssueStep FirstFailure { get; }

		/// <summary>
		/// Minimum changelist number that failed
		/// </summary>
		public IIssueStep LastFailure { get; }

		/// <summary>
		/// The following successful build
		/// </summary>
		public IIssueStep? NextSuccess { get; }

		/// <summary>
		/// Whether to promote issues containing this span by default
		/// </summary>
		public bool PromoteByDefault { get; }

		/// <summary>
		/// Suspects for this span
		/// </summary>
		public IReadOnlyList<IIssueSpanSuspect> Suspects { get; }

		/// <summary>
		/// The maximum rank achieved during suspect scoring for this span. A value of 0 means all suspects
		/// are tied with no meaningful differentiation (no file/symbol/note matches). Null if not yet computed
		/// or for spans created before this property was added.
		/// </summary>
		public int? MaxSuspectRank { get; }

		/// <summary>
		/// The issue that this span is assigned to
		/// </summary>
		public int IssueId { get; }
	}

	/// <summary>
	/// Information about a suspect changelist that may have caused an issue
	/// </summary>
	public interface IIssueSpanSuspect
	{
		/// <summary>
		/// The submitted changelist
		/// </summary>
		public CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// Id of the changelist's author
		/// </summary>
		public UserId AuthorId { get; }

		/// <summary>
		/// The original changelist number, if merged from another branch. For changes merged between several branches, this is the originally submitted change.
		/// </summary>
		public CommitIdWithOrder? SourceCommitId { get; }
	}
}
