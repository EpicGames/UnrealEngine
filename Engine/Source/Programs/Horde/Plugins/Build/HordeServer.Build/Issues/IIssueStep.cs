// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Logs;
using MongoDB.Bson;

namespace HordeServer.Issues
{
	/// <summary>
	/// Identifies a particular commit and job that contributes to a span
	/// </summary>
	public interface IIssueStep
	{
		/// <summary>
		/// The span that this step belongs to
		/// </summary>
		public ObjectId SpanId { get; }

		/// <summary>
		/// The commit
		/// </summary>
		public CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// Severity of the issue in this step
		/// </summary>
		public IssueSeverity Severity { get; }

		/// <summary>
		/// Name of the job
		/// </summary>
		public string JobName { get; }

		/// <summary>
		/// The unique job id
		/// </summary>
		public JobId JobId { get; }

		/// <summary>
		/// Unique id of the batch within the job
		/// </summary>
		public JobStepBatchId BatchId { get; }

		/// <summary>
		/// Unique id of the step within the job
		/// </summary>
		public JobStepId StepId { get; }

		/// <summary>
		/// Time that the step started
		/// </summary>
		public DateTime StepTime { get; }

		/// <summary>
		/// Annotations for this step
		/// </summary>
		public IReadOnlyNodeAnnotations Annotations { get; }

		/// <summary>
		/// Whether to promote spans including this step by default
		/// </summary>
		public bool PromoteByDefault { get; }

		/// <summary>
		/// The log id for this step
		/// </summary>
		public LogId? LogId { get; }
	}
}
