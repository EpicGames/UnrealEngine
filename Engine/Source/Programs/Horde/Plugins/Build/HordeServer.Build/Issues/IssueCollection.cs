// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using EpicGames.Horde.Users;
using EpicGames.Redis.Utility;
using HordeServer.Auditing;
using HordeServer.Commits;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Issues
{
	class IssueCollection : IIssueCollection, ILogExtIssueProvider
	{
		[SingletonDocument("issue-ledger", "5e4c226440ce25fa3207a9af")]
		class IssueLedger : SingletonBase
		{
			public int NextId { get; set; }
		}

		[DebuggerDisplay("{Id}: {Summary}")]
		class IssueDocument : IIssue
		{
			[BsonId]
			public int Id { get; set; }

			public string Summary { get; set; }

			[BsonIgnoreIfNull]
			public string? UserSummary { get; set; }

			[BsonIgnoreIfNull]
			public string? Description { get; set; }

			[BsonIgnoreIfNull]
			public IssueFingerprintDocument? Fingerprint { get; set; }
			public List<IssueFingerprintDocument>? Fingerprints { get; set; }

			public IssueSeverity Severity { get; set; }

			[BsonElement("PromotedV2")]
			public bool Promoted { get; set; }

			[BsonIgnoreIfNull]
			public bool? ManuallyPromoted { get; set; }

			[BsonIgnoreIfNull, BsonElement("Promoted")]
			public bool? ManuallyPromotedDeprecated { get; set; }

			[BsonIgnoreIfNull]
			public UserId? OwnerId { get; set; }

			[BsonIgnoreIfNull]
			public UserId? DefaultOwnerId { get; set; }

			[BsonIgnoreIfNull]
			public UserId? NominatedById { get; set; }

			public DateTime CreatedAt { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? NominatedAt { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? AcknowledgedAt { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? ResolvedAt { get; set; }

			[BsonIgnoreIfNull]
			public UserId? ResolvedById { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? VerifiedAt { get; set; }

			public DateTime LastSeenAt { get; set; }

			[BsonIgnore]
			public CommitId? FixCommitId
			{
				get => (FixCommitName != null) ? new CommitId(FixCommitName) : (FixChange != null) ? CommitId.FromPerforceChange(FixChange.Value) : null;
				set => (FixCommitName, FixChange) = (value?.Name, null);
			}

			[BsonElement("FixCommit"), BsonIgnoreIfNull]
			public string? FixCommitName { get; set; }

			[BsonElement("FixChange"), BsonIgnoreIfNull]
			public int? FixChange { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool FixSystemic { get; set; }

			[BsonIgnore]
			public CommitId? RootCommitId 
			{
				get => (RootCommitName != null) ? new CommitId(RootCommitName) : null;
				set => RootCommitName = value?.Name;
			}

			[BsonElement("RootCommit"), BsonIgnoreIfNull]
			public string? RootCommitName { get; set; }

			[BsonIgnoreIfNull]
			public UserId? RootCauseOwnerId { get; set; }

			[BsonIgnoreIfNull]
			public string? RootCauseCategory { get; set; }

			[BsonIgnoreIfNull]
			public string? RootCauseSummary { get; set; }

			[BsonIgnoreIfNull]
			public int? DuplicateIssueId { get; set; }

			public List<IssueStreamDocument> Streams { get; set; } = new List<IssueStreamDocument>();

			[BsonElement("Suspects"), BsonIgnoreIfNull]
			public List<IssueSuspectDocument>? SuspectsDeprecated { get; set; }

			[BsonIgnoreIfNull]
			public List<ObjectId>? ExcludeSpans { get; set; }

			public int UpdateIndex { get; set; }

			IReadOnlyList<IIssueFingerprint> IIssue.Fingerprints => Fingerprints ?? ((Fingerprint == null) ? new List<IssueFingerprintDocument>() : new List<IssueFingerprintDocument> { Fingerprint });
			UserId? IIssue.OwnerId => OwnerId ?? DefaultOwnerId ?? GetDefaultOwnerId();
			IReadOnlyList<IIssueStream> IIssue.Streams => Streams;
			DateTime IIssue.LastSeenAt => (LastSeenAt == default) ? DateTime.UtcNow : LastSeenAt;

			[BsonIgnoreIfNull]
			public string? ExternalIssueKey { get; set; }

			[BsonIgnoreIfNull]
			public UserId? QuarantinedByUserId { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? QuarantineTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public UserId? ForceClosedByUserId { get; set; }

			[BsonIgnoreIfNull]
			public Uri? WorkflowThreadUrl { get; set; }

			[BsonConstructor]
			private IssueDocument()
			{
				Summary = String.Empty;
				Fingerprint = null!;
			}

			public IssueDocument(int id, string summary)
			{
				Id = id;
				Summary = summary;
				CreatedAt = DateTime.UtcNow;
				LastSeenAt = DateTime.UtcNow;
			}

			UserId? GetDefaultOwnerId()
			{
				if (SuspectsDeprecated != null && SuspectsDeprecated.Count > 0)
				{
					UserId possibleOwner = SuspectsDeprecated[0].AuthorId;
					if (SuspectsDeprecated.All(x => x.AuthorId == possibleOwner) && SuspectsDeprecated.Any(x => x.DeclinedAt == null))
					{
						return SuspectsDeprecated[0].AuthorId;
					}
				}
				return null;
			}

			public string FingerprintsDesc
			{
				get
				{
					if (Fingerprints == null || Fingerprints.Count == 0)
					{
						return String.Empty;
					}

					return String.Join(", ", Fingerprints.Select(x =>
					{
						return $"(Type: {x.Type} / Keys: {String.Join(", ", x.Keys)} / RejectKeys: {String.Join(", ", x.RejectKeys ?? new HashSet<IssueKey>())})";
					}));
				}
			}
		}

		class IssueStreamDocument : IIssueStream
		{
			public StreamId StreamId { get; set; }
			public bool? MergeOrigin { get; set; }
			public bool? ContainsFix { get; set; }
			public bool? FixFailed { get; set; }

			public IssueStreamDocument()
			{
			}

			public IssueStreamDocument(IIssueStream other)
			{
				StreamId = other.StreamId;
				MergeOrigin = other.MergeOrigin;
				ContainsFix = other.ContainsFix;
				FixFailed = other.FixFailed;
			}
		}

		class IssueSuspectDocument : IIssueSuspect
		{
			public ObjectId Id { get; set; }
			public int IssueId { get; set; }
			public UserId AuthorId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("Commit")]
			public string? CommitName { get; set; }

			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			public DateTime? DeclinedAt { get; set; }
			public DateTime? ResolvedAt { get; set; } // Degenerate

			private IssueSuspectDocument()
			{
			}

			public IssueSuspectDocument(int issueId, NewIssueSuspectData newSuspect, DateTime? resolvedAt)
			{
				Id = ObjectId.GenerateNewId();
				IssueId = issueId;
				AuthorId = newSuspect.AuthorId;
				CommitId = newSuspect.CommitId;
				ResolvedAt = resolvedAt;
			}

			public IssueSuspectDocument(int issueId, IIssueSpanSuspect suspect)
				: this(issueId, suspect.AuthorId, suspect.SourceCommitId ?? suspect.CommitId, null, null)
			{
			}

			public IssueSuspectDocument(int issueId, UserId authorId, CommitIdWithOrder commit, DateTime? declinedAt, DateTime? resolvedAt)
			{
				Id = ObjectId.GenerateNewId();
				IssueId = issueId;
				AuthorId = authorId;
				CommitId = commit;
				DeclinedAt = declinedAt;
				ResolvedAt = resolvedAt;
			}
		}

		class IssueFingerprintDocument : IIssueFingerprint
		{
			public string Type { get; set; }

			[BsonElement("summary"), BsonIgnoreIfNull]
			string? SummaryTemplateValue { get; set; }

			[BsonIgnore]
			public string SummaryTemplate => SummaryTemplateValue ?? GetLegacyHandlerInfo(Type).SummaryTemplate;

			[BsonElement("inc")]
			public HashSet<IssueKey> Keys { get; set; } = new HashSet<IssueKey>();

			IReadOnlySet<IssueKey> IIssueFingerprint.Keys => Keys;

			[BsonElement("exc"), BsonIgnoreIfNull]
			public HashSet<IssueKey>? RejectKeys { get; set; } = new HashSet<IssueKey>();

			IReadOnlySet<IssueKey>? IIssueFingerprint.RejectKeys => RejectKeys;

			[BsonElement("met"), BsonIgnoreIfNull]
			public HashSet<IssueMetadata>? Metadata { get; set; } = new HashSet<IssueMetadata>();

			IReadOnlySet<IssueMetadata>? IIssueFingerprint.Metadata => Metadata;

			[BsonElement("fl2")]
			public string? ChangeFilter { get; set; }

			[Obsolete("Use ChangeFilter instead")]
			[BsonElement("flt"), BsonIgnoreIfNull]
			public List<string>? LegacyChangeFilter
			{
				get => null;
				set => ChangeFilter = (value == null) ? null : String.Join(";", value);
			}

			string IIssueFingerprint.ChangeFilter => ChangeFilter ?? String.Join(";", GetLegacyHandlerInfo(Type).ChangeFilter);

			[BsonConstructor]
			private IssueFingerprintDocument()
			{
				Type = String.Empty;
			}

			public IssueFingerprintDocument(IIssueFingerprint fingerprint)
			{
				Type = fingerprint.Type;
				SummaryTemplateValue = fingerprint.SummaryTemplate;

				Keys = new HashSet<IssueKey>(fingerprint.Keys);
				if (fingerprint.RejectKeys != null && fingerprint.RejectKeys.Count > 0)
				{
					RejectKeys = new HashSet<IssueKey>(fingerprint.RejectKeys);
				}
				if (fingerprint.Metadata != null && fingerprint.Metadata.Count > 0)
				{
					Metadata = new HashSet<IssueMetadata>(fingerprint.Metadata);
				}

				ChangeFilter = fingerprint.ChangeFilter;
			}

			#region Legacy

#pragma warning disable IDE0051
			[BsonElement("Keys"), BsonIgnoreIfNull]
			CaseInsensitiveStringSet? LegacyKeys
			{
				get => null;
				set => Keys = ParseKeySet(value) ?? new HashSet<IssueKey>();
			}

			[BsonElement("RejectKeys"), BsonIgnoreIfNull]
			CaseInsensitiveStringSet? LegacyRejectKeys
			{
				get => null;
				set => RejectKeys = ParseKeySet(value);
			}

			[BsonElement("Metadata"), BsonIgnoreIfNull]
			CaseInsensitiveStringSet? LegacyMetadata
			{
				get => null;
				set => Metadata = ParseMetadata(value);
			}
#pragma warning restore IDE0051

			[return: NotNullIfNotNull("set")]
			static HashSet<IssueKey>? ParseKeySet(CaseInsensitiveStringSet? set) => (set == null) ? null : new HashSet<IssueKey>(set.Select(x => ParseKey(x)));

			static IssueKey ParseKey(string key)
			{
				int colonIdx = key.IndexOf(':', StringComparison.Ordinal);
				if (colonIdx != -1)
				{
					ReadOnlySpan<char> prefix = key.AsSpan(0, colonIdx);
					if (prefix.Equals("hash", StringComparison.Ordinal))
					{
						return new IssueKey(key.Substring(colonIdx + 1), IssueKeyType.Hash);
					}
					if (prefix.Equals("note", StringComparison.Ordinal))
					{
						return new IssueKey(key.Substring(colonIdx + 1), IssueKeyType.Note);
					}
					if (prefix.Equals("step", StringComparison.Ordinal))
					{
						return new IssueKey(key.Substring(colonIdx + 1), IssueKeyType.None);
					}
				}
				return new IssueKey(key, IssueKeyType.None);
			}

			[return: NotNullIfNotNull("set")]
			static HashSet<IssueMetadata>? ParseMetadata(CaseInsensitiveStringSet? set)
			{
				HashSet<IssueMetadata>? entries = null;
				if (set != null)
				{
					entries = new HashSet<IssueMetadata>();
					foreach (string entry in set)
					{
						int idx = entry.IndexOf('=', StringComparison.Ordinal);
						entries.Add(new IssueMetadata(entry.Substring(0, idx), entry.Substring(idx + 1)));
					}
				}
				return entries;
			}

			record class LegacyHandlerInfo(string SummaryTemplate, IReadOnlyList<string> ChangeFilter);

			static readonly Dictionary<StringView, LegacyHandlerInfo> s_legacyHandlers = new Dictionary<StringView, LegacyHandlerInfo>
			{
				["BuildGraph"] = new("BuildGraph {Severity} in {Files}", IssueChangeFilter.All),
				["Compile"] = new("{Meta:CompileType} {Severity} in {Files}", IssueChangeFilter.Code),
				["Content"] = new("{Severity} in {Files}", IssueChangeFilter.Content),
				["Copyright"] = new("Missing copyright notice in {Files}", IssueChangeFilter.Code),
				["Default"] = new("{Severity} in {Nodes}", IssueChangeFilter.All),
				["Gauntlet"] = new("Gauntlet {Meta:Type} {Severity} {Meta:Context}", IssueChangeFilter.Code),
				["Hashed"] = new("{Severity} in {Meta:Node}", IssueChangeFilter.All),
				["Localization"] = new("Localization {Severity} in {Files}", IssueChangeFilter.Code),
				["PerforceCase"] = new("Inconsistent case for {Files}", IssueChangeFilter.All),
				["Scoped"] = new("{Severity} in {Meta:Node} - {Meta:Scope}", IssueChangeFilter.All),
				["Shader"] = new("Shader compile {Severity} in {Files}", IssueChangeFilter.Code),
				["Symbol"] = new("{LegacySymbolIssueHandler}", IssueChangeFilter.Code),
				["Systemic"] = new("Systemic {Severity} in {Nodes}", IssueChangeFilter.None),
				["UnacceptableWords"] = new("Unacceptable words in {Files}", IssueChangeFilter.Code)
			};

			static LegacyHandlerInfo GetLegacyHandlerInfo(string type)
			{
				StringView baseType = type;

				int endIdx = type.IndexOf(':', StringComparison.Ordinal);
				if (endIdx != -1)
				{
					baseType = new StringView(type, 0, endIdx);
				}

				s_legacyHandlers.TryGetValue(baseType, out LegacyHandlerInfo? info);
				return info ?? new LegacyHandlerInfo($"{type} {{Severity}}", IssueChangeFilter.All);
			}

			#endregion
		}

		class IssueSpanDocument : IIssueSpan
		{
			public ObjectId Id { get; set; }

			[BsonRequired]
			public StreamId StreamId { get; set; }

			[BsonRequired]
			public string StreamName { get; set; }

			[BsonRequired]
			public TemplateId TemplateRefId { get; set; }

			[BsonRequired]
			public string NodeName { get; set; }
			public DateTime? ResolvedAt { get; set; } // Propagated from the owning issue

			[BsonRequired]
			public IssueFingerprintDocument Fingerprint { get; set; }

			[BsonElement("MinChange")]
			public int MinCommitOrder { get; set; }

			[BsonElement("MaxChange")]
			public int MaxCommitOrder { get; set; } = Int32.MaxValue;

			public IssueStepDocument? LastSuccess { get; set; }

			[BsonRequired]
			public IssueStepDocument FirstFailure { get; set; }

			[BsonRequired]
			public IssueStepDocument LastFailure { get; set; }

			public IssueStepDocument? NextSuccess { get; set; }

			public bool? PromoteByDefault { get; set; }

			[BsonElement("NotifySuspects"), BsonIgnoreIfDefault(false)]
			public bool NotifySuspectsDeprecated { get; set; }

			bool IIssueSpan.PromoteByDefault => PromoteByDefault ?? NotifySuspectsDeprecated;

			public List<IssueSpanSuspectDocument> Suspects { get; set; }

			[BsonIgnoreIfNull]
			public int? MaxSuspectRank { get; set; }

			public int IssueId { get; set; }
			public int UpdateIndex { get; set; }

			IIssueStep? IIssueSpan.LastSuccess => LastSuccess;
			IIssueStep IIssueSpan.FirstFailure => FirstFailure;
			IIssueStep IIssueSpan.LastFailure => LastFailure;
			IIssueStep? IIssueSpan.NextSuccess => NextSuccess;
			IReadOnlyList<IIssueSpanSuspect> IIssueSpan.Suspects => Suspects;
			int? IIssueSpan.MaxSuspectRank => MaxSuspectRank;
			IIssueFingerprint IIssueSpan.Fingerprint => Fingerprint;

			private IssueSpanDocument()
			{
				StreamName = null!;
				NodeName = null!;
				Fingerprint = null!;
				FirstFailure = null!;
				LastFailure = null!;
				Suspects = new List<IssueSpanSuspectDocument>();
			}

			public IssueSpanDocument(int issueId, NewIssueSpanData newSpan)
			{
				Id = ObjectId.GenerateNewId();
				StreamId = newSpan.StreamId;
				StreamName = newSpan.StreamName;
				TemplateRefId = newSpan.TemplateRefId;
				NodeName = newSpan.NodeName;
				Fingerprint = new IssueFingerprintDocument(newSpan.Fingerprint);
				if (newSpan.LastSuccess != null)
				{
					MinCommitOrder = newSpan.LastSuccess.CommitId.Order;
					LastSuccess = new IssueStepDocument(Id, newSpan.LastSuccess);
				}
				FirstFailure = new IssueStepDocument(Id, newSpan.FirstFailure);
				LastFailure = new IssueStepDocument(Id, newSpan.FirstFailure);
				if (newSpan.NextSuccess != null)
				{
					MaxCommitOrder = newSpan.NextSuccess.CommitId.Order;
					NextSuccess = new IssueStepDocument(Id, newSpan.NextSuccess);
				}
				PromoteByDefault = newSpan.FirstFailure.PromoteByDefault;
				Suspects = newSpan.Suspects.ConvertAll(x => new IssueSpanSuspectDocument(x));
				MaxSuspectRank = newSpan.MaxSuspectRank;
				IssueId = issueId;
			}
		}

		class IssueSpanSuspectDocument : IIssueSpanSuspect
		{
			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("Commit")]
			private string? CommitName { get; set; }

			[BsonElement("Change")]
			private int CommitOrder { get; set; }

			public UserId AuthorId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder? SourceCommitId
			{
				get => (SourceCommitName != null) ? new CommitIdWithOrder(SourceCommitName, SourceCommitOrder ?? 0) : (SourceCommitOrder != null) ? CommitIdWithOrder.FromPerforceChange(SourceCommitOrder.Value) : null;
				set => (SourceCommitName, SourceCommitOrder) = (value?.Name, value?.Order);
			}

			[BsonElement("SourceCommit")]
			public string? SourceCommitName { get; set; }

			[BsonElement("OriginatingChange")]
			public int? SourceCommitOrder { get; set; }

			[BsonConstructor]
			private IssueSpanSuspectDocument()
			{
			}

			public IssueSpanSuspectDocument(NewIssueSpanSuspectData newSuspectData)
			{
				CommitId = newSuspectData.CommitId;
				AuthorId = newSuspectData.AuthorId;
				SourceCommitId = newSuspectData.SourceCommitId;
			}
		}

		class IssueStepDocument : IIssueStep
		{
			public ObjectId Id { get; set; }
			public ObjectId SpanId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("Commit")]
			public string? CommitName { get; set; }

			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			public IssueSeverity Severity { get; set; }

			[BsonRequired]
			public string JobName { get; set; }

			[BsonRequired]
			public JobId JobId { get; set; }

			[BsonRequired]
			public JobStepBatchId BatchId { get; set; }

			[BsonRequired]
			public JobStepId StepId { get; set; }

			public DateTime StepTime { get; set; }

			public LogId? LogId { get; set; }

			[BsonIgnoreIfNull]
			public NodeAnnotations? Annotations { get; set; }

			public bool? PromoteByDefault { get; set; }

			[BsonElement("NotifySuspects"), BsonIgnoreIfDefault(false)]
			public bool NotifySuspectsDeprecated { get; set; }

			IReadOnlyNodeAnnotations IIssueStep.Annotations => Annotations ?? NodeAnnotations.Empty;
			bool IIssueStep.PromoteByDefault => PromoteByDefault ?? NotifySuspectsDeprecated;

			[BsonConstructor]
			private IssueStepDocument()
			{
				JobName = null!;
			}

			public IssueStepDocument(ObjectId spanId, NewIssueStepData stepData)
			{
				Id = ObjectId.GenerateNewId();
				SpanId = spanId;
				CommitId = stepData.CommitId;
				Severity = stepData.Severity;
				JobName = stepData.JobName;
				JobId = stepData.JobId;
				BatchId = stepData.BatchId;
				StepId = stepData.StepId;
				StepTime = stepData.StepTime;
				LogId = stepData.LogId;
				Annotations = stepData.Annotations;
				PromoteByDefault = stepData.PromoteByDefault;
			}
		}

		// Wraps a redis lock for the issue collection
		sealed class IssueLock : IAsyncDisposable
		{
			readonly RedisLock _redisLock;
			readonly string _purpose;
			readonly Stopwatch _timer = Stopwatch.StartNew();
			readonly TelemetrySpan _telemetrySpan;
			readonly ILogger _logger;

			public IssueLock(RedisLock redisLock, string purpose, Tracer tracer, ILogger logger)
			{
				_redisLock = redisLock;
				_purpose = purpose;
				_telemetrySpan = tracer.StartActiveSpan("Holding Issue Lock").SetAttribute("horde.purpose", _purpose);
				_logger = logger;
			}

			public async ValueTask DisposeAsync()
			{
				try
			{
				_telemetrySpan.Dispose();
				if (_timer.Elapsed.TotalSeconds >= 2.5)
				{
						_logger.LogWarning("Issue lock '{Purpose}' held for {TimeSpan}s", _purpose, (int)_timer.Elapsed.TotalSeconds);
					_timer.Reset();
				}
				}
				finally
				{
				await _redisLock.DisposeAsync();
				}
			}
		}

		readonly IRedisService _redisService;
		readonly IUserCollection _userCollection;
		readonly ICommitService _commitService;
		readonly ISingletonDocument<IssueLedger> _ledgerSingleton;
		readonly IMongoCollection<IssueDocument> _issues;
		readonly IMongoCollection<IssueSpanDocument> _issueSpans;
		readonly IMongoCollection<IssueStepDocument> _issueSteps;
		readonly IMongoCollection<IssueSuspectDocument> _issueSuspects;
		readonly IAuditLog<int> _auditLog;
		readonly ITelemetryWriter _telemetryWriter;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		static IssueCollection()
		{
			BsonClassMap.RegisterClassMap<IssueKey>(cm =>
			{
				cm.MapConstructor(() => new IssueKey("", IssueKeyType.None), nameof(IssueKey.Name), nameof(IssueKey.Type));
				cm.MapConstructor(() => new IssueKey("", IssueKeyType.None, null), nameof(IssueKey.Name), nameof(IssueKey.Type), nameof(IssueKey.Scope));
				cm.MapProperty(x => x.Name).SetElementName("n");
				cm.MapProperty(x => x.Type).SetElementName("t");
				cm.MapProperty(x => x.Scope).SetElementName("s").SetIgnoreIfNull(true);
			});

			BsonClassMap.RegisterClassMap<IssueMetadata>(cm =>
			{
				cm.MapConstructor(() => new IssueMetadata("", ""), nameof(IssueMetadata.Key), nameof(IssueMetadata.Value));
				cm.MapProperty(x => x.Key).SetElementName("k");
				cm.MapProperty(x => x.Value).SetElementName("v");
			});
		}

		public IssueCollection(IMongoService mongoService, IRedisService redisService, IUserCollection userCollection, ICommitService commitService, IAuditLogFactory<int> auditLogFactory, ITelemetryWriter telemetryWriter, IOptionsMonitor<BuildConfig> buildConfig, Tracer tracer, ILogger<IssueCollection> logger)
		{
			_redisService = redisService;
			_userCollection = userCollection;
			_commitService = commitService;
			_telemetryWriter = telemetryWriter;
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;

			_ledgerSingleton = new SingletonDocument<IssueLedger>(mongoService);

			List<MongoIndex<IssueDocument>> issueIndexes = new List<MongoIndex<IssueDocument>>();
			issueIndexes.Add(keys => keys.Ascending(x => x.ResolvedAt));
			issueIndexes.Add(keys => keys.Ascending(x => x.VerifiedAt));
			_issues = mongoService.GetCollection<IssueDocument>("IssuesV2");

			List<MongoIndex<IssueSpanDocument>> issueSpanIndexes = new List<MongoIndex<IssueSpanDocument>>();
			issueSpanIndexes.Add(keys => keys.Ascending(x => x.IssueId));
			issueSpanIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.MinCommitOrder).Ascending(x => x.MaxCommitOrder));
			// This index is created, though we need to move away from these C# index declarations, so only documenting it here
			// issueSpanIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.ResolvedAt));
			issueSpanIndexes.Add("StreamChanges", keys => keys.Ascending(x => x.StreamId).Ascending(x => x.TemplateRefId).Ascending(x => x.NodeName).Ascending(x => x.MinCommitOrder).Ascending(x => x.MaxCommitOrder));
			_issueSpans = mongoService.GetCollection<IssueSpanDocument>("IssuesV2.Spans", issueSpanIndexes);

			List<MongoIndex<IssueStepDocument>> issueStepIndexes = new List<MongoIndex<IssueStepDocument>>();
			issueStepIndexes.Add(keys => keys.Ascending(x => x.SpanId));
			issueStepIndexes.Add(keys => keys.Ascending(x => x.JobId).Ascending(x => x.BatchId).Ascending(x => x.StepId));
			_issueSteps = mongoService.GetCollection<IssueStepDocument>("IssuesV2.Steps", issueStepIndexes);

			List<MongoIndex<IssueSuspectDocument>> issueSuspectIndexes = new List<MongoIndex<IssueSuspectDocument>>();
			issueSuspectIndexes.Add(keys => keys.Ascending(x => x.CommitOrder));
			issueSuspectIndexes.Add(keys => keys.Ascending(x => x.AuthorId).Ascending(x => x.ResolvedAt));
			issueSuspectIndexes.Add(keys => keys.Ascending(x => x.IssueId).Ascending(x => x.CommitOrder), unique: true);
			_issueSuspects = mongoService.GetCollection<IssueSuspectDocument>("IssuesV2.Suspects", issueSuspectIndexes);

			_auditLog = auditLogFactory.Create("IssuesV2.History", "IssueId");
		}

		/// <inheritdoc/>
		public async Task<IAsyncDisposable> EnterCriticalSectionAsync(string purpose)
		{
			Stopwatch timer = Stopwatch.StartNew();
			TimeSpan nextNotifyTime = TimeSpan.FromSeconds(10.0);

			RedisLock issueLock = new(_redisService.GetDatabase(), "issues/lock");
			using (TelemetrySpan telemetrySpan = _tracer.StartActiveSpan("Wait for Issue Lock").SetAttribute("horde.purpose", purpose))
			{
				while (!await issueLock.AcquireAsync(TimeSpan.FromMinutes(1)))
				{
					if (timer.Elapsed > nextNotifyTime)
					{
						_logger.LogWarning("Waiting on lock '{Purpose}' over issue collection for {TimeSpan} ms", purpose, timer.ElapsedMilliseconds);
						nextNotifyTime *= 2;
					}
					await Task.Delay(TimeSpan.FromMilliseconds(100));
				}
			}
			return new IssueLock(issueLock, purpose, _tracer, _logger);
		}

		async Task<IssueDocument?> TryUpdateIssueAsync(IIssue issue, UpdateDefinition<IssueDocument> update, CancellationToken cancellationToken)
		{
			IssueDocument issueDocument = (IssueDocument)issue;

			int prevUpdateIndex = issueDocument.UpdateIndex;
			update = update.Set(x => x.UpdateIndex, prevUpdateIndex + 1);

			FindOneAndUpdateOptions<IssueDocument, IssueDocument> options = new FindOneAndUpdateOptions<IssueDocument, IssueDocument> { ReturnDocument = ReturnDocument.After };

			IssueDocument? newIssue = await _issues.FindOneAndUpdateAsync<IssueDocument>(x => x.Id == issueDocument.Id && x.UpdateIndex == prevUpdateIndex, update, options, cancellationToken);
			if (newIssue != null)
			{
				SendTelemetry(newIssue);
			}
			return newIssue;
		}

		async Task<IssueSpanDocument?> TryUpdateSpanAsync(IIssueSpan issueSpan, UpdateDefinition<IssueSpanDocument> update, CancellationToken cancellationToken)
		{
			IssueSpanDocument issueSpanDocument = (IssueSpanDocument)issueSpan;

			int prevUpdateIndex = issueSpanDocument.UpdateIndex;
			update = update.Set(x => x.UpdateIndex, prevUpdateIndex + 1);

			FindOneAndUpdateOptions<IssueSpanDocument, IssueSpanDocument> options = new FindOneAndUpdateOptions<IssueSpanDocument, IssueSpanDocument> { ReturnDocument = ReturnDocument.After };

			IssueSpanDocument? newIssueSpan = await _issueSpans.FindOneAndUpdateAsync<IssueSpanDocument>(x => x.Id == issueSpanDocument.Id && x.UpdateIndex == prevUpdateIndex, update, options, cancellationToken);
			if (newIssueSpan != null)
			{
				SendTelemetry(newIssueSpan);
			}
			return newIssueSpan;
		}

		#region Issues

		/// <inheritdoc/>
		public async Task<IIssue> AddIssueAsync(string summary, CancellationToken cancellationToken)
		{
			IssueLedger ledger = await _ledgerSingleton.UpdateAsync(x => x.NextId++, cancellationToken);

			IssueDocument newIssue = new IssueDocument(ledger.NextId, summary);
			await _issues.InsertOneAsync(newIssue, null, cancellationToken);
			SendTelemetry(newIssue);

			ILogger issueLogger = GetLogger(newIssue.Id);
			issueLogger.LogInformation("Created issue {IssueId}", newIssue.Id);

			return newIssue;
		}

		void SendTelemetry(IIssue issue)
		{
			List<TelemetryStoreId> telemetryStoreIds = new List<TelemetryStoreId>();
			foreach (IIssueStream stream in issue.Streams)
			{
				if (_buildConfig.CurrentValue.TryGetStream(stream.StreamId, out StreamConfig? streamConfig) && !streamConfig.TelemetryStoreId.IsEmpty)
				{
					telemetryStoreIds.Add(streamConfig.TelemetryStoreId);
				}
			}

			foreach (TelemetryStoreId telemetryStoreId in telemetryStoreIds)
			{
				_telemetryWriter.WriteEvent(telemetryStoreId, new IssueTelemetry(issue));
			}
		}

		async ValueTask<string> GetUserNameAsync(UserId? userId, CancellationToken cancellationToken)
		{
			if (userId == null)
			{
				return "null";
			}
			else if (userId == IIssue.ResolvedByUnknownId)
			{
				return "Horde (Unknown)";
			}
			else if (userId == IIssue.ResolvedByTimeoutId)
			{
				return "Horde (Timeout)";
			}

			IUser? user = await _userCollection.GetCachedUserAsync(userId, cancellationToken);
			if (user == null)
			{
				return "Unknown user";
			}

			return user.Name;
		}

		async Task LogIssueChangesAsync(UserId? initiatedByUserId, IssueDocument oldIssue, IssueDocument newIssue, CancellationToken cancellationToken)
		{
			ILogger issueLogger = GetLogger(oldIssue.Id);
			using IDisposable? scope = issueLogger.BeginScope("User {UserName} ({UserId})", await GetUserNameAsync(initiatedByUserId, cancellationToken), initiatedByUserId ?? UserId.Empty);
			await LogIssueChangesImplAsync(issueLogger, oldIssue, newIssue, cancellationToken);
		}

		async Task LogIssueChangesImplAsync(ILogger issueLogger, IssueDocument oldIssue, IssueDocument newIssue, CancellationToken cancellationToken)
		{
			if (newIssue.Severity != oldIssue.Severity)
			{
				issueLogger.LogInformation("Changed severity to {Severity}", newIssue.Severity);
			}
			if (newIssue.Summary != oldIssue.Summary)
			{
				issueLogger.LogInformation("Changed summary to \"{Summary}\"", newIssue.Summary);
			}
			if (newIssue.Description != oldIssue.Description)
			{
				issueLogger.LogInformation("Description set to {Value}", newIssue.Description);
			}
			if (((IIssue)newIssue).Promoted != ((IIssue)oldIssue).Promoted)
			{
				issueLogger.LogInformation("Promoted set to {Value}", ((IIssue)newIssue).Promoted);
			}
			if (newIssue.OwnerId != oldIssue.OwnerId)
			{
				if (newIssue.NominatedById != null)
				{
					issueLogger.LogInformation("User {UserName} ({UserId}) was nominated by {NominatedByUserName} ({NominatedByUserId})", await GetUserNameAsync(newIssue.OwnerId, cancellationToken), newIssue.OwnerId, await GetUserNameAsync(newIssue.NominatedById, cancellationToken), newIssue.NominatedById);
				}
				else
				{
					issueLogger.LogInformation("User {UserName} ({UserId}) was nominated by default", await GetUserNameAsync(newIssue.OwnerId, cancellationToken), newIssue.OwnerId);
				}
			}
			if (newIssue.AcknowledgedAt != oldIssue.AcknowledgedAt)
			{
				if (newIssue.AcknowledgedAt == null)
				{
					issueLogger.LogInformation("Issue was un-acknowledged by {UserName} ({UserId})", await GetUserNameAsync(oldIssue.OwnerId, cancellationToken), oldIssue.OwnerId);
				}
				else
				{
					issueLogger.LogInformation("Issue was acknowledged by {UserName} ({UserId})", await GetUserNameAsync(newIssue.OwnerId, cancellationToken), newIssue.OwnerId);
				}
			}
			if (newIssue.FixCommitId != oldIssue.FixCommitId)
			{
				if (newIssue.FixCommitId == null)
				{
					issueLogger.LogInformation("Issue was marked as not fixed");
				}
				else
				{
					issueLogger.LogInformation("Issue was marked as fixed in {Commit}", newIssue.FixCommitId);
				}
			}
			if (newIssue.FixSystemic != oldIssue.FixSystemic)
			{
				if (newIssue.FixSystemic)
				{
					issueLogger.LogInformation("Issue was marked fixed as a systemic issue");
				}
				else
				{
					issueLogger.LogInformation("Issue was marked as not fixed as a systemic issue");
				}
			}

			if (newIssue.RootCommitId != oldIssue.RootCommitId)
			{
				if (newIssue.RootCommitId == null)
				{
					issueLogger.LogInformation("Root commit for issue was cleared");
				}
				else
				{
					issueLogger.LogInformation("{Commit} marked as root commit for Issue", newIssue.RootCommitId);
				}
			}

			if (newIssue.RootCauseOwnerId != oldIssue.RootCauseOwnerId)
			{
				if (newIssue.RootCauseOwnerId == null)
				{
					issueLogger.LogInformation("Root cause owner was cleared");
				}
				else
				{
					issueLogger.LogInformation("Root cause owner is {UserName} ({UserId})", await GetUserNameAsync(newIssue.RootCauseOwnerId, cancellationToken), newIssue.RootCauseOwnerId);
				}
			}

			if (newIssue.RootCauseCategory != oldIssue.RootCauseCategory)
			{
				issueLogger.LogInformation("Changed root cause category to \"{Category}\"", newIssue.RootCauseCategory);
			}

			if (newIssue.RootCauseSummary != oldIssue.RootCauseSummary)
			{
				issueLogger.LogInformation("Changed root cause summary to \"{Summary}\"", newIssue.RootCauseSummary);
			}

			if (newIssue.DuplicateIssueId != oldIssue.DuplicateIssueId)
			{
				if (newIssue.DuplicateIssueId == null)
				{
					issueLogger.LogInformation("Duplicate Issue Id cleared");
				}
				else
				{
					issueLogger.LogInformation("Duplicate Issue Id set to {IssueId}", newIssue.DuplicateIssueId);
				}
			}

			if (newIssue.ResolvedById != oldIssue.ResolvedById)
			{
				if (newIssue.ResolvedById == null)
				{
					issueLogger.LogInformation("Marking as unresolved");
				}
				else
				{
					issueLogger.LogInformation("Resolved by {UserName} ({UserId})", await GetUserNameAsync(newIssue.ResolvedById, cancellationToken), newIssue.ResolvedById);
				}
			}

			if (newIssue.ResolvedAt != oldIssue.ResolvedAt)
			{
				if (newIssue.ResolvedAt == null)
				{
					issueLogger.LogInformation("Clearing resolved at time");
				}
				else
				{
					issueLogger.LogInformation("Setting resolved at time");
				}
			}

			if (newIssue.VerifiedAt != oldIssue.VerifiedAt)
			{
				if (newIssue.VerifiedAt == null)
				{
					issueLogger.LogInformation("Clearing verified at time");
				}
				else
				{
					issueLogger.LogInformation("Setting verified at time");
				}
			}

			if (newIssue.ExternalIssueKey != oldIssue.ExternalIssueKey)
			{
				if (newIssue.ExternalIssueKey != null)
				{
					issueLogger.LogInformation("Linked to external issue {ExternalIssueKey}", newIssue.ExternalIssueKey);
				}
				else
				{
					issueLogger.LogInformation("Unlinked from external issue {ExternalIssueKey}", oldIssue.ExternalIssueKey);
				}
			}

			if (newIssue.QuarantinedByUserId != oldIssue.QuarantinedByUserId)
			{
				if (newIssue.QuarantinedByUserId != null)
				{
					issueLogger.LogInformation("Quarantined by {UserName} ({UserId})", await GetUserNameAsync(newIssue.QuarantinedByUserId, cancellationToken), newIssue.QuarantinedByUserId);
				}
				else
				{
					issueLogger.LogInformation("Quarantine cleared");
				}
			}

			if (newIssue.ForceClosedByUserId != oldIssue.ForceClosedByUserId)
			{
				if (newIssue.ForceClosedByUserId != null)
				{
					issueLogger.LogInformation("Forced closed by {UserName} ({UserId})", await GetUserNameAsync(newIssue.ForceClosedByUserId, cancellationToken), newIssue.ForceClosedByUserId);
				}
				else
				{
					issueLogger.LogInformation("Force closed cleared");
				}
			}

			string oldFingerprints = oldIssue.FingerprintsDesc;
			string newFingerprints = newIssue.FingerprintsDesc;
			if (oldFingerprints != newFingerprints)
			{
				issueLogger.LogInformation("Fingerprints changed {Fingerprints}", newFingerprints);
			}

			HashSet<StreamId> oldFixStreams = new HashSet<StreamId>(oldIssue.Streams.Where(x => x.ContainsFix ?? false).Select(x => x.StreamId));
			HashSet<StreamId> newFixStreams = new HashSet<StreamId>(newIssue.Streams.Where(x => x.ContainsFix ?? false).Select(x => x.StreamId));
			foreach (StreamId streamId in newFixStreams.Where(x => !oldFixStreams.Contains(x)))
			{
				issueLogger.LogInformation("Marking stream {StreamId} as fixed", streamId);
			}
			foreach (StreamId streamId in oldFixStreams.Where(x => !newFixStreams.Contains(x)))
			{
				issueLogger.LogInformation("Marking stream {StreamId} as not fixed", streamId);
			}
		}

		async Task LogIssueSuspectChangesAsync(ILogger issueLogger, IReadOnlyList<IssueSuspectDocument> oldIssueSuspects, List<IssueSuspectDocument> newIssueSuspects, CancellationToken cancellationToken)
		{
			HashSet<(UserId, CommitId)> oldSuspects = new HashSet<(UserId, CommitId)>(oldIssueSuspects.Select(x => (x.AuthorId, (CommitId)x.CommitId)));
			HashSet<(UserId, CommitId)> newSuspects = new HashSet<(UserId, CommitId)>(newIssueSuspects.Select(x => (x.AuthorId, (CommitId)x.CommitId)));
			int addedSuspects = 0;
			int removedSuspects = 0;

			foreach ((UserId userId, CommitId commit) in newSuspects.Where(x => !oldSuspects.Contains(x)))
			{
				issueLogger.LogDebug("Added suspect {UserName} ({UserId}) for commit {Commit}", await GetUserNameAsync(userId, cancellationToken), userId, commit);
				addedSuspects++;
			}
			foreach ((UserId userId, CommitId commit) in oldSuspects.Where(x => !newSuspects.Contains(x)))
			{
				issueLogger.LogDebug("Removed suspect {UserName} ({UserId}) for commit {Commit}", await GetUserNameAsync(userId, cancellationToken), userId, commit);
				removedSuspects++;
			}

			issueLogger.LogInformation("Added {AddedSuspects} suspect(s). See Debug level logging for details.", addedSuspects);
			issueLogger.LogInformation("Removed {RemovedSuspects} suspect(s). See Debug level logging for details.", removedSuspects);

			HashSet<UserId> oldDeclinedBy = new HashSet<UserId>(oldIssueSuspects.Where(x => x.DeclinedAt != null).Select(x => x.AuthorId));
			HashSet<UserId> newDeclinedBy = new HashSet<UserId>(newIssueSuspects.Where(x => x.DeclinedAt != null).Select(x => x.AuthorId));
			foreach (UserId addDeclinedBy in newDeclinedBy.Where(x => !oldDeclinedBy.Contains(x)))
			{
				issueLogger.LogInformation("Declined by {UserName} ({UserId})", await GetUserNameAsync(addDeclinedBy, cancellationToken), addDeclinedBy);
			}
			foreach (UserId removeDeclinedBy in oldDeclinedBy.Where(x => !newDeclinedBy.Contains(x)))
			{
				issueLogger.LogInformation("Un-declined by {UserName} ({UserId})", await GetUserNameAsync(removeDeclinedBy, cancellationToken), removeDeclinedBy);
			}
		}

		/// <inheritdoc/>
		public async Task<IIssue?> GetIssueAsync(int issueId, CancellationToken cancellationToken)
		{
			IssueDocument issue = await _issues.Find(x => x.Id == issueId).FirstOrDefaultAsync(cancellationToken);
			return issue;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueSuspect>> FindSuspectsAsync(int issueId, CancellationToken cancellationToken)
		{
			return await _issueSuspects.Find(x => x.IssueId == issueId).ToListAsync(cancellationToken);
		}

		class ProjectedIssueId
		{
			[System.Diagnostics.CodeAnalysis.SuppressMessage("Style", "IDE1006:Naming Styles")]
			public int? _id { get; set; }
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssue>> FindIssuesAsync(IEnumerable<int>? ids = null, UserId? ownerId = null, StreamId? streamId = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, bool? resolved = null, bool? promoted = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<IIssue> results;

			if (ownerId == null)
			{
				results = await FilterIssuesByStreamIdAsync(ids, streamId, minCommitId, maxCommitId, resolved ?? false, promoted, index ?? 0, count, cancellationToken);
			}
			else
			{
				results = await _issues.Find(x => x.OwnerId == ownerId).ToListAsync(cancellationToken);
			}

			return results;
		}

		async Task<IReadOnlyList<IIssue>> FilterIssuesByStreamIdAsync(IEnumerable<int>? ids, StreamId? streamId, CommitId? minCommitId, CommitId? maxCommitId, bool? resolved, bool? promoted, int index, int? count, CancellationToken cancellationToken)
		{
			if (streamId == null)
			{
				if (minCommitId != null || maxCommitId != null)
				{
					throw new ArgumentException("Cannot search commit range without stream parameter");
				}

				return await FilterIssuesByOtherFieldsAsync(ids, resolved, promoted, index, count, cancellationToken);
			}
			else
			{
				FilterDefinition<IssueSpanDocument> filter = Builders<IssueSpanDocument>.Filter.Eq(x => x.StreamId, streamId.Value);
				if (ids != null)
				{
					filter &= Builders<IssueSpanDocument>.Filter.In(x => x.IssueId, ids.Select<int, int?>(x => x));
				}
				else
				{
					filter &= Builders<IssueSpanDocument>.Filter.Exists(x => x.IssueId);
				}

				if (minCommitId != null)
				{
					CommitIdWithOrder minCommitWithOrder = await _commitService.GetOrderedAsync(streamId.Value, minCommitId, cancellationToken);
					filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Lt(x => x.MaxCommitOrder, minCommitWithOrder.Order));
				}
				if (maxCommitId != null)
				{
					CommitIdWithOrder maxCommitWithOrder = await _commitService.GetOrderedAsync(streamId.Value, maxCommitId, cancellationToken);
					filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Gt(x => x.MinCommitOrder, maxCommitWithOrder.Order));
				}

				if (resolved != null)
				{
					if (resolved.Value)
					{
						filter &= Builders<IssueSpanDocument>.Filter.Ne(x => x.ResolvedAt, null);
					}
					else
					{
						filter &= Builders<IssueSpanDocument>.Filter.Eq(x => x.ResolvedAt, null);
					}
				}

				using (IAsyncCursor<ProjectedIssueId> cursor = await _issueSpans.Aggregate().Match(filter).Group(x => x.IssueId, x => new ProjectedIssueId { _id = x.Key }).SortByDescending(x => x._id).ToCursorAsync(cancellationToken))
				{
					List<IssueDocument> results = await PaginatedJoinAsync(cursor, (nextIds, nextIndex, nextCount) => FilterIssuesByOtherFieldsAsync(nextIds, null, promoted, nextIndex, nextCount, cancellationToken), index, count, cancellationToken);
					if (resolved != null)
					{
						for (int idx = results.Count - 1; idx >= 0; idx--)
						{
							IssueDocument issue = results[idx];
							if ((issue.ResolvedAt != null) != resolved.Value)
							{
								if (issue.ResolvedById != IIssue.ResolvedByTimeoutId) // TODO: We should always be marking spans as resolved when timing out an issue, but handling here for now.
								{
									_logger.LogWarning("Issue {IssueId} has resolved state out of sync with spans", issue.Id);
								}
								results.RemoveAt(idx);
							}
						}
					}
					return results;
				}
			}
		}

		async Task<List<IssueDocument>> FilterIssuesByOtherFieldsAsync(IEnumerable<int>? ids, bool? resolved, bool? promoted, int index, int? count, CancellationToken cancellationToken)
		{
			FilterDefinition<IssueDocument> filter = FilterDefinition<IssueDocument>.Empty;
			if (ids != null)
			{
				filter &= Builders<IssueDocument>.Filter.In(x => x.Id, ids);
			}
			if (resolved != null)
			{
				if (resolved.Value)
				{
					filter &= Builders<IssueDocument>.Filter.Ne(x => x.ResolvedAt, null);
				}
				else
				{
					filter &= Builders<IssueDocument>.Filter.Eq(x => x.ResolvedAt, null);
				}
			}
			if (promoted != null)
			{
				if (promoted.Value)
				{
					filter &= Builders<IssueDocument>.Filter.Eq(x => x.Promoted, true);
				}
				else
				{
					filter &= Builders<IssueDocument>.Filter.Ne(x => x.Promoted, true); // Handle the field not existing as well as being set to false.
				}
			}
			return await _issues.Find(filter).SortByDescending(x => x.Id).Range(index, count).ToListAsync(cancellationToken);
		}

		/// <summary>
		/// Performs a client-side join of a filtered set of issues against another query
		/// </summary>
		/// <param name="cursor"></param>
		/// <param name="nextStageFunc"></param>
		/// <param name="index"></param>
		/// <param name="count"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		static async Task<List<IssueDocument>> PaginatedJoinAsync(IAsyncCursor<ProjectedIssueId> cursor, Func<IEnumerable<int>, int, int?, Task<List<IssueDocument>>> nextStageFunc, int index, int? count, CancellationToken cancellationToken)
		{
			if (count == null)
			{
				List<ProjectedIssueId> issueIds = await cursor.ToListAsync(cancellationToken);
				return await nextStageFunc(issueIds.Where(x => x._id != null).Select(x => x._id!.Value), index, null);
			}
			else
			{
				List<IssueDocument> results = new List<IssueDocument>();
				while (await cursor.MoveNextAsync(cancellationToken) && results.Count < count.Value)
				{
					List<IssueDocument> nextResults = await nextStageFunc(cursor.Current.Where(x => x._id != null).Select(x => x._id!.Value), 0, count.Value - results.Count);
					int removeCount = Math.Min(index, nextResults.Count);
					nextResults.RemoveRange(0, removeCount);
					index -= removeCount;
					results.AddRange(nextResults);
				}
				return results;
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssue>> FindIssuesForChangesAsync(List<CommitIdWithOrder> commits, CancellationToken cancellationToken)
		{
			List<int> commitOrders = commits.ConvertAll(x => x.Order);
			List<int> issueIds = await (await _issueSuspects.DistinctAsync(x => x.IssueId, Builders<IssueSuspectDocument>.Filter.In(x => x.CommitOrder, commitOrders), cancellationToken: cancellationToken)).ToListAsync(cancellationToken);
			return await _issues.Find(Builders<IssueDocument>.Filter.In(x => x.Id, issueIds)).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IIssue?> TryUpdateIssueAsync(IIssue issue, UserId? initiatedByUserId, UpdateIssueOptions options, CancellationToken cancellationToken = default)
		{
			IssueDocument issueDocument = (IssueDocument)issue;

			if (options.DeclinedById != null && options.DeclinedById == issueDocument.OwnerId)
			{
				options = options with { OwnerId = UserId.Empty };
			}

			if (issue.ResolvedById == null && (options.ForceClosedById != null && options.ResolvedById == null))
			{
				options = options with { ResolvedById = options.ForceClosedById };
			}

			DateTime utcNow = DateTime.UtcNow;

			List<UpdateDefinition<IssueDocument>> updates = new List<UpdateDefinition<IssueDocument>>();
			if (options.Severity != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Severity, options.Severity.Value));
			}
			if (options.Summary != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Summary, options.Summary));
			}
			if (options.UserSummary != null)
			{
				if (options.UserSummary.Length == 0)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.UserSummary!));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.UserSummary, options.UserSummary));
				}
			}
			if (options.Description != null)
			{
				if (options.Description.Length == 0)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.Description));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.Description, options.Description));
				}
			}
			if (options.Promoted != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.ManuallyPromoted, options.Promoted.Value));
			}
			if (options.ResolvedById != null)
			{
				if (options.OwnerId != null && options.RootCauseOwnerId == null && issueDocument.RootCauseOwnerId == null)
				{
					options = options with { RootCauseOwnerId = options.OwnerId };
				}
				options = options with { OwnerId = options.OwnerId ?? options.ResolvedById, Acknowledged = options.Acknowledged ?? true };
			}
			if (options.OwnerId != null)
			{
				if (options.OwnerId.Value == UserId.Empty)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.OwnerId!));
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.NominatedAt!));
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.NominatedById!));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.OwnerId!, options.OwnerId.Value));

					updates.Add(Builders<IssueDocument>.Update.Set(x => x.NominatedAt, DateTime.UtcNow));
					if (options.NominatedById == null)
					{
						updates.Add(Builders<IssueDocument>.Update.Unset(x => x.NominatedById!));
					}
					else
					{
						updates.Add(Builders<IssueDocument>.Update.Set(x => x.NominatedById, options.NominatedById.Value));
					}
					options = options with { Acknowledged = options.Acknowledged ?? false };
				}
			}
			if (options.Acknowledged != null)
			{
				if (options.Acknowledged.Value)
				{
					if (issueDocument.AcknowledgedAt == null)
					{
						updates.Add(Builders<IssueDocument>.Update.Set(x => x.AcknowledgedAt, utcNow));
					}
				}
				else
				{
					if (issueDocument.AcknowledgedAt != null)
					{
						updates.Add(Builders<IssueDocument>.Update.Unset(x => x.AcknowledgedAt!));
					}
				}
			}
			if (options.FixCommitId != null)
			{
				if (String.IsNullOrEmpty(options.FixCommitId.Name))
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.FixCommitName).Unset(x => x.FixChange).Unset(x => x.FixSystemic));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.FixCommitName, options.FixCommitId.Name).Unset(x => x.FixChange).Unset(x => x.FixSystemic));
				}
			}
			else if (options.FixSystemic != null)
			{
				if (options.FixSystemic.Value)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.FixCommitName).Unset(x => x.FixChange).Set(x => x.FixSystemic, true));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.FixCommitName).Unset(x => x.FixChange).Unset(x => x.FixSystemic));
				}
			}
			if (options.RootCommitId != null)
			{
				if (String.IsNullOrEmpty(options.RootCommitId.Name))
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.RootCommitName));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.RootCommitName, options.RootCommitId.Name));
				}
			}

			if (options.RootCauseOwnerId != null)
			{
				if (options.RootCauseOwnerId.Value == UserId.Empty)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.RootCauseOwnerId));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.RootCauseOwnerId, options.RootCauseOwnerId.Value));
				}
			}

			if (options.RootCauseCategory != null)
			{
				if (options.RootCauseCategory.Length == 0)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.RootCauseCategory));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.RootCauseCategory, options.RootCauseCategory));
				}
			}

			if (options.RootCauseSummary != null)
			{
				if (options.RootCauseSummary.Length == 0)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.RootCauseSummary));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.RootCauseSummary, options.RootCauseSummary));
				}
			}

			if (options.DuplicateIssueId != null)
			{
				if (options.DuplicateIssueId < 0)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.DuplicateIssueId));
				}
				else if (options.DuplicateIssueId != issue.Id)
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.DuplicateIssueId, options.DuplicateIssueId));
				}
			}

			if (options.ResolvedById != null)
			{
				if (options.ResolvedById.Value != UserId.Empty)
				{
					if (issueDocument.ResolvedAt == null || issueDocument.ResolvedById != options.ResolvedById)
					{
						updates.Add(Builders<IssueDocument>.Update.Set(x => x.ResolvedAt, utcNow));
						updates.Add(Builders<IssueDocument>.Update.Set(x => x.ResolvedById, options.ResolvedById.Value));
					}
				}
				else
				{
					if (issueDocument.ResolvedAt != null)
					{
						updates.Add(Builders<IssueDocument>.Update.Unset(x => x.ResolvedAt!));
					}
					if (issueDocument.ResolvedById != null)
					{
						updates.Add(Builders<IssueDocument>.Update.Unset(x => x.ResolvedById!));
					}
				}
			}
			if (options.ExcludeSpanIds != null)
			{
				List<ObjectId> newCombinedExcludeSpanIds = options.ExcludeSpanIds;
				if (issue.ExcludeSpans != null)
				{
					newCombinedExcludeSpanIds = newCombinedExcludeSpanIds.Union(issue.ExcludeSpans).ToList();
				}
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.ExcludeSpans, newCombinedExcludeSpanIds));
			}
			if (options.LastSeenAt != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.LastSeenAt, options.LastSeenAt.Value));
			}

			if (options.DeclinedById != null)
			{
				GetLogger(issue.Id).LogInformation("Declined by {UserId}", options.DeclinedById.Value);
				await _issueSuspects.UpdateManyAsync(x => x.IssueId == issue.Id && x.AuthorId == options.DeclinedById.Value, Builders<IssueSuspectDocument>.Update.Set(x => x.DeclinedAt, DateTime.UtcNow), null, cancellationToken);
			}
			if (options.QuarantinedById != null)
			{
				if (options.QuarantinedById.Value == UserId.Empty)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.QuarantinedByUserId));
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.QuarantineTimeUtc));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.QuarantinedByUserId!, options.QuarantinedById.Value));
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.QuarantineTimeUtc, DateTime.UtcNow));
				}
			}
			else if ((options.ResolvedById != null && options.ResolvedById.Value != UserId.Empty) || (options.ForceClosedById != null && options.ForceClosedById.Value != UserId.Empty))
			{
				// Clear quarantine if being resolved or if being force closed
				updates.Add(Builders<IssueDocument>.Update.Unset(x => x.QuarantinedByUserId));
				updates.Add(Builders<IssueDocument>.Update.Unset(x => x.QuarantineTimeUtc));
			}

			if (options.ForceClosedById != null)
			{
				if (options.ForceClosedById.Value == UserId.Empty)
				{
					updates.Add(Builders<IssueDocument>.Update.Unset(x => x.ForceClosedByUserId));
				}
				else
				{
					updates.Add(Builders<IssueDocument>.Update.Set(x => x.ForceClosedByUserId, options.ForceClosedById.Value));
				}
			}

			if (options.ExternalIssueKey != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.ExternalIssueKey, options.ExternalIssueKey.Length == 0 ? null : options.ExternalIssueKey));
			}

			if (options.WorkflowThreadUrl != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.WorkflowThreadUrl, options.WorkflowThreadUrl.ToString().Length == 0 ? null : options.WorkflowThreadUrl));
			}

			if (updates.Count == 0)
			{
				return issueDocument;
			}

			IssueDocument? newIssue = await TryUpdateIssueAsync(issue, Builders<IssueDocument>.Update.Combine(updates), cancellationToken);
			if (newIssue == null)
			{
				return null;
			}

			await LogIssueChangesAsync(initiatedByUserId, issueDocument, newIssue, cancellationToken);
			return newIssue;
		}

		/// <inheritdoc/>
		public async Task<IIssue?> TryUpdateIssueDerivedDataAsync(IIssue issue, string newSummary, IssueSeverity newSeverity, List<NewIssueFingerprint> newFingerprints, List<NewIssueStream> newStreams, List<NewIssueSuspectData> newSuspects, DateTime? newResolvedAt, DateTime? newVerifiedAt, DateTime newLastSeenAt, CancellationToken cancellationToken)
		{
			IssueDocument issueImpl = (IssueDocument)issue;

			// Update all the suspects for this issue
			IReadOnlyList<IssueSuspectDocument> oldSuspectImpls = await _issueSuspects.Find(x => x.IssueId == issue.Id).ToListAsync(cancellationToken);
			List<IssueSuspectDocument> newSuspectImpls = await UpdateIssueSuspectsAsync(issue.Id, oldSuspectImpls, newSuspects, newResolvedAt, cancellationToken);

			// Find the spans for this issue
			List<IssueSpanDocument> newSpans = await _issueSpans.Find(x => x.IssueId == issue.Id).ToListAsync(cancellationToken);

			// Update the resolved time on any issues
			List<ObjectId> updateSpanIds = newSpans.Where(x => x.ResolvedAt != newResolvedAt).Select(x => x.Id).ToList();
			if (updateSpanIds.Count > 0)
			{
				FilterDefinition<IssueSpanDocument> filter = Builders<IssueSpanDocument>.Filter.In(x => x.Id, updateSpanIds);
				await _issueSpans.UpdateManyAsync(filter, Builders<IssueSpanDocument>.Update.Set(x => x.ResolvedAt, newResolvedAt), null, cancellationToken);
			}

			// Figure out if this issue should be promoted
			bool newPromoted;
			if (issueImpl.ManuallyPromoted.HasValue)
			{
				newPromoted = issueImpl.ManuallyPromoted.Value;
			}
			else if (issueImpl.ManuallyPromotedDeprecated.HasValue)
			{
				newPromoted = issueImpl.ManuallyPromotedDeprecated.Value;
			}
			else
			{
				newPromoted = newSpans.Any(x => ((IIssueSpan)x).PromoteByDefault);
			}

			// Find an owner automatically if annotations requires it
			UserId? newDefaultOwnerId = null;
			bool tryAutoAssign = newSpans.Any(x => x.LastFailure.Annotations?.AutoAssign ?? false);
			if (tryAutoAssign)
			{
				newDefaultOwnerId = TryAutoAssignFromSuspects(newSuspectImpls);
			}
			// Find the default owner (as a fallback) if annotations provides it
			// This behavior has a lower priority than autoAssign to allow using both (in which case, autoAssignToUser is a fallback)
			if (newDefaultOwnerId == null)
			{
				string? autoAssignToUser = newSpans.Select(x => x.LastFailure.Annotations?.AutoAssignToUser).Where(x => x != null).FirstOrDefault();
				if (autoAssignToUser != null)
				{
					IUser? user = await _userCollection.FindUserByLoginAsync(autoAssignToUser, cancellationToken);
					if (user != null)
					{
						newDefaultOwnerId = user.Id;
					}
				}
			}
			// Find an owner automatically if this issue is promoted (and we did not try already to AutoAssign)
			// This behavior has a lower priority than autoAssignToUser to maintain backward compatibility
			if (newDefaultOwnerId == null)
			{
				tryAutoAssign = !tryAutoAssign && newPromoted;
				if (tryAutoAssign)
				{
					newDefaultOwnerId = TryAutoAssignFromSuspects(newSuspectImpls);
				}
			}

			// Perform the actual update with this data
			List<UpdateDefinition<IssueDocument>> updates = new List<UpdateDefinition<IssueDocument>>();
			if (!String.Equals(issue.Summary, newSummary, StringComparison.Ordinal))
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Summary, newSummary));
			}
			if (issue.Severity != newSeverity)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Severity, newSeverity));
			}
			if (issue.Promoted != newPromoted)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Promoted, newPromoted));
			}
			if (issue.Fingerprints.Count != newFingerprints.Count || !newFingerprints.Zip(issue.Fingerprints).All(x => x.First.Equals(x.Second)))
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Fingerprints, newFingerprints.Select(x => new IssueFingerprintDocument(x))));
			}
			if (issue.Streams.Count != newStreams.Count || !newStreams.Zip(issue.Streams).All(x => x.First.StreamId == x.Second.StreamId && x.First.ContainsFix == x.Second.ContainsFix))
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.Streams, newStreams.Select(x => new IssueStreamDocument(x))));
			}
			if (issueImpl.DefaultOwnerId != newDefaultOwnerId)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.DefaultOwnerId, newDefaultOwnerId));
			}
			if (issue.ResolvedAt != newResolvedAt)
			{
				updates.Add(Builders<IssueDocument>.Update.SetOrUnsetNull(x => x.ResolvedAt, newResolvedAt));
			}
			if (newResolvedAt == null && issue.ResolvedById != null)
			{
				updates.Add(Builders<IssueDocument>.Update.Unset(x => x.ResolvedById));
			}
			if (issue.VerifiedAt != newVerifiedAt)
			{
				updates.Add(Builders<IssueDocument>.Update.SetOrUnsetNull(x => x.VerifiedAt, newVerifiedAt));
			}
			if (issue.LastSeenAt != newLastSeenAt)
			{
				updates.Add(Builders<IssueDocument>.Update.Set(x => x.LastSeenAt, newLastSeenAt));
			}

			IssueDocument? newIssue = await TryUpdateIssueAsync(issue, Builders<IssueDocument>.Update.Combine(updates), cancellationToken);
			if (newIssue != null)
			{
				await LogIssueChangesAsync(null, issueImpl, newIssue, cancellationToken);
				await LogIssueSuspectChangesAsync(GetLogger(issue.Id), oldSuspectImpls, newSuspectImpls, cancellationToken);
				return newIssue;
			}
			return null;
		}

		async Task<List<IssueSuspectDocument>> UpdateIssueSuspectsAsync(int issueId, IReadOnlyList<IssueSuspectDocument> oldSuspectImpls, List<NewIssueSuspectData> newSuspects, DateTime? resolvedAt, CancellationToken cancellationToken)
		{
			List<IssueSuspectDocument> newSuspectImpls = new List<IssueSuspectDocument>(oldSuspectImpls);

			// Find the current list of suspects
			HashSet<(UserId, CommitId)> curSuspectKeys = new HashSet<(UserId, CommitId)>(oldSuspectImpls.Select(x => (x.AuthorId, (CommitId)x.CommitId)));
			List<IssueSuspectDocument> createSuspects = newSuspects.Where(x => !curSuspectKeys.Contains((x.AuthorId, x.CommitId))).Select(x => new IssueSuspectDocument(issueId, x, resolvedAt)).ToList();

			HashSet<(UserId, CommitId)> newSuspectKeys = new HashSet<(UserId, CommitId)>(newSuspects.Select(x => (x.AuthorId, (CommitId)x.CommitId)));
			List<IssueSuspectDocument> deleteSuspects = oldSuspectImpls.Where(x => !newSuspectKeys.Contains((x.AuthorId, x.CommitId))).ToList();

			// Apply the suspect changes
			if (createSuspects.Count > 0)
			{
				await _issueSuspects.InsertManyIgnoreDuplicatesAsync(createSuspects, cancellationToken);
				newSuspectImpls.AddRange(createSuspects);
			}
			if (deleteSuspects.Count > 0)
			{
				await _issueSuspects.DeleteManyAsync(Builders<IssueSuspectDocument>.Filter.In(x => x.Id, deleteSuspects.Select(y => y.Id)), cancellationToken);
				newSuspectImpls.RemoveAll(x => !newSuspectKeys.Contains((x.AuthorId, x.CommitId)));
			}

			// Make sure all the remaining suspects have the correct resolved time
			if (newSuspectImpls.Any(x => x.ResolvedAt != resolvedAt))
			{
				await _issueSuspects.UpdateManyAsync(Builders<IssueSuspectDocument>.Filter.Eq(x => x.IssueId, issueId), Builders<IssueSuspectDocument>.Update.Set(x => x.ResolvedAt, resolvedAt), null, cancellationToken);
			}
			return newSuspectImpls;
		}

		private static UserId? TryAutoAssignFromSuspects(List<IssueSuspectDocument> suspects)
		{
			if (suspects.Count > 0)
			{
				UserId possibleOwnerId = suspects[0].AuthorId;
				if (suspects.All(x => x.AuthorId == possibleOwnerId) && suspects.Any(x => x.DeclinedAt == null))
				{
					return possibleOwnerId;
				}
			}
			return null;
		}

		#endregion

		#region Spans

		/// <inheritdoc/>
		public async Task<IIssueSpan> AddSpanAsync(int issueId, NewIssueSpanData newSpan, CancellationToken cancellationToken)
		{
			IssueSpanDocument span = new IssueSpanDocument(issueId, newSpan);
			await _issueSpans.InsertOneAsync(span, (InsertOneOptions?)null, cancellationToken);
			SendTelemetry(span);
			return span;
		}

		void SendTelemetry(IIssueSpan issueSpan)
		{
			if (_buildConfig.CurrentValue.TryGetStream(issueSpan.StreamId, out StreamConfig? streamConfig) && !streamConfig.TelemetryStoreId.IsEmpty)
			{
				_telemetryWriter.WriteEvent(streamConfig.TelemetryStoreId, new IssueSpanTelemetry(issueSpan));
			}
		}

		/// <inheritdoc/>
		public async Task<IIssueSpan?> GetSpanAsync(ObjectId spanId, CancellationToken cancellationToken)
		{
			return await _issueSpans.Find(Builders<IssueSpanDocument>.Filter.Eq(x => x.Id, spanId)).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IIssueSpan?> TryUpdateSpanAsync(IIssueSpan span, NewIssueStepData? newLastSuccess = null, NewIssueStepData? newFailure = null, NewIssueStepData? newNextSuccess = null, List<NewIssueSpanSuspectData>? newSuspects = null, int? newMaxSuspectRank = null, int? newIssueId = null, CancellationToken cancellationToken = default)
		{
			List<UpdateDefinition<IssueSpanDocument>> updates = new List<UpdateDefinition<IssueSpanDocument>>();
			if (newLastSuccess != null)
			{
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.MinCommitOrder, newLastSuccess.CommitId.Order));
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.LastSuccess, new IssueStepDocument(span.Id, newLastSuccess)));
			}
			if (newFailure != null)
			{
				if (newFailure.CommitId < span.FirstFailure.CommitId)
				{
					updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.FirstFailure, new IssueStepDocument(span.Id, newFailure)));
				}
				if (newFailure.CommitId >= span.LastFailure.CommitId)
				{
					updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.LastFailure, new IssueStepDocument(span.Id, newFailure)));
				}
				if (newFailure.PromoteByDefault != span.PromoteByDefault && newFailure.CommitId >= span.LastFailure.CommitId)
				{
					updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.PromoteByDefault, newFailure.PromoteByDefault));
				}
			}
			if (newNextSuccess != null)
			{
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.MaxCommitOrder, newNextSuccess.CommitId.Order));
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.NextSuccess, new IssueStepDocument(span.Id, newNextSuccess)));
			}
			if (newSuspects != null)
			{
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.Suspects, newSuspects.ConvertAll(x => new IssueSpanSuspectDocument(x))));
			}
			if (newMaxSuspectRank != null)
			{
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.MaxSuspectRank, newMaxSuspectRank.Value));
			}
			if (newIssueId != null)
			{
				updates.Add(Builders<IssueSpanDocument>.Update.Set(x => x.IssueId, newIssueId.Value));
			}

			if (updates.Count == 0)
			{
				return span;
			}

			IssueSpanDocument? newSpan = await TryUpdateSpanAsync(span, Builders<IssueSpanDocument>.Update.Combine(updates), cancellationToken);
			if (newSpan != null)
			{
				ILogger logger = GetLogger(newSpan.IssueId);
				if (newLastSuccess != null)
				{
					logger.LogInformation("Set last success for span {SpanId} to job {JobId} at commit {Commit}", newSpan.Id, newLastSuccess.JobId, newLastSuccess.CommitId);
				}
				if (newNextSuccess != null)
				{
					logger.LogInformation("Set next success for span {SpanId} to job {JobId} at commit {Commit}", newSpan.Id, newNextSuccess.JobId, newNextSuccess.CommitId);
				}
				if (newFailure != null)
				{
					logger.LogInformation("Added failure for span {SpanId} in job {JobId} at commit {Commit}", newSpan.Id, newFailure.JobId, newFailure.CommitId);
				}
				SendTelemetry(newSpan);
			}
			return newSpan;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(int issueId, CancellationToken cancellationToken)
		{
			return await _issueSpans.Find(x => x.IssueId == issueId).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(IEnumerable<ObjectId> spanIds, CancellationToken cancellationToken)
		{
			return await _issueSpans.Find(Builders<IssueSpanDocument>.Filter.In(x => x.Id, spanIds)).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueSpan>> FindOpenSpansAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitId commitId, CancellationToken cancellationToken)
		{
			CommitIdWithOrder commitIdWithOrder = await _commitService.GetOrderedAsync(streamId, commitId, cancellationToken);
			return await _issueSpans.Find(x => x.StreamId == streamId && x.TemplateRefId == templateId && x.NodeName == nodeName && commitIdWithOrder.Order >= x.MinCommitOrder && commitIdWithOrder.Order <= x.MaxCommitOrder).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueSpan>> FindSpansAsync(IEnumerable<ObjectId>? spanIds, IEnumerable<int>? issueIds, StreamId? streamId, CommitId? minCommitId, CommitId? maxCommitId, bool? resolved, int? index, int? count, CancellationToken cancellationToken)
		{
			FilterDefinition<IssueSpanDocument> filter = FilterDefinition<IssueSpanDocument>.Empty;

			if (spanIds != null)
			{
				filter &= Builders<IssueSpanDocument>.Filter.In(x => x.Id, spanIds);
			}

			if (streamId != null)
			{
				filter &= Builders<IssueSpanDocument>.Filter.Eq(x => x.StreamId, streamId);
				if (minCommitId != null)
				{
					CommitIdWithOrder minCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, minCommitId, cancellationToken);
					filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Lt(x => x.MaxCommitOrder, minCommitIdWithOrder.Order));
				}
				if (maxCommitId != null)
				{
					CommitIdWithOrder maxCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, maxCommitId, cancellationToken);
					filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Gt(x => x.MinCommitOrder, maxCommitIdWithOrder.Order));
				}
			}

			if (issueIds != null)
			{
				filter &= Builders<IssueSpanDocument>.Filter.In(x => x.IssueId, issueIds.Select<int, int?>(x => x));
			}
			if (resolved != null)
			{
				if (resolved.Value)
				{
					filter &= Builders<IssueSpanDocument>.Filter.Ne(x => x.ResolvedAt, null);
				}
				else
				{
					filter &= Builders<IssueSpanDocument>.Filter.Eq(x => x.ResolvedAt, null);
				}
			}

			return await _issueSpans.Find(filter).Range(index, count).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<int>> FindSpanIssueIdsAsync(StreamId streamId, CommitId? minCommitId, CommitId? maxCommitId, bool? resolved, CancellationToken cancellationToken)
		{
			FilterDefinition<IssueSpanDocument> filter = Builders<IssueSpanDocument>.Filter.Eq(x => x.StreamId, streamId);

			if (minCommitId != null)
			{
				CommitIdWithOrder minCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId, minCommitId, cancellationToken);
				filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Lt(x => x.MaxCommitOrder, minCommitIdWithOrder.Order));
			}
			if (maxCommitId != null)
			{
				CommitIdWithOrder maxCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId, maxCommitId, cancellationToken);
				filter &= Builders<IssueSpanDocument>.Filter.Not(Builders<IssueSpanDocument>.Filter.Gt(x => x.MinCommitOrder, maxCommitIdWithOrder.Order));
			}

			if (resolved != null)
			{
				if (resolved.Value)
				{
					filter &= Builders<IssueSpanDocument>.Filter.Ne(x => x.ResolvedAt, null);
				}
				else
				{
					filter &= Builders<IssueSpanDocument>.Filter.Eq(x => x.ResolvedAt, null);
				}
			}

			List<ProjectedIssueId> result = await _issueSpans.Aggregate().Match(filter).Group(x => x.IssueId, x => new ProjectedIssueId { _id = x.Key }).ToListAsync(cancellationToken);
			return result.Where(x => x._id != null).Select(x => x._id!.Value).ToList();
		}

		#endregion

		#region Steps

		/// <inheritdoc/>
		public async Task<IIssueStep> AddStepAsync(ObjectId spanId, NewIssueStepData newStep, CancellationToken cancellationToken)
		{
			IssueStepDocument step = new IssueStepDocument(spanId, newStep);
			await _issueSteps.InsertOneAsync(step, (InsertOneOptions?)null, cancellationToken);
			return step;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueStep>> FindStepsAsync(IEnumerable<ObjectId> spanIds, CancellationToken cancellationToken)
		{
			FilterDefinition<IssueStepDocument> filter = Builders<IssueStepDocument>.Filter.In(x => x.SpanId, spanIds);
			return await _issueSteps.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IIssueStep>> FindStepsAsync(JobId jobId, JobStepBatchId? batchId, JobStepId? stepId, CancellationToken cancellationToken)
		{
			FilterDefinition<IssueStepDocument> filter = Builders<IssueStepDocument>.Filter.Eq(x => x.JobId, jobId);
			if (batchId != null)
			{
				filter &= Builders<IssueStepDocument>.Filter.Eq(x => x.BatchId, batchId.Value);
			}
			if (stepId != null)
			{
				filter &= Builders<IssueStepDocument>.Filter.Eq(x => x.StepId, stepId.Value);
			}
			return await _issueSteps.Find(filter).ToListAsync(cancellationToken);
		}

		#endregion

		/// <inheritdoc/>
		public IAuditLogChannel<int> GetLogger(int issueId)
		{
			return _auditLog[issueId];
		}

		/// <inheritdoc/>
		public async ValueTask<int?> GetIssueIdAsync(ObjectId spanId, CancellationToken cancellationToken = default)
		{
			IIssueSpan? span = await GetSpanAsync(spanId, cancellationToken);
			return span?.IssueId;
		}
	}
}
