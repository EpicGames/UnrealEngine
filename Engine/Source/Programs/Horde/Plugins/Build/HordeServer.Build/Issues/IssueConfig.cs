// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using HordeServer.Configuration;
using HordeServer.Utilities;

namespace HordeServer.Issues
{

	/// <summary>
	/// Configuration for grouping to apply based on logging channels
	/// </summary>
	public class IssueChannelGroupConfig
	{
		/// <summary>
		/// A list of channel logging boundaries to group issues on
		/// </summary>
		public List<string>? Channels { get; set; }
	}

	/// <summary>
	/// Stores configuration for build health issues
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/issues")]
	[JsonSchemaCatalog("Horde Issue", "Horde issue configuration file", new[] { "*.issues.json", "Issues/*.json" })]
	[ConfigDoc("*.issues.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Issues.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	public class IssueConfig
	{
		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Issue grouping channels
		/// </summary>
		public List<IssueChannelGroupConfig>? ChannelGroups { get; set; }

		/// <summary>
		/// Process Issue Config
		/// </summary>
		public void PostLoad()
		{
			ChannelGroups?.ForEach(group =>
			{
				group.Channels?.Sort();
			});
		}
	}
}
