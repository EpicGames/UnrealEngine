// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using HordeServer.Jobs;
using HordeServer.Jobs.Graphs;
using HordeServer.Notifications;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;

namespace HordeServer.Issues
{
#pragma warning disable CS1591 // Missing XML comment for publicly visible type or member
	public class WorkflowStats
	{
		public int NumSteps { get; set; }
		public int NumPassingSteps { get; set; }
	}

	public class IssueReport
	{
		public StreamId StreamId { get; }
		public WorkflowId WorkflowId { get; }
		public WorkflowStats WorkflowStats { get; }
		public string? TriageChannel { get; }
		public List<IIssue> Issues { get; } = new List<IIssue>();
		public List<IIssueSpan> IssueSpans { get; } = new List<IIssueSpan>();
		public bool GroupByTemplate { get; }

		public IssueReport(StreamId streamId, WorkflowId workflowId, WorkflowStats workflowStats, string? triageChannel, bool groupByTemplate)
		{
			StreamId = streamId;
			WorkflowId = workflowId;
			WorkflowStats = workflowStats;
			TriageChannel = triageChannel;
			GroupByTemplate = groupByTemplate;
		}
	}

	public class IssueReportGroup
	{
		public string Channel { get; }
		public DateTime Time { get; }
		public List<IssueReport> Reports { get; } = new List<IssueReport>();

		public IssueReportGroup(string channel, DateTime time)
		{
			Channel = channel;
			Time = time;
		}
	}

#pragma warning restore CS1591 // Missing XML comment for publicly visible type or member

	[SingletonDocument("issue-report-state", "6268871c211d05611b3e4fd8")]
	class IssueReportState : SingletonBase
	{
		[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
		public Dictionary<string, DateTime> ReportTimes { get; set; } = new Dictionary<string, DateTime>();
	}

	/// <summary>
	/// Posts summaries for all the open issues in different streams to Slack channels
	/// </summary>
	public class IssueReportService : IHostedService
	{
		readonly SingletonDocument<IssueReportState> _state;
		readonly IIssueCollection _issueCollection;
		readonly IGraphCollection _graphCollection;
		readonly IJobCollection _jobCollection;
		readonly INotificationService _notificationService;
		readonly IClock _clock;
		readonly ITicker _ticker;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger<IssueReportService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueReportService(IMongoService mongoService, IIssueCollection issueCollection, IGraphCollection graphCollection, IJobCollection jobCollection, INotificationService notificationService, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<IssueReportService> logger)
		{
			_state = new SingletonDocument<IssueReportState>(mongoService);
			_issueCollection = issueCollection;
			_graphCollection = graphCollection;
			_jobCollection = jobCollection;
			_notificationService = notificationService;
			_clock = clock;
			_ticker = clock.AddSharedTicker<IssueReportService>(TimeSpan.FromMinutes(5.0), TickAsync, logger);
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			IssueReportState state = await _state.GetAsync(cancellationToken);
			HashSet<string> invalidKeys = new HashSet<string>(state.ReportTimes.Keys, StringComparer.Ordinal);

			DateTime currentTime = _clock.UtcNow;

			List<string> updateKeys = new List<string>();
			List<IssueReportGroup> groups = new List<IssueReportGroup>();

			BuildConfig buildConfig = _buildConfig.CurrentValue;
			foreach (StreamConfig streamConfig in buildConfig.Streams)
			{
				if (streamConfig.Workflows.Count > 0)
				{
					IReadOnlyList<IIssue>? issues = null;
					IReadOnlyList<IIssueSpan>? spans = null;

					foreach (WorkflowConfig workflowConfig in streamConfig.Workflows)
					{
						if (workflowConfig.ReportChannel == null)
						{
							continue;
						}

						string key = $"{streamConfig.Id}:{workflowConfig.Id}";
						invalidKeys.Remove(key);

						DateTime lastReportTime;
						if (!state.ReportTimes.TryGetValue(key, out lastReportTime))
						{
							state = await _state.UpdateAsync(s => s.ReportTimes[key] = currentTime, cancellationToken);
							continue;
						}

						DateTime lastScheduledReportTime = GetLastScheduledReportTime(workflowConfig, currentTime, _clock.TimeZone);
						if (lastReportTime > lastScheduledReportTime)
						{
							continue;
						}

						DateTime prevScheduledReportTime = GetLastScheduledReportTime(workflowConfig, lastScheduledReportTime - TimeSpan.FromMinutes(1.0), _clock.TimeZone);

						_logger.LogInformation("Creating report for {StreamId} workflow {WorkflowId}", streamConfig.Id, workflowConfig.Id);

						issues ??= await _issueCollection.FindIssuesAsync(streamId: streamConfig.Id, cancellationToken: cancellationToken);
						spans ??= await _issueCollection.FindSpansAsync(issueIds: issues.Select(x => x.Id).ToArray(), cancellationToken: cancellationToken);

						Dictionary<WorkflowId, WorkflowStats> workflowIdToStats = await GetWorkflowStatsAsync(streamConfig, prevScheduledReportTime, cancellationToken);
						if (!workflowIdToStats.TryGetValue(workflowConfig.Id, out WorkflowStats? workflowStats))
						{
							workflowStats = new WorkflowStats();
						}

						if (spans.Count > 0 || !workflowConfig.SkipWhenEmpty)
						{
							IssueReport report = new IssueReport(streamConfig.Id, workflowConfig.Id, workflowStats, workflowConfig.TriageChannel, workflowConfig.GroupIssuesByTemplate);
							foreach (IIssueSpan span in spans)
							{
								if (span.LastFailure.Annotations.WorkflowId == workflowConfig.Id)
								{
									report.IssueSpans.Add(span);
								}
							}

							HashSet<int> issueIds = new HashSet<int>(report.IssueSpans.Select(x => x.IssueId));
							report.Issues.AddRange(issues.Where(x => issueIds.Contains(x.Id)));

							if (!workflowConfig.ReportWarnings)
							{
								report.Issues.RemoveAll(x => x.Severity == IssueSeverity.Warning);
							}

							DateTime reportTime = lastScheduledReportTime;

							IssueReportGroup? group = groups.FirstOrDefault(x => x.Channel == workflowConfig.ReportChannel && x.Time == reportTime);
							if (group == null)
							{
								group = new IssueReportGroup(workflowConfig.ReportChannel, reportTime);
								groups.Add(group);
							}
							group.Reports.Add(report);
						}

						updateKeys.Add(key);
					}
				}
			}

			foreach (IssueReportGroup group in groups)
			{
				await _notificationService.SendIssueReportAsync(group, cancellationToken);
			}

			if (updateKeys.Count > 0 || invalidKeys.Count > 0)
			{
				void UpdateKeys(IssueReportState state)
				{
					foreach (string updateKey in updateKeys)
					{
						state.ReportTimes[updateKey] = currentTime;
					}
					foreach (string invalidKey in invalidKeys)
					{
						state.ReportTimes.Remove(invalidKey);
					}
				}
				state = await _state.UpdateAsync(UpdateKeys, cancellationToken);
			}
		}

		private async Task<Dictionary<WorkflowId, WorkflowStats>> GetWorkflowStatsAsync(StreamConfig streamConfig, DateTime minTime, CancellationToken cancellationToken)
		{
			IReadOnlyList<IJob> jobs = await _jobCollection.FindAsync(new FindJobOptions(StreamId: streamConfig.Id, MinCreateTime: minTime), cancellationToken: cancellationToken);

			Dictionary<WorkflowId, WorkflowStats> workflowIdToStats = new Dictionary<WorkflowId, WorkflowStats>();
			foreach (IGrouping<TemplateId, IJob> templateGroup in jobs.GroupBy(x => x.TemplateId))
			{
				WorkflowId? templateWorkflowId = null;
				if (streamConfig.TryGetTemplate(templateGroup.Key, out TemplateRefConfig? templateRefConfig))
				{
					templateWorkflowId = templateRefConfig.Annotations.WorkflowId;
				}

				foreach (IGrouping<ContentHash, IJob> graphGroup in templateGroup.GroupBy(x => x.GraphHash))
				{
					IGraph graph = await _graphCollection.GetAsync(graphGroup.Key, cancellationToken);
					foreach (IJob job in graphGroup)
					{
						foreach (IJobStepBatch batch in job.Batches)
						{
							foreach (IJobStep step in batch.Steps)
							{
								if (step.State == JobStepState.Completed)
								{
									INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
									WorkflowId? workflowId = node.Annotations.WorkflowId ?? templateWorkflowId;
									if (workflowId != null)
									{
										WorkflowStats? stats;
										if (!workflowIdToStats.TryGetValue(workflowId.Value, out stats))
										{
											stats = new WorkflowStats();
											workflowIdToStats.Add(workflowId.Value, stats);
										}

										stats.NumSteps++;
										if (step.Outcome == JobStepOutcome.Success)
										{
											stats.NumPassingSteps++;
										}
									}
								}
							}
						}
					}
				}
			}

			return workflowIdToStats;
		}

		static DateTime GetLastScheduledReportTime(WorkflowConfig workflow, DateTime currentTimeUtc, TimeZoneInfo timeZone)
		{
			if (workflow.ReportTimes.Count > 0)
			{
				DateTime startOfDayUtc = timeZone.GetStartOfDayUtc(currentTimeUtc);
				for (; ; )
				{
					for (int idx = workflow.ReportTimes.Count - 1; idx >= 0; idx--)
					{
						DateTime reportTime = startOfDayUtc + workflow.ReportTimes[idx];
						if (reportTime < currentTimeUtc)
						{
							return reportTime;
						}
					}
					startOfDayUtc -= TimeSpan.FromDays(1.0);
				}
			}
			return DateTime.MinValue;
		}
	}
}
