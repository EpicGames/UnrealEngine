// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Text;
using System.Threading.Channels;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Commits;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.RCA;
using HordeServer.Streams;
using HordeServer.Users;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using OpenTelemetry.Trace;

namespace HordeServer.Issues
{
	/// <summary>
	/// Detailed issue information
	/// </summary>
	class IssueDetails : IIssueDetails
	{
		/// <inheritdoc/>
		public IIssue Issue { get; }

		/// <inheritdoc/>
		public IUser? Owner { get; }

		/// <inheritdoc/>
		public IUser? NominatedBy { get; }

		/// <inheritdoc/>
		public IUser? ResolvedBy { get; }

		/// <inheritdoc/>
		public IUser? RootCauseOwner { get; }

		/// <inheritdoc/>
		public IUser? QuarantinedBy { get; set; }

		/// <inheritdoc/>
		public DateTime? QuarantineTimeUtc { get; set; }

		/// <inheritdoc/>
		public IUser? ForceClosedBy { get; set; }

		/// <inheritdoc/>
		public string? ExternalIssueKey { get; }

		/// <inheritdoc/>
		public IReadOnlyList<IIssueSpan> Spans { get; }

		/// <inheritdoc/>
		public IReadOnlyList<IIssueStep> Steps { get; }

		/// <inheritdoc/>
		public IReadOnlyList<IIssueSuspect> Suspects { get; }

		/// <inheritdoc/>
		public IReadOnlyList<IUser> SuspectUsers { get; }

		/// <inheritdoc/>
		readonly bool _showDesktopAlerts;

		/// <inheritdoc/>
		readonly HashSet<UserId> _notifyUsers;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="issue"></param>
		/// <param name="owner"></param>
		/// <param name="nominatedBy"></param>
		/// <param name="resolvedBy"></param>
		/// <param name="quarantinedBy"></param>
		/// <param name="forceClosedBy"></param>
		/// <param name="rootCauseOwner"></param>
		/// <param name="spans">Spans for this issue</param>
		/// <param name="steps">Steps for this issue</param>
		/// <param name="suspects"></param>
		/// <param name="suspectUsers"></param>
		/// <param name="showDesktopAlerts"></param>
		/// <param name="externalIssueKey"></param>
		public IssueDetails(IIssue issue, IUser? owner, IUser? nominatedBy, IUser? resolvedBy, IUser? quarantinedBy, IUser? forceClosedBy, IUser? rootCauseOwner, IReadOnlyList<IIssueSpan> spans, IReadOnlyList<IIssueStep> steps, IReadOnlyList<IIssueSuspect> suspects, IReadOnlyList<IUser> suspectUsers, bool showDesktopAlerts, string? externalIssueKey)
		{
			Issue = issue;
			Owner = owner;
			NominatedBy = nominatedBy;
			ResolvedBy = resolvedBy;
			QuarantinedBy = quarantinedBy;
			QuarantineTimeUtc = issue.QuarantineTimeUtc;
			ForceClosedBy = forceClosedBy;
			RootCauseOwner = rootCauseOwner;
			Spans = spans;
			Steps = steps;
			Suspects = suspects;
			SuspectUsers = suspectUsers;
			ExternalIssueKey = externalIssueKey;
			_showDesktopAlerts = showDesktopAlerts;

			if (issue.OwnerId == null)
			{
				_notifyUsers = new HashSet<UserId>(suspects.Select(x => x.AuthorId));
			}
			else
			{
				_notifyUsers = new HashSet<UserId> { issue.OwnerId.Value };
			}
		}

		/// <summary>
		/// Determines whether the given user should be notified about the given issue
		/// </summary>
		/// <returns>True if the user should be notified for this change</returns>
		public bool ShowNotifications()
		{
			return _showDesktopAlerts && Issue.Fingerprints.Any(x => x.Type != "Default");
		}

		/// <summary>
		/// Determines if the issue is relevant to the given user
		/// </summary>
		/// <param name="userId">The user to query</param>
		/// <returns>True if the issue is relevant to the given user</returns>
		public bool IncludeForUser(UserId userId)
		{
			return _notifyUsers.Contains(userId);
		}
	}

	/// <summary>
	/// Wraps functionality for manipulating build health issues
	/// </summary>
	public sealed class IssueService : IHostedService, IAsyncDisposable
	{
		class IssueEventInternal : IssueEvent
		{
			public ILogAnchor Anchor { get; }
			public ILogEventData EventData { get; }

			public IssueEventInternal(ILogAnchor anchor, ILogEventData logEventData)
				: base(anchor.LineIndex, GetLogLevelFromSeverity(logEventData.Severity), logEventData.EventId, logEventData.Lines)
			{
				Anchor = anchor;
				EventData = logEventData;
			}

			static LogLevel GetLogLevelFromSeverity(LogEventSeverity severity) => severity switch
			{
				LogEventSeverity.Error => LogLevel.Error,
				LogEventSeverity.Warning => LogLevel.Warning,
				_ => LogLevel.Information
			};
		}

		class IssueEventGroupInternal
		{
			public int Id { get; }
			public NewIssueFingerprint Fingerprint { get; }
			public List<IssueEventInternal> Events { get; } = new List<IssueEventInternal>();

			static int s_nextId = 1;

			public IssueEventGroupInternal(NewIssueFingerprint fingerprint)
			{
				Id = Interlocked.Increment(ref s_nextId);
				Fingerprint = fingerprint;
			}

			public IssueEventGroupInternal MergeWith(IssueEventGroupInternal otherGroup)
			{
				IssueEventGroupInternal newGroup = new IssueEventGroupInternal(NewIssueFingerprint.Merge(Fingerprint, otherGroup.Fingerprint));
				newGroup.Events.AddRange(Events);
				newGroup.Events.AddRange(otherGroup.Events);
				return newGroup;
			}

			/// <inheritdoc/>
			public override string ToString() => Fingerprint.ToString();
		}

		record class CompleteStep(JobId JobId, JobStepBatchId BatchId, JobStepId StepId);

		readonly JobService _jobService;
		readonly IJobStepRefCollection _jobStepRefs;
		readonly IIssueCollection _issueCollection;
		readonly ICommitService _commitService;
		readonly IUserCollection _userCollection;
		readonly ILogCollection _logCollection;
		readonly IClock _clock;
		readonly ITicker _ticker;
		readonly Channel<CompleteStep> _completeSteps;
		readonly BackgroundTask _completeStepUpdateTask;

		/// <summary>
		/// Accessor for the issue collection
		/// </summary>
		public IIssueCollection Collection => _issueCollection;

		/// <summary>
		/// Delegate for issue creation events
		/// </summary>
		public delegate void IssueUpdatedEvent(IIssue issue);

		/// <summary>
		/// 
		/// </summary>
		public event IssueUpdatedEvent? OnIssueUpdated;

		// All discovered issue handlers, sorted by priority
		readonly (Type Type, IssueHandlerAttribute Attribute)[] _handlerTypes;

		/// <summary>
		/// Cached list of currently open issues
		/// </summary>
		IReadOnlyList<IIssueDetails> _cachedIssues = new List<IIssueDetails>();

		/// <summary>
		/// Cache of templates to show desktop alerts for
		/// </summary>
		Dictionary<StreamId, HashSet<TemplateId>> _cachedDesktopAlerts = new Dictionary<StreamId, HashSet<TemplateId>>();

		/// <summary>
		/// Tracer
		/// </summary>
		readonly Tracer _tracer;

		/// <summary>
		/// Logger for tracing
		/// </summary>
		readonly ILogger<IssueService> _logger;

		/// <summary>
		/// Accessor for list of cached open issues
		/// </summary>
		public IReadOnlyList<IIssueDetails> CachedOpenIssues => _cachedIssues;

		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IRcaEnqueuer? _rcaEnqueuer;

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueService(JobService jobService, IIssueCollection issueCollection, ICommitService commitService, IJobStepRefCollection jobStepRefs, IUserCollection userCollection, ILogCollection logCollection, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, Tracer tracer, ILogger<IssueService> logger, IRcaEnqueuer? rcaEnqueuer = null)
		{
			Type[] issueTypes = Assembly.GetExecutingAssembly().GetTypes().Where(x => !x.IsAbstract && typeof(IIssue).IsAssignableFrom(x)).ToArray();
			foreach (Type issueType in issueTypes)
			{
				BsonClassMap.LookupClassMap(issueType);
			}

			// Get all the collections
			_jobService = jobService;
			_issueCollection = issueCollection;
			_commitService = commitService;
			_jobStepRefs = jobStepRefs;			
			_userCollection = userCollection;
			_logCollection = logCollection;
			_clock = clock;
			_ticker = clock.AddTicker<IssueService>(TimeSpan.FromMinutes(1.0), TickAsync, logger);
			_completeSteps = Channel.CreateUnbounded<CompleteStep>(new UnboundedChannelOptions { SingleReader = true });
			_completeStepUpdateTask = new BackgroundTask(UpdateCompleteStepsAsync);
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;
			_rcaEnqueuer = rcaEnqueuer;

			// Find all the issue handler types
			List<(Type Type, IssueHandlerAttribute Attribute)> handlerTypes = new List<(Type, IssueHandlerAttribute)>();
			foreach (Type type in typeof(IssueHandler).Assembly.GetTypes())
			{
				IssueHandlerAttribute? attribute = type.GetCustomAttribute<IssueHandlerAttribute>();
				if (attribute != null)
				{
					handlerTypes.Add((type, attribute));
				}
			}
			_handlerTypes = handlerTypes.ToArray();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_jobService.OnJobStepComplete += OnJobStepComplete;

			_completeStepUpdateTask.Start();
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();

			_completeSteps.Writer.TryComplete();
			await _completeStepUpdateTask.WaitAsync(cancellationToken);

			_jobService.OnJobStepComplete -= OnJobStepComplete;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
			await _completeStepUpdateTask.DisposeAsync();
		}

		void OnJobStepComplete(IJob job, IGraph graph, JobStepBatchId batchId, JobStepId stepId)
		{
			if (job.UpdateIssues)
			{
				_completeSteps.Writer.TryWrite(new CompleteStep(job.Id, batchId, stepId));
			}
		}

		/// <summary>
		/// Periodically update the list of cached open issues
		/// </summary>
		/// <param name="cancellationToken">Token to indicate that the service should stop</param>
		/// <returns>Async task</returns>
		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(TickAsync)}");
			DateTime utcNow = _clock.UtcNow;

			Dictionary<StreamId, HashSet<TemplateId>> newCachedDesktopAlerts = new Dictionary<StreamId, HashSet<TemplateId>>();

			BuildConfig buildConfig = _buildConfig.CurrentValue;

			foreach (StreamConfig streamConfig in buildConfig.Streams)
			{
				HashSet<TemplateId> templates = new HashSet<TemplateId>(streamConfig.Templates.Where(x => x.ShowUgsAlerts).Select(x => x.Id));
				if (templates.Count > 0)
				{
					newCachedDesktopAlerts[streamConfig.Id] = templates;
				}
			}

			_cachedDesktopAlerts = newCachedDesktopAlerts;

			// Resolve any issues that haven't been seen in a week
			List<IIssue> openIssues = (await _issueCollection.FindIssuesAsync(resolved: false, cancellationToken: cancellationToken)).ToList();
			span.SetAttribute("horde.open_issues", openIssues.Count);
			for (int idx = 0; idx < openIssues.Count; idx++)
			{
				IIssue openIssue = openIssues[idx];

				// Do not automatically close quarantined issues
				if (openIssue.QuarantinedByUserId != null)
				{
					continue;
				}

				if (openIssue.LastSeenAt < utcNow - TimeSpan.FromDays(7.0))
				{
					await _issueCollection.TryUpdateIssueAsync(openIssue, null, new UpdateIssueOptions { ResolvedById = IIssue.ResolvedByTimeoutId }, cancellationToken: cancellationToken);
					await UpdateIssueDerivedDataAsync(openIssue, cancellationToken);
					openIssues.RemoveAt(idx--);
					continue;
				}
			}

			// Update any issues that are listed as valid for a stream that no longer exists
			HashSet<StreamId> validStreamIds = new HashSet<StreamId>(buildConfig.Streams.Select(x => x.Id));
			for (int idx = 0; idx < openIssues.Count; idx++)
			{
				IIssue? openIssue = openIssues[idx];
				if (openIssue.Streams.Any(x => !validStreamIds.Contains(x.StreamId)))
				{
					openIssue = await UpdateIssueDerivedDataAsync(openIssue, cancellationToken);
					if (openIssue != null && openIssue.ResolvedAt == null)
					{
						openIssues[idx] = openIssue;
					}
					else
					{
						openIssues.RemoveAt(idx--);
					}
				}
			}

			// Cache the details for any issues that are still open
			List<IIssueDetails> newCachedOpenIssues = new List<IIssueDetails>();
			foreach (IIssue openIssue in openIssues)
			{
				newCachedOpenIssues.Add(await GetIssueDetailsAsync(openIssue, cancellationToken));
			}
			_cachedIssues = newCachedOpenIssues;
		}

		/// <inheritdoc/>
		public async Task<IIssueDetails?> GetCachedIssueDetailsAsync(int issueId, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(GetCachedIssueDetailsAsync)}");
			IIssueDetails? cachedIssue = _cachedIssues.FirstOrDefault(x => x.Issue.Id == issueId);
			if (cachedIssue == null)
			{
				IIssue? issue = await _issueCollection.GetIssueAsync(issueId, cancellationToken);
				if (issue != null)
				{
					cachedIssue = await GetIssueDetailsAsync(issue, cancellationToken);
				}
			}
			return cachedIssue;
		}

		/// <inheritdoc/>
		public async Task<IIssueDetails> GetIssueDetailsAsync(IIssue issue, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(GetIssueDetailsAsync)}");
			
			IUser? owner = issue.OwnerId.HasValue ? await _userCollection.GetCachedUserAsync(issue.OwnerId.Value, cancellationToken) : null;
			IUser? nominatedBy = issue.NominatedById.HasValue ? await _userCollection.GetCachedUserAsync(issue.NominatedById.Value, cancellationToken) : null;
			IUser? resolvedBy = (issue.ResolvedById.HasValue && issue.ResolvedById != IIssue.ResolvedByTimeoutId && issue.ResolvedById != IIssue.ResolvedByUnknownId) ? await _userCollection.GetCachedUserAsync(issue.ResolvedById.Value, cancellationToken) : null;
			IUser? quarantinedBy = issue.QuarantinedByUserId.HasValue ? await _userCollection.GetCachedUserAsync(issue.QuarantinedByUserId.Value, cancellationToken) : null;
			IUser? forceClosedBy = issue.ForceClosedByUserId.HasValue ? await _userCollection.GetCachedUserAsync(issue.ForceClosedByUserId.Value, cancellationToken) : null;
			IUser? rootCauseOwner = issue.RootCauseOwnerId.HasValue ? await _userCollection.GetCachedUserAsync(issue.RootCauseOwnerId.Value, cancellationToken) : null;

			IReadOnlyList<IIssueSpan> spans = await _issueCollection.FindSpansAsync(issue.Id, cancellationToken);
			IReadOnlyList<IIssueStep> steps = await _issueCollection.FindStepsAsync(spans.Select(x => x.Id), cancellationToken);
			IReadOnlyList<IIssueSuspect> suspects = await _issueCollection.FindSuspectsAsync(issue, cancellationToken);

			List<IUser> suspectUsers = new List<IUser>();
			foreach (UserId suspectUserId in suspects.Select(x => x.AuthorId).Distinct())
			{
				IUser? suspectUser = await _userCollection.GetCachedUserAsync(suspectUserId, cancellationToken);
				if (suspectUser != null)
				{
					suspectUsers.Add(suspectUser);
				}
			}

			return new IssueDetails(issue, owner, nominatedBy, resolvedBy, quarantinedBy, forceClosedBy, rootCauseOwner, spans, steps, suspects, suspectUsers, ShowDesktopAlertsForIssue(issue, spans), issue.ExternalIssueKey);
		}

		/// <inheritdoc/>
		public bool ShowDesktopAlertsForIssue(IIssue issue, IReadOnlyList<IIssueSpan> spans)
		{
			_ = issue;

			bool showDesktopAlerts = false;

			HashSet<(StreamId, TemplateId)> checkedTemplates = new HashSet<(StreamId, TemplateId)>();
			foreach (IIssueSpan span in spans)
			{
				if (span.NextSuccess == null && checkedTemplates.Add((span.StreamId, span.TemplateRefId)))
				{
					HashSet<TemplateId>? templates;
					if (_cachedDesktopAlerts.TryGetValue(span.StreamId, out templates) && templates.Contains(span.TemplateRefId))
					{
						showDesktopAlerts = true;
						break;
					}
				}
			}

			return showDesktopAlerts;
		}

		/// <summary>
		/// Updates the state of an issue
		/// </summary>
		/// <param name="id">The current issue id</param>
		/// <param name="summary">New summary for the issue</param>
		/// <param name="description">New description for the issue</param>
		/// <param name="promoted">Whether the issue should be set as promoted</param>
		/// <param name="ownerId">New owner of the issue</param>
		/// <param name="nominatedById">Person that nominated the new owner</param>
		/// <param name="acknowledged">Whether the issue has been acknowledged</param>
		/// <param name="declinedById">Name of the user that has declined this issue</param>
		/// <param name="fixCommitId">Fix changelist for the issue</param>
		/// <param name="fixSystemic">There was a systemic fix for this issue</param>
		/// <param name="rootCommitId">Root changelist for the issue</param>
		/// <param name="rootCauseOwnerId">New owner of the root cause</param>
		/// <param name="rootCauseCategory">Root cause category for the issue</param>
		/// <param name="rootCauseSummary">Root cause summary for the issue</param>
		/// <param name="duplicateIssueId">Duplicate issue id</param>
		/// <param name="resolvedById">Whether the issue has been resolved</param>
		/// <param name="addSpanIds">Add spans to this issue</param>
		/// <param name="removeSpanIds">Remove spans from this issue</param>
		/// <param name="externalIssueKey">Key for external issue tracking</param>
		/// <param name="quarantinedById">User who has quarantined the issue</param>
		/// <param name="forceClosedById">User who has force closed the issue</param>
		/// <param name="initiatedById">User initiating the changes, for auditing purposes</param>
		/// <param name="workflowThreadUrl">The workflow thread created for the issue</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the issue was updated</returns>
		public async Task<bool> UpdateIssueAsync(int id, string? summary = null, string? description = null, bool? promoted = null, UserId? ownerId = null, UserId? nominatedById = null, bool? acknowledged = null, UserId? declinedById = null, CommitId? fixCommitId = null, bool? fixSystemic = null, CommitId? rootCommitId = null, UserId? rootCauseOwnerId = null, string? rootCauseCategory = null, string? rootCauseSummary = null, int? duplicateIssueId = null, UserId? resolvedById = null, List<ObjectId>? addSpanIds = null, List<ObjectId>? removeSpanIds = null, string? externalIssueKey = null, UserId? quarantinedById = null, UserId? forceClosedById = null, UserId? initiatedById = null, Uri? workflowThreadUrl = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(UpdateIssueAsync)}");
			
			IIssue? issue;

			for (; ; )
			{
				issue = await _issueCollection.GetIssueAsync(id, cancellationToken);
				if (issue == null)
				{
					return false;
				}

				issue = await _issueCollection.TryUpdateIssueAsync(issue, initiatedById, new UpdateIssueOptions { UserSummary = summary, Description = description, Promoted = promoted, OwnerId = ownerId ?? resolvedById, NominatedById = nominatedById, DeclinedById = declinedById, Acknowledged = acknowledged, FixCommitId = fixCommitId, FixSystemic = fixSystemic, RootCommitId = rootCommitId, RootCauseOwnerId = rootCauseOwnerId, RootCauseCategory = rootCauseCategory, RootCauseSummary = rootCauseSummary, DuplicateIssueId = duplicateIssueId, ResolvedById = resolvedById, ExcludeSpanIds = removeSpanIds, ExternalIssueKey = externalIssueKey, QuarantinedById = quarantinedById, ForceClosedById = forceClosedById, WorkflowThreadUrl = workflowThreadUrl }, cancellationToken: cancellationToken);
				if (issue != null)
				{
					break;
				}
			}

			await using IAsyncDisposable csLock = await _issueCollection.EnterCriticalSectionAsync(nameof(UpdateIssueAsync));

			List<IIssue> updateIssues = new List<IIssue>();
			updateIssues.Add(issue);

			if (forceClosedById != null)
			{
				IReadOnlyList<IIssueSpan> spans = await _issueCollection.FindSpansAsync(issue.Id, cancellationToken);

				foreach (IIssueSpan span in spans)
				{
					if (span.NextSuccess != null)
					{
						continue;
					}

					IIssueStep? step = span.LastFailure ?? span.FirstFailure;

					if (step == null)
					{
						continue;
					}

					await _issueCollection.TryUpdateSpanAsync(span, newNextSuccess: new NewIssueStepData(step.CommitId, step.Severity, step.JobName, step.JobId, step.BatchId, step.StepId, step.StepTime, step.LogId, step.Annotations, step.PromoteByDefault), cancellationToken: cancellationToken);
				}
			}

			if (addSpanIds != null)
			{
				foreach (ObjectId addSpanId in addSpanIds)
				{
					IIssueSpan? span = await _issueCollection.GetSpanAsync(addSpanId, cancellationToken);
					if (span != null)
					{
						IIssue? oldIssue = await _issueCollection.GetIssueAsync(span.IssueId, cancellationToken);
						updateIssues.Add(oldIssue ?? issue);
						await _issueCollection.TryUpdateSpanAsync(span, newIssueId: issue.Id, cancellationToken: cancellationToken);
					}
				}
			}

			if (removeSpanIds != null)
			{
				foreach (ObjectId removeSpanId in removeSpanIds)
				{
					IIssueSpan? span = await _issueCollection.GetSpanAsync(removeSpanId, cancellationToken);
					if (span != null)
					{
						IIssue newIssue = await FindOrAddIssueForSpanAsync(span, cancellationToken);
						updateIssues.Add(newIssue);
						await _issueCollection.TryUpdateSpanAsync(span, newIssueId: newIssue.Id, cancellationToken: cancellationToken);
					}
				}
			}

			foreach (IIssue updateIssue in updateIssues.GroupBy(x => x.Id).Select(x => x.First()))
			{
				await UpdateIssueDerivedDataAsync(updateIssue, cancellationToken);
			}

			return true;
		}

		async Task UpdateCompleteStepsAsync(CancellationToken cancellationToken)
		{
			try
			{
				while (await _completeSteps.Reader.WaitToReadAsync(cancellationToken))
				{
					if (!_completeSteps.Reader.TryRead(out CompleteStep? completeStep))
					{
						continue;
					}
					
					try
					{
						IJob? job = await _jobService.GetJobAsync(completeStep.JobId, cancellationToken);
						if (job != null)
						{
							await UpdateCompleteStepAsync(job, job.Graph, completeStep.BatchId, completeStep.StepId, CancellationToken.None);
						}
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while updating issues for {JobId}:{BatchId}:{StepId} - {Message}", completeStep.JobId, completeStep.BatchId, completeStep.StepId, ex.Message);
					}
				}
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Fatal error in {MethodName} - background thread stopping", nameof(UpdateCompleteStepsAsync));
				throw;
			}
		}

		internal async Task UpdateCompleteStepAsync(IJob job, IGraph graph, JobStepBatchId batchId, JobStepId stepId, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(UpdateCompleteStepAsync)}");
			span.SetAttribute("jobId", job.Id.ToString());
			span.SetAttribute("batchId", batchId.ToString());
			span.SetAttribute("stepId", stepId.ToString());

			_logger.LogInformation("Updating issues for {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);

			StreamConfig? streamConfig;
			if (!_buildConfig.CurrentValue.TryGetStream(job.StreamId, out streamConfig))
			{
				throw new Exception($"Invalid stream id '{job.StreamId}' on job '{job.Id}'");
			}

			// Find the batch for this event
			IJobStepBatch? batch;
			if (!job.TryGetBatch(batchId, out batch))
			{
				throw new ArgumentException($"Invalid batch id {batchId}");
			}

			// Find the step for this event
			IJobStep? step;
			if (!job.TryGetStep(stepId, out step))
			{
				throw new ArgumentException($"Invalid step id {stepId}");
			}

			span.SetAttribute("logId", step.LogId.ToString());

			// Get all the annotations for this template
			TemplateRefConfig? templateRef;
			if (!streamConfig.TryGetTemplate(job.TemplateId, out templateRef))
			{
				return;
			}

			NodeAnnotations annotations = new NodeAnnotations();
			annotations.Merge(templateRef.Annotations);

			// Get the node associated with this step
			INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
			annotations.Merge(graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx].Annotations);

			// Get the workflow for this step
			WorkflowConfig? workflow = null;
			WorkflowId? workflowId = annotations.WorkflowId;
			if (workflowId != null)
			{
				if (!streamConfig.TryGetWorkflow(workflowId.Value, out workflow))
				{
					_logger.LogWarning("Unable to find workflow {WorkflowId} for {JobId}:{BatchId}:{StepId}", workflowId, job.Id, batchId, stepId);
					return;
				}
				annotations.Merge(workflow.Annotations);
			}

			// If the workflow disables issue creation, bail out now
			if (!(annotations.CreateIssues ?? true))
			{
				_logger.LogInformation("Issue creation for step {JobId}:{BatchId}:{StepId} disabled", job.Id, batchId, stepId);
				return;
			}

			// Gets the events for this step grouped by fingerprint
			HashSet<IssueEventGroupInternal> eventGroups = await GetEventGroupsForStepAsync(job, batch, step, node, annotations, workflow, cancellationToken);

			// Try to update all the events. We may need to restart this due to optimistic transactions, so keep track of any existing spans we do not need to check against.
			await using (IAsyncDisposable issueLock = await _issueCollection.EnterCriticalSectionAsync(nameof(UpdateCompleteStepAsync)))
			{
				HashSet<ObjectId> checkedSpanIds = new HashSet<ObjectId>();
				for (; ; )
				{
					// Get the spans that are currently open
					List<IIssueSpan> openSpans = (await _issueCollection.FindOpenSpansAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, cancellationToken)).ToList();
					_logger.LogDebug("{NumSpans} spans are open in {StreamId} at CL {Change} for template {TemplateId}, node {Node}", openSpans.Count, job.StreamId, job.CommitId, job.TemplateId, node.Name);

					// Add the events to existing issues, and create new issues for everything else
					if (eventGroups.Count > 0)
					{
						if (!await AddEventsToExistingSpansAsync(job, batch, step, eventGroups, openSpans, checkedSpanIds, annotations, job.PromoteIssuesByDefault, cancellationToken))
						{
							continue;
						}
						if (!await AddEventsToNewSpansAsync(streamConfig, job, batch, step, node, openSpans, eventGroups, annotations, job.PromoteIssuesByDefault, cancellationToken))
						{
							continue;
						}
					}

					// Try to update the sentinels for any other open steps
					if (!await TryUpdateSentinelsAsync(openSpans, streamConfig, job, batch, step, cancellationToken))
					{
						continue;
					}

					break;
				}
			}

			IReadOnlyList<IIssue> issues = await Collection.FindIssuesForJobAsync(job, graph, stepId, batchId, cancellationToken: cancellationToken);
			if (issues.Count > 0)
			{
				await _jobStepRefs.UpdateAsync(job.Id, batchId, stepId, issues.Select(i => i.Id).ToList());
			}
		}

		/// <summary>
		/// Gets a set of events for the given step 
		/// </summary>
		/// <param name="job">The job containing the step</param>
		/// <param name="batch">Unique id of the batch</param>
		/// <param name="step">Unique id of the step</param>
		/// <param name="node">The node corresponding to the step</param>
		/// <param name="annotations">Annotations for this node</param>
		/// <param name="workflow">The current workflow if any</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Set of new events</returns>
		async Task<HashSet<IssueEventGroupInternal>> GetEventGroupsForStepAsync(IJob job, IJobStepBatch batch, IJobStep step, INode node, IReadOnlyNodeAnnotations annotations, WorkflowConfig? workflow, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(GetEventGroupsForStepAsync)}");
			
			// Make sure the step has a log file
			if (step.LogId == null)
			{
				throw new ArgumentException($"Step id {step.Id} does not have any log set");
			}

			// Find the log file for this step
			ILog? log = await _logCollection.GetAsync(step.LogId.Value, CancellationToken.None);
			if (log == null)
			{
				throw new ArgumentException($"Unable to retrieve log {step.LogId}");
			}
			
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			// Create the DI container for issue handlers
			ServiceCollection services = new ServiceCollection();
			services.AddSingleton(new IssueHandlerContext(job.StreamId, job.TemplateId, node.Name, buildConfig.IssueChannelGroups, node.Annotations));

			using ServiceProvider serviceProvider = services.BuildServiceProvider();

			// Create the issue handler pipeline
			List<IssueHandler> handlers = new List<IssueHandler>();
			foreach ((Type type, IssueHandlerAttribute attribute) in _handlerTypes)
			{
				if (attribute.Tag == null || workflow?.IssueHandlers?.Contains(attribute.Tag) == true)
				{
					handlers.Add((IssueHandler)ActivatorUtilities.CreateInstance(serviceProvider, type));
				}
			}
			handlers.SortBy(x => -x.Priority);

			// General events which have no significant context
			HashSet<EventId> generalEvents = new HashSet<EventId> { KnownLogEvents.ExitCode };

			// Generate internal issue events from log anchors
			List<ILogAnchor> stepAnchors = await log.GetAnchorsAsync(cancellationToken: cancellationToken);
			List<IssueEventInternal> issueEvents = new List<IssueEventInternal>();

			bool createWarningIssues = annotations.CreateWarningIssues ?? true;

			bool found = false;
			foreach (ILogAnchor stepAnchor in stepAnchors)
			{
				ILogEventData stepEventData = await stepAnchor.GetDataAsync(cancellationToken);

				if (!createWarningIssues && stepEventData.Severity == LogEventSeverity.Warning)
				{
					continue;
				}

				bool general = stepEventData.EventId.HasValue && generalEvents.Contains(stepEventData.EventId.Value);

				// Remove any general events after specific events, as these generate noise in those cases
				if (found && general)
				{
					continue;
				}

				if (!found && !general)
				{
					found = true;
				}

				issueEvents.Add(new IssueEventInternal(stepAnchor, stepEventData));
			}

			// Create all the issue definitions by passing each log event to the handlers in order until one attaches it to an issue
			foreach (IssueEventInternal issueEvent in issueEvents)
			{
				// Attach issue audit log buffers to each issue event
				issueEvent.AuditLogger = new IssueAuditLogger();
				issueEvent.AuditLogger.LogInformation("Logging events from Job {JobId}, step {StepId}", job.Id, step.Id);

				foreach (IssueHandler handler in handlers)
				{
					if (handler.HandleEvent(issueEvent))
					{
						break;
					}
				}
			}

			// Gather all the issue event groups
			List<IssueEventGroup> issues = new List<IssueEventGroup>();
			foreach (IssueHandler handler in handlers)
			{
				issues.AddRange(handler.GetIssues());
			}

			// If the node has an annotation to prevent grouping issues together, update all the fingerprints to match
			string? group = annotations.IssueGroup;
			if (group != null)
			{
				foreach (IssueEventGroup issue in issues)
				{
					issue.Type = $"{issue.Type}:{group}";
					foreach (IssueEvent e in issue.Events)
					{
						e.AuditLogger?.LogDebug("Fingerprint: '{IssueType}' modified by node annotation: '{Group}'", issue.Type, group);
					}
				}
			}

			// Create the internal group objects
			HashSet<IssueEventGroupInternal> eventGroups = new HashSet<IssueEventGroupInternal>();
			foreach (IssueEventGroup issue in issues)
			{
				NewIssueFingerprint fingerprint = new NewIssueFingerprint(issue.Type, issue.SummaryTemplate, issue.ChangeFilter);
				fingerprint.Keys.UnionWith(issue.Keys);
				fingerprint.Metadata.UnionWith(issue.Metadata);

				IssueEventGroupInternal internalGroup = new IssueEventGroupInternal(fingerprint);
				internalGroup.Events.AddRange(issue.Events.Select(x => (IssueEventInternal)x));

				eventGroups.Add(internalGroup);
			}

			// Print the list of new events
			_logger.LogInformation("UpdateCompleteStep({JobId}, {BatchId}, {StepId}): {NumEvents} events, {NumFingerprints} unique fingerprints", job.Id, batch.Id, step.Id, stepAnchors.Count, eventGroups.Count);
			foreach (IssueEventGroupInternal eventGroup in eventGroups)
			{
				_logger.LogInformation("Group {Digest}: Type '{FingerprintType}', keys '{FingerprintKeys}', {NumEvents} events", eventGroup.Id.ToString(), eventGroup.Fingerprint.Type, String.Join(", ", eventGroup.Fingerprint.Keys), eventGroup.Events.Count);
				foreach (IssueEvent eventItem in eventGroup.Events)
				{
					_logger.LogDebug("Group {Digest}: [{Line}] {Message}", eventGroup.Id.ToString(), eventItem.LineIndex, eventItem.Render());
				}
			}

			return eventGroups;
		}

		/// <summary>
		/// Adds events to open spans
		/// </summary>
		/// <param name="job">The job containing the step</param>
		/// <param name="batch">Unique id of the batch</param>
		/// <param name="step">Unique id of the step</param>
		/// <param name="newEventGroups">Set of events to try to add</param>
		/// <param name="openSpans">List of open spans</param>
		/// <param name="checkedSpanIds">Set of span ids that have been checked</param>
		/// <param name="annotations">Annotations for this step</param>
		/// <param name="promoteByDefault"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the adding completed</returns>
		async Task<bool> AddEventsToExistingSpansAsync(IJob job, IJobStepBatch batch, IJobStep step, HashSet<IssueEventGroupInternal> newEventGroups, IReadOnlyList<IIssueSpan> openSpans, HashSet<ObjectId> checkedSpanIds, IReadOnlyNodeAnnotations? annotations, bool promoteByDefault, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(AddEventsToExistingSpansAsync)}");
			for (int spanIdx = 0; spanIdx < openSpans.Count; spanIdx++)
			{
				IIssueSpan openSpan = openSpans[spanIdx];
				if (!checkedSpanIds.Contains(openSpan.Id))
				{
					// Filter out the events which match the span's fingerprint
					List<IssueEventGroupInternal> matchEventGroups = newEventGroups.Where(x => openSpan.Fingerprint.IsMatch(x.Fingerprint)).ToList();
					if (matchEventGroups.Count > 0)
					{
						// Add the new step data
						NewIssueStepData newFailure = new NewIssueStepData(job, batch, step, GetIssueSeverity(matchEventGroups.SelectMany(x => x.Events)), annotations, promoteByDefault);
						await _issueCollection.AddStepAsync(openSpan.Id, newFailure, cancellationToken);

						// Update the span if this changes the current range
						IIssueSpan? newSpan = openSpan;
						if (newFailure.CommitId <= openSpan.FirstFailure.CommitId || newFailure.CommitId >= openSpan.LastFailure.CommitId)
						{
							newSpan = await _issueCollection.TryUpdateSpanAsync(openSpan, newFailure: newFailure, cancellationToken: cancellationToken);
							if (newSpan == null)
							{
								return false;
							}
							await UpdateIssueDerivedDataAsync(newSpan.IssueId, cancellationToken);
						}

						// Write out all the merged events
						foreach (IssueEventGroupInternal eventGroup in matchEventGroups)
						{
							_logger.LogDebug("Matched fingerprint {Digest} ({NumLogEvents} log events) to span {SpanId}", eventGroup.Id, eventGroup.Events.Count, newSpan.Id);

							foreach (IssueEvent e in eventGroup.Events)
							{
								e.AuditLogger?.LogDebug("Matched {Event} log event to span {SpanId}", e.Render(), newSpan.Id);
							}
						}

						// Assign all the events to the span
						await _logCollection.AddSpanToEventsAsync(matchEventGroups.SelectMany(x => x.Events.Select(x => x.Anchor)), newSpan.Id, cancellationToken);

						// Remove the matches from the set of events
						newEventGroups.ExceptWith(matchEventGroups);

						// Flush audit log buffers in event groups
						ILogger issueLogger = _issueCollection.GetLogger(newSpan.IssueId);
						EmitIssueAuditLogs(issueLogger, matchEventGroups);
					}

					// Add the span id to the list of checked spans, so we don't have to check it again
					checkedSpanIds.Add(openSpan.Id);
				}
			}
			return true;
		}

		/// <summary>
		/// Gets an issue severity from its events
		/// </summary>
		/// <param name="events">Set of event severity values</param>
		/// <returns>Severity of this issue</returns>
		static IssueSeverity GetIssueSeverity(IEnumerable<IssueEvent> events)
		{
			IssueSeverity severity;
			if (events.Any(x => x.Severity >= LogLevel.Error))
			{
				severity = IssueSeverity.Error;
			}
			else if (events.Any(x => x.Severity >= LogLevel.Warning))
			{
				severity = IssueSeverity.Warning;
			}
			else
			{
				severity = IssueSeverity.Unspecified;
			}
			return severity;
		}

		/// <summary>
		/// Dedupes and emits all buffered audit log entries in the event groups
		/// </summary>
		/// <param name="auditLogger">Issue audit log chanel to log to</param>
		/// <param name="eventGroups">List of event groups to emit buffered log entries for</param>
		static void EmitIssueAuditLogs(ILogger auditLogger, List<IssueEventGroupInternal> eventGroups)
		{
			HashSet<IssueAuditLogEntry> uniqueEntries = new HashSet<IssueAuditLogEntry>();
			foreach (IssueEventGroupInternal eventGroup in eventGroups)
			{
				uniqueEntries.UnionWith(eventGroup.Events.SelectMany(e => (e.AuditLogger != null && e.AuditLogger?.GetEntries().Count > 0) ? e.AuditLogger.GetEntries() : []));
			}

			if (uniqueEntries != null && uniqueEntries.Count > 0)
			{
				foreach (IssueAuditLogEntry e in uniqueEntries)
				{
					#pragma warning disable CA2254 // Template should be a static 
					auditLogger.Log(e.GetLogLevel(), e.GetMessageTemplate(), e.GetArgs() ?? []);
					#pragma warning restore CA2254 // Template should be a static 
				}
			}
		}

		/// <summary>
		/// Adds new spans for the given events
		/// </summary>
		/// <param name="streamConfig">The stream containing the job</param>
		/// <param name="job">The job instance</param>
		/// <param name="batch">The job batch</param>
		/// <param name="step">The job step</param>
		/// <param name="node">Node run in the step</param>
		/// <param name="openSpans">List of open spans. New issues will be added to this list.</param>
		/// <param name="newEventGroups">Set of remaining events</param>
		/// <param name="annotations"></param>
		/// <param name="promoteByDefault"></param>
		/// <param name="cancellationToken"></param>
		/// <returns>True if all events were added</returns>
		async Task<bool> AddEventsToNewSpansAsync(StreamConfig streamConfig, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<IIssueSpan> openSpans, HashSet<IssueEventGroupInternal> newEventGroups, IReadOnlyNodeAnnotations? annotations, bool promoteByDefault, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(AddEventsToNewSpansAsync)}");
			while (newEventGroups.Count > 0)
			{
				// Keep track of the event groups we merge together
				List<IssueEventGroupInternal> sourceEventGroups = new List<IssueEventGroupInternal>();
				sourceEventGroups.Add(newEventGroups.First());

				// Take the first event, and find all other events that match against it
				IssueEventGroupInternal eventGroup = sourceEventGroups[0];
				foreach (IssueEventGroupInternal otherEventGroup in newEventGroups.Skip(1))
				{
					if (otherEventGroup.Fingerprint.IsMatchForNewSpan(eventGroup.Fingerprint))
					{
						IssueEventGroupInternal newEventGroup = eventGroup.MergeWith(otherEventGroup);
						_logger.LogDebug("Merging group {Group} with group {OtherGroup} to form {NewGroup}", eventGroup.Id.ToString(), otherEventGroup.Id.ToString(), newEventGroup.Id.ToString());
						foreach (IssueEventInternal e in eventGroup.Events)
						{
							e.AuditLogger?.LogDebug("Group {Group} merged with group {OtherGroup} to form group {NewGroup} based on matching fingerprint: '{Fingerprint}'", eventGroup.Id.ToString(), otherEventGroup.Id.ToString(), newEventGroup.Id.ToString(), eventGroup.Fingerprint);
						}
			
						sourceEventGroups.Add(otherEventGroup);
						eventGroup = newEventGroup;
					}
				}

				// Get the step data
				NewIssueStepData stepData = new NewIssueStepData(job, batch, step, GetIssueSeverity(eventGroup.Events), annotations, promoteByDefault);

				// Get the span data
				NewIssueSpanData spanData = new NewIssueSpanData(streamConfig.Id, streamConfig.Name, job.TemplateId, node.Name, eventGroup.Fingerprint, stepData);

				// References to events, and thus their audit loggers, are lost when we create spans, so we take first audit logger in event group and use that downstream. Doesn't matter which as they're all emitted to same issue audit log.
				ILogger? groupAuditLogger = eventGroup.Events.FirstOrDefault(e => e.AuditLogger != null)?.AuditLogger;

				IJobStepRef? prevJob = await _jobStepRefs.GetPrevStepForNodeAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, JobStepOutcome.Success, true);
				if (prevJob != null)
				{
					spanData.LastSuccess = new NewIssueStepData(prevJob);
					(spanData.Suspects, spanData.MaxSuspectRank) = await FindSuspectsForSpanAsync(streamConfig, spanData.Fingerprint, spanData.LastSuccess.CommitId, spanData.LastSuccess.StepTime, spanData.FirstFailure.CommitId, spanData.FirstFailure.StepTime, groupAuditLogger, cancellationToken);
				}

				IJobStepRef? nextJob = await _jobStepRefs.GetNextStepForNodeAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, JobStepOutcome.Success, true);
				if (nextJob != null)
				{
					spanData.NextSuccess = new NewIssueStepData(nextJob);
				}

				// Add all the new objects
				IIssue newIssue = await FindOrAddIssueForSpanAsync(spanData, groupAuditLogger, cancellationToken);
				IIssueSpan newSpan = await _issueCollection.AddSpanAsync(newIssue.Id, spanData, cancellationToken);
				IIssueStep newStep = await _issueCollection.AddStepAsync(newSpan.Id, stepData, cancellationToken);

				// Flush audit log buffers in event group
				ILogger issueLogger = _issueCollection.GetLogger(newIssue.Id);
				EmitIssueAuditLogs(issueLogger, [eventGroup]);

				await UpdateIssueDerivedDataAsync(newIssue, cancellationToken);

				// Update the log events
				_logger.LogDebug("Created new span {SpanId} from event group {Group}", newSpan.Id, eventGroup.Id.ToString());
				await _logCollection.AddSpanToEventsAsync(eventGroup.Events.Select(x => x.Anchor), newSpan.Id, cancellationToken);

				// Remove the events from the remaining list of events to match
				newEventGroups.ExceptWith(sourceEventGroups);
				openSpans.Add(newSpan);
			}
			return true;
		}

		internal Task<IIssue?> UpdateDerivedDataAsync(int issueId, CancellationToken cancellationToken)
			=> UpdateIssueDerivedDataAsync(issueId, cancellationToken);

		async Task<IIssue?> UpdateIssueDerivedDataAsync(int issueId, CancellationToken cancellationToken)
		{
			IIssue? issue = await _issueCollection.GetIssueAsync(issueId, cancellationToken);
			if (issue != null)
			{
				issue = await UpdateIssueDerivedDataAsync(issue, cancellationToken);
			}
			return issue;
		}

		static void GetChangeMap(IIssueSpan span, Dictionary<CommitId, CommitIdWithOrder> originChangeToSuspectChange)
		{
			foreach (IIssueSpanSuspect suspect in span.Suspects)
			{
				originChangeToSuspectChange[suspect.SourceCommitId ?? suspect.CommitId] = suspect.CommitId;
			}
		}

		static int CompareSpans(Dictionary<CommitId, CommitIdWithOrder> originChangeToSuspectChange, IIssueSpan span)
		{
			foreach (IIssueSpanSuspect suspect in span.Suspects)
			{
				CommitId originChange = suspect.SourceCommitId ?? suspect.CommitId;
				if (originChangeToSuspectChange.TryGetValue(originChange, out CommitIdWithOrder? suspectChange) && suspectChange != suspect.CommitId)
				{
					return suspectChange.Order - suspect.CommitId.Order;
				}
			}
			return 0;
		}

		internal static List<IIssueSpan> FindMergeOriginSpans(IReadOnlyList<IIssueSpan> spans)
		{
			// Determine the stream(s) highest up in the merge hierarchy, as determined by the minimum changelist number
			// for any robomerged change.
			List<IIssueSpan> originSpans = new List<IIssueSpan>();
			if (spans.Count > 0)
			{
				originSpans.Add(spans[0]);

				Dictionary<CommitId, CommitIdWithOrder> originChangeToSuspectChange = new Dictionary<CommitId, CommitIdWithOrder>();
				GetChangeMap(spans[0], originChangeToSuspectChange);

				for (int idx = 1; idx < spans.Count; idx++)
				{
					IIssueSpan span = spans[idx];
					int comparison = CompareSpans(originChangeToSuspectChange, span);
					if (comparison > 0)
					{
						originChangeToSuspectChange.Clear();
						originSpans.Clear();
					}
					if (comparison >= 0)
					{
						GetChangeMap(span, originChangeToSuspectChange);
						originSpans.Add(span);
					}
				}
			}
			return originSpans;
		}

		async Task<IIssue?> UpdateIssueDerivedDataAsync(IIssue issue, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(UpdateIssueDerivedDataAsync)}");
			Dictionary<(StreamId, CommitId), bool> fixChangeCache = new Dictionary<(StreamId, CommitId), bool>();
			for (; ; )
			{
				BuildConfig buildConfig = _buildConfig.CurrentValue;

				// Find all the spans that are attached to the issue
				List<IIssueSpan> spans = (await _issueCollection.FindSpansAsync(issue.Id, cancellationToken)).ToList();

				// Remove any spans for streams that have been deleted
				for (int idx = spans.Count - 1; idx >= 0; idx--)
				{
					if (!buildConfig.TryGetStream(spans[idx].StreamId, out _))
					{
						spans.RemoveAt(idx);
					}
				}

				// Update the suspects for this issue
				List<NewIssueSuspectData> newSuspects = new List<NewIssueSuspectData>();
				if (spans.Count > 0)
				{
					List<IIssueSpan> spansWithSuspects = spans.Where(x => x.LastSuccess != null).ToList();
					if (spansWithSuspects.Count > 0)
					{
						HashSet<CommitId> suspectChanges = new HashSet<CommitId>(spansWithSuspects[0].Suspects.Select(x => (CommitId)(x.SourceCommitId ?? x.CommitId)));
						for (int spanIdx = 1; spanIdx < spansWithSuspects.Count; spanIdx++)
						{
							suspectChanges.IntersectWith(spansWithSuspects[spanIdx].Suspects.Select(x => (CommitId)(x.SourceCommitId ?? x.CommitId)));
						}
						newSuspects = spansWithSuspects[0].Suspects.Where(x => suspectChanges.Contains(x.SourceCommitId ?? x.CommitId)).Select(x => new NewIssueSuspectData(x.AuthorId, x.SourceCommitId ?? x.CommitId)).ToList();
					}
				}

				// Create the combined fingerprint for this issue
				List<NewIssueFingerprint> newFingerprints = new List<NewIssueFingerprint>();
				foreach (IIssueSpan span in spans)
				{
					NewIssueFingerprint? fingerprint = newFingerprints.FirstOrDefault(x => x.Type == span.Fingerprint.Type);
					if (fingerprint == null)
					{
						newFingerprints.Add(new NewIssueFingerprint(span.Fingerprint));
					}
					else
					{
						fingerprint.MergeWith(span.Fingerprint);
					}
				}

				// Find the severity for this issue
				IssueSeverity newSeverity;
				if (spans.Count == 0 || spans.All(x => x.NextSuccess != null))
				{
					newSeverity = issue.Severity;
				}
				else
				{
					newSeverity = spans.Any(x => x.NextSuccess == null && x.LastFailure.Severity == IssueSeverity.Error) ? IssueSeverity.Error : IssueSeverity.Warning;
				}

				// Find the default summary for this issue
				string newSummary;
				if (newFingerprints.Count == 0)
				{
					newSummary = issue.Summary;
				}
				else if (newFingerprints.Count == 1)
				{
					newSummary = GetSummary(newFingerprints[0], newSeverity);
				}
				else
				{
					newSummary = (newSeverity == IssueSeverity.Error) ? "Errors in multiple steps" : "Warnings in multiple steps";
				}

				// Calculate the last time that this issue was seen
				DateTime newLastSeenAt = issue.CreatedAt;
				foreach (IIssueSpan span in spans)
				{
					if (span.LastFailure.StepTime > newLastSeenAt)
					{
						newLastSeenAt = span.LastFailure.StepTime;
					}
				}

				// Calculate the time at which the issue is verified fixed
				DateTime? newVerifiedAt = null;
				if (issue.QuarantinedByUserId == null)
				{
					if (spans.Count == 0)
					{
						newVerifiedAt = issue.VerifiedAt ?? _clock.UtcNow;
					}
					else
					{
						foreach (IIssueSpan span in spans)
						{
							if (span.NextSuccess == null)
							{
								newVerifiedAt = null;
								break;
							}
							else if (newVerifiedAt == null || span.NextSuccess.StepTime < newVerifiedAt.Value)
							{
								newVerifiedAt = span.NextSuccess.StepTime;
							}
						}
					}
				}

				// Get the new resolved timestamp
				DateTime? newResolvedAt = newVerifiedAt;
				if (issue.ResolvedById != null && issue.ResolvedAt != null)
				{
					newResolvedAt = issue.ResolvedAt;
				}

				// Get the new list of streams
				List<NewIssueStream> newStreams = spans.Select(x => x.StreamId).OrderBy(x => x.ToString(), StringComparer.Ordinal).Distinct().Select(x => new NewIssueStream(x)).ToList();
				if (issue.FixCommitId != null)
				{
					foreach (NewIssueStream newStream in newStreams)
					{
						IIssueStream? stream = issue.Streams.FirstOrDefault(x => x.StreamId == newStream.StreamId);
						if (stream != null)
						{
							newStream.ContainsFix = stream.ContainsFix;
						}

						newStream.ContainsFix ??= await ContainsFixChangeAsync(buildConfig, newStream.StreamId, issue.FixCommitId, fixChangeCache, cancellationToken);

						CommitIdWithOrder commitIdWithOrder = await _commitService.GetOrderedAsync(newStream.StreamId, issue.FixCommitId, cancellationToken);
						if (spans.Any(x => x.StreamId == newStream.StreamId && x.LastFailure.CommitId > commitIdWithOrder))
						{
							newStream.FixFailed = true;
						}
					}
				}

				// Find the stream where the error was introduced.
				List<IIssueSpan> mergeOrigins = FindMergeOriginSpans(spans);
				foreach (IIssueSpan mergeOrigin in mergeOrigins)
				{
					NewIssueStream? newStream = newStreams.FirstOrDefault(x => x.StreamId == mergeOrigin.StreamId);
					if (newStream != null)
					{
						newStream.MergeOrigin = true;
					}
				}

				// If a fix changelist has been specified and it's not valid, clear it out
				if (newVerifiedAt == null && newResolvedAt != null)
				{
					if (await HasFixFailedAsync(spans, issue.FixCommitId, newResolvedAt.Value, newStreams, cancellationToken))
					{
						newStreams.ForEach(x => x.ContainsFix = null);
						newResolvedAt = null;
					}
				}

				// Update the issue
				IIssue? newIssue = await _issueCollection.TryUpdateIssueDerivedDataAsync(issue, newSummary, newSeverity, newFingerprints, newStreams, newSuspects, newResolvedAt, newVerifiedAt, newLastSeenAt, cancellationToken);
				if (newIssue != null)
				{
					OnIssueUpdated?.Invoke(newIssue);
					return newIssue;
				}

				// Fetch the issue and try again
				newIssue = await _issueCollection.GetIssueAsync(issue.Id, cancellationToken);
				if (newIssue == null)
				{
					return null;
				}
				issue = newIssue;
			}
		}

		/// <summary>
		/// Determine if an issue should be marked as fix failed
		/// </summary>
		async Task<bool> HasFixFailedAsync(IEnumerable<IIssueSpan> spans, CommitId? fixCommit, DateTime resolvedAt, IReadOnlyList<NewIssueStream> streams, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(HasFixFailedAsync)}");
			span.SetAttribute("horde.fix_commit", fixCommit?.ToString());
			
			foreach (IGrouping<StreamId, IIssueSpan> group in spans.Where(x => x.NextSuccess == null).GroupBy(x => x.StreamId))
			{
				NewIssueStream? stream = streams.FirstOrDefault(x => x.StreamId == group.Key);
				if (stream != null && fixCommit != null && (stream.ContainsFix ?? false))
				{
					CommitIdWithOrder commitIdWithOrder = await _commitService.GetOrderedAsync(stream.StreamId, fixCommit, cancellationToken);
					if (group.Any(x => x.LastFailure.CommitId > commitIdWithOrder))
					{
						return true;
					}
				}
				else
				{
					if (group.Any(x => x.LastFailure.StepTime > resolvedAt + TimeSpan.FromHours(24.0)))
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Figure out if a stream contains a fix changelist
		/// </summary>
		async ValueTask<bool> ContainsFixChangeAsync(BuildConfig buildConfig, StreamId streamId, CommitId fixChange, Dictionary<(StreamId, CommitId), bool> cachedContainsFixChange, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(ContainsFixChangeAsync)}");
			bool containsFixChange;
			if (!cachedContainsFixChange.TryGetValue((streamId, fixChange), out containsFixChange))
			{
				StreamConfig? streamConfig;
				if (buildConfig.TryGetStream(streamId, out streamConfig))
				{
					_logger.LogInformation("Querying fix changelist {FixChange} in {StreamId}", fixChange, streamId);
					ICommit? change = await _commitService.GetCollection(streamConfig).FindAsync(fixChange, fixChange, 1, cancellationToken: cancellationToken).FirstOrDefaultAsync(cancellationToken);
					containsFixChange = change != null;
				}
				cachedContainsFixChange[(streamId, fixChange)] = containsFixChange;
			}
			return containsFixChange;
		}

		/// <summary>
		/// Maximum number of files to look at when looking for suspects
		/// </summary>
		const int MaxFilesForSuspects = 100;

		/// <summary>
		/// Find suspects that can match a given span
		/// </summary>
		/// <param name="streamConfig">The stream name</param>
		/// <param name="fingerprint">The fingerprint for the span</param>
		/// <param name="lastSuccess">The last successful commit</param>
		/// <param name="lastSuccessTime">The time of the last successful commit</param>
		/// <param name="firstFailure">The first failing commit</param>
		/// <param name="firstFailureTime">The time of the first failing commit</param>
		/// <param name="issueLogger">The audit logger for the issue that the span is associated with</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of suspects</returns>
		async Task<(List<NewIssueSpanSuspectData> Suspects, int? MaxRank)> FindSuspectsForSpanAsync(StreamConfig streamConfig, IIssueFingerprint fingerprint, CommitId lastSuccess, DateTime lastSuccessTime, CommitId firstFailure, DateTime firstFailureTime, ILogger? issueLogger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(FindSuspectsForSpanAsync)}");

			List<NewIssueSpanSuspectData> suspects = new List<NewIssueSpanSuspectData>();
			int maxRank = -1;

			try
			{
				if (!String.IsNullOrEmpty(fingerprint.ChangeFilter))
				{
					if (lastSuccessTime == default || firstFailureTime == default)
					{
						LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Debug)], "Skipping IssueFindSuspectsMaxSuccessDays check, due to default time value for commit {DefaultTimeCommit}.", lastSuccessTime == default ? lastSuccess.Name : firstFailure.Name);
					}
					else
					{
						TimeSpan delta = firstFailureTime - lastSuccessTime;

						if (Math.Abs(delta.TotalDays) >= _buildConfig.CurrentValue.IssueFindSuspectsMaxSuccessDays)
						{
							LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Debug)], "Ignoring commits for {StreamName}: {LastSuccess} and {FirstFailure}, delta is {DeltaDays}", streamConfig.Name, lastSuccess.Name, firstFailure.Name, delta.TotalDays);
							return (suspects, null);
						}
					}

					LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)], "Querying for changes in {StreamName} between {LastSuccess} and {FirstFailure}", streamConfig.Name, lastSuccess, firstFailure);

					// Use fingerprint type to get commit tags and narrow search window
					List<CommitTag>? tags = null;
					if (fingerprint.Type.StartsWith("Compile", StringComparison.OrdinalIgnoreCase))
					{
						tags = [CommitTag.Code];
					}

					// Get the submitted changes before this job
					int maxChanges = _buildConfig.CurrentValue.IssueFindSuspectsMaxChanges;
					List<ICommit> changes;

					if (_buildConfig.CurrentValue.IssueFindSuspectsEnableBidirectionalQuery)
					{
						// Bidirectional query: Get commits from both ends of the range to capture
						// both recent commits (likely to fix) and old commits (likely root cause)
						int halfLimit = maxChanges / 2;

						// Query from firstFailure going backwards (descending - newest first near the failure)
						List<ICommit> recentChanges = await _commitService.GetCollection(streamConfig)
							.FindAsync(lastSuccess, includeMinCommit: false, firstFailure, includeMaxCommit: true, halfLimit, tags, CommitSortOrder.Descending, cancellationToken)
							.ToListAsync(cancellationToken);

						// Query from lastSuccess going forwards (ascending - oldest first near the last success)
						List<ICommit> oldChanges = await _commitService.GetCollection(streamConfig)
							.FindAsync(lastSuccess, includeMinCommit: false, firstFailure, includeMaxCommit: true, halfLimit, tags, CommitSortOrder.Ascending, cancellationToken)
							.ToListAsync(cancellationToken);

						// Merge and deduplicate, maintaining descending order by commit number
						HashSet<int> seenChanges = new HashSet<int>();
						changes = new List<ICommit>();
						foreach (ICommit commit in recentChanges.Concat(oldChanges).OrderByDescending(c => c.Id.Order))
						{
							if (seenChanges.Add(commit.Id.GetPerforceChange()))
							{
								changes.Add(commit);
							}
						}

						LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)],
							"Found {NumResults} changes using bidirectional query (recent: {RecentCount}, old: {OldCount})",
							changes.Count, recentChanges.Count, oldChanges.Count);

						span.SetAttribute("horde.suspects.bidirectional_query", true);
						span.SetAttribute("horde.suspects.recent_commits_count", recentChanges.Count);
						span.SetAttribute("horde.suspects.old_commits_count", oldChanges.Count);
					}
					else
					{
						// Standard single-direction query (descending - newest first)
						// Explicitly pass CommitSortOrder.Descending to ensure original behavior when bidirectional is disabled
						changes = await _commitService.GetCollection(streamConfig).FindAsync(lastSuccess, includeMinCommit: false, firstFailure, includeMaxCommit: true, maxChanges, tags, CommitSortOrder.Descending, cancellationToken).ToListAsync(cancellationToken);
						LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)], "Found {NumResults} changes", changes.Count);
						span.SetAttribute("horde.suspects.bidirectional_query", false);
					}

					// Get all the parameters used to rank suspects
					FileFilter filter = new FileFilter(fingerprint.ChangeFilter.Split(";", StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries));

					// Build a set of all the files to include as suspects
					HashSet<string> suspectFiles = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
					foreach (IssueKey key in fingerprint.Keys)
					{
						if (key.Type == IssueKeyType.File)
						{
							suspectFiles.Add(key.Name);
						}
					}

					if (fingerprint.Metadata != null)
					{
						foreach (IssueMetadata metadata in fingerprint.Metadata)
						{
							if (metadata.Key == "File")
							{
								suspectFiles.Add(metadata.Value);
							}
						}
					}

					// Build a set of 'notes' to include as suspects
					HashSet<string> suspectNotes = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
					foreach (IssueKey key in fingerprint.Keys)
					{
						if (key.Type == IssueKeyType.Note)
						{
							suspectNotes.Add(key.Name);
						}
					}

					// Build a set of symbols to include as suspects
					HashSet<string> suspectSymbols = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
					foreach (IssueKey key in fingerprint.Keys)
					{
						if (key.Type == IssueKeyType.Symbol)
						{
							suspectSymbols.UnionWith(key.Name.Split("::", StringSplitOptions.RemoveEmptyEntries));
						}
					}

					DateTime scoringStart = DateTime.Now;

					// Get the filter for changes to include in the suspect list
					if (_buildConfig.CurrentValue.IssueFindSuspectsEnableParallelScoring)
					{
						// Phase 1: Score all commits in parallel
						int maxParallelism = _buildConfig.CurrentValue.IssueFindSuspectsMaxParallelism;
						using SemaphoreSlim semaphore = new SemaphoreSlim(maxParallelism);
						using CancellationTokenSource timeoutCts = new CancellationTokenSource(TimeSpan.FromSeconds(_buildConfig.CurrentValue.IssueFindSuspectsMaxScoringSeconds));
						using CancellationTokenSource linkedCts = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken, timeoutCts.Token);

						// Track errors to avoid log flooding from parallel tasks
						int scoringErrorCount = 0;

						List<Task<(ICommit commit, int rank)>> scoringTasks = changes.Select(async commit =>
						{
							await semaphore.WaitAsync(linkedCts.Token);
							try
							{
								int rank = -1;
								try
								{
									rank = await GetChangeScoreAsync(commit, filter, suspectFiles, suspectNotes, suspectSymbols, _tracer, issueLogger);
								}
								catch (NullReferenceException)
								{
									Interlocked.Increment(ref scoringErrorCount);
								}
								catch (EpicGames.Perforce.PerforceException perforceEx)
								{
									Interlocked.Increment(ref scoringErrorCount);
									if (!perforceEx.Message.Contains("does not contain any files in", System.StringComparison.OrdinalIgnoreCase))
									{
										throw;
									}
								}
								return (commit, rank);
							}
							finally
							{
								semaphore.Release();
							}
						}).ToList();

						(ICommit commit, int rank)[] scoredCommits;
						try
						{
							scoredCommits = await Task.WhenAll(scoringTasks);
						}
						catch (OperationCanceledException) when (timeoutCts.IsCancellationRequested)
						{
							LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Information)], "Parallel scoring timed out after {ScoringSeconds} seconds", _buildConfig.CurrentValue.IssueFindSuspectsMaxScoringSeconds);
							scoredCommits = scoringTasks
								.Where(t => t.IsCompletedSuccessfully)
								.Select(t => t.Result)
								.ToArray();
						}

						// Log error summary instead of individual errors to avoid log flooding
						if (scoringErrorCount > 0)
						{
							LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Information)], "Parallel scoring encountered {ErrorCount} errors out of {TotalCount} commits", scoringErrorCount, changes.Count);
						}

						// Phase 2: Find max rank and filter (sequential, fast)
						if (scoredCommits.Length > 0)
						{
							maxRank = scoredCommits.Max(x => x.rank);
							if (maxRank >= 0)
							{
								foreach ((ICommit commit, int rank) in scoredCommits.Where(x => x.rank == maxRank))
								{
									NewIssueSpanSuspectData suspect = new NewIssueSpanSuspectData(commit.Id, commit.AuthorId == UserId.Empty || commit.AuthorId == UserId.Anonymous ? commit.OwnerId : commit.AuthorId);
									suspect.SourceCommitId = commit.OriginalCommitId;
									suspects.Add(suspect);
								}
							}
						}

						span.SetAttribute("horde.suspects.parallel_scoring", true);
						span.SetAttribute("horde.suspects.max_parallelism", maxParallelism);
						span.SetAttribute("horde.suspects.scoring_errors", scoringErrorCount);
					}
					else
					{
						// Sequential scoring (original implementation)
						bool earlyExitEnabled = _buildConfig.CurrentValue.IssueFindSuspectsEnableEarlyExit;
						int theoreticalMaxScore = earlyExitEnabled ? CalculateTheoreticalMaxScore(suspectFiles, suspectSymbols) : int.MaxValue;
						bool earlyExitTriggered = false;

						foreach (ICommit commit in changes)
						{
							TimeSpan scoringDuration = DateTime.Now - scoringStart;
							if (scoringDuration.TotalSeconds >= _buildConfig.CurrentValue.IssueFindSuspectsMaxScoringSeconds)
							{
								LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Information)], "Scoring suspect commits took longer than {ScoringSeconds} seconds, giving up", _buildConfig.CurrentValue.IssueFindSuspectsMaxScoringSeconds);
								break;
							}

							int rank = -1;
							try
							{
								rank = await GetChangeScoreAsync(commit, filter, suspectFiles, suspectNotes, suspectSymbols, _tracer, issueLogger);
							}
							catch (NullReferenceException nullRefEx)
							{
								LoggerUtils.MultiLog([(_logger, LogLevel.Error), (issueLogger, LogLevel.Information)], "Exception ranking suspect change {CommitId}: {Message} - {Callstack}", commit.Id, nullRefEx.Message, nullRefEx.StackTrace);

							}
							catch (EpicGames.Perforce.PerforceException perforceEx)
							{
								LoggerUtils.MultiLog([(_logger, LogLevel.Error), (issueLogger, LogLevel.Information)], "Exception ranking suspect change {CommitId}: {Message} - {Callstack}", commit.Id, perforceEx.Message, perforceEx.StackTrace);

								// re-throw any perforce exceptions that aren't known or expected here
								if (!perforceEx.Message.Contains("does not contain any files in", System.StringComparison.OrdinalIgnoreCase))
								{
									throw;
								}
							}
							if (rank > maxRank)
							{
								suspects.Clear();
								maxRank = rank;
							}

							if (rank == maxRank)
							{
								NewIssueSpanSuspectData suspect = new NewIssueSpanSuspectData(commit.Id, commit.AuthorId == UserId.Empty || commit.AuthorId == UserId.Anonymous ? commit.OwnerId : commit.AuthorId);
								suspect.SourceCommitId = commit.OriginalCommitId;
								suspects.Add(suspect);
							}

							// Early exit ONLY if we've achieved the theoretical maximum score
							if (earlyExitEnabled && maxRank >= theoreticalMaxScore && suspects.Count > 0)
							{
								LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)], "Early exit: achieved theoretical max score {MaxRank} with {Count} suspects", maxRank, suspects.Count);
								earlyExitTriggered = true;
								break;
							}
						}

						span.SetAttribute("horde.suspects.parallel_scoring", false);
						span.SetAttribute("horde.suspects.early_exit_enabled", earlyExitEnabled);
						span.SetAttribute("horde.suspects.early_exit_triggered", earlyExitTriggered);
						if (earlyExitEnabled)
						{
							span.SetAttribute("horde.suspects.theoretical_max_score", theoreticalMaxScore);
						}
					}

					// Temporary logging to gather information on how long these are taking
					TimeSpan totalScoringDuration = DateTime.Now - scoringStart;
					int warningDuration = 20;
					if (totalScoringDuration.TotalSeconds >= warningDuration)
					{
						LoggerUtils.MultiLog([(_logger, LogLevel.Warning), (issueLogger, LogLevel.Information)], "Scoring all suspect commits took {TotalScoringSeconds} seconds", totalScoringDuration.TotalSeconds);
					}

					issueLogger?.LogDebug("Max rank of {MaxRank} achieved during suspect finding", maxRank);

					// Add telemetry attributes for suspect finding metrics
					span.SetAttribute("horde.suspects.commits_examined", changes.Count);
					span.SetAttribute("horde.suspects.max_changes_limit", maxChanges);
					span.SetAttribute("horde.suspects.truncated", changes.Count >= maxChanges);
					span.SetAttribute("horde.suspects.found_count", suspects.Count);
					span.SetAttribute("horde.suspects.max_rank", maxRank);
					span.SetAttribute("horde.suspects.scoring_duration_ms", totalScoringDuration.TotalMilliseconds);
					span.SetAttribute("horde.suspects.stream_id", streamConfig.Id.ToString());
					span.SetAttribute("horde.suspects.fingerprint_type", fingerprint.Type);
				}
			}
			catch (Exception ex)
			{
				LoggerUtils.MultiLog([(_logger, LogLevel.Error), (issueLogger, LogLevel.Information)], ex, "Exception finding suspects for {StreamName}: {LastSuccess} and {FirstFailure}", streamConfig.Name, lastSuccess, firstFailure.Name);
			}
			return (suspects, maxRank >= 0 ? maxRank : null);
		}

		static async ValueTask<int> GetChangeScoreAsync(ICommit commit, FileFilter filter, HashSet<string> suspectFiles, HashSet<string> suspectNotes, HashSet<string> suspectSymbols, Tracer tracer, ILogger? issueLogger)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(GetChangeScoreAsync)}");
			
			int rank = -1;

			using CancellationTokenSource cts = new CancellationTokenSource(TimeSpan.FromMinutes(2));
			IReadOnlyList<string> commitFiles = await commit.GetFilesAsync(MaxFilesForSuspects, cts.Token);
			foreach (string commitFile in commitFiles)
			{
				if (filter.Matches(commitFile))
				{
					int fileRank = 0;

					string fileName = commitFile.Substring(commitFile.LastIndexOf('/') + 1);
					if (suspectFiles.Contains(fileName))
					{
						fileRank += 20;
						issueLogger?.LogDebug("Commit {CommitId} suspect rank increased based on file: {File}", commit.Id, fileName);
					}
					else if (suspectNotes.Contains(fileName))
					{
						fileRank += 10;
						issueLogger?.LogDebug("Commit {CommitId} suspect rank increased based on note: {Note}", commit.Id, fileName);
					}

					if (suspectSymbols.Count > 0)
					{
						int matches = suspectSymbols.Count(x => commitFile.Contains(x, StringComparison.OrdinalIgnoreCase));
						if (matches > 0)
						{
							fileRank = Math.Max(fileRank, 10 + (10 * matches));
							fileRank += 10 + (10 * matches);
							issueLogger?.LogDebug("Commit {CommitId} suspect rank increased based on {Matches} symbol matches.", commit.Id, matches);
						}
					}

					rank = Math.Max(rank, fileRank);
				}
			}

			return rank;
		}

		/// <summary>
		/// Calculate the theoretical maximum score for a fingerprint.
		/// Used for early exit optimization when the best possible score has been achieved.
		/// </summary>
		/// <param name="suspectFiles">Set of files to match</param>
		/// <param name="suspectSymbols">Set of symbols to match</param>
		/// <returns>The theoretical maximum score</returns>
		static int CalculateTheoreticalMaxScore(HashSet<string> suspectFiles, HashSet<string> suspectSymbols)
		{
			int maxScore = 0;

			// File match = 20
			if (suspectFiles.Count > 0)
			{
				maxScore = 20;
			}

			// Symbol matches add to score: 10 + (10 * numMatches)
			// Assume all symbols could match in one file
			if (suspectSymbols.Count > 0)
			{
				maxScore += 10 + (10 * suspectSymbols.Count);
			}

			return Math.Max(maxScore, 20); // At least file match level
		}

		/// <summary>
		/// Find an existing issue that may match one of the given suspect changes
		/// </summary>
		/// <param name="span">The span to find an issue for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The matching issue</returns>
		async Task<IIssue> FindOrAddIssueForSpanAsync(IIssueSpan span, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(FindOrAddIssueForSpanAsync)}");
			IReadOnlyList<IIssue> existingIssues;
			if (span.LastSuccess == null) // TODO: Do we want to add 'span.Suspects.Count == 0' condition here too? 
			{
				existingIssues = await _issueCollection.FindIssuesAsync(streamId: span.StreamId, minCommitId: span.FirstFailure.CommitId, maxCommitId: span.FirstFailure.CommitId, resolved: false, cancellationToken: cancellationToken);
			}
			else
			{
				existingIssues = await _issueCollection.FindIssuesForChangesAsync(span.Suspects.ConvertAll(x => x.SourceCommitId ?? x.CommitId), cancellationToken);
			}

			IIssue? issue = existingIssues.FirstOrDefault(x => x.Fingerprints.Any(y => y.IsMatch(span.Fingerprint)));
			if (issue == null)
			{
				string summary = GetSummary(span.Fingerprint, span.FirstFailure.Severity);
				issue = await _issueCollection.AddIssueAsync(summary, cancellationToken);
			}
			return issue;
		}

		/// <summary>
		/// Find an existing issue that may match one of the given suspect changes
		/// </summary>
		/// <param name="span">The span to find an issue for</param>
		/// <param name="issueLogger">The buffered logger used to write to the issue's eventual audit log</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The matching issue</returns>
		async Task<IIssue> FindOrAddIssueForSpanAsync(NewIssueSpanData span, ILogger? issueLogger, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(FindOrAddIssueForSpanAsync)}.NewIssueSpan");
			IReadOnlyList<IIssue> existingIssues;
			if (span.LastSuccess == null || span.Suspects.Count == 0)
			{
				existingIssues = await _issueCollection.FindIssuesAsync(streamId: span.StreamId, minCommitId: span.FirstFailure.CommitId, maxCommitId: span.FirstFailure.CommitId, resolved: false, cancellationToken: cancellationToken);
				LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)], "Found {NumIssues} open issues at {Change} in {StreamId}", existingIssues.Count, span.FirstFailure.CommitId, span.StreamId);
			}
			else
			{
				existingIssues = await _issueCollection.FindIssuesForChangesAsync(span.Suspects.ConvertAll(x => x.SourceCommitId ?? x.CommitId), cancellationToken);
				LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)], "Found {NumIssues} open issues in {StreamId} from [{ChangeList}]", existingIssues.Count, span.StreamId, String.Join(", ", span.Suspects.ConvertAll(x => (x.SourceCommitId ?? x.CommitId).ToString())));
			}

			IIssue? issue = existingIssues.FirstOrDefault(x => x.VerifiedAt == null && x.Fingerprints.Any(y => y.IsMatch(span.Fingerprint)));

			// Fingerprint-based fallback: When no suspects found and no matching issue at the commit,
			// search for recent issues with matching fingerprints to prevent duplicates
			if (issue == null && span.Suspects.Count == 0 && _buildConfig.CurrentValue.IssueFindSuspectsEnableFingerprintFallback)
			{
				issue = await TryFindIssueByFingerprintFallbackAsync(span, issueLogger, cancellationToken);
				if (issue != null)
				{
					tracerSpan.SetAttribute("horde.issue.fingerprint_fallback_matched", true);
				}
			}

			if (issue == null)
			{
				string summary = GetSummary(span.Fingerprint, span.FirstFailure.Severity);
				issue = await _issueCollection.AddIssueAsync(summary, cancellationToken);
				LoggerUtils.MultiLog([(_logger, LogLevel.Information), (issueLogger, LogLevel.Information)], "Created issue {IssueId}", issue.Id);

				if (_rcaEnqueuer != null)
				{
					WorkflowId? workflowId = span.FirstFailure.Annotations?.WorkflowId;
					await _rcaEnqueuer.EnqueueAsync(issue.Id, span.StreamId, workflowId, cancellationToken);
				}
			}
			else
			{
				LoggerUtils.MultiLog([(_logger, LogLevel.Information), (issueLogger, LogLevel.Information)], "Matched to issue {IssueId}", issue.Id);
			}
			return issue;
		}

		/// <summary>
		/// Attempt to find a matching issue by fingerprint when no suspects were found.
		/// This helps prevent duplicate issues when the commit range exceeds MaxChanges.
		/// </summary>
		async Task<IIssue?> TryFindIssueByFingerprintFallbackAsync(NewIssueSpanData span, ILogger? issueLogger, CancellationToken cancellationToken)
		{
			// Don't use fingerprint fallback for generic issue types that match too broadly
			List<string> excludedTypes = _buildConfig.CurrentValue.IssueFingerprintFallbackExcludedTypes;
			if (excludedTypes.Any(t => span.Fingerprint.Type.Equals(t, StringComparison.OrdinalIgnoreCase)))
			{
				LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)],
					"Skipping fingerprint fallback for generic type: {Type}", span.Fingerprint.Type);
				return null;
			}

			// Search for recent unresolved issues with matching fingerprints
			int maxAgeDays = _buildConfig.CurrentValue.IssueFingerprintFallbackMaxAgeDays;
			DateTime cutoffTime = DateTime.UtcNow.AddDays(-maxAgeDays);

			IReadOnlyList<IIssue> recentIssues = await _issueCollection.FindIssuesAsync(
				streamId: span.StreamId,
				resolved: false,
				cancellationToken: cancellationToken);

			// Filter to issues created after cutoff and match fingerprint
			foreach (IIssue candidateIssue in recentIssues)
			{
				if (candidateIssue.CreatedAt < cutoffTime)
				{
					continue;
				}

				if (candidateIssue.VerifiedAt != null)
				{
					continue;
				}

				foreach (IIssueFingerprint fingerprint in candidateIssue.Fingerprints)
				{
					int matchScore = CalculateFingerprintMatchScore(span.Fingerprint, fingerprint);
					if (matchScore >= _buildConfig.CurrentValue.IssueFingerprintFallbackMinScore)
					{
						LoggerUtils.MultiLog([(_logger, LogLevel.Information), (issueLogger, LogLevel.Information)],
							"Fingerprint fallback matched issue {IssueId} with score {Score} (type: {Type})",
							candidateIssue.Id, matchScore, span.Fingerprint.Type);
						return candidateIssue;
					}
				}
			}

			LoggerUtils.MultiLog([(_logger, LogLevel.Debug), (issueLogger, LogLevel.Debug)],
				"No fingerprint fallback match found for type {Type}", span.Fingerprint.Type);
			return null;
		}

		/// <summary>
		/// Calculate a match score between two fingerprints.
		/// Higher scores indicate stronger matches.
		/// </summary>
		static int CalculateFingerprintMatchScore(IIssueFingerprint spanFingerprint, IIssueFingerprint issueFingerprint)
		{
			// Must be the same type
			if (!spanFingerprint.Type.Equals(issueFingerprint.Type, StringComparison.OrdinalIgnoreCase))
			{
				return 0;
			}

			int score = 0;

			// Type match gives base score
			score += 20;

			// Check for matching keys
			HashSet<IssueKey> spanKeys = new HashSet<IssueKey>(spanFingerprint.Keys);
			HashSet<IssueKey> issueKeys = new HashSet<IssueKey>(issueFingerprint.Keys);
			int matchingKeys = spanKeys.Intersect(issueKeys).Count();
			int totalKeys = Math.Max(spanKeys.Count, issueKeys.Count);

			if (totalKeys > 0)
			{
				// More matching keys = higher score
				score += (matchingKeys * 60) / totalKeys;
			}

			// Bonus for exact key match
			if (spanKeys.SetEquals(issueKeys))
			{
				score += 20;
			}

			return score;
		}

		/// <summary>
		/// Gets the summary text for a particular fingerprint
		/// </summary>
		/// <param name="fingerprint">The issue fingerprint</param>
		/// <param name="severity">Severity of the issue</param>
		/// <returns>The summary text</returns>
		static string GetSummary(IIssueFingerprint fingerprint, IssueSeverity severity)
		{
			string template = fingerprint.SummaryTemplate;

			StringBuilder summary = new StringBuilder();
			for (int pos = 0; ;)
			{
				int bracePos = template.IndexOf('{', pos);
				if (bracePos == -1)
				{
					summary.Append(template.AsSpan(pos));
					break;
				}

				int endBracePos = template.IndexOf('}', bracePos + 1);
				if (endBracePos == -1)
				{
					summary.Append(template.AsSpan(pos));
					break;
				}

				summary.Append(template.AsSpan(pos, bracePos - pos));

				ReadOnlySpan<char> placeholderName = template.AsSpan(bracePos + 1, endBracePos - (bracePos + 1));
				if (!TryAppendSummaryVar(placeholderName, fingerprint, severity, summary))
				{
					summary.Append(template.AsSpan(bracePos, endBracePos - bracePos));
				}

				pos = endBracePos + 1;
			}
			return summary.ToString();
		}

		static bool TryAppendSummaryVar(ReadOnlySpan<char> name, IIssueFingerprint fingerprint, IssueSeverity severity, StringBuilder summary)
		{
			if (name.Equals("Severity", StringComparison.OrdinalIgnoreCase))
			{
				if (summary.Length == 0)
				{
					summary.Append((severity == IssueSeverity.Error) ? "Errors" : "Warnings");
				}
				else
				{
					summary.Append((severity == IssueSeverity.Error) ? "errors" : "warnings");
				}
				return true;
			}
			if (name.Equals("Files", StringComparison.OrdinalIgnoreCase))
			{
				string[] files = fingerprint.Keys.Where(x => x.Type == IssueKeyType.File).Select(x => x.Name).ToArray();
				summary.Append(StringUtils.FormatList(files, 3));
				return true;
			}
			if (name.Equals("Nodes", StringComparison.OrdinalIgnoreCase))
			{
				string[] nodes = fingerprint.Keys.Where(x => x.Type == IssueKeyType.File).Select(x => x.Name.Substring(x.Name.LastIndexOf(':') + 1)).ToArray();
				summary.Append(StringUtils.FormatList(nodes, 3));
				return true;
			}
			if (name.Equals("LegacySymbolIssueHandler", StringComparison.OrdinalIgnoreCase))
			{
				summary.Append(GetSymbolIssueSummary(fingerprint, severity));
				return true;
			}

			const string MetaPrefix = "Meta:";
			if (name.StartsWith("Meta:", StringComparison.OrdinalIgnoreCase))
			{
				string[] values = fingerprint.Metadata?.FindValues(name.Slice(MetaPrefix.Length).ToString()).ToArray() ?? Array.Empty<string>();
				summary.Append(StringUtils.FormatList(values, 3));
				return true;
			}

			return false;
		}

		static readonly IssueMetadata s_duplicateEventIdMetadata = new IssueMetadata("EventId", KnownLogEvents.Linker_DuplicateSymbol.Id.ToString());

		static string GetSymbolIssueSummary(IIssueFingerprint fingerprint, IssueSeverity severity)
		{
			HashSet<string> symbols = new HashSet<string>(fingerprint.Keys.Where(x => x.Type == IssueKeyType.Symbol).Select(x => x.Name));
			if (symbols.Count == 0)
			{
				string[] nodes = fingerprint.Metadata?.FindValues("Node").ToArray() ?? Array.Empty<string>();

				StringBuilder summary = new StringBuilder("Linker ");
				summary.Append((severity == IssueSeverity.Warning) ? "warnings" : "errors");
				if (nodes.Length > 0)
				{
					summary.Append($" in {StringUtils.FormatList(nodes, 2)}");
				}

				return summary.ToString();
			}
			else
			{
				string problemType = (fingerprint.Metadata?.Contains(s_duplicateEventIdMetadata) == true) ? "Duplicate" : "Undefined";
				if (symbols.Count == 1)
				{
					return $"{problemType} symbol '{symbols.First()}'";
				}
				else
				{
					return $"{problemType} symbols: {StringUtils.FormatList(symbols.ToArray(), 3)}";
				}
			}
		}

		/// <summary>
		/// Update the sentinels for the given list of issues
		/// </summary>
		/// <param name="spans"></param>
		/// <param name="streamConfig">The stream instance</param>
		/// <param name="job"></param>
		/// <param name="batch"></param>
		/// <param name="step"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		async Task<bool> TryUpdateSentinelsAsync(IReadOnlyList<IIssueSpan> spans, StreamConfig streamConfig, IJob job, IJobStepBatch batch, IJobStep step, CancellationToken cancellationToken)
		{
			using TelemetrySpan tracerSpan = _tracer.StartActiveSpan($"{nameof(IssueService)}.{nameof(TryUpdateSentinelsAsync)}");
			foreach (IIssueSpan span in spans)
			{
				if (job.CommitId < span.FirstFailure.CommitId && (span.LastSuccess == null || job.CommitId > span.LastSuccess.CommitId))
				{
					NewIssueStepData newLastSuccess = new NewIssueStepData(job.CommitId, IssueSeverity.Unspecified, job.Name, job.Id, batch.Id, step.Id, step.StartTimeUtc ?? default, step.LogId, null, false);
					ILogger issueLogger = _issueCollection.GetLogger(span.IssueId);
					(List<NewIssueSpanSuspectData> newSuspects, int? newMaxSuspectRank) = await FindSuspectsForSpanAsync(streamConfig, span.Fingerprint, newLastSuccess.CommitId, newLastSuccess.StepTime, span.FirstFailure.CommitId, span.FirstFailure.StepTime, issueLogger, cancellationToken);

					if (await _issueCollection.TryUpdateSpanAsync(span, newLastSuccess: newLastSuccess, newSuspects: newSuspects, newMaxSuspectRank: newMaxSuspectRank, cancellationToken: cancellationToken) == null)
					{
						return false;
					}

					_logger.LogInformation("Set last success for issue {IssueId}, template {TemplateId}, node {Node} as job {JobId}, cl {Change}", span.IssueId, job.TemplateId, span.NodeName, job.Id, job.CommitId);
					await UpdateIssueDerivedDataAsync(span.IssueId, cancellationToken);
				}
				else if (job.CommitId > span.LastFailure.CommitId && (span.NextSuccess == null || job.CommitId < span.NextSuccess.CommitId))
				{
					IIssue? issue = await _issueCollection.GetIssueAsync(span.IssueId, cancellationToken);
					if (issue == null || issue.QuarantinedByUserId == null)
					{
						NewIssueStepData newNextSucccess = new NewIssueStepData(job.CommitId, IssueSeverity.Unspecified, job.Name, job.Id, batch.Id, step.Id, step.StartTimeUtc ?? default, step.LogId, null, false);

						if (await _issueCollection.TryUpdateSpanAsync(span, newNextSuccess: newNextSucccess, cancellationToken: cancellationToken) == null)
						{
							return false;
						}

						_logger.LogInformation("Set next success for issue {IssueId}, template {TemplateId}, node {Node} as job {JobId}, cl {Change}", span.IssueId, job.TemplateId, span.NodeName, job.Id, job.CommitId);
					}
					else
					{
						// If the issue is quarantined, spans may have been updated by log events, though in the case they aren't
						// need to add a step to the span history
						IReadOnlyList<IIssueStep> steps = await _issueCollection.FindStepsAsync(span.Id, cancellationToken);
						if (steps.FirstOrDefault(x => x.JobId == job.Id && x.StepId == step.Id) == null)
						{
							NewIssueStepData newStep = new NewIssueStepData(job, batch, step, IssueSeverity.Unspecified, null, false);
							await _issueCollection.AddStepAsync(span.Id, newStep, cancellationToken);
							_logger.LogInformation("Adding step to quarantined issue {IssueId}, template {TemplateId}, node {Node} job {JobId} batch {BatchId} step {StepId} cl {Change}", span.IssueId, job.TemplateId, span.NodeName, job.Id, batch.Id, step.Id, job.CommitId);
						}
					}

					await UpdateIssueDerivedDataAsync(span.IssueId, cancellationToken);
				}
			}
			return true;
		}
	}
}
