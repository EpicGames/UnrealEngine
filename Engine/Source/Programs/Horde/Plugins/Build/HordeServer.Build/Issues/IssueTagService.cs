// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using HordeServer.Commits;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;

namespace HordeServer.Issues
{
	/// <summary>
	/// Polls revision control for fix changelist numbers in commits using the syntax '#horde 1234'
	/// </summary>
	public sealed class IssueTagService : IHostedService, IAsyncDisposable
	{
		[SingletonDocument("issue-tags")]
		class State : SingletonBase
		{
			[BsonElement("streams"), BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<StreamId, CommitId> Streams { get; set; } = new Dictionary<StreamId, CommitId>();
		}

		readonly ISingletonDocument<State> _state;
		readonly ICommitService _commitService;
		readonly IIssueCollection _issueCollection;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITicker _ticker;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public IssueTagService(IMongoService mongoService, ICommitService commitService, IIssueCollection issueCollection, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<IssueTagService> logger)
		{
			_state = new SingletonDocument<State>(mongoService);
			_commitService = commitService;
			_issueCollection = issueCollection;
			_buildConfig = buildConfig;
			_ticker = clock.AddSharedTicker<IssueTagService>(TimeSpan.FromSeconds(30.0), TickAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => _ticker.DisposeAsync();

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			_logger.LogDebug("Starting scan for changes with {Tag} tag", _buildConfig.CurrentValue.IssueFixedTag);

			using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
			using IDisposable? listener = _buildConfig.OnChange((_, _) => cancellationSource.Cancel());

			BuildConfig buildConfig = _buildConfig.CurrentValue;

			State initialState = await _state.GetAsync(cancellationSource.Token);
			foreach (StreamId streamId in initialState.Streams.Keys)
			{
				if (!buildConfig.TryGetStream(streamId, out _))
				{
					await _state.UpdateAsync(x => x.Streams.Remove(streamId), cancellationToken);
				}
			}

			if (buildConfig.Streams.Count > 0)
			{
				List<Task> tasks = new List<Task>();
				try
				{
					foreach (StreamConfig streamConfig in buildConfig.Streams)
					{
						tasks.Add(Task.Run(() => TickStreamGuardedAsync(streamConfig, initialState, cancellationSource.Token), cancellationSource.Token));
					}
					await Task.WhenAny(tasks);
				}
				finally
				{
					try
					{
						await cancellationSource.CancelAsync();
						await Task.WhenAll(tasks);
					}
					catch
					{
					}
				}
			}

			_logger.LogDebug("Stopping scan for changes with {Tag} tag.", _buildConfig.CurrentValue.IssueFixedTag);
		}

		async Task TickStreamGuardedAsync(StreamConfig streamConfig, State initialState, CancellationToken cancellationToken)
		{
			try
			{
				await TickStreamAsync(streamConfig, initialState, cancellationToken);
			}
			catch (OperationCanceledException)
			{
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while scanning for {Tag} tags: {Message}", _buildConfig.CurrentValue.IssueFixedTag, ex.Message);
			}
		}

		async ValueTask TickStreamAsync(StreamConfig streamConfig, State initialState, CancellationToken cancellationToken)
		{
			ICommitCollection commits = _commitService.GetCollection(streamConfig);

			CommitId? minCommitId;
			if (!initialState.Streams.TryGetValue(streamConfig.Id, out minCommitId))
			{
				minCommitId = await commits.GetLastCommitIdAsync(cancellationToken);
			}

			await foreach (ICommit commit in commits.SubscribeAsync(minCommitId, null, cancellationToken))
			{
				_logger.LogDebug("Checking commit {Change} in {StreamId}", commit.Id, streamConfig.Id);
				foreach (int issueId in ParseTags(_buildConfig.CurrentValue.IssueFixedTag, commit.Description))
				{
					for (; ; )
					{
						IIssue? issue = await _issueCollection.GetIssueAsync(issueId, cancellationToken);
						if (issue == null)
						{
							_logger.LogInformation("Commit {Change} by {Author} in {StreamId} has invalid issue id {IssueId}", commit.Id, commit.AuthorId, streamConfig.Id, issueId);
							break;
						}

						issue = await _issueCollection.TryUpdateIssueAsync(issue, commit.AuthorId, new UpdateIssueOptions { FixCommitId = commit.Id, ResolvedById = commit.AuthorId }, cancellationToken: cancellationToken);
						if (issue != null)
						{
							_logger.LogInformation("Commit {Change} by {Author} in {StreamId} fixes issue id {IssueId}", commit.Id, commit.AuthorId, streamConfig.Id, issueId);
							break;
						}
					}
				}
				await _state.UpdateAsync(x => x.Streams[streamConfig.Id] = commit.Id, cancellationToken);
			}
		}

		internal static IEnumerable<int> ParseTags(string issueFixedTag, string description)
		{
			if (!Regex.IsMatch(description, @"^\s*#ROBOMERGE-SOURCE", RegexOptions.Multiline))
			{
				foreach (Match match in Regex.Matches(description, $"^\\s*{issueFixedTag}\\s+(.*)$", RegexOptions.Multiline))
				{
					string[] issues = match.Groups[1].Value.Split(new[] { ' ', '\t', ',' }, StringSplitOptions.TrimEntries | StringSplitOptions.RemoveEmptyEntries);
					foreach (string issue in issues)
					{
						if (Int32.TryParse(issue, System.Globalization.NumberStyles.None, null, out int issueId))
						{
							yield return issueId;
						}
					}
				}
			}
		}
	}
}

