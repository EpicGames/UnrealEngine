// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Issues
{
	/// <summary>
	/// Record used for Issue telemetry. Models a <see cref="IIssue"/>.
	/// </summary>
	/// <remarks>Modifications to this record's structure should be versioned appropriately through <see cref="AbstractTelemetryRecord.SchemaVersion"/>, and call sites should continue to publish in a backwards compatible manner.</remarks>
	[AnalyticsTableGen]
	[TelemetryEvent(DefaultEventName)]
	[Table("horde.state_issue", Schema = "ingest")]
	public record IssueTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// Default event name for the IssueTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.Issue";

		/// <summary>
		/// Default context for the Horde Issue Telemetry.
		/// </summary>
		public const string DefaultContext = "Horde";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 4;

		/// <summary>
		/// The context for this issue telemetry.
		/// </summary>
		[Column("context")]
		public string Context { get; init; }

		/// <summary>
		/// The issue ID.
		/// </summary>
		[Column("id")]
		public int Id { get; init; }

		/// <summary>
		/// The time the issue was acknowledged.
		/// </summary>
		[Column("acknowledged_at")]
		public DateTime? AcknowledgedAt { get; init; }

		/// <summary>
		/// The time the issue was created.
		/// </summary>
		[Column("created_at")]
		public DateTime CreatedAt { get; init; }

		/// <summary>
		/// The owner of the issue.
		/// </summary>
		[Column("owner_id")]
		public UserId? OwnerId { get; init; }

		/// <summary>
		/// The external issue key (e.g., Jira ticket).
		/// </summary>
		[Column("external_issue_key")]
		public string? ExternalIssueKey { get; init; }

		/// <summary>
		/// The commit that fixed this issue.
		/// </summary>
		[Column("fix_change")]
		public CommitId? FixChange { get; init; }

		/// <summary>
		/// Whether the fix was systemic.
		/// </summary>
		[Column("fixed_systemic")]
		public bool FixedSystemic { get; init; }

		/// <summary>
		/// The root commit that caused this issue.
		/// </summary>
		[Column("root_change")]
		public CommitId? RootChange { get; init; }

		/// <summary>
		/// The time the issue was last seen.
		/// </summary>
		[Column("last_seen_at")]
		public DateTime LastSeenAt { get; init; }

		/// <summary>
		/// The time the issue was nominated.
		/// </summary>
		[Column("nominated_at")]
		public DateTime? NominatedAt { get; init; }

		/// <summary>
		/// The user who nominated the issue.
		/// </summary>
		[Column("nominated_by_id")]
		public UserId? NominatedById { get; init; }

		/// <summary>
		/// The time the issue was resolved.
		/// </summary>
		[Column("resolved_at")]
		public DateTime? ResolvedAt { get; init; }

		/// <summary>
		/// The user who resolved the issue.
		/// </summary>
		[Column("resolved_by_id")]
		public UserId? ResolvedById { get; init; }

		/// <summary>
		/// The severity of the issue.
		/// </summary>
		[Column("severity")]
		public IssueSeverity Severity { get; init; }

		/// <summary>
		/// The summary of the issue.
		/// </summary>
		[Column("summary")]
		public string Summary { get; init; }

		/// <summary>
		/// The time the issue was verified.
		/// </summary>
		[Column("verified_at")]
		public DateTime? VerifiedAt { get; init; }

		/// <summary>
		/// The ID of the duplicate issue, if any.
		/// </summary>
		[Column("duplicate_issue_id")]
		public int? DuplicateIssueId { get; init; }

		/// <summary>
		/// The owner of the root cause.
		/// </summary>
		[Column("root_cause_owner_id")]
		public UserId? RootCauseOwnerId { get; init; }

		/// <summary>
		/// The category of the root cause.
		/// </summary>
		[Column("root_cause_category")]
		public string? RootCauseCategory { get; init; }

		/// <summary>
		/// The summary of the root cause.
		/// </summary>
		[Column("root_cause_summary")]
		public string? RootCauseSummary { get; init; }

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public IssueTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			Context = DefaultContext;
			Id = 0;
			CreatedAt = DateTime.UtcNow;
			LastSeenAt = DateTime.UtcNow;
			Severity = IssueSeverity.Unspecified;
			Summary = String.Empty;
		}

		/// <summary>
		/// Telemetry payload for <see cref="IIssue"/>.
		/// </summary>
		public IssueTelemetry(IIssue issue) : base(DefaultEventName, CurrentSchemaVersion)
		{
			Context = DefaultContext;
			Id = issue.Id;
			AcknowledgedAt = issue.AcknowledgedAt;
			CreatedAt = issue.CreatedAt;
			OwnerId = issue.OwnerId;
			ExternalIssueKey = issue.ExternalIssueKey;
			FixChange = issue.FixCommitId;
			FixedSystemic = issue.FixSystemic;
			RootChange = issue.RootCommitId;
			LastSeenAt = issue.LastSeenAt;
			NominatedAt = issue.NominatedAt;
			NominatedById = issue.NominatedById;
			ResolvedAt = issue.ResolvedAt;
			ResolvedById = issue.ResolvedById;
			Severity = issue.Severity;
			Summary = issue.Summary;
			VerifiedAt = issue.VerifiedAt;
			DuplicateIssueId = issue.DuplicateIssueId;
			RootCauseOwnerId = issue.RootCauseOwnerId;
			RootCauseCategory = issue.RootCauseCategory;
			RootCauseSummary = issue.RootCauseSummary;
		}
	}

	/// <summary>
	/// Record used for IssueFingerprint telemetry. Models a <see cref="IIssueFingerprint"/>.
	/// </summary>
	/// <remarks>Modifications to this record's structure should be versioned appropriately, and call sites should continue to publish in a backwards compatible manner.</remarks>
	[AnalyticsTableGen]
	[Table("default")]
	public record FingerprintTelemetry
	{
		/// <summary>
		/// The type of the fingerprint.
		/// </summary>
		[Column("type")]
		public string Type { get; init; }

		/// <summary>
		/// The keys associated with the fingerprint.
		/// </summary>
		[Column("keys")]
		public IReadOnlySet<IssueKey> Keys { get; init; }

		/// <summary>
		/// Default constructor for ORM mapping.
		/// </summary>
		public FingerprintTelemetry()
		{
			Type = String.Empty;
			Keys = new HashSet<IssueKey>();
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="type">The type of the fingerprint.</param>
		/// <param name="keys">The keys associated with the fingerprint.</param>
		public FingerprintTelemetry(string type, IReadOnlySet<IssueKey> keys)
		{
			Type = type;
			Keys = keys;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="issueFingerprint">The issueFingerprint to use when constructing the telemetry.</param>
		public FingerprintTelemetry(IIssueFingerprint issueFingerprint)
			: this(issueFingerprint.Type, issueFingerprint.Keys)
		{
		}
	}

	/// <summary>
	/// Record used for FailureInfo telemetry.
	/// </summary>
	/// <remarks>Modifications to this record's structure should be versioned appropriately, and call sites should continue to publish in a backwards compatible manner.</remarks>
	[AnalyticsTableGen]
	[Table("default")]
	public record FailureInfoTelemetry
	{
		/// <summary>
		/// The job ID.
		/// </summary>
		[Column("job_id")]
		public JobId JobId { get; init; }

		/// <summary>
		/// The job name.
		/// </summary>
		[Column("job_name")]
		public string JobName { get; init; }

		/// <summary>
		/// The change/commit information.
		/// </summary>
		[Column("change")]
		public CommitIdWithOrder Change { get; init; }

		/// <summary>
		/// The step ID.
		/// </summary>
		[Column("step_id")]
		public ObjectId StepId { get; init; }

		/// <summary>
		/// Default constructor for ORM mapping.
		/// </summary>
		public FailureInfoTelemetry()
		{
			JobId = JobId.Empty;
			JobName = String.Empty;
			Change = new CommitIdWithOrder(String.Empty, 0);
			StepId = ObjectId.Empty;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="jobId">The job ID.</param>
		/// <param name="jobName">The job name.</param>
		/// <param name="change">The change/commit information.</param>
		/// <param name="stepId">The step ID.</param>
		public FailureInfoTelemetry(JobId jobId, string jobName, CommitIdWithOrder change, ObjectId stepId)
		{
			JobId = jobId;
			JobName = jobName;
			Change = change;
			StepId = stepId;
		}
	}

	/// <summary>
	/// Record used for IssueSpan telemetry. Models a <see cref="IIssueSpan"/>.
	/// </summary>
	/// <remarks>Modifications to this record's structure should be versioned appropriately through <see cref="AbstractTelemetryRecord.SchemaVersion"/>, and call sites should continue to publish in a backwards compatible manner.</remarks>
	[AnalyticsTableGen]
	[TelemetryEvent(DefaultEventName)]
	[Table("horde.state_issue_span", Schema = "ingest")]
	public record IssueSpanTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// Default event name for the IssueSpanTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.IssueSpan";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// The issue span ID.
		/// </summary>
		[Column("id")]
		public ObjectId Id { get; init; }

		/// <summary>
		/// The issue ID this span belongs to.
		/// </summary>
		[Column("issue_id")]
		public int IssueId { get; init; }

		/// <summary>
		/// The fingerprint of the issue span.
		/// </summary>
		[Column("fingerprint")]
		public FingerprintTelemetry Fingerprint { get; init; }

		/// <summary>
		/// The first failure in this span.
		/// </summary>
		[Column("first_failure")]
		public FailureInfoTelemetry FirstFailure { get; init; }

		/// <summary>
		/// The last failure in this span.
		/// </summary>
		[Column("last_failure")]
		public FailureInfoTelemetry? LastFailure { get; init; }

		/// <summary>
		/// The stream ID.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The stream name.
		/// </summary>
		[Column("stream_name")]
		public string StreamName { get; init; }

		/// <summary>
		/// The template ref ID.
		/// </summary>
		[Column("template_ref_id")]
		public TemplateId TemplateRefId { get; init; }

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public IssueSpanTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			Id = MongoDB.Bson.ObjectId.Empty;
			IssueId = 0;
			Fingerprint = new FingerprintTelemetry(String.Empty, new HashSet<IssueKey>());
			FirstFailure = new FailureInfoTelemetry(JobId.Empty, String.Empty, new CommitIdWithOrder(String.Empty, 0), ObjectId.Empty);
			StreamId = new StreamId(String.Empty);
			StreamName = String.Empty;
			TemplateRefId = new TemplateId(String.Empty);
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="issueSpan">The issueSpan to use when constructing the telemetry.</param>
		public IssueSpanTelemetry(IIssueSpan issueSpan) : base(DefaultEventName, CurrentSchemaVersion)
		{
			Id = issueSpan.Id;
			IssueId = issueSpan.IssueId;
			Fingerprint = new FingerprintTelemetry(issueSpan.Fingerprint);
			FirstFailure = new FailureInfoTelemetry(issueSpan.FirstFailure.JobId, issueSpan.FirstFailure.JobName, issueSpan.FirstFailure.CommitId, issueSpan.FirstFailure.SpanId);
			LastFailure = issueSpan.LastFailure != null
				? new FailureInfoTelemetry(issueSpan.LastFailure.JobId, issueSpan.LastFailure.JobName, issueSpan.LastFailure.CommitId, issueSpan.LastFailure.SpanId)
				: null;
			StreamId = issueSpan.StreamId;
			StreamName = issueSpan.StreamName;
			TemplateRefId = issueSpan.TemplateRefId;
		}
	}
}
