// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Issues.External;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;

namespace HordeServer.Issues
{
	/// <summary>
	/// Controller for the /api/v1/issues endpoint
	/// </summary>
	[Authorize]
	[ApiController]
	[Route("[controller]")]
	public class IssuesController : HordeControllerBase
	{
		private readonly IIssueCollection _issueCollection;
		private readonly IssueService _issueService;
		private readonly IExternalIssueService _externalIssueService;
		private readonly JobService _jobService;
		private readonly IUserCollection _userCollection;
		private readonly ILogCollection _logCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly ILogger<IssuesController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public IssuesController(IIssueCollection issueCollection, IssueService issueService, JobService jobService, IUserCollection userCollection, ILogCollection logCollection, IExternalIssueService externalIssueService, IOptionsSnapshot<BuildConfig> buildConfig, ILogger<IssuesController> logger)
		{
			_issueCollection = issueCollection;
			_issueService = issueService;
			_jobService = jobService;
			_userCollection = userCollection;
			_logCollection = logCollection;
			_externalIssueService = externalIssueService;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="ids">Set of issue ids to find</param>
		/// <param name="streamId">The stream to query for</param>
		/// <param name="minChange">The minimum changelist range to query, inclusive</param>
		/// <param name="maxChange">The minimum changelist range to query, inclusive</param>
		/// <param name="resolved">Whether to include resolved issues</param>
		/// <param name="index">Starting offset of the window of results to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v2/issues")]
		[ProducesResponseType(typeof(List<FindIssueResponse>), 200)]
		public async Task<ActionResult<object>> FindIssuesV2Async([FromQuery(Name = "Id")] int[]? ids = null, [FromQuery] StreamId? streamId = null, [FromQuery] CommitId? minChange = null, [FromQuery] CommitId? maxChange = null, [FromQuery] bool? resolved = null, [FromQuery] int index = 0, [FromQuery] int count = 10, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (ids != null && ids.Length == 0)
			{
				ids = null;
			}

			List<object> responses = new List<object>();
			if (streamId != null)
			{
				BuildConfig buildConfig = _buildConfig.Value;

				StreamConfig? streamConfig;
				if (!buildConfig.TryGetStream(streamId.Value, out streamConfig))
				{
					return NotFound(streamId.Value);
				}
				if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
				{
					return Forbid(StreamAclAction.ViewStream, streamId.Value);
				}

				// Get distinct issue ids from spans using aggregate (avoids transferring full span documents)
				IReadOnlyList<int> issueIds = await _issueCollection.FindSpanIssueIdsAsync(streamId.Value, minChange, maxChange, resolved, cancellationToken);
				if (ids != null)
				{
					HashSet<int> spanIssueIds = new HashSet<int>(issueIds);
					issueIds = ids.Where(x => spanIssueIds.Contains(x)).ToArray();
				}

				if (issueIds.Count > 0)
				{
					// Paginate the issues
					IReadOnlyList<IIssue> issues = await _issueCollection.FindIssuesAsync(issueIds, index: index, count: count, cancellationToken: cancellationToken);

					// Fetch spans only for the paginated issues
					IReadOnlyList<IIssueSpan> spans = await _issueCollection.FindSpansAsync(null, issues.Select(x => x.Id), streamId.Value, minChange, maxChange, resolved, cancellationToken: cancellationToken);

					// Group spans by issue id
					Dictionary<int, List<IIssueSpan>> issueIdToSpans = new Dictionary<int, List<IIssueSpan>>();
					foreach (IIssueSpan span in spans)
					{
						if (!issueIdToSpans.TryGetValue(span.IssueId, out List<IIssueSpan>? spansForIssue))
						{
							spansForIssue = new List<IIssueSpan>();
							issueIdToSpans.Add(span.IssueId, spansForIssue);
						}
						spansForIssue.Add(span);
					}

					// Create the corresponding responses
					foreach (IIssue issue in issues.OrderByDescending(x => x.Id))
					{
						IssueSeverity streamSeverity = IssueSeverity.Unspecified;

						HashSet<WorkflowId> openWorkflowIds = new HashSet<WorkflowId>();

						List<FindIssueSpanResponse> spanResponses = new List<FindIssueSpanResponse>();
						if (issueIdToSpans.TryGetValue(issue.Id, out List<IIssueSpan>? spansForIssue))
						{
							// Find the current severity in the stream
							DateTime lastStepTime = DateTime.MinValue;
							foreach (IIssueSpan span in spansForIssue)
							{
								if (span.LastFailure != null && span.LastFailure.StepTime > lastStepTime)
								{
									lastStepTime = span.LastFailure.StepTime;
									streamSeverity = span.LastFailure.Severity;
								}
							}

							// Convert each issue to a response
							foreach (IIssueSpan span in spansForIssue)
							{
								spanResponses.Add(NewFindIssueSpanResponse(span, span.LastFailure.Annotations.WorkflowId));
							}

							// Find which workflows are affected
							foreach (IIssueSpan span in spansForIssue)
							{
								WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
								if (workflowId != null && span.NextSuccess == null && span.StreamId == streamId)
								{
									openWorkflowIds.Add(workflowId.Value);
								}
							}
						}

						IUser? owner = null;
						IUser? nominatedBy = null;
						IUser? resolvedBy = null;
						IUser? quarantinedBy = null;
						IUser? rootCauseOwner = null;

						if (issue.OwnerId != null)
						{
							owner = await _userCollection.GetCachedUserAsync(issue.OwnerId.Value, cancellationToken);
						}

						if (issue.NominatedById != null)
						{
							nominatedBy = await _userCollection.GetCachedUserAsync(issue.NominatedById.Value, cancellationToken);
						}

						if (issue.ResolvedById != null)
						{
							resolvedBy = await _userCollection.GetCachedUserAsync(issue.ResolvedById.Value, cancellationToken);
						}

						if (issue.QuarantinedByUserId != null)
						{
							quarantinedBy = await _userCollection.GetCachedUserAsync(issue.QuarantinedByUserId.Value, cancellationToken);
						}

						if (issue.RootCauseOwnerId != null)
						{
							rootCauseOwner = await _userCollection.GetCachedUserAsync(issue.RootCauseOwnerId.Value, cancellationToken);
						}

						FindIssueResponse response = NewFindIssueResponse(issue, owner, nominatedBy, resolvedBy, quarantinedBy, rootCauseOwner, streamSeverity, spanResponses, openWorkflowIds.ToList());
						responses.Add(PropertyFilter.Apply(response, filter));
					}
				}
			}
			else
			{
				return BadRequest("Missing StreamId on request");
			}

			return responses;
		}

		static FindIssueSpanResponse NewFindIssueSpanResponse(IIssueSpan span, WorkflowId? workflowId)
		{
			FindIssueSpanResponse response = new FindIssueSpanResponse();
			response.Id = span.Id.ToString();
			response.TemplateId = span.TemplateRefId.ToString();
			response.Name = span.NodeName;
			response.WorkflowId = workflowId;
			if (span.LastSuccess != null)
			{
				response.LastSuccess = NewGetIssueStepResponse(span.LastSuccess);
			}
			if (span.NextSuccess != null)
			{
				response.NextSuccess = NewGetIssueStepResponse(span.NextSuccess);
			}
			return response;
		}

		static GetIssueStepResponse NewGetIssueStepResponse(IIssueStep issueStep)
		{
			GetIssueStepResponse response = new GetIssueStepResponse();
			response.CommitId = issueStep.CommitId;
			response.Severity = issueStep.Severity;
			response.JobName = issueStep.JobName;
			response.JobId = issueStep.JobId;
			response.BatchId = issueStep.BatchId;
			response.StepId = issueStep.StepId;
			response.StepTime = issueStep.StepTime;
			response.LogId = issueStep.LogId;
			return response;
		}

		/// <summary>
		/// Constructs a new issue
		/// </summary>
		/// <param name="issue">The isseu information</param>
		/// <param name="owner">Owner of the issue</param>
		/// <param name="nominatedBy">User that nominated the current fixer</param>
		/// <param name="resolvedBy">User that resolved the issue</param>
		/// <param name="quarantinedBy">User that quarantined the issue</param>
		/// <param name="rootCauseOwner">Owner of the root cause analysis</param>
		/// <param name="streamSeverity">The current severity in the stream</param>
		/// <param name="spans">Spans for this issue</param>
		/// <param name="openWorkflows">List of workflows for which this issue is open</param>
		static FindIssueResponse NewFindIssueResponse(IIssue issue, IUser? owner, IUser? nominatedBy, IUser? resolvedBy, IUser? quarantinedBy, IUser? rootCauseOwner, IssueSeverity? streamSeverity, List<FindIssueSpanResponse> spans, List<WorkflowId> openWorkflows)
		{
			FindIssueResponse response = new FindIssueResponse();
			response.Id = issue.Id;
			response.CreatedAt = issue.CreatedAt;
			response.RetrievedAt = DateTime.UtcNow;
			response.Summary = String.IsNullOrEmpty(issue.UserSummary) ? issue.Summary : issue.UserSummary;
			response.Description = issue.Description;
			response.Severity = issue.Severity;
			response.StreamSeverity = streamSeverity;
			response.Promoted = issue.Promoted;
			if (owner != null)
			{
				response.Owner = owner.ToThinApiResponse();
			}
			if (nominatedBy != null)
			{
				response.NominatedBy = nominatedBy.ToThinApiResponse();
			}
			response.AcknowledgedAt = issue.AcknowledgedAt;
			response.FixCommitId = issue.FixCommitId;
			response.FixSystemic = issue.FixSystemic;
			response.RootCommitId = issue.RootCommitId;
			if (rootCauseOwner != null)
			{
				response.RootCauseOwner = rootCauseOwner.ToThinApiResponse();
			}
			response.RootCauseCategory = issue.RootCauseCategory;
			response.RootCauseSummary = issue.RootCauseSummary;
			response.DuplicateIssueId = issue.DuplicateIssueId;
			response.ResolvedAt = issue.ResolvedAt;
			if (resolvedBy != null)
			{
				response.ResolvedBy = resolvedBy.ToThinApiResponse();
			}
			response.VerifiedAt = issue.VerifiedAt;
			response.LastSeenAt = issue.LastSeenAt;
			response.Spans = spans;
			response.ExternalIssueKey = issue.ExternalIssueKey;
			if (quarantinedBy != null)
			{
				response.QuarantinedBy = quarantinedBy.ToThinApiResponse();
				response.QuarantineTimeUtc = issue.QuarantineTimeUtc;
			}
			response.WorkflowThreadUrl = issue.WorkflowThreadUrl;
			response.OpenWorkflows = openWorkflows;
			return response;
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="ids">Set of issue ids to find</param>
		/// <param name="streamId">The stream to query for</param>
		/// <param name="change">The changelist to query</param>
		/// <param name="minChange">The minimum changelist range to query, inclusive</param>
		/// <param name="maxChange">The minimum changelist range to query, inclusive</param>
		/// <param name="jobId">Job id to filter by</param>
		/// <param name="batchId">The batch to filter by</param>
		/// <param name="stepId">The step to filter by</param>
		/// <param name="labelIdx">The label within the job to filter by</param>
		/// <param name="ownerId">User to filter issues for</param>
		/// <param name="resolved">Whether to include resolved issues</param>
		/// <param name="promoted">Whether to include promoted issues</param>
		/// <param name="index">Starting offset of the window of results to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/issues")]
		[ProducesResponseType(typeof(List<GetIssueResponse>), 200)]
		public async Task<ActionResult<object>> FindIssuesAsync([FromQuery(Name = "Id")] int[]? ids = null, [FromQuery] string? streamId = null, [FromQuery] CommitId? change = null, [FromQuery] CommitId? minChange = null, [FromQuery] CommitId? maxChange = null, [FromQuery] JobId? jobId = null, [FromQuery] JobStepBatchId? batchId = null, [FromQuery] JobStepId? stepId = null, [FromQuery(Name = "label")] int? labelIdx = null, [FromQuery] string? ownerId = null, [FromQuery] bool? resolved = null, [FromQuery] bool? promoted = null, [FromQuery] int index = 0, [FromQuery] int count = 10, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (ids != null && ids.Length == 0)
			{
				ids = null;
			}

			UserId? ownerIdValue = null;
			if (ownerId != null)
			{
				ownerIdValue = UserId.Parse(ownerId);
			}

			IReadOnlyList<IIssue> issues;
			if (jobId == null)
			{
				StreamId? streamIdValue = null;
				if (streamId != null)
				{
					streamIdValue = new StreamId(streamId);
				}

				issues = await _issueService.Collection.FindIssuesAsync(ids, ownerIdValue, streamIdValue, minChange ?? change, maxChange ?? change, resolved, promoted, index, count, cancellationToken);
			}
			else
			{
				IJob? job = await _jobService.GetJobAsync(jobId.Value, cancellationToken);
				if (job == null)
				{
					return NotFound(jobId.Value);
				}
				if (!_buildConfig.Value.Authorize(job, JobAclAction.ViewJob, User))
				{
					return Forbid(JobAclAction.ViewJob, jobId.Value);
				}

				IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
				issues = await _issueService.Collection.FindIssuesForJobAsync(job, graph, stepId, batchId, labelIdx, ownerIdValue, resolved, promoted, index, count, cancellationToken);
			}

			List<object> responses = new List<object>();
			foreach (IIssue issue in issues)
			{
				IIssueDetails details = await _issueService.GetIssueDetailsAsync(issue, cancellationToken);
				if (AuthorizeIssue(details))
				{
					bool showDesktopAlerts = _issueService.ShowDesktopAlertsForIssue(issue, details.Spans);
					GetIssueResponse response = CreateIssueResponse(details, showDesktopAlerts);
					responses.Add(PropertyFilter.Apply(response, filter));
				}
			}
			return responses;
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/issues/{issueId}")]
		[ProducesResponseType(typeof(GetIssueResponse), 200)]
		public async Task<ActionResult<object>> GetIssueAsync(int issueId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IIssueDetails? details = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (details == null)
			{
				return NotFound();
			}
			if (!AuthorizeIssue(details))
			{
				return Forbid();
			}

			bool showDesktopAlerts = _issueService.ShowDesktopAlertsForIssue(details.Issue, details.Spans);
			return PropertyFilter.Apply(CreateIssueResponse(details, showDesktopAlerts), filter);
		}

		/// <summary>
		/// Retrieve historical information about a specific issue
		/// </summary>
		/// <param name="issueId">Id of the agent to get information about</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/issues/{issueId}/history")]
		public async Task GetAgentHistoryAsync(int issueId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50)
		{
			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync();
			await _issueCollection.GetLogger(issueId).FindAsync(Response.BodyWriter, minTime, maxTime, index, count);
		}

		/// <summary>
		/// Create an issue response object
		/// </summary>
		/// <param name="details"></param>
		/// <param name="showDesktopAlerts"></param>
		/// <returns></returns>
		GetIssueResponse CreateIssueResponse(IIssueDetails details, bool showDesktopAlerts)
		{
			List<GetIssueAffectedStreamResponse> affectedStreams = new List<GetIssueAffectedStreamResponse>();
			foreach (IGrouping<StreamId, IIssueSpan> streamSpans in details.Spans.GroupBy(x => x.StreamId))
			{
				try
				{
					_buildConfig.Value.TryGetStream(streamSpans.Key, out StreamConfig? streamConfig);
					affectedStreams.Add(NewGetIssueAffectedStreamResponse(details, streamConfig, streamSpans));
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Unable to get {StreamId} for span key on issue {IssueId}", streamSpans.Key, details.Issue.Id);
				}
			}
			return NewGetIssueResponse(details, affectedStreams, showDesktopAlerts);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="details">Issue to construct from</param>
		/// <param name="streamConfig"></param>
		/// <param name="spans">The spans to construct from</param>
		static GetIssueAffectedStreamResponse NewGetIssueAffectedStreamResponse(IIssueDetails details, StreamConfig? streamConfig, IEnumerable<IIssueSpan> spans)
		{
			GetIssueAffectedStreamResponse response = new GetIssueAffectedStreamResponse();

			IIssueSpan firstSpan = spans.First();
			response.StreamId = firstSpan.StreamId;
			response.StreamName = firstSpan.StreamName;
			response.Resolved = spans.All(x => x.NextSuccess != null);

			response.AffectedTemplates = new List<GetIssueAffectedTemplateResponse>();
			foreach (IGrouping<TemplateId, IIssueSpan> template in spans.GroupBy(x => x.TemplateRefId))
			{
				string templateName = template.Key.ToString();
				if (streamConfig != null && streamConfig.TryGetTemplate(template.Key, out TemplateRefConfig? templateRefConfig))
				{
					templateName = templateRefConfig.Name;
				}

				HashSet<ObjectId> unresolvedTemplateSpans = new HashSet<ObjectId>(template.Where(x => x.NextSuccess == null).Select(x => x.Id));

				IIssueStep? templateStep = details.Steps.Where(x => unresolvedTemplateSpans.Contains(x.SpanId)).OrderByDescending(x => x.StepTime).FirstOrDefault();

				response.AffectedTemplates.Add(NewGetIssueAffectedTemplateResponse(template.Key, templateName, template.All(x => x.NextSuccess != null), templateStep?.Severity ?? IssueSeverity.Unspecified));
			}

			HashSet<TemplateId> templateIdsSet = new HashSet<TemplateId>(spans.Select(x => x.TemplateRefId));
			response.TemplateIds = templateIdsSet.Select(x => x.ToString()).ToList();

			HashSet<TemplateId> unresolvedTemplateIdsSet = new HashSet<TemplateId>(spans.Where(x => x.NextSuccess == null).Select(x => x.TemplateRefId));
			response.UnresolvedTemplateIds = unresolvedTemplateIdsSet.Select(x => x.ToString()).ToList();
			response.ResolvedTemplateIds = templateIdsSet.Except(unresolvedTemplateIdsSet).Select(x => x.ToString()).ToList();

			return response;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="templateId"></param>
		/// <param name="templateName"></param>
		/// <param name="resolved"></param>
		/// <param name="severity"></param>
		static GetIssueAffectedTemplateResponse NewGetIssueAffectedTemplateResponse(TemplateId templateId, string templateName, bool resolved, IssueSeverity severity = IssueSeverity.Unspecified)
		{
			GetIssueAffectedTemplateResponse response = new GetIssueAffectedTemplateResponse();
			response.TemplateId = templateId;
			response.TemplateName = templateName;
			response.Resolved = resolved;
			response.Severity = severity;
			return response;
		}

		/// <summary>
		/// Constructs a new issue
		/// </summary>
		/// <param name="details">Issue to construct from</param>
		/// <param name="affectedStreams">The affected streams</param>
		/// <param name="showDesktopAlerts">Whether to show alerts for this issue</param>
		static GetIssueResponse NewGetIssueResponse(IIssueDetails details, List<GetIssueAffectedStreamResponse> affectedStreams, bool showDesktopAlerts)
		{
			GetIssueResponse response = new GetIssueResponse();
			IIssue issue = details.Issue;
			response.Id = issue.Id;
			response.CreatedAt = issue.CreatedAt;
			response.RetrievedAt = DateTime.UtcNow;
			response.Summary = String.IsNullOrEmpty(issue.UserSummary) ? issue.Summary : issue.UserSummary;
			response.Description = issue.Description;
			response.Severity = issue.Severity;
			response.Promoted = issue.Promoted;
			response.Owner = details.Owner?.Login;
			response.OwnerId = details.Owner?.Id.ToString();
			response.OwnerInfo = details.Owner?.ToThinApiResponse();
			response.NominatedBy = details.NominatedBy?.Login;
			response.NominatedByInfo = details.NominatedBy?.ToThinApiResponse();
			response.AcknowledgedAt = issue.AcknowledgedAt;
			response.FixCommitId = issue.FixCommitId;
			response.FixSystemic = issue.FixSystemic;
			response.RootCommitId = issue.RootCommitId;
			response.RootCauseOwnerInfo = details.RootCauseOwner?.ToThinApiResponse();
			response.RootCauseCategory = issue.RootCauseCategory;
			response.RootCauseSummary = issue.RootCauseSummary;
			response.DuplicateIssueId = issue.DuplicateIssueId;
			response.ResolvedAt = issue.ResolvedAt;
			response.ResolvedBy = details.ResolvedBy?.Login;
			response.ResolvedById = details.ResolvedBy?.Id.ToString();
			response.ResolvedByInfo = details.ResolvedBy?.ToThinApiResponse();
			response.VerifiedAt = issue.VerifiedAt;
			response.LastSeenAt = issue.LastSeenAt;
			response.Streams = details.Spans.Select(x => x.StreamName).Distinct().ToList()!;
			response.ResolvedStreams = new List<string>();
			response.UnresolvedStreams = new List<string>();
			response.AffectedStreams = affectedStreams;
			foreach (IGrouping<StreamId, IIssueSpan> stream in details.Spans.GroupBy(x => x.StreamId))
			{
				if (stream.All(x => x.NextSuccess != null))
				{
					response.ResolvedStreams.Add(stream.Key.ToString());
				}
				else
				{
					response.UnresolvedStreams.Add(stream.Key.ToString());
				}
			}
			response.PrimarySuspects = details.SuspectUsers.Where(x => x.Login != null).Select(x => x.Login).ToList();
			response.PrimarySuspectIds = details.SuspectUsers.Select(x => x.Id.ToString()).ToList();
			response.PrimarySuspectsInfo = details.SuspectUsers.ConvertAll(x => x.ToThinApiResponse());
			response.ShowDesktopAlerts = showDesktopAlerts;
			response.ExternalIssueKey = details.ExternalIssueKey;
			response.QuarantinedByUserInfo = details.QuarantinedBy?.ToThinApiResponse();
			response.QuarantineTimeUtc = details.QuarantineTimeUtc;
			response.ForceClosedByUserInfo = details.ForceClosedBy?.ToThinApiResponse();
			response.WorkflowThreadUrl = issue.WorkflowThreadUrl;

			if (issue.Fingerprints != null && issue.Fingerprints.Count > 0)
			{
				response.FingerprintDescription = String.Join(", ", issue.Fingerprints.Select(x =>
				{
					return $"Type: {x.Type} / Keys: {String.Join(", ", x.Keys)} / RejectKeys: {String.Join(", ", x.RejectKeys ?? new HashSet<IssueKey>())}";
				}));
			}

			return response;
		}

		/// <summary>
		/// Retrieve events for a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/issues/{issueId}/streams")]
		[ProducesResponseType(typeof(List<GetIssueStreamResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetIssueStreamsAsync(int issueId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IIssueDetails? issue = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (issue == null)
			{
				return NotFound();
			}
			if (!AuthorizeIssue(issue))
			{
				return Forbid();
			}

			List<object> responses = new List<object>();
			foreach (IGrouping<StreamId, IIssueSpan> spanGroup in issue.Spans.GroupBy(x => x.StreamId))
			{
				if (_buildConfig.Value.TryGetStream(spanGroup.Key, out StreamConfig? streamConfig) && streamConfig.Authorize(StreamAclAction.ViewStream, User))
				{
					HashSet<ObjectId> spanIds = new HashSet<ObjectId>(spanGroup.Select(x => x.Id));
					List<IIssueStep> steps = issue.Steps.Where(x => spanIds.Contains(x.SpanId)).ToList();
					responses.Add(PropertyFilter.Apply(NewGetIssueStreamResponse(spanGroup.Key, spanGroup.ToList(), steps), filter));
				}
			}
			return responses;
		}

		static GetIssueStreamResponse NewGetIssueStreamResponse(StreamId streamId, List<IIssueSpan> spans, List<IIssueStep> steps)
		{
			GetIssueStreamResponse response = new GetIssueStreamResponse();
			response.StreamId = streamId;

			foreach (IIssueSpan span in spans)
			{
				if (span.LastSuccess != null && (response.MinCommitId == null || span.LastSuccess.CommitId < response.MinCommitId))
				{
					response.MinCommitId = span.LastSuccess.CommitId;
				}
				if (span.NextSuccess != null && (response.MaxCommitId == null || span.NextSuccess.CommitId > response.MaxCommitId))
				{
					response.MaxCommitId = span.NextSuccess.CommitId;
				}
				response.Nodes.Add(NewGetIssueSpanResponse(span, steps.Where(y => y.SpanId == span.Id).ToList()));
			}
			return response;
		}

		static GetIssueSpanResponse NewGetIssueSpanResponse(IIssueSpan span, List<IIssueStep> steps)
		{
			GetIssueSpanResponse response = new GetIssueSpanResponse();
			response.Id = span.Id.ToString();
			response.Name = span.NodeName;
			response.TemplateId = span.TemplateRefId;
			response.WorkflowId = span.LastFailure.Annotations.WorkflowId;
			response.LastSuccess = (span.LastSuccess != null) ? NewGetIssueStepResponse(span.LastSuccess) : null;
			response.Steps = steps.ConvertAll(x => NewGetIssueStepResponse(x));
			response.NextSuccess = (span.NextSuccess != null) ? NewGetIssueStepResponse(span.NextSuccess) : null;
			return response;
		}

		/// <summary>
		/// Retrieve events for a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="streamId">The stream id</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/issues/{issueId}/streams/{streamId}")]
		[ProducesResponseType(typeof(List<GetIssueStreamResponse>), 200)]
		public async Task<ActionResult<object>> GetIssueStreamAsync(int issueId, StreamId streamId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IIssueDetails? details = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (details == null)
			{
				return NotFound();
			}
			if (!_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
			{
				return Forbid(StreamAclAction.ViewStream, streamId);
			}

			List<IIssueSpan> spans = details.Spans.Where(x => x.StreamId == streamId).ToList();
			if (spans.Count == 0)
			{
				return NotFound();
			}

			HashSet<ObjectId> spanIds = new HashSet<ObjectId>(spans.Select(x => x.Id));
			List<IIssueStep> steps = details.Steps.Where(x => spanIds.Contains(x.SpanId)).ToList();

			return PropertyFilter.Apply(NewGetIssueStreamResponse(streamId, spans, steps), filter);
		}

		/// <summary>
		/// Retrieve events for a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="jobId">The job id to filter for</param>
		/// <param name="batchId">The batch to filter by</param>
		/// <param name="stepId">The step to filter by</param>
		/// <param name="labelIdx">The label within the job to filter by</param>
		/// <param name="logIds">List of log ids to return issues for</param>
		/// <param name="index">Index of the first event</param>
		/// <param name="count">Number of events to return</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/issues/{issueId}/events")]
		[ProducesResponseType(typeof(List<GetLogEventResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetIssueEventsAsync(
			int issueId,
			[FromQuery] JobId? jobId = null,
			[FromQuery] JobStepBatchId? batchId = null,
			[FromQuery] JobStepId? stepId = null,
			[FromQuery(Name = "label")] int? labelIdx = null,
			[FromQuery] string[]? logIds = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 10,
			[FromQuery] PropertyFilter? filter = null,
			CancellationToken cancellationToken = default)
		{
			HashSet<LogId> logIdValues = new HashSet<LogId>();
			if (jobId != null)
			{
				IJob? job = await _jobService.GetJobAsync(jobId.Value, cancellationToken);
				if (job == null)
				{
					return NotFound();
				}

				if (stepId != null)
				{
					IJobStep? step;
					if (job.TryGetStep(stepId.Value, out step) && step.Outcome != JobStepOutcome.Success && step.LogId != null)
					{
						logIdValues.Add(step.LogId.Value);
					}
				}
				else if (batchId != null)
				{
					IJobStepBatch? batch;
					if (job.TryGetBatch(batchId.Value, out batch))
					{
						logIdValues.UnionWith(batch.Steps.Where(x => x.Outcome != JobStepOutcome.Success && x.LogId != null).Select(x => x.LogId!.Value));
					}
				}
				else if (labelIdx != null)
				{
					IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);

					HashSet<NodeRef> includedNodes = new HashSet<NodeRef>(graph.Labels[labelIdx.Value].IncludedNodes);

					foreach (IJobStepBatch batch in job.Batches)
					{
						foreach (IJobStep step in batch.Steps)
						{
							NodeRef nodeRef = new NodeRef(batch.GroupIdx, step.NodeIdx);
							if (step.Outcome != JobStepOutcome.Success && step.LogId != null && includedNodes.Contains(nodeRef))
							{
								logIdValues.Add(step.LogId.Value);
							}
						}
					}
				}
				else
				{
					logIdValues.UnionWith(job.Batches.SelectMany(x => x.Steps).Where(x => x.Outcome != JobStepOutcome.Success && x.LogId != null).Select(x => x.LogId!.Value));
				}
			}
			if (logIds != null)
			{
				foreach (string logIdStr in logIds)
				{
					if (LogId.TryParse(logIdStr, out LogId parsedLogId))
					{
						logIdValues.Add(parsedLogId);
					}
				}
			}

			IReadOnlyList<IIssueSpan> spans = await _issueCollection.FindSpansAsync(issueId, cancellationToken);
			IReadOnlyList<ILogAnchor> anchors = await _logCollection.FindAnchorsForSpansAsync(spans.Select(x => x.Id), logIdValues.ToArray(), index, count, cancellationToken);

			JobPermissionsCache permissionsCache = new JobPermissionsCache();
			Dictionary<LogId, ILog?> logs = new Dictionary<LogId, ILog?>();

			List<object> responses = new List<object>();
			foreach (ILogAnchor anchor in anchors)
			{
				ILog? log;
				if (!logs.TryGetValue(anchor.LogId, out log))
				{
					log = await _logCollection.GetAsync(anchor.LogId, cancellationToken);
					logs[anchor.LogId] = log;
				}
				if (log != null && await _jobService.AuthorizeAsync(log.JobId, LogAclAction.ViewLog, User, _buildConfig.Value, cancellationToken))
				{
					ILogEventData data = await anchor.GetDataAsync(cancellationToken);
					GetLogEventResponse response = LogsController.CreateGetLogEventResponse(anchor, data, issueId);
					responses.Add(PropertyFilter.Apply(response, filter));
				}
			}
			return responses;
		}

		/// <summary>
		/// Authorize the current user to see an issue
		/// </summary>
		/// <param name="issue">The issue to authorize</param>
		/// <returns>True if the user is authorized to see the issue</returns>
		private bool AuthorizeIssue(IIssueDetails issue)
		{
			foreach (StreamId streamId in issue.Spans.Select(x => x.StreamId).Distinct())
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(StreamAclAction.ViewStream, User))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Update an issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="request">The update information</param>
		/// <returns>List of matching agents</returns>
		[HttpPut]
		[Route("/api/v1/issues/{issueId}")]
		public async Task<ActionResult> UpdateIssueAsync(int issueId, [FromBody] UpdateIssueRequest request)
		{
			UserId? newOwnerId = null;
			if (request.OwnerId != null)
			{
				newOwnerId = request.OwnerId.Length == 0 ? UserId.Empty : UserId.Parse(request.OwnerId);
			}

			UserId? newRootCauseOwnerId = null;
			if (request.RootCauseOwnerId != null)
			{
				newRootCauseOwnerId = request.RootCauseOwnerId.Length == 0 ? UserId.Empty : UserId.Parse(request.RootCauseOwnerId);
			}

			UserId? newNominatedById = null;
			if (request.NominatedById != null)
			{
				newNominatedById = UserId.Parse(request.NominatedById);
			}
			else if (request.OwnerId != null)
			{
				newNominatedById = User.GetUserId();
			}

			UserId? newDeclinedById = null;
			if (request.Declined ?? false)
			{
				newDeclinedById = User.GetUserId();
			}

			UserId? newResolvedById = null;
			if (request.Resolved.HasValue)
			{
				newResolvedById = request.Resolved.Value ? User.GetUserId() : UserId.Empty;
			}

			UserId? newQuarantinedById = null;
			if (request.QuarantinedById != null)
			{
				newQuarantinedById = request.QuarantinedById.Length > 0 ? UserId.Parse(request.QuarantinedById) : UserId.Empty;
			}

			UserId? newForceClosedById = null;
			if (request.ForceClosedById != null)
			{
				newForceClosedById = request.ForceClosedById.Length > 0 ? UserId.Parse(request.ForceClosedById) : UserId.Empty;
			}

			List<ObjectId>? addSpans = null;
			if (request.AddSpans != null && request.AddSpans.Count > 0)
			{
				addSpans = request.AddSpans.ConvertAll(x => ObjectId.Parse(x));
			}

			List<ObjectId>? removeSpans = null;
			if (request.RemoveSpans != null && request.RemoveSpans.Count > 0)
			{
				removeSpans = request.RemoveSpans.ConvertAll(x => ObjectId.Parse(x));
			}

			if (!await _issueService.UpdateIssueAsync(issueId, request.Summary, request.Description, request.Promoted, newOwnerId, newNominatedById, request.Acknowledged, newDeclinedById, request.FixCommitId, request.FixSystemic, request.RootCommitId, newRootCauseOwnerId, request.RootCauseCategory, request.RootCauseSummary, request.DuplicateIssueId, newResolvedById, addSpans, removeSpans, request.ExternalIssueKey, newQuarantinedById, newForceClosedById, initiatedById: User.GetUserId()))
			{
				return NotFound();
			}
			return Ok();
		}

		// External issue tracking

		/// <summary>
		/// Get external issue information for provided keys
		/// </summary>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/issues/external")]
		public async Task<ActionResult<List<GetExternalIssueResponse>>> GetExternalIssuesAsync([FromQuery] StreamId streamId, [FromQuery] string[] keys, CancellationToken cancellationToken)
		{
			if (_externalIssueService == null)
			{
				return NotFound();
			}
			if (!_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
			{
				return Forbid(StreamAclAction.ViewStream, streamId);
			}

			List<GetExternalIssueResponse> response = new List<GetExternalIssueResponse>();

			if (keys.Length != 0)
			{
				List<IExternalIssue> issues = await _externalIssueService.GetIssuesAsync(keys, cancellationToken);

				for (int i = 0; i < issues.Count; i++)
				{
					response.Add(NewGetExternalIssueResponse(issues[i]));
				}
			}

			return response;
		}

		static GetExternalIssueResponse NewGetExternalIssueResponse(IExternalIssue issue)
		{
			GetExternalIssueResponse response = new GetExternalIssueResponse();
			response.Key = issue.Key;
			response.Link = issue.Link;
			response.StatusName = issue.StatusName;
			response.ResolutionName = issue.ResolutionName;
			response.PriorityName = issue.PriorityName;
			response.AssigneeName = issue.AssigneeName;
			response.AssigneeDisplayName = issue.AssigneeDisplayName;
			response.AssigneeEmailAddress = issue.AssigneeEmailAddress;
			return response;
		}

		/// <summary>
		/// Create a new external issue
		/// </summary>
		[HttpPost]
		[Authorize]
		[Route("/api/v1/issues/external")]
		public async Task<ActionResult<CreateExternalIssueResponse>> CreateExternalIssueAsync([FromBody] CreateExternalIssueRequest issueRequest, CancellationToken cancellationToken)
		{
			if (_externalIssueService == null)
			{
				return NotFound();
			}

			StreamId streamIdValue = new StreamId(issueRequest.StreamId);

			if (!_buildConfig.Value.TryGetStream(streamIdValue, out StreamConfig? streamConfig))
			{
				return NotFound(streamIdValue);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
			{
				return Forbid(StreamAclAction.ViewStream, streamIdValue);
			}

			IUser? user = await _userCollection.GetUserAsync(User, cancellationToken);
			if (user == null)
			{
				return BadRequest($"Missing user for {User.GetUserId()}");
			}

			(string? key, string? url) = await _externalIssueService.CreateIssueAsync(user, User.GetExternalIssueUser(), issueRequest.IssueId, issueRequest.Summary, issueRequest.ProjectId, issueRequest.ComponentId, issueRequest.IssueTypeId, issueRequest.Description, issueRequest.HordeIssueLink, cancellationToken);

			if (key == null)
			{
				throw new Exception($"Unable to create external issue");
			}

			return new CreateExternalIssueResponse(key, url);
		}

		/// <summary>
		/// Gets external issue tracking projects associated with the provided stream
		/// </summary>
		/// <param name="streamId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpGet]
		[Authorize]
		[Route("/api/v1/issues/external/projects")]
		public async Task<ActionResult<List<GetExternalIssueProjectResponse>>> GetExternalIssueProjectsAsync([FromQuery] string streamId, CancellationToken cancellationToken)
		{
			if (_externalIssueService == null)
			{
				return NotFound();
			}

			StreamId streamIdValue = new StreamId(streamId);
			if (!_buildConfig.Value.TryGetStream(streamIdValue, out StreamConfig? streamConfig))
			{
				return NotFound(streamIdValue);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
			{
				return Forbid(StreamAclAction.ViewStream, streamIdValue);
			}

			List<IExternalIssueProject> projects = await _externalIssueService.GetProjectsAsync(streamConfig, cancellationToken);
			List<GetExternalIssueProjectResponse> response = new List<GetExternalIssueProjectResponse>();

			projects.ForEach(project =>
			{
				response.Add(NewGetExternalIssueProjectResponse(project.Key, project.Name, project.Id, project.Components, project.IssueTypes));
			});

			return response;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="key"></param>
		/// <param name="name"></param>
		/// <param name="id"></param>
		/// <param name="components"></param>
		/// <param name="issueTypes"></param>
		static GetExternalIssueProjectResponse NewGetExternalIssueProjectResponse(string key, string name, string id, Dictionary<string, string> components, Dictionary<string, string> issueTypes)
		{
			GetExternalIssueProjectResponse response = new GetExternalIssueProjectResponse();
			response.ProjectKey = key;
			response.Name = name;
			response.Id = id;
			response.Components = new Dictionary<string, string>(components);
			response.IssueTypes = new Dictionary<string, string>(issueTypes);
			return response;
		}
	}
}
