// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Redis;
using HordeServer.RCA;
using HordeServer.Server;
using HordeServer.Streams;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Issues
{
	/// <summary>
	/// Redis-backed queue for RCA processing requests.
	/// Ensures every eligible issue gets processed with retry support and crash resilience.
	/// </summary>
	public sealed class RcaQueueService : IRcaEnqueuer, IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Maximum number of retry attempts before discarding a queue item
		/// </summary>
		const int MaxRetries = 3;

		/// <summary>
		/// Minimum age before a queue item is processed, to allow audit logs to be emitted after issue creation
		/// </summary>
		static readonly TimeSpan s_minimumAge = TimeSpan.FromSeconds(5);

		/// <summary>
		/// Item stored in the Redis RCA queue
		/// </summary>
		[RedisConverter(typeof(RedisJsonConverter<>))]
		record class RcaQueueItem(int IssueId, string StreamId, string? WorkflowId, int RetryCount, DateTime EnqueuedUtc);

		static readonly RedisListKey<RcaQueueItem> s_rcaQueue = "rca/queue";

		readonly IRedisService _redisService;

		// Lazy-resolved to break circular singleton dependency:
		// RcaQueueService → IRcaProcessor → LlmRcaProcessor → IRcaNotifier → SlackNotificationSink → IssueService → IRcaEnqueuer → RcaQueueService
		readonly Lazy<IRcaProcessor?> _rcaProcessor;
		readonly IIssueCollection _issueCollection;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITicker _ticker;
		readonly ILogger<RcaQueueService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public RcaQueueService(IRedisService redisService, IIssueCollection issueCollection, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<RcaQueueService> logger, Lazy<IRcaProcessor?> rcaProcessor)
		{
			_redisService = redisService;
			_rcaProcessor = rcaProcessor;
			_issueCollection = issueCollection;
			_buildConfig = buildConfig;
			_ticker = clock.AddSharedTicker<RcaQueueService>(TimeSpan.FromSeconds(30.0), ProcessQueueAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => _ticker.DisposeAsync();

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		/// <inheritdoc/>
		public async Task EnqueueAsync(int issueId, StreamId streamId, WorkflowId? workflowId, CancellationToken cancellationToken)
		{
			if (_rcaProcessor.Value == null)
			{
				return;
			}

			int maxConcurrent = _buildConfig.CurrentValue.MaxConcurrentRcaRequests;
			if (maxConcurrent <= 0)
			{
				return;
			}

			RcaQueueItem item = new RcaQueueItem(issueId, streamId.ToString(), workflowId?.ToString(), 0, DateTime.UtcNow);
			await _redisService.GetDatabase().ListRightPushAsync(s_rcaQueue, item);

			_logger.LogDebug("Enqueued RCA processing for issue {IssueId} in stream {StreamId}", issueId, streamId);
		}

		async ValueTask ProcessQueueAsync(CancellationToken cancellationToken)
		{
			if (_rcaProcessor.Value == null)
			{
				return;
			}

			int maxConcurrent = _buildConfig.CurrentValue.MaxConcurrentRcaRequests;
			if (maxConcurrent <= 0)
			{
				return;
			}

			long count = await _redisService.GetDatabase().ListLengthAsync(s_rcaQueue);
			if (count == 0)
			{
				return;
			}

			// Dequeue up to maxConcurrent items, re-queuing any that are too young
			int batchSize = (int)Math.Min(count, maxConcurrent);
			List<RcaQueueItem> eligibleItems = new List<RcaQueueItem>(batchSize);
			for (int i = 0; i < batchSize; i++)
			{
				RcaQueueItem item = await _redisService.GetDatabase().ListLeftPopAsync(s_rcaQueue);
				if (item == null)
				{
					break;
				}

				// Re-queue items that haven't aged enough for audit logs to be emitted
				if (DateTime.UtcNow - item.EnqueuedUtc < s_minimumAge)
				{
					await _redisService.GetDatabase().ListRightPushAsync(s_rcaQueue, item);
					continue;
				}

				if (!IsEligibleForRca(item))
				{
					_logger.LogDebug("Discarding ineligible RCA queue item for issue {IssueId}", item.IssueId);
					continue;
				}

				eligibleItems.Add(item);
			}

			if (eligibleItems.Count == 0)
			{
				return;
			}

			_logger.LogInformation("Processing {Count} RCA queue items", eligibleItems.Count);

			// Process batch in parallel
			Task[] tasks = new Task[eligibleItems.Count];
			for (int i = 0; i < eligibleItems.Count; i++)
			{
				tasks[i] = ProcessItemAsync(eligibleItems[i], cancellationToken);
			}
			await Task.WhenAll(tasks);
		}

		async Task ProcessItemAsync(RcaQueueItem item, CancellationToken cancellationToken)
		{
			StreamId streamId = new StreamId(item.StreamId);
			WorkflowId? workflowId = item.WorkflowId != null ? new WorkflowId(item.WorkflowId) : null;

			try
			{
				await _rcaProcessor.Value!.ProcessRcaAsync(item.IssueId, streamId, workflowId, cancellationToken);
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				// Shutdown - re-queue so work survives restart
				await _redisService.GetDatabase().ListRightPushAsync(s_rcaQueue, item);
				_logger.LogDebug("Re-queued RCA item for issue {IssueId} due to shutdown", item.IssueId);
			}
			catch (Exception ex)
			{
				ILogger issueLogger = _issueCollection.GetLogger(item.IssueId);

				if (item.RetryCount + 1 >= MaxRetries)
				{
					// Permanent failure - discard
					LoggerUtils.MultiLog(
						[(_logger, LogLevel.Error), (issueLogger, LogLevel.Information)],
						ex, "RCA processing permanently failed for issue {IssueId} after {MaxRetries} attempts", item.IssueId, MaxRetries);
				}
				else
				{
					// Transient failure - re-queue with incremented retry count
					RcaQueueItem retryItem = item with { RetryCount = item.RetryCount + 1 };
					await _redisService.GetDatabase().ListRightPushAsync(s_rcaQueue, retryItem);

					LoggerUtils.MultiLog(
						[(_logger, LogLevel.Warning), (issueLogger, LogLevel.Information)],
						ex, "RCA processing failed for issue {IssueId} (attempt {Attempt}/{MaxRetries}), re-queued", item.IssueId, item.RetryCount + 1, MaxRetries);
				}
			}
		}

		bool IsEligibleForRca(RcaQueueItem item)
		{
			if (item.WorkflowId == null)
			{
				return false;
			}

			StreamId streamId = new StreamId(item.StreamId);
			if (!_buildConfig.CurrentValue.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				return false;
			}

			WorkflowId workflowId = new WorkflowId(item.WorkflowId);
			if (!streamConfig.TryGetWorkflow(workflowId, out WorkflowConfig? workflow))
			{
				return false;
			}

			return workflow.AutoRcaEnabled;
		}
	}
}
