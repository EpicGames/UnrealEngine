// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs.Graphs;

#pragma warning disable CA2227 //  Change x to be read-only by removing the property setter

namespace HordeServer.Issues
{
	/// <summary>
	/// Configuration for an issue workflow
	/// </summary>
	public class WorkflowConfig : IWorkflow
	{
		/// <inheritdoc/>
		public WorkflowId Id { get; set; } = WorkflowId.Empty;

		/// <inheritdoc cref="IWorkflow.ReportTimes"/>
		public List<TimeSpan> ReportTimes { get; set; } = new List<TimeSpan>();
		IReadOnlyList<TimeSpan> IWorkflow.ReportTimes => ReportTimes;

		/// <inheritdoc/>
		public string? SummaryTab { get; set; }

		/// <inheritdoc/>
		public string? ReportChannel { get; set; }

		/// <inheritdoc/>
		public bool ReportWarnings { get; set; } = true;

		/// <inheritdoc/>
		public bool GroupIssuesByTemplate { get; set; } = true;

		/// <inheritdoc/>
		public string? TriageChannel { get; set; }

		/// <inheritdoc/>
		public string? TriagePrefix { get; set; } = "*[NEW]* ";

		/// <inheritdoc/>
		public string? TriageSuffix { get; set; }

		/// <inheritdoc/>
		public string? TriageInstructions { get; set; }

		/// <inheritdoc/>
		public string? TriageAlias { get; set; }

		/// <inheritdoc/>
		public bool TriageErrors { get; set; } = true;

		/// <inheritdoc/>
		public bool TriageWarnings { get; set; } = true;

		/// <inheritdoc cref="IWorkflow.TriageTypeAliases"/>
		public Dictionary<string, string>? TriageTypeAliases { get; set; }
		IReadOnlyDictionary<string, string>? IWorkflow.TriageTypeAliases => TriageTypeAliases;

		/// <inheritdoc/>
		public string? EscalateAlias { get; set; }

		/// <inheritdoc cref="IWorkflow.EscalateTimes"/>
		public List<int> EscalateTimes { get; set; } = new List<int> { 120 };

		IReadOnlyList<int> IWorkflow.EscalateTimes => EscalateTimes;

		/// <inheritdoc cref="IWorkflow.EscalateRootCauseAnalysisTimes"/>
		public List<int> EscalateRootCauseAnalysisTimes { get; set; } = [60 * 24];

		IReadOnlyList<int> IWorkflow.EscalateRootCauseAnalysisTimes => EscalateRootCauseAnalysisTimes;

		/// <inheritdoc/>
		public bool AutoRcaEnabled { get; set; }

		/// <inheritdoc/>
		public bool AutoRcaSlackEnabled { get; set; }

		/// <inheritdoc/>
		public int MaxMentions { get; set; } = 5;

		/// <inheritdoc/>
		public bool AllowMentions { get; set; } = true;

		/// <inheritdoc/>
		public bool InviteRestrictedUsers { get; set; }

		/// <inheritdoc/>
		public bool SkipWhenEmpty { get; set; }

		/// <inheritdoc/>
		public bool ShowMergeWarnings { get; set; }

		/// <inheritdoc cref="IWorkflow.Annotations"/>
		public NodeAnnotations Annotations { get; set; } = new NodeAnnotations();
		IReadOnlyNodeAnnotations IWorkflow.Annotations => Annotations;

		/// <inheritdoc cref="IWorkflow.ExternalIssues"/>
		public WorkflowExternalIssuesConfig? ExternalIssues { get; set; }
		IWorkflowExternalIssues? IWorkflow.ExternalIssues => ExternalIssues;

		/// <inheritdoc cref="IWorkflow.IssueHandlers"/>
		public List<string>? IssueHandlers { get; set; }
		IReadOnlyList<string>? IWorkflow.IssueHandlers => IssueHandlers;
	}

	/// <summary>
	/// External issue tracking configuration for a workflow
	/// </summary>
	public class WorkflowExternalIssuesConfig : IWorkflowExternalIssues
	{
		/// <inheritdoc/>
		[Required]
		public string ProjectKey { get; set; } = String.Empty;

		/// <inheritdoc/>
		[Required]
		public string DefaultComponentId { get; set; } = String.Empty;

		/// <inheritdoc/>
		[Required]
		public string DefaultIssueTypeId { get; set; } = String.Empty;
	}
}
