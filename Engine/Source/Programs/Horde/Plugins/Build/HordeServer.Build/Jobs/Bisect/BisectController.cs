// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Jobs.Graphs;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Jobs.Bisect
{
	/// <summary>
	/// Request to create a new bisect task
	/// </summary>
	public class CreateBisectTaskRequest
	{
		/// <summary>
		/// Job containing the node to check
		/// </summary>
		public JobId JobId { get; set; }

		/// <summary>
		/// Name of the node to query
		/// </summary>
		public string NodeName { get; set; } = String.Empty;

		/// <summary>
		/// Commit tag to filter possible changes against
		/// </summary>
		public List<CommitTag>? CommitTags { get; set; }

		/// <summary>
		/// Set of changes to ignore. Can be modified later through <see cref="UpdateBisectTaskRequest"/>.
		/// </summary>
		[Obsolete("Use IgnoreCommitIds instead")]
		public List<int>? IgnoreChanges
		{
			get => _ignoreChanges ?? _ignoreCommitIds?.ConvertAll(x => x.GetPerforceChangeOrMinusOne());
			set => _ignoreChanges = value;
		}
		List<int>? _ignoreChanges;

		/// <summary>
		/// Set of changes to ignore. Can be modified later through <see cref="UpdateBisectTaskRequest"/>.
		/// </summary>
		public List<CommitId>? IgnoreCommitIds
		{
			get => _ignoreCommitIds ?? _ignoreChanges?.ConvertAll(x => CommitId.FromPerforceChange(x));
			set => _ignoreCommitIds = value;
		}
		List<CommitId>? _ignoreCommitIds;

		/// <summary>
		/// Set of jobs to ignore. Can be modified later through <see cref="UpdateBisectTaskRequest"/>.
		/// </summary>
		public List<JobId>? IgnoreJobs { get; set; }
	}

	/// <summary>
	/// Response from creating a bisect task
	/// </summary>
	public class CreateBisectTaskResponse
	{
		/// <summary>
		/// Identifier for the new bisect task
		/// </summary>
		public BisectTaskId BisectTaskId { get; }

		internal CreateBisectTaskResponse(IBisectTask bisectTask) => BisectTaskId = bisectTask.Id;
	}

	/// <summary>
	/// Information about a bisect task
	/// </summary>
	public class GetBisectTaskResponse
	{
		readonly IBisectTask _bisectTask;

		/// <inheritdoc cref="IBisectTask.Id"/>
		public BisectTaskId Id => _bisectTask.Id;

		/// <inheritdoc cref="IBisectTask.State"/>
		public BisectTaskState State => _bisectTask.State;

		/// <inheritdoc cref="IBisectTask.OwnerId"/>
		public GetThinUserInfoResponse? Owner { get; }

		/// <inheritdoc cref="IBisectTask.StreamId"/>
		public StreamId StreamId => _bisectTask.StreamId;

		/// <inheritdoc cref="IBisectTask.TemplateId"/>
		public TemplateId TemplateId => _bisectTask.TemplateId;

		/// <inheritdoc cref="IBisectTask.NodeName"/>
		public string NodeName => _bisectTask.NodeName;

		/// <inheritdoc cref="IBisectTask.Outcome"/>
		public JobStepOutcome Outcome => _bisectTask.Outcome;

		/// Initial Job Id
		public string InitialJobId => _bisectTask.InitialJobStep.JobId.ToString();

		/// Initial Job Batch Id
		public string InitialBatchId => _bisectTask.InitialJobStep.BatchId.ToString();

		/// Initial Job Step Id
		public string InitialStepId => _bisectTask.InitialJobStep.StepId.ToString();

		/// <inheritdoc cref="IBisectTask.InitialCommitId"/>
		[Obsolete("Use InitialCommitId instead")]
		public int InitialChange => InitialCommitId.GetPerforceChangeOrMinusOne();

		/// <inheritdoc cref="IBisectTask.InitialCommitId"/>
		public CommitIdWithOrder InitialCommitId => _bisectTask.InitialCommitId;

		/// Min Job Id
		public string? MinJobId => _bisectTask.MinJobStep?.JobId.ToString();

		/// Min Job Batch Id
		public string? MinBatchId => _bisectTask.MinJobStep?.BatchId.ToString();

		/// Min Job Step Id
		public string? MinStepId => _bisectTask.MinJobStep?.StepId.ToString();

		/// <inheritdoc cref="IBisectTask.MinCommitId"/>
		[Obsolete("Use MinCommitId instead")]
		public int? MinChange => MinCommitId?.GetPerforceChangeOrMinusOne();

		/// <inheritdoc cref="IBisectTask.MinCommitId"/>
		public CommitIdWithOrder? MinCommitId => _bisectTask.MinCommitId;

		/// Current Job Id
		public string CurrentJobId => _bisectTask.CurrentJobStep.JobId.ToString();

		/// Current Job Batch Id
		public string CurrentBatchId => _bisectTask.CurrentJobStep.BatchId.ToString();

		/// Current Job Step Id
		public string CurrentStepId => _bisectTask.CurrentJobStep.StepId.ToString();

		/// <inheritdoc cref="IBisectTask.CurrentCommitId"/>
		[Obsolete("Use CurrentCommit instead")]
		public int CurrentChange => CurrentCommitId.GetPerforceChangeOrMinusOne();

		/// <inheritdoc cref="IBisectTask.CurrentCommitId"/>
		public CommitIdWithOrder CurrentCommitId => _bisectTask.CurrentCommitId;

		/// <summary>
		/// The next job id for a running bisection task
		/// </summary>
		public JobId? NextJobId { get; }

		/// <summary>
		/// The next job change
		/// </summary>
		[Obsolete("Use NextJobCommit instead")]
		public int? NextJobChange => NextJobCommit?.TryGetPerforceChange();

		/// <summary>
		/// The next job change
		/// </summary>
		public CommitIdWithOrder? NextJobCommit { get; }

		/// <summary>
		/// The steps involved in the bisection
		/// </summary>
		public List<GetJobStepRefResponse> Steps { get; }

		internal GetBisectTaskResponse(IBisectTask bisectTask, GetThinUserInfoResponse? owner, List<IJobStepRef> steps, IJob? nextJob)
		{
			_bisectTask = bisectTask;
			Owner = owner;
			Steps = steps.Select(s => StreamsController.CreateGetJobStepRefResponse(s)).ToList();
			NextJobId = nextJob?.Id;
			NextJobCommit = nextJob?.CommitId;
		}
	}

	/// <summary>
	/// Updates the state of a bisect task
	/// </summary>
	public class UpdateBisectTaskRequest
	{
		/// <summary>
		/// Cancels the current task
		/// </summary>
		public bool? Cancel { get; set; }

		/// <summary>
		/// List of change numbers to include in the search. 
		/// </summary>
		[Obsolete("Use IncludeCommits instead")]
		public List<int> IncludeChanges
		{
			get => _includeChanges ?? _includeCommitIds?.ConvertAll(x => x.GetPerforceChangeOrMinusOne()) ?? new List<int>();
			set => _includeChanges = value;
		}
		List<int>? _includeChanges;

		/// <summary>
		/// List of commits to include in the search. 
		/// </summary>
		public List<CommitId> IncludeCommitIds
		{
			get => _includeCommitIds ?? _includeChanges?.ConvertAll(x => CommitId.FromPerforceChange(x)) ?? new List<CommitId>();
			set => _includeCommitIds = value;
		}
		List<CommitId>? _includeCommitIds;

		/// <summary>
		/// List of change numbers to exclude from the search.
		/// </summary>
		[Obsolete("Use ExcludeCommitIds instead")]
		public List<int> ExcludeChanges
		{
			get => _excludeChanges ?? _excludeCommitIds?.Select(x => x.TryGetPerforceChange() ?? 0).Where(x => x != 0).ToList() ?? new List<int>();
			set => _excludeChanges = value;
		}
		List<int>? _excludeChanges;

		/// <summary>
		/// List of commits to exclude from the search
		/// </summary>
		public List<CommitId> ExcludeCommitIds
		{
			get => _excludeCommitIds ?? _excludeChanges?.ConvertAll(x => CommitId.FromPerforceChange(x)) ?? new List<CommitId>();
			set => _excludeCommitIds = value;
		}
		List<CommitId>? _excludeCommitIds;

		/// <summary>
		/// List of jobs to include in the search.
		/// </summary>
		public List<JobId> IncludeJobs { get; set; } = new List<JobId>();

		/// <summary>
		/// List of jobs to exclude from the search.
		/// </summary>
		public List<JobId> ExcludeJobs { get; set; } = new List<JobId>();
	}

	/// <summary>
	/// Controller for the /api/v1/bisect endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class BisectTasksController : HordeControllerBase
	{
		readonly IBisectTaskCollection _bisectTaskCollection;
		readonly IJobCollection _jobCollection;
		readonly JobService _jobService;
		readonly IJobStepRefCollection _jobStepRefs;
		readonly IGraphCollection _graphCollection;
		readonly IUserCollection _userCollection;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		readonly IOptions<BuildServerConfig> _staticConfig;
		private readonly ILogger<BisectTasksController> _logger;
		readonly Tracer _tracer;

		/// <summary>
		/// Constructor
		/// </summary>
		public BisectTasksController(IBisectTaskCollection bisectTaskCollection, JobService jobService, IJobCollection jobCollection, IJobStepRefCollection jobStepRefs, IGraphCollection graphCollection, IUserCollection userCollection, Tracer tracer, ILogger<BisectTasksController> logger, IOptionsSnapshot<BuildConfig> buildConfig, IOptions<BuildServerConfig> staticConfig)
		{
			_bisectTaskCollection = bisectTaskCollection;
			_jobService = jobService;
			_jobCollection = jobCollection;
			_jobStepRefs = jobStepRefs;
			_graphCollection = graphCollection;
			_userCollection = userCollection;
			_buildConfig = buildConfig;
			_staticConfig = staticConfig;
			_tracer = tracer;
			_logger = logger;
		}

		/// <summary>
		/// Creates a new bisect task
		/// </summary>
		/// <param name="create">Properties of the new bisect task</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Id of the new bisect task</returns>
		[HttpPost]
		[Route("/api/v1/bisect")]
		public async Task<ActionResult<CreateBisectTaskResponse>> CreateAsync([FromBody] CreateBisectTaskRequest create, CancellationToken cancellationToken = default)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return BadRequest("Bisection is currently disabled");
			}

			IJob? job = await _jobCollection.GetAsync(create.JobId, cancellationToken);
			if (job == null)
			{
				return NotFound(create.JobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, create.JobId);
			}
			if (!streamConfig.Authorize(BisectTaskAclAction.CreateBisectTask, User))
			{
				return Forbid(BisectTaskAclAction.CreateBisectTask, streamConfig.Id);
			}

			IGraph graph = await _graphCollection.GetAsync(job.GraphHash, cancellationToken);

			NodeRef nodeRef;
			if (!graph.TryFindNode(create.NodeName, out nodeRef))
			{
				return BadRequest($"Graph does not contain '{create.NodeName}'");
			}

			IJobStep? jobStep;
			if (!job.TryGetStepForNode(nodeRef, out jobStep))
			{
				return BadRequest($"Job does not contain '{create.NodeName}'");
			}
			if (jobStep.IsPending())
			{
				return BadRequest("Step has not finished executing yet");
			}
			if (jobStep.Outcome == JobStepOutcome.Success)
			{
				return BadRequest("Step has not failed");
			}

			IJobStepBatch? initialBatch = null;
			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.Steps.FirstOrDefault(x => x.Id == jobStep.Id) != null)
				{
					initialBatch = batch;
					break;
				}
			}

			if (initialBatch == null)
			{
				return BadRequest("Unable to find batch");
			}

			CreateBisectTaskOptions options = new CreateBisectTaskOptions();
			options.CommitTags = create.CommitTags;
			options.IgnoreCommitIds = create.IgnoreCommitIds;
			options.IgnoreJobIds = create.IgnoreJobs;

			UserId? userId = User.GetUserId();

			IBisectTask bisectTask = await _bisectTaskCollection.CreateAsync(job, initialBatch.Id, jobStep.Id, create.NodeName, jobStep.Outcome, userId ?? UserId.Empty, options, cancellationToken);

			if (userId != null)
			{
				await _userCollection.UpdateSettingsAsync(userId.Value, addBisectTaskIds: new[] { bisectTask.Id }, cancellationToken: cancellationToken);
			}

			return new CreateBisectTaskResponse(bisectTask);
		}

		/// <summary>
		/// Gets information about a bisection task
		/// </summary>
		/// <param name="bisectTaskId">Id of the bisect task to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Id of the new bisect task</returns>
		[HttpGet]
		[Route("/api/v1/bisect/{bisectTaskId}")]
		public async Task<ActionResult<GetBisectTaskResponse>> GetAsync([FromRoute] BisectTaskId bisectTaskId, CancellationToken cancellationToken = default)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return BadRequest("Bisection is currently disabled");
			}

			IBisectTask? bisectTask = await _bisectTaskCollection.GetAsync(bisectTaskId, cancellationToken);
			if (bisectTask == null)
			{
				return NotFound(bisectTaskId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(bisectTask.StreamId, out streamConfig))
			{
				return NotFound(bisectTask.StreamId);
			}
			if (!streamConfig.Authorize(BisectTaskAclAction.ViewBisectTask, User))
			{
				return Forbid(BisectTaskAclAction.ViewBisectTask, streamConfig.Id);
			}

			return await CreateBisectTaskResponseAsync(bisectTask, cancellationToken);
		}

		/// <summary>
		/// Creates a new bisect task
		/// </summary>
		/// <param name="bisectTaskId">Id of the bisect task to retrieve</param>
		/// <param name="request">Updates for the task</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>Id of the new bisect task</returns>
		[HttpPatch]
		[Route("/api/v1/bisect/{bisectTaskId}")]
		public async Task<ActionResult> UpdateAsync([FromRoute] BisectTaskId bisectTaskId, [FromBody] UpdateBisectTaskRequest request, CancellationToken cancellationToken)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return BadRequest("Bisection is currently disabled");
			}

			for (; ; )
			{
				IBisectTask? bisectTask = await _bisectTaskCollection.GetAsync(bisectTaskId, cancellationToken);
				if (bisectTask == null)
				{
					return NotFound(bisectTaskId);
				}

				StreamConfig? streamConfig;
				if (!_buildConfig.Value.TryGetStream(bisectTask.StreamId, out streamConfig))
				{
					return NotFound(bisectTask.StreamId);
				}
				if (!streamConfig.Authorize(BisectTaskAclAction.UpdateBisectTask, User))
				{
					return Forbid(BisectTaskAclAction.UpdateBisectTask, streamConfig.Id);
				}

				UpdateBisectTaskOptions options = new UpdateBisectTaskOptions();
				options.State = (!request.Cancel.HasValue) ? null : request.Cancel.Value ? BisectTaskState.Cancelled : BisectTaskState.Running;
				options.IncludeCommitIds = request.IncludeCommitIds;
				options.ExcludeCommitIds = request.ExcludeCommitIds;
				options.IncludeJobs = request.IncludeJobs;
				options.ExcludeJobs = request.ExcludeJobs;

				IBisectTask? updatedTask = await _bisectTaskCollection.TryUpdateAsync(bisectTask, options, cancellationToken);
				if (updatedTask != null)
				{
					// cancel any running bisection jobs
					if (bisectTask.State != BisectTaskState.Cancelled && updatedTask.State == BisectTaskState.Cancelled)
					{
						IJob? existingJob = await _jobCollection.FindBisectTaskJobsAsync(bisectTask.Id, true, cancellationToken).FirstOrDefaultAsync(cancellationToken);
						if (existingJob != null && existingJob.AbortedByUserId == null)
						{
							await _jobService.UpdateJobAsync(existingJob, null, null, null, User.GetUserId() ?? KnownUsers.System, null, null, null, cancellationToken: cancellationToken);
						}
					}
					return Ok();
				}
			}
		}

		async Task<GetBisectTaskResponse> CreateBisectTaskResponseAsync(IBisectTask task, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(BisectTasksController)}.{nameof(CreateBisectTaskResponseAsync)}");
			span.SetAttribute("TaskId", task.Id.ToString());

			IUser? user = await _userCollection.GetCachedUserAsync(task.OwnerId, cancellationToken);

			List<JobStepRefId> stepIds = new List<JobStepRefId>();
			if (task.MinJobStep != null)
			{
				stepIds.Add(task.MinJobStep.Value);
			}
			stepIds.Add(task.CurrentJobStep);
			stepIds.AddRange(task.Steps);
			stepIds.Add(task.InitialJobStep);

			List<IJobStepRef> steps = await _jobStepRefs.FindAsync(stepIds.ToArray(), cancellationToken);

			IJob? nextJob = task.State == BisectTaskState.Running ? await _jobCollection.FindBisectTaskJobsAsync(task.Id, true, cancellationToken).FirstOrDefaultAsync(cancellationToken) : null;
			return new GetBisectTaskResponse(task, user?.ToThinApiResponse(), steps, nextJob);
		}

		/// <summary>
		/// Gets the bisections run on a specific job
		/// </summary>
		/// <param name="jobId">Id of the job to retrieve</param>		
		/// <param name="cancellationToken" />
		/// <returns>Id of the new bisect task</returns>
		[HttpGet]
		[Route("/api/v1/bisect/job/{jobId}")]
		public async Task<ActionResult<List<GetBisectTaskResponse>>> GetJobBisectTasksAsync([FromRoute] JobId jobId, CancellationToken cancellationToken = default)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return new List<GetBisectTaskResponse>();
			}

			IReadOnlyList<IBisectTask> tasks = await _bisectTaskCollection.FindAsync(null, jobId, null, null, null, null, null, cancellationToken);

			List<GetBisectTaskResponse> response = new List<GetBisectTaskResponse>();

			foreach (IBisectTask bisectTask in tasks)
			{
				StreamConfig? streamConfig;
				if (!_buildConfig.Value.TryGetStream(bisectTask.StreamId, out streamConfig))
				{
					return NotFound(bisectTask.StreamId);
				}
				if (!streamConfig.Authorize(BisectTaskAclAction.ViewBisectTask, User))
				{
					return Forbid(BisectTaskAclAction.ViewBisectTask, streamConfig.Id);
				}

				response.Add(await CreateBisectTaskResponseAsync(bisectTask, cancellationToken));
			}

			return response;
		}

		/// <summary>
		/// Gets bisection tasks based on specified criteria
		/// </summary>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/bisect")]
		[ProducesResponseType(typeof(List<GetBisectTaskResponse>), 200)]
		public async Task<ActionResult<List<GetBisectTaskResponse>>> FindBisectTasksAsync(
			[FromQuery(Name = "id")] string[]? ids = null,
			[FromQuery] string? ownerId = null,
			[FromQuery] string? jobId = null,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 100,
			CancellationToken cancellationToken = default)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return new List<GetBisectTaskResponse>();
			}

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(BisectTasksController)}.{nameof(FindBisectTasksAsync)}");

			List<GetBisectTaskResponse> responses = new List<GetBisectTaskResponse>();
			BisectTaskId[]? bisectTaskIdValues;
			try
			{
				bisectTaskIdValues = (ids == null) ? (BisectTaskId[]?)null : Array.ConvertAll(ids, x => BisectTaskId.Parse(x));
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception parsing ids");
				return BadRequest("Unable to parse ids");
			}

			JobId? jobIdValue;
			try
			{
				jobIdValue = !String.IsNullOrEmpty(jobId) ? JobId.Parse(jobId) : null;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception parsing job id");
				return BadRequest("Unable to parse job id");
			}

			UserId? ownerIdValue;
			try
			{
				ownerIdValue = !String.IsNullOrEmpty(ownerId) ? UserId.Parse(ownerId) : null;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception parsing owner id");
				return BadRequest("Unable to parse owner id");
			}

			IReadOnlyList<IBisectTask> tasks = await _bisectTaskCollection.FindAsync(bisectTaskIdValues, jobIdValue, ownerIdValue, minCreateTime?.UtcDateTime, maxCreateTime?.UtcDateTime, index, count, cancellationToken);

			if (tasks.Count == 0)
			{
				return responses;
			}

			for (int i = 0; i < tasks.Count; i++)
			{
				IBisectTask task = tasks[i];
				responses.Add(await CreateBisectTaskResponseAsync(task, cancellationToken));
			}

			return responses;
		}
	}
}

