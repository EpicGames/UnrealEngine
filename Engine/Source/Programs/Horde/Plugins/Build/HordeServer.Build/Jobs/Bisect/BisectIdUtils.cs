// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs.Bisect;
using HordeServer.Utilities;

namespace HordeServer.Jobs.Bisect
{
	/// <summary>
	/// Server helper methods for <see cref="BisectTaskId"/>
	/// </summary>
	static class BisectTaskIdUtils
	{
		/// <summary>
		/// Creates a new <see cref="BisectTaskId"/>
		/// </summary>
		public static BisectTaskId GenerateNewId() => new BisectTaskId(BinaryIdUtils.CreateNew());
	}
}
