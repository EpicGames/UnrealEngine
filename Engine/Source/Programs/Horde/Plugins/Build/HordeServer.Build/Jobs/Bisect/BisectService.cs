// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using HordeServer.Commits;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Templates;
using HordeServer.Streams;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Driver;

namespace HordeServer.Jobs.Bisect
{
	class BisectService : IHostedService
	{
		readonly IBisectTaskCollection _bisectTaskCollection;
		readonly IJobCollection _jobCollection;
		readonly IJobStepRefCollection _jobStepRefCollection;
		readonly ITemplateCollectionInternal _templateCollection;
		readonly IGraphCollection _graphCollection;
		readonly ICommitService _commitService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IOptions<BuildServerConfig> _staticConfig;
		readonly ILogger _logger;
		readonly ITicker _ticker;

		public BisectService(IBisectTaskCollection bisectTaskCollection, IJobCollection jobCollection, IJobStepRefCollection jobStepRefCollection, ITemplateCollectionInternal templateCollection, IGraphCollection graphCollection, ICommitService commitService, IOptionsMonitor<BuildConfig> buildConfig, IOptions<BuildServerConfig> staticConfig, IClock clock, ILogger<BisectService> logger)
		{
			_bisectTaskCollection = bisectTaskCollection;
			_jobCollection = jobCollection;
			_jobStepRefCollection = jobStepRefCollection;
			_templateCollection = templateCollection;
			_graphCollection = graphCollection;
			_commitService = commitService;
			_buildConfig = buildConfig;
			_staticConfig = staticConfig;
			_logger = logger;
			_ticker = clock.AddSharedTicker<BisectService>(TimeSpan.FromMinutes(1.0), TickAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			if (_staticConfig.Value.DisableBisection)
			{
				return;
			}

			BuildConfig buildConfig = _buildConfig.CurrentValue;
			await foreach (IBisectTask bisectTask in _bisectTaskCollection.FindActiveAsync(cancellationToken))
			{
				IBisectTask? currentBisectTask = bisectTask;
				while (currentBisectTask != null)
				{
					BisectTaskState? newState = await TryUpdateBisectTaskAsync(currentBisectTask, buildConfig, cancellationToken);
					if (newState != null)
					{
						UpdateBisectTaskOptions options = new UpdateBisectTaskOptions();
						options.State = newState;

						if (newState == currentBisectTask.State || await _bisectTaskCollection.TryUpdateAsync(currentBisectTask, options, cancellationToken) != null)
						{
							break;
						}
					}
					currentBisectTask = await _bisectTaskCollection.GetAsync(bisectTask.Id, cancellationToken);
				}
			}
		}

		async Task<BisectTaskState?> TryUpdateBisectTaskAsync(IBisectTask bisectTask, BuildConfig buildConfig, CancellationToken cancellationToken)
		{
			// Check if there are any jobs triggered by this bisect task still running.
			IJob? existingJob = await _jobCollection.FindBisectTaskJobsAsync(bisectTask.Id, true, cancellationToken).FirstOrDefaultAsync(cancellationToken);
			if (existingJob != null)
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Waiting for result of job {JobId}", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, existingJob.Id);
				return BisectTaskState.Running;
			}

			// Find the step that ran before this one, ignoring any excluded jobs
			(IJobStepRef? currentJobStepRef, IJobStepRef? previousJobStepRef) = await GetBisectRangeAsync(bisectTask, cancellationToken);

			_logger.LogDebug("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Current Job Id: {CurJobId}/{CurStepOutcome}, Previous Job Id: {PrevJobId}/{PrevStepOutcome}", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, currentJobStepRef?.Id.JobId, currentJobStepRef?.Outcome, previousJobStepRef?.Id.JobId, previousJobStepRef?.Outcome);

			// Check if the task needs to be updated with the current bisect state
			if (currentJobStepRef != null && bisectTask.CurrentJobStep.JobId != currentJobStepRef.Id.JobId)
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Changing current job from {PrevJobId} ({PrevChange}) -> {NextJobId} ({NextChange}).", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, bisectTask.CurrentJobStep.JobId, bisectTask.CurrentCommitId, currentJobStepRef.Id.JobId, currentJobStepRef.CommitId);
				await _bisectTaskCollection.TryUpdateAsync(bisectTask, new UpdateBisectTaskOptions { CurrentJobStep = (currentJobStepRef.Id, currentJobStepRef.CommitId) }, cancellationToken);
				return null;
			}

			// Check that there's a lower bound for the search
			if (previousJobStepRef == null)
			{
				// No job before the current one
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): No previous node; marking search as complete.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName);
				return BisectTaskState.MissingHistory;
			}
			else if (previousJobStepRef.Outcome == null)
			{
				// Job is still running
				_logger.LogDebug("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Waiting on {JobStepRefId}", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, previousJobStepRef.Id);
				return BisectTaskState.Running;
			}

			// Update bisection lower bound
			if (bisectTask.MinJobStep == null)
			{
				await _bisectTaskCollection.TryUpdateAsync(bisectTask, new UpdateBisectTaskOptions { MinJobStep = (previousJobStepRef.Id, previousJobStepRef.CommitId) }, cancellationToken);
			}

			// Find the next commit to test
			StreamConfig? streamConfig;
			if (!buildConfig.TryGetStream(bisectTask.StreamId, out streamConfig))
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Stream no longer exists.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName);
				return BisectTaskState.MissingStream;
			}

			ICommitCollection commitCollection = _commitService.GetCollection(streamConfig);
			List<ICommit> commits = await commitCollection.FindAsync(previousJobStepRef.CommitId, includeMinCommit: false, bisectTask.CurrentCommitId, includeMaxCommit: false, 100, bisectTask.CommitTags, CommitSortOrder.Descending, cancellationToken).ToListAsync(cancellationToken);
			commits.RemoveAll(x => bisectTask.IgnoreCommitIds.Contains(x.Id));

			if (commits.Count == 0)
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Success - first failure was CL {Change}.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, bisectTask.CurrentCommitId);
				return BisectTaskState.Succeeded;
			}

			ICommit nextCommit = commits[commits.Count / 2];
			ICommit nextCodeCommit = await commitCollection.GetLastCodeChangeAsync(nextCommit.Id, cancellationToken) ?? nextCommit;

			// Get the initial job
			IJob? job = await _jobCollection.GetAsync(bisectTask.InitialJobStep.JobId, cancellationToken);
			if (job == null)
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Missing job {JobId}.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, bisectTask.InitialJobStep.JobId);
				return BisectTaskState.MissingJob;
			}

			// Get the template configuration
			TemplateRefConfig? templateRefConfig;
			if (!streamConfig.TryGetTemplate(bisectTask.TemplateId, out templateRefConfig))
			{
				_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): Template {TemplateId} not found.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, bisectTask.TemplateId);
				return BisectTaskState.MissingTemplate;
			}

			ITemplate template = await _templateCollection.GetOrAddAsync(templateRefConfig);
			IGraph graph = await _graphCollection.AddAsync(template, cancellationToken);

			CreateJobOptions options = new CreateJobOptions(templateRefConfig);
			options.StartedByBisectTaskId = bisectTask.Id;
			options.Priority = Priority.BelowNormal;
			options.UpdateIssues = template.UpdateIssues;
			options.Arguments.AddRange(job.Arguments.Where(x => !x.StartsWith(IJob.TargetArgumentPrefix, StringComparison.OrdinalIgnoreCase)));
			options.Arguments.Add($"{IJob.TargetArgumentPrefix}{bisectTask.NodeName}");

			IJob nextJob = await _jobCollection.AddAsync(JobIdUtils.GenerateNewId(), bisectTask.StreamId, bisectTask.TemplateId, template.Hash, graph, $"{template.Name} (Bisect)", nextCommit.Id, nextCodeCommit.Id, options, cancellationToken);
			_logger.LogInformation("Bisect task {BisectTaskId} ({StreamId}:{TemplateId}:{NodeName}): {NumCommits} possible commits ({MinChange}..{MaxChange}). Started new job {JobId} at CL {Change}.", bisectTask.Id, bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, commits.Count, commits[^1].Id, commits[0].Id, nextJob.Id, nextCommit.Id);

			return BisectTaskState.Running;
		}

		async Task<(IJobStepRef? Current, IJobStepRef? Next)> GetBisectRangeAsync(IBisectTask bisectTask, CancellationToken cancellationToken)
		{
			// Query jobs before the start position
			for (int maxJobCount = 20; ; maxJobCount += 20)
			{
				List<IJobStepRef> jobStepRefs = await _jobStepRefCollection.GetStepsForNodeAsync(bisectTask.StreamId, bisectTask.TemplateId, bisectTask.NodeName, bisectTask.InitialCommitId, true, maxJobCount, false, cancellationToken);

				IJobStepRef? currentJobStepRef = null;
				foreach (IJobStepRef jobStepRef in jobStepRefs.OrderByDescending(x => x.CommitId).ThenBy(x => x.Id))
				{
					if (!bisectTask.IgnoreJobIds.Contains(jobStepRef.Id.JobId))
					{
						if (jobStepRef.Outcome == bisectTask.Outcome)
						{
							currentJobStepRef = jobStepRef;
						}
						else
						{
							return (currentJobStepRef, jobStepRef);
						}
					}
				}

				if (jobStepRefs.Count < maxJobCount)
				{
					return (null, null);
				}
			}
		}
	}
}
