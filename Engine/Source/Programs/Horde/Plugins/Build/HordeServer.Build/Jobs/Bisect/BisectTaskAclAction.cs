// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Jobs.Bisect
{
	/// <summary>
	/// ACL actions valid for manipulating jobs
	/// </summary>
	public static class BisectTaskAclAction
	{
		/// <summary>
		/// Ability to start new bisect tasks
		/// </summary>
		public static AclAction CreateBisectTask { get; } = new AclAction("CreateBisectTask");

		/// <summary>
		/// Ability to update a bisect task
		/// </summary>
		public static AclAction UpdateBisectTask { get; } = new AclAction("UpdateBisectTask");

		/// <summary>
		/// Ability to view a bisect task
		/// </summary>
		public static AclAction ViewBisectTask { get; } = new AclAction("ViewBisectTask");
	}
}
