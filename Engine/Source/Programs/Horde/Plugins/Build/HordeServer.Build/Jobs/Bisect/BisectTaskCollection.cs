// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs.Bisect
{
	class BisectTaskCollection : IBisectTaskCollection
	{
		class BisectTaskDoc : IBisectTask
		{
			public BisectTaskId Id { get; set; }

			[BsonElement("running"), BsonIgnoreIfNull]
			public bool? Running { get; set; }

			[BsonElement("state")]
			public BisectTaskState State { get; set; }

			[BsonElement("oid")]
			public UserId OwnerId { get; set; }

			[BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonElement("tid")]
			public TemplateId TemplateId { get; set; }

			[BsonElement("step")]
			public string NodeName { get; set; } = String.Empty;

			[BsonElement("out")]
			public JobStepOutcome Outcome { get; set; }

			[BsonElement("ijobid")]
			public JobId InitialJobId { get; set; }

			[BsonElement("ijob")]
			public JobStepRefId InitialJobStep { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder InitialCommitId
			{
				get => (InitialCommitName != null) ? new CommitIdWithOrder(InitialCommitName, InitialCommitOrder) : CommitIdWithOrder.FromPerforceChange(InitialCommitOrder);
				set => (InitialCommitName, InitialCommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("icom")]
			public string? InitialCommitName { get; set; }

			[BsonElement("ichg")]
			public int InitialCommitOrder { get; set; }

			[BsonElement("curjob")]
			public JobStepRefId CurrentJobStep { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CurrentCommitId
			{
				get => (CurrentCommitName != null) ? new CommitIdWithOrder(CurrentCommitName, CurrentCommitOrder) : CommitIdWithOrder.FromPerforceChange(CurrentCommitOrder);
				set => (CurrentCommitName, CurrentCommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("curCom")]
			public string? CurrentCommitName { get; set; }

			[BsonElement("curChg")]
			public int CurrentCommitOrder { get; set; }

			// Lower bounds of bisection
			[BsonElement("minJob"), BsonIgnoreIfNull]
			public JobStepRefId? MinJobStep { get; set; }

			public CommitIdWithOrder? MinCommitId
			{
				get => (MinCommitName != null) ? new CommitIdWithOrder(MinCommitName, MinCommitOrder ?? 0) : CommitIdWithOrder.FromPerforceChange(MinCommitOrder);
				set => (MinCommitName, MinCommitOrder) = (value?.Name, value?.Order);
			}

			[BsonElement("minCom"), BsonIgnoreIfNull]
			public string? MinCommitName { get; set; }

			[BsonElement("minChg"), BsonIgnoreIfNull]
			public int? MinCommitOrder { get; set; }

			[BsonElement("steps")]
			public List<JobStepRefId> Steps { get; set; } = new List<JobStepRefId>();
			IReadOnlyList<JobStepRefId> IBisectTask.Steps => Steps;

			[BsonElement("idx")]
			public int UpdateIdx { get; set; }

			[BsonElement("tags"), BsonIgnoreIfNull]
			public List<CommitTag>? CommitTags { get; set; }
			IReadOnlyList<CommitTag>? IBisectTask.CommitTags => CommitTags;

			[BsonElement("nochg")]
			public HashSet<CommitId> IgnoreCommitIds { get; set; } = new HashSet<CommitId>();
			IReadOnlySet<CommitId> IBisectTask.IgnoreCommitIds => IgnoreCommitIds;

			[BsonElement("nojob")]
			public HashSet<JobId> IgnoreJobs { get; set; } = new HashSet<JobId>();
			IReadOnlySet<JobId> IBisectTask.IgnoreJobIds => IgnoreJobs;
		}

		readonly Tracer _tracer;
		readonly IMongoCollection<BisectTaskDoc> _bisectTasks;
		readonly MongoIndex<BisectTaskDoc> _runningIndex;

		public BisectTaskCollection(Tracer tracer, IMongoService mongoService)
		{
			List<MongoIndex<BisectTaskDoc>> indexes = new List<MongoIndex<BisectTaskDoc>>();
			indexes.Add(keys => keys.Ascending(x => x.Id).Ascending(x => x.InitialJobId));

			indexes.Add(_runningIndex = MongoIndex.Create<BisectTaskDoc>(keys => keys.Ascending(x => x.Running), sparse: true));

			_bisectTasks = mongoService.GetCollection<BisectTaskDoc>("BisectTasks", indexes);
			_tracer = tracer;
		}

		/// <inheritdoc/>
		public async Task<IBisectTask> CreateAsync(IJob job, JobStepBatchId batchId, JobStepId stepId, string nodeName, JobStepOutcome outcome, UserId ownerId, CreateBisectTaskOptions? options, CancellationToken cancellationToken = default)
		{
			BisectTaskDoc bisectTaskDoc = new BisectTaskDoc();
			bisectTaskDoc.Id = BisectTaskIdUtils.GenerateNewId();
			bisectTaskDoc.State = BisectTaskState.Running;
			bisectTaskDoc.Running = true;
			bisectTaskDoc.OwnerId = ownerId;
			bisectTaskDoc.StreamId = job.StreamId;
			bisectTaskDoc.TemplateId = job.TemplateId;
			bisectTaskDoc.NodeName = nodeName;
			bisectTaskDoc.Outcome = outcome;
			bisectTaskDoc.InitialJobId = job.Id;
			bisectTaskDoc.InitialJobStep = new JobStepRefId(job.Id, batchId, stepId);
			bisectTaskDoc.InitialCommitId = job.CommitId;
			bisectTaskDoc.CurrentJobStep = new JobStepRefId(job.Id, batchId, stepId);
			bisectTaskDoc.CurrentCommitId = job.CommitId;

			if (options != null)
			{
				if (options.CommitTags != null && options.CommitTags.Count > 0)
				{
					bisectTaskDoc.CommitTags = new List<CommitTag>(options.CommitTags);
				}
				if (options.IgnoreCommitIds != null)
				{
					bisectTaskDoc.IgnoreCommitIds.UnionWith(options.IgnoreCommitIds);
				}
				if (options.IgnoreJobIds != null)
				{
					bisectTaskDoc.IgnoreJobs.UnionWith(options.IgnoreJobIds);
				}
			}

			await _bisectTasks.InsertOneAsync(bisectTaskDoc, cancellationToken: cancellationToken);

			return bisectTaskDoc;
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<IBisectTask> FindActiveAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(BisectTaskCollection)}.{nameof(FindActiveAsync)}");

			FilterDefinitionBuilder<BisectTaskDoc> filterBuilder = Builders<BisectTaskDoc>.Filter;
			FilterDefinition<BisectTaskDoc> filter = filterBuilder.Exists(x => x.Running);
			filter &= filterBuilder.Eq(x => x.Running, true);

			using (IAsyncCursor<BisectTaskDoc> cursor = await _bisectTasks.FindWithHintAsync(filter, _runningIndex.Name, x => x.SortBy(x => x.Id!).ToCursorAsync(cancellationToken)))
			{
				while (await cursor.MoveNextAsync(cancellationToken))
				{
					foreach (BisectTaskDoc bisectTaskDoc in cursor.Current)
					{
						yield return bisectTaskDoc;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IBisectTask?> GetAsync(BisectTaskId bisectTaskId, CancellationToken cancellationToken = default)
		{
			return await _bisectTasks.Find(x => x.Id == bisectTaskId).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IBisectTask>> FindAsync(BisectTaskId[]? taskIds = null, JobId? jobId = null, UserId? ownerId = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(BisectTaskCollection)}.{nameof(FindAsync)}");

			// Find all the bisection tasks matching the given criteria
			FilterDefinitionBuilder<BisectTaskDoc> filterBuilder = Builders<BisectTaskDoc>.Filter;

			FilterDefinition<BisectTaskDoc> filter = FilterDefinition<BisectTaskDoc>.Empty;

			if (taskIds != null && taskIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Id, taskIds);
			}

			if (jobId != null)
			{
				filter &= filterBuilder.Eq(x => x.InitialJobId, jobId);
			}

			if (ownerId != null)
			{
				filter &= filterBuilder.Eq(x => x.OwnerId, ownerId);
			}

			if (minCreateTime != null)
			{
				BisectTaskId minTime = new BisectTaskId(BinaryIdUtils.FromObjectId(ObjectId.GenerateNewId(minCreateTime.Value)));
				filter &= filterBuilder.Gte(x => x.Id!, minTime);

			}
			if (maxCreateTime != null)
			{
				BisectTaskId maxTime = new BisectTaskId(BinaryIdUtils.FromObjectId(ObjectId.GenerateNewId(maxCreateTime.Value)));
				filter &= filterBuilder.Lte(x => x.Id!, maxTime);
			}

			List<BisectTaskDoc> steps = await _bisectTasks.Find(filter).SortByDescending(x => x.InitialCommitOrder).Range(index, count).ToListAsync(cancellationToken);
			return steps.ConvertAll<IBisectTask>(x => x);
		}

		/// <inheritdoc/>
		public async Task<IBisectTask?> TryUpdateAsync(IBisectTask bisectTask, UpdateBisectTaskOptions options, CancellationToken cancellationToken = default)
		{
			BisectTaskDoc bisectTaskDoc = (BisectTaskDoc)bisectTask;

			UpdateDefinition<BisectTaskDoc> update = Builders<BisectTaskDoc>.Update.Inc(x => x.UpdateIdx, 1);
			if (options.CurrentJobStep != null)
			{
				update = update
					.Set(x => x.CurrentJobStep, options.CurrentJobStep.Value.Step)
					.Set(x => x.CurrentCommitName, options.CurrentJobStep.Value.CommitId.Name)
					.Set(x => x.CurrentCommitOrder, options.CurrentJobStep.Value.CommitId.Order);
			}

			if (options.MinJobStep != null)
			{
				update = update
					.Set(x => x.MinJobStep, options.MinJobStep.Value.Step)
					.Set(x => x.MinCommitName, options.MinJobStep.Value.CommitId.Name)
					.Set(x => x.MinCommitOrder, options.MinJobStep.Value.CommitId.Order);
			}

			if (options.State != null)
			{
				update = update.Set(x => x.State, options.State.Value);
				if (options.State == BisectTaskState.Running)
				{
					update = update.Set(x => x.Running, true);
				}
				else
				{
					update = update.Set(x => x.Running, null);
				}
			}

			if (options.NewJobStep != null)
			{
				update = update.AddToSet(x => x.Steps, options.NewJobStep.Value);
			}

			if (options.IncludeCommitIds != null && options.IncludeCommitIds.Count > 0)
			{
				update = update.PullAll(x => x.IgnoreCommitIds, options.IncludeCommitIds);
			}
			else if (options.ExcludeCommitIds != null && options.ExcludeCommitIds.Count > 0)
			{
				update = update.AddToSetEach(x => x.IgnoreCommitIds, options.ExcludeCommitIds);
			}

			if (options.IncludeJobs != null && options.IncludeJobs.Count > 0)
			{
				update = update.PullAll(x => x.IgnoreJobs, options.IncludeJobs);
			}
			else if (options.ExcludeJobs != null && options.ExcludeJobs.Count > 0)
			{
				update = update.AddToSetEach(x => x.IgnoreJobs, options.ExcludeJobs);
			}

			FilterDefinition<BisectTaskDoc> filter = Builders<BisectTaskDoc>.Filter.Expr(x => x.Id == bisectTaskDoc.Id && x.UpdateIdx == bisectTaskDoc.UpdateIdx);
			return await _bisectTasks.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<BisectTaskDoc> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}
	}
}
