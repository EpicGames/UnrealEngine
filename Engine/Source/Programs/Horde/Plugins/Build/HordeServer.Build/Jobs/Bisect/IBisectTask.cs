// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;

namespace HordeServer.Jobs.Bisect
{
	/// <summary>
	/// State of a bisect task
	/// </summary>
	public enum BisectTaskState
	{
		/// <summary>
		/// Currently running
		/// </summary>
		Running,

		/// <summary>
		/// Cancelled by a user
		/// </summary>
		Cancelled,

		/// <summary>
		/// Finished running. The first job/change identifies the first failure.
		/// </summary>
		Succeeded,

		/// <summary>
		/// Task failed due to not having a job before the first failure.
		/// </summary>
		MissingHistory,

		/// <summary>
		/// Task failed due to the stream no longer existing.
		/// </summary>
		MissingStream,

		/// <summary>
		/// Task failed due to the first job no longer existing.
		/// </summary>
		MissingJob,

		/// <summary>
		/// Task failed due to template no longer existing.
		/// </summary>
		MissingTemplate,
	}

	/// <summary>
	/// State of a search for a build breakage
	/// </summary>
	public interface IBisectTask
	{
		/// <summary>
		/// Identifier for this task
		/// </summary>
		public BisectTaskId Id { get; }

		/// <summary>
		/// Current task state
		/// </summary>
		public BisectTaskState State { get; }

		/// <summary>
		/// User that initiated the search
		/// </summary>
		public UserId OwnerId { get; }

		/// <summary>
		/// Stream being searched
		/// </summary>
		public StreamId StreamId { get; }

		/// <summary>
		/// Template within the stream to execute
		/// </summary>
		public TemplateId TemplateId { get; }

		/// <summary>
		/// Name of the step to search for
		/// </summary>
		public string NodeName { get; }

		/// <summary>
		/// Outcome to search for
		/// </summary>
		public JobStepOutcome Outcome { get; }

		/// <summary>
		/// Starting job id for the bisection
		/// </summary>
		public JobStepRefId InitialJobStep { get; }

		/// <summary>
		/// Starting change for the bisection
		/// </summary>
		public CommitIdWithOrder InitialCommitId { get; }

		/// <summary>
		/// First known job id that is broken
		/// </summary>
		public JobStepRefId CurrentJobStep { get; }

		/// <summary>
		/// Changelist number of the first broken job id
		/// </summary>
		public CommitIdWithOrder CurrentCommitId { get; }

		/// <summary>
		/// Lower bound of bisection job Id
		/// </summary>
		public JobStepRefId? MinJobStep { get; }

		/// <summary>
		/// Lower bound of bisection changelist number
		/// </summary>
		public CommitIdWithOrder? MinCommitId { get; }

		/// <summary>
		/// The steps involved in the bisection
		/// </summary>
		public IReadOnlyList<JobStepRefId> Steps { get; }

		/// <summary>
		/// Tags to filter changes to consider for this task
		/// </summary>
		public IReadOnlyList<CommitTag>? CommitTags { get; }

		/// <summary>
		/// Set of changes to ignore while doing the bisection
		/// </summary>
		public IReadOnlySet<CommitId> IgnoreCommitIds { get; }

		/// <summary>
		/// Set of jobs to ignore while doing the bisection
		/// </summary>
		public IReadOnlySet<JobId> IgnoreJobIds { get; }
	}
}