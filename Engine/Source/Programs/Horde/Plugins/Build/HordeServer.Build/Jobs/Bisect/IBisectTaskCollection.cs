// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Users;
using Microsoft.Extensions.Logging;

namespace HordeServer.Jobs.Bisect
{
	/// <summary>
	/// Manages a collection of bisect tasks
	/// </summary>
	public interface IBisectTaskCollection
	{
		/// <summary>
		/// Creates a new bisect task
		/// </summary>
		/// <param name="job">First job to search back from</param>
		/// <param name="batchId">Starting batch id for the bisection</param>
		/// <param name="stepId">Starting batch id for the bisection</param>
		/// <param name="nodeName">Name of the node to search for</param>
		/// <param name="outcome">Outcome of the step to search for</param>
		/// <param name="ownerId">User that initiated the search</param>
		/// <param name="options">Options for the bisection</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new bisect task instance</returns>
		public Task<IBisectTask> CreateAsync(IJob job, JobStepBatchId batchId, JobStepId stepId, string nodeName, JobStepOutcome outcome, UserId ownerId, CreateBisectTaskOptions? options = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all the active bisect tasks
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of bisect tasks</returns>
		public IAsyncEnumerable<IBisectTask> FindActiveAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all bisect tasks for the provided criteria
		/// </summary>
		/// <param name="taskIds">array of bisection task ids</param>
		/// <param name="jobId">jobId of initial bisection</param>
		/// <param name="ownerId">ownerId of bisection</param>
		/// <param name="minCreateTime">min creation time of the bisection task</param>
		/// <param name="maxCreateTime">max creation time of the bisection task</param>
		/// <param name="index"></param>
		/// <param name="count"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of bisect tasks</returns>
		public Task<IReadOnlyList<IBisectTask>> FindAsync(BisectTaskId[]? taskIds = null, JobId? jobId = null, UserId? ownerId = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a bisect task by id
		/// </summary>
		/// <param name="bisectTaskId">Id of the bisect task</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<IBisectTask?> GetAsync(BisectTaskId bisectTaskId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates the state of a bisect task
		/// </summary>
		/// <param name="bisectTask">Task to update</param>
		/// <param name="options">Settings for updating the task</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task<IBisectTask?> TryUpdateAsync(IBisectTask bisectTask, UpdateBisectTaskOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Settings for creating a new bisect task
	/// </summary>
	public class CreateBisectTaskOptions
	{
		/// <summary>
		/// Filter the changes considered for the bisection
		/// </summary>
		public IReadOnlyList<CommitTag>? CommitTags { get; set; }

		/// <summary>
		/// Changes to exclude from the bisection
		/// </summary>
		public IReadOnlyList<CommitId>? IgnoreCommitIds { get; set; }

		/// <summary>
		/// Jobs to exclude from the bisection
		/// </summary>
		public IReadOnlyList<JobId>? IgnoreJobIds { get; set; }
	}

	/// <summary>
	/// Settings for updating a bisect task
	/// </summary>
	public class UpdateBisectTaskOptions
	{
		/// <summary>
		/// First job to bisect from
		/// </summary>
		public (JobStepRefId Step, CommitIdWithOrder CommitId)? CurrentJobStep { get; set; }

		/// <summary>
		/// The lower bounds of the bisection task
		/// </summary>
		public (JobStepRefId Step, CommitIdWithOrder CommitId)? MinJobStep { get; set; }

		/// <summary>
		/// New state for the task
		/// </summary>
		public BisectTaskState? State { get; set; }

		/// <summary>
		/// New job step to add to bisection
		/// </summary>
		public JobStepRefId? NewJobStep { get; set; }

		/// <summary>
		/// Changes to include in the bisection
		/// </summary>
		public IReadOnlyList<CommitId>? IncludeCommitIds { get; set; }

		/// <summary>
		/// Changes to exclude from the bisection
		/// </summary>
		public IReadOnlyList<CommitId>? ExcludeCommitIds { get; set; }

		/// <summary>
		/// Jobs to include in the bisection
		/// </summary>
		public IReadOnlyList<JobId>? IncludeJobs { get; set; }

		/// <summary>
		/// Jobs to exclude from the bisection
		/// </summary>
		public IReadOnlyList<JobId>? ExcludeJobs { get; set; }
	}

	static class BisectTaskCollectionExtensions
	{
		public static async Task UpdateAsync(this IBisectTaskCollection bisectTasks, IJob job, IJobStepBatch batch, IJobStep step, IGraph graph, ILogger? logger = null, CancellationToken cancellationToken = default)
		{
			if (job.StartedByBisectTaskId == null)
			{
				return;
			}

			IBisectTask? bisectTask = await bisectTasks.GetAsync(job.StartedByBisectTaskId.Value, cancellationToken);

			if (bisectTask == null)
			{
				return;
			}

			string nodeName = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx].Name;
			if (nodeName != bisectTask.NodeName)
			{
				return;
			}

			if (logger != null)
			{
				logger.LogInformation("Updating bisection task {TaskId} with reference {StepId} for job {JobId}, batch {BatchId}, with outcome", job.StartedByBisectTaskId, step.Id, job.Id, batch.Id);
			}

			await bisectTasks.TryUpdateAsync(bisectTask, new UpdateBisectTaskOptions() { NewJobStep = new JobStepRefId(job.Id, batch.Id, step.Id) }, cancellationToken);
		}
	}
}
