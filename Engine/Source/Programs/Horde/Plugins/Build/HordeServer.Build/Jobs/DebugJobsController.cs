// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Reflection;
using System.Text;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using HordeServer.Agents.Leases;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Debug functionality for jobs
	/// </summary>
	[ApiController]
	[Authorize]
	[DebugEndpoint]
	[Tags("Debug")]
	public class DebugJobsController : HordeControllerBase
	{
		private readonly JobService _jobService;
		private readonly JobTaskSource _jobTaskSource;
		private readonly ILeaseCollection _leaseCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly ILogger<DebugJobsController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public DebugJobsController(
			JobService jobService,
			JobTaskSource jobTaskSource,
			ILeaseCollection leaseCollection,
			IOptionsSnapshot<BuildConfig> buildConfig,
			ILogger<DebugJobsController> logger)
		{
			_jobService = jobService;
			_jobTaskSource = jobTaskSource;
			_leaseCollection = leaseCollection;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <summary>
		/// Returns diagnostic information about the current state of the queue
		/// </summary>
		/// <returns>Information about the queue</returns>
		[HttpGet]
		[Route("/api/v1/debug/queue")]
		public ActionResult<object> GetQueueStatus()
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			return _jobTaskSource.GetStatus();
		}

		/// <summary>
		/// Display a table listing each template with what job options are enabled
		/// </summary>
		/// <returns>Async task</returns>
		[HttpGet]
		[Route("/api/v1/debug/job-options")]
		public ActionResult GetJobOptions([FromQuery] string? format = "html")
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			List<PropertyInfo> joProps = typeof(JobOptions).GetProperties(BindingFlags.Public | BindingFlags.Instance).OrderBy(x => x.Name).ToList();

			if (format == "csv")
			{
				return GetJobOptionsAsCsv(joProps);
			}

			StringBuilder sb = new();

			sb.AppendLine("<style>");
			sb.AppendLine("body { font-family: 'Helvetica Neue', Helvetica, Arial, sans-serif; }");
			sb.AppendLine("table { border-collapse: collapse; width: 100%; font-size: 12px; }");
			sb.AppendLine("th, td { border: 1px solid black; text-align: left; padding: 8px; }");
			sb.AppendLine("th { background-color: #f2f2f2; }");
			sb.AppendLine("</style>");

			sb.AppendLine("<h1>Job options enabled by stream + template</h1>");
			sb.AppendLine("<table>");
			sb.AppendLine("<thead><tr>");
			sb.Append("<th>Stream</th>");
			sb.Append("<th>Template</th>");
			foreach (PropertyInfo prop in joProps)
			{
				sb.Append($"<th>{prop.Name}</th>");
			}
			sb.AppendLine("</tr></thead>");

			foreach (StreamConfig sc in _buildConfig.Value.Streams)
			{
				foreach (TemplateRefConfig tpl in sc.Templates)
				{
					sb.AppendLine("<tr>");
					sb.Append($"<td>{sc.Id}</td>");
					sb.Append($"<td>{tpl.Id}</td>");
					foreach (PropertyInfo prop in joProps)
					{
						sb.Append($"<td>{prop.GetValue(tpl.JobOptions)}</td>");
					}
					sb.AppendLine("</tr>");
				}
			}

			sb.AppendLine("</table>");
			return new ContentResult { ContentType = "text/html", StatusCode = (int)HttpStatusCode.OK, Content = sb.ToString() };
		}

		private ActionResult GetJobOptionsAsCsv(List<PropertyInfo> jobOptionsProps)
		{
			StringBuilder sb = new();

			List<string> headers = new() { "Stream", "Template" };
			headers.AddRange(jobOptionsProps.Select(prop => prop.Name));
			sb.AppendLine(String.Join('\t', headers));

			foreach (StreamConfig sc in _buildConfig.Value.Streams)
			{
				foreach (TemplateRefConfig tpl in sc.Templates)
				{
					List<string> row = new() { sc.Id.ToString(), tpl.Id.ToString() };
					row.AddRange(jobOptionsProps.Select(prop => prop.GetValue(tpl.JobOptions)?.ToString() ?? ""));
					sb.AppendLine(String.Join('\t', row));
				}
			}

			return new ContentResult { ContentType = "text/csv", StatusCode = (int)HttpStatusCode.OK, Content = sb.ToString() };
		}

		record JobTiming(
			string StreamId,
			string TemplateId,
			string Name,
			IReadOnlySet<string> StepNames,
			TimeSpan BatchSetupDuration,
			TimeSpan BatchWorkDuration,
			TimeSpan BatchTeardownDuration)
		{
			public JobTiming Merge(JobTiming jt)
			{
				if (StreamId != jt.StreamId || TemplateId != jt.TemplateId)
				{
					throw new ArgumentException("StreamId or TemplateId do not match");
				}

				HashSet<string> newNames = [.. StepNames.Union(jt.StepNames)];
				return new JobTiming(StreamId, TemplateId, Name, newNames,
					BatchSetupDuration + jt.BatchSetupDuration,
					BatchWorkDuration + jt.BatchWorkDuration,
					BatchTeardownDuration + jt.BatchTeardownDuration);
			}
		}

		/// <summary>
		/// Display a table listing each template with job timings (setup, work and teardown durations)
		/// </summary>
		/// <returns>Async task</returns>
		[HttpGet]
		[Route("/api/v1/debug/job-timings")]
		public async Task<ActionResult> GetJobTimingsAsync(
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] bool onlySetupBuild = false,
			[FromQuery] string? format = "html")
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			minCreateTime ??= DateTimeOffset.UtcNow.Subtract(TimeSpan.FromDays(1));
			IReadOnlyList<IJob> jobs = await _jobService.FindJobsAsync(new FindJobOptions(MinCreateTime: minCreateTime, MaxCreateTime: maxCreateTime));
			List<JobTiming> allJobTimings = await CalculateJobTimingsAsync(jobs);

			if (onlySetupBuild)
			{
				allJobTimings = allJobTimings.Where(x => x.StepNames.SetEquals(["Setup Build"])).ToList();
			}

			IReadOnlyList<JobTiming> jobTimingsByTemplate = GroupJobTimings(allJobTimings);
			IEnumerable<JobTiming> sortedJobTimings = jobTimingsByTemplate.OrderBy(x => x.BatchSetupDuration).Reverse();

			if (format == "csv")
			{
				return GetJobTimingsAsCsv(sortedJobTimings);
			}

			StringBuilder sb = new();

			sb.AppendLine("<style>");
			sb.AppendLine("body { font-family: 'Helvetica Neue', Helvetica, Arial, sans-serif; }");
			sb.AppendLine("table { border-collapse: collapse; width: 100%; font-size: 12px; }");
			sb.AppendLine("th, td { border: 1px solid black; text-align: left; padding: 8px; }");
			sb.AppendLine("th { background-color: #f2f2f2; }");
			sb.AppendLine("</style>");

			sb.AppendLine("<h1>Job timings by template</h1>");
			sb.AppendLine("<p>Durations specified in seconds.</p>");
			sb.AppendLine("<table>");
			sb.AppendLine("<thead><tr>");
			sb.Append("<th>Stream</th>");
			sb.Append("<th>Template</th>");
			sb.Append("<th>Name</th>");
			sb.Append("<th>Batch Setup</th>");
			sb.Append("<th>Batch Work</th>");
			sb.Append("<th>Batch Teardown</th>");
			sb.Append("<th>Steps</th>");
			sb.AppendLine("</tr></thead>");

			foreach (JobTiming jt in sortedJobTimings)
			{
				sb.AppendLine("<tr>");
				sb.Append($"<td>{jt.StreamId}</td>");
				sb.Append($"<td>{jt.TemplateId}</td>");
				sb.Append($"<td>{jt.Name}</td>");
				sb.Append($"<td>{(int)jt.BatchSetupDuration.TotalSeconds}</td>");
				sb.Append($"<td>{(int)jt.BatchWorkDuration.TotalSeconds}</td>");
				sb.Append($"<td>{(int)jt.BatchTeardownDuration.TotalSeconds}</td>");
				sb.Append($"<td>{String.Join(", ", jt.StepNames.Order())}</td>");
				sb.AppendLine("</tr>");
			}

			sb.AppendLine("</table>");
			return new ContentResult { ContentType = "text/html", StatusCode = (int)HttpStatusCode.OK, Content = sb.ToString() };
		}

		private static ActionResult GetJobTimingsAsCsv(IEnumerable<JobTiming> jobTimings)
		{
			StringBuilder sb = new();
			sb.AppendJoin('\t', ["Stream", "Template", "Name", "Batch Setup", "Batch Work", "Batch Teardown", "Steps"]).AppendLine();

			foreach (JobTiming jt in jobTimings)
			{
				sb.Append($"{jt.StreamId}\t");
				sb.Append($"{jt.TemplateId}\t");
				sb.Append($"{jt.Name}\t");
				sb.Append($"{(int)jt.BatchSetupDuration.TotalSeconds}\t");
				sb.Append($"{(int)jt.BatchWorkDuration.TotalSeconds}\t");
				sb.Append($"{(int)jt.BatchTeardownDuration.TotalSeconds}\t");
				sb.AppendJoin(',', jt.StepNames.Order());
				sb.AppendLine();
			}

			return new ContentResult { ContentType = "text/csv", StatusCode = (int)HttpStatusCode.OK, Content = sb.ToString() };
		}

		private async Task<List<JobTiming>> CalculateJobTimingsAsync(IEnumerable<IJob> jobs)
		{
			List<JobTiming> timings = [];
			foreach (IJob job in jobs)
			{
				IGraph graph = await _jobService.GetGraphAsync(job);

				foreach (IJobStepBatch batch in job.Batches)
				{
					if (batch.State != JobStepBatchState.Complete || batch.StartTimeUtc == null || batch.FinishTimeUtc == null)
					{
						continue;
					}
					DateTime firstStepStartTime = DateTime.MaxValue;
					DateTime lastStepFinishTime = DateTime.MinValue;

					HashSet<string> stepNames = [];
					foreach (IJobStep step in batch.Steps)
					{
						if (step.StartTimeUtc == null || step.FinishTimeUtc == null)
						{
							continue;
						}

						INode node = graph.GetNode(new NodeRef(batch.GroupIdx, step.NodeIdx));
						firstStepStartTime = step.StartTimeUtc.Value < firstStepStartTime ? step.StartTimeUtc.Value : firstStepStartTime;
						lastStepFinishTime = step.FinishTimeUtc.Value > lastStepFinishTime ? step.FinishTimeUtc.Value : lastStepFinishTime;
						stepNames.Add(node.Name);
					}

					if (firstStepStartTime == DateTime.MaxValue || lastStepFinishTime == DateTime.MinValue)
					{
						continue;
					}

					TimeSpan batchSetupDuration = firstStepStartTime - batch.StartTimeUtc.Value;
					TimeSpan batchWorkDuration = lastStepFinishTime - firstStepStartTime;
					TimeSpan batchTeardownDuration = batch.FinishTimeUtc.Value - lastStepFinishTime;
					timings.Add(new JobTiming(job.StreamId.ToString(), job.TemplateId.ToString(), job.Name, stepNames, batchSetupDuration, batchWorkDuration, batchTeardownDuration));
				}
			}

			return timings;
		}

		private static IReadOnlyList<JobTiming> GroupJobTimings(IEnumerable<JobTiming> jobTimings)
		{
			Dictionary<string, JobTiming> groupedTimings = new();
			foreach (JobTiming timing in jobTimings)
			{
				string key = $"{timing.StreamId}-{timing.TemplateId}";
				if (!groupedTimings.TryGetValue(key, out JobTiming? groupTiming))
				{
					groupedTimings[key] = timing;
				}
				else
				{
					groupedTimings[key] = groupTiming.Merge(timing);
				}
			}

			return groupedTimings.Values.ToList();
		}

		/// <summary>
		/// Repairs any inconsistencies with jobs created in a particular time range 
		/// </summary>
		[HttpPost]
		[Route("/api/v1/debug/repair-jobs")]
		public async Task<ActionResult> RepairJobsAsync([FromQuery] DateTime? minTime, [FromQuery] DateTime? maxTime)
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			if (minTime == null || maxTime == null)
			{
				return BadRequest();
			}

			IReadOnlyList<IJob> jobs = await _jobService.FindJobsAsync(new FindJobOptions(MinCreateTime: minTime, MaxCreateTime: maxTime));
			foreach (IJob job in jobs)
			{
				StreamConfig? streamConfig;
				if (_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
				{
					_logger.LogInformation("Checking job {JobId}", job.Id);
					await TryRepairJobAsync(streamConfig, job, HttpContext.RequestAborted);
				}
			}

			_logger.LogInformation("Finished repair");
			return Ok();
		}

		/// <summary>
		/// Repairs any inconsistencies with a particular job 
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		[HttpPost]
		[Route("/api/v1/debug/repair-job/{jobId}")]
		public async Task<ActionResult> RepairJobAsync(JobId jobId)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null || job.TemplateHash == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			for (; ; )
			{
				IJob? newJob = await TryRepairJobAsync(streamConfig, job, HttpContext.RequestAborted);
				if (newJob != null)
				{
					return Ok();
				}

				newJob = await _jobService.GetJobAsync(job.Id, HttpContext.RequestAborted);
				if (newJob == null)
				{
					return NotFound();
				}
			}
		}

		async Task<IJob?> TryRepairJobAsync(StreamConfig streamConfig, IJob job, CancellationToken cancellationToken)
		{
			// Check the lease has not already completed. Workaround for issue where jobs collection came out of sync with leases collection due to Mongo timeouts while updating indexes.
			List<JobStepBatchId> batchIds = job.Batches.Select(x => x.Id).ToList();
			foreach (JobStepBatchId batchId in batchIds)
			{
				IJobStepBatch? batch;
				if (job.TryGetBatch(batchId, out batch) && batch.LeaseId.HasValue && batch.State == JobStepBatchState.Running)
				{
					ILease? lease = await _leaseCollection.GetAsync(batch.LeaseId.Value, cancellationToken);
					if (lease != null && lease.FinishTime.HasValue)
					{
						_logger.LogWarning("Job {JobId} batch {BatchId} is out of sync with lease {LeaseId}", job.Id, batch.Id, lease.Id);

						IJob? newJob = await _jobService.UpdateBatchAsync(job, batch.Id, streamConfig, newState: JobStepBatchState.Complete, cancellationToken: cancellationToken);
						if (newJob == null)
						{
							return null;
						}

						job = newJob;
					}
				}
			}
			return job;
		}

		/// <summary>
		/// Forces an update of a job's batches to debug issues such as updating dependencies
		/// </summary>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/debug/batchupdate/{JobId}/{BatchId}")]
		public async Task<ActionResult> DebugBatchUpdateAsync(string jobId, string batchId)
		{
			if (!_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			IJob? job = await _jobService.GetJobAsync(JobId.Parse(jobId));

			if (job == null)
			{
				return NotFound();
			}

			await _jobService.TryUpdateBatchAsync(job, JobStepBatchId.Parse(batchId), newError: JobStepBatchError.None);

			return Ok();

		}
	}
}
