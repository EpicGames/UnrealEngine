// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using HordeServer.Server;
using Microsoft.Extensions.Caching.Memory;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;

namespace HordeServer.Jobs.Graphs
{
	/// <summary>
	/// Collection of graph documents
	/// </summary>
	public sealed class GraphCollection : IGraphCollection, IDisposable
	{
		/// <summary>
		/// Represents a node in the graph
		/// </summary>
		[DebuggerDisplay("{Name}")]
		class Node : INode
		{
			[BsonRequired]
			public string Name { get; set; }

			[BsonIgnoreIfNull]
			public NodeOutputRef[]? Inputs { get; set; }

			[BsonIgnoreIfNull]
			public NodeOutputRef[]? OptionalInputs { get; set; }

			[BsonIgnoreIfNull]
			public string[]? OutputNames { get; set; }

			[BsonIgnoreIfNull]
			public NodeRef[] InputDependencies { get; set; }

			[BsonIgnoreIfNull]
			public NodeRef[]? OptionalInputDependencies { get; set; }

			[BsonIgnoreIfNull]
			public NodeRef[] OrderDependencies { get; set; }

			public Priority Priority { get; set; }
			public bool AllowRetry { get; set; } = true;
			public bool RunEarly { get; set; }
			public bool Warnings { get; set; } = true;

			[BsonIgnoreIfNull]
			public Dictionary<string, string>? Credentials { get; set; }

			[BsonIgnoreIfNull]
			public Dictionary<string, string>? Properties { get; set; }

			[BsonIgnoreIfNull]
			[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public NodeAnnotations? Annotations { get; set; }

			IReadOnlyList<NodeOutputRef> INode.Inputs => Inputs ?? [];
			IReadOnlyList<NodeOutputRef> INode.OptionalInputs => OptionalInputs ?? [];
			IReadOnlyList<string> INode.OutputNames => OutputNames ?? [];
			IReadOnlyDictionary<string, string>? INode.Credentials => Credentials;
			IReadOnlyDictionary<string, string>? INode.Properties => Properties;
			NodeRef[] INode.OptionalInputDependencies => OptionalInputDependencies ?? [];
			IReadOnlyNodeAnnotations INode.Annotations => Annotations ?? NodeAnnotations.Empty;

			[BsonConstructor]
			private Node()
			{
				Name = null!;
				InputDependencies = null!;
				OrderDependencies = null!;
			}

			public Node(string name, NodeOutputRef[]? inputs, NodeOutputRef[]? optionalInputs, string[]? outputNames, NodeRef[] inputDependencies, NodeRef[]? optionalInputDependencies, NodeRef[] orderDependencies, Priority priority, bool allowRetry, bool runEarly, bool warnings, Dictionary<string, string>? credentials, Dictionary<string, string>? properties, IReadOnlyNodeAnnotations? annotations)
			{
				Name = name;
				Inputs = (inputs != null && inputs.Length > 0) ? inputs : null;
				OptionalInputs = (optionalInputs != null && optionalInputs.Length > 0) ? optionalInputs : null;
				OutputNames = (outputNames != null && outputNames.Length > 0) ? outputNames : null;
				InputDependencies = inputDependencies;
				OptionalInputDependencies = (optionalInputDependencies != null && optionalInputDependencies.Length > 0) ? optionalInputDependencies : null;
				OrderDependencies = orderDependencies;
				Priority = priority;
				AllowRetry = allowRetry;
				RunEarly = runEarly;
				Warnings = warnings;
				Credentials = credentials;
				Properties = properties;
				if (annotations != null && annotations.Count > 0)
				{
					Annotations = new NodeAnnotations(annotations);
				}
			}
		}

		class NodeGroup : INodeGroup
		{
			public string AgentType { get; set; }
			public List<Node> Nodes { get; set; }

			IReadOnlyList<INode> INodeGroup.Nodes => Nodes;

			/// <summary>
			/// Private constructor for BSON serializer
			/// </summary>
			[BsonConstructor]
			private NodeGroup()
			{
				AgentType = null!;
				Nodes = null!;
			}

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="agentType">The type of agent to execute this group</param>
			/// <param name="nodes">Nodes to execute</param>
			public NodeGroup(string agentType, List<Node> nodes)
			{
				AgentType = agentType;
				Nodes = nodes;
			}
		}

		class Aggregate : IAggregate
		{
			public string Name { get; set; }
			public List<NodeRef> Nodes { get; set; }

			IReadOnlyList<NodeRef> IAggregate.Nodes => Nodes;

			private Aggregate()
			{
				Name = null!;
				Nodes = new List<NodeRef>();
			}

			public Aggregate(string name, List<NodeRef> nodes)
			{
				Name = name;
				Nodes = nodes;
			}
		}

		class Label : ILabel
		{
			[BsonIgnoreIfNull]
			public string? Name { get; set; }

			[BsonIgnoreIfNull]
			public string? Category { get; set; }

			[BsonIgnoreIfNull]
			public string? DashboardName { get; set; }

			[BsonIgnoreIfNull]
			public string? DashboardCategory { get; set; }

			[BsonIgnoreIfNull]
			public string? UgsName { get; set; }

			[BsonIgnoreIfNull]
			public string? UgsProject { get; set; }

			[BsonIgnoreIfNull]
			public LabelChange Change { get; set; }

			public List<NodeRef> RequiredNodes { get; set; }
			public List<NodeRef> IncludedNodes { get; set; }

			string? ILabel.DashboardName => DashboardName ?? Name;
			string? ILabel.DashboardCategory => DashboardCategory ?? Category;

			private Label()
			{
				RequiredNodes = new List<NodeRef>();
				IncludedNodes = new List<NodeRef>();
			}

			public Label(string? dashboardName, string? dashboardCategory, string? ugsName, string? ugsProject, LabelChange change, List<NodeRef> requiredNodes, List<NodeRef> includedNodes)
			{
				DashboardName = dashboardName;
				DashboardCategory = dashboardCategory;
				UgsName = ugsName;
				UgsProject = ugsProject;
				Change = change;
				RequiredNodes = requiredNodes;
				IncludedNodes = includedNodes;
			}
		}

		class GraphArtifact : IGraphArtifact
		{
			public ArtifactName Name { get; set; }
			public ArtifactType Type { get; set; }
			public string Description { get; set; }
			public string BasePath { get; set; }
			public List<string> Keys { get; set; }
			public List<string> Metadata { get; set; }
			public string? NodeName { get; set; }
			public string? OutputName { get; set; }

			IReadOnlyList<string> IGraphArtifact.Keys => Keys;
			IReadOnlyList<string> IGraphArtifact.Metadata => Metadata;

			private GraphArtifact()
			{
				Description = String.Empty;
				BasePath = String.Empty;
				Keys = new List<string>();
				Metadata = new List<string>();
			}

			public GraphArtifact(ArtifactName name, ArtifactType type, string description, string basePath, IReadOnlyList<string> keys, IReadOnlyList<string> metadata, string? nodeName, string? outputName)
			{
				Name = name;
				Type = type;
				Description = description;
				BasePath = basePath;
				Keys = keys.ToList();
				Metadata = metadata.ToList();
				NodeName = nodeName;
				OutputName = outputName;
			}
		}

		class GraphDocument : IGraph
		{
			public static GraphDocument Empty { get; } = new GraphDocument(new List<NodeGroup>(), new List<Aggregate>(), new List<Label>(), new List<GraphArtifact>());

			[BsonRequired, BsonId]
			public ContentHash Id { get; private set; } = ContentHash.Empty;

			public int Schema { get; set; }
			public List<NodeGroup> Groups { get; private set; } = new List<NodeGroup>();
			public List<Aggregate> Aggregates { get; private set; } = new List<Aggregate>();
			public List<Label> Labels { get; private set; } = new List<Label>();
			public List<GraphArtifact> Artifacts { get; private set; } = new List<GraphArtifact>();

			[BsonIgnore]
			IReadOnlyDictionary<string, NodeRef>? _cachedNodeNameToRef;

			[BsonIgnore]
			IReadOnlyDictionary<string, NodeOutputRef>? _cachedNodeOutputNameToRef;

			IReadOnlyList<INodeGroup> IGraph.Groups => Groups;
			IReadOnlyList<IAggregate> IGraph.Aggregates => Aggregates;
			IReadOnlyList<ILabel> IGraph.Labels => Labels;
			IReadOnlyList<IGraphArtifact> IGraph.Artifacts => Artifacts;

			[BsonConstructor]
			private GraphDocument()
			{
			}

			public GraphDocument(List<NodeGroup> groups, List<Aggregate> aggregates, List<Label> labels, List<GraphArtifact> artifacts)
			{
				Groups = groups;
				Aggregates = aggregates;
				Labels = labels;
				Artifacts = artifacts;
				Id = ContentHash.SHA1(BsonExtensionMethods.ToBson(this));
			}

			public GraphDocument(GraphDocument baseGraph, List<NewGroup>? newGroupRequests, List<NewAggregate>? newAggregateRequests, List<NewLabel>? newLabelRequests, List<NewGraphArtifact>? newArtifactRequests)
			{
				Dictionary<string, NodeRef> nodeNameToRef = new Dictionary<string, NodeRef>(baseGraph.GetNodeNameToRef(), StringComparer.OrdinalIgnoreCase);
				Dictionary<string, NodeOutputRef> nodeOutputNameToRef = new Dictionary<string, NodeOutputRef>(baseGraph.GetNodeOutputNameToRef(), StringComparer.OrdinalIgnoreCase);

				// Update the new list of groups
				List<NodeGroup> newGroups = new List<NodeGroup>(baseGraph.Groups);
				if (newGroupRequests != null)
				{
					foreach (NewGroup newGroupRequest in newGroupRequests)
					{
						List<Node> nodes = new List<Node>();
						foreach (NewNode newNodeRequest in newGroupRequest.Nodes)
						{
							int nodeIdx = nodes.Count;

							Priority priority = newNodeRequest.Priority ?? Priority.Normal;
							bool allowRetry = newNodeRequest.AllowRetry ?? true;
							bool runEarly = newNodeRequest.RunEarly ?? false;
							bool warnings = newNodeRequest.Warnings ?? true;

							NodeOutputRef[]? inputs = null;
							if (newNodeRequest.Inputs != null && newNodeRequest.Inputs.Count > 0)
							{
								inputs = newNodeRequest.Inputs.Select(x => nodeOutputNameToRef[x]).ToArray();
							}

							NodeOutputRef[]? optionalInputs = null;
							if (newNodeRequest.OptionalInputs != null && newNodeRequest.OptionalInputs.Count > 0)
							{
								optionalInputs = newNodeRequest.OptionalInputs.Select(x => nodeOutputNameToRef[x]).ToArray();
							}

							NodeRef[] inputDependencies = (newNodeRequest.InputDependencies == null) ? [] : newNodeRequest.InputDependencies.Select(x => nodeNameToRef[x]).ToArray();
							NodeRef[] optionalInputDependencies = (newNodeRequest.OptionalInputDependencies == null) ? [] : newNodeRequest.OptionalInputDependencies.Select(x => nodeNameToRef[x]).ToArray();
							NodeRef[] orderDependencies = (newNodeRequest.OrderDependencies == null) ? [] : newNodeRequest.OrderDependencies.Select(x => nodeNameToRef[x]).ToArray();
							orderDependencies = orderDependencies.Union(inputDependencies).Union(optionalInputDependencies).ToArray();
							nodes.Add(new Node(newNodeRequest.Name, inputs, optionalInputs, newNodeRequest.Outputs?.ToArray(), inputDependencies, optionalInputDependencies, orderDependencies, priority, allowRetry, runEarly, warnings, newNodeRequest.Credentials, newNodeRequest.Properties, newNodeRequest.Annotations));

							NodeRef nodeRef = new NodeRef(newGroups.Count, nodeIdx);
							nodeNameToRef.Add(newNodeRequest.Name, nodeRef);

							if (newNodeRequest.Outputs != null)
							{
								for (int outputIdx = 0; outputIdx < newNodeRequest.Outputs.Count; outputIdx++)
								{
									string outputName = newNodeRequest.Outputs[outputIdx];
									nodeOutputNameToRef[outputName] = new NodeOutputRef(nodeRef, outputIdx);
								}
							}
						}
						newGroups.Add(new NodeGroup(newGroupRequest.AgentType, nodes));
					}
				}

				// Update the list of aggregates
				List<Aggregate> newAggregates = new List<Aggregate>(baseGraph.Aggregates);
				if (newAggregateRequests != null)
				{
					foreach (NewAggregate newAggregateRequest in newAggregateRequests)
					{
						List<NodeRef> nodes = newAggregateRequest.Nodes.ConvertAll(x => nodeNameToRef[x]);
						newAggregates.Add(new Aggregate(newAggregateRequest.Name, nodes));
					}
				}

				// Update the list of labels
				List<Label> newLabels = new List<Label>(baseGraph.Labels);
				if (newLabelRequests != null)
				{
					foreach (NewLabel newLabelRequest in newLabelRequests)
					{
						List<NodeRef> requiredNodes = newLabelRequest.RequiredNodes.ConvertAll(x => nodeNameToRef[x]);
						List<NodeRef> includedNodes = newLabelRequest.IncludedNodes.ConvertAll(x => nodeNameToRef[x]);
						newLabels.Add(new Label(newLabelRequest.DashboardName, newLabelRequest.DashboardCategory, newLabelRequest.UgsName, newLabelRequest.UgsProject, newLabelRequest.Change, requiredNodes, includedNodes));
					}
				}

				// Update the list of artifacts
				List<GraphArtifact> newArtifacts = new List<GraphArtifact>(baseGraph.Artifacts);
				if (newArtifactRequests != null)
				{
					foreach (NewGraphArtifact newArtifactRequest in newArtifactRequests)
					{
						newArtifacts.Add(new GraphArtifact(newArtifactRequest.Name, newArtifactRequest.Type, newArtifactRequest.Description, newArtifactRequest.BasePath, newArtifactRequest.Keys, newArtifactRequest.Metadata, newArtifactRequest.NodeName, newArtifactRequest.OutputName));
					}
				}

				HashSet<ArtifactName> uniqueArtifactNames = new HashSet<ArtifactName>();
				foreach (GraphArtifact newArtifact in newArtifacts)
				{
					if (!uniqueArtifactNames.Add(newArtifact.Name))
					{
						throw new InvalidOperationException($"Artifact '{newArtifact.Name}' was registered multiple times in the same graph");
					}
				}

				// Create the new arrays
				Groups = newGroups;
				Aggregates = newAggregates;
				Labels = newLabels;
				Artifacts = newArtifacts;

				// Create the new graph, and save the generated node lookup into it
				_cachedNodeNameToRef = nodeNameToRef;

				// Compute the hash
				Id = ContentHash.SHA1(BsonExtensionMethods.ToBson(this));
			}

			public IReadOnlyDictionary<string, NodeRef> GetNodeNameToRef()
			{
				if (_cachedNodeNameToRef == null)
				{
					Dictionary<string, NodeRef> nodeNameToRef = new Dictionary<string, NodeRef>(StringComparer.OrdinalIgnoreCase);
					for (int groupIdx = 0; groupIdx < Groups.Count; groupIdx++)
					{
						List<Node> nodes = Groups[groupIdx].Nodes;
						for (int nodeIdx = 0; nodeIdx < nodes.Count; nodeIdx++)
						{
							Node node = nodes[nodeIdx];
							nodeNameToRef[node.Name] = new NodeRef(groupIdx, nodeIdx);
						}
					}
					_cachedNodeNameToRef = nodeNameToRef;
				}
				return _cachedNodeNameToRef;
			}

			public IReadOnlyDictionary<string, NodeOutputRef> GetNodeOutputNameToRef()
			{
				if (_cachedNodeOutputNameToRef == null)
				{
					Dictionary<string, NodeOutputRef> nodeOutputNameToRef = new Dictionary<string, NodeOutputRef>(StringComparer.OrdinalIgnoreCase);
					for (int groupIdx = 0; groupIdx < Groups.Count; groupIdx++)
					{
						List<Node> nodes = Groups[groupIdx].Nodes;
						for (int nodeIdx = 0; nodeIdx < nodes.Count; nodeIdx++)
						{
							Node node = nodes[nodeIdx];
							if (node.OutputNames != null)
							{
								for (int outputIdx = 0; outputIdx < node.OutputNames.Length; outputIdx++)
								{
									string outputName = node.OutputNames[outputIdx];
									nodeOutputNameToRef[outputName] = new NodeOutputRef(new NodeRef(groupIdx, nodeIdx), outputIdx);
								}
							}
						}
					}
					_cachedNodeOutputNameToRef = nodeOutputNameToRef;
				}
				return _cachedNodeOutputNameToRef;
			}
		}

		/// <summary>
		/// The jobs collection
		/// </summary>
		readonly IMongoCollection<GraphDocument> _graphs;

		/// <summary>
		/// Maximum number of graphs to keep in the cache
		/// </summary>
		const int MaxGraphs = 2000;

		/// <summary>
		/// Cache for graphs
		/// Use a non-shared cache to ensure enough space for graphs
		/// </summary>
		private readonly MemoryCache _memoryCache = new MemoryCache(new MemoryCacheOptions() { SizeLimit = MaxGraphs });

		/// <summary>
		/// Constructor
		/// </summary>
		public GraphCollection(IMongoService mongoService)
		{
			_graphs = mongoService.GetCollection<GraphDocument>("Graphs");
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_memoryCache.Dispose();
		}

		/// <summary>
		/// Adds a new graph document
		/// </summary>
		/// <param name="graph">The graph to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		async Task AddAsync(GraphDocument graph, CancellationToken cancellationToken)
		{
			if (!await _graphs.Find(x => x.Id == graph.Id).AnyAsync(cancellationToken))
			{
				try
				{
					await _graphs.InsertOneAsync(graph, null, cancellationToken);
				}
				catch (MongoWriteException ex)
				{
					if (ex.WriteError.Category != ServerErrorCategory.DuplicateKey)
					{
						throw;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IGraph> AddAsync(ITemplate template, CancellationToken cancellationToken)
		{
			Node node = new Node(IJob.SetupNodeName, null, null, null, [], [], [], Priority.High, true, false, true, null, null, null);
			NodeGroup group = new NodeGroup(template.InitialAgentType ?? "Win64", new List<Node> { node });

			GraphDocument graph = new GraphDocument(new List<NodeGroup> { group }, new List<Aggregate>(), new List<Label>(), new List<GraphArtifact>());
			await AddAsync(graph, cancellationToken);
			return graph;
		}

		/// <inheritdoc/>
		public async Task<IGraph> AppendAsync(IGraph? baseGraph, List<NewGroup>? newGroupRequests, List<NewAggregate>? newAggregateRequests, List<NewLabel>? newLabelRequests, List<NewGraphArtifact>? newArtifactRequests, CancellationToken cancellationToken)
		{
			GraphDocument graph = new GraphDocument((GraphDocument?)baseGraph ?? GraphDocument.Empty, newGroupRequests, newAggregateRequests, newLabelRequests, newArtifactRequests);
			await AddAsync(graph, cancellationToken);
			return graph;
		}

		/// <inheritdoc/>
		public async Task<IGraph> GetAsync(ContentHash? hash, CancellationToken cancellationToken)
		{
			// Special case for an empty graph request
			if (hash == null || hash == ContentHash.Empty || hash == GraphDocument.Empty.Id)
			{
				return GraphDocument.Empty;
			}

			async Task<GraphDocument> CreateCacheEntry(ICacheEntry cacheEntry)
			{
				cacheEntry.SlidingExpiration = TimeSpan.FromHours(24);
				cacheEntry.Size = 1;

				GraphDocument document = await _graphs.Find<GraphDocument>(x => x.Id == hash).FirstAsync(cancellationToken);
				return document;
			}

			return (await _memoryCache.GetOrCreateAsync(hash, CreateCacheEntry))!;
		}

		/// <inheritdoc/>
		public async Task<List<IGraph>> FindAllAsync(ContentHash[]? hashes, int? index, int? count, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<GraphDocument> filterBuilder = Builders<GraphDocument>.Filter;

			FilterDefinition<GraphDocument> filter = filterBuilder.Empty;
			if (hashes != null)
			{
				filter &= filterBuilder.In(x => x.Id, hashes);
			}

			List<GraphDocument> results;
			IFindFluent<GraphDocument, GraphDocument> search = _graphs.Find(filter);
			if (index != null)
			{
				search = search.Skip(index.Value);
			}
			if (count != null)
			{
				search = search.Limit(count.Value);
			}

			results = await search.ToListAsync(cancellationToken);
			return results.ConvertAll<IGraph>(x => x);
		}
	}
}
