// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;

namespace HordeServer.Jobs.Graphs
{
	/// <summary>
	/// Interface for a collection of graph documents
	/// </summary>
	public interface IGraphCollection
	{
		/// <summary>
		/// Adds a graph from a template
		/// </summary>
		/// <param name="template">The template</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New graph</returns>
		Task<IGraph> AddAsync(ITemplate template, CancellationToken cancellationToken = default);

		/// <summary>
		/// Creates a graph by appending groups and aggregates to an existing graph.
		/// </summary>
		/// <param name="baseGraph">The base graph</param>
		/// <param name="newGroupRequests">List of group requests</param>
		/// <param name="newAggregateRequests">List of aggregate requests</param>
		/// <param name="newLabelRequests">List of label requests</param>
		/// <param name="newArtifactRequests">List of artifact requests</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new graph definition</returns>
		Task<IGraph> AppendAsync(IGraph? baseGraph, List<NewGroup>? newGroupRequests = null, List<NewAggregate>? newAggregateRequests = null, List<NewLabel>? newLabelRequests = null, List<NewGraphArtifact>? newArtifactRequests = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the graph for a job
		/// </summary>
		/// <param name="hash">Hash of the graph to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The graph for this job</returns>
		Task<IGraph> GetAsync(ContentHash hash, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all graphs stored in the collection
		/// </summary>
		/// <param name="hashes">Hashes to filter by</param>
		/// <param name="index">Starting index of the graph to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of graphs</returns>
		Task<List<IGraph>> FindAllAsync(ContentHash[]? hashes = null, int? index = null, int? count = null, CancellationToken cancellationToken = default);
	}
}
