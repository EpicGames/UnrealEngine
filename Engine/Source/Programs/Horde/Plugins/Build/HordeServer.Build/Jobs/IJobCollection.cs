// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Acls;
using HordeServer.Streams;

#pragma warning disable CA2227 //  Change x to be read-only by removing the property setter

namespace HordeServer.Jobs
{
	/// <summary>
	/// Options for creating a new job
	/// </summary>
	public class CreateJobOptions
	{
		/// <inheritdoc cref="IJob.PreflightCommitId"/>
		public CommitId? PreflightCommitId { get; set; }

		/// <inheritdoc cref="IJob.PreflightDescription"/>
		public string? PreflightDescription { get; set; }

		/// <inheritdoc cref="IJob.StartedByUserId"/>
		public UserId? StartedByUserId { get; set; }

		/// <inheritdoc cref="IJob.StartedByBisectTaskId"/>
		public BisectTaskId? StartedByBisectTaskId { get; set; }

		/// <inheritdoc cref="IJob.Priority"/>
		public Priority? Priority { get; set; }

		/// <inheritdoc cref="IJob.AutoSubmit"/>
		public bool? AutoSubmit { get; set; }

		/// <inheritdoc cref="IJob.UpdateIssues"/>
		public bool? UpdateIssues { get; set; }

		/// <inheritdoc cref="IJob.PromoteIssuesByDefault"/>
		public bool? PromoteIssuesByDefault { get; set; }

		/// <inheritdoc cref="IJob.JobOptions"/>
		public JobOptions? JobOptions { get; set; }

		/// <inheritdoc cref="IJob.Claims"/>
		public List<AclClaimConfig> Claims { get; set; } = new List<AclClaimConfig>();

		/// <summary>
		/// List of downstream job triggers
		/// </summary>
		public List<ChainedJobTemplateConfig> JobTriggers { get; } = new List<ChainedJobTemplateConfig>();

		/// <inheritdoc cref="IJob.ShowUgsBadges"/>
		public bool ShowUgsBadges { get; set; }

		/// <inheritdoc cref="IJob.ShowUgsAlerts"/>
		public bool ShowUgsAlerts { get; set; }

		/// <inheritdoc cref="IJob.NotificationChannel"/>
		public string? NotificationChannel { get; set; }

		/// <inheritdoc cref="IJob.NotificationChannelFilter"/>
		public string? NotificationChannelFilter { get; set; }

		/// <inheritdoc cref="IJob.Parameters"/>
		public Dictionary<ParameterId, string> Parameters { get; } = new Dictionary<ParameterId, string>();

		/// <inheritdoc cref="IJob.Arguments"/>
		public List<string> Arguments { get; } = new List<string>();

		/// <inheritdoc cref="IJob.Targets"/>
		public List<string>? Targets { get; set; }

		/// <inheritdoc cref="IJob.AdditionalArguments"/>
		public List<string> AdditionalArguments { get; } = new List<string>();

		/// <inheritdoc cref="IJob.Environment"/>
		public Dictionary<string, string> Environment { get; } = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

		/// <inheritdoc cref="IJob.ParentJobId"/>
		public JobId? ParentJobId { get; set; }

		/// <inheritdoc cref="IJob.ParentJobStepId"/>
		public JobStepId? ParentJobStepId { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public CreateJobOptions()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public CreateJobOptions(TemplateRefConfig templateRefConfig)
		{
			JobOptions = templateRefConfig.JobOptions;
			PromoteIssuesByDefault = templateRefConfig.PromoteIssuesByDefault;
			if (templateRefConfig.ChainedJobs != null)
			{
				JobTriggers.AddRange(templateRefConfig.ChainedJobs);
			}
			ShowUgsBadges = templateRefConfig.ShowUgsBadges;
			ShowUgsAlerts = templateRefConfig.ShowUgsAlerts;
			NotificationChannel = templateRefConfig.NotificationChannel;
			NotificationChannelFilter = templateRefConfig.NotificationChannelFilter;
		}
	}

	/// <summary>
	/// Interface for a collection of job documents
	/// </summary>
	public interface IJobCollection
	{
		/// <summary>
		/// Creates a new job
		/// </summary>
		/// <param name="jobId">A requested job id</param>
		/// <param name="streamId">Unique id of the stream that this job belongs to</param>
		/// <param name="templateRefId">Name of the template ref</param>
		/// <param name="templateHash">Template for this job</param>
		/// <param name="graph">The graph for the new job</param>
		/// <param name="name">Name of the job</param>
		/// <param name="commitId">The commit to build</param>
		/// <param name="codeCommitId">The corresponding code changelist number</param>
		/// <param name="options">Additional options for the new job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new job document</returns>
		Task<IJob> AddAsync(JobId jobId, StreamId streamId, TemplateId templateRefId, ContentHash templateHash, IGraph graph, string name, CommitId commitId, CommitId? codeCommitId, CreateJobOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a job with the given unique id
		/// </summary>
		/// <param name="jobId">Job id to search for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the given job</returns>
		Task<IJob?> GetAsync(JobId jobId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Delete all the jobs for a stream
		/// </summary>
		/// <param name="streamId">Unique id of the stream</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task RemoveStreamAsync(StreamId streamId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for jobs matching the given criteria
		/// </summary>
		/// <param name="options">Options for the search</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of jobs matching the given criteria</returns>
		Task<IReadOnlyList<IJob>> FindAsync(FindJobOptions options, int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for jobs matching the given criteria
		/// </summary>
		/// <param name="bisectTaskId">The bisect task to find jobs for</param>
		/// <param name="running">Whether to filter by running jobs</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of jobs matching the given criteria</returns>
		IAsyncEnumerable<IJob> FindBisectTaskJobsAsync(BisectTaskId bisectTaskId, bool? running, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for jobs in a specified stream and templates
		/// </summary>
		/// <param name="streamId">The stream containing the job</param>
		/// <param name="templates">Templates to look for</param>
		/// <param name="preflightStartedByUser">User for which to include preflight jobs</param>
		/// <param name="maxCreateTime">The maximum creation time</param>
		/// <param name="modifiedAfter">Filter the results by modified time</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="consistentRead">If the database read should be made to the replica server</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IReadOnlyList<IJob>> FindLatestByStreamWithTemplatesAsync(StreamId streamId, TemplateId[] templates, UserId? preflightStartedByUser = null, DateTimeOffset? maxCreateTime = null, DateTimeOffset? modifiedAfter = null, int? index = null, int? count = null, bool consistentRead = false, CancellationToken cancellationToken = default);

		/// <summary>
		/// Adds an issue to a job
		/// </summary>
		/// <param name="jobId">The job id</param>
		/// <param name="issueId">The issue to add</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task AddIssueToJobAsync(JobId jobId, int issueId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a queue of jobs to consider for execution
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sorted list of jobs to execute</returns>
		Task<IReadOnlyList<IJob>> GetDispatchQueueAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Write to the job's audit log
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="message"></param>
		/// <param name="args"></param>
		void JobAudit(JobId jobId, string? message, params object?[] args);

		/// <summary>
		/// Update a job and step metadata
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="jobMetadata"></param>
		/// <param name="stepMetadata"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<IJob?> UpdateMetadataAsync(JobId jobId, List<string>? jobMetadata = null, Dictionary<JobStepId, List<string>>? stepMetadata = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Upgrade all documents in the collection
		/// </summary>
		/// <returns>Async task</returns>
		Task UpgradeDocumentsAsync();
	}

	/// <summary>
	/// Options for finding jobs
	/// </summary>
	/// <param name="JobIds">List of job ids to return</param>
	/// <param name="StreamId">The stream containing the job</param>
	/// <param name="Name">Name of the job</param>
	/// <param name="Templates">Templates to look for</param>
	/// <param name="MinCommitId">The minimum commit</param>
	/// <param name="MaxCommitId">The maximum commit</param>
	/// <param name="PreflightCommitId">Preflight change to find</param>
	/// <param name="PreflightOnly">Whether to only include preflights</param>
	/// <param name="IncludePreflight">Whether to include preflights in the results</param>
	/// <param name="StartedByUser">User id for which to include jobs</param>
	/// <param name="PreflightStartedByUser">User for which to include preflight jobs</param>
	/// <param name="MinCreateTime">The minimum creation time</param>
	/// <param name="MaxCreateTime">The maximum creation time</param>
	/// <param name="ModifiedBefore">Filter the results by modified time</param>
	/// <param name="ModifiedAfter">Filter the results by modified time</param>
	/// <param name="Target">The target to query</param>
	/// <param name="State">State to query</param>
	/// <param name="Outcome">Outcomes to return</param>
	/// <param name="BatchState">One or more batches matches this state</param>
	/// <param name="ExcludeUserJobs">Whether to exclude user jobs from the find</param>
	/// <param name="ExcludeCancelled">Whether to exclude cancelled jobs</param>
	public record class FindJobOptions
	(
		JobId[]? JobIds = null,
		StreamId? StreamId = null,
		string? Name = null,
		TemplateId[]? Templates = null,
		CommitId? MinCommitId = null,
		CommitId? MaxCommitId = null,
		CommitId? PreflightCommitId = null,
		bool? PreflightOnly = null,
		bool? IncludePreflight = null,
		UserId? PreflightStartedByUser = null,
		UserId? StartedByUser = null,
		DateTimeOffset? MinCreateTime = null,
		DateTimeOffset? MaxCreateTime = null,
		DateTimeOffset? ModifiedBefore = null,
		DateTimeOffset? ModifiedAfter = null,
		string? Target = null,
		JobStepState[]? State = null,
		JobStepOutcome[]? Outcome = null,
		JobStepBatchState? BatchState = null,
		bool? ExcludeUserJobs = null,
		bool? ExcludeCancelled = null
	);
}
