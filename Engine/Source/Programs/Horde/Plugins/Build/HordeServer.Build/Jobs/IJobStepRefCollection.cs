// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using Microsoft.Extensions.Logging;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Interface for a collection of JobStepRef documents
	/// </summary>
	public interface IJobStepRefCollection
	{
		/// <summary>
		/// Inserts or replaces a JobStepRef.
		/// </summary>
		/// <param name="id">Unique id for the step being referenced</param>
		/// <param name="jobName">Name of the job</param>
		/// <param name="stepName">Name of the step</param>
		/// <param name="streamId">Unique id for the stream containing the job</param>
		/// <param name="templateId"></param>
		/// <param name="commitId">The commit being built</param>
		/// <param name="logId">The log file id</param>
		/// <param name="poolId">The pool id</param>
		/// <param name="agentId">The agent id</param>
		/// <param name="state">State of this step</param>
		/// <param name="outcome">Outcome of this step, if known</param>
		/// <param name="updateIssues">Whether this step ref is included for issue updates</param>
		/// <param name="lastSuccess">The last change that completed with success</param>
		/// <param name="lastWarning">The last change that completed with a warning (or success)</param>
		/// <param name="waitTime">Time taken for the batch containing this step to start</param>
		/// <param name="initTime">Time taken for the batch containing this step to initializer</param>
		/// <param name="jobStartTimeUtc">Start time of the job</param>
		/// <param name="startTimeUtc">Start time</param>
		/// <param name="finishTimeUtc">Finish time for the step, if known</param>
		/// <param name="stepMetadata">Step meta data</param>
		/// <param name="logger">Logger to use throughout.</param>
		Task<IJobStepRef> InsertOrReplaceAsync(JobStepRefId id, string jobName, string stepName, StreamId streamId, TemplateId templateId, CommitIdWithOrder commitId, LogId? logId, PoolId? poolId, AgentId? agentId, JobStepState? state, JobStepOutcome? outcome, bool updateIssues, CommitIdWithOrder? lastSuccess, CommitIdWithOrder? lastWarning, float waitTime, float initTime, DateTime jobStartTimeUtc, DateTime startTimeUtc, DateTime? finishTimeUtc, List<string>? stepMetadata = null, ILogger? logger = null);

		/// <summary>
		/// Updates a job step ref 
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="batchId"></param>
		/// <param name="stepId"></param>
		/// <param name="issueIds"></param>
		/// <returns></returns>
		Task<IJobStepRef?> UpdateAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, List<int>? issueIds);

		/// <summary>
		/// Gets a specific job step ref
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="batchId"></param>
		/// <param name="stepId"></param>
		/// <returns></returns>
		Task<IJobStepRef?> FindAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId);

		/// <summary>
		/// Gets job step references given an array of ids
		/// </summary>
		/// <param name="ids"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<List<IJobStepRef>> FindAsync(JobStepRefId[] ids, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the history of a given node
		/// </summary>
		/// <param name="streamId">Unique id for a stream</param>
		/// <param name="templateId"></param>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="commitId">The current change</param>
		/// <param name="includeFailed">Whether to include failed nodes</param>
		/// <param name="maxCount">Number of results to return</param>
		/// <param name="includePreStartAndSkippedStates">Whether to include pre-run and skipped states.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of step references</returns>
		Task<List<IJobStepRef>> GetStepsForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder? commitId, bool includeFailed, int maxCount, bool includePreStartAndSkippedStates, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the previous job that ran a given step
		/// </summary>
		/// <param name="streamId">Id of the stream to search</param>
		/// <param name="templateId">The template id</param>
		/// <param name="nodeName">Name of the step to find</param>
		/// <param name="commitId">The current changelist number</param>
		/// <param name="outcome">The outcome to filter by or include all outcomes if null</param>
		/// <param name="updateIssues">If true, constrain to steps which update issues</param>		 
		/// <param name="excludeJobIds">Jobs to exclude from the search</param>
		/// <returns>The previous job, or null.</returns>
		Task<IJobStepRef?> GetPrevStepForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder commitId, JobStepOutcome? outcome = null, bool? updateIssues = null, IEnumerable<JobId>? excludeJobIds = null);

		/// <summary>
		/// Gets the next job that ran a given step
		/// </summary>
		/// <param name="streamId">Id of the stream to search</param>
		/// <param name="templateId">The template id</param>
		/// <param name="nodeName">Name of the step to find</param>
		/// <param name="commitId">The current changelist number</param>
		/// <param name="outcome">The outcome to filter by or include all outcomes if null</param>
		/// <param name="updateIssues">If true, constrain to steps which update issues</param>
		/// <returns>The previous job, or null.</returns>
		Task<IJobStepRef?> GetNextStepForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder commitId, JobStepOutcome? outcome = null, bool? updateIssues = null);

	}

	static class JobStepRefCollectionExtensions
	{
		/// <summary>
		/// Updates a job step ref.
		/// </summary>
		/// <param name="jobStepRefs">The job step ref collection.</param>
		/// <param name="job">The job which the job step belongs to..</param>
		/// <param name="batch">The batch which the job step belongs to.</param>
		/// <param name="step">The step.</param>
		/// <param name="graph">The graph.</param>
		/// <param name="logger">The logger.</param>
		/// <returns>Task object associated with the operation.</returns>
		public static async Task UpdateAsync(this IJobStepRefCollection jobStepRefs, IJob job, IJobStepBatch batch, IJobStep step, IGraph graph, ILogger? logger = null)
		{
			if (job.PreflightCommitId == null)
			{
				float waitTime = (float)(batch.GetWaitTime() ?? TimeSpan.Zero).TotalSeconds;
				float initTime = (float)(batch.GetInitTime() ?? TimeSpan.Zero).TotalSeconds;

				string nodeName = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx].Name;
				JobStepOutcome? outcome = step.IsPending() ? (JobStepOutcome?)null : step.Outcome;

				CommitIdWithOrder? lastSuccess = null;
				CommitIdWithOrder? lastWarning = null;
				if (outcome != JobStepOutcome.Success)
				{
					IJobStepRef? prevStep = await jobStepRefs.GetPrevStepForNodeAsync(job.StreamId, job.TemplateId, nodeName, job.CommitId);
					if (prevStep != null)
					{
						lastSuccess = prevStep.LastSuccess;
						if (outcome != JobStepOutcome.Warnings)
						{
							lastWarning = prevStep.LastWarning;
						}
					}
				}

				if (job.AbortedByUserId != null && outcome == null)
				{
					outcome = JobStepOutcome.Unspecified;
				}

				if (batch.Error != JobStepBatchError.None && outcome == null)
				{
					outcome = JobStepOutcome.Unspecified;
				}

				if (logger != null)
				{
					logger.LogInformation("Updating step reference {StepId} for job {JobId}, batch {BatchId}, with outcome {JobStepOutcome}", step.Id, job.Id, batch.Id, outcome);
				}

				await jobStepRefs.InsertOrReplaceAsync(new JobStepRefId(job.Id, batch.Id, step.Id), job.Name, nodeName, job.StreamId, job.TemplateId, job.CommitId, step.LogId, batch.PoolId, batch.AgentId, step.State, outcome, job.UpdateIssues, lastSuccess, lastWarning, waitTime, initTime, job.CreateTimeUtc, step.StartTimeUtc ?? DateTime.UtcNow, step.FinishTimeUtc, step.Metadata?.ToList(), logger);
			}
		}
	}
}
