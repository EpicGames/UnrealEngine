// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Interface for enriching workspace messages before they are sent to agents.
	/// Plugins can implement this to modify workspace credentials, add custom fields, etc.
	/// </summary>
	public interface IWorkspaceMessageEnricher
	{
		/// <summary>
		/// Enriches a workspace message before it's sent to an agent.
		/// Called after the base workspace message is created but before it's assigned to a task.
		/// </summary>
		/// <param name="workspace">The workspace message to enrich</param>
		/// <param name="workspaceInfo">The workspace configuration info</param>
		/// <param name="agent">The agent that will receive this workspace</param>
		/// <param name="job">The job being executed (provides access to commit info for VCS-specific enrichment)</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Task that completes when enrichment is done</returns>
		Task EnrichAsync(RpcAgentWorkspace workspace, AgentWorkspaceInfo workspaceInfo, IAgent agent, IJob job, CancellationToken cancellationToken);
	}
}
