// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Jobs
{
	/// <summary>
	/// ACL actions valid for manipulating jobs
	/// </summary>
	public static class JobAclAction
	{
		/// <summary>
		/// Ability to start new jobs
		/// </summary>
		public static AclAction CreateJob { get; } = new AclAction("CreateJob");

		/// <summary>
		/// Rename a job, modify its priority, etc...
		/// </summary>
		public static AclAction UpdateJob { get; } = new AclAction("UpdateJob");

		/// <summary>
		/// Delete a job properties
		/// </summary>
		public static AclAction DeleteJob { get; } = new AclAction("DeleteJob");

		/// <summary>
		/// Allows updating a job metadata (name, changelist number, step properties, new groups, job states, etc...). Typically granted to agents. Not user facing.
		/// </summary>
		public static AclAction ExecuteJob { get; } = new AclAction("ExecuteJob");

		/// <summary>
		/// Ability to retry a failed job step
		/// </summary>
		public static AclAction RetryJobStep { get; } = new AclAction("RetryJobStep");

		/// <summary>
		/// Ability to view a job
		/// </summary>
		public static AclAction ViewJob { get; } = new AclAction("ViewJob");
	}
}
