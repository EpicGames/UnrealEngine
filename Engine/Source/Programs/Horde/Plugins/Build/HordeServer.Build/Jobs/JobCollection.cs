// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Runtime.CompilerServices;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Bisect;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Acls;
using HordeServer.Auditing;
using HordeServer.Commits;
using HordeServer.Jobs.Graphs;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.IO;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Wrapper around the jobs collection in a mongo DB
	/// </summary>
	public class JobCollection : IJobCollection
	{
		/// <summary>
		/// Ease of use struct used to pass telemetry context for job telemetry.
		/// </summary>
		internal struct TelemetryContext
		{
			/// <summary>
			/// The job that relates to the telemetry.
			/// </summary>
			internal IJob? Job { get; set; }

			/// <summary>
			/// The build config object to use when deriving telemetry sinks.
			/// </summary>
			internal IOptionsMonitor<BuildConfig> BuildConfig { get; set; }

			/// <summary>
			/// The telemetry writer used to publish telemetry.
			/// </summary>
			internal ITelemetryWriter TelemetryWriter { get; set; }
		}

		class Job : IJob
		{
			readonly JobCollection _collection;
			readonly JobDocument _document;
			readonly IGraph _graph;
			readonly List<JobStepBatch> _batches;

			public JobDocument Document => _document;
			public IGraph Graph => _graph;
			public List<JobStepBatch> Batches => _batches;

			public List<string> Metadata => _document.Metadata;

			JobId IJob.Id => _document.Id;
			StreamId IJob.StreamId => _document.StreamId;
			TemplateId IJob.TemplateId => _document.TemplateId;
			ContentHash? IJob.TemplateHash => _document.TemplateHash;
			ContentHash IJob.GraphHash => _document.GraphHash;
			UserId? IJob.StartedByUserId => _document.StartedByUserId;
			UserId? IJob.AbortedByUserId => _document.AbortedByUserId;
			string? IJob.CancellationReason => _document.CancellationReason;

			BisectTaskId? IJob.StartedByBisectTaskId => _document.StartedByBisectTaskId;
			string IJob.Name => _document.Name;
			CommitIdWithOrder IJob.CommitId => _document.CommitId;
			CommitIdWithOrder? IJob.CodeCommitId => _document.CodeCommitId;
			CommitId? IJob.PreflightCommitId => _document.PreflightCommitId;
			string? IJob.PreflightDescription => _document.PreflightDescription;
			Priority IJob.Priority => _document.Priority;
			bool IJob.AutoSubmit => _document.AutoSubmit;
			int? IJob.AutoSubmitChange => _document.AutoSubmitChange;
			string? IJob.AutoSubmitMessage => _document.AutoSubmitMessage;
			bool IJob.UpdateIssues => _document.UpdateIssues;
			bool IJob.PromoteIssuesByDefault => _document.PromoteIssuesByDefault;
			DateTime IJob.CreateTimeUtc => _document.GetCreateTimeOrDefault();
			JobOptions? IJob.JobOptions => _document.JobOptions;
			IReadOnlyList<IAclClaim> IJob.Claims => _document.Claims;
			IReadOnlyList<IJobStepBatch> IJob.Batches => Batches;
			IReadOnlyDictionary<ParameterId, string> IJob.Parameters => _document.Parameters;
			IReadOnlyList<string> IJob.Arguments => _document.Arguments;
			IReadOnlyList<string>? IJob.Targets => _document.Targets;
			IReadOnlyList<string> IJob.AdditionalArguments => _document.AdditionalArguments;
			IReadOnlyDictionary<string, string> IJob.Environment => _document.Environment;
			IReadOnlyList<int> IJob.Issues => _document.ReferencedByIssues;
			NotificationTriggerId? IJob.NotificationTriggerId => _document.NotificationTriggerId;
			bool IJob.ShowUgsBadges => _document.ShowUgsBadges;
			bool IJob.ShowUgsAlerts => _document.ShowUgsAlerts;
			string? IJob.NotificationChannel => _document.NotificationChannel;
			string? IJob.NotificationChannelFilter => _document.NotificationChannelFilter;
			IReadOnlyDictionary<int, NotificationTriggerId> IJob.LabelIdxToTriggerId => _document._labelNotifications.ToDictionary(x => x._labelIdx, x => x._triggerId);
			IReadOnlyList<IJobReport>? IJob.Reports => _document.Reports;
			IReadOnlyList<IChainedJob> IJob.ChainedJobs => _document.ChainedJobs;

			JobId? IJob.ParentJobId => _document.ParentJobId;
			JobStepId? IJob.ParentJobStepId => _document.ParentJobStepId;

			DateTime IJob.UpdateTimeUtc => _document.UpdateTimeUtc ?? _document.UpdateTime?.UtcDateTime ?? DateTime.UnixEpoch;
			int IJob.UpdateIndex => _document.UpdateIndex;

			Dictionary<JobStepBatchId, JobStepBatch>? _batchIdToBatch;
			Dictionary<JobStepId, JobStep>? _stepIdToStep;

			Dictionary<NodeRef, JobStepId>? _cachedNodeRefToStepId;
			public IReadOnlyDictionary<NodeRef, JobStepId> NodeRefToStepId => _cachedNodeRefToStepId ??= CreateNodeRefToStepId();

			bool TryGetBatch(JobStepBatchId batchId, [NotNullWhen(true)] out JobStepBatch? batch)
			{
				_batchIdToBatch ??= _batches.ToDictionary(x => x.Document.Id, x => x);
				return _batchIdToBatch.TryGetValue(batchId, out batch);
			}

			bool IJob.TryGetBatch(JobStepBatchId batchId, [NotNullWhen(true)] out IJobStepBatch? batch)
			{
				if (TryGetBatch(batchId, out JobStepBatch? typedBatch))
				{
					batch = typedBatch;
					return true;
				}
				else
				{
					batch = null;
					return false;
				}
			}

			bool TryGetStep(JobStepId stepId, [NotNullWhen(true)] out JobStep? step)
			{
				_stepIdToStep ??= _batches.SelectMany(x => x.Steps).ToDictionary(x => x.Document.Id, x => x);
				return _stepIdToStep.TryGetValue(stepId, out step);
			}

			bool IJob.TryGetStep(JobStepId stepId, [NotNullWhen(true)] out IJobStep? step)
			{
				if (TryGetStep(stepId, out JobStep? typedStep))
				{
					step = typedStep;
					return true;
				}
				else
				{
					step = null;
					return false;
				}
			}

			Dictionary<NodeRef, JobStepId> CreateNodeRefToStepId()
			{
				Dictionary<NodeRef, JobStepId> nodeRefToStepRef = new Dictionary<NodeRef, JobStepId>();
				for (int batchIdx = 0; batchIdx < Batches.Count; batchIdx++)
				{
					JobStepBatch batch = Batches[batchIdx];
					for (int stepIdx = 0; stepIdx < batch.Steps.Count; stepIdx++)
					{
						JobStep step = batch.Steps[stepIdx];

						NodeRef nodeRef = new NodeRef(batch.Document.GroupIdx, step.Document.NodeIdx);
						nodeRefToStepRef[nodeRef] = step.Document.Id;
					}
				}
				return nodeRefToStepRef;
			}

			public Job(JobCollection collection, JobDocument document, IGraph graph)
			{
				_collection = collection;
				_document = document;
				_graph = graph;
				_batches = document.Batches.ConvertAll(x => new JobStepBatch(this, x, graph.Groups[x.GroupIdx]));
			}

			public Task<IJob?> RefreshAsync(CancellationToken cancellationToken = default)
				=> _collection.GetAsync(_document.Id, cancellationToken);

			public async Task<IJob?> TryAssignLeaseAsync(int batchIdx, PoolId poolId, AgentId agentId, SessionId sessionId, LeaseId leaseId, LogId logId, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryAssignLeaseAsync(Document, batchIdx, poolId, agentId, sessionId, leaseId, logId, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TryCancelLeaseAsync(int batchIdx, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryCancelLeaseAsync(Document, batchIdx, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public Task<bool> TryDeleteAsync(CancellationToken cancellationToken = default)
				=> _collection.TryDeleteAsync(Document, cancellationToken);

			public async Task<IJob?> TryFailBatchAsync(int batchIdx, JobStepBatchError reason, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryFailBatchAsync(Document, batchIdx, Graph, reason, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TryRemoveFromDispatchQueueAsync(CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryRemoveFromDispatchQueueAsync(Document, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TrySkipAllBatchesAsync(JobStepBatchError reason, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TrySkipAllBatchesAsync(Document, Graph, reason, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TrySkipBatchAsync(JobStepBatchId batchId, JobStepBatchError reason, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TrySkipBatchAsync(Document, batchId, Graph, reason, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TryUpdateBatchAsync(JobStepBatchId batchId, LogId? newLogId, JobStepBatchState? newState, JobStepBatchError? newError, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryUpdateBatchAsync(Document, Graph, batchId, newLogId, newState, newError, cancellationToken);
				return _collection.CreateJobObject(newDocument, _graph);
			}

			public async Task<IJob?> TryUpdateGraphAsync(IGraph newGraph, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryUpdateGraphAsync(Document, Graph, newGraph, cancellationToken);
				return _collection.CreateJobObject(newDocument, newGraph);
			}

			public async Task<IJob?> TryUpdateJobAsync(string? name = null, Priority? priority = null, bool? autoSubmit = null, int? autoSubmitChange = null, string? autoSubmitMessage = null, UserId? abortedByUserId = null, NotificationTriggerId? notificationTriggerId = null, List<JobReport>? reports = null, List<string>? arguments = null, KeyValuePair<int, NotificationTriggerId>? labelIdxToTriggerId = null, KeyValuePair<TemplateId, JobId>? jobTrigger = null, string? cancellationReason = null, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryUpdateJobAsync(Document, Graph, name, priority, autoSubmit, autoSubmitChange, autoSubmitMessage, abortedByUserId, notificationTriggerId, reports, arguments, labelIdxToTriggerId, jobTrigger, cancellationReason, cancellationToken);
				return _collection.CreateJobObject(newDocument, Graph);
			}

			public async Task<IJob?> TryUpdateStepAsync(JobStepBatchId batchId, JobStepId stepId, JobStepState newState = JobStepState.Unspecified, JobStepOutcome newOutcome = JobStepOutcome.Unspecified, JobStepError? newError = null, bool? newAbortRequested = null, UserId? newAbortByUserId = null, LogId? newLogId = null, NotificationTriggerId? newNotificationTriggerId = null, UserId? newRetryByUserId = null, Priority? newPriority = null, List<JobReport>? newReports = null, Dictionary<string, string?>? newProperties = null, string? newCancellationReason = null, JobId? newSpawnedJob = null, CancellationToken cancellationToken = default)
			{
				JobDocument? newDocument = await _collection.TryUpdateStepAsync(_document, _graph, batchId, stepId, newState, newOutcome, newError, newAbortRequested, newAbortByUserId, newLogId, newNotificationTriggerId, newRetryByUserId, newPriority, newReports, newProperties, newCancellationReason, newSpawnedJob, cancellationToken);
				return _collection.CreateJobObject(newDocument, Graph);
			}
		}

		[DebuggerDisplay("{Document.Id} (Group {Document.GroupIdx})")]
		class JobStepBatch : IJobStepBatch
		{
			public Job Job { get; }
			public JobStepBatchDocument Document { get; }
			public INodeGroup Group { get; }
			public List<JobStep> Steps { get; }

			IJob IJobStepBatch.Job => Job;
			JobStepBatchId IJobStepBatch.Id => Document.Id;

			// Group properties
			string IJobStepBatch.AgentType => Group.AgentType;

			// Batch properties
			LogId? IJobStepBatch.LogId => Document.LogId;
			int IJobStepBatch.GroupIdx => Document.GroupIdx;
			JobStepBatchState IJobStepBatch.State => Document.State;
			JobStepBatchError IJobStepBatch.Error => Document.Error;
			IReadOnlyList<IJobStep> IJobStepBatch.Steps => Steps;
			PoolId? IJobStepBatch.PoolId => Document.PoolId;
			AgentId? IJobStepBatch.AgentId => Document.AgentId;
			SessionId? IJobStepBatch.SessionId => Document.SessionId;
			LeaseId? IJobStepBatch.LeaseId => Document.LeaseId;
			int IJobStepBatch.SchedulePriority => Document.SchedulePriority;
			DateTime? IJobStepBatch.ReadyTimeUtc => Document.ReadyTime?.UtcDateTime;
			DateTime? IJobStepBatch.StartTimeUtc => Document.StartTime?.UtcDateTime;
			DateTime? IJobStepBatch.FinishTimeUtc => Document.FinishTime?.UtcDateTime;

			public JobStepBatch(Job job, JobStepBatchDocument document, INodeGroup group)
			{
				Job = job;
				Document = document;
				Group = group;
				Steps = document.Steps.ConvertAll(x => new JobStep(this, x, group.Nodes[x.NodeIdx]));
			}
		}

		[DebuggerDisplay("{Document.Id} (Node {Batch.Document.GroupIdx},{Document.NodeIdx})")]
		class JobStep : IJobStep
		{
			public Job Job => Batch.Job;
			public JobStepBatch Batch { get; }
			public JobStepDocument Document { get; }
			public INode Node { get; }

			IJob IJobStep.Job => Job;
			IJobStepBatch IJobStep.Batch => Batch;
			JobStepId IJobStep.Id => Document.Id;
			INode IJobStep.Node => Node;
			int IJobStep.NodeIdx => Document.NodeIdx;

			// Node properties
			string IJobStep.Name => Node.Name;

			List<JobStepOutputRef>? _inputs;
			IReadOnlyList<JobStepOutputRef> IJobStep.Inputs => _inputs ??= CreateInputDependencyList();

			List<JobStepOutputRef>? _optionalInputs;
			IReadOnlyList<JobStepOutputRef> IJobStep.OptionalInputs => _optionalInputs ??= CreateOptionalInputDependencyList();

			IReadOnlyList<string> IJobStep.OutputNames => Node.OutputNames;

			List<JobStepId>? _inputDependencies;
			IReadOnlyList<JobStepId> IJobStep.InputDependencies => _inputDependencies ??= CreateInputDependenciesList();

			List<JobStepId>? _optionalInputDependencies;
			IReadOnlyList<JobStepId> IJobStep.OptionalInputDependencies => _optionalInputDependencies ??= CreateOptionalInputDependenciesList();

			List<JobStepId>? _orderDependencies;
			IReadOnlyList<JobStepId> IJobStep.OrderDependencies => _orderDependencies ??= CreateOrderDependenciesList();

			bool IJobStep.AllowRetry => Node.AllowRetry;
			bool IJobStep.RunEarly => Node.RunEarly;
			bool IJobStep.Warnings => Node.Warnings;
			IReadOnlyDictionary<string, string>? IJobStep.Credentials => Node.Credentials;
			IReadOnlyNodeAnnotations IJobStep.Annotations => Node.Annotations;
			IReadOnlyList<string> IJobStep.Metadata => Document.Metadata;

			// Step properties
			JobStepState IJobStep.State => Document.State;
			JobStepOutcome IJobStep.Outcome => Document.Outcome;
			JobStepError IJobStep.Error => Document.Error;
			LogId? IJobStep.LogId => Document.LogId;
			NotificationTriggerId? IJobStep.NotificationTriggerId => Document.NotificationTriggerId;
			DateTime? IJobStep.ReadyTimeUtc => Document.ReadyTime?.UtcDateTime;
			DateTime? IJobStep.StartTimeUtc => Document.StartTime?.UtcDateTime;
			DateTime? IJobStep.FinishTimeUtc => Document.FinishTime?.UtcDateTime;
			Priority? IJobStep.Priority => Document.Priority;
			UserId? IJobStep.RetriedByUserId => Document.RetriedByUserId;
			bool IJobStep.AbortRequested => Document.AbortRequested;
			UserId? IJobStep.AbortedByUserId => Document.AbortedByUserId;
			string? IJobStep.CancellationReason => Document.CancellationReason;
			IReadOnlyList<IJobReport>? IJobStep.Reports => Document.Reports;
			IReadOnlyDictionary<string, string>? IJobStep.Properties => Document.Properties;

			IReadOnlyList<JobId>? IJobStep.SpawnedJobs => Document.SpawnedJobs;

			public JobStep(JobStepBatch batch, JobStepDocument document, INode node)
			{
				Batch = batch;
				Document = document;
				Node = node;
			}

			List<JobStepOutputRef> CreateInputDependencyList()
			{
				List<JobStepOutputRef> dependencies = new List<JobStepOutputRef>();
				foreach (NodeOutputRef input in Node.Inputs)
				{
					if (Job.NodeRefToStepId.TryGetValue(input.NodeRef, out JobStepId jobStepId))
					{
						dependencies.Add(new JobStepOutputRef(jobStepId, input.OutputIdx));
					}
					else
					{
						MissingNodeRef(input.NodeRef);
					}
				}
				return dependencies;
			}

			List<JobStepId> CreateInputDependenciesList()
			{
				List<JobStepId> dependencies = new List<JobStepId>();
				foreach (NodeRef nodeRef in Node.InputDependencies)
				{
					if (Job.NodeRefToStepId.TryGetValue(nodeRef, out JobStepId jobStepId))
					{
						dependencies.Add(jobStepId);
					}
					else
					{
						MissingNodeRef(nodeRef);
					}
				}
				return dependencies;
			}

			List<JobStepOutputRef> CreateOptionalInputDependencyList()
			{
				List<JobStepOutputRef> dependencies = [];
				foreach (NodeOutputRef optionalInput in Node.OptionalInputs)
				{
					if (Job.NodeRefToStepId.TryGetValue(optionalInput.NodeRef, out JobStepId jobStepId))
					{
						dependencies.Add(new JobStepOutputRef(jobStepId, optionalInput.OutputIdx));
					}
					else
					{
						MissingNodeRef(optionalInput.NodeRef);
					}
				}
				return dependencies;
			}

			List<JobStepId> CreateOptionalInputDependenciesList()
			{
				List<JobStepId> dependencies = [];
				foreach (NodeRef nodeRef in Node.OptionalInputDependencies)
				{
					if (Job.NodeRefToStepId.TryGetValue(nodeRef, out JobStepId jobStepId))
					{
						dependencies.Add(jobStepId);
					}
					else
					{
						MissingNodeRef(nodeRef);
					}
				}
				return dependencies;
			}

			List<JobStepId> CreateOrderDependenciesList()
			{
				List<JobStepId> dependencies = new List<JobStepId>();
				foreach (NodeRef nodeRef in Node.OrderDependencies)
				{
					if (Job.NodeRefToStepId.TryGetValue(nodeRef, out JobStepId jobStepId))
					{
						dependencies.Add(jobStepId);
					}
					else
					{
						MissingNodeRef(nodeRef);
					}
				}
				return dependencies;
			}

			static void MissingNodeRef(NodeRef nodeRef)
			{
				_ = nodeRef;
			}
		}

		class JobDocument
		{
			[BsonRequired, BsonId]
			public JobId Id { get; set; }

			public StreamId StreamId { get; set; }
			public TemplateId TemplateId { get; set; }
			public ContentHash? TemplateHash { get; set; }
			public ContentHash GraphHash { get; set; }

			[BsonIgnoreIfNull]
			public UserId? StartedByUserId { get; set; }

			[BsonIgnoreIfNull]
			public BisectTaskId? StartedByBisectTaskId { get; set; }

			[BsonIgnoreIfNull, BsonElement("StartedByUser")]
			public string? StartedByUserDeprecated { get; set; }

			[BsonIgnoreIfNull]
			public UserId? AbortedByUserId { get; set; }

			[BsonIgnoreIfNull, BsonElement("AbortedByUser")]
			public string? AbortedByUserDeprecated { get; set; }

			[BsonIgnoreIfNull]
			public string? CancellationReason { get; set; }

			[BsonRequired]
			public string Name { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("Commit")]
			public string? CommitName { get; set; }

			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder? CodeCommitId
			{
				get => (CodeCommitName != null) ? new CommitIdWithOrder(CodeCommitName, CodeCommitOrder) : (CodeCommitOrder != 0) ? CommitIdWithOrder.FromPerforceChange(CodeCommitOrder) : null;
				set => (CodeCommitName, CodeCommitOrder) = (value?.Name, value?.Order ?? 0);
			}

			[BsonElement("CodeCommit")]
			public string? CodeCommitName { get; set; }

			[BsonElement("CodeChange")]
			public int CodeCommitOrder { get; set; }

			[BsonIgnore]
			public CommitId? PreflightCommitId
			{
				get => (PreflightCommitName != null) ? new CommitId(PreflightCommitName) : (PreflightChange != 0) ? EpicGames.Horde.Commits.CommitId.FromPerforceChange(PreflightChange) : null;
				set => (PreflightCommitName, PreflightChange) = (value?.Name, (value == null) ? 0 : value.TryGetPerforceChange() ?? -1);
			}

			[BsonElement("PreflightCommit")]
			public string? PreflightCommitName { get; set; }

			// -1 for non-P4 preflights
			public int PreflightChange { get; set; }

			public string? PreflightDescription { get; set; }
			public Priority Priority { get; set; }

			[BsonIgnoreIfDefault]
			public bool AutoSubmit { get; set; }

			[BsonIgnoreIfNull]
			public int? AutoSubmitChange { get; set; }

			[BsonIgnoreIfNull]
			public string? AutoSubmitMessage { get; set; }

			public bool UpdateIssues { get; set; }

			public bool PromoteIssuesByDefault { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? CreateTime { get; set; }

			[BsonIgnoreIfNull]
			public JobOptions? JobOptions { get; set; }

			public List<AclClaimConfig> Claims { get; set; } = new List<AclClaimConfig>();

			[BsonIgnoreIfNull]
			public DateTime? CreateTimeUtc { get; set; }

			public int SchedulePriority { get; set; }
			public List<JobStepBatchDocument> Batches { get; set; } = new List<JobStepBatchDocument>();
			public List<JobReport>? Reports { get; set; }

			[BsonDictionaryOptions(Representation = DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<ParameterId, string> Parameters { get; set; } = new Dictionary<ParameterId, string>();

			public List<string> Arguments { get; set; } = new List<string>();
			public List<string> AdditionalArguments { get; set; } = new List<string>();

			[BsonIgnoreIfNull]
			public List<string>? Targets { get; set; }

			[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<string, string> Environment { get; set; } = new Dictionary<string, string>();

			public List<int> ReferencedByIssues { get; set; } = new List<int>();
			public NotificationTriggerId? NotificationTriggerId { get; set; }
			public bool ShowUgsBadges { get; set; }
			public bool ShowUgsAlerts { get; set; }
			public string? NotificationChannel { get; set; }
			public string? NotificationChannelFilter { get; set; }
			public List<LabelNotificationDocument> _labelNotifications = new List<LabelNotificationDocument>();
			public List<ChainedJobDocument> ChainedJobs { get; set; } = new List<ChainedJobDocument>();

			[BsonIgnoreIfNull]
			public JobId? ParentJobId { get; set; }
			[BsonIgnoreIfNull]
			public JobStepId? ParentJobStepId { get; set; }

			[BsonIgnoreIfNull]
			public List<NodeRef>? RetriedNodes { get; set; }
			public SubResourceId NextSubResourceId { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? UpdateTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? UpdateTimeUtc { get; set; }

			[BsonElement("Metadata")]
			public List<string> Metadata { get; set; } = new List<string>();

			[BsonRequired]
			public int UpdateIndex { get; set; }

			[BsonConstructor]
			private JobDocument()
			{
				Name = null!;
				GraphHash = null!;
			}

			public JobDocument(JobId id, StreamId streamId, TemplateId templateId, ContentHash templateHash, ContentHash graphHash, string name, CommitIdWithOrder commitId, CommitIdWithOrder? codeCommitId, CreateJobOptions options, DateTime createTimeUtc, List<string> metadata)
			{
				Id = id;
				StreamId = streamId;
				TemplateId = templateId;
				TemplateHash = templateHash;
				GraphHash = graphHash;
				Name = name;
				CommitId = commitId;
				CodeCommitId = codeCommitId;
				PreflightCommitId = options.PreflightCommitId;
				PreflightDescription = options.PreflightDescription;
				StartedByUserId = options.StartedByUserId;
				StartedByBisectTaskId = options.StartedByBisectTaskId;
				Priority = options.Priority ?? Priority.Normal;
				AutoSubmit = options.AutoSubmit ?? false;
				UpdateIssues = options.UpdateIssues ?? (options.StartedByUserId == null && options.PreflightCommitId == null);
				PromoteIssuesByDefault = options.PromoteIssuesByDefault ?? false;
				Claims = options.Claims;
				JobOptions = options.JobOptions;
				CreateTimeUtc = createTimeUtc;
				ChainedJobs.AddRange(options.JobTriggers.Select(x => new ChainedJobDocument(x)));
				ShowUgsBadges = options.ShowUgsBadges;
				ShowUgsAlerts = options.ShowUgsAlerts;
				NotificationChannel = options.NotificationChannel;
				NotificationChannelFilter = options.NotificationChannelFilter;
				Parameters = new Dictionary<ParameterId, string>(options.Parameters);
				Arguments.AddRange(options.Arguments);
				AdditionalArguments.AddRange(options.AdditionalArguments);
				Targets = (options.Targets != null && options.Targets.Count > 0) ? new List<string>(options.Targets) : null;

				foreach (KeyValuePair<string, string> pair in options.Environment)
				{
					Environment[pair.Key] = pair.Value;
				}

				ParentJobId = options.ParentJobId;
				ParentJobStepId = options.ParentJobStepId;

				NextSubResourceId = SubResourceId.GenerateNewId();
				UpdateTimeUtc = createTimeUtc;
				Metadata = metadata;
			}

			public DateTime GetCreateTimeOrDefault()
				=> CreateTimeUtc ?? CreateTime?.UtcDateTime ?? DateTime.UnixEpoch;
		}

		class JobStepBatchDocument
		{
			[BsonRequired]
			public JobStepBatchId Id { get; set; }

			public LogId? LogId { get; set; }

			[BsonRequired]
			public int GroupIdx { get; set; }

			[BsonRequired]
			public JobStepBatchState State { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(JobStepBatchError.None)]
			public JobStepBatchError Error { get; set; }

			public List<JobStepDocument> Steps { get; set; } = new List<JobStepDocument>();

			[BsonIgnoreIfNull]
			public PoolId? PoolId { get; set; }

			[BsonIgnoreIfNull]
			public AgentId? AgentId { get; set; }

			[BsonIgnoreIfNull]
			public SessionId? SessionId { get; set; }

			[BsonIgnoreIfNull]
			public LeaseId? LeaseId { get; set; }

			public int SchedulePriority { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? ReadyTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? StartTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? FinishTime { get; set; }

			[BsonConstructor]
			private JobStepBatchDocument()
			{
			}

			public JobStepBatchDocument(JobStepBatchId id, int groupIdx)
			{
				Id = id;
				GroupIdx = groupIdx;
			}

			public override string ToString()
			{
				StringBuilder description = new StringBuilder($"{Id}: {State}");
				if (Error != JobStepBatchError.None)
				{
					description.Append($" - {Error}");
				}
				description.Append($" ({Steps.Count} steps");
				return description.ToString();
			}
		}

		[BsonIgnoreExtraElements]
		class JobStepDocument
		{
			[BsonRequired]
			public JobStepId Id { get; set; }

			[BsonRequired]
			public int NodeIdx { get; set; }

			[BsonRequired]
			public JobStepState State { get; set; } = JobStepState.Waiting;

			public JobStepOutcome Outcome { get; set; } = JobStepOutcome.Success;

			public JobStepError Error { get; set; } = JobStepError.None;

			public List<string> Metadata { get; set; } = new List<string>();

			[BsonIgnoreIfNull]
			public LogId? LogId { get; set; }

			[BsonIgnoreIfNull]
			public NotificationTriggerId? NotificationTriggerId { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? ReadyTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? StartTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? FinishTime { get; set; }

			[BsonIgnoreIfNull]
			public Priority? Priority { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool Retry { get; set; }

			public UserId? RetriedByUserId { get; set; }

			[BsonElement("RetryByUser")]
			public string? RetriedByUserDeprecated { get; set; }

			public bool AbortRequested { get; set; } = false;

			public UserId? AbortedByUserId { get; set; }

			[BsonElement("AbortByUser")]
			public string? AbortedByUserDeprecated { get; set; }

			[BsonIgnoreIfNull]
			public string? CancellationReason { get; set; }

			[BsonIgnoreIfNull]
			public List<JobReport>? Reports { get; set; }

			[BsonIgnoreIfNull]
			public List<JobId>? SpawnedJobs { get; set; }

			[BsonIgnoreIfNull]
			public Dictionary<string, string>? Properties { get; set; }

			[BsonConstructor]
			private JobStepDocument()
			{
			}

			public JobStepDocument(JobStepId id, int nodeIdx)
			{
				Id = id;
				NodeIdx = nodeIdx;
			}

			public override string ToString()
			{
				StringBuilder description = new StringBuilder($"{Id}: {State}");
				if (Outcome != JobStepOutcome.Unspecified)
				{
					description.Append($" ({Outcome}");
					if (Error != JobStepError.None)
					{
						description.Append($" - {Error}");
					}
					description.Append(')');
				}
				return description.ToString();
			}
		}

		class ChainedJobDocument : IChainedJob
		{
			public string Target { get; set; }
			public TemplateId TemplateRefId { get; set; }
			public JobId? JobId { get; set; }
			public bool UseDefaultChangeForTemplate { get; set; }

			[BsonConstructor]
			private ChainedJobDocument()
			{
				Target = String.Empty;
			}

			public ChainedJobDocument(ChainedJobTemplateConfig trigger)
			{
				Target = trigger.Trigger;
				TemplateRefId = trigger.TemplateId;
				UseDefaultChangeForTemplate = trigger.UseDefaultChangeForTemplate;
			}
		}

		class LabelNotificationDocument
		{
			public int _labelIdx;
			public NotificationTriggerId _triggerId;
		}

		/// <summary>
		/// Maximum number of times a step can be retried (after the original run)
		/// </summary>
		const int MaxRetries = 2;

		readonly IMongoCollection<JobDocument> _jobs;

		[SuppressMessage("CodeQuality", "IDE0052:Remove unread private members")]
		readonly MongoIndex<JobDocument> _streamThenTemplateThenCreationTimeIndex;
		readonly MongoIndex<JobDocument> _startedByBisectTaskIdIndex;
		readonly ITelemetryWriter _telemetryWriter;
		readonly IClock _clock;
		readonly IGraphCollection _graphCollection;
		readonly ILogCollection _logCollection;
		readonly ICommitService _commitService;
		readonly Tracer _tracer;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger<JobCollection> _logger;
		readonly IAuditLog<JobId> _auditLog;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobCollection(IMongoService mongoService, IClock clock, IGraphCollection graphCollection, ILogCollection logCollection, ICommitService commitService, ITelemetryWriter telemetryWriter, IOptionsMonitor<BuildConfig> buildConfig, Tracer tracer, ILogger<JobCollection> logger, IAuditLog<JobId> auditLog)
		{
			_clock = clock;
			_graphCollection = graphCollection;
			_logCollection = logCollection;
			_commitService = commitService;
			_telemetryWriter = telemetryWriter;
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;
			_auditLog = auditLog;

			List<MongoIndex<JobDocument>> indexes = new List<MongoIndex<JobDocument>>();
			indexes.Add(keys => keys.Ascending(x => x.StreamId).Descending(x => x.CreateTimeUtc));
			indexes.Add(_streamThenTemplateThenCreationTimeIndex = MongoIndex.Create<JobDocument>(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.TemplateId).Descending(x => x.CreateTimeUtc)));
			indexes.Add(MongoIndex.Create<JobDocument>(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.TemplateId).Descending(x => x.CommitOrder)));
			indexes.Add(keys => keys.Ascending(x => x.CommitOrder));
			indexes.Add(keys => keys.Ascending(x => x.PreflightCommitName));
			indexes.Add(keys => keys.Ascending(x => x.PreflightChange));
			indexes.Add(MongoIndex.Create<JobDocument>(keys => keys.Descending(x => x.CreateTimeUtc)));
			indexes.Add(keys => keys.Ascending(x => x.StartedByUserId));
			indexes.Add(keys => keys.Descending(x => x.SchedulePriority));
			indexes.Add(_startedByBisectTaskIdIndex = MongoIndex.Create<JobDocument>(keys => keys.Descending(x => x.StartedByBisectTaskId), sparse: true));
			_jobs = mongoService.GetCollection<JobDocument>("Jobs", indexes);
		}

		/// <summary>
		/// Get Job Logger
		/// </summary>
		/// <param name="jobId"></param>
		/// <returns></returns>
		IAuditLogChannel<JobId> GetJobLogger(JobId jobId)
		{
			return _auditLog[jobId];
		}

		/// <summary>
		/// Log to a job audit
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="message"></param>
		/// <param name="args"></param>
		public void JobAudit(JobId jobId, string? message, params object?[] args)
		{
			try
			{
#pragma warning disable CA2254 // Template should be a static expression
				GetJobLogger(jobId).LogInformation(message, args);
#pragma warning restore CA2254 // Template should be a static expression
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error logging to job {JobId} audit", jobId);
			}
		}

		/// <summary>
		/// Log to job audit static flavor
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="logger"></param>
		/// <param name="jobLogger"></param>
		/// <param name="message"></param>
		/// <param name="args"></param>
		static void JobAudit(JobId jobId, ILogger logger, IAuditLogChannel<JobId> jobLogger, string? message, params object?[] args)
		{
			try
			{
#pragma warning disable CA2254 // Template should be a static expression
				jobLogger.LogInformation(message, args);
#pragma warning restore CA2254 // Template should be a static expression
			}
			catch (Exception ex)
			{
				logger.LogError(ex, "Error logging to job {JobId} audit", jobId);
			}
		}

		static Task PostLoadAsync(JobDocument job)
		{
			if (job.GraphHash == ContentHash.Empty)
			{
				job.Batches.Clear();
			}
			return Task.CompletedTask;
		}

		static JobDocument Clone(JobDocument job)
		{
			using (MemoryStream stream = new MemoryStream())
			{
				using (BsonBinaryWriter writer = new BsonBinaryWriter(stream))
				{
					BsonSerializer.Serialize(writer, job);
				}
				return BsonSerializer.Deserialize<JobDocument>(stream.ToArray());
			}
		}

		/// <inheritdoc/>
		public async Task<IJob> AddAsync(JobId jobId, StreamId streamId, TemplateId templateRefId, ContentHash templateHash, IGraph graph, string name, CommitId commitId, CommitId? codeCommitId, CreateJobOptions options, CancellationToken cancellationToken)
		{
			CommitIdWithOrder commitIdWithOrder = await _commitService.GetOrderedAsync(streamId, commitId, cancellationToken);

			CommitIdWithOrder? codeCommitIdWithOrder = null;
			if (codeCommitId != null)
			{
				codeCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId, codeCommitId, cancellationToken);
			}

			JobDocument newJob = new JobDocument(jobId, streamId, templateRefId, templateHash, graph.Id, name, commitIdWithOrder, codeCommitIdWithOrder, options, DateTime.UtcNow, new List<string>());

			JobAudit(jobId, "Job {JobId} added, stream {StreamId}, templateRefId {TemplateId}, name {Name}, commitId {CommitId}", jobId, streamId.Id, templateRefId, name, commitId);

			CreateBatchesInternal(newJob, graph, _logger, GetJobLogger(jobId));

			await _jobs.InsertOneAsync(newJob, null, cancellationToken);

			if (_buildConfig.CurrentValue.TryGetStream(streamId, out StreamConfig? streamConfig) && !streamConfig.TelemetryStoreId.IsEmpty)
			{
				_telemetryWriter.WriteEvent(streamConfig.TelemetryStoreId, new JobCreatedTelemetry(CreateJobObject(newJob, graph)));
			}

			return CreateJobObject(newJob, graph);
		}

		/// <inheritdoc/>
		public async Task<IJob?> GetAsync(JobId jobId, CancellationToken cancellationToken)
		{
			JobDocument? job = await _jobs.Find<JobDocument>(x => x.Id == jobId).FirstOrDefaultAsync(cancellationToken);
			if (job == null)
			{
				return null;
			}

			await PostLoadAsync(job);
			return await CreateJobObjectAsync(job, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<bool> TryDeleteAsync(JobDocument job, CancellationToken cancellationToken)
		{
			DeleteResult result = await _jobs.DeleteOneAsync(x => x.Id == job.Id && x.UpdateIndex == job.UpdateIndex, null, cancellationToken);
			foreach (JobStepDocument step in job.Batches.SelectMany(x => x.Steps))
			{
				if (step.LogId.HasValue)
				{
					ILog? log = await _logCollection.GetAsync(step.LogId.Value, cancellationToken);
					if (log != null)
					{
						await log.DeleteAsync(cancellationToken);
					}
				}
			}

			if (result.DeletedCount > 1)
			{
				JobAudit(job.Id, "Job {JobId} deleted", job.Id);
			}

			return result.DeletedCount > 0;
		}

		/// <inheritdoc/>
		public async Task RemoveStreamAsync(StreamId streamId, CancellationToken cancellationToken)
		{
			await _jobs.DeleteManyAsync(x => x.StreamId == streamId, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJob>> FindAsync(FindJobOptions options, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			// JobId[]? jobIds = null, StreamId? streamId = null, string? name = null, TemplateId[]? templates = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CommitId? preflightCommitId = null, bool? preflightOnly = null, bool? includePreflights = null, UserId? preflightStartedByUser = null, UserId? startedByUser = null, DateTimeOffset? minCreateTime = null, DateTimeOffset? maxCreateTime = null, string? target = null, JobStepBatchState? batchState = null, JobStepState[]? state = null, JobStepOutcome[]? outcome = null, DateTimeOffset? modifiedBefore = null, DateTimeOffset? modifiedAfter = null, int? index = null, int? count = null, bool consistentRead = true, bool? excludeUserJobs = null, bool? excludeCancelled = null, 
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobCollection)}.{nameof(FindAsync)}");
			span.SetAttribute("JobIds", (options.JobIds == null) ? null : String.Join(',', options.JobIds));
			span.SetAttribute("StreamId", options.StreamId);
			span.SetAttribute("Name", options.Name);
			span.SetAttribute("Templates", options.Templates);
			span.SetAttribute("MinCommitId", options.MinCommitId?.ToString());
			span.SetAttribute("MaxCommitId", options.MaxCommitId?.ToString());
			span.SetAttribute("PreflightCommitId", options.PreflightCommitId?.ToString());
			span.SetAttribute("PreflightStartedByUser", options.PreflightStartedByUser?.ToString());
			span.SetAttribute("StartedByUser", options.StartedByUser?.ToString());
			span.SetAttribute("MinCreateTime", options.MinCreateTime);
			span.SetAttribute("MaxCreateTime", options.MaxCreateTime);
			span.SetAttribute("Target", options.Target);
			span.SetAttribute("State", options.State?.ToString());
			span.SetAttribute("Outcome", options.Outcome?.ToString());
			span.SetAttribute("ModifiedBefore", options.ModifiedBefore);
			span.SetAttribute("ModifiedAfter", options.ModifiedAfter);
			span.SetAttribute("Index", index);
			span.SetAttribute("Count", count);

			if (options.Target == null && (options.State == null || options.State.Length == 0) && (options.Outcome == null || options.Outcome.Length == 0))
			{
				return await FindInternalAsync(options, index, count, cancellationToken: cancellationToken);
			}
			else
			{
				List<IJob> results = new List<IJob>();
				_logger.LogInformation("Performing scan for job with ");

				const int Count = 5;
				bool excludeMaxCommitId = false;
				int scanIndex = 0;
				int maxCount = (count ?? 1);
				while (results.Count < maxCount)
				{
					IReadOnlyList<IJob> scanJobs = await FindInternalAsync(options, scanIndex, Count, cancellationToken: cancellationToken);
					if (scanJobs.Count == 0)
					{
						break;
					}

					foreach (IJob job in scanJobs.OrderByDescending(x => x.CommitId))
					{
						if (excludeMaxCommitId && job.CommitId == options.MaxCommitId)
						{
							continue;
						}

						if (options.ExcludeCancelled != null && options.ExcludeCancelled.Value && WasCancelled(job))
						{
							continue;
						}

						(JobStepState, JobStepOutcome)? result;
						if (options.Target == null)
						{
							result = job.GetTargetState();
						}
						else
						{
							result = job.GetTargetState(await _graphCollection.GetAsync(job.GraphHash, cancellationToken), options.Target);
						}

						if (result != null)
						{
							(JobStepState jobState, JobStepOutcome jobOutcome) = result.Value;
							if ((options.State == null || options.State.Length == 0 || options.State.Contains(jobState)) && (options.Outcome == null || options.Outcome.Length == 0 || options.Outcome.Contains(jobOutcome)))
							{
								results.Add(job);
								if (results.Count == maxCount)
								{
									break;
								}
							}
						}
					}

					scanIndex += Count;
					excludeMaxCommitId = true;
				}

				return results;
			}
		}

		/// <summary>
		/// Test whether a job was cancelled
		/// </summary>
		static bool WasCancelled(IJob job)
		{
			return job.AbortedByUserId != null || job.Batches.Any(x => x.Steps.Any(y => y.AbortedByUserId != null));
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJob>> FindInternalAsync(FindJobOptions options, int? index = null, int? count = null, bool consistentRead = false, string? indexHint = null, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<JobDocument> filterBuilder = Builders<JobDocument>.Filter;

			FilterDefinition<JobDocument> filter = filterBuilder.Empty;
			if (options.JobIds != null && options.JobIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Id, options.JobIds);
			}
			if (options.StreamId != null)
			{
				filter &= filterBuilder.Eq(x => x.StreamId, options.StreamId.Value);
				if (options.MinCommitId != null)
				{
					CommitIdWithOrder minCommitIdWithOrder = await _commitService.GetOrderedAsync(options.StreamId.Value, options.MinCommitId, cancellationToken);
					filter &= filterBuilder.Gte(x => x.CommitOrder, minCommitIdWithOrder.Order);
				}
				if (options.MaxCommitId != null)
				{
					CommitIdWithOrder maxCommitIdWithOrder = await _commitService.GetOrderedAsync(options.StreamId.Value, options.MaxCommitId, cancellationToken);
					filter &= filterBuilder.Lte(x => x.CommitOrder, maxCommitIdWithOrder.Order);
				}
			}
			if (options.Name != null)
			{
				if (options.Name.StartsWith("$", StringComparison.InvariantCulture))
				{
					BsonRegularExpression regex = new BsonRegularExpression(options.Name.Substring(1), "i");
					filter &= filterBuilder.Regex(x => x.Name, regex);
				}
				else
				{
					filter &= filterBuilder.Eq(x => x.Name, options.Name);
				}
			}
			if (options.Templates != null)
			{
				if (options.Templates.Length == 1)
				{
					filter &= filterBuilder.Eq(x => x.TemplateId, options.Templates[0]);
				}
				else
				{
					filter &= filterBuilder.In(x => x.TemplateId, options.Templates);
				}
			}
			if (options.PreflightCommitId != null)
			{
				int? preflightChange = options.PreflightCommitId.TryGetPerforceChange();
				if (preflightChange == null)
				{
					filter &= filterBuilder.Eq(x => x.PreflightCommitName, options.PreflightCommitId.Name);
				}
				else
				{
					filter &= filterBuilder.Eq(x => x.PreflightChange, preflightChange);
				}
			}
			if (options.IncludePreflight != null && !options.IncludePreflight.Value)
			{
				filter &= filterBuilder.Eq(x => x.PreflightChange, 0);
			}
			if (options.PreflightOnly != null && options.PreflightOnly.Value)
			{
				filter &= filterBuilder.Ne(x => x.PreflightChange, 0);
			}
			if (options.ExcludeUserJobs != null && options.ExcludeUserJobs.Value)
			{
				filter &= filterBuilder.Eq(x => x.StartedByUserId, null);
			}
			else
			{
				if (options.PreflightStartedByUser != null)
				{
					filter &= filterBuilder.Or(filterBuilder.Eq(x => x.PreflightChange, 0), filterBuilder.Eq(x => x.StartedByUserId, options.PreflightStartedByUser));
				}
				if (options.StartedByUser != null)
				{
					filter &= filterBuilder.Eq(x => x.StartedByUserId, options.StartedByUser);
				}
			}
			if (options.MinCreateTime != null)
			{
				filter &= filterBuilder.Gte(x => x.CreateTimeUtc!, options.MinCreateTime.Value.UtcDateTime);
			}
			if (options.MaxCreateTime != null)
			{
				filter &= filterBuilder.Lte(x => x.CreateTimeUtc!, options.MaxCreateTime.Value.UtcDateTime);
			}
			if (options.ModifiedBefore != null)
			{
				filter &= filterBuilder.Lte(x => x.UpdateTimeUtc!, options.ModifiedBefore.Value.UtcDateTime);
			}
			if (options.ModifiedAfter != null)
			{
				filter &= filterBuilder.Gte(x => x.UpdateTimeUtc!, options.ModifiedAfter.Value.UtcDateTime);
			}
			if (options.BatchState != null)
			{
				filter &= filterBuilder.ElemMatch(x => x.Batches, batch => batch.State == options.BatchState);
			}

			List<JobDocument> documents;
			using (TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobCollection)}.{nameof(FindAsync)}"))
			{
				IMongoCollection<JobDocument> collection = consistentRead ? _jobs : _jobs.WithReadPreference(ReadPreference.SecondaryPreferred);
				documents = await collection.FindWithHintAsync(filter, indexHint, x => x.SortByDescending(x => x.CreateTimeUtc!).Range(index, count).ToListAsync());
			}

			List<IJob> jobs = new List<IJob>();
			foreach (JobDocument document in documents)
			{
				await PostLoadAsync(document);
				jobs.Add(await CreateJobObjectAsync(document, cancellationToken));
			}
			return jobs;
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<IJob> FindBisectTaskJobsAsync(BisectTaskId bisectTaskId, bool? running, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobCollection)}.{nameof(FindBisectTaskJobsAsync)}");
			span.SetAttribute("TaskId", bisectTaskId.Id.ToString());

			FilterDefinitionBuilder<JobDocument> filterBuilder = Builders<JobDocument>.Filter;
			FilterDefinition<JobDocument> filter = filterBuilder.Exists(x => x.StartedByBisectTaskId);
			filter &= filterBuilder.Eq(x => x.StartedByBisectTaskId, bisectTaskId);
			List<JobDocument> results = await _jobs.FindWithHintAsync(filter, _startedByBisectTaskIdIndex.Name, x => x.SortByDescending(x => x.CreateTimeUtc!).ToListAsync(cancellationToken));
			foreach (JobDocument jobDoc in results)
			{
				Job job = await CreateJobObjectAsync(jobDoc, cancellationToken);

				if (running.HasValue && running.Value)
				{
					JobState state = job.GetState();
					if (state == JobState.Complete)
					{
						continue;
					}
				}

				await PostLoadAsync(jobDoc);
				yield return job;
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJob>> FindLatestByStreamWithTemplatesAsync(StreamId streamId, TemplateId[] templates, UserId? preflightStartedByUser, DateTimeOffset? maxCreateTime, DateTimeOffset? modifiedAfter, int? index, int? count, bool consistentRead, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobCollection)}.{nameof(FindLatestByStreamWithTemplatesAsync)}");
			span.SetAttribute("StreamId", streamId);
			span.SetAttribute("Templates", templates);
			span.SetAttribute("PreflightStartedByUser", preflightStartedByUser?.ToString());
			span.SetAttribute("MaxCreateTime", maxCreateTime);
			span.SetAttribute("ModifiedAfter", modifiedAfter);
			span.SetAttribute("Index", index);
			span.SetAttribute("Count", count);

			// This find call uses an index hint. Modifying the parameter passed to FindAsync can affect execution time a lot as the query planner is forced to use the specified index.
			return await FindInternalAsync(
				new FindJobOptions(
					StreamId: streamId,
					Templates: templates,
					PreflightStartedByUser: preflightStartedByUser,
					ModifiedAfter: modifiedAfter,
					MaxCreateTime: maxCreateTime),
				index: index,
				count: count,
				consistentRead: consistentRead,
				cancellationToken: cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryUpdateJobAsync(JobDocument jobDocument, IGraph graph, string? name, Priority? priority, bool? autoSubmit, int? autoSubmitChange, string? autoSubmitMessage, UserId? abortedByUserId, NotificationTriggerId? notificationTriggerId, List<JobReport>? reports, List<string>? arguments, KeyValuePair<int, NotificationTriggerId>? labelIdxToTriggerId, KeyValuePair<TemplateId, JobId>? jobTrigger, string? cancellationReason, CancellationToken cancellationToken)
		{
			// Create the update 
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			// Flag for whether to update batches
			bool updateBatches = false;

			JobId jobId = jobDocument.Id;

			// Build the update list
			jobDocument = Clone(jobDocument);

			if (name != null)
			{
				jobDocument.Name = name;
				updates.Add(updateBuilder.Set(x => x.Name, jobDocument.Name));
				JobAudit(jobId, "Job Name set {Name}", name);
			}
			if (priority != null)
			{
				jobDocument.Priority = priority.Value;
				updates.Add(updateBuilder.Set(x => x.Priority, jobDocument.Priority));
				JobAudit(jobId, "Job Priority set {Priority}", priority.Value);
			}
			if (autoSubmit != null)
			{
				jobDocument.AutoSubmit = autoSubmit.Value;
				updates.Add(updateBuilder.Set(x => x.AutoSubmit, jobDocument.AutoSubmit));
				JobAudit(jobId, "Job AutoSubmit set to {AutoSubmit}", autoSubmit.Value);
			}
			if (autoSubmitChange != null)
			{
				jobDocument.AutoSubmitChange = autoSubmitChange.Value;
				updates.Add(updateBuilder.Set(x => x.AutoSubmitChange, jobDocument.AutoSubmitChange));
				JobAudit(jobId, "Job AutoSubmitChange set {AutoSubmitChange}", autoSubmitChange.Value);
			}
			if (autoSubmitMessage != null)
			{
				jobDocument.AutoSubmitMessage = (autoSubmitMessage.Length == 0) ? null : autoSubmitMessage;
				updates.Add(updateBuilder.SetOrUnsetNullRef(x => x.AutoSubmitMessage, jobDocument.AutoSubmitMessage));
				JobAudit(jobId, "Job AutoSubmitMessage set {AutoSubmitMessage}", autoSubmitMessage);
			}
			if (abortedByUserId != null && jobDocument.AbortedByUserId == null)
			{
				jobDocument.AbortedByUserId = abortedByUserId;
				updates.Add(updateBuilder.Set(x => x.AbortedByUserId, jobDocument.AbortedByUserId));
				JobAudit(jobId, "Job AbortedByUserId set {AbortedByUserId}", abortedByUserId.Value);
				updateBatches = true;
			}

			if (!String.IsNullOrEmpty(cancellationReason))
			{
				jobDocument.CancellationReason = cancellationReason;
				updates.Add(updateBuilder.Set(x => x.CancellationReason, jobDocument.CancellationReason));
				JobAudit(jobId, "Job CancellationReason set {CancellationReason}", cancellationReason);
			}

			if (notificationTriggerId != null)
			{
				jobDocument.NotificationTriggerId = notificationTriggerId.Value;
				updates.Add(updateBuilder.Set(x => x.NotificationTriggerId, notificationTriggerId));
				JobAudit(jobId, "Job NotificationTriggerId set {NotificationTriggerId}", notificationTriggerId.Value);
			}
			if (labelIdxToTriggerId != null)
			{
				if (jobDocument._labelNotifications.Any(x => x._labelIdx == labelIdxToTriggerId.Value.Key))
				{
					throw new ArgumentException("Cannot update label trigger that already exists");
				}
				jobDocument._labelNotifications.Add(new LabelNotificationDocument { _labelIdx = labelIdxToTriggerId.Value.Key, _triggerId = labelIdxToTriggerId.Value.Value });
				updates.Add(updateBuilder.Set(x => x._labelNotifications, jobDocument._labelNotifications));
			}
			if (jobTrigger != null)
			{
				for (int idx = 0; idx < jobDocument.ChainedJobs.Count; idx++)
				{
					ChainedJobDocument jobTriggerDocument = jobDocument.ChainedJobs[idx];
					if (jobTriggerDocument.TemplateRefId == jobTrigger.Value.Key)
					{
						int localIdx = idx;
						jobTriggerDocument.JobId = jobTrigger.Value.Value;
						updates.Add(updateBuilder.Set(x => x.ChainedJobs[localIdx].JobId, jobTrigger.Value.Value));
						try
						{
							JobAudit(jobId, "Job {JobId}, ChainedJobs Index {Index}, set to JobTrigger {JobId}", jobId, localIdx, jobTrigger.Value.Value.Id);
						}
						catch (Exception)
						{
						}
					}
				}
			}
			if (reports != null)
			{
				jobDocument.Reports ??= new List<JobReport>();
				jobDocument.Reports.RemoveAll(x => reports.Any(y => y.Name == x.Name));
				jobDocument.Reports.AddRange(reports);
				updates.Add(updateBuilder.Set(x => x.Reports, jobDocument.Reports));

				JobAudit(jobId, "Job, adding Reports {Reports}", String.Join(',', reports.Select(x => x.Name)));
			}

			if (arguments != null)
			{
				HashSet<string> modifiedArguments = new HashSet<string>(jobDocument.Arguments);
				modifiedArguments.SymmetricExceptWith(arguments);

				foreach (string modifiedArgument in modifiedArguments)
				{
					if (modifiedArgument.StartsWith(IJob.TargetArgumentPrefix, StringComparison.OrdinalIgnoreCase))
					{
						updateBatches = true;
					}
				}

				jobDocument.Arguments = arguments.ToList();
				updates.Add(updateBuilder.Set(x => x.Arguments, jobDocument.Arguments));
				JobAudit(jobId, "Job setting Arguments {Arguments}", String.Join(',', jobDocument.Arguments));
			}

			// Update the batches
			if (updateBatches)
			{
				UpdateBatchesInternal(jobDocument, graph, updates, _logger, GetJobLogger(jobId));
			}

			// Update the new list of job steps
			return await TryUpdateAsync(jobDocument, updateBuilder.Combine(updates), cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryUpdateBatchAsync(JobDocument jobDocument, IGraph graph, JobStepBatchId batchId, LogId? newLogId, JobStepBatchState? newState, JobStepBatchError? newError, CancellationToken cancellationToken)
		{
			jobDocument = Clone(jobDocument);

			// Find the index of the appropriate batch
			int batchIdx = jobDocument.Batches.FindIndex(x => x.Id == batchId);
			if (batchIdx == -1)
			{
				return null;
			}

			// If we're marking the batch as complete and there are still steps to run (eg. because the agent crashed), we need to mark all the steps as complete first
			JobStepBatchDocument batch = jobDocument.Batches[batchIdx];

			// Create the update 
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			// Update the batch
			if (newLogId != null)
			{
				batch.LogId = newLogId.Value;
				updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].LogId, batch.LogId));
			}
			if (newState != null)
			{
				batch.State = newState.Value;
				updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].State, batch.State));

				if (batch.StartTime == null && newState >= JobStepBatchState.Starting)
				{
					batch.StartTime = _clock.UtcNow;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].StartTime, batch.StartTime));
				}
				if (newState == JobStepBatchState.Complete)
				{
					batch.FinishTime = _clock.UtcNow;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].FinishTime, batch.FinishTime));
				}
			}
			if (newError != null && newError.Value != batch.Error)
			{
				batch.Error = newError.Value;

				// Only allow retrying nodes that may succeed if run again
				bool allowRetrying = !IsFatalBatchError(newError.Value);

				// Update the state of the nodes
				List<NodeRef> retriedNodes = jobDocument.RetriedNodes ?? new List<NodeRef>();
				foreach (JobStepDocument step in batch.Steps)
				{
					if (step.State == JobStepState.Running)
					{
						step.State = JobStepState.Completed;
						step.Outcome = JobStepOutcome.Failure;
						step.Error = JobStepError.Incomplete;

						if (allowRetrying && CanRetryNode(jobDocument, batch.GroupIdx, step.NodeIdx))
						{
							step.Retry = true;
							retriedNodes.Add(new NodeRef(batch.GroupIdx, step.NodeIdx));
						}
					}
					else if (step.State == JobStepState.Ready || step.State == JobStepState.Waiting)
					{
						if (allowRetrying && CanRetryNode(jobDocument, batch.GroupIdx, step.NodeIdx))
						{
							retriedNodes.Add(new NodeRef(batch.GroupIdx, step.NodeIdx));
						}
						else
						{
							step.State = JobStepState.Skipped;
						}
					}
				}

				// Force an update of all batches in the job. This will fail or reschedule any nodes that can no longer be executed in this batch.
				updates.Clear();
				UpdateBatchesInternal(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id));

				if (retriedNodes.Count > 0)
				{
					jobDocument.RetriedNodes = retriedNodes;
					updates.Add(updateBuilder.Set(x => x.RetriedNodes, jobDocument.RetriedNodes));
				}
			}

			// Update the new list of job steps
			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		/// <summary>
		/// Marks a node for retry and - if it's a skipped node - all of the nodes that caused it to be skipped
		/// </summary>
		static void RetryNodesRecursive(JobDocument jobDocument, IGraph graph, int groupIdx, int nodeIdx, UserId retryByUserId, List<UpdateDefinition<JobDocument>> updates)
		{
			HashSet<NodeRef> retryNodes = new HashSet<NodeRef>();
			retryNodes.Add(new NodeRef(groupIdx, nodeIdx));

			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			for (int batchIdx = jobDocument.Batches.Count - 1; batchIdx >= 0; batchIdx--)
			{
				JobStepBatchDocument batch = jobDocument.Batches[batchIdx];
				for (int stepIdx = batch.Steps.Count - 1; stepIdx >= 0; stepIdx--)
				{
					JobStepDocument step = batch.Steps[stepIdx];

					NodeRef nodeRef = new NodeRef(batch.GroupIdx, step.NodeIdx);
					if (retryNodes.Remove(nodeRef))
					{
						if (step.State == JobStepState.Skipped && batch.Error == JobStepBatchError.None)
						{
							// Add all the dependencies to be retried
							NodeRef[] dependencies = graph.GetNode(nodeRef).InputDependencies;
							retryNodes.UnionWith(dependencies);
							NodeRef[] optionalDependencies = graph.GetNode(nodeRef).OptionalInputDependencies;
							retryNodes.UnionWith(optionalDependencies);
						}
						else
						{
							// Retry this step
							int lambdaBatchIdx = batchIdx;
							int lambdaStepIdx = stepIdx;

							step.Retry = true;
							updates.Add(updateBuilder.Set(x => x.Batches[lambdaBatchIdx].Steps[lambdaStepIdx].Retry, true));

							step.RetriedByUserId = retryByUserId;
							updates.Add(updateBuilder.Set(x => x.Batches[lambdaBatchIdx].Steps[lambdaStepIdx].RetriedByUserId, step.RetriedByUserId));

							// If the step is currently running, abort it and mark the batch as no longer needed
							// This ensures the agent stops executing the old step and the lease is cancelled
							if (step is { State: JobStepState.Running, AbortRequested: false })
							{
								step.AbortRequested = true;
								updates.Add(updateBuilder.Set(x => x.Batches[lambdaBatchIdx].Steps[lambdaStepIdx].AbortRequested, true));

								if (batch.Error == JobStepBatchError.None)
								{
									batch.Error = JobStepBatchError.NoLongerNeeded;
									updates.Add(updateBuilder.Set(x => x.Batches[lambdaBatchIdx].Error, JobStepBatchError.NoLongerNeeded));
								}
							}
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryUpdateStepAsync(JobDocument jobDocument, IGraph graph, JobStepBatchId batchId, JobStepId stepId, JobStepState newState, JobStepOutcome newOutcome, JobStepError? newError, bool? newAbortRequested, UserId? newAbortByUserId, LogId? newLogId, NotificationTriggerId? newNotificationTriggerId, UserId? newRetryByUserId, Priority? newPriority, List<JobReport>? newReports, Dictionary<string, string?>? newProperties, string? newCancellationReason, JobId? newSpawnedJob, CancellationToken cancellationToken)
		{
			jobDocument = Clone(jobDocument);

			// Create the update 
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			// Update the appropriate batch
			bool refreshBatches = false;
			bool refreshDependentJobSteps = false;
			for (int loopBatchIdx = 0; loopBatchIdx < jobDocument.Batches.Count; loopBatchIdx++)
			{
				int batchIdx = loopBatchIdx; // For lambda capture
				JobStepBatchDocument batch = jobDocument.Batches[batchIdx];
				if (batch.Id == batchId)
				{
					for (int loopStepIdx = 0; loopStepIdx < batch.Steps.Count; loopStepIdx++)
					{
						int stepIdx = loopStepIdx; // For lambda capture
						JobStepDocument step = batch.Steps[stepIdx];
						if (step.Id == stepId)
						{
							// Update the request abort status
							if (newAbortRequested != null && step.AbortRequested == false)
							{
								step.AbortRequested = newAbortRequested.Value;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].AbortRequested, step.AbortRequested));

								// If the step is pending, and not running on an agent, set to aborted
								if (JobStepExtensions.IsPendingState(step.State) && step.State != JobStepState.Running)
								{
									newState = JobStepState.Aborted;
									newOutcome = JobStepOutcome.Failure;
								}
								// If the step IS running, mark the batch for cancellation to trigger lease cancellation
								else if (step.State == JobStepState.Running && batch.Error == JobStepBatchError.None)
								{
									batch.Error = JobStepBatchError.Cancelled;
									updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Error, JobStepBatchError.Cancelled));
								}

								refreshDependentJobSteps = true;
							}

							// Update the user that requested the abort
							if (newAbortByUserId != null && step.AbortedByUserId == null)
							{
								step.AbortedByUserId = newAbortByUserId;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].AbortedByUserId, step.AbortedByUserId));

								refreshDependentJobSteps = true;
							}

							// Update the reason the step was canceled
							if (!String.IsNullOrEmpty(newCancellationReason))
							{
								step.CancellationReason = newCancellationReason;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].CancellationReason, step.CancellationReason));

								refreshDependentJobSteps = true;
							}

							// Update the state
							if (newState != JobStepState.Unspecified && step.State != newState)
							{
								if (batch.State == JobStepBatchState.Starting)
								{
									batch.State = JobStepBatchState.Running;
									updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].State, batch.State));
								}

								step.State = newState;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].State, step.State));

								if (step.State == JobStepState.Running)
								{
									step.StartTime = _clock.UtcNow;
									updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].StartTime, step.StartTime));
								}
								else if (step.State == JobStepState.Completed || step.State == JobStepState.Aborted)
								{
									step.FinishTime = _clock.UtcNow;
									updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].FinishTime, step.FinishTime));
								}

								refreshDependentJobSteps = true;
							}

							// Update the job outcome
							if (newOutcome != JobStepOutcome.Unspecified && step.Outcome != newOutcome)
							{
								step.Outcome = newOutcome;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Outcome, step.Outcome));

								refreshDependentJobSteps = true;
							}

							// Update the job step error
							if (newError != null)
							{
								step.Error = newError.Value;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Error, step.Error));
							}

							// Update the log id
							if (newLogId != null && step.LogId != newLogId.Value)
							{
								step.LogId = newLogId.Value;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].LogId, step.LogId));

								refreshDependentJobSteps = true;
							}

							// Update the notification trigger id
							if (newNotificationTriggerId != null && step.NotificationTriggerId == null)
							{
								step.NotificationTriggerId = newNotificationTriggerId.Value;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].NotificationTriggerId, step.NotificationTriggerId));
							}

							// Update the retry flag
							if (newRetryByUserId != null && step.RetriedByUserId == null)
							{
								RetryNodesRecursive(jobDocument, graph, batch.GroupIdx, step.NodeIdx, newRetryByUserId.Value, updates);
								refreshBatches = true;
							}

							// Update the priority
							if (newPriority != null && newPriority.Value != step.Priority)
							{
								step.Priority = newPriority.Value;
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Priority, step.Priority));

								refreshBatches = true;
							}

							// Add any new reports
							if (newReports != null)
							{
								step.Reports ??= new List<JobReport>();
								step.Reports.RemoveAll(x => newReports.Any(y => y.Name == x.Name));
								step.Reports.AddRange(newReports);
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Reports, step.Reports));
							}

							if (newSpawnedJob != null)
							{
								step.SpawnedJobs ??= new List<JobId>();
								step.SpawnedJobs.Add(newSpawnedJob.Value);
								updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].SpawnedJobs, step.SpawnedJobs));
							}

							// Apply any property updates
							if (newProperties != null)
							{
								step.Properties ??= new Dictionary<string, string>(StringComparer.Ordinal);

								foreach (KeyValuePair<string, string?> pair in newProperties)
								{
									if (pair.Value == null)
									{
										step.Properties.Remove(pair.Key);
									}
									else
									{
										step.Properties[pair.Key] = pair.Value;
									}
								}

								if (step.Properties.Count == 0)
								{
									step.Properties = null;
									updates.Add(updateBuilder.Unset(x => x.Batches[batchIdx].Steps[stepIdx].Properties));
								}
								else
								{
									foreach (KeyValuePair<string, string?> pair in newProperties)
									{
										if (pair.Value == null)
										{
											updates.Add(updateBuilder.Unset(x => x.Batches[batchIdx].Steps[stepIdx].Properties![pair.Key]));
										}
										else
										{
											updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Properties![pair.Key], pair.Value));
										}
									}
								}
							}
							break;
						}
					}
					break;
				}
			}

			// Update the batches
			if (refreshBatches)
			{
				updates.Clear(); // UpdateBatches will update the entire batches list. We need to remove all the individual step updates to avoid an exception.
				UpdateBatchesInternal(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id));
			}

			// Update the state of dependent jobsteps
			if (refreshDependentJobSteps)
			{
				IJob? jobObject = CreateJobObject(jobDocument, graph);
				RefreshDependentJobSteps(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id), new TelemetryContext() { BuildConfig = _buildConfig, TelemetryWriter = _telemetryWriter, Job = jobObject });
				RefreshJobPriority(jobDocument, updates, _logger, GetJobLogger(jobDocument.Id));
			}

			// Update the new list of job steps
			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		[return: NotNullIfNotNull("document")]
		Job? CreateJobObject(JobDocument? document, IGraph graph)
		{
			if (document == null)
			{
				return null;
			}
			else
			{
				return new Job(this, document, graph);
			}
		}

		[return: NotNullIfNotNull("document")]
		async Task<Job> CreateJobObjectAsync(JobDocument document, CancellationToken cancellationToken)
		{
			IGraph graph = await _graphCollection.GetAsync(document.GraphHash, cancellationToken);
			return new Job(this, document, graph);
		}

		Task<JobDocument?> TryUpdateAsync(JobDocument job, List<UpdateDefinition<JobDocument>> updates, CancellationToken cancellationToken)
		{
			if (updates.Count == 0)
			{
				return Task.FromResult<JobDocument?>(job);
			}
			else
			{
				return TryUpdateAsync(job, Builders<JobDocument>.Update.Combine(updates), cancellationToken);
			}
		}

		async Task<JobDocument?> TryUpdateAsync(JobDocument job, UpdateDefinition<JobDocument> update, CancellationToken cancellationToken)
		{
			int newUpdateIndex = job.UpdateIndex + 1;
			update = update.Set(x => x.UpdateIndex, newUpdateIndex);

			DateTime newUpdateTimeUtc = DateTime.UtcNow;
			update = update.Set(x => x.UpdateTimeUtc, newUpdateTimeUtc);

			return await _jobs.FindOneAndUpdateAsync<JobDocument>(x => x.Id == job.Id && x.UpdateIndex == job.UpdateIndex, update, new FindOneAndUpdateOptions<JobDocument, JobDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryRemoveFromDispatchQueueAsync(JobDocument jobDocument, CancellationToken cancellationToken)
		{
			jobDocument = Clone(jobDocument);

			// Create the update 
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			jobDocument.SchedulePriority = 0;
			updates.Add(updateBuilder.Set(x => x.SchedulePriority, jobDocument.SchedulePriority));

			// Update the new list of job steps
			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryUpdateGraphAsync(JobDocument jobDocument, IGraph oldGraph, IGraph newGraph, CancellationToken cancellationToken)
		{
			// Update all the references in the job to use references within the new graph
			List<int> newNodeIndexes = new List<int>();
			foreach (JobStepBatchDocument batch in jobDocument.Batches)
			{
				INodeGroup oldGroup = oldGraph.Groups[batch.GroupIdx];
				newNodeIndexes.Clear();

				// Find the new node indexes for this group
				int newGroupIdx = -1;
				for (int oldNodeIdx = 0; oldNodeIdx < oldGroup.Nodes.Count; oldNodeIdx++)
				{
					INode oldNode = oldGroup.Nodes[oldNodeIdx];

					NodeRef newNodeRef;
					if (!newGraph.TryFindNode(oldNode.Name, out newNodeRef))
					{
						throw new InvalidOperationException($"Node '{oldNode.Name}' exists in graph {oldGraph.Id}; does not exist in graph {newGraph.Id}");
					}

					if (newGroupIdx == -1)
					{
						newGroupIdx = newNodeRef.GroupIdx;
					}
					else if (newGroupIdx != newNodeRef.GroupIdx)
					{
						throw new InvalidOperationException($"Node '{oldNode.Name}' is in different group in graph {oldGraph.Id} than graph {newGraph.Id}");
					}

					newNodeIndexes.Add(newNodeRef.NodeIdx);
				}
				if (newGroupIdx == -1)
				{
					throw new InvalidOperationException($"Group {batch.GroupIdx} in graph {oldGraph.Id} does not have any nodes");
				}

				// Update all the steps
				batch.GroupIdx = newGroupIdx;
				INodeGroup newGroup = newGraph.Groups[newGroupIdx];

				foreach (JobStepDocument step in batch.Steps)
				{
					int oldNodeIdx = step.NodeIdx;
					int newNodeIdx = newNodeIndexes[oldNodeIdx];

					INode oldNode = oldGroup.Nodes[oldNodeIdx];
					INode newNode = newGroup.Nodes[newNodeIdx];

					if (!JobStepExtensions.IsPendingState(step.State) && !NodesMatch(oldGraph, oldNode, newGraph, newNode))
					{
						// Modified skipped steps are ignored to allow other non-dependent subtrees to still be run
						if (step.State != JobStepState.Skipped)
						{
							throw new InvalidOperationException($"Definition for node '{oldNode.Name}' has changed.");
						}
					}

					step.NodeIdx = newNodeIdx;
				}
			}

			// Create the update 
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			jobDocument.GraphHash = newGraph.Id;
			updates.Add(updateBuilder.Set(x => x.GraphHash, jobDocument.GraphHash));

			UpdateBatchesInternal(jobDocument, newGraph, updates, _logger, GetJobLogger(jobDocument.Id));

			// Update the new list of job steps
			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		static bool NodesMatch(IGraph oldGraph, INode oldNode, IGraph newGraph, INode newNode)
		{
			IEnumerable<string> oldInputDependencies = oldNode.InputDependencies.Select(x => oldGraph.GetNode(x).Name);
			IEnumerable<string> newInputDependencies = newNode.InputDependencies.Select(x => newGraph.GetNode(x).Name);
			if (!CompareListsIgnoreOrder(oldInputDependencies, newInputDependencies))
			{
				return false;
			}

			IEnumerable<string> oldOptionalInputDependencies = oldNode.OptionalInputDependencies.Select(x => oldGraph.GetNode(x).Name);
			IEnumerable<string> newOptionalInputDependencies = newNode.OptionalInputDependencies.Select(x => newGraph.GetNode(x).Name);
			if (!CompareListsIgnoreOrder(oldOptionalInputDependencies, newOptionalInputDependencies))
			{
				return false;
			}

			IEnumerable<string> oldInputs = oldNode.Inputs.Select(x => oldGraph.GetNode(x.NodeRef).OutputNames[x.OutputIdx]);
			IEnumerable<string> newInputs = newNode.Inputs.Select(x => newGraph.GetNode(x.NodeRef).OutputNames[x.OutputIdx]);
			if (!CompareListsIgnoreOrder(oldInputs, newInputs))
			{
				return false;
			}

			IEnumerable<string> oldOptionalInputs = oldNode.OptionalInputs.Select(x => oldGraph.GetNode(x.NodeRef).OutputNames[x.OutputIdx]);
			IEnumerable<string> newOptionalInputs = newNode.OptionalInputs.Select(x => newGraph.GetNode(x.NodeRef).OutputNames[x.OutputIdx]);
			if (!CompareListsIgnoreOrder(oldOptionalInputs, newOptionalInputs))
			{
				return false;
			}

			return CompareListsIgnoreOrder(oldNode.OutputNames, newNode.OutputNames);
		}

		static bool CompareListsIgnoreOrder(IEnumerable<string> seq1, IEnumerable<string> seq2)
		{
			return new HashSet<string>(seq1, StringComparer.OrdinalIgnoreCase).SetEquals(seq2);
		}

		/// <inheritdoc/>
		public async Task AddIssueToJobAsync(JobId jobId, int issueId, CancellationToken cancellationToken)
		{
			FilterDefinition<JobDocument> jobFilter = Builders<JobDocument>.Filter.Eq(x => x.Id, jobId);
			UpdateDefinition<JobDocument> jobUpdate = Builders<JobDocument>.Update.AddToSet(x => x.ReferencedByIssues, issueId).Inc(x => x.UpdateIndex, 1).Max(x => x.UpdateTimeUtc, DateTime.UtcNow);
			await _jobs.UpdateOneAsync(jobFilter, jobUpdate, null, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IJob>> GetDispatchQueueAsync(CancellationToken cancellationToken)
		{
			List<JobDocument> newJobs = await _jobs.Find(x => x.SchedulePriority > 0).SortByDescending(x => x.SchedulePriority).ThenBy(x => x.CreateTimeUtc).ToListAsync(cancellationToken);

			List<Job> results = new List<Job>();
			foreach (JobDocument result in newJobs)
			{
				await PostLoadAsync(result);
				results.Add(await CreateJobObjectAsync(result, cancellationToken));
			}
			return results;
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TrySkipAllBatchesAsync(JobDocument jobDocument, IGraph graph, JobStepBatchError reason, CancellationToken cancellationToken)
		{
			for (int batchIdx = 0; batchIdx < jobDocument.Batches.Count; batchIdx++)
			{
				JobStepBatchDocument batch = jobDocument.Batches[batchIdx];
				if (batch.State == JobStepBatchState.Ready || batch.State == JobStepBatchState.Waiting)
				{
					_logger.LogInformation("Skipping all batches: job {JobId}, batch {BatchId}", jobDocument.Id, batch.Id);
					JobAudit(jobDocument.Id, "Skipping all batches: job {JobId}, batch {BatchId}", jobDocument.Id, batch.Id);

					batch.State = JobStepBatchState.Complete;
					batch.Error = reason;
					batch.FinishTime = DateTimeOffset.UtcNow;

					for (int stepIdx = 0; stepIdx < batch.Steps.Count; stepIdx++)
					{
						JobStepDocument step = batch.Steps[stepIdx];
						if (step.State == JobStepState.Ready || step.State == JobStepState.Waiting)
						{
							step.State = JobStepState.Completed;
							step.Outcome = JobStepOutcome.Failure;
						}
					}
				}
			}

			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();
			UpdateBatchesInternal(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id));

			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TrySkipBatchAsync(JobDocument jobDocument, JobStepBatchId batchId, IGraph graph, JobStepBatchError reason, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Trying to skip batch: job {JobId}, batch {BatchId}", jobDocument.Id, batchId);
			JobAudit(jobDocument.Id, "Trying to skip batch: {BatchId}", batchId);

			JobStepBatchDocument? batch = jobDocument.Batches.FirstOrDefault(x => x.Id == batchId);
			if (batch == null)
			{
				return jobDocument;
			}

			batch.State = JobStepBatchState.Complete;
			batch.Error = reason;
			batch.FinishTime = DateTimeOffset.UtcNow;

			foreach (JobStepDocument step in batch.Steps)
			{
				if (step.State != JobStepState.Skipped)
				{
					step.State = JobStepState.Skipped;
					step.Outcome = JobStepOutcome.Failure;
				}
			}

			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();
			UpdateBatchesInternal(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id));

			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryAssignLeaseAsync(JobDocument jobDocument, int batchIdx, PoolId poolId, AgentId agentId, SessionId sessionId, LeaseId leaseId, LogId logId, CancellationToken cancellationToken)
		{
			// Try to update the job with this agent id
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].PoolId, poolId));
			updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].AgentId, agentId));
			updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].SessionId, sessionId));
			updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].LeaseId, leaseId));
			updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].LogId, logId));

			// Extra logs for catching why session IDs sometimes doesn't match in prod. Resulting in PermissionDenied errors.
			if (batchIdx < jobDocument.Batches.Count && jobDocument.Batches[batchIdx].SessionId != null)
			{
				string currentSessionId = jobDocument.Batches[batchIdx].SessionId!.Value.ToString();
				_logger.LogError("Attempt to replace current session ID {CurrSessionId} with {NewSessionId} for batch {JobId}:{BatchId}", currentSessionId, sessionId.ToString(), jobDocument.Id.ToString(), jobDocument.Batches[batchIdx].Id);
				JobAudit(jobDocument.Id, "Attempt to replace current session ID {CurrSessionId} with {NewSessionId} for batch {JobId}:{BatchId}", currentSessionId, sessionId.ToString(), jobDocument.Id.ToString(), jobDocument.Batches[batchIdx].Id);

				return null;
			}

			return await TryUpdateAsync(jobDocument, updates, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryCancelLeaseAsync(JobDocument jobDocument, int batchIdx, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Cancelling lease {LeaseId} for agent {AgentId}", jobDocument.Batches[batchIdx].LeaseId, jobDocument.Batches[batchIdx].AgentId);
			JobAudit(jobDocument.Id, "Cancelling lease {LeaseId} for agent {AgentId}", jobDocument.Batches[batchIdx].LeaseId, jobDocument.Batches[batchIdx].AgentId);

			jobDocument = Clone(jobDocument);

			UpdateDefinition<JobDocument> update = Builders<JobDocument>.Update.Unset(x => x.Batches[batchIdx].AgentId).Unset(x => x.Batches[batchIdx].SessionId).Unset(x => x.Batches[batchIdx].LeaseId);
			if (await TryUpdateAsync(jobDocument, update, cancellationToken) == null)
			{
				return null;
			}

			JobStepBatchDocument batch = jobDocument.Batches[batchIdx];
			batch.AgentId = null;
			batch.SessionId = null;
			batch.LeaseId = null;

			return jobDocument;
		}

		/// <inheritdoc/>
		async Task<JobDocument?> TryFailBatchAsync(JobDocument jobDocument, int batchIdx, IGraph graph, JobStepBatchError error, CancellationToken cancellationToken)
		{
			jobDocument = Clone(jobDocument);

			JobStepBatchDocument batch = jobDocument.Batches[batchIdx];
			_logger.LogInformation("Failing batch {JobId}:{BatchId} with error {Error}", jobDocument.Id, batch.Id, error);
			JobAudit(jobDocument.Id, "Failing batch {JobId}:{BatchId} with error {Error}", jobDocument.Id, batch.Id, error);

			UpdateDefinitionBuilder<JobDocument> updateBuilder = new UpdateDefinitionBuilder<JobDocument>();
			List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

			if (batch.State != JobStepBatchState.Complete)
			{
				batch.State = JobStepBatchState.Complete;
				updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].State, batch.State));

				batch.Error = error;
				updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Error, batch.Error));

				batch.FinishTime = DateTimeOffset.UtcNow;
				updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].FinishTime, batch.FinishTime));
			}

			for (int stepIdx = 0; stepIdx < batch.Steps.Count; stepIdx++)
			{
				JobStepDocument step = batch.Steps[stepIdx];
				if (step.State == JobStepState.Running)
				{
					int stepIdxCopy = stepIdx;

					step.State = JobStepState.Aborted;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdxCopy].State, step.State));

					step.Outcome = JobStepOutcome.Failure;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdxCopy].Outcome, step.Outcome));

					step.FinishTime = _clock.UtcNow;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdxCopy].FinishTime, step.FinishTime));
				}
				else if (step.State == JobStepState.Ready)
				{
					int stepIdxCopy = stepIdx;

					step.State = JobStepState.Skipped;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdxCopy].State, step.State));

					step.Outcome = JobStepOutcome.Failure;
					updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdxCopy].Outcome, step.Outcome));
				}
			}

			IJob? jobObject = CreateJobObject(jobDocument, graph);
			RefreshDependentJobSteps(jobDocument, graph, updates, _logger, GetJobLogger(jobDocument.Id), new TelemetryContext() { BuildConfig = _buildConfig, TelemetryWriter = _telemetryWriter, Job = jobObject });
			RefreshJobPriority(jobDocument, updates, _logger, GetJobLogger(jobDocument.Id));

			return await TryUpdateAsync(jobDocument, updateBuilder.Combine(updates), cancellationToken);
		}

#pragma warning disable CA1822 // Mark members as static
		private void UpdateBatchesInternal(JobDocument job, IGraph graph, List<UpdateDefinition<JobDocument>> updates, ILogger logger, IAuditLogChannel<JobId> jobAudit)
#pragma warning restore CA1822 // Mark members as static
		{
			UpdateBatches(job, graph, updates, logger, jobAudit);
		}

#pragma warning disable CA1822 // Mark members as static
		private void CreateBatchesInternal(JobDocument job, IGraph graph, ILogger logger, IAuditLogChannel<JobId> jobAudit)
#pragma warning restore CA1822 // Mark members as static
		{
			CreateBatches(job, graph, logger, jobAudit);
		}

		/// <summary>
		/// Populate the list of batches for a job
		/// </summary>
		/// <param name="job">The job to update</param>
		/// <param name="graph">The graph definition for this job</param>
		/// <param name="logger">Logger for output messages</param>
		/// <param name="jobAudit"></param>
		private void CreateBatches(JobDocument job, IGraph graph, ILogger logger, IAuditLogChannel<JobId> jobAudit)
		{
			UpdateBatches(job, graph, new List<UpdateDefinition<JobDocument>>(), logger, jobAudit);
		}

		/// <summary>
		/// Updates the list of batches for a job
		/// </summary>
		/// <param name="job">Job to update</param>
		/// <param name="graph">Graph definition for this job</param>
		/// <param name="updates">List of updates for the job</param>
		/// <param name="logger">Logger for updates</param>
		/// <param name="jobAudit"></param>
		void UpdateBatches(JobDocument job, IGraph graph, List<UpdateDefinition<JobDocument>> updates, ILogger logger, IAuditLogChannel<JobId> jobAudit)
		{
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;

			// Update the list of batches
			CreateOrUpdateBatches(job, graph, logger, jobAudit);
			updates.Add(updateBuilder.Set(x => x.Batches, job.Batches));
			updates.Add(updateBuilder.Set(x => x.NextSubResourceId, job.NextSubResourceId));

			// Update all the dependencies
			IJob? jobObject = CreateJobObject(job, graph);
			RefreshDependentJobSteps(job, graph, new List<UpdateDefinition<JobDocument>>(), logger, jobAudit, new TelemetryContext() { BuildConfig = _buildConfig, TelemetryWriter = _telemetryWriter, Job = jobObject });
			RefreshJobPriority(job, updates, logger, jobAudit);
		}

		static void RemoveSteps(JobStepBatchDocument batch, Predicate<JobStepDocument> predicate, Dictionary<NodeRef, JobStepId> recycleStepIds)
		{
			for (int idx = batch.Steps.Count - 1; idx >= 0; idx--)
			{
				JobStepDocument step = batch.Steps[idx];
				if (predicate(step))
				{
					recycleStepIds[new NodeRef(batch.GroupIdx, step.NodeIdx)] = step.Id;
					batch.Steps.RemoveAt(idx);
				}
			}
		}

		static void RemoveBatches(JobDocument job, Predicate<JobStepBatchDocument> predicate, Dictionary<int, JobStepBatchId> recycleBatchIds)
		{
			for (int idx = job.Batches.Count - 1; idx >= 0; idx--)
			{
				JobStepBatchDocument batch = job.Batches[idx];
				if (predicate(batch))
				{
					recycleBatchIds[batch.GroupIdx] = batch.Id;
					job.Batches.RemoveAt(idx);
				}
			}
		}

		/// <summary>
		/// Update the jobsteps for the given node graph 
		/// </summary>
		/// <param name="job">The job to update</param>
		/// <param name="graph">The graph for this job</param>
		/// <param name="logger">Logger for any changes</param>
		/// <param name="jobAudit"></param>
		private static void CreateOrUpdateBatches(JobDocument job, IGraph graph, ILogger logger, IAuditLogChannel<JobId> jobAudit)
		{

			JobAudit(job.Id, logger, jobAudit, "CreateOrUpdateBatches: Begin");

			// Find the priorities of each node, incorporating all the per-step overrides
			Dictionary<INode, Priority> nodePriorities = new Dictionary<INode, Priority>();
			foreach (INodeGroup group in graph.Groups)
			{
				foreach (INode node in group.Nodes)
				{
					nodePriorities[node] = node.Priority;
				}
			}
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				INodeGroup group = graph.Groups[batch.GroupIdx];
				foreach (JobStepDocument step in batch.Steps)
				{
					if (step.Priority != null)
					{
						INode node = group.Nodes[step.NodeIdx];
						nodePriorities[node] = step.Priority.Value;
					}
				}
			}

			// Remove any steps and batches that haven't started yet, saving their ids so we can re-use them if we re-add them
			Dictionary<NodeRef, JobStepId> recycleStepIds = new Dictionary<NodeRef, JobStepId>();
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				RemoveSteps(batch, x => x.State == JobStepState.Waiting || x.State == JobStepState.Ready, recycleStepIds);
			}

			// Mark any steps in failed batches as skipped
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				if (batch.State == JobStepBatchState.Complete && batch.Error != JobStepBatchError.Incomplete)
				{
					foreach (JobStepDocument step in batch.Steps)
					{
						if (JobStepExtensions.IsPendingState(step.State))
						{
							step.State = JobStepState.Skipped;
							JobAudit(job.Id, logger, jobAudit, "Setting batch {BatchId}, step {StepId} to Skipped", batch.Id, step.Id);
						}
					}
				}
			}

			// Remove any skipped nodes whose skipped state is no longer valid
			HashSet<INode> failedNodes = new HashSet<INode>();
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				INodeGroup group = graph.Groups[batch.GroupIdx];
				foreach (JobStepDocument step in batch.Steps)
				{
					INode node = group.Nodes[step.NodeIdx];
					if (step.Retry)
					{
						failedNodes.Remove(node);
					}
					else if (batch.State == JobStepBatchState.Complete && IsFatalBatchError(batch.Error))
					{
						failedNodes.Add(node);
					}
					else if (step.State == JobStepState.Skipped)
					{
						if (node.InputDependencies.Any(x => failedNodes.Contains(graph.GetNode(x))) || !CanRetryNode(job, batch.GroupIdx, step.NodeIdx))
						{
							failedNodes.Add(node);
						}
						else
						{
							failedNodes.Remove(node);
						}

						// Optional input dependencies are not checked here, failing optional dependencies do not cause the node to be skipped
					}
					else if (step.Outcome == JobStepOutcome.Failure)
					{
						failedNodes.Add(node);
					}
					else
					{
						failedNodes.Remove(node);
					}
				}
				RemoveSteps(batch, x => x.State == JobStepState.Skipped && !failedNodes.Contains(group.Nodes[x.NodeIdx]), recycleStepIds);
			}

			// Remove any batches which are now empty
			Dictionary<int, JobStepBatchId> recycleBatchIds = new Dictionary<int, JobStepBatchId>();
			RemoveBatches(job, x => x.Steps.Count == 0 && x.LeaseId == null && x.Error == JobStepBatchError.None, recycleBatchIds);
			if (recycleBatchIds.Count > 0)
			{
				JobAudit(job.Id, logger, jobAudit, "Recycled removed batch ids {RecycledIds}", String.Join(',', recycleBatchIds.Values));
			}

			// Find all the targets in this job
			HashSet<string> targets = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (job.AbortedByUserId == null)
			{
				foreach (string argument in job.Arguments)
				{
					if (argument.StartsWith(IJob.TargetArgumentPrefix, StringComparison.OrdinalIgnoreCase))
					{
						targets.UnionWith(argument.Substring(IJob.TargetArgumentPrefix.Length).Split(';'));
					}
				}
				targets.Add(IJob.SetupNodeName);
			}

			JobAudit(job.Id, logger, jobAudit, "Job targets {Targets}", String.Join(',', targets));

			// Add all the referenced aggregates
			HashSet<INode> newNodesToExecute = new HashSet<INode>();
			foreach (IAggregate aggregate in graph.Aggregates)
			{
				if (targets.Contains(aggregate.Name))
				{
					newNodesToExecute.UnionWith(aggregate.Nodes.Select(x => graph.GetNode(x)));
				}
			}

			// Add any individual nodes
			foreach (INode node in graph.Groups.SelectMany(x => x.Nodes))
			{
				if (targets.Contains(node.Name))
				{
					newNodesToExecute.Add(node);
				}
			}

			// Also add any dependencies of these nodes
			for (int groupIdx = graph.Groups.Count - 1; groupIdx >= 0; groupIdx--)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = group.Nodes.Count - 1; nodeIdx >= 0; nodeIdx--)
				{
					INode node = group.Nodes[nodeIdx];
					if (newNodesToExecute.Contains(node))
					{
						foreach (NodeRef dependency in node.InputDependencies)
						{
							newNodesToExecute.Add(graph.GetNode(dependency));
						}
						foreach (NodeRef dependency in node.OptionalInputDependencies)
						{
							newNodesToExecute.Add(graph.GetNode(dependency));
						}
					}
				}
			}

			// Cancel any batches which are still running but are no longer required
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				if (batch.State == JobStepBatchState.Starting && batch.Steps.Count == 0)
				{
					// This batch is still starting but hasn't executed anything yet. Don't cancel it; we can still append to it.
				}
				else if (batch.State == JobStepBatchState.Starting || batch.State == JobStepBatchState.Running)
				{
					INodeGroup group = graph.Groups[batch.GroupIdx];
					if (!batch.Steps.Any(x => newNodesToExecute.Contains(group.Nodes[x.NodeIdx])))
					{
						logger.LogInformation("Job {JobId} batch {BatchId} (lease {LeaseId}) is being cancelled; {NodeCount} nodes are not set to be executed", job.Id, batch.Id, batch.LeaseId, batch.Steps.Count);
						foreach (JobStepDocument step in batch.Steps)
						{
							logger.LogInformation("Step {JobId}:{BatchId}:{StepId} is no longer needed ({NodeName})", job.Id, batch.Id, step.Id, group.Nodes[step.NodeIdx].Name);
						}
						batch.Error = JobStepBatchError.NoLongerNeeded;
					}
				}
			}

			// Remove all the nodes which have already succeeded
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				foreach (JobStepDocument step in batch.Steps)
				{
					if ((step.State == JobStepState.Running && !step.Retry)
						|| (step.State == JobStepState.Completed && !step.Retry)
						|| (step.State == JobStepState.Aborted && !step.Retry)
						|| (step.State == JobStepState.Skipped))
					{
						newNodesToExecute.Remove(graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx]);
					}
				}
			}

			// Re-add all the nodes that have input dependencies in the same group.
			for (int groupIdx = graph.Groups.Count - 1; groupIdx >= 0; groupIdx--)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = group.Nodes.Count - 1; nodeIdx >= 0; nodeIdx--)
				{
					INode node = group.Nodes[nodeIdx];
					if (newNodesToExecute.Contains(node))
					{
						foreach (NodeRef dependency in node.InputDependencies)
						{
							if (dependency.GroupIdx == groupIdx)
							{
								newNodesToExecute.Add(group.Nodes[dependency.NodeIdx]);
							}
						}
						foreach (NodeRef dependency in node.OptionalInputDependencies)
						{
							if (dependency.GroupIdx == groupIdx)
							{
								newNodesToExecute.Add(group.Nodes[dependency.NodeIdx]);
							}
						}
					}
				}
			}

			// Build a list of nodes which are currently set to be executed
			HashSet<INode> existingNodesToExecute = new HashSet<INode>();
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				INodeGroup group = graph.Groups[batch.GroupIdx];
				foreach (JobStepDocument step in batch.Steps)
				{
					if (!step.Retry)
					{
						INode node = group.Nodes[step.NodeIdx];
						existingNodesToExecute.Add(node);
					}
				}
			}

			if (existingNodesToExecute.Count > 0)
			{
				JobAudit(job.Id, logger, jobAudit, "Existing Nodes to execute {Nodes}", String.Join(',', existingNodesToExecute.Select(x => x.Name)));
			}

			if (newNodesToExecute.Count > 0)
			{
				JobAudit(job.Id, logger, jobAudit, "New Nodes to execute {Nodes}", String.Join(',', newNodesToExecute.Select(x => x.Name)));
			}

			// Figure out the existing batch for each group
			JobStepBatchDocument?[] appendToBatches = new JobStepBatchDocument?[graph.Groups.Count];
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				if (batch.State <= JobStepBatchState.Running)
				{
					INodeGroup group = graph.Groups[batch.GroupIdx];
					appendToBatches[batch.GroupIdx] = batch;
				}
			}

			// Invalidate all the entries for groups where we're too late to append new entries (ie. we need to execute an earlier node that wasn't executed previously)
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
				{
					INode node = group.Nodes[nodeIdx];
					if (newNodesToExecute.Contains(node) && !existingNodesToExecute.Contains(node))
					{
						JobStepBatchDocument? batch = appendToBatches[groupIdx];
						if (batch != null && batch.Steps.Count > 0)
						{
							JobStepDocument lastStep = batch.Steps[batch.Steps.Count - 1];
							if (nodeIdx <= lastStep.NodeIdx)
							{
								appendToBatches[groupIdx] = null;
							}
						}
					}
				}
			}

			// Create all the new jobsteps
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
				{
					INode node = group.Nodes[nodeIdx];
					if (newNodesToExecute.Contains(node))
					{
						JobStepBatchDocument? batch = appendToBatches[groupIdx];
						if (batch == null)
						{
							JobStepBatchId batchId;
							if (!recycleBatchIds.Remove(groupIdx, out batchId))
							{
								job.NextSubResourceId = job.NextSubResourceId.Next();
								batchId = new JobStepBatchId(job.NextSubResourceId);
							}

							batch = new JobStepBatchDocument(batchId, groupIdx);
							job.Batches.Add(batch);
							appendToBatches[groupIdx] = batch;

							JobAudit(job.Id, logger, jobAudit, "Created new batch {BatchId} for {NodeName}", batchId, node.Name);
						}

						// Don't re-add nodes that have already executed in this batch. If we were missing a dependency, we would have already nulled out the entry in appendToBatches above; anything else
						// is already valid.
						if (batch.Steps.Count == 0 || nodeIdx > batch.Steps[^1].NodeIdx)
						{
							JobStepId stepId;
							if (!recycleStepIds.Remove(new NodeRef(groupIdx, nodeIdx), out stepId))
							{
								job.NextSubResourceId = job.NextSubResourceId.Next();
								stepId = new JobStepId(job.NextSubResourceId);
							}

							JobStepDocument step = new JobStepDocument(stepId, nodeIdx);
							batch.Steps.Add(step);
							JobAudit(job.Id, logger, jobAudit, "Created step {StepId} to batch {BatchId} for {StepName}", step.Id, batch.Id, node.Name);
						}
					}
				}
			}

			// Find the priority of each node, propagating dependencies from dependent nodes
			for (int groupIdx = 0; groupIdx < graph.Groups.Count; groupIdx++)
			{
				INodeGroup group = graph.Groups[groupIdx];
				for (int nodeIdx = 0; nodeIdx < group.Nodes.Count; nodeIdx++)
				{
					INode node = group.Nodes[nodeIdx];
					Priority nodePriority = nodePriorities[node];

					foreach (NodeRef dependencyRef in node.OrderDependencies)
					{
						INode dependency = graph.Groups[dependencyRef.GroupIdx].Nodes[dependencyRef.NodeIdx];
						if (nodePriorities[node] > nodePriority)
						{
							nodePriorities[dependency] = nodePriority;
						}
					}
				}
			}
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				if (batch.Steps.Count > 0)
				{
					INodeGroup group = graph.Groups[batch.GroupIdx];
					Priority nodePriority = batch.Steps.Max(x => nodePriorities[group.Nodes[x.NodeIdx]]);
					batch.SchedulePriority = ((int)job.Priority * 10) + (int)nodePriority + 1; // Reserve '0' for none.
				}
			}

			// Check we're not running a node which doesn't allow retries more than once
			Dictionary<INode, int> nodeExecutionCount = new Dictionary<INode, int>();
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				INodeGroup group = graph.Groups[batch.GroupIdx];
				foreach (JobStepDocument step in batch.Steps)
				{
					INode node = group.Nodes[step.NodeIdx];

					int count;
					nodeExecutionCount.TryGetValue(node, out count);

					if (!node.AllowRetry && count > 0)
					{
						throw new RetryNotAllowedException(node.Name);
					}

					nodeExecutionCount[node] = count + 1;
				}
			}

			JobAudit(job.Id, logger, jobAudit, "CreateOrUpdateBatches: End");
		}

		/// <summary>
		/// Test whether the nodes in a batch can still be run
		/// </summary>
		static bool IsFatalBatchError(JobStepBatchError error)
		{
			return error != JobStepBatchError.None && error != JobStepBatchError.Incomplete;
		}

		/// <summary>
		/// Tests whether a node can be retried again
		/// </summary>
		static bool CanRetryNode(JobDocument job, int groupIdx, int nodeIdx)
		{
			return job.RetriedNodes == null || job.RetriedNodes.Count(x => x.GroupIdx == groupIdx && x.NodeIdx == nodeIdx) < MaxRetries;
		}

		/// <summary>
		/// Gets the scheduling priority of this job
		/// </summary>
		/// <param name="job">Job to consider</param>
		public static int GetSchedulePriority(IJob job)
		{
			return GetSchedulePriority(((Job)job).Document);
		}

		static int GetSchedulePriority(JobDocument job)
		{
			int newSchedulePriority = 0;
			foreach (JobStepBatchDocument batch in job.Batches)
			{
				if (batch.State == JobStepBatchState.Ready)
				{
					newSchedulePriority = Math.Max(batch.SchedulePriority, newSchedulePriority);
				}
			}
			return newSchedulePriority;
		}

		/// <summary>
		/// Update the state of any jobsteps that are dependent on other jobsteps (eg. transition them from waiting to ready based on other steps completing)
		/// </summary>
		/// <param name="job">The job to update</param>
		/// <param name="graph">The graph for this job</param>
		/// <param name="updates">List of updates to the job</param>
		/// <param name="logger">Logger instance</param>
		/// <param name="jobAudit"></param>
		/// <param name="telemetryContext">Context object used in telemetry constuction, and emissions.</param>
		static void RefreshDependentJobSteps(JobDocument job, IGraph graph, List<UpdateDefinition<JobDocument>> updates, ILogger logger, IAuditLogChannel<JobId> jobAudit, TelemetryContext? telemetryContext = null)
		{
			// Update the batches
			UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
			if (job.Batches != null)
			{
				// First pass: build a complete mapping of nodes to steps so that dependency lookups
				// are not affected by the order batches appear in the list. This fixes a bug where
				// steps in earlier batches could have their dependencies missed when those dependencies
				// were in batches appended later (e.g. during a graph update while a batch is Running).
				Dictionary<INode, JobStepDocument> stepForNode = new Dictionary<INode, JobStepDocument>();
				foreach (JobStepBatchDocument batch in job.Batches)
				{
					foreach (JobStepDocument step in batch.Steps)
					{
						INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
						stepForNode[node] = step;
					}
				}

				// Second pass: evaluate dependencies and update step/batch states
				for (int loopBatchIdx = 0; loopBatchIdx < job.Batches.Count; loopBatchIdx++)
				{
					int batchIdx = loopBatchIdx; // For lambda capture
					JobStepBatchDocument batch = job.Batches[batchIdx];

					for (int loopStepIdx = 0; loopStepIdx < batch.Steps.Count; loopStepIdx++)
					{
						int stepIdx = loopStepIdx; // For lambda capture
						JobStepDocument step = batch.Steps[stepIdx];

						JobStepState newState = step.State;
						JobStepOutcome newOutcome = step.Outcome;

						INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
						if (newState == JobStepState.Waiting)
						{
							List<JobStepDocument> steps = GetDependentSteps(graph, node, stepForNode);
							HashSet<JobStepDocument> requiredSteps = new(steps);
							List<JobStepDocument> optionalSteps = GetOptionalInputDependentSteps(graph, node, stepForNode);
							requiredSteps.ExceptWith(optionalSteps);
							// An optional input step aborted by the driver (not user requested) still allows the step to move to ready.
							// A skipped optional input step does not allow continuation; only allow continuation when the dependency ran to
							// completion and failed, not when it was skipped entirely.
							// Check all dependencies for skipped, including optional ones.
							// A skipped optional dependency means its required subgraph failed, which is different from ran and failed.
							// An optional dependency that ran and failed is tolerated but one that was skipped entirely is not.
							if (steps.Any(x => x.AbortRequested || x.State == JobStepState.Skipped) || requiredSteps.Any(x => x.Outcome == JobStepOutcome.Failure))
							{
								newState = JobStepState.Skipped;
								newOutcome = JobStepOutcome.Failure;

								// [TEMPORARY][START] This explicit emission of a skipped job step ref should be removed upon UE-357024.
								try
								{
									if (telemetryContext != null && telemetryContext.Value.BuildConfig.CurrentValue.TryGetStream(job.StreamId, out StreamConfig? streamConfig) && !streamConfig.TelemetryStoreId.IsEmpty)
									{
										JobStepRefTelemetry? telemetry = JobStepRefTelemetry.ConstructJobStepRefTelemetryFromJob(telemetryContext.Value.Job, batchIdx, stepIdx, newOutcome, newState);

										if (telemetry != null)
										{
											telemetryContext.Value.TelemetryWriter.WriteEvent(streamConfig.TelemetryStoreId, telemetry);
										}
									}
								}
								catch (Exception ex)
								{
									logger.LogError(ex, "Unable to create synthesized job step ref telemetry.");
								}
								// [TEMPORARY][END] This explicit emission of a skipped job step ref should be removed upon UE-357024.
							}
							else if (!steps.Any(x => !x.AbortRequested && JobStepExtensions.IsPendingState(x.State)))
							{
								if (optionalSteps.Any(x => x.Outcome == JobStepOutcome.Failure))
								{
									logger.LogInformation("Allowing job {JobId}, batch {BatchId}, step {StepId} to transition to ready state when optional dependencies failed ({Dependencies})", job.Id, batch.Id, step.Id, String.Join(", ", optionalSteps.Select(x => x.Id.ToString())));
									JobAudit(job.Id, logger, jobAudit, "Allowing job {JobId}, batch {BatchId}, step {StepId} to transition to ready state when optional dependencies failed ({Dependencies})", job.Id, batch.Id, step.Id, String.Join(", ", optionalSteps.Select(x => x.Id.ToString())));
								}

								logger.LogInformation("Transitioning job {JobId}, batch {BatchId}, step {StepId} to ready state ({Dependencies})", job.Id, batch.Id, step.Id, String.Join(", ", steps.Select(x => x.Id.ToString())));
								JobAudit(job.Id, logger, jobAudit, "Transitioning job {JobId}, batch {BatchId}, step {StepId} to ready state ({Dependencies})", job.Id, batch.Id, step.Id, String.Join(", ", steps.Select(x => x.Id.ToString())));
								newState = JobStepState.Ready;
							}
						}

						if (newState != step.State)
						{
							step.State = newState;
							updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].State, newState));
							JobAudit(job.Id, logger, jobAudit, "Setting step {StepId} state to {StepState}", step.Id, newState);
						}

						if (newOutcome != step.Outcome)
						{
							step.Outcome = newOutcome;
							updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].Steps[stepIdx].Outcome, newOutcome));
							JobAudit(job.Id, logger, jobAudit, "Setting step {StepId} outcome to {StepOutcome}", step.Id, newOutcome);
						}
					}

					if (batch.State == JobStepBatchState.Waiting || batch.State == JobStepBatchState.Ready)
					{
						DateTime? newReadyTime;
						JobStepBatchState newState = GetBatchState(job, graph, batch, stepForNode, out newReadyTime);
						if (batch.State != newState)
						{
							batch.State = newState;
							logger.LogInformation("Transitioning job {JobId}, batch {BatchId} to state {JobStepBatchState}", job.Id, batch.Id, newState);
							JobAudit(job.Id, logger, jobAudit, "Transitioning job {JobId}, batch {BatchId} to state {JobStepBatchState}", job.Id, batch.Id, newState);
							updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].State, batch.State));
						}
						if (batch.ReadyTime != newReadyTime)
						{
							logger.LogInformation("Setting job {JobId}, batch {BatchId} ready time", job.Id, batch.Id);
							JobAudit(job.Id, logger, jobAudit, "Setting job {JobId}, batch {BatchId} ready time", job.Id, batch.Id);
							batch.ReadyTime = newReadyTime;
							updates.Add(updateBuilder.Set(x => x.Batches[batchIdx].ReadyTime, batch.ReadyTime));
						}
					}
				}
			}
		}

		/// <summary>
		/// Updates the schedule priority of a job
		/// </summary>
		/// <param name="job"></param>
		/// <param name="updates"></param>
		/// <param name="logger"></param>
		/// <param name="jobAudit"></param>
		static void RefreshJobPriority(JobDocument job, List<UpdateDefinition<JobDocument>> updates, ILogger logger, IAuditLogChannel<JobId> jobAudit)
		{
			// Update the weighted priority for the job
			int newSchedulePriority = GetSchedulePriority(job);
			if (job.SchedulePriority != newSchedulePriority)
			{
				job.SchedulePriority = newSchedulePriority;
				updates.Add(Builders<JobDocument>.Update.Set(x => x.SchedulePriority, newSchedulePriority));
				JobAudit(job.Id, logger, jobAudit, "Setting job schedule priority {SchedulePriority}", newSchedulePriority);
			}
		}

		/// <summary>
		/// Gets the steps that a node depends on
		/// </summary>
		/// <param name="graph">The graph for this job</param>
		/// <param name="node">The node to test</param>
		/// <param name="stepForNode">Map of node to step</param>
		/// <returns></returns>
		static List<JobStepDocument> GetDependentSteps(IGraph graph, INode node, Dictionary<INode, JobStepDocument> stepForNode)
		{
			List<JobStepDocument> steps = new List<JobStepDocument>();
			foreach (NodeRef orderDependencyRef in node.OrderDependencies)
			{
				JobStepDocument? step;
				if (stepForNode.TryGetValue(graph.GetNode(orderDependencyRef), out step))
				{
					steps.Add(step);
				}
			}
			return steps;
		}

		static List<JobStepDocument> GetOptionalInputDependentSteps(IGraph graph, INode node, Dictionary<INode, JobStepDocument> stepForNode)
		{
			List<JobStepDocument> steps = [];
			foreach (NodeRef optionalInputRef in node.OptionalInputDependencies)
			{
				if (stepForNode.TryGetValue(graph.GetNode(optionalInputRef), out JobStepDocument? step))
				{
					steps.Add(step);
				}
			}
			return steps;
		}

		/// <summary>
		/// Gets the new state for a batch
		/// </summary>
		/// <param name="job">The job being executed</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="batch">List of nodes in the job</param>
		/// <param name="stepForNode">Array mapping each index to the appropriate step for that node</param>
		/// <param name="outReadyTimeUtc">Receives the time at which the batch was ready to execute</param>
		/// <returns>True if the batch is ready, false otherwise</returns>
		static JobStepBatchState GetBatchState(JobDocument job, IGraph graph, JobStepBatchDocument batch, Dictionary<INode, JobStepDocument> stepForNode, out DateTime? outReadyTimeUtc)
		{
			// Check if the batch is already complete
			if (batch.Steps.All(x => x.State == JobStepState.Skipped || x.State == JobStepState.Completed || x.State == JobStepState.Aborted))
			{
				outReadyTimeUtc = batch.ReadyTime?.UtcDateTime;
				return JobStepBatchState.Complete;
			}

			// Get the dependencies for this batch to start. Some steps may be "after" dependencies that are optional parts of the graph.
			List<INode> nodeDependencies = JobStepBatchExtensions.GetStartDependencies(batch.Steps.ConvertAll(x => graph.Groups[batch.GroupIdx].Nodes[x.NodeIdx]), graph.Groups).ToList();

			// Check if we're still waiting on anything
			DateTime readyTimeUtc = job.GetCreateTimeOrDefault();
			foreach (INode nodeDependency in nodeDependencies)
			{
				JobStepDocument? stepDependency;
				if (stepForNode.TryGetValue(nodeDependency, out stepDependency))
				{
					if (stepDependency.State != JobStepState.Completed && stepDependency.State != JobStepState.Skipped && stepDependency.State != JobStepState.Aborted)
					{
						outReadyTimeUtc = null;
						return JobStepBatchState.Waiting;
					}

					if (stepDependency.FinishTime != null && stepDependency.FinishTime.Value.UtcDateTime > readyTimeUtc)
					{
						readyTimeUtc = stepDependency.FinishTime.Value.UtcDateTime;
					}
				}
			}

			// Otherwise return the ready state
			outReadyTimeUtc = readyTimeUtc;
			return JobStepBatchState.Ready;
		}

		/// <inheritdoc/>
		public async Task<IJob?> UpdateMetadataAsync(JobId jobId, List<string>? jobMetadata = null, Dictionary<JobStepId, List<string>>? stepMetadata = null, CancellationToken cancellationToken = default)
		{
			try
			{
				JobDocument? job = await _jobs.Find(x => x.Id == jobId).FirstOrDefaultAsync(cancellationToken);
				job = Clone(job);

				// Create the update 
				UpdateDefinitionBuilder<JobDocument> updateBuilder = Builders<JobDocument>.Update;
				List<UpdateDefinition<JobDocument>> updates = new List<UpdateDefinition<JobDocument>>();

				if (job == null)
				{
					return null;
				}

				if (jobMetadata != null && jobMetadata.Count > 0)
				{
					updates.Add(updateBuilder.AddToSetEach(x => x.Metadata, jobMetadata));
				}

				if (stepMetadata != null)
				{
					foreach (KeyValuePair<JobStepId, List<string>> entry in stepMetadata)
					{
						int batchIdx = job.Batches.FindIndex(x => x.Steps.FirstOrDefault(y => y.Id == entry.Key) != null);
						if (batchIdx >= 0)
						{
							int stepIdx = job.Batches[batchIdx].Steps.FindIndex(x => x.Id == entry.Key);
							updates.Add(updateBuilder.AddToSetEach(x => x.Batches[batchIdx].Steps[stepIdx].Metadata, entry.Value));
						}
					}
				}

				job = await TryUpdateAsync(job, updates, cancellationToken);

				return job == null ? null : await CreateJobObjectAsync(job, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to update job {JobId} meta data", jobId);
				return null;
			}
		}

		/// <inheritdoc/>
		public async Task UpgradeDocumentsAsync()
		{
			IAsyncCursor<JobDocument> cursor = await _jobs.Find(Builders<JobDocument>.Filter.Eq(x => x.UpdateTimeUtc, null)).ToCursorAsync();

			int numUpdated = 0;
			while (await cursor.MoveNextAsync())
			{
				foreach (JobDocument document in cursor.Current)
				{
					UpdateDefinition<JobDocument> update = Builders<JobDocument>.Update.Set(x => x.CreateTimeUtc, ((IJob)document).CreateTimeUtc).Set(x => x.UpdateTimeUtc, ((IJob)document).UpdateTimeUtc);
					await _jobs.UpdateOneAsync(Builders<JobDocument>.Filter.Eq(x => x.Id, document.Id), update);
					numUpdated++;
				}
				_logger.LogInformation("Updated {NumDocuments} documents", numUpdated);
			}
		}
	}
}
