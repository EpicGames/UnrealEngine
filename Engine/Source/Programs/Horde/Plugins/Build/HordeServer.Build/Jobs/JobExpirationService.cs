// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Core;
using EpicGames.Horde.Jobs;
using HordeServer.Streams;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Expires jobs a certain amount of time after they have run
	/// </summary>
	public sealed class JobExpirationService : IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Maximum duration a step can run before being timed out
		/// </summary>
		public static readonly TimeSpan StepTimeout = TimeSpan.FromHours(24.0);

		readonly IJobCollection _jobCollection;
		readonly IClock _clock;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITicker _ticker;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobExpirationService(IJobCollection jobCollection, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<JobExpirationService> logger)
		{
			_clock = clock;
			_buildConfig = buildConfig;
			_jobCollection = jobCollection;
			_ticker = clock.AddSharedTicker<JobExpirationService>(TimeSpan.FromHours(1.0), TickAsync, logger);
			_logger = logger;
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync() => _ticker.DisposeAsync();

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			DateTime currentTime = _clock.UtcNow;

			// Timeout stuck running steps
			// await TimeoutStuckStepsAsync(currentTime, cancellationToken);

			// Expire old jobs
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			foreach (StreamConfig streamConfig in buildConfig.Streams)
			{
				// Check if job expiration is explicitly disabled for this stream
				if (streamConfig.RetentionOptions.EnableJobExpiration == false)
				{
					continue;
				}

				int expireAfterDays = streamConfig.JobOptions.ExpireAfterDays ?? 0;
				if (expireAfterDays != 0)
				{
					int maxDeletions = streamConfig.RetentionOptions.MaxJobDeletionsPerTick ?? 1000;
					int deletionCount = 0; // Counts all attempts (including failures) to prevent runaway loops on undeletable jobs
					int consecutiveFailures = 0;
					bool reachedLimit = false;

					DateTimeOffset maxCreateTime = new DateTimeOffset(currentTime) - TimeSpan.FromDays(expireAfterDays);
					for (; ; )
					{
						IReadOnlyList<IJob> jobs = await _jobCollection.FindAsync(new FindJobOptions(StreamId: streamConfig.Id, MaxCreateTime: maxCreateTime), count: 50, cancellationToken: cancellationToken);
						if (jobs.Count == 0)
						{
							break;
						}

						foreach (IJob job in jobs)
						{
							if (maxDeletions > 0 && deletionCount >= maxDeletions)
							{
								_logger.LogInformation(
									"Reached max deletions per tick ({Max}) for stream {StreamId}",
									maxDeletions, streamConfig.Id);
								reachedLimit = true;
								break;
							}

							if (await job.TryDeleteAsync(cancellationToken))
							{
								int age = (int)(currentTime - job.CreateTimeUtc).TotalDays;
								_logger.LogDebug("Deleted job {JobId} ({NumDays}d >= {ExpireDays}d)", job.Id, age, expireAfterDays);
								consecutiveFailures = 0;
							}
							else
							{
								consecutiveFailures++;
								if (consecutiveFailures >= 100)
								{
									_logger.LogWarning("Aborting expiration for stream {StreamId}: {Count} consecutive delete failures", streamConfig.Id, consecutiveFailures);
									reachedLimit = true;
									break;
								}
							}
							deletionCount++;
						}

						if (reachedLimit)
						{
							break;
						}
					}
				}
			}
		}

		[SuppressMessage("CodeQuality", "IDE0051:Remove unused private members", Justification = "Stuck job steps disabled")]
		async Task TimeoutStuckStepsAsync(DateTime currentTime, CancellationToken cancellationToken)
		{
			DateTime maxStepStartTime = currentTime - StepTimeout;

			// Find jobs with running batches that might have stuck steps
			IReadOnlyList<IJob> jobs;
			try
			{
				jobs = await _jobCollection.FindAsync(
					new FindJobOptions(BatchState: JobStepBatchState.Running),
					count: 100,
					cancellationToken: cancellationToken);
			}
			catch (ArgumentOutOfRangeException ex)
			{
				_logger.LogWarning(ex, "Failed to find jobs with running batches, skipping stuck step timeout check");
				return;
			}

			foreach (IJob job in jobs)
			{
				foreach (IJobStepBatch batch in job.Batches)
				{
					if (batch.State != JobStepBatchState.Running)
					{
						continue;
					}

					foreach (IJobStep step in batch.Steps)
					{
						if (step.State == JobStepState.Running &&
							step.StartTimeUtc != null &&
							step.StartTimeUtc.Value < maxStepStartTime &&
							!step.AbortRequested)
						{
							double runningHours = (currentTime - step.StartTimeUtc.Value).TotalHours;
							_logger.LogWarning("Timing out stuck step {JobId}:{BatchId}:{StepId} (running for {Hours:F1}h)",
								job.Id, batch.Id, step.Id, runningHours);

							await job.TryUpdateStepAsync(batch.Id, step.Id,
								newAbortRequested: true,
								newError: JobStepError.TimedOut,
								cancellationToken: cancellationToken);
						}
					}
				}
			}
		}
	}
}
