// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using HordeServer.Utilities;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Utility methods for <see cref="JobId"/>
	/// </summary>
	public static class JobIdUtils
	{
		/// <summary>
		/// Creates a new, random JobId
		/// </summary>
		public static JobId GenerateNewId()
		{
			return new JobId(BinaryIdUtils.CreateNew());
		}
	}
}
