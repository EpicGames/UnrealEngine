// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using Google.Protobuf.WellKnownTypes;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Artifacts;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Tasks;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Implements the Horde gRPC service for bots updating their status and dequeing work
	/// </summary>
	[Authorize]
	public class JobRpcService : JobRpc.JobRpcBase
	{
		readonly IAclService _aclService;
		readonly JobService _jobService;
		readonly AgentService _agentService;
		readonly PoolService _poolService;
		readonly ConformTaskSource _conformTaskSource;
		readonly IArtifactCollection _artifactCollection;
		readonly IJobCollection _jobCollection;
		readonly ILogCollection _logCollection;
		readonly IGraphCollection _graphs;
		readonly ITestDataCollection _testData;
		readonly ITestDataCollectionV2 _testDataV2;
		readonly IJobStepRefCollection _jobStepRefCollection;
		readonly ITemplateCollection _templateCollection;
		readonly HttpClient _httpClient;
		readonly IClock _clock;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobRpcService(IAclService aclService, JobService jobService, AgentService agentService, PoolService poolService, ConformTaskSource conformTaskSource, IArtifactCollection artifactCollection, IJobCollection jobCollection, ILogCollection logCollection, IGraphCollection graphs, ITestDataCollection testData, ITestDataCollectionV2 testDataV2, IJobStepRefCollection jobStepRefCollection, ITemplateCollection templateCollection, HttpClient httpClient, IClock clock, IOptionsSnapshot<BuildConfig> buildConfig, ILogger<JobRpcService> logger)
		{
			_aclService = aclService;
			_jobService = jobService;
			_agentService = agentService;
			_poolService = poolService;
			_conformTaskSource = conformTaskSource;
			_jobCollection = jobCollection;
			_artifactCollection = artifactCollection;
			_logCollection = logCollection;
			_graphs = graphs;
			_testData = testData;
			_testDataV2 = testDataV2;
			_jobStepRefCollection = jobStepRefCollection;
			_templateCollection = templateCollection;
			_httpClient = httpClient;
			_clock = clock;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public override async Task<RpcUpdateAgentWorkspacesResponse> UpdateAgentWorkspaces(RpcUpdateAgentWorkspacesRequest request, ServerCallContext context)
		{
			for (; ; )
			{
				// Get the current agent state
				IAgent? agent = await _agentService.GetAgentAsync(new AgentId(request.AgentId));
				if (agent == null)
				{
					throw new StructuredRpcException(StatusCode.OutOfRange, "Agent {AgentId} does not exist", request.AgentId);
				}

				// Get the new workspaces
				List<AgentWorkspaceInfo> currentWorkspaces = request.Workspaces.Select(x => new AgentWorkspaceInfo(x)).ToList();

				// Get the set of workspaces that are currently required
				HashSet<AgentWorkspaceInfo> conformWorkspaces = await _poolService.GetWorkspacesAsync(agent, DateTime.UtcNow, _buildConfig.Value, context.CancellationToken);
				bool workspaceSetEquals = conformWorkspaces.SetEquals(currentWorkspaces);
				bool pendingConform = !workspaceSetEquals || (agent.RequestFullConform && !request.RemoveUntrackedFiles);

				// Update the workspaces
				if (await _agentService.TryUpdateWorkspacesAsync(agent, currentWorkspaces, pendingConform, context.CancellationToken))
				{
					RpcUpdateAgentWorkspacesResponse response = new RpcUpdateAgentWorkspacesResponse();
					if (pendingConform)
					{
						response.Retry = await _conformTaskSource.GetWorkspacesAsync(agent, response.PendingWorkspaces, context.CancellationToken);
						response.RemoveUntrackedFiles = request.RemoveUntrackedFiles || agent.RequestFullConform;

						if (response.Retry)
						{
							HashSet<string> identifiers = new HashSet<string>();
							identifiers.UnionWith(currentWorkspaces.Select(x => x.Identifier));
							identifiers.UnionWith(conformWorkspaces.Select(x => x.Identifier));

							int numChanged = 0;
							foreach (string identifier in identifiers)
							{
								AgentWorkspaceInfo? prevWorkspace = currentWorkspaces.FirstOrDefault(x => x.Identifier.Equals(identifier, StringComparison.OrdinalIgnoreCase));
								AgentWorkspaceInfo? nextWorkspace = conformWorkspaces.FirstOrDefault(x => x.Identifier.Equals(identifier, StringComparison.OrdinalIgnoreCase));
								string prevWorkspaceText = (prevWorkspace == null) ? "(does not exist)" : JsonSerializer.Serialize(prevWorkspace);
								string nextWorkspaceText = (nextWorkspace == null) ? "(does not exist)" : JsonSerializer.Serialize(nextWorkspace);

								if (!String.Equals(prevWorkspaceText, nextWorkspaceText, StringComparison.Ordinal))
								{
									_logger.LogInformation("{AgentId} {Identifier} was: {OldWorkspace}", agent.Id, identifier, prevWorkspaceText);
									_logger.LogInformation("{AgentId} {Identifier} now: {NewWorkspace}", agent.Id, identifier, nextWorkspaceText);
									numChanged++;
								}
							}

							_logger.LogInformation("Retrying conform for {AgentId} ({NumChanged} changed, set equals: {SetEquals}, request conform: {RequestConform}, request full conform: {RequestFullConform}, remove untracked: {RemoveUntracked})", agent.Id, numChanged, workspaceSetEquals, agent.RequestConform, agent.RequestFullConform, request.RemoveUntrackedFiles);
						}
					}
					return response;
				}
			}
		}

		/// <summary>
		/// Gets information about a job
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcGetJobResponse> GetJob(RpcGetJobRequest request, ServerCallContext context)
		{
			JobId jobIdValue = JobId.Parse(request.JobId);

			IJob? job = await _jobService.GetJobAsync(jobIdValue);
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} does not exist", request.JobId);
			}
			if (!JobService.AuthorizeSession(job, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Not authenticated to access job {JobId}", request.JobId);
			}

			RpcGetJobResponse response = new RpcGetJobResponse();
			response.StreamId = job.StreamId.ToString();
			response.Change = job.CommitId.TryGetPerforceChange() ?? 0;
			response.CodeChange = job.CodeCommitId?.TryGetPerforceChange() ?? 0;
			response.PreflightChange = job.PreflightCommitId?.TryGetPerforceChange() ?? 0;
			response.CommitId = job.CommitId.ToString();
			if (job.PreflightCommitId != null)
			{
				response.PreflightCommitId = job.PreflightCommitId.ToString();
			}
			response.Arguments.Add(job.Arguments);
			return response;
		}

		/// <summary>
		/// Updates properties on a job
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<Empty> UpdateJob(RpcUpdateJobRequest request, ServerCallContext context)
		{
			JobId jobIdValue = JobId.Parse(request.JobId);

			IJob? job = await _jobService.GetJobAsync(jobIdValue);
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} does not exist", request.JobId);
			}
			if (!JobService.AuthorizeSession(job, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Not authenticated to modify job {JobId}", request.JobId);
			}

			await _jobService.UpdateJobAsync(job, name: request.Name);
			return new Empty();
		}

		/// <summary>
		/// Starts executing a batch
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcBeginBatchResponse> BeginBatch(RpcBeginBatchRequest request, ServerCallContext context)
		{
			JobStepBatchId batchId = JobStepBatchId.Parse(request.BatchId);

			IJob? job = await _jobService.GetJobAsync(JobId.Parse(request.JobId));
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} not found", request.JobId);
			}
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Stream {StreamId} not found", job.StreamId);
			}

			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);
			job = await _jobService.UpdateBatchAsync(job, batchId, streamConfig, newState: JobStepBatchState.Starting);

			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Batch {JobId}:{BatchId} not found for updating", request.JobId, request.BatchId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job);
			AgentConfig agentConfig = streamConfig.AgentTypes[graph.Groups[batch.GroupIdx].AgentType];

			RpcBeginBatchResponse response = new RpcBeginBatchResponse();
			response.LogId = batch.LogId.ToString();
			response.AgentType = graph.Groups[batch.GroupIdx].AgentType;
			response.StreamId = streamConfig.Id.ToString();
			response.StreamName = streamConfig.Name;
			response.EnginePath = streamConfig.EnginePath;
			response.Change = job.CommitId.TryGetPerforceChange() ?? 0;
			response.CodeChange = job.CodeCommitId?.TryGetPerforceChange() ?? 0;
			response.PreflightChange = job.PreflightCommitId?.TryGetPerforceChange() ?? 0;
			response.CommitId = job.CommitId.ToString();
			if (job.PreflightCommitId != null)
			{
				response.PreflightCommitId = job.PreflightCommitId.ToString();
			}
			response.Arguments.AddRange(job.Arguments);
			if (agentConfig.TempStorageDir != null)
			{
				response.TempStorageDir = agentConfig.TempStorageDir;
			}
			if (agentConfig.Environment != null)
			{
				response.Environment.Add(agentConfig.Environment);
			}
			response.ValidAgentTypes.Add(streamConfig.AgentTypes.Keys);

			bool expandedIgnorePatterns = _buildConfig.Value.EnableExpandedIgnorePatterns;
			response.EnableExpandedIgnorePatterns = expandedIgnorePatterns;
			if (expandedIgnorePatterns && streamConfig.IgnorePatternFiles != null)
			{
				response.IgnorePatternFiles.AddRange(streamConfig.IgnorePatternFiles);
			}

			return response;
		}

		/// <summary>
		/// Finishes executing a batch
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<Empty> FinishBatch(RpcFinishBatchRequest request, ServerCallContext context)
		{
			IJob? job = await _jobService.GetJobAsync(JobId.Parse(request.JobId));
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} not found", request.JobId);
			}
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Stream {StreamId} not found", job.StreamId);
			}

			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);
			await _jobService.UpdateBatchAsync(job, batch.Id, streamConfig, newState: JobStepBatchState.Complete);
			return new Empty();
		}

		/// <summary>
		/// Starts executing a step
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcBeginStepResponse> BeginStep(RpcBeginStepRequest request, ServerCallContext context)
		{
			Boxed<ILog?> log = new Boxed<ILog?>(null);
			for (; ; )
			{
				RpcBeginStepResponse? response = await TryBeginStepAsync(request, log, context);
				if (response != null)
				{
					return response;
				}
			}
		}

		async Task<RpcBeginStepResponse?> TryBeginStepAsync(RpcBeginStepRequest request, Boxed<ILog?> log, ServerCallContext context)
		{
			// Check the job exists and we can access it
			IJob? job = await _jobService.GetJobAsync(JobId.Parse(request.JobId));
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} not found", request.JobId);
			}
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Stream {StreamId} not found", job.StreamId);
			}

			// Find the batch being executed
			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);
			if (batch.State != JobStepBatchState.Starting && batch.State != JobStepBatchState.Running)
			{
				return new RpcBeginStepResponse { State = RpcBeginStepResponse.Types.Result.Complete };
			}

			// Figure out which step to execute next
			IJobStep? step;
			for (int stepIdx = 0; ; stepIdx++)
			{
				// If there aren't any more steps, send a complete message
				if (stepIdx == batch.Steps.Count)
				{
					_logger.LogDebug("Job {JobId} batch {BatchId} is complete", job.Id, batch.Id);
					if (await _jobService.TryUpdateBatchAsync(job, batch.Id, newState: JobStepBatchState.Stopping) == null)
					{
						return null;
					}
					return new RpcBeginStepResponse { State = RpcBeginStepResponse.Types.Result.Complete };
				}

				// Check if this step is ready to be executed
				step = batch.Steps[stepIdx];
				if (step.State == JobStepState.Ready)
				{
					break;
				}
				if (step.State == JobStepState.Waiting)
				{
					_logger.LogDebug("Waiting for job {JobId}, batch {BatchId}, step {StepId}", job.Id, batch.Id, step.Id);
					return new RpcBeginStepResponse { State = RpcBeginStepResponse.Types.Result.Waiting };
				}
			}

			// Create a log file if necessary
			log.Value ??= await _logCollection.AddAsync(job.Id, batch.LeaseId, batch.SessionId, LogType.Json, aclScopeName: streamConfig.Acl.ScopeName);

			// Get the node for this step
			IGraph graph = await _jobService.GetGraphAsync(job);
			INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];

			// Figure out all the credentials for it (and check we can access them)
			Dictionary<string, string> credentials = new Dictionary<string, string>();
			//				if (Node.Credentials != null)
			//				{
			//					ClaimsPrincipal Principal = new ClaimsPrincipal(new ClaimsIdentity(Job.Claims.Select(x => new Claim(x.Type, x.Value))));
			//					if (!await GetCredentialsForStep(Principal, Node, Credentials, Message => FailStep(Job, Batch.Id, Step.Id, Log, Message)))
			//					{
			//						Log = null;
			//						continue;
			//					}
			//				}

			// Update the step state
			IJob? newJob = await _jobService.TryUpdateStepAsync(job, batch.Id, step.Id, streamConfig, JobStepState.Running, JobStepOutcome.Unspecified, newLogId: log.Value.Id);
			if (newJob != null)
			{
				RpcBeginStepResponse response = new RpcBeginStepResponse();
				response.State = RpcBeginStepResponse.Types.Result.Ready;
				response.LogId = log.Value.Id.ToString();
				response.StepId = step.Id.ToString();
				response.Name = node.Name;
				response.Credentials.Add(credentials);

				foreach (NodeOutputRef input in node.Inputs)
				{
					INode inputNode = graph.GetNode(input.NodeRef);
					response.Inputs.Add($"{inputNode.Name}/{inputNode.OutputNames[input.OutputIdx]}");
				}

				foreach (NodeOutputRef optionalInput in node.OptionalInputs)
				{
					INode optionalInputNode = graph.GetNode(optionalInput.NodeRef);
					response.OptionalInputs.Add($"{optionalInputNode.Name}/{optionalInputNode.OutputNames[optionalInput.OutputIdx]}");
				}

				response.OutputNames.Add(node.OutputNames);

				foreach (INodeGroup otherGroup in graph.Groups)
				{
					if (otherGroup != graph.Groups[batch.GroupIdx])
					{
						//						AnomalyDetectorConfiguration INPUTDEPENDENCIES
						foreach (NodeOutputRef outputRef in otherGroup.Nodes.SelectMany(x => x.Inputs))
						{
							if (outputRef.NodeRef.GroupIdx == batch.GroupIdx && outputRef.NodeRef.NodeIdx == step.NodeIdx && !response.PublishOutputs.Contains(outputRef.OutputIdx))
							{
								response.PublishOutputs.Add(outputRef.OutputIdx);
							}
						}
						foreach (NodeOutputRef outputRef in otherGroup.Nodes.SelectMany(x => x.OptionalInputs))
						{
							if (outputRef.NodeRef.GroupIdx == batch.GroupIdx && outputRef.NodeRef.NodeIdx == step.NodeIdx && !response.PublishOutputs.Contains(outputRef.OutputIdx))
							{
								response.PublishOutputs.Add(outputRef.OutputIdx);
							}
						}
					}
				}

				string templateName = "<unknown>";
				if (job.TemplateHash != null)
				{
					ITemplate? template = await _templateCollection.GetAsync(job.TemplateHash);
					templateName = template != null ? template.Name : templateName;
				}

				response.EnvVars.Add("UE_HORDE_TEMPLATEID", job.TemplateId.ToString());
				response.EnvVars.Add("UE_HORDE_TEMPLATENAME", templateName);
				response.EnvVars.Add("UE_HORDE_STEPNAME", node.Name);

				foreach (StreamTag tag in streamConfig.StreamTags)
				{
					response.EnvVars.Add($"UE_HORDE_STREAMTAG_{tag.Name.ToUpperInvariant()}", tag.Enabled.ToString());
				}

				if (job.Environment != null)
				{
					response.EnvVars.Add(job.Environment);
				}

				IJobStepRef? lastStep = await _jobStepRefCollection.GetPrevStepForNodeAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId);
				if (lastStep != null)
				{
					response.EnvVars.Add("UE_HORDE_LAST_CL", lastStep.CommitId.ToString());

					CommitIdWithOrder? lastSuccessCommitId = null;
					if (lastStep.Outcome == JobStepOutcome.Success)
					{
						lastSuccessCommitId = lastStep.CommitId;
					}
					else if (lastStep.LastSuccess != null)
					{
						lastSuccessCommitId = lastStep.LastSuccess;
					}
					else
					{
						// Previous job hasn't finished yet; need to search for *current* last success step
						IJobStepRef? lastSuccess = await _jobStepRefCollection.GetPrevStepForNodeAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, outcome: JobStepOutcome.Success);
						if (lastSuccess != null)
						{
							lastSuccessCommitId = lastSuccess.CommitId;
						}
					}

					CommitIdWithOrder? lastWarningCommitId = null;
					if (lastStep.Outcome == JobStepOutcome.Success || lastStep.Outcome == JobStepOutcome.Warnings)
					{
						lastWarningCommitId = lastStep.CommitId;
					}
					else if (lastStep.LastWarning != null)
					{
						lastWarningCommitId = lastStep.LastWarning;
					}
					else
					{
						// Previous job hasn't finished yet; need to search for *current* last warning step
						IJobStepRef? lastWarnings = await _jobStepRefCollection.GetPrevStepForNodeAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, outcome: JobStepOutcome.Warnings);
						if (lastWarnings != null && (lastSuccessCommitId == null || lastWarnings.CommitId > lastSuccessCommitId))
						{
							lastWarningCommitId = lastWarnings.CommitId;
						}
						else
						{
							lastWarningCommitId = lastSuccessCommitId;
						}
					}

					if (lastSuccessCommitId != null)
					{
						response.EnvVars.Add("UE_HORDE_LAST_SUCCESS_CL", lastSuccessCommitId.ToString());
					}
					if (lastWarningCommitId != null)
					{
						response.EnvVars.Add("UE_HORDE_LAST_WARNING_CL", lastWarningCommitId.ToString());
					}
				}

				List<TokenConfig> tokenConfigs = new List<TokenConfig>(streamConfig.Tokens);

				INodeGroup group = graph.Groups[batch.GroupIdx];
				if (streamConfig.AgentTypes.TryGetValue(group.AgentType, out AgentConfig? agentConfig) && agentConfig.Tokens != null)
				{
					tokenConfigs.AddRange(agentConfig.Tokens);
				}

				foreach (TokenConfig tokenConfig in tokenConfigs)
				{
					string? value = await AllocateTokenAsync(tokenConfig, context.CancellationToken);
					if (value == null)
					{
						_logger.LogWarning("Unable to allocate token for job {JobId}:{BatchId}:{StepId} from {Url}", job.Id, batch.Id, step.Id, tokenConfig.Url);
					}
					else
					{
						response.Credentials.Add(tokenConfig.EnvVar, value);
					}
				}

				if (node.Properties != null)
				{
					response.Properties.Add(node.Properties);
				}
				response.Warnings = node.Warnings;

				foreach (IGraphArtifact artifact in graph.Artifacts)
				{
					if (artifact.OutputName != null && node.OutputNames.Contains(artifact.OutputName))
					{
						RpcCreateGraphArtifactRequest stepArtifact = new RpcCreateGraphArtifactRequest { Name = artifact.Name.ToString(), Type = artifact.Type.ToString(), Description = artifact.Description, BasePath = artifact.BasePath, OutputName = artifact.OutputName };
						stepArtifact.Keys.AddRange(artifact.Keys);
						stepArtifact.Metadata.AddRange(artifact.Metadata);
						response.Artifacts.Add(stepArtifact);
					}
				}

				return response;
			}

			return null;
		}

		class GetTokenResponse
		{
			[JsonPropertyName("token_type")]
			public string TokenType { get; set; } = String.Empty;

			[JsonPropertyName("access_token")]
			public string AccessToken { get; set; } = String.Empty;
		}

		async Task<string?> AllocateTokenAsync(TokenConfig config, CancellationToken cancellationToken)
		{
			List<KeyValuePair<string, string>> content = new List<KeyValuePair<string, string>>();
			content.Add(KeyValuePair.Create("grant_type", "client_credentials"));
			content.Add(KeyValuePair.Create("scope", "cache_access"));
			content.Add(KeyValuePair.Create("client_id", config.ClientId));
			content.Add(KeyValuePair.Create("client_secret", config.ClientSecret));

			using (HttpRequestMessage request = new HttpRequestMessage(HttpMethod.Post, config.Url))
			{
				request.Content = new FormUrlEncodedContent(content);
				using (HttpResponseMessage response = await _httpClient.SendAsync(request, cancellationToken))
				{
					string text = await response.Content.ReadAsStringAsync(cancellationToken);
					if (!response.IsSuccessStatusCode)
					{
						throw new Exception($"Unexpected response while allocating token from {config.Url}: {text}");
					}

					try
					{
						return JsonSerializer.Deserialize<GetTokenResponse>(text)?.AccessToken;
					}
					catch (Exception ex)
					{
						throw new Exception($"Error allocating token from {config.Url}", ex);
					}
				}
			}
		}

		async Task<IJob> GetJobAsync(JobId jobId, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} not found", jobId);
			}
			return job;
		}

		static IJobStepBatch AuthorizeBatch(IJob job, JobStepBatchId batchId, ServerCallContext context)
		{
			IJobStepBatch? batch;
			if (!job.TryGetBatch(batchId, out batch))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to find batch {JobId}:{BatchId}", job.Id, batchId);
			}
			if (batch.SessionId == null)
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Batch {JobId}:{BatchId} has no session id", job.Id, batchId);
			}
			if (batch.LeaseId == null)
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Batch {JobId}:{BatchId} has no lease id", job.Id, batchId);
			}

			ClaimsPrincipal principal = context.GetHttpContext().User;
			if (!principal.HasSessionClaim(batch.SessionId.Value) && !principal.HasLeaseClaim(batch.LeaseId.Value))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Session id {SessionId} not valid for batch {JobId}:{BatchId}. Expected {ExpectedSessionId}.", principal.GetSessionClaim() ?? SessionId.Empty, job.Id, batchId, batch.SessionId.Value);
			}

			return batch;
		}

		/// <summary>
		/// Updates the state of a jobstep
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<Empty> UpdateStep(RpcUpdateStepRequest request, ServerCallContext context)
		{
			IJob? job = await _jobService.GetJobAsync(JobId.Parse(request.JobId));
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Job {JobId} not found", request.JobId);
			}
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Stream {StreamId} not found", job.StreamId);
			}

			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);

			JobStepId stepId = JobStepId.Parse(request.StepId);
			if (!batch.TryGetStep(stepId, out IJobStep? step))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to find step {JobId}:{BatchId}:{StepId}", job.Id, batch.Id, stepId);
			}

			JobStepError? error = null;
			if (step.Error == JobStepError.None && step.HasTimedOut(_clock.UtcNow))
			{
				error = JobStepError.TimedOut;
			}

			await _jobService.UpdateStepAsync(job, batch.Id, JobStepId.Parse(request.StepId), streamConfig, (JobStepState)request.State, (JobStepOutcome)request.Outcome, error, null, null, null, null, null);
			return new Empty();
		}

		/// <summary>
		/// Get the state of a jobstep
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the step</returns>
		public override async Task<RpcGetStepResponse> GetStep(RpcGetStepRequest request, ServerCallContext context)
		{
			IJob job = await GetJobAsync(JobId.Parse(request.JobId), context.CancellationToken);
			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);

			JobStepId stepId = JobStepId.Parse(request.StepId);
			if (!batch.TryGetStep(stepId, out IJobStep? step))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to find step {JobId}:{BatchId}:{StepId}", job.Id, batch.Id, stepId);
			}

			return new RpcGetStepResponse { Outcome = (int)step.Outcome, State = (int)step.State, AbortRequested = step.AbortRequested || step.HasTimedOut(_clock.UtcNow) };
		}

		/// <summary>
		/// Updates the state of a jobstep
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcUpdateGraphResponse> UpdateGraph(RpcUpdateGraphRequest request, ServerCallContext context)
		{
			JobId jobIdValue = JobId.Parse(request.JobId);
			for (; ; )
			{
				IJob? job = await _jobService.GetJobAsync(jobIdValue);
				if (job == null)
				{
					throw new StructuredRpcException(StatusCode.NotFound, "Resource not found");
				}
				if (!JobService.AuthorizeSession(job, context.GetHttpContext().User))
				{
					throw new StructuredRpcException(StatusCode.PermissionDenied, "Access denied");
				}

				// Get the graph state
				IGraph oldGraph = await _graphs.GetAsync(job.GraphHash);

				// Add all the new nodes
				List<NewGroup> newGroups = new List<NewGroup> { new NewGroup(oldGraph, oldGraph.Groups[0]) };
				foreach (RpcCreateGroupRequest group in request.Groups)
				{
					List<NewNode> newNodes = new List<NewNode>();
					foreach (RpcCreateNodeRequest node in group.Nodes)
					{
						NewNode newNode = new NewNode(node.Name, node.Inputs.ToList(), node.OptionalInputs.ToList(), node.Outputs.ToList(), node.InputDependencies.ToList(), node.OptionalInputDependencies.ToList(), node.OrderDependencies.ToList(), (Priority)node.Priority, node.AllowRetry, node.RunEarly, node.Warnings, new Dictionary<string, string>(node.Credentials), new Dictionary<string, string>(node.Properties), new NodeAnnotations(node.Annotations));
						newNodes.Add(newNode);
					}
					newGroups.Add(new NewGroup(group.AgentType, newNodes));
				}

				// Add all the new aggregates
				List<NewAggregate> newAggregates = new List<NewAggregate>();
				foreach (RpcCreateAggregateRequest aggregate in request.Aggregates)
				{
					NewAggregate newAggregate = new NewAggregate(aggregate.Name, aggregate.Nodes.ToList());
					newAggregates.Add(newAggregate);
				}

				// Add all the new labels
				List<NewLabel> newLabels = new List<NewLabel>();
				foreach (RpcCreateLabelRequest label in request.Labels)
				{
					NewLabel newLabel = new NewLabel();
					newLabel.DashboardName = String.IsNullOrEmpty(label.DashboardName) ? null : label.DashboardName;
					newLabel.DashboardCategory = String.IsNullOrEmpty(label.DashboardCategory) ? null : label.DashboardCategory;
					newLabel.UgsName = String.IsNullOrEmpty(label.UgsName) ? null : label.UgsName;
					newLabel.UgsProject = String.IsNullOrEmpty(label.UgsProject) ? null : label.UgsProject;
					newLabel.Change = (LabelChange)label.Change;
					newLabel.RequiredNodes = label.RequiredNodes.ToList();
					newLabel.IncludedNodes = label.IncludedNodes.ToList();
					newLabels.Add(newLabel);
				}

				// Add all the new artifacts
				List<NewGraphArtifact> newArtifacts = new List<NewGraphArtifact>();
				foreach (RpcCreateGraphArtifactRequest artifact in request.Artifacts)
				{
					ArtifactName name = new ArtifactName(StringId.Sanitize(artifact.Name));
					ArtifactType type = new ArtifactType(StringId.Sanitize(artifact.Type));

					string description = artifact.Description;
					if (String.IsNullOrEmpty(description))
					{
						description = artifact.Name;
					}

					string? nodeName = null;
					if (!String.IsNullOrEmpty(artifact.NodeName))
					{
						nodeName = artifact.NodeName;
					}

					string? outputName = null;
					if (!String.IsNullOrEmpty(artifact.OutputName))
					{
						outputName = artifact.OutputName;
					}

					newArtifacts.Add(new NewGraphArtifact(name, type, description, artifact.BasePath, artifact.Keys.ToList(), artifact.Metadata.ToList(), nodeName, outputName));
				}

				// Create the new graph
				IGraph newGraph = await _graphs.AppendAsync(null, newGroups, newAggregates, newLabels, newArtifacts);

				// Try to update the graph with the new value
				IJob? newJob = await _jobService.TryUpdateGraphAsync(job, newGraph, context.CancellationToken);
				if (newJob != null)
				{
					_logger.LogInformation("Updating graph for {JobId} from {OldGraphHash} to {NewGraphHash}", job.Id, oldGraph.Id, newJob.GraphHash);
					return new RpcUpdateGraphResponse();
				}
			}
		}

		/// <summary>
		/// Creates a set of events
		/// </summary>
		/// <param name="request">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		[Obsolete("Use LogRpc.CreateLogEvents instead")]
		public override async Task<Empty> CreateEvents(RpcCreateLogEventsRequest request, ServerCallContext context)
		{
			if (!_buildConfig.Value.Authorize(LogAclAction.CreateEvent, context.GetHttpContext().User))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Access denied");
			}

			foreach (IGrouping<string, RpcCreateLogEventRequest> createEventGroup in request.Events.GroupBy(x => x.LogId))
			{
				ILog? log = await _logCollection.GetAsync(LogId.Parse(createEventGroup.Key), context.CancellationToken);
				if (log == null)
				{
					throw new StructuredRpcException(StatusCode.NotFound, "Log not found");
				}

				List<NewLogEventData> newEvents = new List<NewLogEventData>();
				foreach (RpcCreateLogEventRequest createEvent in createEventGroup)
				{
					NewLogEventData newEvent = new NewLogEventData();
					newEvent.Severity = (LogEventSeverity)createEvent.Severity;
					newEvent.LineIndex = createEvent.LineIndex;
					newEvent.LineCount = createEvent.LineCount;
					newEvents.Add(newEvent);
				}

				await log.AddEventsAsync(newEvents, context.CancellationToken);
			}

			return new Empty();
		}

		/// <summary>
		/// Uploads new test data
		/// </summary>
		/// <param name="reader">Request arguments</param>
		/// <param name="context">Context for the RPC call</param>
		/// <returns>Information about the new agent</returns>
		public override async Task<RpcUploadTestDataResponse> UploadTestData(IAsyncStreamReader<RpcUploadTestDataRequest> reader, ServerCallContext context)
		{
			IJob? job = null;
			IJobStep? jobStep = null;

			List<(string key, BsonDocument document)> data = new List<(string key, BsonDocument document)>();

			while (await reader.MoveNext())
			{
				RpcUploadTestDataRequest request = reader.Current;

				JobId jobId = JobId.Parse(request.JobId);
				if (job == null)
				{
					job = await _jobService.GetJobAsync(jobId, context.CancellationToken);
					if (job == null)
					{
						throw new StructuredRpcException(StatusCode.NotFound, "Unable to find job {JobId}", jobId);
					}
				}
				else if (jobId != job.Id)
				{
					throw new StructuredRpcException(StatusCode.InvalidArgument, "Job {JobId} does not match previous Job {JobId} in request", jobId, job.Id);
				}

				JobStepId jobStepId = JobStepId.Parse(request.JobStepId);

				if (jobStep == null)
				{
					if (!job.TryGetStep(jobStepId, out jobStep))
					{
						throw new StructuredRpcException(StatusCode.NotFound, "Unable to find step {JobStepId} on job {JobId}", jobStepId, jobId);
					}
				}
				else if (jobStep.Id != jobStepId)
				{
					throw new StructuredRpcException(StatusCode.InvalidArgument, "Job step {JobStepId} does not match previous Job step {JobStepId} in request", jobStepId, jobStep.Id);
				}

				string text = Encoding.UTF8.GetString(request.Value.ToArray());
				BsonDocument document = BsonSerializer.Deserialize<BsonDocument>(text);
				data.Add((request.Key, document));
			}

			if (job != null && jobStep != null)
			{
				await _testData.AddAsync(job, jobStep, data.ToArray(), context.CancellationToken);
				await _testDataV2.AddAsync(job, jobStep, data.ToArray(), context.CancellationToken);
			}
			else
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to get job or step for test data upload");
			}

			return new RpcUploadTestDataResponse();
		}

		/// <inheritdoc/>
		public override async Task<RpcCreateJobArtifactResponseV2> CreateArtifactV2(RpcCreateJobArtifactRequestV2 request, ServerCallContext context)
		{
			(IJob job, _, IJobStep step) = await AuthorizeAsync(request.JobId, request.StepId, context);

			ArtifactName name = new ArtifactName(request.Name);
			ArtifactType type = new ArtifactType(request.Type);

			List<string> keys = new List<string>();
			keys.Add(job.GetArtifactKey());
			keys.Add(job.GetArtifactKey(step));
			keys.AddRange(request.Keys);

			string? description = request.Description;
			if (String.IsNullOrEmpty(description))
			{
				description = request.Name;
			}

			IArtifactBuilder artifact = await _artifactCollection.CreateAsync(name, type, description, job.StreamId, job.CommitId, keys, request.Metadata, context.CancellationToken);

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(HordeClaimTypes.WriteNamespace, $"{artifact.NamespaceId}:{artifact.RefName}"));

			string token = await _aclService.IssueBearerTokenAsync(claims, TimeSpan.FromHours(8.0), context.CancellationToken);

			RpcCreateJobArtifactResponseV2 response = new RpcCreateJobArtifactResponseV2();
			response.Id = artifact.Id.ToString();
			response.NamespaceId = artifact.NamespaceId.ToString();
			response.RefName = artifact.RefName.ToString();
			response.Token = token;
			return response;
		}

		/// <inheritdoc/>
		public override async Task<RpcGetJobArtifactResponse> GetArtifact(RpcGetJobArtifactRequest request, ServerCallContext context)
		{
			(IJob job, _, _) = await AuthorizeAsync(request.JobId, request.StepId, context);

			ArtifactName name = new ArtifactName(request.Name);
			ArtifactType type = new ArtifactType(request.Type);

			IArtifact? artifact = await _artifactCollection.FindAsync(name: name, type: type, keys: new[] { job.GetArtifactKey() }, cancellationToken: context.CancellationToken).FirstOrDefaultAsync(context.CancellationToken);
			if (artifact == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "No artifact {ArtifactName} of type {ArtifactType} was found for job {JobId}", name, type, job.Id);
			}

			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(new AclClaimConfig(HordeClaimTypes.ReadNamespace, $"{artifact.NamespaceId}:{artifact.RefName}"));

			string token = await _aclService.IssueBearerTokenAsync(claims, TimeSpan.FromHours(8.0), context.CancellationToken);

			RpcGetJobArtifactResponse response = new RpcGetJobArtifactResponse();
			response.Id = artifact.Id.ToString();
			response.NamespaceId = artifact.NamespaceId.ToString();
			response.RefName = artifact.RefName.ToString();
			response.Token = token;
			return response;
		}

		Task<(IJob, IJobStepBatch, IJobStep)> AuthorizeAsync(string jobId, string stepId, ServerCallContext context)
		{
			return AuthorizeAsync(JobId.Parse(jobId), JobStepId.Parse(stepId), context);
		}

		async Task<(IJob, IJobStepBatch, IJobStep)> AuthorizeAsync(JobId jobId, JobStepId stepId, ServerCallContext context)
		{
			IJob? job = await _jobCollection.GetAsync(jobId);
			if (job == null)
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to find job {JobId}", jobId);
			}
			if (!job.TryGetStep(stepId, out IJobStep? step))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Unable to find step {JobId}:{StepId}", job.Id, stepId);
			}

			IJobStepBatch batch = step.Batch;
			if (batch.SessionId == null)
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Batch {JobId}:{BatchId} has no session id", job.Id, batch.Id);
			}
			if (batch.LeaseId == null)
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Batch {JobId}:{BatchId} has no lease id", job.Id, batch.Id);
			}

			ClaimsPrincipal principal = context.GetHttpContext().User;
			if (!principal.HasSessionClaim(batch.SessionId.Value) && !principal.HasLeaseClaim(batch.LeaseId.Value))
			{
				throw new StructuredRpcException(StatusCode.PermissionDenied, "Session id {SessionId} not valid for step {JobId}:{BatchId}:{StepId}. Expected {ExpectedSessionId}.", principal.GetSessionClaim() ?? SessionId.Empty, job.Id, batch.Id, step.Id, batch.SessionId.Value);
			}

			return (job, batch, step);
		}

		/// <summary>
		/// Create a new report on a job or job step
		/// </summary>
		/// <param name="request"></param>
		/// <param name="context"></param>
		/// <returns></returns>
		public override async Task<RpcCreateReportResponse> CreateReport(RpcCreateReportRequest request, ServerCallContext context)
		{
			IJob job = await GetJobAsync(JobId.Parse(request.JobId), context.CancellationToken);
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				throw new StructuredRpcException(StatusCode.NotFound, "Stream {StreamId} not found", job.StreamId);
			}

			IJobStepBatch batch = AuthorizeBatch(job, JobStepBatchId.Parse(request.BatchId), context);

			JobReport newReport = new JobReport { Name = request.Name, Placement = (JobReportPlacement)request.Placement, Content = request.Content, Message = request.Message, Severity = request.Severity };
			if (request.Scope == RpcReportScope.Job)
			{
				_logger.LogDebug("Adding report to job {JobId}: {Name} -> {Content}", job.Id, request.Name, request.Content);
				await _jobService.UpdateJobAsync(job, reports: new List<JobReport> { newReport });
			}
			else
			{
				JobStepId stepId = JobStepId.Parse(request.StepId);
				_logger.LogDebug("Adding report to step {JobId}:{BatchId}:{StepId}: {Name} -> {Content}", job.Id, batch.Id, request.StepId, request.Name, request.Content);
				await _jobService.UpdateStepAsync(job, batch.Id, stepId, streamConfig, newReports: new List<JobReport> { newReport });

				if (!String.IsNullOrEmpty(newReport.Message))
				{
					List<string> reportMetaData = new List<string>() { $"{JobReportMetadataKeys.MessageKey}={newReport.Message}"};

					if (!String.IsNullOrEmpty(newReport.Severity))
					{
						reportMetaData.Add($"{JobReportMetadataKeys.MessageSeverityKey}={newReport.Severity}");
					}

					if (!String.IsNullOrEmpty(newReport.Color))
					{
						reportMetaData.Add($"{JobReportMetadataKeys.ColorKey}={newReport.Color}");
					}

					Dictionary<JobStepId, List<string>> stepMetaData = new Dictionary<JobStepId, List<string>>() { { stepId, reportMetaData } };					
					
					await _jobService.UpdateMetadataAsync(job.Id, null, stepMetaData, context.CancellationToken);
				}
			}

			return new RpcCreateReportResponse();
		}
	}
}
