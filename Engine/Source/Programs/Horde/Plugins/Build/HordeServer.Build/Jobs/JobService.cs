// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.Timing;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Auditing;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Templates;
using HordeServer.Jobs.Timing;
using HordeServer.Logs;
using HordeServer.Notifications;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Users;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Exception thrown when attempting to retry executing a node that does not allow retries
	/// </summary>
	public class RetryNotAllowedException : Exception
	{
		/// <summary>
		/// The node name
		/// </summary>
		public string NodeName { get; private set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="nodeName">Name of the node that does not permit retries</param>
		public RetryNotAllowedException(string nodeName)
			: base($"Node '{nodeName}' does not permit retries")
		{
			NodeName = nodeName;
		}
	}

	/// <summary>
	/// Cache of information about job ACLs
	/// </summary>
	public class JobPermissionsCache
	{
		/// <summary>
		/// Map of job id to permissions for that job
		/// </summary>
		public Dictionary<JobId, StreamId> Jobs { get; set; } = new Dictionary<JobId, StreamId>();
	}

	/// <summary>
	/// Wraps funtionality for manipulating jobs, jobsteps, and jobstep runs
	/// </summary>
	public class JobService : ILogExtAuthProvider
	{
		readonly IJobCollection _jobs;
		readonly IGraphCollection _graphs;
		readonly IAgentCollection _agents;
		readonly IJobStepRefCollection _jobStepRefs;
		readonly IBisectTaskCollection _bisectTasks;
		readonly IJobTimingCollection _jobTimings;
		readonly IUserCollection _userCollection;
		readonly INotificationTriggerCollection _triggerCollection;
		readonly JobTaskSource _jobTaskSource;
		readonly ITemplateCollectionInternal _templateCollection;
		readonly IPerforceService _perforceService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly IAuditLog<JobId> _auditLog;
		readonly ITelemetryWriter _telemetryWriter;

		/// <summary>
		/// Delegate for label update events
		/// </summary>
		public delegate void LabelUpdateEvent(IJob job, IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates);

		/// <summary>
		/// Event triggered when a label state changes
		/// </summary>
		public event LabelUpdateEvent? OnLabelUpdate;

		/// <summary>
		/// Delegate for job aborted events
		/// </summary>
		public delegate void JobAbortedEvent(IJob job, IGraph graph);

		/// <summary>
		/// Event triggered when a job has been aborted
		/// </summary>
		public event JobAbortedEvent? OnJobAborted;

		/// <summary>
		/// Delegate for job step complete events
		/// </summary>
		public delegate void JobStepCompleteEvent(IJob job, IGraph graph, JobStepBatchId batchId, JobStepId stepId);

		/// <summary>
		/// Event triggered when a job step completes
		/// </summary>
		public event JobStepCompleteEvent? OnJobStepComplete;

		/// <summary>
		/// Event triggered when a job is scheduled on the underlying JobTaskSource
		///
		/// Exposed and duplicated here to avoid dependency on the exact JobTaskSource
		/// </summary>
		public event JobTaskSource.JobScheduleEvent? OnJobScheduled;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobService(IJobCollection jobs, IGraphCollection graphs, IAgentCollection agents, IJobStepRefCollection jobStepRefs, IBisectTaskCollection bisectTasks, IJobTimingCollection jobTimings, IUserCollection userCollection, INotificationTriggerCollection triggerCollection, JobTaskSource jobTaskSource, ITemplateCollectionInternal templateCollection, IPerforceService perforceService, Tracer tracer, IOptionsMonitor<BuildConfig> buildConfig, ILogger<JobService> logger, IAuditLog<JobId> auditLog, ITelemetryWriter telemetryWriter)
		{
			_jobs = jobs;
			_graphs = graphs;
			_agents = agents;
			_jobStepRefs = jobStepRefs;
			_bisectTasks = bisectTasks;
			_jobTimings = jobTimings;
			_userCollection = userCollection;
			_triggerCollection = triggerCollection;
			_jobTaskSource = jobTaskSource;
			_templateCollection = templateCollection;
			_perforceService = perforceService;
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;
			_auditLog = auditLog;
			_telemetryWriter = telemetryWriter;

			_jobTaskSource.OnJobScheduled += (pool, numAgentsOnline, job, graph, batchId) =>
			{
				OnJobScheduled?.Invoke(pool, numAgentsOnline, job, graph, batchId);
			};
		}

		/// <summary>
		/// Creates a new job
		/// </summary>
		/// <param name="jobId">A requested job id</param>
		/// <param name="streamConfig">The stream that this job belongs to</param>
		/// <param name="templateRefId">Name of the template ref</param>
		/// <param name="templateHash">Template for this job</param>
		/// <param name="graph">The graph for the new job</param>
		/// <param name="name">Name of the job</param>
		/// <param name="commitId">The change to build</param>
		/// <param name="codeCommitId">The corresponding code changelist</param>
		/// <param name="options">Options for the new job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Unique id representing the job</returns>
		public async Task<IJob> CreateJobAsync(JobId? jobId, StreamConfig streamConfig, TemplateId templateRefId, ContentHash templateHash, IGraph graph, string name, CommitId commitId, CommitId? codeCommitId, CreateJobOptions options, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(CreateJobAsync)}");
			span.SetAttribute("JobId", jobId?.ToString());
			span.SetAttribute("Stream", streamConfig.Name);
			span.SetAttribute("TemplateRefId", templateRefId);
			span.SetAttribute("TemplateHash", templateHash);
			span.SetAttribute("GraphId", graph.Id);
			span.SetAttribute("Name", name);
			span.SetAttribute("Change", commitId.ToString());
			span.SetAttribute("CodeChange", codeCommitId?.ToString());
			span.SetAttribute("PreflightChange", options.PreflightCommitId?.ToString());
			span.SetAttribute("StartedByUserId", options.StartedByUserId.ToString());
			span.SetAttribute("Priority", options.Priority.ToString());
			span.SetAttribute("ShowUgsBadges", options.ShowUgsBadges);
			span.SetAttribute("NotificationChannel", options.NotificationChannel ?? "null");
			span.SetAttribute("NotificationChannelFilter", options.NotificationChannelFilter ?? "null");
			span.SetAttribute("Arguments", String.Join(',', options.Arguments));

			if (options.AutoSubmit != null)
			{
				span.SetAttribute("AutoSubmit", options.AutoSubmit.Value);
			}
			if (options.UpdateIssues != null)
			{
				span.SetAttribute("UpdateIssues", options.UpdateIssues.Value);
			}
			if (options.JobTriggers != null)
			{
				span.SetAttribute("JobTriggers.Count", options.JobTriggers.Count);
			}

			JobId jobIdValue = jobId ?? JobIdUtils.GenerateNewId();
			using IDisposable? scope = _logger.BeginScope("CreateJobAsync({JobId})", jobIdValue);

			_logger.LogInformation("Creating job at CL {Change}, code CL {CodeChange}, preflight CL {PreflightChange}", commitId, codeCommitId, options.PreflightCommitId);
			_jobs.JobAudit(jobIdValue, "Creating job at CL {Change}, code CL {CodeChange}, preflight CL {PreflightChange}", commitId, codeCommitId, options.PreflightCommitId);

			Dictionary<string, string> properties = new Dictionary<string, string>();
			properties["Change"] = commitId.ToString();
			properties["CodeChange"] = codeCommitId?.ToString() ?? String.Empty;
			properties["PreflightChange"] = options.PreflightCommitId?.ToString() ?? String.Empty;
			properties["StreamId"] = streamConfig.Id.ToString();
			properties["TemplateId"] = templateRefId.ToString();
			properties["JobId"] = jobIdValue.ToString();

			for (int idx = 0; idx < options.Arguments.Count; idx++)
			{
				options.Arguments[idx] = StringUtils.ExpandProperties(options.Arguments[idx], properties);
			}

			name = StringUtils.ExpandProperties(name, properties);

			IJob newJob = await _jobs.AddAsync(jobIdValue, streamConfig.Id, templateRefId, templateHash, graph, name, commitId, codeCommitId, options, cancellationToken);
			_jobTaskSource.UpdateQueuedJob(newJob, graph);

			await _jobTaskSource.UpdateUgsBadgesAsync(newJob, graph, new List<(LabelState, LabelOutcome)>(), cancellationToken);

			if (options.StartedByUserId != null)
			{
				await _userCollection.UpdateSettingsAsync(options.StartedByUserId.Value, addPinnedJobIds: new[] { newJob.Id }, templateOptions: new UpdateUserJobTemplateOptions { StreamId = streamConfig.Id, TemplateId = templateRefId, TemplateHash = templateHash.ToString(), Arguments = options.Arguments }, cancellationToken: cancellationToken);
			}

			await AbortAnyDuplicateJobsAsync(newJob, cancellationToken);

			return newJob;
		}

		private async Task AbortAnyDuplicateJobsAsync(IJob newJob, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(AbortAnyDuplicateJobsAsync)}");
			span.SetAttribute("JobId", newJob.Id.ToString());
			span.SetAttribute("JobName", newJob.Name);

			IReadOnlyList<IJob> jobsToAbort = new List<IJob>();
			if (newJob.PreflightCommitId != null)
			{
				jobsToAbort = await _jobs.FindAsync(new FindJobOptions(PreflightCommitId: newJob.PreflightCommitId), cancellationToken: cancellationToken);
			}

			foreach (IJob job in jobsToAbort)
			{
				if (job.GetState() == JobState.Complete)
				{
					continue;
				}
				if (job.Id == newJob.Id)
				{
					continue; // Don't remove the new job
				}
				if (job.TemplateId != newJob.TemplateId)
				{
					continue;
				}
				if (job.TemplateHash != newJob.TemplateHash)
				{
					continue;
				}
				if (job.StartedByUserId != newJob.StartedByUserId)
				{
					continue;
				}
				if (String.Join(",", job.Arguments) != String.Join(",", newJob.Arguments))
				{
					continue;
				}

				IJob? updatedJob = await UpdateJobAsync(job, null, null, null, KnownUsers.System, null, null, null, cancellationToken: cancellationToken);
				if (updatedJob == null)
				{
					_logger.LogError("Failed marking duplicate job as aborted! Job ID: {JobId}", job.Id);
					_jobs.JobAudit(job.Id, "Failed marking duplicate job as aborted!");
				}

				IJob? updatedJob2 = await GetJobAsync(job.Id, cancellationToken);
				if (updatedJob2?.AbortedByUserId != updatedJob?.AbortedByUserId)
				{
					throw new NotImplementedException();
				}
			}
		}

		/// <summary>
		/// Deletes a job
		/// </summary>
		/// <param name="job">The job to delete</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<bool> DeleteJobAsync(IJob job, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(DeleteJobAsync)}");
			span.SetAttribute("JobId", job.Id.ToString());
			span.SetAttribute("JobName", job.Name);

			using IDisposable? scope = _logger.BeginScope("DeleteJobAsync({JobId})", job.Id);

			// Delete the job
			while (!await job.TryDeleteAsync(cancellationToken))
			{
				IJob? newJob = await job.RefreshAsync(cancellationToken);
				if (newJob == null)
				{
					return false;
				}
				job = newJob;
			}

			// Remove all the triggers from it
			List<NotificationTriggerId> triggerIds = new List<NotificationTriggerId>();
			if (job.NotificationTriggerId != null)
			{
				triggerIds.Add(job.NotificationTriggerId.Value);
			}
			foreach (IJobStep step in job.Batches.SelectMany(x => x.Steps))
			{
				if (step.NotificationTriggerId != null)
				{
					triggerIds.Add(step.NotificationTriggerId.Value);
				}
			}
			await _triggerCollection.DeleteAsync(triggerIds, cancellationToken);

			return true;
		}

		/// <summary>
		/// Delete all the jobs for a stream
		/// </summary>
		/// <param name="streamId">Unique id of the stream</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public async Task DeleteJobsForStreamAsync(StreamId streamId, CancellationToken cancellationToken)
		{
			await _jobs.RemoveStreamAsync(streamId, cancellationToken);
		}

		/// <summary>
		/// Updates a new job
		/// </summary>
		/// <param name="job">The job document to update</param>
		/// <param name="name">Name of the job</param>
		/// <param name="priority">Priority of the job</param>
		/// <param name="autoSubmit">Whether to automatically submit the preflighted change on completion</param>
		/// <param name="abortedByUserId">Name of the user that aborted this job</param>
		/// <param name="onCompleteTriggerId">Object id for a notification trigger</param>
		/// <param name="reports">New reports to add</param>
		/// <param name="arguments">New arguments for the job</param>
		/// <param name="labelIdxToTriggerId">New trigger ID for a label in the job</param>
		/// <param name="cancellationReason">Reason the job was canceled</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<IJob?> UpdateJobAsync(IJob job, string? name = null, Priority? priority = null, bool? autoSubmit = null, UserId? abortedByUserId = null, NotificationTriggerId? onCompleteTriggerId = null, List<JobReport>? reports = null, List<string>? arguments = null, KeyValuePair<int, NotificationTriggerId>? labelIdxToTriggerId = null, string? cancellationReason = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(UpdateJobAsync)}");
			span.SetAttribute("JobId", job.Id.ToString());
			span.SetAttribute("Name", name);

			using IDisposable? scope = _logger.BeginScope("UpdateJobAsync({JobId})", job.Id);
			for (IJob? newJob = job; newJob != null; newJob = await GetJobAsync(job.Id, cancellationToken))
			{
				IGraph graph = await GetGraphAsync(newJob, cancellationToken);

				// Capture the previous label states
				IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates = newJob.GetLabelStates(graph);

				// Update the new list of job steps
				newJob = await newJob.TryUpdateJobAsync(name, priority, autoSubmit, null, null, abortedByUserId, onCompleteTriggerId, reports, arguments, labelIdxToTriggerId, cancellationReason: cancellationReason, cancellationToken: cancellationToken);
				if (newJob != null)
				{
					// Update any badges that have been modified
					await _jobTaskSource.UpdateUgsBadgesAsync(newJob, graph, oldLabelStates, cancellationToken);

					// Cancel any leases which are no longer required
					foreach (IJobStepBatch batch in newJob.Batches)
					{
						if ((batch.Error == JobStepBatchError.Cancelled || batch.Error == JobStepBatchError.NoLongerNeeded) && (batch.State == JobStepBatchState.Starting || batch.State == JobStepBatchState.Running) && batch.AgentId != null && batch.LeaseId != null)
						{
							LeaseOutcomeReason reason = LeaseOutcomeReason.ServerInitiated;
							if (abortedByUserId != null)
							{
								reason |= LeaseOutcomeReason.UserInitiated;
							}
							if (batch.Error == JobStepBatchError.NoLongerNeeded)
							{
								reason |= LeaseOutcomeReason.NoLongerNeeded;
							}
							await CancelLeaseAsync(batch.AgentId.Value, batch.LeaseId.Value, reason, cancellationToken);
						}
					}

					// Notify subscribers that the job has been canceled
					if (newJob.GetState() == JobState.Complete && abortedByUserId is not null)
					{
						using TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(UpdateJobAsync)}.NotifySubscribers");
						OnJobAborted?.Invoke(newJob, graph);
					}

					return newJob;
				}
			}
			return null;
		}

		/// <summary>
		/// Evaluate a change query to determine which CL to run a job at
		/// </summary>
		/// <returns></returns>
		public async Task<CommitIdWithOrder?> EvaluateChangeQueriesAsync(StreamId streamId, IEnumerable<IChangeQuery> queries, List<CommitTag>? commitTags, ICommitCollection commits, CancellationToken cancellationToken)
		{
			foreach (IChangeQuery query in queries)
			{
				CommitIdWithOrder? commitId = await EvaluateChangeQueryAsync(streamId, query, commitTags, commits, cancellationToken);
				if (commitId != null)
				{
					return commitId;
				}
			}
			return null;
		}

		/// <summary>
		/// Evaluate a change query to determine which CL to run a job at
		/// </summary>
		/// <returns></returns>
		public async ValueTask<CommitIdWithOrder?> EvaluateChangeQueryAsync(StreamId streamId, IChangeQuery query, List<CommitTag>? commitTags, ICommitCollection commits, CancellationToken cancellationToken)
		{
			if (query.Condition == null || query.Condition.Evaluate(propertyName => GetTagPropertyValues(propertyName, commitTags)))
			{
				// Find the last change with the listed tag
				if (query.CommitTag != null)
				{
					ICommit? taggedCommit = await commits.FindAsync(null, null, 1, new[] { query.CommitTag.Value }, cancellationToken).FirstOrDefaultAsync(cancellationToken);
					if (taggedCommit != null)
					{
						_logger.LogInformation("Last commit with tag '{Tag}' was {Change}", query.CommitTag.Value, taggedCommit.Id);
						return taggedCommit.Id;
					}
				}

				// Find the job with the given result
				if (query.TemplateId != null)
				{
					IReadOnlyList<JobStepOutcome> outcomes = query.Outcomes ?? new List<JobStepOutcome> { JobStepOutcome.Success };

					IReadOnlyList<IJob> jobs = await FindJobsAsync(new FindJobOptions(StreamId: streamId, Templates: new[] { query.TemplateId.Value }, Target: query.Target, State: new[] { JobStepState.Completed }, Outcome: outcomes.ToArray(), ExcludeUserJobs: true, ExcludeCancelled: true), count: 1, cancellationToken: cancellationToken);
					if (jobs.Count > 0)
					{
						_logger.LogInformation("Last successful build of {TemplateId} target {Target} was job {JobId} at change {Change}", query.TemplateId, query.Target, jobs[0].Id, jobs[0].CommitId);
						return jobs[0].CommitId;
					}
				}
			}
			return null;
		}

		static IEnumerable<string> GetTagPropertyValues(string propertyName, List<CommitTag>? commitTags)
		{
			const string TagPrefix = "tag.";
			if (propertyName.StartsWith(TagPrefix, StringComparison.Ordinal) && commitTags != null)
			{
				string tagName = propertyName.Substring(TagPrefix.Length);
				foreach (CommitTag commitTag in commitTags)
				{
					if (tagName.Equals(commitTag.Text, StringComparison.OrdinalIgnoreCase))
					{
						yield return "1";
					}
				}
			}
		}

		/// <summary>
		/// Cancel an active lease
		/// </summary>
		/// <param name="agentId">The agent to retreive</param>
		/// <param name="leaseId">The lease id to update</param>
		/// <param name="reason">Reason for cancelling the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		async Task CancelLeaseAsync(AgentId agentId, LeaseId leaseId, LeaseOutcomeReason reason, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(CancelLeaseAsync)}");
			span.SetAttribute("AgentId", agentId.ToString());
			span.SetAttribute("LeaseId", leaseId.ToString());
			span.SetAttribute("Reason", reason.ToString());

			for (; ; )
			{
				IAgent? agent = await _agents.GetAsync(agentId, cancellationToken);
				if (agent == null)
				{
					break;
				}

				int index = 0;
				while (index < agent.Leases.Count && agent.Leases[index].Id != leaseId)
				{
					index++;
				}
				if (index == agent.Leases.Count)
				{
					break;
				}

				IAgentLease lease = agent.Leases[index];
				if (lease.State != LeaseState.Active)
				{
					break;
				}

				IAgent? newAgent = await agent.TryCancelLeaseAsync(index, reason, cancellationToken);
				if (newAgent != null)
				{
					_jobTaskSource.CancelLongPollForAgent(agent.Id);
					break;
				}
			}
		}

		/// <summary>
		/// Gets a job with the given unique id
		/// </summary>
		/// <param name="jobId">Job id to search for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the given job</returns>
		public Task<IJob?> GetJobAsync(JobId jobId, CancellationToken cancellationToken = default)
		{
			return _jobs.GetAsync(jobId, cancellationToken);
		}

		/// <summary>
		/// Gets the graph for a job
		/// </summary>
		/// <param name="job">Job to retrieve the graph for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The graph for this job</returns>
		public Task<IGraph> GetGraphAsync(IJob job, CancellationToken cancellationToken = default)
		{
			return _graphs.GetAsync(job.GraphHash, cancellationToken);
		}

		/// <summary>
		/// Searches for jobs matching the given criteria
		/// </summary>
		/// <param name="options">Options for the search</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of jobs matching the given criteria</returns>
		public Task<IReadOnlyList<IJob>> FindJobsAsync(FindJobOptions options, int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			return _jobs.FindAsync(options, index, count, cancellationToken);
		}

		/// <summary>
		/// Searches for jobs matching a stream with given templates
		/// </summary>
		/// <param name="streamId">The stream containing the job</param>
		/// <param name="templates">Templates to look for</param>
		/// <param name="preflightStartedByUser">User for which to include preflight jobs</param>
		/// <param name="maxCreateTime">The maximum creation time</param>
		/// <param name="modifiedAfter">Filter the results by last modified time</param>	
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="consistentRead">If the database read should be made to the replica server</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of jobs matching the given criteria</returns>
		public async Task<IReadOnlyList<IJob>> FindJobsByStreamWithTemplatesAsync(StreamId streamId, TemplateId[] templates, UserId? preflightStartedByUser = null, DateTimeOffset? maxCreateTime = null, DateTimeOffset? modifiedAfter = null, int? index = null, int? count = null, bool consistentRead = true, CancellationToken cancellationToken = default)
		{
			return await _jobs.FindLatestByStreamWithTemplatesAsync(streamId, templates, preflightStartedByUser, maxCreateTime, modifiedAfter, index, count, consistentRead, cancellationToken);
		}

		/// <summary>
		/// Attempts to update the node groups to be executed for a job. Fails if another write happens in the meantime.
		/// </summary>
		/// <param name="job">The job to update</param>
		/// <param name="newGraph">New graph for this job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the groups were updated to the given list. False if another write happened first.</returns>
		public async Task<IJob?> TryUpdateGraphAsync(IJob job, IGraph newGraph, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateGraphAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("NewGraph", newGraph.Id);

			using IDisposable? scope = _logger.BeginScope("TryUpdateGraphAsync({JobId})", job.Id);

			IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates = job.GetLabelStates(newGraph);

			IJob? newJob = await job.TryUpdateGraphAsync(newGraph, cancellationToken);
			if (newJob != null)
			{
				await _jobTaskSource.UpdateUgsBadgesAsync(newJob, newGraph, oldLabelStates, cancellationToken);
				_jobTaskSource.UpdateQueuedJob(newJob, newGraph);
			}
			return newJob;
		}

		/// <summary>
		/// Gets the timing info for a particular job.
		/// </summary>
		/// <param name="job">The job to get timing info for</param>
		/// <param name="logger"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Timing info for the given job</returns>
		public async Task<IJobTiming> GetJobTimingAsync(IJob job, ILogger logger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(GetJobTimingAsync)}");
			span.SetAttribute("Job", job.Id.ToString());

			using IDisposable? scope = _logger.BeginScope("GetJobTimingAsync({JobId})", job.Id);

			IGraph graph = await _graphs.GetAsync(job.GraphHash, cancellationToken);

			Dictionary<string, JobStepTimingData> cachedNewSteps = new Dictionary<string, JobStepTimingData>();
			for (; ; )
			{
				// Try to get the current timing document
				IJobTiming? jobTiming = await _jobTimings.TryGetAsync(job.Id);

				// Calculate timing info for any missing steps
				List<JobStepTimingData> newSteps = new List<JobStepTimingData>();
				foreach (IJobStepBatch batch in job.Batches)
				{
					INodeGroup group = graph.Groups[batch.GroupIdx];
					foreach (IJobStep step in batch.Steps)
					{
						INode node = group.Nodes[step.NodeIdx];
						if (jobTiming == null || !jobTiming.TryGetStepTiming(node.Name, logger, out _))
						{
							JobStepTimingData? stepTimingData;
							if (!cachedNewSteps.TryGetValue(node.Name, out stepTimingData))
							{
								stepTimingData = await GetStepTimingInfoAsync(job.StreamId, job.TemplateId, node.Name, job.CommitId, cancellationToken);
							}
							newSteps.Add(stepTimingData);
						}
					}
				}

				// Try to add or update the timing document
				if (jobTiming == null)
				{
					IJobTiming? newJobTiming = await _jobTimings.TryAddAsync(job.Id, newSteps);
					if (newJobTiming != null)
					{
						return newJobTiming;
					}
				}
				else if (newSteps.Count == 0)
				{
					return jobTiming;
				}
				else
				{
					IJobTiming? newJobTiming = await _jobTimings.TryAddStepsAsync(jobTiming, newSteps);
					if (newJobTiming != null)
					{
						return newJobTiming;
					}
				}

				// Update the cached list of steps for the next iteration
				foreach (JobStepTimingData newStep in newSteps)
				{
					cachedNewSteps[newStep.Name] = newStep;
				}
			}
		}

		/// <summary>
		/// Finds the average duration for the given step
		/// </summary>
		/// <param name="streamId">The stream to search</param>
		/// <param name="templateId">The template id</param>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="commitId">Maximum changelist to consider</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Expected duration for the given step</returns>
		async Task<JobStepTimingData> GetStepTimingInfoAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder? commitId, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(GetStepTimingInfoAsync)}");
			span.SetAttribute("StreamId", streamId);
			span.SetAttribute("TemplateId", templateId);
			span.SetAttribute("NodeName", nodeName);
			span.SetAttribute("Change", commitId?.ToString());

			// Find all the steps matching the given criteria
			List<IJobStepRef> steps = await _jobStepRefs.GetStepsForNodeAsync(streamId, templateId, nodeName, commitId, false, 10, false, cancellationToken);

			// Sum up all the durations and wait times
			int count = 0;
			float waitTimeSum = 0;
			float initTimeSum = 0;
			float durationSum = 0;

			foreach (IJobStepRef step in steps)
			{
				if (step.FinishTimeUtc != null)
				{
					waitTimeSum += step.BatchWaitTime;
					initTimeSum += step.BatchInitTime;
					durationSum += (float)(step.FinishTimeUtc.Value - step.StartTimeUtc).TotalSeconds;
					count++;
				}
			}

			// Compute the averages
			if (count == 0)
			{
				return new JobStepTimingData(nodeName, null, null, null);
			}
			else
			{
				return new JobStepTimingData(nodeName, waitTimeSum / count, initTimeSum / count, durationSum / count);
			}
		}

		/// <summary>
		/// Updates the state of a batch
		/// </summary>
		/// <param name="job">Job to update</param>
		/// <param name="batchId">Unique id of the batch to update</param>
		/// <param name="streamConfig">The current stream config</param>
		/// <param name="newLogId">The new log file id</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was updated, false if it was deleted</returns>
		public async Task<IJob?> UpdateBatchAsync(IJob job, JobStepBatchId batchId, StreamConfig streamConfig, LogId? newLogId = null, JobStepBatchState? newState = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(UpdateBatchAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("BatchId", batchId.ToString());
			span.SetAttribute("NewLogId", newLogId?.ToString());
			span.SetAttribute("NewState", newState.ToString());

			using IDisposable? scope = _logger.BeginScope("UpdateBatchAsync({JobId})", job.Id);

			JobStepBatchError? error = null;

			bool checkForBadAgent = true;
			for (; ; )
			{
				// Find the index of the appropriate batch
				int batchIdx = job.Batches.FindIndex(x => x.Id == batchId);
				if (batchIdx == -1)
				{
					return null;
				}

				IJobStepBatch batch = job.Batches[batchIdx];

				// If the batch has already been marked as complete, error out
				if (batch.State == JobStepBatchState.Complete)
				{
					return null;
				}

				// If we're marking the batch as complete before the agent has run everything, mark it to conform
				JobStepBatchError? newError = null;
				if (newState == JobStepBatchState.Complete && batch.AgentId != null)
				{
					if (batch.Steps.Any(x => x.State == JobStepState.Waiting || x.State == JobStepState.Ready || x.State == JobStepState.Running))
					{
						// Check if the job is valid. If not, we will fail with a specific error code for it.
						error ??= await CheckJobAsync(job, streamConfig, cancellationToken);

						// Find the agent and set the conform flag
						if (error == null && checkForBadAgent)
						{
							for (; ; )
							{
								IAgent? agent = await _agents.GetAsync(batch.AgentId.Value, cancellationToken);
								if (agent == null || agent.RequestConform)
								{
									break;
								}
								if (await agent.TryUpdateAsync(new UpdateAgentOptions { RequestConform = true }, cancellationToken: cancellationToken) != null)
								{
									_logger.LogError("Agent {AgentId} did not complete lease; marking for conform", agent.Id);
									_jobs.JobAudit(job.Id, "Agent {AgentId} did not complete lease; marking for conform", agent.Id);
									break;
								}
							}
							checkForBadAgent = false;
						}

						// If it failed during setup, use a dedicated error code.
						if (error != null)
						{
							newError = error.Value;
							if (newError == JobStepBatchError.UnknownShelf)
							{
								_logger.LogWarning("Batch {BatchId} marked with error, Unknown Shelved CL: {PreflightCommitId}", batchId, job.PreflightCommitId?.ToString() ?? "Unknown");
							}
							else
							{
								_logger.LogError("Batch {BatchId} marked with error {JobStepBatchError}", batchId, error.Value.ToString());
							}
							_jobs.JobAudit(job.Id, "Batch {BatchId} marked with error {JobStepBatchError}", batchId, error.Value.ToString());
						}
						else if (!batch.Steps.Any(x => x.StartTimeUtc != null))
						{
							newError = JobStepBatchError.SyncingFailed;
							_logger.LogError("Batch {BatchId} marked with error SyncingFailed", batchId);
							_jobs.JobAudit(job.Id, "Batch {BatchId} marked with error SyncingFailed", batchId);
						}
						else
						{
							newError = JobStepBatchError.Incomplete;
							_logger.LogError("Batch {BatchId} marked with error Incomplete", batchId);
							_jobs.JobAudit(job.Id, "Batch {BatchId} marked with error Incomplete", batchId);
						}
					}
				}

				// Update the batch state
				IJob? newJob = await TryUpdateBatchAsync(job, batchId, newLogId, newState, newError, cancellationToken);
				if (newJob != null)
				{
					return newJob;
				}

				// Update the job
				newJob = await GetJobAsync(job.Id, cancellationToken);
				if (newJob == null)
				{
					return null;
				}

				job = newJob;
			}
		}

		async Task<JobStepBatchError?> CheckJobAsync(IJob job, StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			try
			{
				// Only check P4 shelves for numeric preflight commit IDs (P4 changelists)
				// Hash-based preflights use commit IDs and don't have shelves to check
				int? preflightChange = job.PreflightCommitId?.TryGetPerforceChange();
				if (preflightChange != null)
				{
					(CheckShelfResult result, _) = await _perforceService.CheckShelfAsync(streamConfig, preflightChange.Value, cancellationToken);
					if (result != CheckShelfResult.Ok)
					{
						_logger.LogWarning("Job {JobId} is no longer valid - check shelf returned {Result}", job.Id, result);
						_jobs.JobAudit(job.Id, "Job {JobId} is no longer valid - check shelf returned {Result}", job.Id, result);
						return JobStepBatchError.UnknownShelf;
					}
				}
				return null;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Job {JobId} is no longer valid.", job.Id);
				_jobs.JobAudit(job.Id, "Job {JobId} is no longer valid, {ExceptionMessage}.", job.Id, ex.Message);
				return JobStepBatchError.ExecutionError;
			}
		}

		/// <summary>
		/// Attempts to update the state of a batch
		/// </summary>
		/// <param name="job">Job to update</param>
		/// <param name="batchId">Unique id of the batch to update</param>
		/// <param name="newLogId">The new log file id</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="newError">New error state</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated job, otherwise null</returns>
		public async Task<IJob?> TryUpdateBatchAsync(IJob job, JobStepBatchId batchId, LogId? newLogId = null, JobStepBatchState? newState = null, JobStepBatchError? newError = null, CancellationToken cancellationToken = default)
		{
			_ = this;
			return await job.TryUpdateBatchAsync(batchId, newLogId, newState, newError, cancellationToken);
		}

		/// <summary>
		/// Update a jobstep state
		/// </summary>
		/// <param name="job">Job to update</param>
		/// <param name="batchId">Unique id of the batch containing the step</param>
		/// <param name="stepId">Unique id of the step to update</param>
		/// <param name="streamConfig">Current global config</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="newOutcome">New outcome of the jobstep</param>
		/// <param name="newError">New error for the step</param>
		/// <param name="newAbortRequested">New state of abort request</param>
		/// <param name="newAbortByUserId">New user that requested the abort</param>
		/// <param name="newLogId">New log id for the jobstep</param>
		/// <param name="newNotificationTriggerId">New notification trigger id for the jobstep</param>
		/// <param name="newRetryByUserId">Whether the step should be retried</param>
		/// <param name="newPriority">New priority for this step</param>
		/// <param name="newReports">New list of reports</param>
		/// <param name="newProperties">Property changes. Any properties with a null value will be removed.</param>
		/// <param name="newCancellationReason">The reason the step was canceled</param>
		/// <param name="newSpawnedJob">A new job this step spawned</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was updated, false if it was deleted in the meantime</returns>
		public async Task<IJob?> UpdateStepAsync(IJob job, JobStepBatchId batchId, JobStepId stepId, StreamConfig streamConfig, JobStepState newState = JobStepState.Unspecified, JobStepOutcome newOutcome = JobStepOutcome.Unspecified, JobStepError? newError = null, bool? newAbortRequested = null, UserId? newAbortByUserId = null, LogId? newLogId = null, NotificationTriggerId? newNotificationTriggerId = null, UserId? newRetryByUserId = null, Priority? newPriority = null, List<JobReport>? newReports = null, Dictionary<string, string?>? newProperties = null, string? newCancellationReason = null, JobId? newSpawnedJob = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(UpdateStepAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("BatchId", batchId.ToString());
			span.SetAttribute("StepId", stepId.ToString());

			using IDisposable? scope = _logger.BeginScope("UpdateStepAsync({JobId}:{BatchId}:{StepId})", job.Id, batchId, stepId);
			for (; ; )
			{
				IJob? newJob = await TryUpdateStepAsync(job, batchId, stepId, streamConfig, newState, newOutcome, newError, newAbortRequested, newAbortByUserId, newLogId, newNotificationTriggerId, newRetryByUserId, newPriority, newReports, newProperties, newCancellationReason, newSpawnedJob, cancellationToken);
				if (newJob != null)
				{
					return newJob;
				}

				newJob = await GetJobAsync(job.Id, cancellationToken);
				if (newJob == null)
				{
					return null;
				}

				job = newJob;
			}
		}

		/// <summary>
		/// Update a jobstep state
		/// </summary>
		/// <param name="job">Job to update</param>
		/// <param name="batchId">Unique id of the batch containing the step</param>
		/// <param name="stepId">Unique id of the step to update</param>
		/// <param name="streamConfig">The current stream config</param>
		/// <param name="newState">New state of the jobstep</param>
		/// <param name="newOutcome">New outcome of the jobstep</param>
		/// <param name="newError">New error for the step</param>
		/// <param name="newAbortRequested">New state for request abort</param>
		/// <param name="newAbortByUserId">New name of user that requested the abort</param>
		/// <param name="newLogId">New log id for the jobstep</param>
		/// <param name="newTriggerId">New trigger id for the jobstep</param>
		/// <param name="newRetryByUserId">Whether the step should be retried</param>
		/// <param name="newPriority">New priority for this step</param>
		/// <param name="newReports">New reports</param>
		/// <param name="newProperties">Property changes. Any properties with a null value will be removed.</param>
		/// <param name="newCancellationReason">Property changes. Any properties with a null value will be removed.</param>
		/// <param name="newSpawnedJob">A new spawned job this step added</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the job was updated, false if it was deleted in the meantime</returns>
		public async Task<IJob?> TryUpdateStepAsync(IJob job, JobStepBatchId batchId, JobStepId stepId, StreamConfig streamConfig, JobStepState newState = JobStepState.Unspecified, JobStepOutcome newOutcome = JobStepOutcome.Unspecified, JobStepError? newError = null, bool? newAbortRequested = null, UserId? newAbortByUserId = null, LogId? newLogId = null, NotificationTriggerId? newTriggerId = null, UserId? newRetryByUserId = null, Priority? newPriority = null, List<JobReport>? newReports = null, Dictionary<string, string?>? newProperties = null, string? newCancellationReason = null, JobId? newSpawnedJob = null, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("BatchId", batchId.ToString());
			span.SetAttribute("StepId", stepId.ToString());

			using IDisposable? scope = _logger.BeginScope("TryUpdateStepAsync({JobId}:{BatchId}:{StepId})", job.Id, batchId, stepId);

			// Get the graph for this job
			IGraph graph = await GetGraphAsync(job, cancellationToken);

			// Make sure the job state is set to unspecified
			JobStepState oldState = JobStepState.Unspecified;
			JobStepOutcome oldOutcome = JobStepOutcome.Unspecified;
			if (newState != JobStepState.Unspecified)
			{
				if (job.TryGetStep(batchId, stepId, out IJobStep? step))
				{
					oldState = step.State;
					oldOutcome = step.Outcome;
				}
			}

			// If we're changing the state of a step, capture the label states beforehand so we can update any that change
			IReadOnlyList<(LabelState, LabelOutcome)>? oldLabelStates = null;
			if (oldState != newState || oldOutcome != newOutcome)
			{
				oldLabelStates = job.GetLabelStates(graph);
			}

			// Update the step
			IJob? newJob = await job.TryUpdateStepAsync(batchId, stepId, newState, newOutcome, newError, newAbortRequested, newAbortByUserId, newLogId, newTriggerId, newRetryByUserId, newPriority, newReports, newProperties, newCancellationReason, newSpawnedJob, cancellationToken);
			if (newJob != null)
			{
				job = newJob;

				using TelemetrySpan updateSpan = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.Update");
				updateSpan.SetAttribute("JobId", job.Id.ToString());
				updateSpan.SetAttribute("BatchId", batchId.ToString());
				updateSpan.SetAttribute("StepId", stepId.ToString());

				// Get the current state after updating the step
				// To minimize impact we only want to perform this if newAbortRequested has been set
				JobStepState updatedState = newState;
				JobStepOutcome updatedOutcome = newOutcome;
				IJobStep? step;
				if (newAbortRequested.HasValue && newAbortRequested.Value && job.TryGetStep(batchId, stepId, out step))
				{
					updatedState = step.State;
					updatedOutcome = step.Outcome;
				}
				if (oldState != updatedState || oldOutcome != updatedOutcome)
				{
					_logger.LogInformation("Transitioned job {JobId}, batch {BatchId}, step {StepId} from {OldState} to {NewState}", job.Id, batchId, stepId, oldState, updatedState);
					_jobs.JobAudit(job.Id, "Transitioned job {JobId}, batch {BatchId}, step {StepId} from {OldState} to {NewState}", job.Id, batchId, stepId, oldState, updatedState);

					// Send any updates for modified badges
					if (oldLabelStates != null)
					{
						using TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.SendBadgeUpdates");
						IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates = job.GetLabelStates(graph);
						OnLabelUpdate?.Invoke(job, oldLabelStates, newLabelStates);
						await _jobTaskSource.UpdateUgsBadgesAsync(job, graph, oldLabelStates, newLabelStates, cancellationToken);
					}

					// Submit the change if auto-submit is enabled
					if (job.PreflightCommitId != null && updatedState == JobStepState.Completed)
					{
						(JobStepState state, JobStepOutcome outcome) = job.GetTargetState();
						if (state == JobStepState.Completed)
						{
							if (job.AutoSubmit && outcome == JobStepOutcome.Success && job.AbortedByUserId == null)
							{
								job = await AutoSubmitChangeAsync(streamConfig, job, graph, cancellationToken);
							}
							else if (job.StartedByUserId.HasValue && outcome == JobStepOutcome.Success && job.AbortedByUserId == null)
							{
								// Only tag P4 changelists (numeric preflight commit IDs), not hash-based revisions
								int? p4Change = job.PreflightCommitId?.TryGetPerforceChange();
								if (p4Change != null)
								{
									IUserSettings settings = await _userCollection.GetSettingsAsync(job.StartedByUserId.Value, cancellationToken);
									if (settings.AlwaysTagPreflightCL)
									{
										_logger.LogInformation("Updating description for {PreflightChange} for {UserId} user settings", job.PreflightCommitId, job.StartedByUserId.Value);
										_jobs.JobAudit(job.Id, "Updating description for {PreflightChange} for {UserId} user settings", job.PreflightCommitId, job.StartedByUserId.Value);
										await _perforceService.UpdateChangelistDescriptionAsync(streamConfig.ClusterName, p4Change.Value, x => x.TrimEnd() + $"\n#preflight {job.Id}", cancellationToken);
									}
								}
							}
						}
					}

					// Update the jobstep ref if it completed
					if (updatedState == JobStepState.Running || updatedState == JobStepState.Completed || updatedState == JobStepState.Aborted)
					{
						using TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.UpdateJobStepRef");
						if (job.TryGetBatch(batchId, out IJobStepBatch? batch) && batch.TryGetStep(stepId, out step) && step.StartTimeUtc != null)
						{
							await _jobStepRefs.UpdateAsync(job, batch, step, graph);
							await _bisectTasks.UpdateAsync(job, batch, step, graph, cancellationToken: cancellationToken);
						}
					}

					// Notify subscribers
					if (updatedState == JobStepState.Completed || updatedState == JobStepState.Aborted)
					{
						using TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.NotifySubscribers");
						OnJobStepComplete?.Invoke(job, graph, batchId, stepId);
					}

					// Emit telemetry for job completion.
					if (updatedState == JobStepState.Completed || updatedState == JobStepState.Aborted)
					{
						if (job.GetState() == JobState.Complete)
						{
							IJobStep? completingJobStep;
							if (job.TryGetStep(batchId, stepId, out completingJobStep) && completingJobStep.FinishTimeUtc != null)
							{
								if (!streamConfig.TelemetryStoreId.IsEmpty)
								{
									try
									{
										_telemetryWriter.WriteEvent(streamConfig.TelemetryStoreId, new JobSummaryTelemetry(job, completingJobStep.FinishTimeUtc.Value));
										JobLabelTelemetryHelpers.EmitJobLabelTelemetry(_telemetryWriter, streamConfig.TelemetryStoreId, job, graph);
									}
									catch (Exception ex)
									{
										_logger.LogError(ex, "Failed to publish JobSummary telemetry.");
									}
								}
							}
						}
					}

					// Create any downstream jobs
					if (updatedState == JobStepState.Completed && updatedOutcome != JobStepOutcome.Failure)
					{
						using TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.CreateDownstreamJobs");
						for (int idx = 0; idx < job.ChainedJobs.Count; idx++)
						{
							IChainedJob jobTrigger = job.ChainedJobs[idx];
							if (jobTrigger.JobId == null)
							{
								// Check the state of the trigger's target here; the step state above may not be the thing the trigger is waiting on.
								(JobStepState, JobStepOutcome)? state = job.GetTargetState(graph, jobTrigger.Target);
								if (state != null && state.Value.Item1 == JobStepState.Completed)
								{
									JobStepOutcome jobTriggerOutcome = state.Value.Item2;
									if (jobTriggerOutcome == JobStepOutcome.Success || jobTriggerOutcome == JobStepOutcome.Warnings)
									{
										job = await FireJobTriggerAsync(job, graph, jobTrigger, streamConfig) ?? job;
									}
								}
							}
						}
					}
				}

				using (TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(TryUpdateStepAsync)}.UpdateQueuedJobs"))
				{
					// Notify the dispatch service that the job has changed
					_jobTaskSource.UpdateQueuedJob(job, graph);
				}

				return job;
			}

			return null;
		}

		/// <summary>
		/// Update a job and step metadata
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="jobMetadata"></param>
		/// <param name="stepMetadata"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<IJob?> UpdateMetadataAsync(JobId jobId, List<string>? jobMetadata = null, Dictionary<JobStepId, List<string>>? stepMetadata = null, CancellationToken cancellationToken = default)
		{
			return await _jobs.UpdateMetadataAsync(jobId, jobMetadata, stepMetadata, cancellationToken);
		}

		/// <summary>
		/// Submit the given change for a preflight
		/// </summary>
		/// <param name="streamConfig">The current stream config</param>
		/// <param name="job">The job being run</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		private async Task<IJob> AutoSubmitChangeAsync(StreamConfig streamConfig, IJob job, IGraph graph, CancellationToken cancellationToken)
		{
			if (job.PreflightCommitId == null)
			{
				return job;
			}

			// Auto-submit is only supported for P4 preflights (numeric commit IDs), not hash-based preflights
			if (job.PreflightCommitId.TryGetPerforceChange() == null)
			{
				_logger.LogInformation("Skipping auto-submit for job {JobId} - preflight commit {CommitId} is not a P4 changelist", job.Id, job.PreflightCommitId);
				return job;
			}

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(AutoSubmitChangeAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("Graph", graph.Id);

			int? change;
			string message;
			try
			{
				CommitId preflightCommitId = job.PreflightCommitId;

				_logger.LogInformation("Updating description for {PreflightChange}", preflightCommitId);
				_jobs.JobAudit(job.Id, "Updating description for {PreflightChange}", preflightCommitId);

				await _perforceService.UpdateChangelistDescriptionAsync(streamConfig.ClusterName, preflightCommitId.GetPerforceChange(), x => x.TrimEnd() + $"\n#preflight {job.Id}", cancellationToken);

				_logger.LogInformation("Submitting change {Change} (through {ChangeCopy}) after successful completion of {JobId}", job.PreflightCommitId, preflightCommitId, job.Id);
				(change, message) = await _perforceService.SubmitShelvedChangeAsync(streamConfig, preflightCommitId.GetPerforceChange(), job.PreflightCommitId.GetPerforceChange(), cancellationToken);

				_logger.LogInformation("Attempt to submit {Change} (through {ChangeCopy}): {Message}", job.PreflightCommitId, preflightCommitId, message);
				_jobs.JobAudit(job.Id, "Attempt to submit {Change} (through {ChangeCopy}): {Message}", job.PreflightCommitId, preflightCommitId, message);

				if (!String.IsNullOrEmpty(message))
				{
					message = Regex.Replace(message, @"^Submit validation failed.*\n(?:\s*'[^']*' validation failed:\s*\n)?", "");
				}

				if (change != null)
				{
					await DeleteShelvedChangeAsync(streamConfig.ClusterName, job.PreflightCommitId);
				}
			}
			catch (Exception ex)
			{
				(change, message) = ((int?)null, "Internal error");
				_logger.LogError(ex, "Unable to submit shelved change");
				_jobs.JobAudit(job.Id, "Unable to submit shelved change: {ExceptionMessage}", ex.Message);

			}

			for (; ; )
			{
				IJob? newJob = await job.TryUpdateJobAsync(autoSubmitChange: change, autoSubmitMessage: message, cancellationToken: cancellationToken);
				if (newJob != null)
				{
					return newJob;
				}

				newJob = await GetJobAsync(job.Id, cancellationToken);
				if (newJob == null)
				{
					return job;
				}

				job = newJob;
			}
		}

		/// <summary>
		/// Deletes a shelved changelist, catching and logging any exceptions
		/// </summary>
		/// <param name="clusterName"></param>
		/// <param name="change">The changelist to delete</param>
		/// <returns>True if the change was deleted successfully, false otherwise</returns>
		private Task<bool> DeleteShelvedChangeAsync(string clusterName, CommitId change)
		{
			_ = clusterName;
			_logger.LogInformation("Leaving shelved change {Change}", change);
			return Task.FromResult<bool>(true);
			/*
						_logger.LogInformation("Removing shelf {Change}", change);
						try
						{
							await _perforceService.DeleteShelvedChangeAsync(clusterName, change);
							return true;
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Unable to delete shelved change {Change}", change);
							return false;
						}*/
		}

		/// <summary>
		/// Fires a trigger for a chained job
		/// </summary>
		/// <param name="job">The job containing the trigger</param>
		/// <param name="graph">Graph for the job containing the trigger</param>
		/// <param name="jobTrigger">The trigger object to fire</param>
		/// <param name="streamConfig">Current config for the stream containing the job</param>
		/// <returns>New job instance</returns>
		private async Task<IJob?> FireJobTriggerAsync(IJob job, IGraph graph, IChainedJob jobTrigger, StreamConfig streamConfig)
		{
			CancellationToken cancellationToken = CancellationToken.None;

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobService)}.{nameof(FireJobTriggerAsync)}");
			span.SetAttribute("Job", job.Id.ToString());
			span.SetAttribute("Graph", graph.Id);
			span.SetAttribute("JobTrigger.Target", jobTrigger.Target);
			span.SetAttribute("JobTrigger.JobId", jobTrigger.JobId?.ToString());
			span.SetAttribute("JobTrigger.TemplateRefId", jobTrigger.TemplateRefId);

			for (; ; )
			{
				// Update the job
				JobId chainedJobId = JobIdUtils.GenerateNewId();

				IJob? newJob = await job.TryUpdateJobAsync(jobTrigger: new KeyValuePair<TemplateId, JobId>(jobTrigger.TemplateRefId, chainedJobId));
				if (newJob != null)
				{
					TemplateRefConfig? templateRefConfig;
					if (!streamConfig.TryGetTemplate(jobTrigger.TemplateRefId, out templateRefConfig))
					{
						_logger.LogWarning("Cannot find template {TemplateId} in stream {StreamId}", jobTrigger.TemplateRefId, newJob.StreamId);
						_jobs.JobAudit(newJob.Id, "Cannot find template {TemplateId} in stream {StreamId}", jobTrigger.TemplateRefId, newJob.StreamId);
						break;
					}

					ITemplate template = await _templateCollection.GetOrAddAsync(templateRefConfig);

					IGraph triggerGraph = await _graphs.AddAsync(template, cancellationToken);

					CommitIdWithOrder? commitId = null;
					CommitIdWithOrder? codeCommitId;
					if (jobTrigger.UseDefaultChangeForTemplate)
					{
						ICommitCollection commits = _perforceService.GetCommits(streamConfig);

						if (templateRefConfig.DefaultChange != null)
						{
							commitId = await EvaluateChangeQueriesAsync(streamConfig.Id, templateRefConfig.DefaultChange, null, commits, cancellationToken);
						}
						if (commitId == null)
						{
							commitId = await commits.GetLastCommitIdAsync(cancellationToken);
						}

						ICommit? codeCommit = await commits.GetLastCodeChangeAsync(commitId, cancellationToken);
						codeCommitId = codeCommit?.Id;
					}
					else
					{
						commitId = job.CommitId;
						codeCommitId = job.CodeCommitId;
					}

					_logger.LogInformation("Creating downstream job {ChainedJobId} from job {JobId} at change {Change}", chainedJobId, newJob.Id, commitId);
					_jobs.JobAudit(newJob.Id, "Creating downstream job {ChainedJobId} from job {JobId} at change {Change}", chainedJobId, newJob.Id, commitId);

					CreateJobOptions options = new CreateJobOptions(templateRefConfig);
					options.PreflightCommitId = newJob.PreflightCommitId;
					options.PreflightDescription = newJob.PreflightDescription;
					template.GetDefaultParameters(options.Parameters, true);
					template.GetArgumentsForParameters(options.Parameters, options.Arguments);

					await CreateJobAsync(chainedJobId, streamConfig, jobTrigger.TemplateRefId, template.Hash, triggerGraph, templateRefConfig.Name, commitId, codeCommitId, options);
					return newJob;
				}

				// Fetch the job again
				newJob = await _jobs.GetAsync(job.Id);
				if (newJob == null)
				{
					break;
				}

				job = newJob;
			}
			return null;
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular stream
		/// </summary>
		/// <param name="jobId">The job to check</param>
		/// <param name="action">The action being performed</param>
		/// <param name="user">The principal to authorize</param>
		/// <param name="buildConfig">Current global config instance</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the action is authorized</returns>
		public async Task<bool> AuthorizeAsync(JobId jobId, AclAction action, ClaimsPrincipal user, BuildConfig buildConfig, CancellationToken cancellationToken)
		{
			IJob? job = await GetJobAsync(jobId, cancellationToken);
			return buildConfig.Authorize(job, action, user);
		}

		/// <inheritdoc/>
		public async ValueTask<bool> AuthorizeAsync(ILog log, AclAction action, ClaimsPrincipal principal, CancellationToken cancellationToken)
		{
			return log.JobId != JobId.Empty && await AuthorizeAsync(log.JobId, action, principal, _buildConfig.CurrentValue, cancellationToken);
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular stream
		/// </summary>
		/// <param name="job">The job to check</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True if the action is authorized</returns>
		public static bool AuthorizeSession(IJob job, ClaimsPrincipal user)
		{
			return job.Batches.Any(x => AuthorizeSession(x, user));
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular stream
		/// </summary>
		/// <param name="batch">The batch being executed</param>
		/// <param name="user">The principal to authorize</param>
		/// <returns>True if the action is authorized</returns>
		public static bool AuthorizeSession(IJobStepBatch batch, ClaimsPrincipal user)
		{
			if (batch.SessionId != null)
			{
				foreach (Claim claim in user.Claims)
				{
					if (claim.Type == HordeClaimTypes.AgentSessionId)
					{
						SessionId sessionIdValue;
						if (SessionId.TryParse(claim.Value, out sessionIdValue) && sessionIdValue == batch.SessionId.Value)
						{
							return true;
						}
					}
				}
			}
			if (batch.LeaseId != null)
			{
				foreach (Claim claim in user.Claims)
				{
					LeaseId leaseIdValue;
					if (LeaseId.TryParse(claim.Value, out leaseIdValue) && leaseIdValue == batch.LeaseId.Value)
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Get a list of each batch's session ID formatted as a string for debugging purposes
		/// </summary>
		/// <param name="job">The job to list</param>
		/// <returns>List of session IDs</returns>
		public static string GetAllBatchSessionIds(IJob job)
		{
			return String.Join(",", job.Batches.Select(b => b.SessionId));
		}

		/// <summary>
		/// Get the logger for the specified job id
		/// </summary>
		/// <param name="jobId"></param>
		/// <returns></returns>
		public IAuditLogChannel<JobId> GetJobLogger(JobId jobId)
		{
			return _auditLog[jobId];
		}
	}
}
