// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Collection of JobStepRef documents
	/// </summary>
	public class JobStepRefCollection : IJobStepRefCollection
	{
		class JobStepRef : IJobStepRef
		{
			[BsonId]
			public JobStepRefId Id { get; set; }

			public JobId? JobId { get; set; }

			public JobStepBatchId? BatchId { get; set; }

			public JobStepId? StepId { get; set; }

			public string JobName { get; set; } = "Unknown";

			public string Name { get; set; }

			public StreamId StreamId { get; set; }
			public TemplateId TemplateId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			[BsonElement("Commit")]
			public string? CommitName { get; set; }

			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			public LogId? LogId { get; set; }
			public PoolId? PoolId { get; set; }
			public AgentId? AgentId { get; set; }
			public JobStepState? State { get; set; }
			public JobStepOutcome? Outcome { get; set; }

			[BsonElement("md"), BsonIgnoreIfNull]
			public List<string>? Metadata { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder? LastSuccess
			{
				get => (LastSuccessName != null) ? new CommitIdWithOrder(LastSuccessName, LastSuccessOrder ?? 0) : (LastSuccessOrder != null) ? CommitIdWithOrder.FromPerforceChange(LastSuccessOrder.Value) : null;
				set => (LastSuccessName, LastSuccessOrder) = (value?.Name, value?.Order);
			}

			[BsonIgnoreIfNull]
			public string? LastSuccessName { get; set; }

			[BsonElement("LastSuccess"), BsonIgnoreIfNull]
			public int? LastSuccessOrder { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder? LastWarning
			{
				get => (LastWarningName != null) ? new CommitIdWithOrder(LastWarningName, LastWarningOrder ?? 0) : (LastWarningOrder != null) ? CommitIdWithOrder.FromPerforceChange(LastWarningOrder.Value) : null;
				set => (LastWarningName, LastWarningOrder) = (value?.Name, value?.Order);
			}

			[BsonIgnoreIfNull]
			public string? LastWarningName { get; set; }

			[BsonElement("LastWarning"), BsonIgnoreIfNull]
			public int? LastWarningOrder { get; set; }

			public float BatchWaitTime { get; set; }
			public float BatchInitTime { get; set; }

			public DateTime JobStartTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? StartTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? StartTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public DateTimeOffset? FinishTime { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? FinishTimeUtc { get; set; }

			[BsonIgnoreIfNull]
			public bool? UpdateIssues { get; set; }

			[BsonIgnoreIfNull]
			public List<int>? IssueIds { get; set; }

			DateTime IJobStepRef.StartTimeUtc => StartTimeUtc ?? StartTime?.UtcDateTime ?? default;
			DateTime? IJobStepRef.FinishTimeUtc => FinishTimeUtc ?? FinishTime?.UtcDateTime;
			string IJobStepRef.NodeName => Name;
			bool IJobStepRef.UpdateIssues => UpdateIssues ?? false;
			IReadOnlyList<int>? IJobStepRef.IssueIds => IssueIds;
			IReadOnlyList<string>? IJobStepRef.Metadata => Metadata;

			public JobStepRef(JobStepRefId id, JobId? jobId, JobStepBatchId? batchId, JobStepId? stepId, string jobName, string nodeName, StreamId streamId, TemplateId templateId, CommitIdWithOrder commitId, LogId? logId, PoolId? poolId, AgentId? agentId, JobStepState? state, JobStepOutcome? outcome, bool updateIssues, CommitIdWithOrder? lastSuccess, CommitIdWithOrder? lastWarning, float batchWaitTime, float batchInitTime, DateTime jobStartTimeUtc, DateTime startTimeUtc, DateTime? finishTimeUtc, List<string>? stepMetadata = null)
			{
				Id = id;
				JobId = jobId;
				BatchId = batchId;
				StepId = stepId;
				JobName = jobName;
				Name = nodeName;
				StreamId = streamId;
				TemplateId = templateId;
				CommitId = commitId;
				LogId = logId;
				PoolId = poolId;
				AgentId = agentId;
				State = state;
				Outcome = outcome;
				UpdateIssues = updateIssues;
				LastSuccess = lastSuccess;
				LastWarning = lastWarning;
				BatchWaitTime = batchWaitTime;
				BatchInitTime = batchInitTime;
				JobStartTimeUtc = jobStartTimeUtc;
				StartTimeUtc = startTimeUtc;
				FinishTimeUtc = finishTimeUtc;
				Metadata = stepMetadata;
			}
		}

		readonly IMongoCollection<JobStepRef> _jobStepRefs;
		readonly ITelemetryWriter _telemetryWriter;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobStepRefCollection(IMongoService mongoService, ITelemetryWriter telemetryWriter, IOptionsMonitor<BuildConfig> buildConfig)
		{
			List<MongoIndex<JobStepRef>> indexes = new List<MongoIndex<JobStepRef>>();
			indexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.TemplateId).Ascending(x => x.Name).Descending(x => x.CommitOrder));

			// Index for obId, batchId, stepId hierarchical queries.
			// Using this index, we can cost-effectively query:
			//	- All job step refs belonging to a specific job
			//	- All job step refs belonging to a batch (of a job)
			indexes.Add(keys => keys.Ascending(x => x.JobId).Ascending(x => x.BatchId).Ascending(x => x.StepId));

			_jobStepRefs = mongoService.GetCollection<JobStepRef>("JobStepRefs", indexes);
			_telemetryWriter = telemetryWriter;
			_buildConfig = buildConfig;
		}

		/// <inheritdoc/>
		public async Task<IJobStepRef> InsertOrReplaceAsync(JobStepRefId id, string jobName, string stepName, StreamId streamId, TemplateId templateId, CommitIdWithOrder commitId, LogId? logId, PoolId? poolId, AgentId? agentId, JobStepState? state, JobStepOutcome? outcome, bool updateIssues, CommitIdWithOrder? lastSuccess, CommitIdWithOrder? lastWarning, float waitTime, float initTime, DateTime jobStartTimeUtc, DateTime startTimeUtc, DateTime? finishTimeUtc, List<string>? stepMetadata = null, ILogger? logger = null)
		{
			JobStepRef newJobStepRef = new JobStepRef(id, id.JobId, id.BatchId, id.StepId, jobName, stepName, streamId, templateId, commitId, logId, poolId, agentId, state, outcome, updateIssues, lastSuccess, lastWarning, waitTime, initTime, jobStartTimeUtc, startTimeUtc, finishTimeUtc, stepMetadata);
			await _jobStepRefs.ReplaceOneAsync(Builders<JobStepRef>.Filter.Eq(x => x.Id, newJobStepRef.Id), newJobStepRef, new ReplaceOptions { IsUpsert = true });

			try
			{
				if (_buildConfig.CurrentValue.TryGetStream(streamId, out StreamConfig? streamConfig) && !streamConfig.TelemetryStoreId.IsEmpty)
				{
					string? message = stepMetadata?.FirstOrDefault(m => m.StartsWith(JobReportMetadataKeys.MessageKey + "=", StringComparison.Ordinal))?.Substring(JobReportMetadataKeys.MessageKey.Length + 1);
					string? messageSeverity = stepMetadata?.FirstOrDefault(m => m.StartsWith(JobReportMetadataKeys.MessageSeverityKey + "=", StringComparison.Ordinal))?.Substring(JobReportMetadataKeys.MessageSeverityKey.Length + 1);
					
					_telemetryWriter.WriteEvent(streamConfig.TelemetryStoreId, new JobStepRefTelemetry(
						id.ToString(),
						id.JobId.ToString(),
						id.BatchId.ToString(),
						id.StepId.ToString(),
						agentId,
						initTime,
						waitTime,
						commitId?.Name,
						finishTimeUtc,
						jobName,
						jobStartTimeUtc,
						stepName,
						state,
						outcome,
						poolId,
						startTimeUtc,
						streamId,
						templateId,
						updateIssues,
						(finishTimeUtc != null) ? (double?)(finishTimeUtc.Value - startTimeUtc).TotalSeconds : null,
						message,
						messageSeverity)
					);
				}
			}
			catch (Exception ex)
			{
				logger?.LogError(ex, "Unable to create job step ref telemetry.");
			}

			return newJobStepRef;
		}

		/// <inheritdoc/>
		public async Task<IJobStepRef?> UpdateAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, List<int>? issueIds)
		{
			UpdateDefinitionBuilder<JobStepRef> updateBuilder = Builders<JobStepRef>.Update;
			List<UpdateDefinition<JobStepRef>> updates = new List<UpdateDefinition<JobStepRef>>();

			if (issueIds != null)
			{
				updates.Add(updateBuilder.Set(x => x.IssueIds, issueIds));
			}

			if (updates.Count == 0)
			{
				return await FindAsync(jobId, batchId, stepId);
			}

			JobStepRefId id = new JobStepRefId(jobId, batchId, stepId);
			return await _jobStepRefs.FindOneAndUpdateAsync(x => x.Id.Equals(id), updateBuilder.Combine(updates));
		}

		/// <inheritdoc/>
		public async Task<IJobStepRef?> FindAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId)
		{
			JobStepRefId id = new JobStepRefId(jobId, batchId, stepId);
			return await _jobStepRefs.Find(x => x.Id.Equals(id)).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<List<IJobStepRef>> FindAsync(JobStepRefId[] ids, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<JobStepRef> filterBuilder = Builders<JobStepRef>.Filter;
			FilterDefinition<JobStepRef> filter = FilterDefinition<JobStepRef>.Empty;
			filter &= filterBuilder.In(x => x.Id, ids);

			List<JobStepRef> steps = await _jobStepRefs.Find(filter).SortByDescending(x => x.CommitOrder).ToListAsync(cancellationToken);
			return steps.ConvertAll<IJobStepRef>(x => x);
		}

		/// <inheritdoc/>
		public async Task<List<IJobStepRef>> GetStepsForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder? commitId, bool includeFailed, int maxCount, bool includePreStartAndSkippedStates, CancellationToken cancellationToken)
		{
			// Find all the steps matching the given criteria
			FilterDefinitionBuilder<JobStepRef> filterBuilder = Builders<JobStepRef>.Filter;

			FilterDefinition<JobStepRef> filter = FilterDefinition<JobStepRef>.Empty;
			filter &= filterBuilder.Eq(x => x.StreamId, streamId);
			filter &= filterBuilder.Eq(x => x.TemplateId, templateId);
			filter &= filterBuilder.Eq(x => x.Name, nodeName);
			if (commitId != null)
			{
				filter &= filterBuilder.Lte(x => x.CommitOrder, commitId.Order);
			}
			if (!includeFailed)
			{
				filter &= filterBuilder.Ne(x => x.Outcome, JobStepOutcome.Failure);
			}
			if (!includePreStartAndSkippedStates)
			{
				filter &= filterBuilder.Ne(x => x.State, JobStepState.Skipped);
				filter &= filterBuilder.Ne(x => x.State, JobStepState.Ready);
				filter &= filterBuilder.Ne(x => x.State, JobStepState.Waiting);
			}

			List<JobStepRef> steps = await _jobStepRefs.Find(filter).SortByDescending(x => x.CommitOrder).ThenByDescending(x => x.StartTimeUtc).Limit(maxCount).ToListAsync(cancellationToken);
			return steps.ConvertAll<IJobStepRef>(x => x);
		}

		/// <inheritdoc/>
		public async Task<IJobStepRef?> GetPrevStepForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder commitId, JobStepOutcome? outcome = null, bool? updateIssues = null, IEnumerable<JobId>? excludeJobIds = null)
		{
			FilterDefinitionBuilder<JobStepRef> filterBuilder = Builders<JobStepRef>.Filter;

			FilterDefinition<JobStepRef> filter = FilterDefinition<JobStepRef>.Empty;
			filter &= filterBuilder.Eq(x => x.StreamId, streamId);
			filter &= filterBuilder.Eq(x => x.TemplateId, templateId);
			filter &= filterBuilder.Eq(x => x.Name, nodeName);
			filter &= filterBuilder.Lt(x => x.CommitOrder, commitId.Order);

			if (outcome != null)
			{
				filter &= filterBuilder.Eq(x => x.Outcome, outcome);
			}
			else
			{
				filter &= filterBuilder.Ne(x => x.Outcome, null);
			}

			if (updateIssues != null)
			{
				filter &= filterBuilder.Ne(x => x.UpdateIssues, false);
			}

			if (excludeJobIds != null && excludeJobIds.Any())
			{
				filter &= filterBuilder.Nin(x => x.Id.JobId, excludeJobIds);
			}

			return await _jobStepRefs.Find(filter).SortByDescending(x => x.CommitOrder).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<IJobStepRef?> GetNextStepForNodeAsync(StreamId streamId, TemplateId templateId, string nodeName, CommitIdWithOrder commitId, JobStepOutcome? outcome = null, bool? updateIssues = null)
		{
			FilterDefinitionBuilder<JobStepRef> filterBuilder = Builders<JobStepRef>.Filter;

			FilterDefinition<JobStepRef> filter = FilterDefinition<JobStepRef>.Empty;
			filter &= filterBuilder.Eq(x => x.StreamId, streamId);
			filter &= filterBuilder.Eq(x => x.TemplateId, templateId);
			filter &= filterBuilder.Eq(x => x.Name, nodeName);
			filter &= filterBuilder.Gt(x => x.CommitOrder, commitId.Order);

			if (outcome != null)
			{
				filter &= filterBuilder.Eq(x => x.Outcome, outcome);
			}
			else
			{
				filter &= filterBuilder.Ne(x => x.Outcome, null);
			}

			if (updateIssues != null)
			{
				filter &= filterBuilder.Ne(x => x.UpdateIssues, false);
			}

			return await _jobStepRefs.Find(filter).SortBy(x => x.CommitOrder).FirstOrDefaultAsync();
		}
	}
}
