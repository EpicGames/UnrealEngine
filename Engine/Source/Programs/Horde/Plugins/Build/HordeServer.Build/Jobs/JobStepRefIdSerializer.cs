// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using MongoDB.Bson.Serialization;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Serializer for JobStepRefId objects
	/// </summary>
	public sealed class JobStepRefIdSerializer : IBsonSerializer<JobStepRefId>
	{
		/// <inheritdoc/>
		public Type ValueType => typeof(JobStepRefId);

		/// <inheritdoc/>
		void IBsonSerializer.Serialize(BsonSerializationContext context, BsonSerializationArgs args, object value)
		{
			Serialize(context, args, (JobStepRefId)value);
		}

		/// <inheritdoc/>
		object IBsonSerializer.Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			return ((IBsonSerializer<JobStepRefId>)this).Deserialize(context, args);
		}

		/// <inheritdoc/>
		public JobStepRefId Deserialize(BsonDeserializationContext context, BsonDeserializationArgs args)
		{
			return JobStepRefId.Parse(context.Reader.ReadString());
		}

		/// <inheritdoc/>
		public void Serialize(BsonSerializationContext context, BsonSerializationArgs args, JobStepRefId value)
		{
			context.Writer.WriteString(((JobStepRefId)value).ToString());
		}
	}
}
