// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Ugs;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.Graphs;
using HordeServer.Logs;
using HordeServer.VersionControl.Perforce;
using HordeServer.Streams;
using HordeServer.Tasks;
using HordeServer.Ugs;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Background service to dispatch pending work to agents in priority order.
	/// </summary>
	public sealed class JobTaskSource : TaskSourceBase<ExecuteJobTask>, IHostedService, IAsyncDisposable
	{
		/// <inheritdoc/>
		public override string Type => "Job";

		/// <inheritdoc/>
		public override TaskSourceFlags Flags => TaskSourceFlags.None;

		/// <summary>
		/// An item in the queue to be executed
		/// </summary>
		[DebuggerDisplay("{_job.Id}:{Batch.Id} ({_poolId})")]
		internal class QueueItem
		{
			/// <summary>
			/// The job instance
			/// </summary>
			public IJob _job;

			/// <summary>
			/// Index of the batch within this job to be executed
			/// </summary>
			public int _batchIdx;

			/// <summary>
			/// The pool of machines to allocate from
			/// </summary>
			public PoolId _poolId;

			/// <summary>
			/// The type of workspace that this item should run in
			/// </summary>
			public AgentWorkspaceInfo _workspace;

			/// <summary>
			/// Whether or not to use the AutoSDK.
			/// </summary>
			public bool _useAutoSdk;

			/// <summary>
			/// Task for creating a lease and assigning to a waiter
			/// </summary>
			public Task? _assignTask;

			/// <summary>
			/// Accessor for the batch referenced by this item
			/// </summary>
			public IJobStepBatch Batch => _job.Batches[_batchIdx];

			/// <summary>
			/// Returns an identifier describing this unique batch
			/// </summary>
			public (JobId, JobStepBatchId) Id => (_job.Id, Batch.Id);

			/// <summary>
			/// Whether the item has been removed from the list
			/// </summary>
			public bool _removed;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="job">The job instance</param>
			/// <param name="batchIdx">The batch index to execute</param>
			/// <param name="poolId">Unique id of the pool of machines to allocate from</param>
			/// <param name="workspace">The workspace that this job should run in</param>
			/// <param name="useAutoSdk">Whether or not to use the AutoSDK</param>
			public QueueItem(IJob job, int batchIdx, PoolId poolId, AgentWorkspaceInfo workspace, bool useAutoSdk)
			{
				_job = job;
				_batchIdx = batchIdx;
				_poolId = poolId;
				_workspace = workspace;
				_useAutoSdk = useAutoSdk;
			}
		}

		/// <summary>
		/// Comparer for items in the queue
		/// </summary>
		class QueueItemComparer : IComparer<QueueItem>
		{
			/// <summary>
			/// Compare two items
			/// </summary>
			/// <param name="x">First item to compare</param>
			/// <param name="y">Second item to compare</param>
			/// <returns>Negative value if X is a higher priority than Y</returns>
			public int Compare([AllowNull] QueueItem x, [AllowNull] QueueItem y)
			{
				if (x == null)
				{
					return (y == null) ? 0 : -1;
				}
				else if (y == null)
				{
					return 1;
				}

				int delta = y.Batch.SchedulePriority - x.Batch.SchedulePriority;
				if (delta == 0)
				{
					delta = x._job.Id.Id.CompareTo(y._job.Id.Id);
					if (delta == 0)
					{
						delta = (int)x.Batch.Id.SubResourceId.Value - (int)y.Batch.Id.SubResourceId.Value;
					}
				}
				return delta;
			}
		}

		/// <summary>
		/// Information about an agent waiting for work
		/// </summary>
		class QueueWaiter
		{
			/// <summary>
			/// The agent performing the wait
			/// </summary>
			public IAgent Agent { get; }

			/// <summary>
			/// Task to wait for a lease to be assigned
			/// </summary>
			public Task<CreateLeaseOptions?> Task => LeaseSource.Task;

			/// <summary>
			/// Completion source for the waiting agent. If a new queue item becomes available, the result will be passed through 
			/// </summary>
			public TaskCompletionSource<CreateLeaseOptions?> LeaseSource { get; } = new TaskCompletionSource<CreateLeaseOptions?>(TaskCreationOptions.RunContinuationsAsynchronously);

			/// <summary>
			/// Ignore the waiting agent when trying to assign a new queue item
			/// </summary>
			public bool Skip { get; set; }

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="agent">The agent waiting for a task</param>
			public QueueWaiter(IAgent agent)
			{
				Agent = agent;
			}
		}

		/// <summary>
		/// Use lazy initialization of agent service to prevent circular dependency as AgentService depend on this class
		/// </summary>
		readonly Lazy<AgentService> _agentService;

		readonly IAclService _aclService;
		readonly IStreamCollection _streamCollection;
		readonly ILogCollection _logCollection;
		readonly IJobCollection _jobs;
		readonly IJobStepRefCollection _jobStepRefs;
		readonly IGraphCollection _graphs;
		readonly IPoolCollection _poolCollection;
		readonly IBisectTaskCollection _bisectTasks;
		readonly PoolService _poolService;
		readonly IUgsMetadataCollection _ugsMetadataCollection;
		readonly PerforceLoadBalancer _perforceLoadBalancer;
		readonly IClock _clock;
		readonly IServerInfo _serverInfo;
		readonly IOptionsMonitor<BuildServerConfig> _settings;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger<JobTaskSource> _logger;
		readonly Tracer _tracer;
		readonly ITicker _ticker;
		readonly IEnumerable<IWorkspaceMessageEnricher> _workspaceEnrichers;

		// Object used for ensuring mutual exclusion to the queues
		readonly object _lockObject = new object();

		// List of items waiting to be executed
		SortedSet<QueueItem> _queue = new SortedSet<QueueItem>(new QueueItemComparer());

		// Map from batch id to the corresponding queue item
		Dictionary<(JobId, JobStepBatchId), QueueItem> _batchIdToQueueItem = new Dictionary<(JobId, JobStepBatchId), QueueItem>();

		// Set of long-poll tasks waiting to be satisfied
		readonly HashSet<QueueWaiter> _waiters = new HashSet<QueueWaiter>();

		// Batches for which OnJobScheduled has already been fired (prevents re-firing on every queue refresh)
		readonly HashSet<(JobId, JobStepBatchId)> _scheduledNotificationFired = new HashSet<(JobId, JobStepBatchId)>();

		// During a background queue refresh operation, any updated batches are added to this dictionary for merging into the updated queue.
		List<QueueItem>? _newQueueItemsDuringUpdate;

		/// <summary>
		/// Delegate for job schedule events
		/// </summary>
		public delegate void JobScheduleEvent(IPoolConfig pool, bool hasAgentsOnline, IJob job, IGraph graph, JobStepBatchId batchId);

		/// <summary>
		/// Event triggered when a job is scheduled
		/// </summary>
		public event JobScheduleEvent? OnJobScheduled;

		// Interval between querying the database for jobs to execute
		static readonly TimeSpan s_refreshInterval = TimeSpan.FromSeconds(5.0);

		/// <summary>
		/// Constructor
		/// </summary>
		public JobTaskSource(IAclService aclService, Lazy<AgentService> agentService, IJobCollection jobs, IJobStepRefCollection jobStepRefs, IBisectTaskCollection bisectTasks, IGraphCollection graphs, IPoolCollection pools, PoolService poolService, IUgsMetadataCollection ugsMetadataCollection, IStreamCollection streamCollection, ILogCollection logCollection, PerforceLoadBalancer perforceLoadBalancer, IClock clock, IServerInfo serverInfo, IOptionsMonitor<BuildServerConfig> settings, IOptionsMonitor<BuildConfig> buildConfig, Tracer tracer, ILogger<JobTaskSource> logger, IEnumerable<IWorkspaceMessageEnricher> workspaceEnrichers)
		{
			_aclService = aclService;
			_agentService = agentService;
			_jobs = jobs;
			_jobStepRefs = jobStepRefs;
			_bisectTasks = bisectTasks;
			_graphs = graphs;
			_poolCollection = pools;
			_poolService = poolService;
			_ugsMetadataCollection = ugsMetadataCollection;
			_streamCollection = streamCollection;
			_logCollection = logCollection;
			_perforceLoadBalancer = perforceLoadBalancer;
			_clock = clock;
			_ticker = clock.AddTicker<JobTaskSource>(s_refreshInterval, TickAsync, logger);
			_serverInfo = serverInfo;
			_buildConfig = buildConfig;
			_settings = settings;
			_tracer = tracer;
			_logger = logger;
			_workspaceEnrichers = workspaceEnrichers;

			OnLeaseStartedProperties.Add(nameof(ExecuteJobTask.JobId), x => JobId.Parse(x.JobId)).Add(nameof(ExecuteJobTask.BatchId), x => JobStepBatchId.Parse(x.BatchId)).Add(nameof(ExecuteJobTask.LogId), x => LogId.Parse(x.LogId));
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		/// <inheritdoc/>
		public async ValueTask DisposeAsync() => await _ticker.DisposeAsync();

		/// <summary>
		/// Gets an object containing the stats of the queue for diagnostic purposes.
		/// </summary>
		/// <returns>Status object</returns>
		public object GetStatus()
		{
			lock (_lockObject)
			{
				List<object> outputItems = new List<object>();
				foreach (QueueItem queueItem in _queue)
				{
					outputItems.Add(new { JobId = queueItem._job.Id.ToString(), BatchId = queueItem.Batch.Id.ToString(), PoolId = queueItem._poolId.ToString(), Workspace = queueItem._workspace });
				}

				List<object> outputWaiters = new List<object>();
				foreach (QueueWaiter waiter in _waiters)
				{
					outputWaiters.Add(new { Id = waiter.Agent.Id.ToString(), Pools = waiter.Agent.Pools.Select(x => x.ToString()).ToList(), waiter.Agent.Workspaces });
				}

				return new { Items = outputItems, Waiters = outputWaiters };
			}
		}

		/// <summary>
		/// Cancel any pending wait for an agent, allowing it to cycle its session state immediately
		/// </summary>
		/// <param name="agentId">The agent id</param>
		public void CancelLongPollForAgent(AgentId agentId)
		{
			QueueWaiter? waiter;
			lock (_lockObject)
			{
				waiter = _waiters.FirstOrDefault(x => x.Agent.Id == agentId);
			}
			if (waiter != null)
			{
				waiter.LeaseSource.TrySetCanceled(CancellationToken.None);
			}
		}

		internal class PoolStatus
		{
			public readonly IPoolConfig Pool;
			public readonly bool HasAgents;
			public readonly bool HasEnabledAgents;
			public readonly bool HasOnlineAgents;
			public bool IsAutoScaled => Pool.EnableAutoscaling;

			public PoolStatus(IPoolConfig pool, bool hasAgents, bool hasEnabledAgents, bool hasOnlineAgents)
			{
				Pool = pool;
				HasAgents = hasAgents;
				HasEnabledAgents = hasEnabledAgents;
				HasOnlineAgents = hasOnlineAgents;
			}
		}

		/// <summary>
		/// Calculate status of agents for a pool
		/// </summary>
		/// <param name="utcNow">Current time</param>
		/// <param name="pools">List of all available pools</param>
		/// <param name="agents">List of all available agents</param>
		/// <returns></returns>
		internal static Dictionary<PoolId, PoolStatus> GetPoolStatus(DateTime utcNow, IReadOnlyList<IPoolConfig> pools, IReadOnlyList<IAgent> agents)
		{
			Dictionary<PoolId, PoolStatus> poolStatus = new();

			foreach (IPoolConfig pool in pools)
			{
				Condition poolCondition = pool.Condition ?? Condition.Parse("false");
				List<IAgent> poolAgents = agents.Where(x => x.ExplicitPools.Contains(pool.Id) || x.SatisfiesCondition(poolCondition)).ToList();

				bool hasAgents = poolAgents.Count > 0;
				bool hasEnabledAgents = poolAgents.Any(x => x.Enabled);
				bool hasOnlineAgents = poolAgents.Any(x => x.Enabled && x.IsSessionValid(utcNow));

				poolStatus[pool.Id] = new PoolStatus(pool, hasAgents, hasEnabledAgents, hasOnlineAgents);
			}

			return poolStatus;
		}

		/// <summary>
		/// Background task
		/// </summary>
		/// <param name="cancellationToken">Token that indicates that the service should shut down</param>
		/// <returns>Async task</returns>
		internal async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(JobTaskSource)}.{nameof(TickAsync)}");

			// Set the NewBatchIdToQueueItem member, so we capture any updated jobs during the DB query.
			lock (_lockObject)
			{
				_newQueueItemsDuringUpdate = new List<QueueItem>();
			}

			using TelemetrySpan setupSpan = _tracer.StartActiveSpan($"{nameof(JobTaskSource)}.{nameof(TickAsync)}.Setup");

			// Query all the current streams
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			IReadOnlyList<IStream> streamsList = await _streamCollection.FindAsync(buildConfig.Streams.Select(x => x.Id).ToList(), cancellationToken);
			Dictionary<StreamId, IStream> streams = streamsList.ToDictionary(x => x.Id, x => x);

			// Find all the pools which are valid (ie. have at least one online agent)
			IReadOnlyList<IAgent> agents = await _agentService.Value.GetCachedAgentsAsync(cancellationToken: cancellationToken);
			IReadOnlyList<IPoolConfig> pools = await _poolCollection.GetConfigsAsync(cancellationToken);
			Dictionary<PoolId, PoolStatus> poolStatus = GetPoolStatus(_clock.UtcNow, pools, agents);

			// New list of queue items
			SortedSet<QueueItem> newQueue = new SortedSet<QueueItem>(_queue.Comparer);
			Dictionary<(JobId, JobStepBatchId), QueueItem> newBatchIdToQueueItem = new Dictionary<(JobId, JobStepBatchId), QueueItem>();

			// Query for a new list of jobs for the queue
			IReadOnlyList<IJob> newJobs = await _jobs.GetDispatchQueueAsync(cancellationToken);

			setupSpan.End();
			span.SetAttribute("numNewJobs", newJobs.Count);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("numPools", pools.Count);
			span.SetAttribute("numStreams", streamsList.Count);

			for (int idx = 0; idx < newJobs.Count; idx++)
			{
				IJob? newJob = newJobs[idx];
				using TelemetrySpan newJobSpan = _tracer.StartActiveSpan($"{nameof(JobTaskSource)}.{nameof(TickAsync)}.NewJob");
				newJobSpan.SetAttribute("jobId", newJob.Id.ToString());

				if (newJob.GraphHash == null)
				{
					_logger.LogError("Job {JobId} has a null graph hash and can't be started.", newJob.Id);
					await newJob.TryRemoveFromDispatchQueueAsync(cancellationToken);
					continue;
				}
				if (newJob.AbortedByUserId != null)
				{
					_logger.LogError("Job {JobId} was aborted but not removed from dispatch queue", newJob.Id);
					await newJob.TryRemoveFromDispatchQueueAsync(cancellationToken);
					continue;
				}

				// Get the graph for this job
				IGraph graph = await _graphs.GetAsync(newJob.GraphHash, cancellationToken);

				// Get the stream. If it fails, skip the whole job.
				IStream? stream;
				if (!streams.TryGetValue(newJob.StreamId, out stream))
				{
					newJob = await newJob.SkipAllBatchesAsync(JobStepBatchError.UnknownStream, cancellationToken);
					continue;
				}

				// Update all the batches
				HashSet<JobStepBatchId> checkedBatchIds = new HashSet<JobStepBatchId>();
				while (newJob != null)
				{
					// Find the next batch within this job that is ready
					int batchIdx = newJob.Batches.FindIndex(x => x.State == JobStepBatchState.Ready && checkedBatchIds.Add(x.Id));
					if (batchIdx == -1)
					{
						break;
					}

					// Validate the agent type and workspace settings
					IJobStepBatch batch = newJob.Batches[batchIdx];
					if (!stream.AgentTypes.TryGetValue(graph.Groups[batch.GroupIdx].AgentType, out IAgentType? agentType))
					{
						newJob = await SkipBatchAsync(newJob, batch.Id, graph, JobStepBatchError.UnknownAgentType, cancellationToken);
					}
					else if (!poolStatus.ContainsKey(agentType.Pool))
					{
						newJob = await SkipBatchAsync(newJob, batch.Id, graph, JobStepBatchError.UnknownPool, cancellationToken);
					}
					else if (!poolStatus[agentType.Pool].HasAgents)
					{
						newJob = await SkipBatchAsync(newJob, batch.Id, graph, JobStepBatchError.NoAgentsInPool, cancellationToken);
					}
					else if (!buildConfig.TryGetStream(stream.Id, out StreamConfig? streamConfig) || !streamConfig.TryGetAgentWorkspace(agentType, out AgentWorkspaceInfo? workspace, out AutoSdkConfig? autoSdkConfig))
					{
						newJob = await SkipBatchAsync(newJob, batch.Id, graph, JobStepBatchError.UnknownWorkspace, cancellationToken);
					}
					else
					{
						ITemplateRef? templateRef;
						if (stream.Templates.TryGetValue(newJob.TemplateId, out templateRef))
						{
							if (templateRef.StepStates != null)
							{
								for (int i = 0; i < templateRef.StepStates.Count; i++)
								{
									ITemplateStep state = templateRef.StepStates[i];

									IJobStep? step = batch.Steps.FirstOrDefault(x => graph.Groups[batch.GroupIdx].Nodes[x.NodeIdx].Name.Equals(state.Name, StringComparison.Ordinal));
									if (step != null)
									{
										JobId jobId = newJob.Id;
										newJob = await newJob.TryUpdateStepAsync(batch.Id, step.Id, JobStepState.Skipped, newError: JobStepError.Paused, cancellationToken: cancellationToken);
										if (newJob == null)
										{
											_logger.LogError("Job {JobId} failed to update step {StepName} pause state", jobId, state.Name);
											break;
										}
										else
										{
											_logger.LogInformation("Job {JobId} step {StepName} has been skipped due to being paused", jobId, state.Name);
										}
									}
								}
							}
						}

						if (newJob != null)
						{
							QueueItem newQueueItem = new QueueItem(newJob, batchIdx, agentType.Pool, workspace, autoSdkConfig != null);
							newQueue.Add(newQueueItem);
							newBatchIdToQueueItem[(newJob.Id, batch.Id)] = newQueueItem;

							IPoolConfig? newJobPool = pools.FirstOrDefault(p => p.Id == agentType.Pool);
							if (newJobPool != null && _scheduledNotificationFired.Add((newJob.Id, batch.Id)))
							{
								OnJobScheduled?.Invoke(newJobPool, poolStatus[agentType.Pool].HasOnlineAgents, newJob, graph, batch.Id);
							}
						}
					}
				}

				if (newJob != null)
				{
					if (!newJob.Batches.Any(batch => batch.State == JobStepBatchState.Ready || batch.State == JobStepBatchState.Starting || batch.State == JobStepBatchState.Running || batch.State == JobStepBatchState.Stopping))
					{
						_logger.LogError("Job {JobId} is in dispatch queue but not currently executing", newJob.Id);
						await newJob.TryRemoveFromDispatchQueueAsync(cancellationToken);
					}
				}
			}

			// Update the queue
			lock (_lockObject)
			{
				_queue = newQueue;
				_batchIdToQueueItem = newBatchIdToQueueItem;

				// Merge the new queue items with the queue
				foreach (QueueItem newQueueItem in _newQueueItemsDuringUpdate)
				{
					QueueItem? existingQueueItem;
					if (!newBatchIdToQueueItem.TryGetValue((newQueueItem._job.Id, newQueueItem.Batch.Id), out existingQueueItem))
					{
						// Always just add this item
						_queue.Add(newQueueItem);
						_batchIdToQueueItem[newQueueItem.Id] = newQueueItem;
					}
					else if (newQueueItem._job.UpdateIndex > existingQueueItem._job.UpdateIndex)
					{
						// Replace the existing item
						_queue.Remove(existingQueueItem);
						_queue.Add(newQueueItem);
						_batchIdToQueueItem[newQueueItem.Id] = newQueueItem;
					}
				}

				// Clear out the list to capture queue items during an update
				_newQueueItemsDuringUpdate = null;

				// Prune notification tracking to only batches still in the queue
				_scheduledNotificationFired.IntersectWith(_batchIdToQueueItem.Keys);
			}
		}

		private async Task<IJob?> SkipBatchAsync(IJob job, JobStepBatchId batchId, IGraph graph, JobStepBatchError reason, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Skipping batch {BatchId} for job {JobId} (reason: {Reason})", batchId, job.Id, reason);

			IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates = job.GetLabelStates(graph);
			IJob? newJob = await job.SkipBatchAsync(batchId, reason, cancellationToken);
			if (newJob != null)
			{
				IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates = newJob.GetLabelStates(graph);
				await UpdateUgsBadgesAsync(newJob, graph, oldLabelStates, newLabelStates, cancellationToken);
			}
			return newJob;
		}

		/// <summary>
		/// Get the queue, for internal testing only
		/// </summary>
		/// <returns>A copy of the queue</returns>
		internal SortedSet<QueueItem> GetQueueForTesting()
		{
			lock (_lockObject)
			{
				return new SortedSet<QueueItem>(_queue);
			}
		}

		void AssignAnyQueueItemToWaiter(QueueWaiter waiter)
		{
			lock (_waiters)
			{
				foreach (QueueItem item in _batchIdToQueueItem.Values)
				{
					if (TryAssignItemToWaiter(item, waiter))
					{
						break;
					}
				}
			}
		}

		/// <summary>
		/// Attempt to find a waiter that can handle the given queue item
		/// </summary>
		/// <param name="item">The queue item</param>
		/// <returns></returns>
		[SuppressMessage("Maintainability", "CA1508:Avoid dead conditional code", Justification = "<Pending>")]
		void AssignQueueItemToAnyWaiter(QueueItem item)
		{
			if (item._assignTask == null && item.Batch.SessionId == null)
			{
				lock (_waiters)
				{
					if (item._assignTask == null && item.Batch.SessionId == null)
					{
						foreach (QueueWaiter waiter in _waiters)
						{
							if (TryAssignItemToWaiter(item, waiter))
							{
								break;
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Attempts to assign a queue item to an agent waiting for work
		/// </summary>
		/// <param name="item"></param>
		/// <param name="waiter"></param>
		/// <returns></returns>
		bool TryAssignItemToWaiter(QueueItem item, QueueWaiter waiter)
		{
			if (item._assignTask == null && item.Batch.SessionId == null && waiter.Agent.Enabled && waiter.Agent.Leases.Count == 0 && waiter.Agent.IsInPool(item._poolId) && !waiter.Skip)
			{
				Task startTask = new Task<Task>(() => TryCreateLeaseAsync(item, waiter, CancellationToken.None));
				Task executeTask = startTask.ContinueWith(task => task, TaskScheduler.Default);

				if (Interlocked.CompareExchange(ref item._assignTask, executeTask, null) == null)
				{
					startTask.Start(TaskScheduler.Default);
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Updates the current state of a job
		/// </summary>
		/// <param name="job">The job that has been updated</param>
		/// <param name="graph">Graph for the job</param>
		public void UpdateQueuedJob(IJob job, IGraph graph)
		{
			StreamConfig? streamConfig;
			_buildConfig.CurrentValue.TryGetStream(job.StreamId, out streamConfig);

			List<TaskCompletionSource<bool>> completeWaiters = new List<TaskCompletionSource<bool>>();
			lock (_lockObject)
			{
				for (int batchIdx = 0; batchIdx < job.Batches.Count; batchIdx++)
				{
					IJobStepBatch batch = job.Batches[batchIdx];
					if (batch.State == JobStepBatchState.Ready && streamConfig != null && batch.AgentId == null)
					{
						// Check if this item is already in the list.
						QueueItem? existingItem;
						if (_batchIdToQueueItem.TryGetValue((job.Id, batch.Id), out existingItem))
						{
							// Make sure this is a newer version of the job. There's no guarantee that this is the latest revision.
							if (job.UpdateIndex > existingItem._job.UpdateIndex)
							{
								if (batch.SchedulePriority == existingItem.Batch.SchedulePriority)
								{
									_logger.LogInformation("Updating job in queue for {JobId}:{BatchId} to {UpdateIndex})", job.Id, batch.Id, job.UpdateIndex);
									existingItem._job = job;
									existingItem._batchIdx = batchIdx;
								}
								else
								{
									_logger.LogInformation("Updating job in queue for {JobId}:{BatchId} to {UpdateIndex}) (insert/replace)", job.Id, batch.Id, job.UpdateIndex);
									RemoveQueueItem(existingItem);
									InsertQueueItem(job, batchIdx, existingItem._poolId, existingItem._workspace, existingItem._useAutoSdk);
								}
							}
							continue;
						}

						// Get the group being executed by this batch
						INodeGroup group = graph.Groups[batch.GroupIdx];

						// Get the requirements for the new queue item
						AgentConfig? agentType;
						if (streamConfig.AgentTypes.TryGetValue(group.AgentType, out agentType))
						{
							if (streamConfig.TryGetAgentWorkspace(agentType, out AgentWorkspaceInfo? agentWorkspace, out AutoSdkConfig? autoSdkConfig))
							{
								InsertQueueItem(job, batchIdx, agentType.Pool, agentWorkspace, autoSdkConfig != null);
							}
						}
					}
					else
					{
						// Check if this item is already in the list. Remove it if it is.
						QueueItem? existingItem;
						if (_batchIdToQueueItem.TryGetValue((job.Id, batch.Id), out existingItem))
						{
							if (job.UpdateIndex > existingItem._job.UpdateIndex)
							{
								RemoveQueueItem(existingItem);
							}
							else
							{
								_logger.LogInformation("Ignoring update for {JobId}:{BatchId} - existing update index is newer ({ExistingUpdateIndex} vs {NewUpdateIndex})", job.Id, batch.Id, existingItem._job.UpdateIndex, job.UpdateIndex);
							}
						}
					}
				}

				List<QueueItem> removeItems = _queue.Where(x => x._job.Id == job.Id && x._job.UpdateIndex < job.UpdateIndex).ToList();
				foreach (QueueItem removeItem in removeItems)
				{
					_logger.LogInformation("Removing stale job queue entry {JobId}:{BatchId}", removeItem.Id.Item1, removeItem.Id.Item2);
					RemoveQueueItem(removeItem);
				}
			}

			// Awake all the threads that have been assigned new work items. Has do be done outside the lock to prevent continuations running within it (see Waiter.CompletionSource for more info).
			foreach (TaskCompletionSource<bool> completeWaiter in completeWaiters)
			{
				completeWaiter.TrySetResult(true);
			}
		}

		/// <inheritdoc/>
		public override Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (agent.Mode == AgentMode.Workstation)
			{
				return Task.FromResult(SkipAsync(cancellationToken));
			}
			
			QueueWaiter waiter = new QueueWaiter(agent);
			lock (_lockObject)
			{
				AssignAnyQueueItemToWaiter(waiter);
				if (waiter.LeaseSource.Task.TryGetResult(out CreateLeaseOptions? result))
				{
					if (result == null)
					{
						return Task.FromResult(SkipAsync(cancellationToken));
					}
					return Task.FromResult(LeaseAsync(result));
				}
				_waiters.Add(waiter);
			}
			return Task.FromResult(WaitForLeaseAsync(waiter, cancellationToken));
		}

		private async Task<CreateLeaseOptions?> WaitForLeaseAsync(QueueWaiter waiter, CancellationToken cancellationToken)
		{
			try
			{
				using (cancellationToken.Register(() => waiter.LeaseSource.TrySetResult(null)))
				{
					return await waiter.Task;
				}
			}
			finally
			{
				lock (_lockObject)
				{
					_waiters.Remove(waiter);
				}
			}
		}

		/// <inheritdoc/>
		public override Task CancelLeaseAsync(IAgent agent, LeaseId leaseId, ExecuteJobTask task, CancellationToken cancellationToken)
		{
			return CancelLeaseAsync(agent, JobId.Parse(task.JobId), JobStepBatchId.Parse(task.BatchId), cancellationToken);
		}

		private async Task<bool> AutoConformAgentAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (!_buildConfig.CurrentValue.EnableJobTaskConformCheck)
			{
				return false;
			}

			if (!(_settings.CurrentValue.EnableConformTasks && _buildConfig.CurrentValue.EnableConformTasks))
			{
				return false;
			}

			if (agent.ConformAttemptCount is null or 0 && agent.RequestFullConform)
			{
				_logger.LogInformation("Skipping assigning lease to {AgentId}, as a conform is already pending", agent.Id);
				return true;
			}

			return await ConformUtilities.AutoConformAgentAsync(agent, _poolService, _buildConfig.CurrentValue, _logger, cancellationToken);
		}

		/// <summary>
		/// Assign a new batch to be executed by the given agent
		/// </summary>
		/// <param name="item">The item to create a lease for</param>
		/// <param name="waiter">The agent waiting for work</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New work to execute</returns>
		private async Task<CreateLeaseOptions?> TryCreateLeaseAsync(QueueItem item, QueueWaiter waiter, CancellationToken cancellationToken)
		{
			IJob job = item._job;
			IJobStepBatch batch = item.Batch;
			IAgent agent = waiter.Agent;
			LeaseId leaseId = new(BinaryIdUtils.CreateNew()); // Generate a new unique id for the lease

			using IDisposable? logScope = _logger
				.WithProperty("JobId", job.Id.ToString())
				.WithProperty("BatchId", batch.Id.ToString())
				.WithProperty("AgentId", agent.Id.ToString())
				.WithProperty("LeaseId", leaseId.ToString())
				.BeginScope();

			bool conformRequested = await AutoConformAgentAsync(waiter.Agent, cancellationToken);
			if (conformRequested)
			{
				// Won't be assigning a lease to this waiter
				if (waiter.LeaseSource.TrySetResult(null))
				{
					_logger.LogInformation("Skipping assigning lease to {AgentId}, as a conform has been requested", agent.Id);

					// Don't try to assign another item to this waiter to avoid duplicating a conform request
					waiter.Skip = true;

					// Clear out the assignment for this item, and try to reassign it
					item._assignTask = null;
					if (!item._removed)
					{
						AssignQueueItemToAnyWaiter(item);
					}
					return null;
				}
			}

			_logger.LogInformation("Assigning job to waiter");

			// Allocate a log ID but hold off creating the actual log file until the lease has been accepted
			LogId logId = LogIdUtils.GenerateNewId();

			// Try to update the job with this agent id
			IJob? newJob = await item._job.TryAssignLeaseAsync(item._batchIdx, item._poolId, agent.Id, agent.SessionId!.Value, leaseId, logId, cancellationToken);
			if (newJob != null)
			{
				job = newJob;

				StreamConfig? streamConfig;
				if (!_buildConfig.CurrentValue.TryGetStream(newJob.StreamId, out streamConfig))
				{
					return null;
				}

				// Get the lease name
				StringBuilder leaseName = new StringBuilder($"{streamConfig.Name} - ");
				if (job.PreflightCommitId != null)
				{
					leaseName.Append($"Preflight CL {job.PreflightCommitId} against CL {job.CommitId}");
				}
				else
				{
					leaseName.Append($"CL {job.CommitId}");
				}
				leaseName.Append(CultureInfo.InvariantCulture, $" - {job.Name}");

				// Get the autosdk workspace
				AgentWorkspaceInfo? autoSdkWorkspace = null;
				if (item._useAutoSdk)
				{
					PerforceCluster cluster = _buildConfig.CurrentValue.FindPerforceCluster(streamConfig.ClusterName)!;
					autoSdkWorkspace = await _poolService.GetAutoSdkWorkspaceAsync(agent, cluster, DateTime.UtcNow - TimeSpan.FromSeconds(10.0), cancellationToken);
				}

				// Encode the payload
				ExecuteJobTask? task = await CreateExecuteJobTaskAsync(leaseId, streamConfig, job, batch, agent, item._workspace, autoSdkWorkspace, logId, cancellationToken);
				if (task != null)
				{
					// Create the lease and try to set it on the waiter. If this fails, the waiter has already moved on, and the lease can be cancelled.
					CreateLeaseOptions lease = new CreateLeaseOptions(leaseId, null, leaseName.ToString(), job.StreamId, item._poolId, logId, null, true, task);
					if (waiter.LeaseSource.TrySetResult(lease))
					{
						_logger.LogInformation("Assigned lease to agent");
						await _logCollection.AddAsync(job.Id, leaseId, agent.SessionId, LogType.Json, logId, streamConfig.Acl.ScopeName, cancellationToken);
						return lease;
					}
				}

				// Cancel the lease
				_logger.LogInformation("Unable to assign lease, cancelling");
				await CancelLeaseAsync(waiter.Agent, job.Id, batch.Id, cancellationToken);
			}
			else
			{
				// Unable to assign job
				_logger.LogInformation("Failed to assign job/batch to agent. Refreshing queue entries");

				// Get the new copy of the job
				newJob = await _jobs.GetAsync(job.Id, cancellationToken);
				if (newJob == null)
				{
					lock (_lockObject)
					{
						List<QueueItem> removeItems = _queue.Where(x => x._job == job).ToList();
						foreach (QueueItem removeItem in removeItems)
						{
							RemoveQueueItem(removeItem);
						}
					}
				}
				else
				{
					_logger.LogInformation("Updating job {JobId} in queue from {OldUpdateIndex} -> {NewUpdateIndex}", job.Id, job.UpdateIndex, newJob.UpdateIndex);
					IGraph graph = await _graphs.GetAsync(newJob.GraphHash, cancellationToken);
					UpdateQueuedJob(newJob, graph);
				}
			}

			// Clear out the assignment for this item, and try to reassign it
			item._assignTask = null;
			if (!item._removed)
			{
				AssignQueueItemToAnyWaiter(item);
			}
			return null;
		}

		async Task<ExecuteJobTask?> CreateExecuteJobTaskAsync(LeaseId leaseId, StreamConfig streamConfig, IJob job, IJobStepBatch batch, IAgent agent, AgentWorkspaceInfo workspace, AgentWorkspaceInfo? autoSdkWorkspace, LogId logId, CancellationToken cancellationToken)
		{
			// Get the lease name
			StringBuilder leaseName = new StringBuilder($"{streamConfig.Name} - ");
			if (job.PreflightCommitId != null)
			{
				leaseName.Append($"Preflight CL {job.PreflightCommitId} against CL {job.CommitId}");
			}
			else
			{
				leaseName.Append($"CL {job.CommitId}");
			}
			leaseName.Append(CultureInfo.InvariantCulture, $" - {job.Name}");

			// Get the global settings
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			// Create a bearer token for the job executor
			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(HordeClaims.AgentDedicatedRoleClaim);
			claims.Add(new AclClaimConfig(HordeClaimTypes.Lease, leaseId.ToString()));
			claims.Add(new AclClaimConfig(HordeClaimTypes.LeaseStream, streamConfig.Id.ToString()));
			claims.Add(new AclClaimConfig(HordeClaimTypes.LeaseProject, streamConfig.ProjectConfig.Id.ToString()));
			claims.Add(new AclClaimConfig(HordeClaimTypes.LeaseTemplate, job.TemplateId.ToString()));

			// Copy job claims, except for expiration as we do not want that forwarded to the job executor token
			claims.AddRange(job.Claims.Where(x => x.Type != "exp").ToList().ConvertAll(x => new AclClaimConfig(x.Type, x.Value)));

			// Encode the payload
			ExecuteJobTask task = new ExecuteJobTask();
			task.JobId = job.Id.ToString();
			task.BatchId = batch.Id.ToString();
			task.LogId = logId.ToString();
			task.JobName = leaseName.ToString();
			task.JobOptions = (job.JobOptions != null) ? GetRpcJobOptions(job.JobOptions) : null;
			task.Token = await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);

			List<RpcAgentWorkspace> workspaces = new();

			PerforceCluster? cluster = buildConfig.FindPerforceCluster(workspace.Cluster);
			if (cluster == null)
			{
				return null;
			}

			if (autoSdkWorkspace != null)
			{
				autoSdkWorkspace.Method = workspace.Method;

				if (!await agent.TryAddWorkspaceMessageAsync(autoSdkWorkspace, cluster, _perforceLoadBalancer, workspaces, cancellationToken))
				{
					return null;
				}

				task.AutoSdkWorkspace = workspaces.Last();
			}

			if (!await agent.TryAddWorkspaceMessageAsync(workspace, cluster, _perforceLoadBalancer, workspaces, cancellationToken))
			{
				return null;
			}

			task.Workspace = workspaces.Last();

			// For hash-based version control systems, set the revision hash on the workspace
			if (job.CommitId.TryGetPerforceChange() == null)
			{
				task.Workspace.RevisionHash = job.CommitId.ToString();
			}

			// Allow plugins to enrich workspace messages (e.g., add VCS-specific credentials)
			foreach (IWorkspaceMessageEnricher enricher in _workspaceEnrichers)
			{
				try
				{
					if (task.AutoSdkWorkspace != null && autoSdkWorkspace != null)
					{
						await enricher.EnrichAsync(task.AutoSdkWorkspace, autoSdkWorkspace, agent, job, cancellationToken);
					}
					await enricher.EnrichAsync(task.Workspace, workspace, agent, job, cancellationToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Workspace enricher {Enricher} threw an exception for job {JobId}", enricher.GetType().Name, job.Id);
				}
			}

			return task;
		}

		static RpcJobOptions GetRpcJobOptions(JobOptions options)
		{
			RpcJobOptions rpcOptions = new RpcJobOptions();
			rpcOptions.Executor = options.Executor;
			rpcOptions.UseWine = options.UseWine;
			rpcOptions.RunInSeparateProcess = options.RunInSeparateProcess;
			rpcOptions.WorkspaceMaterializer = options.WorkspaceMaterializer;
			if (options.Container != null)
			{
				rpcOptions.Container = GetRpcJobContainerOptions(options.Container);
			}
			rpcOptions.Driver = options.Driver;
			return rpcOptions;
		}

		static RpcJobContainerOptions GetRpcJobContainerOptions(JobContainerOptions options)
		{
			RpcJobContainerOptions rpcContainerOptions = new RpcJobContainerOptions();
			rpcContainerOptions.Enabled = options.Enabled;
			rpcContainerOptions.ImageUrl = options.ImageUrl;
			rpcContainerOptions.ContainerEngineExecutable = options.ContainerEngineExecutable;
			rpcContainerOptions.ExtraArguments = options.ExtraArguments;
			return rpcContainerOptions;
		}

		/// <summary>
		/// Send any badge updates for this job
		/// </summary>
		/// <param name="job">The job being updated</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="oldLabelStates">Previous badge states for the job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public async Task UpdateUgsBadgesAsync(IJob job, IGraph graph, IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates, CancellationToken cancellationToken)
		{
			await UpdateUgsBadgesAsync(job, graph, oldLabelStates, job.GetLabelStates(graph), cancellationToken);
		}

		/// <summary>
		/// Send any badge updates for this job
		/// </summary>
		/// <param name="job">The job being updated</param>
		/// <param name="graph">Graph for the job</param>
		/// <param name="oldLabelStates">Previous badge states for the job</param>
		/// <param name="newLabelStates">The new badge states for the job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public async Task UpdateUgsBadgesAsync(IJob job, IGraph graph, IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates, CancellationToken cancellationToken)
		{
			if (!job.ShowUgsBadges || job.PreflightCommitId != null)
			{
				return;
			}

			IReadOnlyDictionary<int, UgsBadgeState> oldStates = job.GetUgsBadgeStates(graph, oldLabelStates);
			IReadOnlyDictionary<int, UgsBadgeState> newStates = job.GetUgsBadgeStates(graph, newLabelStates);

			// Figure out a list of all the badges that have been modified
			List<int> updateLabels = new List<int>();
			foreach (KeyValuePair<int, UgsBadgeState> pair in oldStates)
			{
				if (!newStates.ContainsKey(pair.Key))
				{
					updateLabels.Add(pair.Key);
				}
			}
			foreach (KeyValuePair<int, UgsBadgeState> pair in newStates)
			{
				if (!oldStates.TryGetValue(pair.Key, out UgsBadgeState prevState) || prevState != pair.Value)
				{
					updateLabels.Add(pair.Key);
				}
			}

			// Cached stream for this job
			StreamConfig? streamConfig = null;

			// Send all the updates
			Dictionary<int, IUgsMetadata> metadataCache = new Dictionary<int, IUgsMetadata>();
			foreach (int labelIdx in updateLabels)
			{
				ILabel label = graph.Labels[labelIdx];

				// Skip if this label has no UGS name.
				if (label.UgsName == null)
				{
					continue;
				}

				// Get the new state
				if (!newStates.TryGetValue(labelIdx, out UgsBadgeState newState))
				{
					newState = UgsBadgeState.Skipped;
				}

				// Get the stream
				if (streamConfig == null)
				{
					if (!_buildConfig.CurrentValue.TryGetStream(job.StreamId, out streamConfig))
					{
						_logger.LogError("Unable to fetch definition for stream {StreamId}", job.StreamId);
						break;
					}
				}

				// The changelist number to display the badge for
				int change;
				if (label.Change == LabelChange.Code)
				{
					change = job.CodeCommitId?.GetPerforceChange() ?? 0;
				}
				else
				{
					change = job.CommitId.GetPerforceChange();
				}

				// Get the current metadata state
				IUgsMetadata? metadata;
				if (!metadataCache.TryGetValue(change, out metadata))
				{
					metadata = await _ugsMetadataCollection.FindOrAddAsync(streamConfig.Name, change, label.UgsProject, cancellationToken);
					metadataCache[change] = metadata;
				}

				// Try/catch UpdateBadgeAsync call as DocumentDB has sporadically been throwing write exceptions related to this call
				// Rather than failing the upstream request, which usually are UpdateStep or UpdateGraph gRPC calls, the error is logged
				try
				{
					// Apply the update
					Uri labelUrl = new Uri(_serverInfo.DashboardUrl, $"job/{job.Id}?label={labelIdx}");
					_logger.LogInformation("Updating state of badge {BadgeName} at {Change} to {NewState} ({LabelUrl})", label.UgsName, change, newState, labelUrl);
					metadata = await _ugsMetadataCollection.UpdateBadgeAsync(metadata, label.UgsName!, labelUrl, newState, cancellationToken);
					metadataCache[change] = metadata;
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed updating UGS metadata badge!");
				}
			}
		}

		/// <inheritdoc/>
		public override async Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, ExecuteJobTask task, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken)
		{
			await base.OnLeaseFinishedAsync(agent, leaseId, task, outcome, output, logger, cancellationToken);

			if (outcome != LeaseOutcome.Success)
			{
				AgentId agentId = agent.Id;
				JobId jobId = JobId.Parse(task.JobId);
				JobStepBatchId batchId = JobStepBatchId.Parse(task.BatchId);

				// Update the batch
				for (; ; )
				{
					IJob? job = await _jobs.GetAsync(jobId, cancellationToken);
					if (job == null)
					{
						break;
					}

					int batchIdx = job.Batches.FindIndex(x => x.Id == batchId);
					if (batchIdx == -1)
					{
						break;
					}

					IJobStepBatch batch = job.Batches[batchIdx];
					if (batch.AgentId != agentId)
					{
						break;
					}

					int runningStepIdx = batch.Steps.FindIndex(x => x.State == JobStepState.Running);

					JobStepBatchError error;
					if (outcome == LeaseOutcome.Cancelled)
					{
						error = JobStepBatchError.Cancelled;
					}
					else
					{
						error = JobStepBatchError.ExecutionError;
					}

					IGraph graph = await _graphs.GetAsync(job.GraphHash, cancellationToken);
					job = await job.TryFailBatchAsync(batchIdx, error, cancellationToken);

					if (job != null)
					{
						IJobStepBatch? newBatch;
						if (job.TryGetBatch(batch.Id, out newBatch))
						{
							batch = newBatch;
						}
						else
						{
							logger.LogInformation("New job is missing failed batch {JobId}:{BatchId}", job.Id, batch.Id);
						}

						if (batch.Error != JobStepBatchError.None)
						{
							logger.LogInformation("Failed lease {LeaseId}, job {JobId}, batch {BatchId} with error {Error}", leaseId, job.Id, batch.Id, batch.Error);
						}
						if (runningStepIdx != -1)
						{
							await _jobStepRefs.UpdateAsync(job, batch, batch.Steps[runningStepIdx], graph, logger);
							await _bisectTasks.UpdateAsync(job, batch, batch.Steps[runningStepIdx], graph, logger, cancellationToken);
						}
						break;
					}
				}
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="jobId"></param>
		/// <param name="batchId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		async Task CancelLeaseAsync(IAgent agent, JobId jobId, JobStepBatchId batchId, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Cancelling lease for job {JobId}, batch {BatchId}", jobId, batchId);

			// Update the batch
			for (; ; )
			{
				IJob? job = await _jobs.GetAsync(jobId, cancellationToken);
				if (job == null)
				{
					break;
				}

				int batchIdx = job.Batches.FindIndex(x => x.Id == batchId);
				if (batchIdx == -1)
				{
					break;
				}

				IJobStepBatch batch = job.Batches[batchIdx];
				if (batch.AgentId != agent.Id)
				{
					break;
				}

				IJob? newJob = await job.TryCancelLeaseAsync(batchIdx, cancellationToken);
				if (newJob != null)
				{
					break;
				}
			}
		}

		/// <summary>
		/// Inserts an item into the queue
		/// </summary>
		/// <param name="job"></param>
		/// <param name="batchIdx"></param>
		/// <param name="poolId">The pool to use</param>
		/// <param name="workspace">The workspace for this item to run in</param>
		/// <param name="useAutoSdk"></param>
		/// <returns></returns>
		void InsertQueueItem(IJob job, int batchIdx, PoolId poolId, AgentWorkspaceInfo workspace, bool useAutoSdk)
		{
			_logger.LogDebug("Adding queued job {JobId}, batch {BatchId} [Pool: {Pool}, Workspace: {Workspace}, AutoSdk: {AutoSdk}]", job.Id, job.Batches[batchIdx].Id, poolId, workspace.Identifier, useAutoSdk);

			QueueItem newItem = new QueueItem(job, batchIdx, poolId, workspace, useAutoSdk);
			_batchIdToQueueItem[newItem.Id] = newItem;
			_queue.Add(newItem);

			AssignQueueItemToAnyWaiter(newItem);
		}

		/// <summary>
		/// Removes an item from the queue
		/// </summary>
		/// <param name="item">Item to remove</param>
		void RemoveQueueItem(QueueItem item)
		{
			_logger.LogDebug("Removing queued job {JobId}, batch {BatchId}", item._job.Id, item.Batch.Id);

			_queue.Remove(item);
			_batchIdToQueueItem.Remove(item.Id);

			item._removed = true;
		}
	}
}
