// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using TelemetryEventAttribute = EpicGames.Analytics.Telemetry.TelemetryEventAttribute;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using HordeServer.Telemetry;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Record used for Job Summary telemetry. Models a <see cref="IJob"/>.
	/// </summary>
	[AnalyticsTableGen]
	[Table("horde.state_job_summary", Schema = "ingest")]
	[TelemetryEvent(DefaultEventName)]
	public record JobSummaryTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The job Id.
		/// </summary>
		[Column("job_id")]
		public JobId JobId { get; init; }

		/// <summary>
		/// The template of the job.
		/// </summary>
		[Column("template_id")]
		public TemplateId TemplateId { get; init; }

		/// <summary>
		/// The stream of the job.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The commit of the job. <see cref="IJob.CommitId"/>.
		/// </summary>
		[Column("commit_id")]
		public string CommitId { get; init; }

		/// <summary>
		/// The code commit of the job. <see cref="IJob.CodeCommitId"/>.
		/// </summary>
		[Column("code_commit_id")]
		public string CodeCommitId { get; init; }

		/// <summary>
		/// The created time of the job.
		/// </summary>
		[Column("create_time_utc")]
		public DateTime CreateTimeUtc { get; init; }

		/// <summary>
		/// The finish time of the job.
		/// </summary>
		[Column("finish_time_utc")]
		public DateTime FinishTimeUtc { get; init; }

		/// <summary>
		/// The summed duration of all steps.
		/// </summary>
		[Column("job_steps_total_time")]
		public float JobStepsTotalTime { get; init; }

		/// <summary>
		/// The wall time of the job, in seconds.
		/// </summary>
		[Column("job_wall_time")]
		public float JobWallTime { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.PassRatio"/>.
		/// </summary>
		[Column("pass_ratio")]
		public float PassRatio { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.PassWithWarningRatio"/>.
		/// </summary>
		[Column("pass_with_warning_ratio")]
		public float PassWithWarningRatio { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.WarningRatio"/>.
		/// </summary>
		[Column("warning_ratio")]
		public float WarningRatio { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.FailureRatio"/>.
		/// </summary>
		[Column("failure_ratio")]
		public float FailureRatio { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.StepPassCount"/>.
		/// </summary>
		[Column("step_pass_count")]
		public int StepPassCount { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.StepWarningCount"/>.
		/// </summary>
		[Column("step_warning_count")]
		public int StepWarningCount { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.StepFailureCount"/>.
		/// </summary>
		[Column("step_failure_count")]
		public int StepFailureCount { get; init; }

		/// <summary>
		/// <see cref="JobStepsCompletionInfo.StepTotalCount"/>.
		/// </summary>
		[Column("step_total_count")]
		public int StepTotalCount { get; init; }

		/// <summary>
		/// Whether the job was a preflight or not.
		/// </summary>
		[Column("is_preflight")]
		public bool IsPreflight { get; init; }

		/// <summary>
		/// Default event name for the JobSummaryTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.JobSummary";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 2;

		/// <summary>
		/// Constructs a JobSummary Telemetry event.
		/// </summary>
		/// <param name="job">The job to base this telemetry event on.</param>
		/// <param name="finishTimeUtc">The completion time of this job.</param>
		public JobSummaryTelemetry(IJob job, DateTime finishTimeUtc) : base(DefaultEventName, CurrentSchemaVersion)
		{
			JobId = job.Id;
			TemplateId = job.TemplateId;
			StreamId = job.StreamId;
			CommitId = job.CommitId?.Name ?? "0";
			CodeCommitId = job.CodeCommitId?.Name ?? "0";
			CreateTimeUtc = job.CreateTimeUtc;
			FinishTimeUtc = finishTimeUtc;
			JobWallTime = (float)(FinishTimeUtc - CreateTimeUtc).TotalSeconds;
			IsPreflight = job.PreflightCommitId != null;

			JobStepsCompletionInfo jobStepsCompletionInfo = job.GetStepCompletionInfo();

			StepTotalCount = jobStepsCompletionInfo.StepTotalCount;
			StepPassCount = jobStepsCompletionInfo.StepPassCount;
			StepWarningCount = jobStepsCompletionInfo.StepWarningCount;
			StepFailureCount = jobStepsCompletionInfo.StepFailureCount;
			JobStepsTotalTime = jobStepsCompletionInfo.JobStepsTotalTime;
			PassRatio = jobStepsCompletionInfo.PassRatio;
			PassWithWarningRatio = jobStepsCompletionInfo.PassWithWarningRatio;
			WarningRatio = jobStepsCompletionInfo.WarningRatio;
			FailureRatio = jobStepsCompletionInfo.FailureRatio;
		}

		/// <summary>
		/// Default constructor for an empty telemetry object,
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public JobSummaryTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			CodeCommitId = String.Empty;
			JobId = JobId.Empty;
			TemplateId = new TemplateId(String.Empty);
			StreamId = new StreamId(String.Empty);
			CommitId = String.Empty;
			CodeCommitId = String.Empty;
			CreateTimeUtc = DateTime.UtcNow;
			FinishTimeUtc = DateTime.UtcNow;
			JobWallTime = 0;
			IsPreflight = false;
			StepTotalCount = 0;
			StepPassCount = 0;
			StepWarningCount = 0;
			StepFailureCount = 0;
			JobStepsTotalTime = 0;
			PassRatio = 0;
			PassWithWarningRatio = 0;
			WarningRatio = 0;
			FailureRatio = 0;
		}
	}

	/// <summary>
	/// Helpers associated with producing <see cref="JobLabelTelemetry"/> and <see cref="JobLabelSummaryTelemetry"/>.
	/// </summary>
	public static class JobLabelTelemetryHelpers
	{
		/// <summary>
		/// Delimiter used in separating the Label Category and the Label Name.
		/// </summary>
		public const string FullyQualifiedLabelDelimiter = "-";

		/// <summary>
		/// Creates a fully qualified label name.
		/// </summary>
		/// <param name="response">The label response to convert to a fully qualified label name.</param>
		/// <returns>The fully qualified label name.</returns>
		/// <remarks>If there is no valid category, the label name will simply be the <see cref="GetLabelStateResponse.DashboardName"/>. If no valid dashboard name, the function will return <see cref="String.Empty"/>.</remarks>
		public static string CreateFullyQualifiedLabel(GetLabelStateResponse response)
		{
			return CreateFullyQualifiedLabel(response.DashboardCategory, response.DashboardName);
		}

		/// <summary>
		/// Creates a fully qualified label name.
		/// </summary>
		/// <param name="dashboardCategory">The category of the label.</param>
		/// <param name="dashboardName">The name of the label.</param>
		/// <returns>The fully qualified label name.</returns>
		/// <remarks>If there is no valid category, the label name will simply be the <see cref="GetLabelStateResponse.DashboardName"/>. If no valid dashboard name, the function will return <see cref="String.Empty"/>.</remarks>
		public static string CreateFullyQualifiedLabel(string? dashboardCategory, string? dashboardName)
		{
			return (dashboardCategory != null && dashboardName != null) ? $"{dashboardCategory}{FullyQualifiedLabelDelimiter}{dashboardName}" : (dashboardName != null ? dashboardName : String.Empty);
		}

		/// <summary>
		/// Creates a list of job label telemetry events for a given job and graph.
		/// </summary>
		/// <param name="job">The job to base the job label telemetry events on.</param>
		/// <param name="graph">The graph to base the job label telemetry events on.</param>
		/// <returns>A list of resulting job label telemetry events.</returns>
		public static IList<JobLabelTelemetry> CreateJobLabelTelemetryEvents(IJob job, IGraph graph)
		{
			List<GetLabelStateResponse> responses = [];
			job.GetLabelStateResponses(graph, responses);

			IList<JobLabelTelemetry> telemetryEntries = new List<JobLabelTelemetry>(responses.Count);

			for (int i = 0; i < responses.Count; ++i)
			{
				GetLabelStateResponse response = responses[i];
				telemetryEntries.Add(new JobLabelTelemetry(job, graph, response.DashboardCategory, response.DashboardName, response.Outcome, response.State));
			}

			return telemetryEntries;
		}

		/// <summary>
		/// Emits all job label telemetry for a given job and graph.
		/// </summary>
		/// <param name="telemetryWriter">The telemetry writer interface to use for emission.</param>
		/// <param name="telemetryStoreId">The telemetry store to use for emission.</param>
		/// <param name="job">The job to base the job label telemetry events on.</param>
		/// <param name="graph">The graph to base the job label telemetry events on.</param>
		public static void EmitJobLabelTelemetry(ITelemetryWriter telemetryWriter, TelemetryStoreId telemetryStoreId, IJob job, IGraph graph)
		{
			telemetryWriter.WriteEvent(telemetryStoreId, new JobLabelSummaryTelemetry(job, graph));
			IList<JobLabelTelemetry> jobLabelTelemetries = CreateJobLabelTelemetryEvents(job, graph);

			for (int i = 0; i < jobLabelTelemetries.Count; ++i)
			{
				telemetryWriter.WriteEvent(telemetryStoreId, jobLabelTelemetries[i]);
			}
		}
	}

	/// <summary>
	/// Record used for Job Label Summary telemetry. Models a <see cref="IJob"/> and <see cref="IGraph"/>.
	/// </summary>
	[AnalyticsTableGen]
	[Table("horde.state_job_label_summary", Schema = "ingest")]
	[TelemetryEvent(DefaultEventName)]
	public record JobLabelSummaryTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The job Id.
		/// </summary>
		[Column("job_id")]
		public JobId JobId { get; init; }

		/// <summary>
		/// The graph content hash.
		/// </summary>
		[Column("graph_content_hash")]
		public string GraphContentHash { get; init; }

		/// <summary>
		/// The template of the job.
		/// </summary>
		[Column("template_id")]
		public TemplateId TemplateId { get; init; }

		/// <summary>
		/// The stream of the job.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The commit of the job. <see cref="IJob.CommitId"/>.
		/// </summary>
		[Column("commit_id")]
		public string CommitId { get; init; }

		/// <summary>
		/// Array representing all labels that were successful in the job.
		/// </summary>
		[Column("successful_labels")]
		public string?[] SuccessfulLabels { get; init; }

		/// <summary>
		/// Array representing all labels that were warnings in the job.
		/// </summary>
		[Column("warning_labels")]
		public string?[] WarningLabels { get; init; }

		/// <summary>
		/// Array representing all labels that were failures in the job.
		/// </summary>
		[Column("failed_labels")]
		public string?[] FailedLabels { get; init; }

		/// <summary>
		/// Array representing all categories that were successful in the job.
		/// </summary>
		[Column("successful_categories")]
		public string?[] SuccessfulCategories { get; init; }

		/// <summary>
		/// Array representing all categories that were warnings in the job.
		/// </summary>
		[Column("warning_categories")]
		public string?[] WarningCategories { get; init; }

		/// <summary>
		/// Array representing all categories that were failures in the job.
		/// </summary>
		[Column("failed_categories")]
		public string?[] FailedCategories { get; init; }

		/// <summary>
		/// Default event name for the JobLabelSummaryTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.JobLabelSummary";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public JobLabelSummaryTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			JobId = JobId.Empty;
			GraphContentHash = String.Empty;
			TemplateId = new TemplateId(String.Empty);
			StreamId = new StreamId(String.Empty);
			CommitId = String.Empty;
			SuccessfulLabels = [];
			WarningLabels = [];
			FailedLabels = [];
			SuccessfulCategories = [];
			WarningCategories = [];
			FailedCategories = [];
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="job">The job to base the telemetry on.</param>
		/// <param name="graph">The graph to base the telemetry on.</param>
		public JobLabelSummaryTelemetry(IJob job, IGraph graph) : base(DefaultEventName, CurrentSchemaVersion)
		{
			SuccessfulLabels = [];
			WarningLabels = [];
			FailedLabels = [];
			EventName = DefaultEventName;
			JobId = job.Id;
			GraphContentHash = graph.Id.ToString();
			TemplateId = job.TemplateId;
			StreamId = job.StreamId;
			CommitId = job.CommitId?.Name ?? "0";

			List<GetLabelStateResponse> responses = [];
			job.GetLabelStateResponses(graph, responses);

			SuccessfulLabels = [.. responses.Where(r => r.State == LabelState.Complete && r.Outcome == LabelOutcome.Success && r.DashboardName != null).Select(r => JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(r))];
			WarningLabels = [.. responses.Where(r => r.State == LabelState.Complete && r.Outcome == LabelOutcome.Warnings && r.DashboardName != null).Select(r => JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(r))];
			FailedLabels = [.. responses.Where(r => r.State == LabelState.Complete && r.Outcome == LabelOutcome.Failure && r.DashboardName != null).Select(r => JobLabelTelemetryHelpers.CreateFullyQualifiedLabel(r))];

			Dictionary<string, LabelOutcome> categoryToOutcome = [];

			for (int i = 0; i < responses.Count; ++i)
			{
				GetLabelStateResponse response = responses[i];
				if (response.DashboardCategory == null || response.Outcome == null || response.State != LabelState.Complete)
				{
					continue;
				}

				if (!categoryToOutcome.ContainsKey(response.DashboardCategory))
				{
					categoryToOutcome.Add(response.DashboardCategory, response.Outcome.Value);
				}
				else
				{
					if (categoryToOutcome[response.DashboardCategory] > response.Outcome.Value)
					{
						categoryToOutcome[response.DashboardCategory] = response.Outcome.Value;
					}
				}
			}

			List<KeyValuePair<string, LabelOutcome>> flattenedCategoryList = categoryToOutcome.ToList();

			SuccessfulCategories = [.. flattenedCategoryList.Where(r => r.Value == LabelOutcome.Success).Select(r => r.Key)];
			WarningCategories = [.. flattenedCategoryList.Where(r => r.Value == LabelOutcome.Warnings).Select(r => r.Key)];
			FailedCategories = [.. flattenedCategoryList.Where(r => r.Value == LabelOutcome.Failure).Select(r => r.Key)];
		}
	}

	/// <summary>
	/// Record used for Job Label telemetry. Models a <see cref="IJob"/> and <see cref="IGraph"/>.
	/// </summary>
	[AnalyticsTableGen]
	[Table("horde.state_job_label", Schema = "ingest")]
	[TelemetryEvent(DefaultEventName)]
	public record JobLabelTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The job Id.
		/// </summary>
		[Column("job_id")]
		public JobId JobId { get; init; }

		/// <summary>
		/// The graph content hash.
		/// </summary>
		[Column("graph_content_hash")]
		public string GraphContentHash { get; init; }

		/// <summary>
		/// The template of the job.
		/// </summary>
		[Column("template_id")]
		public TemplateId TemplateId { get; init; }

		/// <summary>
		/// The stream of the job.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The commit of the job. <see cref="IJob.CommitId"/>.
		/// </summary>
		[Column("commit_id")]
		public string CommitId { get; init; }

		/// <summary>
		/// The category of the label.
		/// </summary>
		[Column("category")]
		public string? Category { get; init; }

		/// <summary>
		/// The label.
		/// </summary>
		[Column("label")]
		public string? Label { get; init; }

		/// <summary>
		/// The outcome of the label.
		/// </summary>
		[Column("outcome")]
		public LabelOutcome? Outcome { get; init; }

		/// <summary>
		/// The state of the label.
		/// </summary>
		[Column("state")]
		public LabelState? State { get; init; }

		/// <summary>
		/// Default event name for the JobLabelTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.JobLabel";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public JobLabelTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			JobId = JobId.Empty;
			GraphContentHash = String.Empty;
			TemplateId = new TemplateId(String.Empty);
			StreamId = new StreamId(String.Empty);
			CommitId = String.Empty;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="job">The job to base the job label telemetry events on.</param>
		/// <param name="graph">The graph to base the job label telemetry events on.</param>
		/// <param name="category">The category of the label.</param>
		/// <param name="name">The name of the label.</param>
		/// <param name="outcome">The outcome of the label.</param>
		/// <param name="state">The state of the label.</param>
		public JobLabelTelemetry(IJob job, IGraph graph, string? category, string? name, LabelOutcome? outcome, LabelState? state) : base(DefaultEventName, CurrentSchemaVersion)
		{
			GraphContentHash = graph.Id.ToString();
			StreamId = job.StreamId;
			TemplateId = job.TemplateId;
			JobId = job.Id;
			CommitId = job.CommitId?.Name ?? "0";
			Category = category;
			Label = name;
			Outcome = outcome;
			State = state;
		}
	}

	/// <summary>
	/// Record used for Job Step Ref telemetry. Models a Job Step Ref.
	/// </summary>
	[AnalyticsTableGen]
	[Table("horde.state_job_step_ref", Schema = "ingest")]
	[TelemetryEvent(DefaultEventName)]
	public record JobStepRefTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The job step ref id.
		/// </summary>
		[Column("id")]
		public string Id { get; init; }

		/// <summary>
		/// The job id.
		/// </summary>
		[Column("job_id")]
		public string JobId { get; init; }

		/// <summary>
		/// The batch id.
		/// </summary>
		[Column("batch_id")]
		public string BatchId { get; init; }

		/// <summary>
		/// The step id.
		/// </summary>
		[Column("step_id")]
		public string StepId { get; init; }

		/// <summary>
		/// The init time of the batch.
		/// </summary>
		[Column("batch_init_time")]
		public float BatchInitTime { get; init; }

		/// <summary>
		/// The wait time of the batch.
		/// </summary>
		[Column("batch_wait_time")]
		public float BatchWaitTime { get; init; }

		/// <summary>
		/// The commit id.
		/// </summary>
		[Column("change")]
		public string? Change { get; init; }

		/// <summary>
		/// The job name.
		/// </summary>
		[Column("job_name")]
		public string JobName { get; init; }

		/// <summary>
		/// The step finish time.
		/// </summary>
		[Column("finish_time")]
		public DateTime? FinishTime { get; init; }

		/// <summary>
		/// The owning job's start time.
		/// </summary>
		[Column("job_start_time")]
		public DateTime? JobStartTime { get; init; }

		/// <summary>
		/// The step name.
		/// </summary>
		[Column("step_name")]
		public string StepName { get; init; }

		/// <summary>
		/// The step state.
		/// </summary>
		[Column("state")]
		public JobStepState? State { get; init; }

		/// <summary>
		/// The step outcome.
		/// </summary>
		[Column("outcome")]
		public JobStepOutcome? Outcome { get; init; }

		/// <summary>
		/// The pool id of the associated agent.
		/// </summary>
		[Column("pool_id")]
		public PoolId? PoolId { get; init; }

		/// <summary>
		/// The start time of the step.
		/// </summary>
		[Column("start_time")]
		public DateTime? StartTime { get; init; }

		/// <summary>
		/// The stream id.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The template id of the owning job.
		/// </summary>
		[Column("template_id")]
		public TemplateId TemplateId { get; init; }

		/// <summary>
		/// The id of the agent that ran the step.
		/// </summary>
		[Column("agent_id")]
		public AgentId? AgentId { get; init; }

		/// <summary>
		/// Whether issues have been updated or not.
		/// </summary>
		[Column("update_issues")]
		public bool UpdateIssues { get; init; }

		/// <summary>
		/// The duration of the step.
		/// </summary>
		[Column("duration")]
		public double? Duration { get; init; }

		/// <summary>
		/// The message from the step report, if any.
		/// </summary>
		[Column("message")]
		public string? Message { get; init; }

		/// <summary>
		/// The severity from the step report message, if any.
		/// </summary>
		[Column("message_severity")]
		public string? MessageSeverity { get; init; }

		/// <summary>
		/// The default event name.
		/// </summary>
		public const string DefaultEventName = "State.JobStepRef";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 2;

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public JobStepRefTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			Id = String.Empty;
			JobId = String.Empty;
			BatchId = String.Empty;
			StepId = String.Empty;
			BatchInitTime = 0;
			BatchWaitTime = 0;
			JobName = String.Empty;
			StepName = String.Empty;
			StreamId = new StreamId(String.Empty);
			TemplateId = new TemplateId(String.Empty);
			UpdateIssues = false;
		}

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="id">The job step ref id.</param>
		/// <param name="jobId">The job id.</param>
		/// <param name="batchId">The batch id.</param>
		/// <param name="stepId">The step id.</param>
		/// <param name="agentId">The id of the agent that ran the step.</param>
		/// <param name="batchInitTime">The init time of the batch.</param>
		/// <param name="batchWaitTime">The wait time of the batch.</param>
		/// <param name="change">The commit id.</param>
		/// <param name="finishTime">The step finish time.</param>
		/// <param name="jobName">The job name.</param>
		/// <param name="jobStartTime">The owning job's start time.</param>
		/// <param name="stepName">The step name.</param>
		/// <param name="state">The step state.</param>
		/// <param name="outcome">The step outcome.</param>
		/// <param name="poolId">The pool id of the associated agent.</param>
		/// <param name="startTime">The start time of the step.</param>
		/// <param name="streamId">The stream id.</param>
		/// <param name="templateId">The template id of the owning job.</param>
		/// <param name="updateIssues">Whether issues have been updated or not.</param>
		/// <param name="duration">The duration of the step.</param>
		/// <param name="message">The message from the step report, if any.</param>
		/// <param name="messageSeverity">The severity from the step report message, if any.</param>
		public JobStepRefTelemetry(string id, string jobId, string batchId, string stepId, AgentId? agentId, float batchInitTime, float batchWaitTime, string? change, DateTime? finishTime, string jobName, DateTime? jobStartTime, string stepName, JobStepState? state, JobStepOutcome? outcome, PoolId? poolId, DateTime? startTime, StreamId streamId, TemplateId templateId, bool updateIssues, double? duration, string? message = null, string? messageSeverity = null) : base(DefaultEventName, CurrentSchemaVersion)
		{
			Id = id;
			JobId = jobId;
			BatchId = batchId;
			StepId = stepId;
			BatchInitTime = batchInitTime;
			BatchWaitTime = batchWaitTime;
			Change = change;
			FinishTime = finishTime;
			JobName = jobName;
			JobStartTime = jobStartTime;
			StepName = stepName;
			State = state;
			Outcome = outcome;
			PoolId = poolId;
			StartTime = startTime;
			StreamId = streamId;
			TemplateId = templateId;
			AgentId = agentId;
			UpdateIssues = updateIssues;
			Duration = duration;
			Message = message;
			MessageSeverity = messageSeverity;
		}

		/// <summary>
		/// Create method used to construct a JobStepRef telemetry event from a job, the provided batch index, and step index. This will take a new outcome and state to use in the telemetry emission.
		/// </summary>
		/// <param name="job">The job to use in constructing the telemetry event.</param>
		/// <param name="batchIdx">The index of the job batch (into the job's batches) to use in constructing the telemetry event.</param>
		/// <param name="stepIdx">The index of the job step (into the batch's steps) to use in constructing the telemetry event.</param>
		/// <param name="newOutcome">The job step ref telemetry event's step outcome.</param>
		/// <param name="newStepState">The job step ref telemetry event's step state.</param>
		/// <returns>The JobStepRefTelemetry event, null if unable to construct due to invalid parameters.</returns>
		/// <remarks>Should only be used until full state replication is completed in UE-357021.</remarks>
		public static JobStepRefTelemetry? ConstructJobStepRefTelemetryFromJob(IJob? job, int batchIdx, int stepIdx, JobStepOutcome newOutcome, JobStepState newStepState)
		{
			if (job == null)
			{
				return null;
			}

			if (batchIdx < 0 || job.Batches.Count <= batchIdx)
			{
				return null;
			}

			IJobStepBatch batch = job.Batches[batchIdx];

			if (stepIdx < 0 || batch.Steps.Count <= stepIdx)
			{
				return null;
			}

			IJobStep step = batch.Steps[stepIdx];
			JobStepRefId jobStepRefId = new JobStepRefId(job.Id, batch.Id, step.Id);
			float waitTime = (float)(batch.GetWaitTime() ?? TimeSpan.Zero).TotalSeconds;
			float initTime = (float)(batch.GetInitTime() ?? TimeSpan.Zero).TotalSeconds;

			JobStepRefTelemetry telemetry = new JobStepRefTelemetry(
				jobStepRefId.ToString(),
				job.Id.ToString(),
				batch.Id.ToString(),
				step.Id.ToString(),
				batch.AgentId,
				initTime,
				waitTime,
				job.CommitId?.Name ?? "0",
				step.FinishTimeUtc,
				job.Name,
				job.CreateTimeUtc,
				step.Name,
				newStepState,
				newOutcome,
				batch.PoolId,
				step.StartTimeUtc,
				job.StreamId,
				job.TemplateId,
				job.UpdateIssues,
				(step.FinishTimeUtc != null && step.StartTimeUtc != null) ? (double)(step.FinishTimeUtc.Value - step.StartTimeUtc.Value).TotalSeconds : null);

			return telemetry;
		}
	}

	/// <summary>
	/// Record used for Job Created telemetry. Emitted when a new job is created.
	/// </summary>
	[AnalyticsTableGen]
	[Table("horde.state_job", Schema = "ingest")]
	[TelemetryEvent(DefaultEventName)]
	public record JobCreatedTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// The job Id.
		/// </summary>
		[Column("job_id")]
		public JobId JobId { get; init; }

		/// <summary>
		/// The stream of the job.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The arguments passed to the job.
		/// </summary>
		[Column("args")]
		public string[] Args { get; init; }

		/// <summary>
		/// Whether the job should auto-submit on completion.
		/// </summary>
		[Column("auto_submit")]
		public bool? AutoSubmit { get; init; }

		/// <summary>
		/// The commit/change for this job.
		/// </summary>
		[Column("change")]
		public string Change { get; init; }

		/// <summary>
		/// The code commit/change for this job.
		/// </summary>
		[Column("code_change")]
		public string? CodeChange { get; init; }

		/// <summary>
		/// The created time of the job.
		/// </summary>
		[Column("create_time_utc")]
		public DateTime CreateTimeUtc { get; init; }

		/// <summary>
		/// The graph hash for this job.
		/// </summary>
		[Column("graph_hash")]
		public string GraphHash { get; init; }

		/// <summary>
		/// The name of the job.
		/// </summary>
		[Column("name")]
		public string Name { get; init; }

		/// <summary>
		/// The preflight commit/change, if this is a preflight job.
		/// </summary>
		[Column("preflight_change")]
		public string? PreflightChange { get; init; }

		/// <summary>
		/// Description of the preflight, if this is a preflight job.
		/// </summary>
		[Column("preflight_description")]
		public string? PreflightDescription { get; init; }

		/// <summary>
		/// The priority of the job.
		/// </summary>
		[Column("priority")]
		public Priority Priority { get; init; }

		/// <summary>
		/// The user who started this job.
		/// </summary>
		[Column("started_by_user_id")]
		public string? StartedByUserId { get; init; }

		/// <summary>
		/// The template of the job.
		/// </summary>
		[Column("template_id")]
		public TemplateId TemplateId { get; init; }

		/// <summary>
		/// Default event name for the JobCreatedTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.Job";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 1;

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public JobCreatedTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			JobId = JobId.Empty;
			StreamId = new StreamId(String.Empty);
			Args = Array.Empty<string>();
			Change = String.Empty;
			GraphHash = String.Empty;
			Name = String.Empty;
			TemplateId = new TemplateId(String.Empty);
		}

		/// <summary>
		/// Constructs a JobCreatedTelemetry event from a job.
		/// </summary>
		/// <param name="job">The job to base this telemetry event on.</param>
		public JobCreatedTelemetry(IJob job) : base(DefaultEventName, CurrentSchemaVersion)
		{
			JobId = job.Id;
			StreamId = job.StreamId;
			Args = job.Arguments.ToArray();
			AutoSubmit = job.AutoSubmit;
			Change = job.CommitId?.Name ?? String.Empty;
			CodeChange = job.CodeCommitId?.Name;
			CreateTimeUtc = job.CreateTimeUtc;
			GraphHash = job.GraphHash.ToString();
			Name = job.Name;
			PreflightChange = job.PreflightCommitId?.Name;
			PreflightDescription = job.PreflightDescription;
			Priority = job.Priority;
			StartedByUserId = job.StartedByUserId?.ToString();
			TemplateId = job.TemplateId;
		}
	}

	/// <summary>
	/// Keys used to store job report data in step metadata.
	/// </summary>
	static class JobReportMetadataKeys
	{
		/// <summary>
		/// Key for the report message.
		/// </summary>
		public const string MessageKey = "ReportStepMessage";

		/// <summary>
		/// Key for the report severity.
		/// </summary>
		public const string MessageSeverityKey = "ReportStepSeverity";

		/// <summary>
		/// Key for the report color.
		/// </summary>
		public const string ColorKey = "ReportStepColor";
	}
}