// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.Timing;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Acls;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Commits;
using HordeServer.Issues;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Templates;
using HordeServer.Notifications;
using HordeServer.Logs;
using HordeServer.RoboMerge;
using HordeServer.VersionControl.Perforce;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Jobs
{
	/// <summary>
	/// Controller for the /api/v1/jobs endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class JobsController : HordeControllerBase
	{
		private readonly IGraphCollection _graphs;
		private readonly ICommitService _commitService;
		private readonly IPerforceService _perforce;
		private readonly JobService _jobService;
		private readonly ITemplateCollectionInternal _templateCollection;
		private readonly IArtifactCollection _artifactCollection;
		private readonly IUserCollection _userCollection;
		private readonly INotificationService _notificationService;
		private readonly AgentService _agentService;
		private readonly ILeaseCollection _leaseCollection;
		private readonly IIssueCollection _issueCollection;
		private readonly IMergeGraphService _mergeGraphService;
		private readonly ILogCollection _logCollection;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly ILogger<JobsController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public JobsController(IGraphCollection graphs, ICommitService commitService, IPerforceService perforce, JobService jobService, ITemplateCollectionInternal templateCollection, IArtifactCollection artifactCollection, IUserCollection userCollection, INotificationService notificationService, AgentService agentService, ILeaseCollection leaseCollection, IIssueCollection issueCollection, IMergeGraphService mergeGraphService, ILogCollection logCollection, IOptionsSnapshot<BuildConfig> buildConfig, ILogger<JobsController> logger)
		{
			_graphs = graphs;
			_commitService = commitService;
			_perforce = perforce;
			_jobService = jobService;
			_templateCollection = templateCollection;
			_artifactCollection = artifactCollection;
			_userCollection = userCollection;
			_notificationService = notificationService;
			_agentService = agentService;
			_leaseCollection = leaseCollection;
			_issueCollection = issueCollection;
			_mergeGraphService = mergeGraphService;
			_logCollection = logCollection;
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <summary>
		/// Creates a new job
		/// </summary>
		/// <param name="create">Properties of the new job</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Id of the new job</returns>
		[HttpPost]
		[Route("/api/v1/jobs")]
		public async Task<ActionResult<CreateJobResponse>> CreateJobAsync([FromBody] CreateJobRequest create, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(create.StreamId, out streamConfig))
			{
				return NotFound(create.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.CreateJob, User))
			{
				return Forbid(JobAclAction.CreateJob, streamConfig.Id);
			}

			if (create.RunAsScheduler == true && !_buildConfig.Value.Authorize(ServerAclAction.Debug, User))
			{
				return Forbid(ServerAclAction.Debug);
			}

			// Get the name of the template ref
			TemplateId templateRefId = create.TemplateId;

			// Augment the request with template properties
			TemplateRefConfig? templateRefConfig;
			if (!streamConfig.TryGetTemplate(templateRefId, out templateRefConfig))
			{
				return BadRequest($"Template {create.TemplateId} is not available for stream {streamConfig.Id}");
			}
			if (!templateRefConfig.Authorize(JobAclAction.CreateJob, User))
			{
				return Forbid(JobAclAction.CreateJob, streamConfig.Id);
			}

			ITemplate? template = await _templateCollection.GetOrAddAsync(templateRefConfig);
			if (template == null)
			{
				return BadRequest("Missing template referenced by {TemplateId}", create.TemplateId);
			}
			if (!template.AllowPreflights && create.PreflightCommitId != null)
			{
				return BadRequest("Template {TemplateId} does not allow preflights", create.TemplateId);
			}

			// Get the name of the new job
			string name = create.Name ?? template.Name;
			if (create.TemplateId == new TemplateId("stage-to-marketplace") && create.Arguments != null)
			{
				foreach (string argument in create.Arguments)
				{
					const string Prefix = "-set:UserContentItems=";
					if (argument.StartsWith(Prefix, StringComparison.Ordinal))
					{
						name += $" - {argument.Substring(Prefix.Length)}";
						break;
					}
				}
			}

			// Environment variables for the job
			Dictionary<string, string> environment = new Dictionary<string, string>();

			// Check the preflight change is valid
			ShelfInfo? shelfInfo = null;

			if (create.PreflightCommitId != null)
			{
				int? preflightChange = create.PreflightCommitId.TryGetPerforceChange();
				if (preflightChange != null && String.Equals(streamConfig.VCS, "Perforce", StringComparison.OrdinalIgnoreCase))
				{
					// P4 path: full shelf validation
					(CheckShelfResult result, shelfInfo) = await _perforce.CheckShelfAsync(streamConfig, preflightChange.Value, HttpContext.RequestAborted);
					switch (result)
					{
						case CheckShelfResult.Ok:
							break;
						case CheckShelfResult.NoChange:
							return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} does not exist", preflightChange);
						case CheckShelfResult.NoShelvedFiles:
							return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} does not contain any shelved files", preflightChange);
						case CheckShelfResult.WrongStream:
							return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} does not contain files in {Stream}", preflightChange, streamConfig.Name);
						case CheckShelfResult.MixedStream:
							return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} contains files from multiple streams", preflightChange);
						default:
							return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} cannot be preflighted ({Result})", preflightChange, result);
					}

					if (shelfInfo!.Tags != null)
					{
						string tagList = String.Join(";", shelfInfo.Tags.Select(x => x.ToString()));
						environment.Add("UE_HORDE_PREFLIGHT_TAGS", tagList);
					}
				}
				else
				{
					// Hash-based VCS path: get commit info
					ICommitCollection commitCollection = _commitService.GetCollection(streamConfig);
					try
					{
						ICommit preflightCommit = await commitCollection.GetAsync(create.PreflightCommitId, cancellationToken);
						shelfInfo = new ShelfInfo(preflightCommit.Description, new List<CommitTag>());

						// Normalize to the canonical commit ID (revision number) for consistent storage and display
						create.PreflightCommitId = new CommitId(preflightCommit.Id.Name);
					}
					catch (CommitCollectionException)
					{
						return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "Commit {CommitId} does not exist", create.PreflightCommitId);
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Failed to look up preflight commit {CommitId}", create.PreflightCommitId);
						return StatusCode(502, $"Failed to look up preflight commit: {ex.Message}");
					}
				}
			}

			// Get the priority of the new job
			Priority priority = create.Priority ?? template.Priority ?? Priority.Normal;

			// New groups for the job
			IGraph graph = await _graphs.AddAsync(template, cancellationToken);

			// Get the commits for this stream
			ICommitCollection commits = _commitService.GetCollection(streamConfig);
			
			CommitIdWithOrder commitId;
			CommitIdWithOrder? codeCommitId;
			try
			{
				// Get the change to build
				commitId = await GetChangeToBuildAsync(create, streamConfig.Id, template, shelfInfo, commits, HttpContext.RequestAborted);
				
				// And get the matching code changelist
				ICommit? lastCodeCommit = await commits.GetLastCodeChangeAsync(commitId, HttpContext.RequestAborted);
				codeCommitId = lastCodeCommit?.Id;
			}
			catch (CommitCollectionException e)
			{
				// Prevent exceptions from bubbling up when VCS operations fail due to misconfigured streams or templates.
				// Log these at warning level since they're often configuration issues rather than true system failures.
				// While some failures may be due to user input or configuration, we conservatively return 500 to indicate service unavailability.
				_logger.LogWarning(e, "Error performing VCS operation(s): {Message}", e.Message);
				return StatusCode(StatusCodes.Status500InternalServerError, e.Message);
			}

			// New properties for the job
			bool? updateIssues = null;
			if (template.UpdateIssues)
			{
				updateIssues = true;
			}
			else if (create.UpdateIssues.HasValue)
			{
				updateIssues = create.UpdateIssues.Value;
			}

			// Create options for the new job
			CreateJobOptions options = new CreateJobOptions(templateRefConfig);
			options.PreflightCommitId = create.PreflightCommitId;
			options.PreflightDescription = shelfInfo?.Description;
			options.Priority = priority;
			options.AutoSubmit = create.AutoSubmit;
			options.UpdateIssues = updateIssues;
			options.ParentJobId = create.ParentJobId == null ? null : JobId.Parse(create.ParentJobId);
			options.ParentJobStepId = create.ParentJobStepId == null ? null : JobStepId.Parse(create.ParentJobStepId);
			if (!create.RunAsScheduler.HasValue || !create.RunAsScheduler.Value)
			{
				options.StartedByUserId = User.GetUserId();
				options.Claims.AddRange(User.Claims.Select(x => new AclClaimConfig(x)));
			}			
			options.JobOptions ??= create.JobOptions;

			if (create.AdditionalArguments != null)
			{
				options.AdditionalArguments.AddRange(create.AdditionalArguments);
			}

			if (create.Arguments != null && create.Arguments.Count > 0)
			{
				// Use the specific argument list specified in the request
				options.Arguments.AddRange(create.Arguments);

				// Make sure we're not trying to specify parameters as well
				if (create.Parameters != null && create.Parameters.Count > 0)
				{
					return BadRequest("Cannot specify argument list and parameter list at the same time.");
				}
			}
			else
			{
				// Find all the default parameters, and override any settings with the values in the request
				template.GetDefaultParameters(options.Parameters, false);

				if (create.Parameters != null)
				{
					foreach ((ParameterId parameter, string value) in create.Parameters)
					{
						options.Parameters[parameter] = value;
					}
				}

				// Build the final arguments list from the combined parameter set
				template.GetArgumentsForParameters(options.Parameters, options.Arguments);

				// Add the additional arguments
				options.Arguments.AddRange(options.AdditionalArguments);
			}

			// Override the targets for the job if specified
			if (create.Targets != null && create.Targets.Count > 0)
			{
				options.Targets = create.Targets.ToList();

				options.Arguments.RemoveAll(x => x.StartsWith("-Target=", StringComparison.OrdinalIgnoreCase));
				options.Arguments.AddRange(create.Targets.Select(x => $"-Target={x}"));
			}

			// Merge the environment variables
			foreach ((string key, string value) in environment)
			{
				options.Environment[key] = value;
			}

			// Create the job
			IJob job = await _jobService.CreateJobAsync(null, streamConfig, templateRefId, template.Hash, graph, name, commitId, codeCommitId, options, cancellationToken);
			await UpdateNotificationsAsync(job.Id, new UpdateNotificationsRequest { Slack = true }, cancellationToken);

			if (options.ParentJobId != null && options.ParentJobStepId != null)
			{
				IJob? parentJob = await _jobService.GetJobAsync(options.ParentJobId.Value, cancellationToken);
				if (parentJob != null)
				{
					IJobStep? parentStep;
					if (parentJob.TryGetStep(options.ParentJobStepId.Value, out parentStep))
					{
						await _jobService.UpdateStepAsync(parentJob, parentStep.Batch.Id, parentStep.Id, streamConfig, newSpawnedJob: job.Id, cancellationToken: cancellationToken);
					}					
				}				
			}

			return new CreateJobResponse(job.Id.ToString());
		}

		async ValueTask<CommitIdWithOrder> GetChangeToBuildAsync(CreateJobRequest create, StreamId streamId, ITemplate template, ShelfInfo? shelfInfo, ICommitCollection commits, CancellationToken cancellationToken)
		{
			// If there's an explicit change specified, use that
			if (create.CommitId != null)
			{
				return await _commitService.GetOrderedAsync(streamId, create.CommitId, cancellationToken);
			}

			// Evaluate the change queries
			if (create.ChangeQueries != null && create.ChangeQueries.Count > 0)
			{
				CommitIdWithOrder? commitId = await _jobService.EvaluateChangeQueriesAsync(streamId, create.ChangeQueries, shelfInfo?.Tags, commits, cancellationToken);
				if (commitId != null)
				{
					return commitId;
				}
			}

			// If we need to submit a new change, do that
			if (create.PreflightCommitId == null && template.SubmitNewChange != null)
			{
				return await commits.CreateNewAsync(template, cancellationToken);
			}

			// Otherwise return the latest change
			return await commits.GetLastCommitIdAsync(cancellationToken);
		}

		/// <summary>
		/// Deletes a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to delete</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		[HttpDelete]
		[Route("/api/v1/jobs/{jobId}")]
		public async Task<ActionResult> DeleteJobAsync(JobId jobId, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}
			if (!_buildConfig.Value.Authorize(job, JobAclAction.DeleteJob, User))
			{
				return Forbid(JobAclAction.DeleteJob, jobId);
			}
			if (!await _jobService.DeleteJobAsync(job, cancellationToken))
			{
				return NotFound(jobId);
			}
			return Ok();
		}

		/// <summary>
		/// Updates a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="request">Settings to update in the job</param>
		/// <returns>Async task</returns>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}")]
		public async Task<ActionResult> UpdateJobAsync(JobId jobId, [FromBody] UpdateJobRequest request)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.UpdateJob, User))
			{
				return Forbid(JobAclAction.UpdateJob, jobId);
			}

			// Convert legacy behavior of clearing out the argument to setting the aborted flag
			if (request.Arguments != null && request.Arguments.Count == 0)
			{
				request.Aborted = true;
				request.Arguments = null;
			}

			UserId? abortedByUserId = null;
			if (request.Aborted ?? false)
			{
				abortedByUserId = User.GetUserId();
			}

			IJob? newJob = await _jobService.UpdateJobAsync(job, name: request.Name, priority: request.Priority, autoSubmit: request.AutoSubmit, abortedByUserId: abortedByUserId, arguments: request.Arguments, cancellationReason: request.CancellationReason);
			if (newJob == null)
			{
				return NotFound(jobId);
			}
			return Ok();
		}

		/// <summary>
		/// Updates a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="request">Settings to update in the job</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}/metadata")]
		public async Task<ActionResult> UpdateJobMetaDataAsync(JobId jobId, [FromBody] PutJobMetadataRequest request, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}

			if (!streamConfig.Authorize(JobAclAction.UpdateJob, User))
			{
				if (job.StartedByUserId != null || !JobService.AuthorizeSession(job, User))
				{
					return Forbid(JobAclAction.UpdateJob, jobId);
				}					
			}

			Dictionary<JobStepId, List<string>>? stepMetaData = null;

			if (request.StepMetaData != null)
			{
				stepMetaData = new Dictionary<JobStepId, List<string>>();
				foreach(KeyValuePair<string, List<string>> entry in request.StepMetaData)
				{
					stepMetaData[JobStepId.Parse(entry.Key)] = entry.Value;
				}
			}

			await _jobService.UpdateMetadataAsync(jobId, request.JobMetaData, stepMetaData, cancellationToken);
			
			return Ok();
		}

		/// <summary>
		/// Updates notifications for a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="request">The notification request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}/notifications")]
		public async Task<ActionResult> UpdateNotificationsAsync(JobId jobId, [FromBody] UpdateNotificationsRequest request, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(NotificationAclAction.CreateSubscription, User))
			{
				return Forbid(NotificationAclAction.CreateSubscription, jobId);
			}

			NotificationTriggerId triggerId = job.NotificationTriggerId ?? new NotificationTriggerId(BinaryIdUtils.CreateNew());

			job = await _jobService.UpdateJobAsync(job, null, null, null, null, triggerId, null, null, cancellationToken: cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			await _notificationService.UpdateSubscriptionsAsync(triggerId, User, request.Email, request.Slack, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Gets information about a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/notifications")]
		public async Task<ActionResult<GetNotificationResponse>> GetNotificationsAsync(JobId jobId, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}

			if (!streamConfig.Authorize(NotificationAclAction.CreateSubscription, User))
			{
				return Forbid(NotificationAclAction.CreateSubscription, jobId);
			}

			INotificationSubscription? subscription;
			if (job.NotificationTriggerId == null)
			{
				subscription = null;
			}
			else
			{
				subscription = await _notificationService.GetSubscriptionsAsync(job.NotificationTriggerId.Value, User, cancellationToken);
			}
			return new GetNotificationResponse(subscription);
		}

		/// <summary>
		/// Gets information about a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="modifiedAfter">If specified, returns an empty response unless the job's update time is equal to or less than the given value</param>
		/// <param name="filter">Filter for the fields to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}")]
		[ProducesResponseType(typeof(GetJobResponse), 200)]
		public async Task<ActionResult<object>> GetJobAsync(JobId jobId, [FromQuery] DateTimeOffset? modifiedAfter = null, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}

			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				if (job.StartedByUserId != null || !JobService.AuthorizeSession(job, User))
				{
					return Forbid(JobAclAction.ViewJob, job.StreamId);
				}				
			}

			if (modifiedAfter != null && job.UpdateTimeUtc <= modifiedAfter.Value)
			{
				return new Dictionary<string, object>();
			}

			IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
			bool includeCosts = streamConfig.Authorize(ServerAclAction.ViewCosts, User);
			return await CreateJobResponseAsync(job, graph, includeCosts, filter, cancellationToken);
		}

		/// <summary>
		/// Creates a response containing information about this object
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="graph">The graph for this job</param>
		/// <param name="includeCosts">Whether to include costs in the response</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Object containing the requested properties</returns>
		async Task<object> CreateJobResponseAsync(IJob job, IGraph graph, bool includeCosts, PropertyFilter? filter, CancellationToken cancellationToken)
		{
			if (filter == null)
			{
				return await CreateJobResponseAsync(job, graph, true, true, includeCosts, true, cancellationToken);
			}
			else
			{
				return filter.ApplyTo(await CreateJobResponseAsync(job, graph, filter.Includes(nameof(GetJobResponse.Batches)), filter.Includes(nameof(GetJobResponse.Labels)) || filter.Includes(nameof(GetJobResponse.DefaultLabel)), includeCosts, filter.Includes(nameof(GetJobResponse.Artifacts)), cancellationToken));
			}
		}

		/// <summary>
		/// Creates a response containing information about this object
		/// </summary>
		/// <param name="job">The job document</param>
		/// <param name="graph">The graph definition</param>
		/// <param name="includeBatches">Whether to include the job batches in the response</param>
		/// <param name="includeLabels">Whether to include the job aggregates in the response</param>
		/// <param name="includeCosts">Whether to include costs of running particular agents</param>
		/// <param name="includeArtifacts">Whether to include artifacts in the response</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The response object</returns>
		async ValueTask<GetJobResponse> CreateJobResponseAsync(IJob job, IGraph graph, bool includeBatches, bool includeLabels, bool includeCosts, bool includeArtifacts, CancellationToken cancellationToken)
		{
			GetThinUserInfoResponse? startedByUserInfo = null;
			if (job.StartedByUserId != null)
			{
				startedByUserInfo = (await _userCollection.GetCachedUserAsync(job.StartedByUserId.Value, cancellationToken))?.ToThinApiResponse();
			}

			GetThinUserInfoResponse? abortedByUserInfo = null;
			if (job.AbortedByUserId != null)
			{
				abortedByUserInfo = (await _userCollection.GetCachedUserAsync(job.AbortedByUserId.Value, cancellationToken))?.ToThinApiResponse();
			}

			bool canUpdate = false;
			if (_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				canUpdate = streamConfig.Authorize(JobAclAction.UpdateJob, User);
			}

			GetJobResponse response = CreateGetJobResponse(job, startedByUserInfo, abortedByUserInfo, canUpdate);
			if (includeBatches || includeLabels)
			{
				if (includeBatches)
				{
					response.Batches = new List<GetJobBatchResponse>();
					foreach (IJobStepBatch batch in job.Batches)
					{
						response.Batches.Add(await CreateBatchResponseAsync(batch, includeCosts, cancellationToken));
					}
				}
				if (includeLabels)
				{
					response.Labels = new List<GetLabelStateResponse>();
					response.DefaultLabel = job.GetLabelStateResponses(graph, response.Labels);
				}
			}
			if (includeArtifacts)
			{
				response.Artifacts = new List<GetJobArtifactResponse>();

				HashSet<(ArtifactName, JobStepId)> addedArtifacts = new HashSet<(ArtifactName, JobStepId)>();

				string artifactKey = $"job:{job.Id}";
				string artifactStepKeyPrefix = $"job:{job.Id}/step:";
				await foreach (IArtifact artifact in _artifactCollection.FindAsync(keys: new[] { artifactKey }, maxResults: 5000, cancellationToken: cancellationToken))
				{
					if (IncludeArtifactInResponse(artifact))
					{
						string? stepKey = artifact.Keys.FirstOrDefault(x => x.StartsWith(artifactStepKeyPrefix, StringComparison.Ordinal));
						if (stepKey != null && JobStepId.TryParse(stepKey.Substring(artifactStepKeyPrefix.Length), out JobStepId jobStepId))
						{
							response.Artifacts.Add(new GetJobArtifactResponse(artifact.Id, artifact.Name, artifact.Type, artifact.Description, artifact.Keys.ToList(), artifact.Metadata.ToList(), jobStepId));
							addedArtifacts.Add((artifact.Name, jobStepId));
						}
					}
				}

				Dictionary<string, IGraphArtifact> nodeNameToArtifact = new Dictionary<string, IGraphArtifact>(StringComparer.OrdinalIgnoreCase);
				Dictionary<string, IGraphArtifact> outputNameToArtifact = new Dictionary<string, IGraphArtifact>(StringComparer.OrdinalIgnoreCase);
				foreach (IGraphArtifact artifact in graph.Artifacts)
				{
					if (artifact.NodeName != null)
					{
						nodeNameToArtifact[artifact.NodeName] = artifact;
					}
					if (artifact.OutputName != null)
					{
						outputNameToArtifact[artifact.OutputName] = artifact;
					}
				}

				foreach (IJobStepBatch batch in job.Batches)
				{
					INodeGroup group = graph.Groups[batch.GroupIdx];
					foreach (IJobStep step in batch.Steps)
					{
						INode node = group.Nodes[step.NodeIdx];

						IGraphArtifact? graphArtifact;
						if (nodeNameToArtifact.TryGetValue(node.Name, out graphArtifact) && !addedArtifacts.Contains((graphArtifact.Name, step.Id)))
						{
							response.Artifacts.Add(new GetJobArtifactResponse(null, graphArtifact.Name, graphArtifact.Type, graphArtifact.Description, graphArtifact.Keys.ToList(), graphArtifact.Metadata.ToList(), step.Id));
						}
						foreach (string outputName in node.OutputNames)
						{
							if (outputNameToArtifact.TryGetValue(outputName, out graphArtifact) && !addedArtifacts.Contains((graphArtifact.Name, step.Id)))
							{
								response.Artifacts.Add(new GetJobArtifactResponse(null, graphArtifact.Name, graphArtifact.Type, graphArtifact.Description, graphArtifact.Keys.ToList(), graphArtifact.Metadata.ToList(), step.Id));
							}
						}
					}
				}
			}
			return response;
		}

		static bool IncludeArtifactInResponse(IArtifact artifact)
		{
			return artifact.Type != ArtifactType.StepOutput
				&& artifact.Type != ArtifactType.StepSaved
				&& artifact.Type != ArtifactType.StepTrace
				&& artifact.Type != ArtifactType.StepTestData;
		}

		static GetJobResponse CreateGetJobResponse(IJob job, GetThinUserInfoResponse? startedByUserInfo, GetThinUserInfoResponse? abortedByUserInfo, bool canUpdate)
		{
			GetJobResponse response = new GetJobResponse(job.Id, job.StreamId, job.TemplateId, job.Name);
			response.CommitId = job.CommitId;
			response.CodeCommitId = job.CodeCommitId;
			response.PreflightCommitId = job.PreflightCommitId;
			response.PreflightDescription = job.PreflightDescription;
			response.TemplateHash = job.TemplateHash?.ToString() ?? String.Empty;
			response.GraphHash = job.GraphHash.ToString();
			response.StartedByUserId = job.StartedByUserId?.ToString();
			response.StartedByUser = startedByUserInfo?.Login;
			response.StartedByUserInfo = startedByUserInfo;
			response.StartedByBisectTaskId = job.StartedByBisectTaskId;
			response.AbortedByUser = abortedByUserInfo?.Login;
			response.AbortedByUserInfo = abortedByUserInfo;
			response.CancellationReason = job.CancellationReason;
			response.CreateTime = new DateTimeOffset(job.CreateTimeUtc);
			response.State = job.GetState();
			response.Priority = job.Priority;
			response.AutoSubmit = job.AutoSubmit;
			response.AutoSubmitChange = job.AutoSubmitChange;
			response.AutoSubmitMessage = job.AutoSubmitMessage;
			response.Reports = job.Reports?.ConvertAll(x => CreateGetReportResponse(x));
			response.Parameters = job.Parameters.ToDictionary();
			response.Arguments = job.Arguments.ToList();
			response.AdditionalArguments = job.AdditionalArguments.ToList();
			response.Targets = (job.Targets != null && job.Targets.Count > 0) ? job.Targets.ToList() : null;
			response.UpdateTime = new DateTimeOffset(job.UpdateTimeUtc);
			response.UseArtifactsV2 = true;
			response.UpdateIssues = job.UpdateIssues;
			response.CanUpdate = canUpdate;
			response.ParentJobId = job.ParentJobId.ToString();
			response.ParentJobStepId = job.ParentJobStepId.ToString();
			response.Metadata = job.Metadata.ToList();
			return response;
		}

		static GetJobReportResponse CreateGetReportResponse(IJobReport report)
		{
			GetJobReportResponse response = new GetJobReportResponse(report.Name, report.Placement);
			response.ArtifactId = report.ArtifactId?.ToString();
			response.Content = report.Content;
			response.Message = report.Message;
			response.Severity = report.Severity;
			return response;
		}

		/// <summary>
		/// Get the response object for a batch
		/// </summary>
		/// <param name="batch"></param>
		/// <param name="includeCosts"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		async ValueTask<GetJobBatchResponse> CreateBatchResponseAsync(IJobStepBatch batch, bool includeCosts, CancellationToken cancellationToken)
		{
			List<GetJobStepResponse> steps = new List<GetJobStepResponse>();
			foreach (IJobStep step in batch.Steps)
			{
				steps.Add(await CreateStepResponseAsync(step, cancellationToken));
			}

			double? agentRate = null;
			if (batch.AgentId != null && includeCosts)
			{
				agentRate = await _agentService.GetRateAsync(batch.AgentId.Value, cancellationToken);
			}

			// Only fetch lease for outcome reasons when batch has an error
			List<string>? leaseOutcomeReasons = null;
			if (batch.Error != JobStepBatchError.None && batch.LeaseId != null)
			{
				ILease? lease = await _leaseCollection.GetAsync(batch.LeaseId.Value, cancellationToken);
				if (lease != null)
				{
					leaseOutcomeReasons = ToFlagNames(lease.OutcomeReason);
				}
			}

			return CreateGetBatchResponse(batch, steps, agentRate, leaseOutcomeReasons);
		}

		/// <summary>
		/// Converts this batch into a public response object
		/// </summary>
		/// <param name="batch">The batch to construct from</param>
		/// <param name="steps">Steps in this batch</param>
		/// <param name="agentRate">Rate for this agent</param>
		/// <param name="leaseOutcomeReasons">Lease outcome reasons (only when batch has an error)</param>
		/// <returns>Response instance</returns>
		static GetJobBatchResponse CreateGetBatchResponse(IJobStepBatch batch, List<GetJobStepResponse> steps, double? agentRate, List<string>? leaseOutcomeReasons)
		{
			GetJobBatchResponse response = new GetJobBatchResponse();
			response.Id = batch.Id;
			response.LogId = batch.LogId?.ToString();
			response.GroupIdx = batch.GroupIdx;
			response.AgentType = batch.AgentType;
			response.State = batch.State;
			response.Error = batch.Error;
			response.Steps.AddRange(steps);
			response.AgentId = batch.AgentId?.ToString();
			response.AgentRate = agentRate;
			response.SessionId = batch.SessionId?.ToString();
			response.LeaseId = batch.LeaseId?.ToString();
			response.LeaseOutcomeReasons = leaseOutcomeReasons;
			response.WeightedPriority = batch.SchedulePriority;
			response.StartTime = batch.StartTimeUtc;
			response.FinishTime = batch.FinishTimeUtc;
			response.ReadyTime = batch.ReadyTimeUtc;
			return response;
		}

		/// <summary>
		/// Converts a LeaseOutcomeReason flags enum to a list of flag names
		/// </summary>
		/// <param name="reason">The flags enum value</param>
		/// <returns>List of flag names, or null if none set</returns>
		static List<string>? ToFlagNames(LeaseOutcomeReason reason)
		{
			if (reason == LeaseOutcomeReason.None)
			{
				return null;
			}

			return Enum.GetValues(typeof(LeaseOutcomeReason))
				.Cast<LeaseOutcomeReason>()
				.Where(flag => flag != LeaseOutcomeReason.None && reason.HasFlag(flag))
				.Select(flag => flag.ToString())
				.ToList();
		}

		/// <summary>
		/// Get the response object for a step
		/// </summary>
		/// <param name="step"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		async ValueTask<GetJobStepResponse> CreateStepResponseAsync(IJobStep step, CancellationToken cancellationToken)
		{
			GetThinUserInfoResponse? abortedByUserInfo = null;
			if (step.AbortedByUserId != null)
			{
				abortedByUserInfo = (await _userCollection.GetCachedUserAsync(step.AbortedByUserId.Value, cancellationToken))?.ToThinApiResponse();
			}

			GetThinUserInfoResponse? retriedByUserInfo = null;
			if (step.RetriedByUserId != null)
			{
				retriedByUserInfo = (await _userCollection.GetCachedUserAsync(step.RetriedByUserId.Value, cancellationToken))?.ToThinApiResponse();
			}

			return CreateGetStepResponse(step, abortedByUserInfo, retriedByUserInfo);
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="step">The step to construct from</param>
		/// <param name="abortedByUserInfo">User that aborted this step</param>
		/// <param name="retriedByUserInfo">User that retried this step</param>
		static GetJobStepResponse CreateGetStepResponse(IJobStep step, GetThinUserInfoResponse? abortedByUserInfo, GetThinUserInfoResponse? retriedByUserInfo)
		{
			GetJobStepResponse response = new GetJobStepResponse();
			response.Id = step.Id;
			response.NodeIdx = step.NodeIdx;

			// Node properties
			response.Name = step.Name;
			if (step.Inputs.Count > 0)
			{
				response.Inputs = step.Inputs.ToList();
			}
			if (step.OptionalInputs.Count > 0)
			{
				response.OptionalInputs = step.OptionalInputs.ToList();
			}
			if (step.OutputNames.Count > 0)
			{
				response.OutputNames = step.OutputNames.ToList();
			}
			if (step.InputDependencies.Count > 0)
			{
				response.InputDependencies = step.InputDependencies.ToList();
			}
			if (step.OptionalInputDependencies.Count > 0)
			{
				response.OptionalInputDependencies = step.OptionalInputDependencies.ToList();
			}
			if (step.OrderDependencies.Count > 0)
			{
				response.OrderDependencies = step.OrderDependencies.ToList();
			}
			response.AllowRetry = step.AllowRetry;
			response.RunEarly = step.RunEarly;
			response.Warnings = step.Warnings;
			response.Credentials = step.Credentials;
			response.Annotations = step.Annotations;

			// Step properties
			response.State = step.State;
			response.Outcome = step.Outcome;
			response.Error = step.Error;
			response.AbortRequested = step.AbortRequested;
			response.AbortByUser = abortedByUserInfo?.Login;
			response.AbortedByUserInfo = abortedByUserInfo;
			response.CancellationReason = step.CancellationReason;
			response.RetryByUser = retriedByUserInfo?.Login;
			response.RetriedByUserInfo = retriedByUserInfo;
			response.LogId = step.LogId?.ToString();
			response.ReadyTime = step.ReadyTimeUtc;
			response.StartTime = step.StartTimeUtc;
			response.FinishTime = step.FinishTimeUtc;
			response.Reports = step.Reports?.ConvertAll(x => CreateGetReportResponse(x));
			response.SpawnedJobs = step.SpawnedJobs?.ConvertAll(x => x.ToString());
			response.Metadata = step.Metadata.ToList();

			if (step.Properties != null && step.Properties.Count > 0)
			{
				response.Properties = step.Properties.ToDictionary();
			}
			return response;
		}

		/// <summary>
		/// Gets information about the graph for a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="filter">Filter for the fields to return</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/graph")]
		[ProducesResponseType(typeof(GetGraphResponse), 200)]
		public async Task<ActionResult<object>> GetJobGraphAsync(JobId jobId, [FromQuery] PropertyFilter? filter = null)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job);
			return PropertyFilter.Apply(CreateGetGraphResponse(graph), filter);
		}

		static GetGraphResponse CreateGetGraphResponse(IGraph graph)
		{
			GetGraphResponse response = new GetGraphResponse(graph.Id.ToString());
			if (graph.Groups.Count > 0)
			{
				response.Groups = graph.Groups.ConvertAll(x => CreateGetGroupResponse(x, graph.Groups));
			}
			if (graph.Aggregates.Count > 0)
			{
				response.Aggregates = graph.Aggregates.ConvertAll(x => CreateGetAggregateResponse(x, graph.Groups));
			}
			if (graph.Labels.Count > 0)
			{
				response.Labels = graph.Labels.ConvertAll(x => CreateGetLabelResponse(x, graph.Groups));
			}
			return response;
		}

		static GetGroupResponse CreateGetGroupResponse(INodeGroup group, IReadOnlyList<INodeGroup> groups)
		{
			GetGroupResponse response = new GetGroupResponse(group.AgentType);
			response.Nodes.AddRange(group.Nodes.Select(x => CreateGetNodeResponse(x, groups)));
			return response;
		}

		static GetNodeResponse CreateGetNodeResponse(INode node, IReadOnlyList<INodeGroup> groups)
		{
			GetNodeResponse response = new GetNodeResponse(node.Name);
			response.Inputs.AddRange(node.Inputs.Select(x => groups[x.NodeRef.GroupIdx].Nodes[x.NodeRef.NodeIdx].OutputNames[x.OutputIdx]));
			response.OptionalInputs.AddRange(node.OptionalInputs.Select(x => groups[x.NodeRef.GroupIdx].Nodes[x.NodeRef.NodeIdx].OutputNames[x.OutputIdx]));
			response.Outputs.AddRange(node.OutputNames);
			response.InputDependencies.AddRange(node.InputDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx].Name));
			response.OptionalInputDependencies.AddRange(node.OptionalInputDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx].Name));
			response.OrderDependencies.AddRange(node.OrderDependencies.Select(x => groups[x.GroupIdx].Nodes[x.NodeIdx].Name));
			response.Priority = node.Priority;
			response.AllowRetry = node.AllowRetry;
			response.RunEarly = node.RunEarly;
			response.Warnings = node.Warnings;
			response.Credentials = node.Credentials;
			response.Properties = node.Properties;
			response.Annotations = node.Annotations;
			return response;
		}

		static GetAggregateResponse CreateGetAggregateResponse(IAggregate aggregate, IReadOnlyList<INodeGroup> groups)
		{
			GetAggregateResponse response = new GetAggregateResponse(aggregate.Name);
			response.Nodes = aggregate.Nodes.ConvertAll(x => x.ToNode(groups).Name);
			return response;
		}

		static GetLabelResponse CreateGetLabelResponse(ILabel label, IReadOnlyList<INodeGroup> groups)
		{
			GetLabelResponse response = new GetLabelResponse();
			response.DashboardName = label.DashboardName;
			response.DashboardCategory = label.DashboardCategory;
			response.UgsName = label.UgsName;
			response.UgsProject = label.UgsProject;
			response.RequiredNodes = label.RequiredNodes.ConvertAll(x => x.ToNode(groups).Name);
			response.IncludedNodes = label.IncludedNodes.ConvertAll(x => x.ToNode(groups).Name);
			return response;
		}

		/// <summary>
		/// Gets timing information about the graph for a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="filter">Filter for the fields to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/timing")]
		[ProducesResponseType(typeof(GetJobTimingResponse), 200)]
		public async Task<ActionResult<object>> GetJobTimingAsync(JobId jobId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IJobTiming jobTiming = await _jobService.GetJobTimingAsync(job, _logger, cancellationToken);
			IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
			return PropertyFilter.Apply(await CreateJobTimingResponseAsync(job, graph, jobTiming, _logger, cancellationToken: cancellationToken), filter);
		}

		private async Task<GetJobTimingResponse> CreateJobTimingResponseAsync(IJob job, IGraph graph, IJobTiming jobTiming, ILogger logger, bool includeJobResponse = false, CancellationToken cancellationToken = default)
		{
			Dictionary<INode, TimingInfo> nodeToTimingInfo = job.GetTimingInfo(graph, jobTiming, logger);

			Dictionary<string, GetStepTimingInfoResponse> steps = new Dictionary<string, GetStepTimingInfoResponse>();
			foreach (IJobStepBatch batch in job.Batches)
			{
				foreach (IJobStep step in batch.Steps)
				{
					INode node = graph.Groups[batch.GroupIdx].Nodes[step.NodeIdx];
					steps[step.Id.ToString()] = CreateGetStepTimingInfoResponse(node.Name, nodeToTimingInfo[node]);
				}
			}

			List<GetLabelTimingInfoResponse> labels = new List<GetLabelTimingInfoResponse>();
			foreach (ILabel label in graph.Labels)
			{
				TimingInfo timingInfo = TimingInfo.Max(label.GetDependencies(graph.Groups).Select(x => nodeToTimingInfo[x]));
				labels.Add(CreateGetLabelTimingInfoResponse(label, timingInfo));
			}

			GetJobResponse? jobResponse = null;
			if (includeJobResponse)
			{
				jobResponse = await CreateJobResponseAsync(job, graph, true, true, true, true, cancellationToken);
			}

			return new GetJobTimingResponse(jobResponse, steps, labels);
		}

		static GetLabelTimingInfoResponse CreateGetLabelTimingInfoResponse(ILabel label, TimingInfo timingInfo)
		{
			GetLabelTimingInfoResponse response = new GetLabelTimingInfoResponse();
			timingInfo.CopyToResponse(response);

			response.DashboardName = label.DashboardName;
			response.DashboardCategory = label.DashboardCategory;
			response.UgsName = label.UgsName;
			response.UgsProject = label.UgsProject;
			return response;
		}

		static GetStepTimingInfoResponse CreateGetStepTimingInfoResponse(string? name, TimingInfo jobTimingInfo)
		{
			GetStepTimingInfoResponse response = new GetStepTimingInfoResponse();
			jobTimingInfo.CopyToResponse(response);

			response.Name = name;
			response.AverageStepWaitTime = jobTimingInfo.StepTiming?.AverageWaitTime;
			response.AverageStepInitTime = jobTimingInfo.StepTiming?.AverageInitTime;
			response.AverageStepDuration = jobTimingInfo.StepTiming?.AverageDuration;
			return response;
		}

		/// <summary>
		/// Find timing information about the graph for multiple jobs
		/// </summary>
		/// <param name="streamId">The stream to search in</param>
		/// <param name="templates">List of templates to find</param>
		/// <param name="filter">Filter for the fields to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Job timings for each job ID</returns>
		[HttpGet]
		[Route("/api/v1/jobs/timing")]
		[ProducesResponseType(typeof(FindJobTimingsResponse), 200)]
		public async Task<ActionResult<object>> FindJobTimingsAsync(
			[FromQuery] string? streamId = null,
			[FromQuery(Name = "template")] string[]? templates = null,
			[FromQuery] PropertyFilter? filter = null,
			[FromQuery] int count = 100,
			CancellationToken cancellationToken = default)
		{
			if (streamId == null)
			{
				return BadRequest("Missing/invalid query parameter streamId");
			}

			TemplateId[] templateRefIds = templates switch
			{
				{ Length: > 0 } => templates.Select(x => new TemplateId(x)).ToArray(),
				_ => Array.Empty<TemplateId>()
			};

			IReadOnlyList<IJob> jobs = await _jobService.FindJobsByStreamWithTemplatesAsync(new StreamId(streamId), templateRefIds, count: count, consistentRead: false, cancellationToken: cancellationToken);

			Dictionary<string, GetJobTimingResponse> jobTimings = [];
			await foreach (IJob job in jobs.ToAsyncEnumerable()
				.Where(job => _buildConfig.Value.Authorize(job, JobAclAction.ViewJob, User))
				.WithCancellation(cancellationToken))
			{
				IJobTiming jobTiming = await _jobService.GetJobTimingAsync(job, _logger, cancellationToken);
				IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
				jobTimings[job.Id.ToString()] = await CreateJobTimingResponseAsync(job, graph, jobTiming, _logger, true, cancellationToken);
			}

			return PropertyFilter.Apply(new FindJobTimingsResponse(jobTimings), filter);
		}

		/// <summary>
		/// Gets information about the template for a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="filter">Filter for the fields to return</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/template")]
		[ProducesResponseType(typeof(GetTemplateResponse), 200)]
		public async Task<ActionResult<object>> GetJobTemplateAsync(JobId jobId, [FromQuery] PropertyFilter? filter = null)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null || job.TemplateHash == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			ITemplate? template = await _templateCollection.GetAsync(job.TemplateHash);
			if (template == null)
			{
				return NotFound(job.StreamId, job.TemplateId);
			}

			return new GetTemplateResponse(template).ApplyFilter(filter);
		}

		/// <summary>
		/// Find jobs matching a criteria
		/// </summary>
		/// <param name="ids">The job ids to return</param>
		/// <param name="name">Name of the job to find</param>
		/// <param name="templates">List of templates to find</param>
		/// <param name="templateIds">Alternative to templates query param, used to shorten URLs with many templates</param>
		/// <param name="streamId">The stream to search for</param>
		/// <param name="minChange">The minimum changelist number</param>
		/// <param name="maxChange">The maximum changelist number</param>
		/// <param name="includePreflight">Whether to include preflight jobs</param>
		/// <param name="preflightOnly">Whether to only include preflight jobs</param>
		/// <param name="preflightChange">The preflighted changelist</param>
		/// <param name="startedByUserId">User id for which to include jobs</param>
		/// <param name="preflightStartedByUserId">User id for which to include preflight jobs</param>
		/// <param name="minCreateTime">Minimum creation time</param>
		/// <param name="maxCreateTime">Maximum creation time</param>
		/// <param name="modifiedBefore">If specified, only jobs updated before the give time will be returned</param>
		/// <param name="modifiedAfter">If specified, only jobs updated after the give time will be returned</param>
		/// <param name="target">Target to filter the returned jobs by</param>
		/// <param name="state">Filter state of the returned jobs</param>
		/// <param name="outcome">Filter outcome of the returned jobs</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <param name="index">Index of the first result to be returned</param>
		/// <param name="count">Number of results to return</param>
		/// <returns>List of jobs</returns>
		[HttpGet]
		[Route("/api/v1/jobs")]
		[ProducesResponseType(typeof(List<GetJobResponse>), 200)]
		public async Task<ActionResult<List<object>>> FindJobsAsync(
			[FromQuery(Name = "Id")] string[]? ids = null,
			[FromQuery] string? streamId = null,
			[FromQuery] string? name = null,
			[FromQuery(Name = "template")] string[]? templates = null,
			[FromQuery(Name = "t")] string[]? templateIds = null,
			[FromQuery] CommitId? minChange = null,
			[FromQuery] CommitId? maxChange = null,
			[FromQuery] bool includePreflight = true,
			[FromQuery] bool? preflightOnly = null,
			[FromQuery] CommitId? preflightChange = null,
			[FromQuery] string? preflightStartedByUserId = null,
			[FromQuery] string? startedByUserId = null,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] DateTimeOffset? modifiedBefore = null,
			[FromQuery] DateTimeOffset? modifiedAfter = null,
			[FromQuery] string? target = null,
			[FromQuery] JobStepState[]? state = null,
			[FromQuery] JobStepOutcome[]? outcome = null,
			[FromQuery] PropertyFilter? filter = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 100)
		{
			JobId[]? jobIdValues = (ids == null) ? (JobId[]?)null : Array.ConvertAll(ids, x => JobId.Parse(x));
			StreamId? streamIdValue = (streamId == null) ? (StreamId?)null : new StreamId(streamId);

			IEnumerable<TemplateId>? allTemplates = null;
			if (templates != null && templates.Length > 0 && templateIds != null && templateIds.Length > 0)
			{
				allTemplates = templates.Select(x => new TemplateId(x)).Union(templateIds.Select(x => new TemplateId(x)));
			}
			else if (templates != null && templates.Length > 0)
			{
				allTemplates = templates.Select(x => new TemplateId(x));
			}
			else if (templateIds != null && templateIds.Length > 0)
			{
				allTemplates = templateIds.Select(x => new TemplateId(x));
			}

			TemplateId[]? templateRefIds = (allTemplates != null) ? allTemplates.ToArray(): null;

			UserId? preflightStartedByUserIdValue = null;

			if (preflightStartedByUserId != null)
			{
				preflightStartedByUserIdValue = UserId.Parse(preflightStartedByUserId);
			}

			UserId? startedByUserIdValue = null;

			if (startedByUserId != null)
			{
				startedByUserIdValue = UserId.Parse(startedByUserId);
			}

			IReadOnlyList<IJob> jobs;
			jobs = await _jobService.FindJobsAsync(new FindJobOptions(jobIdValues, streamIdValue, name, templateRefIds, minChange,
				maxChange, preflightChange, preflightOnly, includePreflight, preflightStartedByUserIdValue, startedByUserIdValue, minCreateTime?.UtcDateTime, maxCreateTime?.UtcDateTime, Target: target, State: state, Outcome: outcome,
				ModifiedBefore: modifiedBefore, ModifiedAfter: modifiedAfter), index, count);

			return await CreateAuthorizedJobResponsesAsync(jobs, filter);
		}

		/// <summary>
		/// Find jobs for a stream with given templates, sorted by creation date
		/// </summary>
		/// <param name="streamId">The stream to search for</param>
		/// <param name="templates">List of templates to find</param>
		/// <param name="templateIds">Alternative to templates query param, used to shorten URLs with many templates</param>
		/// <param name="preflightStartedByUserId">User id for which to include preflight jobs</param>
		/// <param name="maxCreateTime">Maximum creation time</param>
		/// <param name="modifiedAfter">If specified, only jobs updated after the given time will be returned</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <param name="index">Index of the first result to be returned</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="consistentRead">If a read to the primary database is required, for read consistency. Usually not required.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of jobs</returns>
		[HttpGet]
		[Route("/api/v1/jobs/streams/{streamId}")]
		[ProducesResponseType(typeof(List<GetJobResponse>), 200)]
		public async Task<ActionResult<List<object>>> FindJobsByStreamWithTemplatesAsync(
			string streamId,
			[FromQuery(Name = "template")] string[]? templates = null,
			[FromQuery(Name = "t")] string[]? templateIds = null,
			[FromQuery] string? preflightStartedByUserId = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] DateTimeOffset? modifiedAfter = null,
			[FromQuery] PropertyFilter? filter = null,
			[FromQuery] int index = 0,
			[FromQuery] int count = 100,
			[FromQuery] bool consistentRead = false,
			CancellationToken cancellationToken = default)
		{
			StreamId streamIdValue = new StreamId(streamId);

			IEnumerable<TemplateId>? allTemplates = null;
			if (templates != null && templates.Length > 0 && templateIds != null && templateIds.Length > 0)
			{
				allTemplates = templates.Select(x => new TemplateId(x)).Union(templateIds.Select(x => new TemplateId(x)));
			}
			else if (templates != null && templates.Length > 0)
			{
				allTemplates = templates.Select(x => new TemplateId(x));
			}
			else if (templateIds != null && templateIds.Length > 0)
			{
				allTemplates = templateIds.Select(x => new TemplateId(x));
			}
			else
			{
				return BadRequest("Templates must be included in query, use template or t parameters.");
			}

			TemplateId[] templateRefIds = allTemplates.ToArray();

			UserId? preflightStartedByUserIdValue = preflightStartedByUserId != null ? UserId.Parse(preflightStartedByUserId) : null;
			count = Math.Min(1000, count);

			IReadOnlyList<IJob> jobs = await _jobService.FindJobsByStreamWithTemplatesAsync(streamIdValue, templateRefIds, preflightStartedByUserIdValue, maxCreateTime, modifiedAfter, index, count, consistentRead, cancellationToken: cancellationToken);
			return await CreateAuthorizedJobResponsesAsync(jobs, filter, cancellationToken);
		}

		private async Task<List<object>> CreateAuthorizedJobResponsesAsync(IReadOnlyList<IJob> jobs, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			List<object> responses = new();
			foreach (IGrouping<StreamId, IJob> grouping in jobs.GroupBy(x => x.StreamId))
			{
				StreamConfig? streamConfig;
				if (_buildConfig.Value.TryGetStream(grouping.Key, out streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					bool includeCosts = streamConfig.Authorize(ServerAclAction.ViewCosts, User);
					foreach (IJob job in grouping)
					{
						IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
						responses.Add(await CreateJobResponseAsync(job, graph, includeCosts, filter, cancellationToken));
					}
				}
			}
			return responses;
		}

		/// <summary>
		/// Adds an array of nodes to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="requests">Properties of the new nodes</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Id of the new job</returns>
		[HttpPost]
		[Obsolete("Modifying graph through REST API is not supported")]
		[Route("/api/v1/jobs/{jobId}/groups")]
		public async Task<ActionResult> CreateGroupsAsync(JobId jobId, [FromBody] List<NewGroup> requests, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
				if (job == null)
				{
					return NotFound(jobId);
				}

				StreamConfig? streamConfig;
				if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
				{
					return NotFound(job.StreamId);
				}
				if (!streamConfig.Authorize(JobAclAction.ExecuteJob, User))
				{
					return Forbid(JobAclAction.ExecuteJob, jobId);
				}

				IGraph newGraph = await _graphs.AppendAsync(job.Graph, requests, null, null, null, cancellationToken);

				IJob? newJob = await _jobService.TryUpdateGraphAsync(job, newGraph, cancellationToken);
				if (newJob != null)
				{
					return Ok();
				}
			}
		}

		/// <summary>
		/// Gets the nodes to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/groups")]
		[ProducesResponseType(typeof(List<GetGroupResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetGroupsAsync(JobId jobId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
			return graph.Groups.ConvertAll(x => CreateGetGroupResponse(x, graph.Groups).ApplyFilter(filter));
		}

		/// <summary>
		/// Gets the nodes in a group to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="groupIdx">The group index</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/groups/{groupIdx}")]
		[ProducesResponseType(typeof(GetGroupResponse), 200)]
		public async Task<ActionResult<object>> GetGroupAsync(JobId jobId, int groupIdx, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
			if (groupIdx < 0 || groupIdx >= graph.Groups.Count)
			{
				return NotFound(jobId, groupIdx);
			}

			return CreateGetGroupResponse(graph.Groups[groupIdx], graph.Groups).ApplyFilter(filter);
		}

		/// <summary>
		/// Gets the nodes for a particular group
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="groupIdx">Index of the group containing the node to update</param>
		/// <param name="filter">Filter for the properties to return</param>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/groups/{groupIdx}/nodes")]
		[ProducesResponseType(typeof(List<GetNodeResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetNodesAsync(JobId jobId, int groupIdx, [FromQuery] PropertyFilter? filter = null)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job);
			if (groupIdx < 0 || groupIdx >= graph.Groups.Count)
			{
				return NotFound(jobId, groupIdx);
			}

			return graph.Groups[groupIdx].Nodes.ConvertAll(x => CreateGetNodeResponse(x, graph.Groups).ApplyFilter(filter));
		}

		/// <summary>
		/// Gets a particular node definition
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="groupIdx">Index of the group containing the node to update</param>
		/// <param name="nodeIdx">Index of the node to update</param>
		/// <param name="filter">Filter for the properties to return</param>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/groups/{groupIdx}/nodes/{nodeIdx}")]
		[ProducesResponseType(typeof(GetNodeResponse), 200)]
		public async Task<ActionResult<object>> GetNodeAsync(JobId jobId, int groupIdx, int nodeIdx, [FromQuery] PropertyFilter? filter = null)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			IGraph graph = await _jobService.GetGraphAsync(job);
			if (groupIdx < 0 || groupIdx >= graph.Groups.Count)
			{
				return NotFound(jobId, groupIdx);
			}
			if (nodeIdx < 0 || nodeIdx >= graph.Groups[groupIdx].Nodes.Count)
			{
				return NotFound(jobId, groupIdx, nodeIdx);
			}

			return CreateGetNodeResponse(graph.Groups[groupIdx].Nodes[nodeIdx], graph.Groups).ApplyFilter(filter);
		}

		/// <summary>
		/// Gets the steps currently scheduled to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/batches")]
		[ProducesResponseType(typeof(List<GetJobBatchResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetBatchesAsync(JobId jobId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, job.StreamId);
			}

			bool includeCosts = streamConfig.Authorize(ServerAclAction.ViewCosts, User);

			List<object> responses = new List<object>();
			foreach (IJobStepBatch batch in job.Batches)
			{
				GetJobBatchResponse response = await CreateBatchResponseAsync(batch, includeCosts, cancellationToken);
				responses.Add(response.ApplyFilter(filter));
			}
			return responses;
		}

		/// <summary>
		/// Updates the state of a jobstep
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the step</param>
		/// <param name="request">Updates to apply to the node</param>
		[HttpPut]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}")]
		public async Task<ActionResult> UpdateBatchAsync(JobId jobId, JobStepBatchId batchId, [FromBody] UpdateBatchRequest request)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				return NotFound(job.StreamId);
			}

			IJobStepBatch? batch = job.Batches.FirstOrDefault(x => x.Id == batchId);
			if (batch == null)
			{
				return NotFound(jobId, batchId);
			}
			if (batch.SessionId == null || !User.HasSessionClaim(batch.SessionId.Value))
			{
				return Forbid("Missing session claim for job {JobId} batch {BatchId}", jobId, batchId);
			}

			IJob? newJob = await _jobService.UpdateBatchAsync(job, batchId, streamConfig, (request.LogId == null) ? null : LogId.Parse(request.LogId), request.State);
			if (newJob == null)
			{
				return NotFound(jobId);
			}
			return Ok();
		}

		/// <summary>
		/// Gets a particular step currently scheduled to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the step</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}")]
		[ProducesResponseType(typeof(GetJobBatchResponse), 200)]
		public async Task<ActionResult<object>> GetBatchAsync(JobId jobId, JobStepBatchId batchId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, job.StreamId);
			}

			bool includeCosts = streamConfig.Authorize(ServerAclAction.ViewCosts, User);
			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.Id == batchId)
				{
					GetJobBatchResponse response = await CreateBatchResponseAsync(batch, includeCosts, cancellationToken);
					return response.ApplyFilter(filter);
				}
			}

			return NotFound(jobId, batchId);
		}

		/// <summary>
		/// Gets the steps currently scheduled to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the batch</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}/steps")]
		[ProducesResponseType(typeof(List<GetJobStepResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetStepsAsync(JobId jobId, JobStepBatchId batchId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.Id == batchId)
				{
					List<object> responses = new List<object>();
					foreach (IJobStep step in batch.Steps)
					{
						GetJobStepResponse response = await CreateStepResponseAsync(step, cancellationToken);
						responses.Add(response.ApplyFilter(filter));
					}
					return responses;
				}
			}

			return NotFound(jobId, batchId);
		}

		/// <summary>
		/// Updates the state of a jobstep
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the batch</param>
		/// <param name="stepId">Unique id for the step</param>
		/// <param name="request">Updates to apply to the node</param>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}/steps/{stepId}")]
		public async Task<ActionResult<UpdateStepResponse>> UpdateStepAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, [FromBody] UpdateStepRequest request)
		{
			IJob? job = await _jobService.GetJobAsync(jobId);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}

			// Check permissions for updating this step. Only the agent executing the step can modify the state of it.
			if (request.State != JobStepState.Unspecified || request.Outcome != JobStepOutcome.Unspecified)
			{
				IJobStepBatch? batch = job.Batches.FirstOrDefault(x => x.Id == batchId);
				if (batch == null)
				{
					return NotFound(jobId, batchId);
				}
				if (!batch.SessionId.HasValue || !User.HasSessionClaim(batch.SessionId.Value))
				{
					return Forbid();
				}
			}

			if (request.Retry != null || request.Priority != null)
			{
				if (!streamConfig.Authorize(JobAclAction.RetryJobStep, User))
				{
					return Forbid(JobAclAction.RetryJobStep, jobId);
				}
			}
			if (request.Properties != null)
			{
				if (!streamConfig.Authorize(JobAclAction.UpdateJob, User))
				{
					return Forbid(JobAclAction.UpdateJob, jobId);
				}
			}

			UserId? retryByUser = (request.Retry.HasValue && request.Retry.Value) ? User.GetUserId() : null;
			UserId? abortByUser = (request.AbortRequested.HasValue && request.AbortRequested.Value) ? User.GetUserId() : null;

			try
			{
				NodeRef? retryNodeRef = null;
				if (request.Retry != null && job.TryGetStep(stepId, out IJobStep? step))
				{
					retryNodeRef = new NodeRef(step.Batch.GroupIdx, step.NodeIdx);
				}

				IJob? newJob = await _jobService.UpdateStepAsync(job, batchId, stepId, streamConfig, request.State, request.Outcome, null, request.AbortRequested, abortByUser, (request.LogId == null) ? null : LogId.Parse(request.LogId), null, retryByUser, request.Priority, null, request.Properties, request.CancellationReason);
				if (newJob == null)
				{
					return NotFound(jobId);
				}

				UpdateStepResponse response = new UpdateStepResponse();
				if (retryNodeRef != null)
				{
					JobStepRefId? retriedStepId = newJob.FindLatestStepForNode(retryNodeRef);
					if (retriedStepId != null)
					{
						response.BatchId = retriedStepId.Value.BatchId.ToString();
						response.StepId = retriedStepId.Value.StepId.ToString();
					}
				}
				return response;
			}
			catch (RetryNotAllowedException ex)
			{
				return BadRequest(ex.Message);
			}
		}

		/// <summary>
		/// Gets a particular step currently scheduled to be executed for a job
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the batch</param>
		/// <param name="stepId">Unique id for the step</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of nodes to be executed</returns>
		[HttpGet]
		[Obsolete("Query entire job instead")]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}/steps/{stepId}")]
		[ProducesResponseType(typeof(GetJobStepResponse), 200)]
		public async Task<ActionResult<object>> GetStepAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, jobId);
			}

			foreach (IJobStepBatch batch in job.Batches)
			{
				if (batch.Id == batchId)
				{
					foreach (IJobStep step in batch.Steps)
					{
						if (step.Id == stepId)
						{
							GetJobStepResponse response = await CreateStepResponseAsync(step, cancellationToken);
							return response.ApplyFilter(filter);
						}
					}
					return NotFound(jobId, batchId, stepId);
				}
			}

			return NotFound(jobId, batchId);
		}

		/// <summary>
		/// Updates notifications for a specific job.
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="batchId">Unique id for the batch</param>
		/// <param name="stepId">Unique id for the step</param>
		/// <param name="request">The notification request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}/steps/{stepId}/notifications")]
		public async Task<ActionResult> UpdateStepNotificationsAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, [FromBody] UpdateNotificationsRequest request, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(NotificationAclAction.CreateSubscription, User))
			{
				return Forbid(NotificationAclAction.CreateSubscription, jobId);
			}

			if (!job.TryGetBatch(batchId, out IJobStepBatch? batch))
			{
				return NotFound(jobId, batchId);
			}

			if (!batch.TryGetStep(stepId, out IJobStep? step))
			{
				return NotFound(jobId, batchId, stepId);
			}

			NotificationTriggerId? triggerId = step.NotificationTriggerId;
			if (triggerId == null)
			{
				triggerId = new NotificationTriggerId(BinaryIdUtils.CreateNew());
				if (await _jobService.UpdateStepAsync(job, batchId, stepId, streamConfig, JobStepState.Unspecified, JobStepOutcome.Unspecified, newNotificationTriggerId: triggerId, cancellationToken: cancellationToken) == null)
				{
					return NotFound(jobId, batchId, stepId);
				}
			}

			await _notificationService.UpdateSubscriptionsAsync(triggerId.Value, User, request.Email, request.Slack, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Gets information about a specific job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="batchId">Unique id for the batch</param>
		/// <param name="stepId">Unique id for the step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/batches/{batchId}/steps/{stepId}/notifications")]
		public async Task<ActionResult<GetNotificationResponse>> GetStepNotificationsAsync(JobId jobId, JobStepBatchId batchId, JobStepId stepId, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			IJobStep? step;
			if (!job.TryGetBatch(batchId, out IJobStepBatch? batch))
			{
				return NotFound(jobId, batchId);
			}
			if (!batch.TryGetStep(stepId, out step))
			{
				return NotFound(jobId, batchId, stepId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(NotificationAclAction.CreateSubscription, User))
			{
				return Forbid(NotificationAclAction.CreateSubscription, jobId);
			}

			INotificationSubscription? subscription;
			if (step.NotificationTriggerId == null)
			{
				subscription = null;
			}
			else
			{
				subscription = await _notificationService.GetSubscriptionsAsync(step.NotificationTriggerId.Value, User, cancellationToken);
			}
			return new GetNotificationResponse(subscription);
		}

		/// <summary>
		/// Updates notifications for a specific label.
		/// </summary>
		/// <param name="jobId">Unique id for the job</param>
		/// <param name="labelIndex">Index for the label</param>
		/// <param name="request">The notification request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPut]
		[Route("/api/v1/jobs/{jobId}/labels/{labelIndex}/notifications")]
		public async Task<ActionResult> UpdateLabelNotificationsAsync(JobId jobId, int labelIndex, [FromBody] UpdateNotificationsRequest request, CancellationToken cancellationToken)
		{
			NotificationTriggerId triggerId;
			for (; ; )
			{
				IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
				if (job == null)
				{
					return NotFound(jobId);
				}

				StreamConfig? streamConfig;
				if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
				{
					return NotFound(job.StreamId);
				}
				if (!streamConfig.Authorize(NotificationAclAction.CreateSubscription, User))
				{
					return Forbid(NotificationAclAction.CreateSubscription, jobId);
				}

				NotificationTriggerId newTriggerId;
				if (job.LabelIdxToTriggerId.TryGetValue(labelIndex, out newTriggerId))
				{
					triggerId = newTriggerId;
					break;
				}

				newTriggerId = new NotificationTriggerId(BinaryIdUtils.CreateNew());

				IJob? newJob = await _jobService.UpdateJobAsync(job, labelIdxToTriggerId: new KeyValuePair<int, NotificationTriggerId>(labelIndex, newTriggerId), cancellationToken: cancellationToken);
				if (newJob != null)
				{
					triggerId = newTriggerId;
					break;
				}
			}

			await _notificationService.UpdateSubscriptionsAsync(triggerId, User, request.Email, request.Slack, cancellationToken);
			return Ok();
		}

		/// <summary>
		/// Gets notification info about a specific label in a job.
		/// </summary>
		/// <param name="jobId">Id of the job to find</param>
		/// <param name="labelIndex">Index for the label</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Notification info for the requested label in the job</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/labels/{labelIndex}/notifications")]
		public async Task<ActionResult<GetNotificationResponse>> GetLabelNotificationsAsync(JobId jobId, int labelIndex, CancellationToken cancellationToken)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			INotificationSubscription? subscription;
			if (!job.LabelIdxToTriggerId.ContainsKey(labelIndex))
			{
				subscription = null;
			}
			else
			{
				subscription = await _notificationService.GetSubscriptionsAsync(job.LabelIdxToTriggerId[labelIndex], User, cancellationToken);
			}
			return new GetNotificationResponse(subscription);
		}

		/// <summary>
		/// Retrieve historical information about a specific job
		/// </summary>
		/// <param name="jobId">Id of the job to get information about</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/history")]
		public async Task<ActionResult> GetJobHistoryAsync(JobId jobId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50, CancellationToken cancellationToken = default)
		{
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, job.StreamId);
			}

			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync(cancellationToken);
			await _jobService.GetJobLogger(jobId).FindAsync(Response.BodyWriter, minTime, maxTime, index, count, cancellationToken);
			await Response.CompleteAsync();

			return Empty;
		}

		/// <summary>
		/// Get aggregated label details for hover card display.
		/// Returns steps, log events, issues, previous build status, and cross-stream merge chain in a single response.
		/// </summary>
		/// <param name="jobId">Job ID</param>
		/// <param name="labelIndex">Label index within the job's graph</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Aggregated label details</returns>
		[HttpGet]
		[Route("/api/v1/jobs/{jobId}/labels/{labelIndex}/details")]
		[ProducesResponseType(typeof(GetLabelDetailsResponse), 200)]
		public async Task<ActionResult<GetLabelDetailsResponse>> GetLabelDetailsAsync(JobId jobId, int labelIndex, CancellationToken cancellationToken = default)
		{
			// 1. Load job and authorize
			IJob? job = await _jobService.GetJobAsync(jobId, cancellationToken);
			if (job == null)
			{
				return NotFound(jobId);
			}

			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(job.StreamId, out streamConfig))
			{
				return NotFound(job.StreamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, job.StreamId);
			}

			// 2. Resolve graph and validate label index
			IGraph graph = await _jobService.GetGraphAsync(job, cancellationToken);
			if (labelIndex < 0 || labelIndex >= graph.Labels.Count)
			{
				return NotFound($"Label index {labelIndex} out of range (0..{graph.Labels.Count - 1})");
			}

			ILabel label = graph.Labels[labelIndex];

			// 3. Compute label state
			IReadOnlyList<(LabelState, LabelOutcome)> labelStates = job.GetLabelStates(graph);
			(LabelState state, LabelOutcome outcome) = labelStates[labelIndex];

			GetLabelDetailsResponse response = new GetLabelDetailsResponse
			{
				DashboardName = label.DashboardName,
				DashboardCategory = label.DashboardCategory,
				State = state,
				Outcome = outcome
			};

			// 4. Build step list
			IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = job.GetStepForNodeMap();
			List<IJobStep> allSteps = new();
			foreach (NodeRef nodeRef in label.IncludedNodes)
			{
				if (stepForNode.TryGetValue(nodeRef, out IJobStep? step))
				{
					allSteps.Add(step);
				}
			}

			response.TotalStepCount = allSteps.Count;

			// Filter to failing/warning when label is failed/warning
			List<IJobStep> displaySteps;
			if (outcome == LabelOutcome.Failure || outcome == LabelOutcome.Warnings)
			{
				displaySteps = allSteps.Where(s =>
					s.Outcome == JobStepOutcome.Failure ||
					s.Outcome == JobStepOutcome.Warnings).ToList();
			}
			else
			{
				displaySteps = allSteps;
			}

			// Cap at 20 steps
			const int MaxSteps = 20;
			foreach (IJobStep step in displaySteps.Take(MaxSteps))
			{
				// Extract diagnostic message from metadata
				string? diagnosticMessage = null;
				foreach (string meta in step.Metadata)
				{
					if (meta.StartsWith("ReportStepMessage=", StringComparison.Ordinal))
					{
						diagnosticMessage = meta.Substring("ReportStepMessage=".Length);
						break;
					}
				}

				response.Steps.Add(new GetLabelStepResponse
				{
					Id = step.Id.ToString(),
					Name = step.Name,
					State = step.State,
					Outcome = step.Outcome,
					LogId = step.LogId?.ToString(),
					StartTime = step.StartTimeUtc,
					FinishTime = step.FinishTimeUtc,
					DiagnosticMessage = diagnosticMessage
				});
			}

			// 5. Fetch log events for failing steps (fire-and-forget friendly tasks)
			Task<Dictionary<string, GetLabelLogEventsResponse>> logEventsTask = GetLabelLogEventsAsync(response.Steps, cancellationToken);

			// 6. Fetch issues
			Task<List<GetLabelIssueResponse>> issuesTask = GetLabelIssuesAsync(job, graph, labelIndex, cancellationToken);

			// 7. Previous build
			Task<GetLabelPreviousBuildResponse?> previousBuildTask = GetLabelPreviousBuildAsync(job, label, cancellationToken);

			// 8. Cross-stream merge chain
			Task<GetLabelCrossStreamChainResponse?> crossStreamTask = GetLabelCrossStreamChainAsync(job, label, state, outcome, cancellationToken);

			// Await all in parallel
			await Task.WhenAll(logEventsTask, issuesTask, previousBuildTask, crossStreamTask);

			response.LogEvents = await logEventsTask;
			response.Issues = await issuesTask;
			response.PreviousBuild = await previousBuildTask;
			response.CrossStreamChain = await crossStreamTask;

			return response;
		}

		async Task<Dictionary<string, GetLabelLogEventsResponse>> GetLabelLogEventsAsync(List<GetLabelStepResponse> steps, CancellationToken cancellationToken)
		{
			Dictionary<string, GetLabelLogEventsResponse> result = new();

			List<string> failingLogIds = steps
				.Where(s => (s.Outcome == JobStepOutcome.Failure || s.Outcome == JobStepOutcome.Warnings) && s.LogId != null)
				.Select(s => s.LogId!)
				.ToList();

			if (failingLogIds.Count == 0)
			{
				return result;
			}

			// Fetch log events in parallel across all failing logs
			KeyValuePair<string, GetLabelLogEventsResponse>?[] results = await Task.WhenAll(failingLogIds.Select(async logIdStr =>
			{
				try
				{
					LogId logId = LogId.Parse(logIdStr);
					ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
					if (log == null)
					{
						return (KeyValuePair<string, GetLabelLogEventsResponse>?)null;
					}

					const int MaxEventsPerLog = 20;
					List<ILogAnchor> anchors = await log.GetAnchorsAsync(null, null, MaxEventsPerLog + 50, cancellationToken);
					List<ILogAnchor> errorAnchors = anchors.Where(a => a.Severity >= LogEventSeverity.Warning).ToList();

					int total = errorAnchors.Count;
					List<GetLogEventResponse> events = new();

					foreach (ILogAnchor anchor in errorAnchors.Take(MaxEventsPerLog))
					{
						try
						{
							ILogEventData eventData = await anchor.GetDataAsync(cancellationToken);
							GetLogEventResponse eventResponse = new GetLogEventResponse
							{
								Severity = anchor.Severity,
								LogId = anchor.LogId,
								LineIndex = anchor.LineIndex,
								LineCount = anchor.LineCount,
								IssueId = null
							};
							eventResponse.Lines.AddRange(eventData.Lines.Select(x => System.Text.Json.JsonDocument.Parse(x.Data).RootElement));
							events.Add(eventResponse);
						}
						catch
						{
							// Skip events that can't be read (may have been garbage collected)
						}
					}

					return (KeyValuePair<string, GetLabelLogEventsResponse>?)new KeyValuePair<string, GetLabelLogEventsResponse>(logIdStr, new GetLabelLogEventsResponse { Total = total, Events = events });
				}
				catch
				{
					// Skip logs that can't be found
					return null;
				}
			}));

			foreach (KeyValuePair<string, GetLabelLogEventsResponse>? entry in results)
			{
				if (entry != null)
				{
					result[entry.Value.Key] = entry.Value.Value;
				}
			}

			return result;
		}

		async Task<List<GetLabelIssueResponse>> GetLabelIssuesAsync(IJob job, IGraph graph, int labelIndex, CancellationToken cancellationToken)
		{
			List<GetLabelIssueResponse> result = new();

			try
			{
				IReadOnlyList<IIssue> issues = await _issueCollection.FindIssuesForJobAsync(job, graph, labelIdx: labelIndex, count: 10, cancellationToken: cancellationToken);

				foreach (IIssue issue in issues)
				{
					// Check if resolved in this stream
					bool resolvedInStream = false;
					foreach (IIssueStream issueStream in issue.Streams)
					{
						if (issueStream.StreamId == job.StreamId && issueStream.ContainsFix == true)
						{
							resolvedInStream = true;
							break;
						}
					}

					GetThinUserInfoResponse? ownerInfo = null;
					if (issue.OwnerId != null)
					{
						IUser? ownerUser = await _userCollection.GetUserAsync(issue.OwnerId.Value, cancellationToken);
						if (ownerUser != null)
						{
							ownerInfo = ownerUser.ToThinApiResponse();
						}
					}

					result.Add(new GetLabelIssueResponse
					{
						Id = issue.Id,
						Summary = issue.UserSummary ?? issue.Summary,
						Severity = issue.Severity,
						Owner = ownerInfo,
						Acknowledged = issue.AcknowledgedAt != null,
						ResolvedInStream = resolvedInStream,
						ExternalIssueKey = issue.ExternalIssueKey,
						WorkflowThreadUrl = issue.WorkflowThreadUrl
					});
				}
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Failed to fetch issues for job {JobId} label {LabelIndex}", job.Id, labelIndex);
			}

			return result;
		}

		async Task<GetLabelPreviousBuildResponse?> GetLabelPreviousBuildAsync(IJob job, ILabel label, CancellationToken cancellationToken)
		{
			try
			{
				// MaxCreateTime is inclusive (Lte), so query for 2 results and skip the current job
				IReadOnlyList<IJob> previousJobs = await _jobService.FindJobsByStreamWithTemplatesAsync(
					job.StreamId,
					new[] { job.TemplateId },
					maxCreateTime: job.CreateTimeUtc,
					count: 2,
					consistentRead: false,
					cancellationToken: cancellationToken);

				IJob? prevJob = previousJobs.FirstOrDefault(j => j.Id != job.Id);
				if (prevJob == null)
				{
					return null;
				}

				IGraph prevGraph = await _jobService.GetGraphAsync(prevJob, cancellationToken);
				IReadOnlyList<(LabelState, LabelOutcome)> prevLabelStates = prevJob.GetLabelStates(prevGraph);

				// Match label by dashboardName + dashboardCategory
				for (int i = 0; i < prevGraph.Labels.Count && i < prevLabelStates.Count; i++)
				{
					ILabel prevLabel = prevGraph.Labels[i];
					if (String.Equals(prevLabel.DashboardName, label.DashboardName, StringComparison.Ordinal) &&
						String.Equals(prevLabel.DashboardCategory, label.DashboardCategory, StringComparison.Ordinal))
					{
						(LabelState prevState, LabelOutcome prevOutcome) = prevLabelStates[i];
						int change = prevJob.CommitId.TryGetPerforceChange() ?? 0;
						return new GetLabelPreviousBuildResponse
						{
							JobId = prevJob.Id.ToString(),
							Change = change,
							LabelIndex = i,
							State = prevState,
							Outcome = prevOutcome
						};
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Failed to fetch previous build for job {JobId}", job.Id);
			}

			return null;
		}

		async Task<GetLabelCrossStreamChainResponse?> GetLabelCrossStreamChainAsync(IJob job, ILabel label, LabelState currentState, LabelOutcome currentOutcome, CancellationToken cancellationToken)
		{
			try
			{
				// Resolve current stream's Perforce path
				StreamConfig? currentStreamConfig;
				if (!_buildConfig.Value.TryGetStream(job.StreamId, out currentStreamConfig))
				{
					return null;
				}

				string currentStreamPath = currentStreamConfig.Name;
				IReadOnlyList<MergeGraphBranchRef> branchRefs = _mergeGraphService.FindBranchesForStreamPath(currentStreamPath);
				if (branchRefs.Count == 0)
				{
					return null;
				}

				string botName = branchRefs[0].BotName;
				MergeChain? chain = _mergeGraphService.GetMergeChain(botName);
				if (chain == null || chain.Entries.Count == 0)
				{
					return null;
				}

				// Build StreamPath → StreamId lookup from BuildConfig
				Dictionary<string, StreamId> streamPathToId = new(StringComparer.OrdinalIgnoreCase);
				foreach (StreamConfig sc in _buildConfig.Value.Streams)
				{
					streamPathToId.TryAdd(sc.Name, sc.Id);
				}

				// Collect sibling stream lookups
				List<(MergeChainEntry Entry, StreamId StreamId)> resolvedEntries = new();
				foreach (MergeChainEntry entry in chain.Entries)
				{
					if (streamPathToId.TryGetValue(entry.StreamPath, out StreamId streamId))
					{
						resolvedEntries.Add((entry, streamId));
					}
				}

				if (resolvedEntries.Count == 0)
				{
					return null;
				}

				// Parallel lookup of latest job per sibling stream
				Task<IReadOnlyList<IJob>>[] jobTasks = resolvedEntries
					.Where(e => e.StreamId != job.StreamId) // Skip current stream (we already have the data)
					.Select(e => _jobService.FindJobsByStreamWithTemplatesAsync(
						e.StreamId,
						new[] { job.TemplateId },
						count: 1,
						consistentRead: false,
						cancellationToken: cancellationToken))
					.ToArray();

				IReadOnlyList<IJob>[] siblingJobResults = await Task.WhenAll(jobTasks);

				// Build a map of streamId → sibling job
				Dictionary<StreamId, IJob> siblingJobs = new();
				int taskIdx = 0;
				foreach ((MergeChainEntry entry, StreamId streamId) in resolvedEntries)
				{
					if (streamId != job.StreamId)
					{
						IReadOnlyList<IJob> jobs = siblingJobResults[taskIdx++];
						if (jobs.Count > 0)
						{
							siblingJobs[streamId] = jobs[0];
						}
					}
				}

				// Build response entries
				GetLabelCrossStreamChainResponse chainResponse = new GetLabelCrossStreamChainResponse { BotName = botName };

				foreach ((MergeChainEntry entry, StreamId streamId) in resolvedEntries)
				{
					GetLabelCrossStreamEntry crossEntry = new GetLabelCrossStreamEntry
					{
						StreamId = streamId.ToString(),
						BranchName = entry.Branch.Name,
						HasApprovalGateBefore = entry.HasApprovalGateBefore,
						IsCurrent = streamId == job.StreamId
					};

					if (streamId == job.StreamId)
					{
						// Use current job's label data (already computed by caller)
						crossEntry.JobId = job.Id.ToString();
						crossEntry.State = currentState;
						crossEntry.Outcome = currentOutcome;
					}
					else if (siblingJobs.TryGetValue(streamId, out IJob? siblingJob))
					{
						IGraph siblingGraph = await _jobService.GetGraphAsync(siblingJob, cancellationToken);
						IReadOnlyList<(LabelState, LabelOutcome)> siblingStates = siblingJob.GetLabelStates(siblingGraph);

						for (int i = 0; i < siblingGraph.Labels.Count && i < siblingStates.Count; i++)
						{
							ILabel siblingLabel = siblingGraph.Labels[i];
							if (String.Equals(siblingLabel.DashboardName, label.DashboardName, StringComparison.Ordinal) &&
								String.Equals(siblingLabel.DashboardCategory, label.DashboardCategory, StringComparison.Ordinal))
							{
								crossEntry.JobId = siblingJob.Id.ToString();
								crossEntry.State = siblingStates[i].Item1;
								crossEntry.Outcome = siblingStates[i].Item2;
								break;
							}
						}
					}

					chainResponse.Entries.Add(crossEntry);
				}

				return chainResponse.Entries.Count > 0 ? chainResponse : null;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Failed to fetch cross-stream chain for job {JobId}", job.Id);
				return null;
			}
		}
	}
}
