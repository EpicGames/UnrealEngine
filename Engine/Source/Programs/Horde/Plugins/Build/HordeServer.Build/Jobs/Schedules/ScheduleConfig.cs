// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using HordeServer.Streams;

namespace HordeServer.Jobs.Schedules
{
	/// <summary>
	/// Parameters to create a new schedule
	/// </summary>
	public class ScheduleConfig
	{
		/// <summary>
		/// Whether the schedule should be enabled
		/// </summary>
		public bool Enabled { get; set; } = true;

		/// <summary>
		/// Condition for whether the schedule should be enabled.
		/// When a condition is provided the result of the expression will be logically AND with the <see cref="Enabled"/> property to decide
		/// if the schedule should be enabled.
		/// </summary>
		public Condition? Condition { get; set; }

		/// <summary>
		/// Maximum number of builds that can be active at once
		/// </summary>
		public int MaxActive { get; set; }

		/// <summary>
		/// Maximum number of changes the schedule can fall behind head revision. If greater than zero, builds will be triggered for every submitted changelist until the backlog is this size.
		/// </summary>
		public int MaxChanges { get; set; }

		/// <summary>
		/// Whether the build requires a change to be submitted
		/// </summary>
		public bool RequireSubmittedChange { get; set; } = true;

		/// <summary>
		/// Gate allowing the schedule to trigger
		/// </summary>
		public ScheduleGateConfig? Gate { get; set; }

		/// <summary>
		/// Commit tags for this schedule
		/// </summary>
		public List<CommitTag> Commits { get; set; } = new List<CommitTag>();

		/// <summary>
		/// Roles to impersonate for this schedule
		/// </summary>
		public List<ScheduleClaimConfig>? Claims { get; set; }

		/// <summary>
		/// The types of changes to run for
		/// </summary>
		[Obsolete("Use Commits instead")]
		public List<ChangeContentFlags>? Filter { get; set; }

		/// <summary>
		/// Files that should cause the job to trigger
		/// </summary>
		public List<string>? Files { get; set; }

		/// <summary>
		/// Parameters for the template
		/// </summary>
		public Dictionary<string, string> TemplateParameters { get; set; } = new Dictionary<string, string>();

		/// <summary>
		/// New patterns for the schedule
		/// </summary>
		public List<SchedulePatternConfig> Patterns { get; set; } = new List<SchedulePatternConfig>();
	}

	/// <summary>
	/// Claim to grant to leases running a scheduled job
	/// </summary>
	public class ScheduleClaimConfig : IScheduleClaim
	{
		/// <inheritdoc/>
		[Required]
		public string Type { get; set; } = String.Empty;

		/// <inheritdoc/>
		[Required]
		public string Value { get; set; } = String.Empty;
	}

	/// <summary>
	/// Gate allowing a schedule to trigger.
	/// </summary>
	public class ScheduleGateConfig : IScheduleGate
	{
		/// <summary>
		/// The template containing the dependency
		/// </summary>
		[Required]
		public TemplateId TemplateId { get; set; }

		/// <summary>
		/// Target to wait for
		/// </summary>
		[Required]
		public string Target { get; set; } = String.Empty;
	}

	/// <summary>
	/// Parameters to create a new schedule
	/// </summary>
	public class SchedulePatternConfig : ISchedulePattern
	{
		/// <summary>
		/// Days of the week to run this schedule on. If null, the schedule will run every day.
		/// </summary>
		public List<DayOfWeek>? DaysOfWeek { get; set; }

		/// <summary>
		/// Time during the day for the first schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		public ScheduleTimeOfDay MinTime { get; set; } = new ScheduleTimeOfDay(0);

		/// <summary>
		/// Time during the day for the last schedule to trigger. Measured in minutes from midnight.
		/// </summary>
		public ScheduleTimeOfDay? MaxTime { get; set; }

		/// <summary>
		/// Interval between each schedule triggering
		/// </summary>
		public ScheduleInterval? Interval { get; set; }

		IReadOnlyList<DayOfWeek>? ISchedulePattern.DaysOfWeek => DaysOfWeek;

		/// <summary>
		/// Constructor
		/// </summary>
		public SchedulePatternConfig()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="daysOfWeek">Which days of the week the schedule should run</param>
		/// <param name="minTime">Time during the day for the first schedule to trigger. Measured in minutes from midnight.</param>
		/// <param name="maxTime">Time during the day for the last schedule to trigger. Measured in minutes from midnight.</param>
		/// <param name="interval">Interval between each schedule triggering</param>
		public SchedulePatternConfig(List<DayOfWeek>? daysOfWeek, int minTime, int? maxTime, int? interval)
		{
			DaysOfWeek = daysOfWeek;
			MinTime = new ScheduleTimeOfDay(minTime);
			MaxTime = (maxTime != null) ? new ScheduleTimeOfDay(maxTime.Value) : null;
			Interval = (interval != null) ? new ScheduleInterval(interval.Value) : null;
		}
	}
}
