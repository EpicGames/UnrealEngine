// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Redis;
using EpicGames.Redis.Utility;
using EpicGames.Serialization;
using HordeServer.Acls;
using HordeServer.Auditing;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.Templates;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using StackExchange.Redis;

namespace HordeServer.Jobs.Schedules
{
	/// <summary>
	/// Identifier for writing log messages to a particular template
	/// </summary>
	public record class ScheduleId(StreamId StreamId, TemplateId TemplateId)
	{
		/// <inheritdoc/>
		public override string ToString()
			=> $"sched:{StreamId}:{TemplateId}";
	}

	/// <summary>
	/// Manipulates schedule instances
	/// </summary>
	public sealed class ScheduleService : IHostedService, IAsyncDisposable
	{
		[RedisConverter(typeof(RedisCbConverter<QueueItem>))]
		class QueueItem
		{
			[CbField("sid")]
			public StreamId StreamId { get; set; }

			[CbField("tid")]
			public TemplateId TemplateId { get; set; }

			public QueueItem()
			{
			}

			public QueueItem(StreamId streamId, TemplateId templateId)
			{
				StreamId = streamId;
				TemplateId = templateId;
			}

			public static DateTime GetTimeFromScore(double score) => DateTime.UnixEpoch + TimeSpan.FromSeconds(score);
			public static double GetScoreFromTime(DateTime time) => (time.ToUniversalTime() - DateTime.UnixEpoch).TotalSeconds;
		}

		readonly IAuditLog<ScheduleId> _auditLog;
		readonly IGraphCollection _graphs;
		readonly IJobCollection _jobCollection;
		readonly IDowntimeService _downtimeService;
		readonly JobService _jobService;
		readonly IStreamCollection _streamCollection;
		readonly ITemplateCollectionInternal _templateCollection;
		readonly IClock _clock;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly IRedisService _redis;
		static readonly RedisKey s_baseLockKey = "scheduler/locks";
		static readonly RedisKey s_tickLockKey = s_baseLockKey.Append("/tick"); // Lock to tick the queue
		static readonly RedisSortedSetKey<QueueItem> s_queueKey = "scheduler/queue"; // Items to tick, ordered by time
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITicker _ticker;

		/// <summary>
		/// Constructor
		/// </summary>
		public ScheduleService(IAuditLog<ScheduleId> auditLog, IRedisService redis, IGraphCollection graphs, IJobCollection jobCollection, JobService jobService, IDowntimeService downtimeService, IStreamCollection streamCollection, ITemplateCollectionInternal templateCollection, IMongoService mongoService, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, Tracer tracer, ILogger<ScheduleService> logger)
		{
			_auditLog = auditLog;
			_graphs = graphs;
			_jobCollection = jobCollection;
			_jobService = jobService;
			_downtimeService = downtimeService;
			_streamCollection = streamCollection;
			_templateCollection = templateCollection;
			_clock = clock;
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;

			_redis = redis;
			if (mongoService.ReadOnlyMode)
			{
				_ticker = new NullTicker();
			}
			else
			{
				_ticker = clock.AddTicker<ScheduleService>(TimeSpan.FromMinutes(1.0), TickAsync, logger);
			}
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
		}

		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ScheduleService)}.{nameof(TickAsync)}");
			DateTime utcNow = _clock.UtcNow;

			// Don't start any new jobs during scheduled downtime
			bool isDowntimeActive = _downtimeService.IsDowntimeActive;
			if (isDowntimeActive)
			{
				span.SetAttribute("horde.is_downtime", isDowntimeActive);
				return;
			}

			// Update the current queue
			await using (RedisLock sharedLock = new(_redis.GetDatabase(), s_tickLockKey))
			{
				using TelemetrySpan tickLockSpan = _tracer.StartActiveSpan($"{nameof(ScheduleService)}.{nameof(TickAsync)}.TickLock");
				bool tickLockAcquired = await sharedLock.AcquireAsync(TimeSpan.FromMinutes(1.0), false);
				span.SetAttribute("horde.lock_acquired", tickLockAcquired);

				if (tickLockAcquired)
				{
					await UpdateQueueAsync(utcNow, cancellationToken);
				}
			}

			// Keep updating schedules
			while (!cancellationToken.IsCancellationRequested)
			{
				// Get the item with the lowest score (ie. the one that hasn't been updated in the longest time)
				QueueItem? item = await PopQueueItemAsync();
				if (item == null)
				{
					break;
				}

				// Acquire the lock for this schedule and update it
				await using (RedisLock sharedLock = new RedisLock<QueueItem>(_redis.GetDatabase(), s_baseLockKey, item))
				{
					using TelemetrySpan baseLockSpan = _tracer.StartActiveSpan($"{nameof(ScheduleService)}.{nameof(TickAsync)}.Tick");
					bool baseLockAcquired = await sharedLock.AcquireAsync(TimeSpan.FromMinutes(1.0));
					span.SetAttribute("horde.lock_acquired", baseLockAcquired);
					span.SetAttribute("horde.stream_id", item.StreamId);
					span.SetAttribute("horde.template_id", item.TemplateId);
					if (baseLockAcquired)
					{
						try
						{
							await TriggerAsync(item.StreamId, item.TemplateId, utcNow, cancellationToken);
						}
						catch (OperationCanceledException)
						{
							throw;
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Error while updating schedule for {StreamId}/{TemplateId}", item.StreamId, item.TemplateId);
						}
					}
				}
			}
		}

		async Task<QueueItem?> PopQueueItemAsync()
		{
			IDatabaseAsync target = _redis.GetDatabase();
			for (; ; )
			{
				QueueItem[] items = await target.SortedSetRangeByRankAsync(s_queueKey, 0, 0);
				if (items.Length == 0)
				{
					return null;
				}
				if (await target.SortedSetRemoveAsync(s_queueKey, items[0]))
				{
					return items[0];
				}
			}
		}

		internal async Task ResetAsync()
		{
			IDatabase redis = _redis.GetDatabase();
			await redis.KeyDeleteAsync(s_queueKey);
			await redis.KeyDeleteAsync(s_tickLockKey);
		}

		internal async Task TickForTestingAsync()
		{
			await UpdateQueueAsync(_clock.UtcNow, CancellationToken.None);
			await TickAsync(CancellationToken.None);
		}

		/// <summary>
		/// Get the current set of streams and ensure there's an entry for each item
		/// </summary>
		public async Task UpdateQueueAsync(DateTime utcNow, CancellationToken cancellationToken)
		{
			const string SpanName = $"{nameof(ScheduleService)}.{nameof(UpdateQueueAsync)}";
			using TelemetrySpan span = _tracer.StartActiveSpan(SpanName);
			List<SortedSetEntry<QueueItem>> queueItems = new List<SortedSetEntry<QueueItem>>();

			BuildConfig buildConfig = _buildConfig.CurrentValue;
			foreach (StreamConfig streamConfig in buildConfig.Streams)
			{
				using TelemetrySpan streamSpan = _tracer.StartActiveSpan(SpanName + ".Stream");
				streamSpan.SetAttribute("horde.streamId", streamConfig.Id);
				IStream? stream = await _streamCollection.GetAsync(streamConfig.Id, cancellationToken);
				if (stream == null)
				{
					continue;
				}

				foreach ((TemplateId templateId, ITemplateRef templateRef) in stream.Templates)
				{
					if (templateRef.Schedule != null)
					{
						DateTime? nextTriggerTimeUtc = templateRef.Schedule.GetNextTriggerTimeUtc(_clock.TimeZone);
						if (nextTriggerTimeUtc != null)
						{
							if (utcNow > nextTriggerTimeUtc.Value)
							{
								double score = QueueItem.GetScoreFromTime(nextTriggerTimeUtc.Value);
								queueItems.Add(new SortedSetEntry<QueueItem>(new QueueItem(stream.Id, templateId), score));

								await stream.TryUpdateScheduleTriggerAsync(templateId, utcNow, cancellationToken: cancellationToken);
							}
						}
					}
				}
			}

			await _redis.GetDatabase().SortedSetAddAsync(s_queueKey, queueItems.ToArray());
		}

		/// <summary>
		/// Gets the audit log for a particular template
		/// </summary>
		/// <param name="streamId">Stream for the schedule</param>
		/// <param name="templateId"></param>
		public IAuditLogChannel GetAuditLog(StreamId streamId, TemplateId templateId)
			=> _auditLog[new ScheduleId(streamId, templateId)];

		/// <summary>
		/// Trigger a schedule to run
		/// </summary>
		/// <param name="streamId">Stream for the schedule</param>
		/// <param name="templateId"></param>
		/// <param name="utcNow"></param>
		/// <param name="cancellationToken"></param>
		/// <returns>Async task</returns>
		internal async Task<bool> TriggerAsync(StreamId streamId, TemplateId templateId, DateTime utcNow, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			if (!buildConfig.TryGetStream(streamId, out StreamConfig? streamConfig))
			{
				return false;
			}

			IStream? stream = await _streamCollection.GetAsync(streamConfig.Id, cancellationToken);
			if (stream == null || !stream.Templates.TryGetValue(templateId, out ITemplateRef? templateRef))
			{
				return false;
			}

			ISchedule? schedule = templateRef.Schedule;
			if (schedule == null)
			{
				return false;
			}

			ForwardingLogger scheduleLogger = new ForwardingLogger(_logger, GetAuditLog(streamId, templateId));
			if (!schedule.Enabled)
			{
				scheduleLogger.LogDebug("Schedule is disabled. No builds started.");
				return false;
			}

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(ScheduleService)}.{nameof(TriggerAsync)}");
			span.SetAttribute("horde.stream_id", stream.Id);
			span.SetAttribute("horde.template_id", templateId);

			Stopwatch stopwatch = Stopwatch.StartNew();
			_logger.LogInformation("Updating schedule for {StreamId} template {TemplateId}", stream.Id, templateId);

			// Get a list of jobs that we need to remove
			List<JobId> removeJobIds = new List<JobId>();
			foreach (JobId activeJobId in schedule.ActiveJobs)
			{
				IJob? job = await _jobService.GetJobAsync(activeJobId, cancellationToken);
				if (job == null || job.Batches.All(x => x.State == JobStepBatchState.Complete))
				{
					_logger.LogInformation("Removing active job {JobId}", activeJobId);
					removeJobIds.Add(activeJobId);
				}
			}
			await stream.TryUpdateScheduleTriggerAsync(templateId, removeJobs: removeJobIds, cancellationToken: cancellationToken);

			// If the stream is paused, bail out
			if (stream.IsPaused(utcNow))
			{
				scheduleLogger.LogInformation("Skipping schedule update for stream {StreamId}. It has been paused until {PausedUntil} with comment '{PauseComment}'.", stream.Id, stream.PausedUntil, stream.PauseComment);
				return false;
			}

			// Trigger this schedule
			try
			{
				await TriggerAsync(stream, templateId, templateRef, schedule, schedule.ActiveJobs.Count - removeJobIds.Count, utcNow, scheduleLogger, span, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to start schedule {StreamId}/{TemplateId}", stream.Id, templateId);
			}

			// Print some timing info
			stopwatch.Stop();
			_logger.LogInformation("Updated schedule for {StreamId} template {TemplateId} in {TimeSeconds}ms", stream.Id, templateId, (long)stopwatch.Elapsed.TotalMilliseconds);
			return true;
		}

		/// <summary>
		/// Trigger a schedule to run
		/// </summary>
		/// <param name="stream">Stream for the schedule</param>
		/// <param name="templateId"></param>
		/// <param name="templateRef"></param>
		/// <param name="schedule"></param>
		/// <param name="numActiveJobs"></param>
		/// <param name="utcNow">The current time</param>
		/// <param name="scheduleLogger"></param>
		/// <param name="linkedSpan">Span to link with</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Async task</returns>
		private async Task TriggerAsync(IStream stream, TemplateId templateId, ITemplateRef templateRef, ISchedule schedule, int numActiveJobs, DateTime utcNow, ILogger scheduleLogger, TelemetrySpan linkedSpan, CancellationToken cancellationToken)
		{
			// Create a new root span as it may contain lots of child spans, this to prevent a huge single trace/span
			using TelemetrySpan span = _tracer.StartRootSpan($"{nameof(ScheduleService)}.{nameof(TriggerAsync)}.Full", links: [new Link(linkedSpan.Context)]);
			span.SetAttribute("horde.stream_id", stream.Id);
			span.SetAttribute("horde.template_id", templateId);
			span.SetAttribute("horde.active_jobs", numActiveJobs);
			
			// Check we're not already at the maximum number of allowed jobs
			if (schedule.MaxActive != 0 && numActiveJobs >= schedule.MaxActive)
			{
				scheduleLogger.LogInformation("Skipping trigger of {StreamId} template {TemplateId} - already have maximum number of jobs running ({NumJobs})", stream.Id, templateId, schedule.MaxActive);
				foreach (JobId jobId in schedule.ActiveJobs)
				{
					_logger.LogInformation("Active job for {StreamId} template {TemplateId}: {JobId}", stream.Id, templateId, jobId);
				}
				return;
			}

			// Get the stream config
			StreamConfig? streamConfig;
			if (!_buildConfig.CurrentValue.TryGetStream(stream.Id, out streamConfig))
			{
				return;
			}

			TemplateRefConfig? templateRefConfig;
			if (!streamConfig.TryGetTemplate(templateRef.Id, out templateRefConfig))
			{
				return;
			}

			// Minimum changelist number, inclusive
			CommitIdWithOrder? minCommitId = schedule.LastTriggerCommitId;
			bool includeMinCommitId = !schedule.RequireSubmittedChange;

			// Maximum changelist number, exclusive
			CommitIdWithOrder? maxCommitId = null;

			// Get the maximum number of changes to trigger
			int maxNewChanges = 1;
			if (schedule.MaxChanges != 0)
			{
				maxNewChanges = schedule.MaxChanges;
			}
			if (schedule.MaxActive != 0)
			{
				maxNewChanges = Math.Min(maxNewChanges, schedule.MaxActive - numActiveJobs);
			}

			// Create a timer to limit the amount we look back through P4 history
			Stopwatch timer = Stopwatch.StartNew();

			// Create a file filter
			FileFilter? fileFilter = null;
			if (schedule.Files != null)
			{
				fileFilter = new FileFilter(schedule.Files);
			}

			// Cache the Perforce history as we're iterating through changes to improve query performance
			ICommitCollection commits = stream.Commits;
			IAsyncEnumerable<ICommit> commitEnumerable = commits.FindAsync(minCommitId, includeMinCommitId, tags: schedule.Commits, cancellationToken: cancellationToken);
			await using IAsyncEnumerator<ICommit> commitEnumerator = commitEnumerable.GetAsyncEnumerator(cancellationToken);

			// Start as many jobs as possible
			int totalExamined = 0;
			int totalFiltered = 0;
			List<(CommitIdWithOrder CommitId, CommitIdWithOrder CodeCommitId)> triggerChanges = new List<(CommitIdWithOrder, CommitIdWithOrder)>();
			while (triggerChanges.Count < maxNewChanges)
			{
				cancellationToken.ThrowIfCancellationRequested();

				// Get the next valid change
				CommitIdWithOrder? commitId;
				ICommit? commit;

				if (schedule.Gate != null)
				{
					commitId = await GetNextChangeForGateAsync(stream.Id, templateId, schedule.Gate, minCommitId, maxCommitId, scheduleLogger, cancellationToken);
					commit = (commitId != null)? await commits.FindAsync(commitId, commitId, 1, null, cancellationToken).FirstOrDefaultAsync(cancellationToken) : null; // May be a change in a different stream
				}
				else if (await commitEnumerator.MoveNextAsync())
				{
					commit = commitEnumerator.Current;
					commitId = commit.Id;
				}
				else
				{
					commit = null;
					commitId = null;
				}

				// Quit if we didn't find anything
				if (commitId == null)
				{
					break;
				}
				if (minCommitId != null)
				{
					if (commitId < minCommitId)
					{
						break;
					}
					if (commitId == minCommitId && (schedule.RequireSubmittedChange || triggerChanges.Count > 0))
					{
						break;
					}
				}

				// Adjust the changelist for the desired filter
				if (commit != null)
				{
					totalExamined++;
				}
				if (commit == null || await ShouldBuildChangeAsync(commit, schedule.Commits, fileFilter, cancellationToken))
				{
					CommitIdWithOrder codeCommitId = commitId;

					ICommit? lastCodeCommit = await commits.GetLastCodeChangeAsync(commitId, cancellationToken);
					if (lastCodeCommit != null)
					{
						codeCommitId = lastCodeCommit.Id;
					}
					else
					{
						_logger.LogWarning("Unable to find code change for CL {Change}", commitId);
					}

					triggerChanges.Add((commitId, codeCommitId));
				}
				else
				{
					totalFiltered++;
				}

				// Check we haven't exceeded the time limit
				if (timer.Elapsed > TimeSpan.FromMinutes(2.0))
				{
					scheduleLogger.LogError("Querying for changes to trigger for {StreamId} template {TemplateId} has taken {Time}. Aborting.", stream.Id, templateId, timer.Elapsed);
					break;
				}

				// Update the remaining range of changes to check for
				maxCommitId = commitId;
				if (minCommitId != null && maxCommitId < minCommitId)
				{
					break;
				}
			}

			// Early out if there's nothing to do
			if (triggerChanges.Count == 0)
			{
				scheduleLogger.LogInformation("Skipping trigger of {StreamName} template {TemplateId} - no candidate changes after CL {LastTriggerChange}", stream.Id, templateId, schedule.LastTriggerCommitId);
				if (maxNewChanges <= 0)
				{
					scheduleLogger.LogInformation("MaxNewChanges is {MaxNewChanges} for {StreamId} template {TemplateId} - no changes were evaluated", maxNewChanges, stream.Id, templateId);
				}
				else if (totalExamined > 0)
				{
					scheduleLogger.LogInformation("Examined {TotalExamined} changes after CL {LastTriggerChange}, all {TotalFiltered} were filtered out", totalExamined, schedule.LastTriggerCommitId, totalFiltered);
				}
				return;
			}

			// Get the matching template
			ITemplate template = await _templateCollection.GetOrAddAsync(templateRefConfig);

			// Register the graph for it
			IGraph graph = await _graphs.AddAsync(template, cancellationToken);

			// We may need to submit a new change for any new jobs. This only makes sense if there's one change.
			if (template.SubmitNewChange != null)
			{
				CommitIdWithOrder newCommitId = await commits.CreateNewAsync(template, cancellationToken);
				ICommit? newCodeCommit = await commits.GetLastCodeChangeAsync(newCommitId, cancellationToken);
				triggerChanges = new List<(CommitIdWithOrder, CommitIdWithOrder)> { (newCommitId, newCodeCommit?.Id ?? newCommitId) };
			}

			// Try to start all the new jobs
			_logger.LogInformation("Starting {NumJobs} new jobs for {StreamId} template {TemplateId} (active: {NumActive}, max new: {MaxNewJobs})", triggerChanges.Count, stream.Id, templateId, numActiveJobs, maxNewChanges);
			foreach ((CommitIdWithOrder commitId, CommitIdWithOrder codeCommitId) in triggerChanges.OrderBy(x => x.CommitId))
			{
				cancellationToken.ThrowIfCancellationRequested();

				CreateJobOptions options = new (templateRefConfig);
				options.Priority = template.Priority;
				options.Claims = schedule.Claims?.Select(x => new AclClaimConfig(x.Type, x.Value)).ToList() ?? [];
				template.GetDefaultParameters(options.Parameters, true);
				template.GetArgumentsForParameters(options.Parameters, options.Arguments);

				IJob newJob = await _jobService.CreateJobAsync(null, streamConfig, templateId, template.Hash, graph, template.Name, commitId, codeCommitId, options, cancellationToken);
				scheduleLogger.LogInformation("Started new job for {StreamId} template {TemplateId} at CL {Change} (Code CL {CodeChange}): {JobId}", stream.Id, templateId, commitId, codeCommitId, newJob.Id);
				await stream.TryUpdateScheduleTriggerAsync(templateId, utcNow, commitId, new List<JobId> { newJob.Id }, new List<JobId>(), cancellationToken);
			}
		}

		/// <summary>
		/// Tests whether a schedule should build a particular change, based on its requested change filters
		/// </summary>
		/// <param name="commit">The commit details</param>
		/// <param name="filterTags">Filter for the tags to trigger a build</param>
		/// <param name="fileFilter">Filter for the files to trigger a build</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		private async ValueTask<bool> ShouldBuildChangeAsync(ICommit commit, IReadOnlyList<CommitTag>? filterTags, FileFilter? fileFilter, CancellationToken cancellationToken)
		{
			if (Regex.IsMatch(commit.Description, @"^\s*#\s*skipci", RegexOptions.Multiline))
			{
				return false;
			}
			if (filterTags != null && filterTags.Count > 0)
			{
				IReadOnlyList<CommitTag> commitTags = await commit.GetTagsAsync(cancellationToken);
				if (!commitTags.Any(x => filterTags.Contains(x)))
				{
					_logger.LogDebug("Not building change {Change} ({ChangeTags}) due to filter tags ({FilterTags})", commit.Id, String.Join(", ", commitTags.Select(x => x.ToString())), String.Join(", ", filterTags.Select(x => x.ToString())));
					return false;
				}
			}
			if (fileFilter != null)
			{
				if (!await commit.MatchesFilterAsync(fileFilter, cancellationToken))
				{
					_logger.LogDebug("Not building change {Change} due to file filter", commit.Id);
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Gets the next change to build for a schedule on a gate
		/// </summary>
		private async Task<CommitIdWithOrder?> GetNextChangeForGateAsync(StreamId streamId, TemplateId templateRefId, IScheduleGate gate, CommitIdWithOrder? minCommitId, CommitIdWithOrder? maxCommitId, ILogger scheduleLogger, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartRootSpan($"{nameof(ScheduleService)}.{nameof(GetNextChangeForGateAsync)}");
			span.SetAttribute("horde.gate.template_id", gate.TemplateId);
			span.SetAttribute("horde.gate.target", gate.Target);
			span.SetAttribute("horde.min_commit_id", minCommitId?.ToString());
			span.SetAttribute("horde.max_commit_id", maxCommitId?.ToString());

			int jobsEvaluated = 0;
			int jobsSkippedFailed = 0;
			int jobsSkippedNotComplete = 0;
			int jobsSkippedNoTarget = 0;

			for (int c = 0; ; c++)
			{
				cancellationToken.ThrowIfCancellationRequested();
				span.SetAttribute("horde.iteration", c);
				span.AddEvent("Fetching job", new SpanAttributes(new KeyValuePair<string, object?>[] { new("horde.max_commit_id", maxCommitId) }));

				FindJobOptions options = new (StreamId: streamId, Templates: [gate.TemplateId], MinCommitId: minCommitId, MaxCommitId: maxCommitId);
				IReadOnlyList<IJob> jobs = await _jobCollection.FindAsync(options, count: 2, cancellationToken: cancellationToken);

				IJob? job = jobs.FirstOrDefault(x => maxCommitId == null || x.CommitId < maxCommitId);
				if (job == null)
				{
					span.AddEvent("Gate search exhausted", new SpanAttributes(new KeyValuePair<string, object?>[]
					{
						new("horde.jobs_evaluated", jobsEvaluated),
						new("horde.jobs_skipped_failed", jobsSkippedFailed),
						new("horde.jobs_skipped_not_complete", jobsSkippedNotComplete),
						new("horde.jobs_skipped_no_target", jobsSkippedNoTarget)
					}));

					if (jobsEvaluated > 0)
					{
						scheduleLogger.LogInformation("Gate search for {StreamId} template {TemplateId} (gate: {GateTemplateId}/{GateTarget}) exhausted after evaluating {JobsEvaluated} jobs: {Failed} failed, {NotComplete} not complete, {NoTarget} missing target",
							streamId, templateRefId, gate.TemplateId, gate.Target, jobsEvaluated, jobsSkippedFailed, jobsSkippedNotComplete, jobsSkippedNoTarget);
					}
					return null;
				}

				jobsEvaluated++;

				IGraph? graph = await _graphs.GetAsync(job.GraphHash, cancellationToken);
				if (graph != null)
				{
					(JobStepState stepState, JobStepOutcome stepOutcome)? state = job.GetTargetState(graph, gate.Target);

					if (state == null)
					{
						jobsSkippedNoTarget++;
						_logger.LogDebug("Gate: Job {JobId} (CL {CommitId}) - target '{Target}' not found in graph or missing steps", job.Id, job.CommitId, gate.Target);
					}
					else if (state is { stepState: JobStepState.Completed })
					{
						if (state is { stepOutcome: JobStepOutcome.Success or JobStepOutcome.Warnings})
						{
							span.SetAttribute("horde.jobs_evaluated", jobsEvaluated);
							return job.CommitId;
						}
						jobsSkippedFailed++;
						_logger.LogDebug("Gate: Job {JobId} (CL {CommitId}) - target '{Target}' completed with {Outcome}", job.Id, job.CommitId, gate.Target, state.Value.stepOutcome);
						scheduleLogger.LogInformation("Skipping trigger of {StreamName} template {TemplateId} - last {OtherTemplateRefId} job ({JobId}) ended with errors", streamId, templateRefId, gate.TemplateId, job.Id);
					}
					else
					{
						jobsSkippedNotComplete++;
						_logger.LogDebug("Gate: Job {JobId} (CL {CommitId}) - target '{Target}' not complete (state: {State})", job.Id, job.CommitId, gate.Target, state.Value.stepState);
					}
				}

				maxCommitId = job.CommitId;
			}
		}
	}
}
