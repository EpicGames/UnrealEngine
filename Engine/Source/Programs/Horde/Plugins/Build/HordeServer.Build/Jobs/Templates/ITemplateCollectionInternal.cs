// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs.Templates;

namespace HordeServer.Jobs.Templates
{
	/// <summary>
	/// Internal implementation of <see cref="ITemplateCollection"/> which
	/// allows registering new template instances.
	/// </summary>
	public interface ITemplateCollectionInternal : ITemplateCollection
	{
		/// <summary>
		/// Gets a template instance from its configuration
		/// </summary>
		/// <param name="templateConfig">The template configuration</param>
		/// <returns>Template instance</returns>
		Task<ITemplate> GetOrAddAsync(TemplateConfig templateConfig);
	}
}
