// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using HordeServer.Server;
using Microsoft.Extensions.Caching.Memory;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Jobs.Templates
{
	/// <summary>
	/// Collection of template documents
	/// </summary>
	public sealed class TemplateCollection : ITemplateCollectionInternal, IDisposable
	{
		/// <summary>
		/// Document describing a job template. These objects are considered immutable once created and uniquely referenced by hash, in order to de-duplicate across all job runs.
		/// </summary>
		class TemplateDocument : ITemplate
		{
			[BsonRequired, BsonId]
			public ContentHash Id { get; private set; } = ContentHash.Empty;

			[BsonRequired]
			public string Name { get; private set; }

			[BsonIgnoreIfNull]
			public string? Description { get; set; }

			public Priority? Priority { get; private set; }
			public bool AllowPreflights { get; set; } = true;
			public bool UpdateIssues { get; set; } = false;
			public bool PromoteIssuesByDefault { get; set; } = false;
			public string? InitialAgentType { get; set; }

			[BsonIgnoreIfNull]
			public string? SubmitNewChange { get; set; }

			[BsonIgnoreIfNull]
			public string? SubmitDescription { get; set; }

			public List<string> Arguments { get; private set; } = new List<string>();
			public List<ParameterDocument> Parameters { get; private set; } = new List<ParameterDocument>();

			ContentHash ITemplate.Hash => Id;
			IReadOnlyList<string> ITemplate.Arguments => Arguments;
			IReadOnlyList<ITemplateParameter> ITemplate.Parameters => Parameters;

			[BsonConstructor]
			private TemplateDocument()
			{
				Name = null!;
			}

			public TemplateDocument(TemplateConfig config)
			{
				Name = config.Name;
				Description = config.Description;
				Priority = config.Priority;
				AllowPreflights = config.AllowPreflights;
				UpdateIssues = config.UpdateIssues;
				PromoteIssuesByDefault = config.PromoteIssuesByDefault;
				InitialAgentType = config.InitialAgentType;
				SubmitNewChange = config.SubmitNewChange;
				SubmitDescription = config.SubmitDescription;
				Arguments = config.Arguments ?? new List<string>();
				Parameters = config.Parameters.ConvertAll(x => CreateParameterDocument(x));

				// Compute the hash once all other fields have been set
				Id = ContentHash.SHA1(BsonExtensionMethods.ToBson(this));
			}

			static ParameterDocument CreateParameterDocument(TemplateParameterConfig config)
			{
				return config switch
				{
					TemplateBoolParameterConfig boolParameter
						=> new BoolParameterDocument(boolParameter),

					TemplateTextParameterConfig textParameter
						=> new TextParameterDocument(textParameter),

					TemplateListParameterConfig listParameter
						=> new ListParameterDocument(listParameter),

					_ => throw new NotImplementedException()
				};
			}
		}

		[BsonDiscriminator(RootClass = true)]
		[BsonKnownTypes(typeof(BoolParameterDocument), typeof(TextParameterDocument), typeof(ListParameterDocument))]
		abstract class ParameterDocument : ITemplateParameter
		{
			public abstract void GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments);
			public abstract void GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild);
		}

		[BsonDiscriminator("BoolParameter")]
		sealed class BoolParameterDocument : ParameterDocument, ITemplateBoolParameter
		{
			public ParameterId Id { get; set; }
			public string Label { get; set; }

			[BsonIgnoreIfNull]
			public string? ArgumentIfEnabled { get; set; }

			[BsonIgnoreIfNull]
			public List<string>? ArgumentsIfEnabled { get; set; }

			[BsonIgnoreIfNull]
			public string? ArgumentIfDisabled { get; set; }

			[BsonIgnoreIfNull]
			public List<string>? ArgumentsIfDisabled { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool Default { get; set; }

			[BsonIgnoreIfNull]
			public bool? ScheduleOverride { get; set; }

			public string? ToolTip { get; set; }

			IReadOnlyList<string>? ITemplateBoolParameter.ArgumentsIfEnabled => ArgumentsIfEnabled;
			IReadOnlyList<string>? ITemplateBoolParameter.ArgumentsIfDisabled => ArgumentsIfDisabled;

			public BoolParameterDocument()
			{
				Label = String.Empty;
			}

			public BoolParameterDocument(TemplateBoolParameterConfig config)
			{
				Id = config.Id;
				Label = config.Label;
				ArgumentIfEnabled = config.ArgumentIfEnabled;
				ArgumentsIfEnabled = config.ArgumentsIfEnabled;
				ArgumentIfDisabled = config.ArgumentIfDisabled;
				ArgumentsIfDisabled = config.ArgumentsIfDisabled;
				Default = config.Default;
				ScheduleOverride = config.ScheduleOverride;
				ToolTip = config.ToolTip;
			}

			public static bool? GetValue(ParameterId parameterId, IReadOnlyDictionary<ParameterId, string> parameters)
			{
				if (parameters.TryGetValue(parameterId, out string? stringValue))
				{
					if (stringValue.Equals("0", StringComparison.Ordinal) || stringValue.Equals("false", StringComparison.OrdinalIgnoreCase))
					{
						return false;
					}
					if (stringValue.Equals("1", StringComparison.Ordinal) || stringValue.Equals("true", StringComparison.OrdinalIgnoreCase))
					{
						return true;
					}
				}
				return null;
			}

			/// <inheritdoc/>
			public override void GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments)
			{
				bool value = GetValue(Id, parameters) ?? (scheduledBuild ? (ScheduleOverride ?? Default) : Default);
				GetCommandLineArgumentsForItem(value, arguments);
			}

			/// <inheritdoc/>
			public override void GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild)
			{
				bool value = scheduledBuild ? (ScheduleOverride ?? Default) : Default;
				parameters[Id] = value.ToString();
			}

			void GetCommandLineArgumentsForItem(bool value, List<string> arguments)
			{
				if (value)
				{
					if (!String.IsNullOrEmpty(ArgumentIfEnabled))
					{
						arguments.Add(ArgumentIfEnabled);
					}
					if (ArgumentsIfEnabled != null)
					{
						arguments.AddRange(ArgumentsIfEnabled);
					}
				}
				else
				{
					if (!String.IsNullOrEmpty(ArgumentIfDisabled))
					{
						arguments.Add(ArgumentIfDisabled);
					}
					if (ArgumentsIfDisabled != null)
					{
						arguments.AddRange(ArgumentsIfDisabled);
					}
				}
			}
		}

		[BsonDiscriminator("TextParameter")]
		sealed class TextParameterDocument : ParameterDocument, ITemplateTextParameter
		{
			public ParameterId Id { get; set; }
			public string Label { get; set; }
			public string Argument { get; set; }
			public string Default { get; set; }
			public string? ScheduleOverride { get; set; }
			public string? Hint { get; set; }

			[BsonIgnoreIfNull]
			public string? Validation { get; set; }

			[BsonIgnoreIfNull]
			public string? ValidationError { get; set; }

			public string? ToolTip { get; set; }

			public TextParameterDocument()
			{
				Label = null!;
				Argument = null!;
				Default = String.Empty;
			}

			public TextParameterDocument(TemplateTextParameterConfig config)
			{
				Id = config.Id;
				Label = config.Label;
				Argument = config.Argument;
				Default = config.Default;
				ScheduleOverride = config.ScheduleOverride;
				Hint = config.Hint;
				Validation = config.Validation;
				ValidationError = config.ValidationError;
				ToolTip = config.ToolTip;
			}

			/// <inheritdoc/>
			public override void GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments)
			{
				if (parameters.TryGetValue(Id, out string? value))
				{
					arguments.Add(Argument + value);
				}
				else
				{
					arguments.Add(Argument + (scheduledBuild ? (ScheduleOverride ?? Default) : Default));
				}
			}

			/// <inheritdoc/>
			public override void GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild)
			{
				parameters[Id] = scheduledBuild ? (ScheduleOverride ?? Default) : Default;
			}
		}

		[BsonDiscriminator("ListParameter")]
		sealed class ListParameterDocument : ParameterDocument, ITemplateListParameter
		{
			public string Label { get; set; }
			public TemplateListParameterStyle Style { get; set; }
			public List<ListParameterItemDocument> Items { get; set; }
			public string? ToolTip { get; set; }

			IReadOnlyList<ITemplateListParameterItem> ITemplateListParameter.Items => Items;

			/// <summary>
			/// Private constructor for serialization
			/// </summary>
			public ListParameterDocument()
			{
				Label = String.Empty;
				Items = new List<ListParameterItemDocument>();
			}

			public ListParameterDocument(TemplateListParameterConfig config)
			{
				Label = config.Label;
				Style = config.Style;
				Items = config.Items.ConvertAll(x => new ListParameterItemDocument(x));
				ToolTip = config.ToolTip;
			}

			/// <inheritdoc/>
			public override void GetArguments(IReadOnlyDictionary<ParameterId, string> parameters, bool scheduledBuild, List<string> arguments)
			{
				foreach (ListParameterItemDocument item in Items)
				{
					bool value = BoolParameterDocument.GetValue(item.Id, parameters) ?? (scheduledBuild ? (item.ScheduleOverride ?? item.Default) : item.Default);
					GetCommandLineArgumentsForItem(item, value, arguments);
				}
			}

			/// <inheritdoc/>
			public override void GetDefaultParameters(Dictionary<ParameterId, string> parameters, bool scheduledBuild)
			{
				foreach (ListParameterItemDocument item in Items)
				{
					bool value = scheduledBuild ? (item.ScheduleOverride ?? item.Default) : item.Default;
					parameters[item.Id] = value.ToString();
				}
			}

			static void GetCommandLineArgumentsForItem(ListParameterItemDocument item, bool value, List<string> arguments)
			{
				if (value)
				{
					if (item.ArgumentIfEnabled != null)
					{
						arguments.Add(item.ArgumentIfEnabled);
					}
					if (item.ArgumentsIfEnabled != null)
					{
						arguments.AddRange(item.ArgumentsIfEnabled);
					}
				}
				else
				{
					if (item.ArgumentIfDisabled != null)
					{
						arguments.Add(item.ArgumentIfDisabled);
					}
					if (item.ArgumentsIfDisabled != null)
					{
						arguments.AddRange(item.ArgumentsIfDisabled);
					}
				}
			}
		}

		class ListParameterItemDocument : ITemplateListParameterItem
		{
			public ParameterId Id { get; set; }

			[BsonIgnoreIfNull]
			public string? Group { get; set; }

			public string Text { get; set; }

			[BsonIgnoreIfNull]
			public string? ArgumentIfEnabled { get; set; }

			[BsonIgnoreIfNull]
			public List<string>? ArgumentsIfEnabled { get; set; }

			[BsonIgnoreIfNull]
			public string? ArgumentIfDisabled { get; set; }

			[BsonIgnoreIfNull]
			public List<string>? ArgumentsIfDisabled { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool Default { get; set; }

			public bool? ScheduleOverride { get; set; }

			IReadOnlyList<string>? ITemplateListParameterItem.ArgumentsIfEnabled => ArgumentsIfEnabled;
			IReadOnlyList<string>? ITemplateListParameterItem.ArgumentsIfDisabled => ArgumentsIfDisabled;

			public ListParameterItemDocument()
			{
				Text = String.Empty;
			}

			public ListParameterItemDocument(TemplateListParameterItemConfig config)
			{
				Id = config.Id;
				Group = config.Group;
				Text = config.Text;
				ArgumentIfEnabled = config.ArgumentIfEnabled;
				ArgumentsIfEnabled = config.ArgumentsIfEnabled;
				ArgumentIfDisabled = config.ArgumentIfDisabled;
				ArgumentsIfDisabled = config.ArgumentsIfDisabled;
				Default = config.Default;
				ScheduleOverride = config.ScheduleOverride;
			}
		}

		/// <summary>
		/// Template documents
		/// </summary>
		readonly IMongoCollection<TemplateDocument> _templates;

		/// <summary>
		/// Cache of template documents
		/// </summary>
		readonly MemoryCache _templateCache;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service singleton</param>
		public TemplateCollection(IMongoService mongoService)
		{
			_templates = mongoService.GetCollection<TemplateDocument>("Templates");

			MemoryCacheOptions options = new MemoryCacheOptions();
			_templateCache = new MemoryCache(options);
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			_templateCache.Dispose();
		}

		/// <inheritdoc/>
		public async Task<ITemplate?> GetAsync(ContentHash templateId)
		{
			object? result;
			if (_templateCache.TryGetValue(templateId, out result))
			{
				return (ITemplate?)result;
			}

			ITemplate? template = await _templates.Find<TemplateDocument>(x => x.Id == templateId).FirstOrDefaultAsync();
			if (template != null)
			{
				using (ICacheEntry entry = _templateCache.CreateEntry(templateId))
				{
					entry.SetSlidingExpiration(TimeSpan.FromMinutes(30.0));
					entry.SetValue(template);
				}
			}

			return template;
		}

		/// <inheritdoc/>
		public async Task<ITemplate> GetOrAddAsync(TemplateConfig config)
		{
			if (config.CachedHash != null)
			{
				ITemplate? existingTemplate = await GetAsync(config.CachedHash);
				if (existingTemplate != null)
				{
					return existingTemplate;
				}
			}

			TemplateDocument template = new TemplateDocument(config);
			if (await GetAsync(template.Id) == null)
			{
				await _templates.ReplaceOneAsync(x => x.Id == template.Id, template, new ReplaceOptions { IsUpsert = true });
			}
			return template;
		}
	}
}
