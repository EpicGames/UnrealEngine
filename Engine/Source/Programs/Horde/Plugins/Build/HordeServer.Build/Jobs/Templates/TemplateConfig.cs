// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Text.Json.Serialization;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using HordeServer.Configuration;

#pragma warning disable CA2227 //  Change x to be read-only by removing the property setter

namespace HordeServer.Jobs.Templates
{
	/// <summary>
	/// Parameters to create a new template
	/// </summary>
	public class TemplateConfig
	{
		/// <summary>
		/// Name for the new template
		/// </summary>
		[Required]
		public string Name { get; set; } = null!;

		/// <summary>
		/// Description for the template
		/// </summary>
		public string? Description { get; set; }

		/// <summary>
		/// Default priority for this job
		/// </summary>
		public Priority? Priority { get; set; }

		/// <summary>
		/// Whether to allow preflights of this template
		/// </summary>
		public bool AllowPreflights { get; set; } = true;

		/// <summary>
		/// Whether issues should be updated for all jobs using this template
		/// </summary>
		public bool UpdateIssues { get; set; } = false;

		/// <summary>
		/// Whether issues should be promoted by default for this template, promoted issues will generate user notifications 
		/// </summary>
		public bool PromoteIssuesByDefault { get; set; } = false;

		/// <summary>
		/// Initial agent type to parse the buildgraph script on
		/// </summary>
		public string? InitialAgentType { get; set; }

		/// <summary>
		/// Path to a file within the stream to submit to generate a new changelist for jobs
		/// </summary>
		public string? SubmitNewChange { get; set; }

		/// <summary>
		/// Description for new changelists
		/// </summary>
		public string? SubmitDescription { get; set; }

		/// <summary>
		/// Default change to build at. Each object has a condition parameter which can evaluated by the server to determine which change to use.
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<ChangeQueryConfig>? DefaultChange { get; set; }

		/// <summary>
		/// Fixed arguments for the new job
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<string> Arguments { get; set; } = new List<string>();

		/// <summary>
		/// Parameters for this template
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<TemplateParameterConfig> Parameters { get; set; } = new List<TemplateParameterConfig>();

		/// <summary>
		/// Default settings for jobs
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Recursive)]
		public JobOptions JobOptions { get; set; } = new JobOptions();

		/// <summary>
		/// The cached hash of this template.
		/// </summary>
		[JsonIgnore]
		internal ContentHash? CachedHash { get; set; }
	}

	/// <summary>
	/// Query selecting the base changelist to use
	/// </summary>
	public class ChangeQueryConfig : IChangeQuery
	{
		/// <inheritdoc/>
		public string? Name { get; set; }

		/// <inheritdoc/>
		public Condition? Condition { get; set; }

		/// <inheritdoc/>
		public TemplateId? TemplateId { get; set; }

		/// <inheritdoc/>
		public string? Target { get; set; }

		/// <inheritdoc/>
		public List<JobStepOutcome>? Outcomes { get; set; }
		IReadOnlyList<JobStepOutcome>? IChangeQuery.Outcomes => Outcomes;

		/// <inheritdoc/>
		public CommitTag? CommitTag { get; set; }
	}

	/// <summary>
	/// Base class for template parameters
	/// </summary>
	[JsonKnownTypes(typeof(TemplateTextParameterConfig), typeof(TemplateListParameterConfig), typeof(TemplateBoolParameterConfig))]
	public abstract class TemplateParameterConfig
	{
		/// <summary>
		/// Callback after a parameter has been read.
		/// </summary>
		public abstract void PostLoad(HashSet<ParameterId> parameterIds);

		/// <summary>
		/// Adds the given parameter id to the set of known ids, or create a new one
		/// </summary>
		/// <param name="explicitParameterId"></param>
		/// <param name="name"></param>
		/// <param name="parameterIds"></param>
		/// <returns></returns>
		protected static ParameterId AddParameterId(ParameterId explicitParameterId, string name, HashSet<ParameterId> parameterIds)
		{
			if (explicitParameterId.Id.IsEmpty)
			{
				return CreateUniqueParameterId(name, parameterIds);
			}
			else if (parameterIds.Add(explicitParameterId))
			{
				return explicitParameterId;
			}
			else
			{
				throw new InvalidOperationException($"Parameter '{explicitParameterId}' has been defined twice");
			}
		}

		/// <summary>
		/// Creates a unique parameter id based on the given name
		/// </summary>
		/// <param name="name">Default name for the parameter</param>
		/// <param name="parameterIds">The existing parameter ids</param>
		protected static ParameterId CreateUniqueParameterId(string name, HashSet<ParameterId> parameterIds)
		{
			string baseName = name;
			if (String.IsNullOrEmpty(baseName))
			{
				baseName = "unnamed";
			}

			const int MaxBaseLength = StringId.MaxLength - 10;
			if (baseName.Length > MaxBaseLength)
			{
				baseName = baseName.Substring(0, MaxBaseLength);
			}

			ParameterId parameterId = ParameterId.Sanitize(baseName);
			for (int idx = 2; !parameterIds.Add(parameterId); idx++)
			{
				parameterId = ParameterId.Sanitize($"{baseName}-{idx}");
			}

			return parameterId;
		}
	}

	/// <summary>
	/// Allows the user to toggle an option on or off
	/// </summary>
	[JsonDiscriminator("Bool")]
	public class TemplateBoolParameterConfig : TemplateParameterConfig
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		public ParameterId Id { get; set; }

		/// <summary>
		/// Name of the parameter associated with this parameter.
		/// </summary>
		public string Label { get; set; }

		/// <summary>
		/// Argument to add if this parameter is enabled
		/// </summary>
		public string? ArgumentIfEnabled { get; set; }

		/// <summary>
		/// Argument to add if this parameter is enabled
		/// </summary>
		public List<string>? ArgumentsIfEnabled { get; set; }

		/// <summary>
		/// Argument to add if this parameter is enabled
		/// </summary>
		public string? ArgumentIfDisabled { get; set; }

		/// <summary>
		/// Arguments to add if this parameter is disabled
		/// </summary>
		public List<string>? ArgumentsIfDisabled { get; set; }

		/// <summary>
		/// Whether this argument is enabled by default
		/// </summary>
		public bool Default { get; set; }

		/// <summary>
		/// Override for this parameter in scheduled builds
		/// </summary>
		public bool? ScheduleOverride { get; set; }

		/// <summary>
		/// Tool tip text to display
		/// </summary>
		public string? ToolTip { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		public TemplateBoolParameterConfig()
		{
			Label = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for this parameter</param>
		/// <param name="label">Label to show next to this parameter</param>
		/// <param name="argumentIfEnabled">Argument to add if this parameter is enabled</param>
		/// <param name="argumentsIfEnabled">Arguments to add if this parameter is enabled</param>
		/// <param name="argumentIfDisabled">Argument to add if this parameter is disabled</param>
		/// <param name="argumentsIfDisabled">Arguments to add if this parameter is disabled</param>
		/// <param name="defaultValue">Whether this option is enabled by default</param>
		/// <param name="scheduleOverride">Override for scheduled builds</param>
		/// <param name="toolTip">The tool tip text to display</param>
		public TemplateBoolParameterConfig(ParameterId id, string label, string? argumentIfEnabled, List<string>? argumentsIfEnabled, string? argumentIfDisabled, List<string>? argumentsIfDisabled, bool defaultValue, bool? scheduleOverride, string? toolTip)
		{
			Id = id;
			Label = label;
			ArgumentIfEnabled = argumentIfEnabled;
			ArgumentsIfEnabled = argumentsIfEnabled;
			ArgumentIfDisabled = argumentIfDisabled;
			ArgumentsIfDisabled = argumentsIfDisabled;
			Default = defaultValue;
			ScheduleOverride = scheduleOverride;
			ToolTip = toolTip;
		}

		/// <inheritdoc/>
		public override void PostLoad(HashSet<ParameterId> parameterIds)
		{
			Id = AddParameterId(Id, Label, parameterIds);

			if (!String.IsNullOrEmpty(ArgumentIfEnabled) && ArgumentsIfEnabled != null && ArgumentsIfEnabled.Count > 0)
			{
				throw new InvalidDataException("Cannot specify both 'ArgumentIfEnabled' and 'ArgumentsIfEnabled'");
			}
			if (!String.IsNullOrEmpty(ArgumentIfDisabled) && ArgumentsIfDisabled != null && ArgumentsIfDisabled.Count > 0)
			{
				throw new InvalidDataException("Cannot specify both 'ArgumentIfDisabled' and 'ArgumentsIfDisabled'");
			}
		}
	}

	/// <summary>
	/// Free-form text entry parameter
	/// </summary>
	[JsonDiscriminator("Text")]
	public class TemplateTextParameterConfig : TemplateParameterConfig
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		public ParameterId Id { get; set; }

		/// <summary>
		/// Name of the parameter associated with this parameter.
		/// </summary>
		public string Label { get; set; }

		/// <summary>
		/// Argument to pass to the executor
		/// </summary>
		public string Argument { get; set; }

		/// <summary>
		/// Default value for this argument
		/// </summary>
		public string Default { get; set; }

		/// <summary>
		/// Override for the default value for this parameter when running a scheduled build
		/// </summary>
		public string? ScheduleOverride { get; set; }

		/// <summary>
		/// Hint text for this parameter
		/// </summary>
		public string? Hint { get; set; }

		/// <summary>
		/// Regex used to validate this parameter
		/// </summary>
		public string? Validation { get; set; }

		/// <summary>
		/// Message displayed if validation fails, informing user of valid values.
		/// </summary>
		public string? ValidationError { get; set; }

		/// <summary>
		/// Tool-tip text to display
		/// </summary>
		public string? ToolTip { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		public TemplateTextParameterConfig()
		{
			Label = null!;
			Argument = null!;
			Default = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for this parameter</param>
		/// <param name="label">Label to show next to the parameter</param>
		/// <param name="argument">Argument to pass this value with</param>
		/// <param name="defaultValue">Default value for this parameter</param>
		/// <param name="scheduleOverride">Default value for scheduled builds</param>
		/// <param name="hint">Hint text to display for this parameter</param>
		/// <param name="validation">Regex used to validate entries</param>
		/// <param name="validationError">Message displayed to explain validation issues</param>
		/// <param name="toolTip">Tool tip text to display</param>
		public TemplateTextParameterConfig(ParameterId id, string label, string argument, string defaultValue, string? scheduleOverride, string? hint, string? validation, string? validationError, string? toolTip)
		{
			Id = id;
			Label = label;
			Argument = argument;
			Default = defaultValue;
			ScheduleOverride = scheduleOverride;
			Hint = hint;
			Validation = validation;
			ValidationError = validationError;
			ToolTip = toolTip;
		}

		/// <inheritdoc/>
		public override void PostLoad(HashSet<ParameterId> parameterIds)
		{
			Id = AddParameterId(Id, Label, parameterIds);
		}
	}

	/// <summary>
	/// Allows the user to select a value from a constrained list of choices
	/// </summary>
	[JsonDiscriminator("List")]
	public class TemplateListParameterConfig : TemplateParameterConfig
	{
		/// <summary>
		/// Label to display next to this parameter. Defaults to the parameter name.
		/// </summary>
		public string Label { get; set; }

		/// <summary>
		/// The type of list parameter
		/// </summary>
		public TemplateListParameterStyle Style { get; set; }

		/// <summary>
		/// List of values to display in the list
		/// </summary>
		public List<TemplateListParameterItemConfig> Items { get; set; }

		/// <summary>
		/// Tool tip text to display
		/// </summary>
		public string? ToolTip { get; set; }

		/// <summary>
		/// Private constructor
		/// </summary>
		public TemplateListParameterConfig()
		{
			Label = null!;
			Items = null!;
		}

		/// <summary>
		/// List of possible values
		/// </summary>
		/// <param name="label">Label to show next to this parameter</param>
		/// <param name="style">Type of picker to show</param>
		/// <param name="items">Entries for this list</param>
		/// <param name="toolTip">Tool tip text to display</param>
		public TemplateListParameterConfig(string label, TemplateListParameterStyle style, List<TemplateListParameterItemConfig> items, string? toolTip)
		{
			Label = label;
			Style = style;
			Items = items;
			ToolTip = toolTip;
		}

		/// <inheritdoc/>
		public override void PostLoad(HashSet<ParameterId> parameterIds)
		{
			foreach (TemplateListParameterItemConfig item in Items)
			{
				item.Id = AddParameterId(item.Id, $"{Label}.{item.Text}", parameterIds);

				if (!String.IsNullOrEmpty(item.ArgumentIfEnabled) && item.ArgumentsIfEnabled != null && item.ArgumentsIfEnabled.Count > 0)
				{
					throw new InvalidDataException("Cannot specify both 'ArgumentIfEnabled' and 'ArgumentsIfEnabled'");
				}
				if (!String.IsNullOrEmpty(item.ArgumentIfDisabled) && item.ArgumentsIfDisabled != null && item.ArgumentsIfDisabled.Count > 0)
				{
					throw new InvalidDataException("Cannot specify both 'ArgumentIfDisabled' and 'ArgumentsIfDisabled'");
				}
			}
		}
	}

	/// <summary>
	/// Possible option for a list parameter
	/// </summary>
	public class TemplateListParameterItemConfig
	{
		/// <summary>
		/// Identifier for this parameter
		/// </summary>
		public ParameterId Id { get; set; }

		/// <summary>
		/// Optional group heading to display this entry under, if the picker style supports it.
		/// </summary>
		public string? Group { get; set; }

		/// <summary>
		/// Name of the parameter associated with this list.
		/// </summary>
		public string Text { get; set; }

		/// <summary>
		/// Argument to pass with this parameter.
		/// </summary>
		public string? ArgumentIfEnabled { get; set; }

		/// <summary>
		/// Arguments to pass with this parameter.
		/// </summary>
		public List<string>? ArgumentsIfEnabled { get; set; }

		/// <summary>
		/// Argument to pass with this parameter.
		/// </summary>
		public string? ArgumentIfDisabled { get; set; }

		/// <summary>
		/// Arguments to pass if this parameter is disabled.
		/// </summary>
		public List<string>? ArgumentsIfDisabled { get; set; }

		/// <summary>
		/// Whether this item is selected by default
		/// </summary>
		public bool Default { get; set; }

		/// <summary>
		/// Overridden value for this property in schedule builds
		/// </summary>
		public bool? ScheduleOverride { get; set; }

		/// <summary>
		/// Private constructor for serialization
		/// </summary>
		public TemplateListParameterItemConfig()
		{
			Text = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for this parameter</param>
		/// <param name="group">The group to put this parameter in</param>
		/// <param name="text">Text to display for this option</param>
		/// <param name="argumentIfEnabled">Argument to pass for this item if it's enabled</param>
		/// <param name="argumentsIfEnabled">Argument to pass for this item if it's enabled</param>
		/// <param name="argumentIfDisabled">Argument to pass for this item if it's enabled</param>
		/// <param name="argumentsIfDisabled">Argument to pass for this item if it's enabled</param>
		/// <param name="defaultValue">Whether this item is selected by default</param>
		/// <param name="scheduleOverride">Overridden value for this item for scheduled builds</param>
		public TemplateListParameterItemConfig(ParameterId id, string? group, string text, string? argumentIfEnabled, List<string>? argumentsIfEnabled, string? argumentIfDisabled, List<string>? argumentsIfDisabled, bool defaultValue, bool? scheduleOverride)
		{
			Id = id;
			Group = group;
			Text = text;
			ArgumentIfEnabled = argumentIfEnabled;
			ArgumentsIfEnabled = argumentsIfEnabled;
			ArgumentIfDisabled = argumentIfDisabled;
			ArgumentsIfDisabled = argumentsIfDisabled;
			Default = defaultValue;
			ScheduleOverride = scheduleOverride;
		}
	}
}

