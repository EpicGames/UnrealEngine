// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.TestData;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Defines a testing environment based on platforms, configurations, targets, etc.
	/// </summary>
	public interface ITestMeta
	{
		/// <summary>
		/// The test meta id
		/// </summary>
		TestMetaId Id { get; }

		/// <summary>
		/// The name of the test platform
		/// </summary>
		IReadOnlyList<string> Platforms { get; }

		/// <summary>
		/// The configuration the test was run on
		/// </summary>
		IReadOnlyList<string> Configurations { get; }

		/// <summary>
		/// The build target, editor, server, client, etc
		/// </summary>
		IReadOnlyList<string> BuildTargets { get; }

		/// <summary>
		/// The uproject name associated with this test, note: may not be directly related to Horde project
		/// </summary>
		string ProjectName { get; }

		/// <summary>
		/// The rendering hardware interface used for the test
		/// </summary>
		string RHI { get; }

		/// <summary>
		/// The variation of the meta data, for example address sanitizing
		/// </summary>
		string Variation { get; }
	}

	/// <summary>
	/// A test that runs in a stream
	/// </summary>
	public interface ITest
	{
		/// <summary>
		/// The test id
		/// </summary>
		TestId Id { get; }

		/// <summary>
		/// The fully qualified name of the test 
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The display name of the test 
		/// </summary>
		string? DisplayName { get; }

		/// <summary>
		/// The name of the associated suite if any
		/// </summary>
		string? SuiteName { get; }

		/// <summary>
		/// The meta data for the test 
		/// </summary>
		IReadOnlyList<TestMetaId> Metadata { get; }
	}

	/// <summary>
	/// A test suite that runs in a stream
	/// </summary>
	public interface ITestSuite
	{
		/// <summary>
		/// The test suite id
		/// </summary>
		TestSuiteId Id { get; }

		/// <summary>
		/// The name of the test suite
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The tests that compose the suite
		/// </summary>
		IReadOnlyList<TestId> Tests { get; }

		/// <summary>
		/// The meta data for the test suite
		/// </summary>
		IReadOnlyList<TestMetaId> Metadata { get; }
	}

	/// <summary>
	/// Suite test data
	/// </summary>
	public interface ISuiteTestData
	{
		/// <summary>
		/// The test id
		/// </summary>
		TestId TestId { get; }

		/// <summary>
		/// The outcome of the suite test
		/// </summary>
		TestOutcome Outcome { get; }

		/// <summary>
		/// How long the suite test ran
		/// </summary>
		TimeSpan Duration { get; }

		/// <summary>
		/// Test UID for looking up in test details
		/// </summary>
		string UID { get; }

		/// <summary>
		/// The number of warnings
		/// </summary>
		int? WarningCount { get; }

		/// <summary>
		/// The number of errors
		/// </summary>
		int? ErrorCount { get; }
	}

	/// <summary>
	/// Data ref with minimal data required for aggregate views
	/// </summary>
	public interface ITestDataRef
	{
		/// <summary>
		/// The test ref id
		/// </summary>
		TestRefId Id { get; }

		/// <summary>
		/// The associated stream
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The associated job
		/// </summary>
		JobId? JobId { get; }

		/// <summary>
		/// The associated job step
		/// </summary>
		JobStepId? StepId { get; }

		/// <summary>
		/// How long the test ran
		/// </summary>
		TimeSpan Duration { get; }

		/// <summary>
		/// The build changelist upon which the test ran, may not correspond to the job changelist
		/// </summary>
		CommitIdWithOrder BuildCommitId { get; }

		/// <summary>
		/// The environment the test ran in
		/// </summary>
		TestMetaId Metadata { get; }

		/// <summary>
		/// The ITest in stream
		/// </summary>
		TestId? TestId { get; }

		/// <summary>
		/// The outcome of the test
		/// </summary>
		TestOutcome? Outcome { get; }

		/// <summary>
		/// The ITestSuite in stream
		/// </summary>
		TestSuiteId? SuiteId { get; }

		/// <summary>
		/// Suite tests skipped
		/// </summary>
		int? SuiteSkipCount { get; }

		/// <summary>
		/// Suite test warnings
		/// </summary>
		int? SuiteWarningCount { get; }

		/// <summary>
		/// Suite test errors
		/// </summary>
		int? SuiteErrorCount { get; }

		/// <summary>
		/// Suite test successes
		/// </summary>
		int? SuiteSuccessCount { get; }
	}

	/// <summary>
	/// Test data details
	/// </summary>
	public interface ITestDataDetails
	{
		/// <summary>
		/// The corresponding test ref		
		/// </summary>
		TestRefId Id { get; }

		/// <summary>
		/// The full details test data for this ref
		/// </summary>
		IReadOnlyList<ObjectId> TestDataIds { get; }

		/// <summary>
		/// Suite test data
		/// </summary>		
		IReadOnlyList<ISuiteTestData>? SuiteTests { get; }
	}

	/// <summary>
	/// The tests and suites running in a given stream
	/// </summary>
	public interface ITestStream
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// Test ids of tests running in the stream
		/// </summary>
		IReadOnlyList<TestId> Tests { get; }

		/// <summary>
		/// Test suite ids
		/// </summary>
		IReadOnlyList<TestSuiteId> TestSuites { get; }
	}

	/// v2

	/// <summary>
	/// A test name reference
	/// </summary>
	public interface ITestNameRef
	{
		/// <summary>
		/// The test id
		/// </summary>
		TestId Id { get; }

		/// <summary>
		/// The test data key that identify the test uniquely
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The display name of the test
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The team associated with the test
		/// </summary>
		string? Team { get; }

		/// <summary>
		/// The test intent
		/// </summary>
		string? Intent { get; }

		/// <summary>
		/// The harness used by the test
		/// </summary>
		string? Harness { get; }

		/// <summary>
		/// The test profile (unit test, functional, performance, etc)
		/// </summary>
		string? Profile { get; }
	}

	/// <summary>
	/// A test phase reference
	/// </summary>
	public interface ITestPhaseRef
	{
		/// <summary>
		/// The test phase id
		/// </summary>
		TestPhaseId Id { get; }

		/// <summary>
		/// The test reference associated with the phase
		/// </summary>
		TestId TestNameRef { get; }

		/// <summary>
		/// The test phase key that identify the phase uniquely
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The test phase (display) name
		/// </summary>
		string Name { get; }
	}

	/// <summary>
	/// A metadata entry key value pair
	/// </summary>
	public interface ITestMetaEntry
	{
		/// <summary>
		/// The meta key
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The meta value
		/// </summary>
		string Value { get; }
	}

	/// <summary>
	/// A metadata key/value pairs reference
	/// </summary>
	public interface ITestMetaRef
	{
		/// <summary>
		/// The metadata id
		/// </summary>
		TestMetaId Id { get; }

		/// <summary>
		/// The key/value pairs that this metadata references
		/// </summary>
		IReadOnlyList<ITestMetaEntry> Entries { get; }
	}

	/// <summary>
	/// A param entry key value pair
	/// </summary>
	public interface ITestParamEntry
	{
		/// <summary>
		/// The param key
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The param value
		/// </summary>
		object Value { get; }
	}

	/// <summary>
	/// A test recipe reference
	/// </summary>
	public interface ITestRecipeRef
	{
		/// <summary>
		/// The test recipe id
		/// </summary>
		TestRecipeId Id { get; }

		/// <summary>
		/// The test name id associated with this recipe
		/// </summary>
		TestId NameRef { get; }

		/// <summary>
		/// The stream id associated with this recipe
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The job template associated with this recipe
		/// </summary>
		TemplateId JobTemplateId { get; }

		/// <summary>
		/// The job step node name associated with this recipe
		/// </summary>
		string JobStep { get; }

		/// <summary>
		/// The workflow associated with this recipe
		/// </summary>
		WorkflowId? WorkflowId { get; }

		/// <summary>
		/// The test node class
		/// </summary>
		string NodeClass { get; }

		/// <summary>
		/// The test version
		/// </summary>
		string Version { get; }

		/// <summary>
		/// The test timeout in minute
		/// </summary>
		float TimeoutMinutes { get; }

		/// <summary>
		/// The test params
		/// </summary>
		IReadOnlyList<ITestParamEntry>? Params { get; }
	}

	/// <summary>
	/// A test health report
	/// </summary>
	public interface ITestHealthReport
	{
		/// <summary>
		/// The report id
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// The test id associated with this report
		/// </summary>
		TestId TestId { get; }

		/// <summary>
		/// The test name associated with this report
		/// </summary>
		string TestName { get; }

		/// <summary>
		/// The stream id associated with this report
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The last update date associated with this report
		/// </summary>
		DateTime LastUpdateDateUtc { get; }

		/// <summary>
		/// Whether or not the test is healthy
		/// </summary>
		bool IsHealthy { get; }

		/// <summary>
		/// The state associated with this test health
		/// </summary>
		string State { get; }

		/// <summary>
		/// The previous state associated with this test health
		/// </summary>
		string? PreviousState { get; }

		/// <summary>
		/// The success rate of the test health
		/// </summary>
		int SuccessRate { get; }

		/// <summary>
		/// The failure rate of the test health
		/// </summary>
		int FailureRate { get; }

		/// <summary>
		/// The catastrophic rate of the failure
		/// </summary>
		int CatastrophicFailureRate { get; }

		/// <summary>
		/// The rate of the redundant errors
		/// </summary>
		int RedundantErrorRate { get; }

		/// <summary>
		/// The date of the last notification sent for this report
		/// </summary>
		DateTime? NotificationLastDateUtc { get; }
	}

	/// <summary>
	/// A test notification settings
	/// </summary>
	public interface ITestNotificationSettings
	{
		/// <summary>
		/// The Stream associated with this entry
		/// </summary>
		StreamId Stream { get; }

		/// <summary>
		/// The Workflow associated with this entry
		/// </summary>
		WorkflowId Workflow { get; }
	}

	/// <summary>
	/// A test audit entry
	/// </summary>
	public interface ITestAudit
	{
		/// <summary>
		/// The test audit id
		/// </summary>
		TestAuditId Id { get; }

		/// <summary>
		/// The test name id associated with this audit
		/// </summary>
		TestId TestId { get; }

		/// <summary>
		/// The date at which this audit was filed
		/// </summary>
		DateTime UserInputDate { get; }

		/// <summary>
		/// Whether or not this entry is archived
		/// </summary>
		bool Archived { get; }

		/// <summary>
		/// The user id that updated the audit
		/// </summary>
		UserId UserId { get; }

		/// <summary>
		/// Whether or not the associated test should be run on the schedule
		/// </summary>
		bool EnableSchedule { get; }

		/// <summary>
		/// The date at which the audit was last officially taken place
		/// </summary>
		DateTime? LastAuditDate { get; }

		/// <summary>
		/// Whether or not the associated test is under audit
		/// </summary>
		bool? IsUnderAudit { get; }

		/// <summary>
		/// The owner user id of the associated test
		/// </summary>
		UserId? Owner { get; }

		/// <summary>
		/// The customer user ids of the associated test
		/// </summary>
		IReadOnlyList<UserId>? Customers { get; }

		/// <summary>
		/// The workflow notification settings associated with this test
		/// </summary>
		IReadOnlyList<ITestNotificationSettings>? Notifications { get; }

		/// <summary>
		/// The team associated with this test
		/// </summary>
		string? Team { get; }

		/// <summary>
		/// The harness associated with this test
		/// </summary>
		string? Harness { get; }

		/// <summary>
		/// The intent associated with this test
		/// </summary>
		string? Intent { get; }

		/// <summary>
		/// The profile associated with this test
		/// </summary>
		string? Profile { get; }

		/// <summary>
		/// Notes associated with this test
		/// </summary>
		string? Notes { get; }
	}

	/// <summary>
	/// A test tag reference
	/// </summary>
	public interface ITestTagRef
	{
		/// <summary>
		/// The tag id
		/// </summary>
		TestTagId Id { get; }

		/// <summary>
		/// The tag name
		/// </summary>
		string Name { get; }
	}

	/// <summary>
	/// Test session
	/// </summary>
	public interface ITestSession
	{
		/// <summary>
		/// The test session id
		/// </summary>
		TestSessionId Id { get; }

		/// <summary>
		/// The metadata associated with the test session
		/// </summary>
		TestMetaId Metadata { get; }

		/// <summary>
		/// The recipe associated with this test session
		/// </summary>
		TestRecipeId? Recipe { get; }

		/// <summary>
		/// The tags associated with the test session
		/// </summary>
		IReadOnlyList<TestTagId>? Tags { get; }

		/// <summary>
		/// The test data id for this test session
		/// </summary>
		ObjectId TestDataId { get; }

		/// <summary>
		/// The build changelist upon which the test ran, may not correspond to the job changelist
		/// </summary>
		CommitIdWithOrder BuildCommitId { get; }

		/// <summary>
		/// Reference Id to test name
		/// </summary>
		TestId NameRef { get; }

		/// <summary>
		/// The associated stream
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The associated job
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The associated job step
		/// </summary>
		JobStepId StepId { get; }

		/// <summary>
		/// How long the test session ran
		/// </summary>
		TimeSpan Duration { get; }

		/// <summary>
		/// Date time at which the test session was started
		/// </summary>
		DateTime StartDateTime { get; }

		/// <summary>
		/// The outcome of the test session
		/// if PhasesUndefinedCount > 0 => Undefiened
		/// if PhasesFailedCount > 0 => Failure
		/// if PhasesTotalCount > 0 and PhasesSucceededCount > 0 => Success
		/// else => Skipped
		/// </summary>
		TestOutcome Outcome { get; }

		/// <summary>
		/// The total number the test session contains
		/// </summary>
		int PhasesTotalCount { get; }

		/// <summary>
		/// The number of phases that succeeded during the test session
		/// </summary>
		int PhasesSucceededCount { get; }

		/// <summary>
		/// The number of phases that were left undefined during the test session
		/// </summary>
		int PhasesUndefinedCount { get; }

		/// <summary>
		/// The number of phases that failed during the test session
		/// </summary>
		int PhasesFailedCount { get; }

		/// <summary>
		/// The unique error fingerprints collected from the phase events
		/// </summary>
		IReadOnlyList<IErrorFingerprintEntry>? ErrorFingerprints { get; }
	}

	/// <summary>
	/// A error fingerprint entry
	/// </summary>
	public interface IErrorFingerprintEntry
	{
		/// <summary>
		/// The error fingerprint key
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The phase keys associated with the error fingerprint
		/// </summary>
		IReadOnlyList<string> Phases { get; }
	}

	/// <summary>
	/// The test sessions running in a given stream
	/// </summary>
	public interface ITestSessionStream
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// Test ids of tests running in the stream
		/// </summary>
		IReadOnlyList<TestId> Tests { get; }

		/// <summary>
		/// Test metadata ids from running tests in the stream 
		/// </summary>
		IReadOnlyList<TestMetaId> Metadata { get; }

		/// <summary>
		/// Test tag ids from running tests in the stream
		/// </summary>
		IReadOnlyList<TestTagId> Tags { get; }
	}

	/// <summary>
	/// A Test phase outcome from a test session
	/// </summary>
	public interface ITestPhaseSession
	{
		/// <summary>
		/// The test phase session id
		/// </summary>
		TestPhaseSessionId Id { get; }

		/// <summary>
		/// The test phase reference id
		/// </summary>
		TestPhaseId PhaseRef { get; }

		/// <summary>
		/// The phase outcome
		/// </summary>
		TestPhaseOutcome Outcome { get; }

		/// <summary>
		/// The test session that run that phase
		/// </summary>
		TestSessionId SessionId { get; }

		/// <summary>
		/// The metadata associated with this test session
		/// </summary>
		TestMetaId Metadata { get; }

		/// <summary>
		/// The path to the event stream
		/// </summary>
		string? EventStreamPath { get; }

		/// <summary>
		/// Whether the test phase encountered at least one warning
		/// </summary>
		bool? HasWarning { get;  }

		/// <summary>
		/// The error fingerprint from the event stream
		/// </summary>
		string? ErrorFingerprint { get; }

		/// <summary>
		/// The tags associated with the phase session
		/// </summary>
		IReadOnlyList<TestTagId>? Tags { get; }

		/// <summary>
		/// The associated stream
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The associated job
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The associated job step
		/// </summary>
		JobStepId StepId { get; }

		/// <summary>
		/// The build changelist upon which the test ran, may not correspond to the job changelist
		/// </summary>
		CommitIdWithOrder BuildCommitId { get; }

		/// <summary>
		/// How long the test phase session ran
		/// </summary>
		TimeSpan Duration { get; }

		/// <summary>
		/// Date time at which the test phase session was started
		/// </summary>
		DateTime StartDateTime { get; }
	}

	/// <summary>
	/// Stores information about the results of a test
	/// </summary>
	public interface ITestData
	{
		/// <summary>
		/// Unique id of the test data
		/// </summary>
		ObjectId Id { get; }

		/// <summary>
		/// Stream that generated the test data
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The template reference id
		/// </summary>
		TemplateId TemplateRefId { get; }

		/// <summary>
		/// The job which produced the data
		/// </summary>
		JobId JobId { get; }

		/// <summary>
		/// The step that ran
		/// </summary>
		JobStepId StepId { get; }

		/// <summary>
		/// The commit that contained the data
		/// </summary>
		CommitIdWithOrder CommitId { get; }

		/// <summary>
		/// The preflight commit id if any
		/// </summary>
		CommitId? PreflightCommitId { get; }

		/// <summary>
		/// Key used to identify the particular data
		/// </summary>
		string Key { get; }

		/// <summary>
		/// The data stored for this test
		/// </summary>
		BsonDocument Data { get; }
	}
}
