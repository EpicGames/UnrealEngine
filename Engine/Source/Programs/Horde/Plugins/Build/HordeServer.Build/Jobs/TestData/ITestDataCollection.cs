// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Collection of test data documents
	/// </summary>
	public interface ITestDataCollection
	{
		/// <summary>
		/// Gets the tests running in provided streams
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestStream>> FindTestStreamsAsync(StreamId[] streamIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// 
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="metaIds"></param>
		/// <param name="testIds"></param>
		/// <param name="suiteIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestDataRef>> FindTestRefsAsync(StreamId[] streamIds, TestMetaId[]? metaIds = null, TestId[]? testIds = null, TestSuiteId[]? suiteIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test details
		/// </summary>
		/// <param name="ids"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestDataDetails>> FindTestDetailsAsync(TestRefId[] ids, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITest>> FindTestsAsync(TestId[] testIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test suites
		/// </summary>
		/// <param name="suiteIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestSuite>> FindTestSuitesAsync(TestSuiteId[] suiteIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Creates a new test data document
		/// </summary>
		/// <param name="job">The job containing the step</param>
		/// <param name="step">The step producing the data</param>
		/// <param name="data">The test data generated by the job step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new stream document</returns>
		Task<IReadOnlyList<ITestData>> AddAsync(IJob job, IJobStep step, (string key, BsonDocument value)[] data, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a stream by ID
		/// </summary>
		/// <param name="id">Unique id of the stream</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The stream document</returns>
		Task<ITestData?> GetAsync(ObjectId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for test data that matches a set of criteria
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="minCommitId">The minimum commit to return (inclusive)</param>
		/// <param name="maxCommitId">The maximum commit to return (inclusive)</param>
		/// <param name="jobId">The job id</param>
		/// <param name="stepId">The unique step id</param>
		/// <param name="key">Key identifying the result to return</param>
		/// <param name="index">Offset within the results to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The stream document</returns>
		Task<IReadOnlyList<ITestData>> FindAsync(StreamId? streamId, CommitId? minCommitId, CommitId? maxCommitId, JobId? jobId, JobStepId? stepId, string? key = null, int index = 0, int count = 10, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test meta data
		/// </summary>
		/// <param name="projectNames"></param>
		/// <param name="platforms"></param>
		/// <param name="configurations"></param>
		/// <param name="buildTargets"></param>
		/// <param name="rhi"></param>
		/// <param name="variation"></param>
		/// <param name="metaIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestMeta>> FindTestMetaAsync(string[]? projectNames = null, string[]? platforms = null, string[]? configurations = null, string[]? buildTargets = null, string? rhi = null, string? variation = null, TestMetaId[]? metaIds = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Delete the test data
		/// </summary>
		/// <param name="id">Unique id of the test data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(ObjectId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Prune test data from collection
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task PruneDataAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Run a tick on the collection
		/// </summary>
		/// <param name="retainMonths"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public Task<bool> UpdateAsync(int retainMonths, CancellationToken cancellationToken = default);

		/// <summary>
		/// Upgrades the collection
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public Task UpgradeAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Collection of test data documents v2
	/// </summary>
	public interface ITestDataCollectionV2
	{
		/// <summary>
		/// Gets the test sessions running in provided streams
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestSessionStream>> FindTestSessionStreamsAsync(StreamId[] streamIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test sessions
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="testIds"></param>
		/// <param name="metaIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestSession>> FindTestSessionsAsync(StreamId[] streamIds, TestId[]? testIds = null, TestMetaId[]? metaIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test name references
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="keys"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestNameRef>> FindTestNameRefsAsync(TestId[]? testIds = null, string[]? keys = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Creates a new test data document
		/// </summary>
		/// <param name="job">The job containing the step</param>
		/// <param name="step">The step producing the data</param>
		/// <param name="data">The test data generated by the job step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new testdata document</returns>
		Task<IReadOnlyList<ITestData>> AddAsync(IJob job, IJobStep step, (string key, BsonDocument value)[] data, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets testdata ids run within a job step
		/// </summary>
		/// <param name="jobId">Unique id of the job</param>
		/// <param name="stepId">Unique id of the step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The testdata ids</returns>
		Task<IReadOnlyList<ITestData>> GetJobStepTestDataAsync(JobId jobId, JobStepId? stepId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a testdata by ID
		/// </summary>
		/// <param name="id">Unique id of the testdata</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The testdata document</returns>
		Task<ITestData?> GetAsync(ObjectId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test metadata set
		/// </summary>
		/// <param name="metaIds"></param>
		/// <param name="keyValues"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestMetaRef>> FindTestMetaAsync(TestMetaId[]? metaIds = null, IReadOnlyDictionary<string, string>? keyValues = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test phase references associated with a set of tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="keys"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestPhaseRef>> FindTestPhasesAsync(TestId[] testIds, string[]? keys = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test tag references
		/// </summary>
		/// <param name="tagIds"></param>
		/// <param name="names"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestTagRef>> FindTestTagsAsync(TestTagId[]? tagIds = null, string[]? names = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test recipe references
		/// </summary>
		/// <param name="recipeIds"></param>
		/// <param name="testIds"></param>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestRecipeRef>> FindTestRecipesAsync(TestRecipeId[]? recipeIds = null, TestId[]? testIds = null, StreamId[]? streamIds = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find test phase sessions from a set of test phases
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="phaseIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestPhaseSession>> FindTestPhaseSessionsAsync(StreamId[] streamIds, TestPhaseId[] phaseIds, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Add test audit entry
		/// </summary>
		/// <param name="userId"></param>
		/// <param name="testId"></param>
		/// <param name="options"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<ITestAudit> AddTestAuditAsync(UserId userId, TestId testId, TestAuditOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find audits related to tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestAudit>> FindTestAuditsAsync(TestId[] testIds, DateTime? minCreateTime = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find audits related to tests with notification workflow
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestAudit>> FindTestWithAuditNotificationWorkflowAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Get the logger for the specified test id
		/// </summary>
		/// <param name="testId"></param>
		/// <returns></returns>
		IAuditLogChannel<TestId> GetTestAuditLogger(TestId testId);

		/// <summary>
		/// Add or update test health report
		/// </summary>
		/// <param name="testId"></param>
		/// <param name="streamId"></param>
		/// <param name="testName"></param>
		/// <param name="testHealth"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<ITestHealthReport> AddOrUpdateTestHealthReportAsync(TestId testId, StreamId streamId, string testName, ComputedTestHealth testHealth, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find Test health reports associated with test ids and stream ids
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="testIds"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task<IReadOnlyList<ITestHealthReport>> FindTestHealthReportsAsync(StreamId[] streamIds, TestId[]? testIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update test health report notification date
		/// </summary>
		/// <param name="reportId"></param>
		/// <param name="resetDate"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		Task UpdateTestHealthReportNotificationDateAsync(ObjectId reportId, bool resetDate, CancellationToken cancellationToken = default);

		/// <summary>
		/// Prune test data from collection
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task PruneDataAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Run a tick on the collection
		/// </summary>
		/// <param name="retainMonths"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<bool> UpdateAsync(int retainMonths, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Test notification setting options
	/// </summary>
	public class TestNotificationSettingsOptions
	{
		/// <summary>
		/// The stream id
		/// </summary>
		public StreamId Stream { get; set; }

		/// <summary>
		/// The workflow id associated with the stream
		/// </summary>
		public WorkflowId Workflow { get; set; }
	}

	/// <summary>
	/// The test audit options
	/// </summary>
	public class TestAuditOptions
	{
		/// <summary>
		/// Whether or not the test schedule is enabled
		/// </summary>
		public bool EnableSchedule { get; set; } = true;

		/// <summary>
		/// The last audit Date
		/// </summary>
		public DateTime? LastAuditDate { get; set; }

		/// <summary>
		/// Whether or not the test is under audit
		/// </summary>
		public bool? IsUnderAudit {  get; set; }

		/// <summary>
		/// The owner of the related test
		/// </summary>
		public UserId? Owner { get; set; }

		/// <summary>
		/// The customers of the related test
		/// </summary>
		public List<UserId>? Customers { get; set; }

		/// <summary>
		/// The notifications settings options
		/// </summary>
		public List<TestNotificationSettingsOptions>? Notifications { get; set; }

		/// <summary>
		/// The team associated with the test
		/// </summary>
		public string? Team { get; set; }

		/// <summary>
		/// The harness associated with the test
		/// </summary>
		public string? Harness { get; set; }

		/// <summary>
		/// The intent of the test
		/// </summary>
		public string? Intent { get; set; }

		/// <summary>
		/// The profile associated with the test
		/// </summary>
		public string? Profile { get; set; }

		/// <summary>
		/// The notes associated with the test audit
		/// </summary>
		public string? Notes { get; set; }
	}

	/// <summary>
	/// The computed health for a report
	/// </summary>
	public class ComputedTestHealth
	{
		/// <summary>
		/// Whether the rates distribution is deemed healthy
		/// </summary>
		public bool IsHealthy { get; set; }

		/// <summary>
		/// A label that describes the health state
		/// </summary>
		public string State { get; set; } = String.Empty;

		/// <summary>
		/// The rate of success
		/// </summary>
		public int SuccessRate { get; set; }

		/// <summary>
		/// The rate of failure
		/// </summary>
		public int FailureRate { get; set; }

		/// <summary>
		/// The rate of catastrophic failure
		/// </summary>
		public int CatastrophicFailureRate { get; set; }

		/// <summary>
		/// The rate of redundant error
		/// </summary>
		public int RedundantErrorRate { get; set; }
	}
}
