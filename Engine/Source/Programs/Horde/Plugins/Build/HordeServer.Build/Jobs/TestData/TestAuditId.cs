// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a test audit
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestAuditId, TestAuditIdConverter>))]
	[ObjectIdConverter(typeof(TestAuditIdConverter))]
	public record struct TestAuditId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for test audit id
		/// </summary>
		public static TestAuditId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestAuditId"/>
		/// </summary>
		public static TestAuditId GenerateNewId() => new TestAuditId(ObjectId.GenerateNewId());

		/// <summary>
		/// Creates a new <see cref="TestAuditId"/>
		/// </summary>
		/// <param name="time"></param>
		/// <returns></returns>
		public static TestAuditId GenerateNewId(DateTime time) => new TestAuditId(ObjectId.GenerateNewId(time));

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestAuditId Parse(string text) => new TestAuditId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestAuditId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestAuditId(objectId);
				return true;
			}

			id = default;
			return false;
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestAuditIdConverter : ObjectIdConverter<TestAuditId>
	{
		/// <inheritdoc/>
		public override TestAuditId FromObjectId(ObjectId id) => new TestAuditId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestAuditId value) => value.Id;
	}
}
