// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// ACL actions which apply to test data
	/// </summary>
	public static class TestDataAclAction
	{
		/// <summary>
		/// Ability to read test health
		/// </summary>
		public static AclAction TestHealthRead { get; } = new AclAction("TestHealthRead");

		/// <summary>
		/// Ability to write test health
		/// </summary>
		public static AclAction TestHealthWrite { get; } = new AclAction("TestHealthWrite");
	}
}
