// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.TestData;
using EpicGames.Horde.Streams;
using HordeServer.Commits;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Collection of test data documents
	/// </summary>
	public class TestDataCollection : ITestDataCollection
	{
		internal interface ITestExpire
		{
			DateTime? LastSeenUtc { get; }
		}

		class TestMetaDocument : ITestMeta, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestMetaId Id { get; set; }

			[BsonRequired, BsonElement("p")]
			public List<string> Platforms { get; set; }
			IReadOnlyList<string> ITestMeta.Platforms => Platforms;

			[BsonRequired, BsonElement("c")]
			public List<string> Configurations { get; set; }
			IReadOnlyList<string> ITestMeta.Configurations => Configurations;

			[BsonRequired, BsonElement("bt")]
			public List<string> BuildTargets { get; set; }
			IReadOnlyList<string> ITestMeta.BuildTargets => BuildTargets;

			[BsonRequired, BsonElement("pn")]
			public string ProjectName { get; set; }

			[BsonRequired, BsonElement("r")]
			public string RHI { get; set; }

			[BsonElement("v")]
			public string Variation { get; set; }

			[BsonIgnoreIfNull, BsonElement("s")]
			public DateTime? LastSeenUtc { get; set; }

			[BsonIgnoreIfNull, BsonElement("vn")]
			public int? Version { get; set; }

			public static int CurrentVersion = 1;

			private TestMetaDocument()
			{
				Platforms = new List<string>();
				Configurations = new List<string>();
				BuildTargets = new List<string>();

				RHI = String.Empty;
				Variation = String.Empty;
				ProjectName = String.Empty;
			}

			public TestMetaDocument(List<string> platforms, List<string> configurations, List<string> buildTargets, string projectName, string? rhi, string? variation)
			{
				Id = TestMetaId.GenerateNewId();
				Platforms = platforms;
				Configurations = configurations;
				BuildTargets = buildTargets;
				ProjectName = projectName;
				RHI = rhi ?? "default";
				Variation = variation ?? "default";
				Version = CurrentVersion;
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		class TestDocument : ITest, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestId Id { get; set; }

			[BsonRequired, BsonElement("n")]
			public string Name { get; set; }

			[BsonIgnoreIfNull, BsonElement("sn")]
			public string? SuiteName { get; set; }

			[BsonIgnoreIfNull, BsonElement("dn")]
			public string? DisplayName { get; set; }

			[BsonRequired, BsonElement("m")]
			public List<TestMetaId> Metadata { get; set; }
			IReadOnlyList<TestMetaId> ITest.Metadata => Metadata;

			[BsonIgnoreIfNull, BsonElement("vn")]
			public int? Version { get; set; }

			[BsonIgnoreIfNull, BsonElement("s")]
			public DateTime? LastSeenUtc { get; set; }

			public static int CurrentVersion = 1;

			private TestDocument()
			{
				Name = String.Empty;
				Metadata = new List<TestMetaId>();
			}

			public TestDocument(string name, List<TestMetaId> meta, string? suiteName = null, string? displayName = null)
			{
				Id = TestId.GenerateNewId();
				Name = name;
				Metadata = meta;
				SuiteName = suiteName;
				DisplayName = displayName;
				Version = CurrentVersion;
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		class TestSuiteDocument : ITestSuite, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestSuiteId Id { get; set; }

			[BsonRequired, BsonElement("n")]
			public string Name { get; set; }

			[BsonRequired, BsonElement("m")]
			public List<TestMetaId> Metadata { get; set; }
			IReadOnlyList<TestMetaId> ITestSuite.Metadata => Metadata;

			[BsonRequired, BsonElement("t")]
			public List<TestId> Tests { get; set; }
			IReadOnlyList<TestId> ITestSuite.Tests => Tests;

			[BsonIgnoreIfNull, BsonElement("s")]
			public DateTime? LastSeenUtc { get; set; }

			[BsonIgnoreIfNull, BsonElement("vn")]
			public int? Version { get; set; }

			public static int CurrentVersion = 1;

			private TestSuiteDocument()
			{
				Name = String.Empty;
				Tests = new List<TestId>();
				Metadata = new List<TestMetaId>();
			}

			public TestSuiteDocument(string name, List<TestMetaId> metaIds, List<TestId> tests)
			{
				Id = TestSuiteId.GenerateNewId();
				Name = name;
				Metadata = metaIds;
				Tests = tests;
				Version = CurrentVersion;
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Stores the tests and suites running in a stream
		/// </summary>
		class TestStreamDocument : ITestStream
		{
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonRequired]
			public StreamId StreamId { get; set; }

			[BsonRequired]
			public List<TestId> Tests { get; set; } = new List<TestId>();
			IReadOnlyList<TestId> ITestStream.Tests => Tests;

			[BsonRequired]
			public List<TestSuiteId> TestSuites { get; set; } = new List<TestSuiteId>();
			IReadOnlyList<TestSuiteId> ITestStream.TestSuites => TestSuites;

			private TestStreamDocument()
			{

			}

			public TestStreamDocument(StreamId streamId)
			{
				Id = ObjectId.GenerateNewId();
				StreamId = streamId;
			}
		}

		class SuiteTestData : ISuiteTestData
		{
			[BsonRequired, BsonElement("tid")]
			public TestId TestId { get; set; }

			[BsonRequired, BsonElement("o")]
			public TestOutcome Outcome { get; set; }

			[BsonRequired, BsonElement("d")]
			public TimeSpan Duration { get; set; }

			[BsonRequired, BsonElement("uid")]
			public string UID { get; set; }

			[BsonIgnoreIfNull, BsonElement("wc")]
			public int? WarningCount { get; set; }

			[BsonIgnoreIfNull, BsonElement("ec")]
			public int? ErrorCount { get; set; }

			public SuiteTestData(TestId id, TestOutcome outcome, TimeSpan duration, string uid, int? warningCount, int? errorCount)
			{
				TestId = id;
				Outcome = outcome;
				Duration = duration;
				UID = uid;
				WarningCount = warningCount;
				ErrorCount = errorCount;
			}
		}

		class TestDataRefDocument : ITestDataRef
		{
			[BsonRequired, BsonId]
			public TestRefId Id { get; set; }

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("m")]
			public TestMetaId Metadata { get; set; }

			public CommitIdWithOrder BuildCommitId
			{
				get => (BuildCommitName != null) ? new CommitIdWithOrder(BuildCommitName, BuildCommitOrder) : CommitIdWithOrder.FromPerforceChange(BuildCommitOrder);
				set => (BuildCommitName, BuildCommitOrder) = (value.Name, value.Order);
			}

			[BsonIgnoreIfNull, BsonElement("bcn")]
			public string? BuildCommitName { get; set; }

			[BsonRequired, BsonElement("bcl")]
			public int BuildCommitOrder { get; set; }

			[BsonRequired, BsonElement("d")]
			public TimeSpan Duration { get; set; }

			// --- Single tests
			[BsonIgnoreIfNull, BsonElement("tid")]
			public TestId? TestId { get; set; }

			[BsonIgnoreIfNull, BsonElement("o")]
			public TestOutcome? Outcome { get; set; }

			// --- Suite tests			
			[BsonIgnoreIfNull, BsonElement("suid")]
			public TestSuiteId? SuiteId { get; set; }

			[BsonIgnoreIfNull, BsonElement("ssc")]
			public int? SuiteSkipCount { get; set; }

			[BsonIgnoreIfNull, BsonElement("swc")]
			public int? SuiteWarningCount { get; set; }

			[BsonIgnoreIfNull, BsonElement("sec")]
			public int? SuiteErrorCount { get; set; }

			[BsonIgnoreIfNull, BsonElement("spc")]
			public int? SuiteSuccessCount { get; set; }

			[BsonIgnoreIfNull, BsonElement("jid")]
			public JobId? JobId { get; set; }

			[BsonIgnoreIfNull, BsonElement("stepid")]
			public JobStepId? StepId { get; set; }

			private TestDataRefDocument()
			{

			}

			public TestDataRefDocument(StreamId streamId)
			{
				Id = TestRefId.GenerateNewId();
				StreamId = streamId;
			}
		}

		class TestDataDetailsDocument : ITestDataDetails
		{
			/// <summary>
			/// The corresponding test ref for these details
			/// </summary>
			[BsonRequired, BsonId]
			public TestRefId Id { get; set; }

			/// <summary>
			/// The full (raw) data for this test, which can be consumed by various clients
			/// </summary>
			[BsonRequired, BsonElement("tdid")]
			public List<ObjectId> TestDataIds { get; set; } = new List<ObjectId>();
			IReadOnlyList<ObjectId> ITestDataDetails.TestDataIds => TestDataIds;

			/// <summary>
			/// Suite test data
			/// </summary>
			[BsonIgnoreIfNull, BsonElement("sts")]
			public List<SuiteTestData>? SuiteTests { get; set; }
			IReadOnlyList<ISuiteTestData>? ITestDataDetails.SuiteTests => SuiteTests;

			private TestDataDetailsDocument()
			{

			}

			public TestDataDetailsDocument(TestRefId id, List<ObjectId> testDataIds, List<SuiteTestData>? suiteTests = null)
			{
				Id = id;
				TestDataIds = testDataIds;
				SuiteTests = suiteTests;
			}
		}

		/// <summary>
		/// Information about a test data document
		/// </summary>
		class TestDataDocument : ITestData
		{
			public ObjectId Id { get; set; }
			public StreamId StreamId { get; set; }
			public TemplateId TemplateRefId { get; set; }
			public JobId JobId { get; set; }
			public JobStepId StepId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			public string? CommitName { get; set; }
			
			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			[BsonIgnore]
			public CommitId? PreflightCommitId => null;

			public string Key { get; set; }
			public BsonDocument Data { get; set; }

			private TestDataDocument()
			{
				Key = String.Empty;
				Data = new BsonDocument();
			}

			public TestDataDocument(IJob job, IJobStep jobStep, string key, BsonDocument value)
			{
				Id = ObjectId.GenerateNewId();
				StreamId = job.StreamId;
				TemplateRefId = job.TemplateId;
				JobId = job.Id;
				StepId = jobStep.Id;
				CommitId = job.CommitId;
				Key = key;
				Data = value;
			}
		}

		readonly IMongoCollection<TestDataDocument> _testDataDocuments;
		readonly IMongoCollection<TestMetaDocument> _testMeta;
		readonly IMongoCollection<TestDocument> _tests;
		readonly IMongoCollection<TestSuiteDocument> _testSuites;
		readonly IMongoCollection<TestDataRefDocument> _testRefs;
		readonly IMongoCollection<TestDataDetailsDocument> _testDetails;
		readonly IMongoCollection<TestStreamDocument> _testStreams;
		readonly ICommitService _commitService;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public TestDataCollection(IMongoService mongoService, ICommitService commitService, Tracer tracer, ILogger<TestDataCollection> logger)
		{
			_commitService = commitService;
			_tracer = tracer;
			_logger = logger;

			List<MongoIndex<TestDataDocument>> indexes = new List<MongoIndex<TestDataDocument>>();
			indexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.CommitOrder).Ascending(x => x.Key));
			indexes.Add(keys => keys.Ascending(x => x.JobId).Ascending(x => x.StepId).Ascending(x => x.Key), unique: true);
			_testDataDocuments = mongoService.GetCollection<TestDataDocument>("TestData", indexes);

			List<MongoIndex<TestMetaDocument>> metaIndexes = new List<MongoIndex<TestMetaDocument>>();
			metaIndexes.Add(keys => keys.Ascending(x => x.ProjectName));
			_testMeta = mongoService.GetCollection<TestMetaDocument>("TestData.MetaV2", metaIndexes);

			List<MongoIndex<TestDocument>> testIndexes = new List<MongoIndex<TestDocument>>();
			testIndexes.Add(keys => keys.Ascending(x => x.Name).Ascending(x => x.Metadata));
			_tests = mongoService.GetCollection<TestDocument>("TestData.TestsV2", testIndexes);

			List<MongoIndex<TestSuiteDocument>> testSuiteIndexes = new List<MongoIndex<TestSuiteDocument>>();
			testSuiteIndexes.Add(keys => keys.Ascending(x => x.Name).Ascending(x => x.Tests));
			_testSuites = mongoService.GetCollection<TestSuiteDocument>("TestData.TestSuitesV2", testSuiteIndexes);

			List<MongoIndex<TestDataRefDocument>> testRefIndexes = new List<MongoIndex<TestDataRefDocument>>();
			testRefIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.Metadata).Descending(x => x.BuildCommitOrder).Ascending(x => x.TestId).Ascending(x => x.SuiteId));
			_testRefs = mongoService.GetCollection<TestDataRefDocument>("TestData.TestRefsV2", testRefIndexes);

			_testDetails = mongoService.GetCollection<TestDataDetailsDocument>("TestData.TestDetailsV2");

			List<MongoIndex<TestStreamDocument>> streamIndexes = new List<MongoIndex<TestStreamDocument>>();
			streamIndexes.Add(keys => keys.Ascending(x => x.StreamId), unique: true);
			_testStreams = mongoService.GetCollection<TestStreamDocument>("TestData.TestStreamsV2", streamIndexes);

		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestDataRef>> FindTestRefsAsync(StreamId[] streamIds, TestMetaId[]? metaIds = null, TestId[]? testIds = null, TestSuiteId[]? suiteIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{

			FilterDefinition<TestDataRefDocument> filter = FilterDefinition<TestDataRefDocument>.Empty;
			FilterDefinitionBuilder<TestDataRefDocument> filterBuilder = Builders<TestDataRefDocument>.Filter;

			filter &= Builders<TestDataRefDocument>.Filter.In(x => x.StreamId, streamIds);

			if (minCreateTime != null)
			{
				TestRefId minTime = TestRefId.GenerateNewId(minCreateTime.Value);
				filter &= filterBuilder.Gte(x => x.Id!, minTime);

			}
			if (maxCreateTime != null)
			{
				TestRefId maxTime = TestRefId.GenerateNewId(maxCreateTime.Value);
				filter &= filterBuilder.Lte(x => x.Id!, maxTime);
			}

			if (metaIds != null && metaIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Metadata, metaIds);
			}

			if ((testIds != null && testIds.Length > 0) && (suiteIds != null && suiteIds.Length > 0))
			{
				filter &= filterBuilder.Or(filterBuilder.And(filterBuilder.Ne(x => x.TestId, null), filterBuilder.In(x => (TestId)x.TestId!, testIds)),
										   filterBuilder.And(filterBuilder.Ne(x => x.SuiteId, null), filterBuilder.In(x => (TestSuiteId)x.SuiteId!, suiteIds)));
			}
			else
			{
				if (testIds != null && testIds.Length > 0)
				{
					filter &= filterBuilder.And(filterBuilder.Ne(x => x.TestId, null), filterBuilder.In(x => (TestId)x.TestId!, testIds));
				}

				if (suiteIds != null && suiteIds.Length > 0)
				{
					filter &= filterBuilder.And(filterBuilder.Ne(x => x.SuiteId, null), filterBuilder.In(x => (TestSuiteId)x.SuiteId!, suiteIds));
				}
			}

			if (minCommitId != null)
			{
				int minCommitOrder = minCommitId.GetPerforceChange(); // TODO: need to resolve against correct stream
				filter &= filterBuilder.Gte(x => x.BuildCommitOrder, minCommitOrder);
			}
			if (maxCommitId != null)
			{
				int maxCommitOrder = maxCommitId.GetPerforceChange(); // TODO: need to resolve against correct stream
				filter &= filterBuilder.Lte(x => x.BuildCommitOrder, maxCommitOrder);
			}

			List<TestDataRefDocument> results;

			using (TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(TestDataCollection)}.{nameof(FindTestRefsAsync)}"))
			{
				results = await _testRefs.Find(filter).ToListAsync(cancellationToken);
			}

			return results.ConvertAll<ITestDataRef>(x => x);

		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestData>> AddAsync(IJob job, IJobStep step, (string key, BsonDocument value)[] data, CancellationToken cancellationToken = default)
		{
			// detailed test data
			List<TestDataDocument> documents = new List<TestDataDocument>();
			for (int i = 0; i < data.Length; i++)
			{
				(string key, BsonDocument document) = data[i];

				int version;
				if (document.TryGetInt32("version", out version) || document.TryGetInt32("Version", out version))
				{
					if (version > 1)
					{
						continue;
					}
				}

				documents.Add(new TestDataDocument(job, step, key, document));
			}

			if (documents.Count == 0)
			{
				return documents;
			}

			await _testDataDocuments.InsertManyAsync(documents, null, cancellationToken);

			try
			{
				await AddTestReportDataAsync(job, step, documents, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception while adding test data  report, jobId: {JobId} stepId: {StepId}", job.Id, step.Id);
			}

			return documents;
		}

		/// <inheritdoc/>
		public async Task<ITestData?> GetAsync(ObjectId id, CancellationToken cancellationToken)
		{
			return await _testDataDocuments.Find<TestDataDocument>(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestData>> FindAsync(StreamId? streamId, CommitId? minCommitId, CommitId? maxCommitId, JobId? jobId, JobStepId? stepId, string? key = null, int index = 0, int count = 10, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestDataDocument> filter = FilterDefinition<TestDataDocument>.Empty;
			if (streamId != null)
			{
				filter &= Builders<TestDataDocument>.Filter.Eq(x => x.StreamId, streamId.Value);
				if (minCommitId != null)
				{
					CommitIdWithOrder minCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, minCommitId, cancellationToken);
					filter &= Builders<TestDataDocument>.Filter.Gte(x => x.CommitOrder, minCommitIdWithOrder.Order);
				}
				if (maxCommitId != null)
				{
					CommitIdWithOrder maxCommitIdWithOrder = await _commitService.GetOrderedAsync(streamId.Value, maxCommitId, cancellationToken);
					filter &= Builders<TestDataDocument>.Filter.Lte(x => x.CommitOrder, maxCommitIdWithOrder.Order);
				}
			}
			if (jobId != null)
			{
				filter &= Builders<TestDataDocument>.Filter.Eq(x => x.JobId, jobId.Value);
				if (stepId != null)
				{
					filter &= Builders<TestDataDocument>.Filter.Eq(x => x.StepId, stepId.Value);
				}
			}
			if (key != null)
			{
				filter &= Builders<TestDataDocument>.Filter.Eq(x => x.Key, key);
			}

			SortDefinition<TestDataDocument> sort = Builders<TestDataDocument>.Sort.Ascending(x => x.StreamId).Descending(x => x.CommitOrder);

			return await _testDataDocuments.Find(filter).Sort(sort).Skip(index).Limit(count).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(ObjectId id, CancellationToken cancellationToken = default)
		{
			await _testDataDocuments.DeleteOneAsync<TestDataDocument>(x => x.Id == id, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestDataDetails>> FindTestDetailsAsync(TestRefId[] ids, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<TestDataDetailsDocument> filterBuilder = Builders<TestDataDetailsDocument>.Filter;
			FilterDefinition<TestDataDetailsDocument> filter = FilterDefinition<TestDataDetailsDocument>.Empty;

			if (ids.Length == 1)
			{
				filter &= filterBuilder.Eq(x => x.Id, ids[0]);
			}
			else
			{
				filter &= filterBuilder.In(x => x.Id, ids);
			}

			return await _testDetails.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestMeta>> FindTestMetaAsync(string[]? projectNames = null, string[]? platforms = null, string[]? configurations = null, string[]? buildTargets = null, string? rhi = null, string? variation = null, TestMetaId[]? metaIds = null, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<TestMetaDocument> filterBuilder = Builders<TestMetaDocument>.Filter;
			FilterDefinition<TestMetaDocument> filter = FilterDefinition<TestMetaDocument>.Empty;

			if (metaIds != null && metaIds.Length > 0)
			{
				if (metaIds.Length == 1)
				{
					filter &= filterBuilder.Eq(x => x.Id, metaIds[0]);
				}
				else
				{
					filter &= filterBuilder.In(x => x.Id, metaIds);
				}
			}

			if (projectNames != null && projectNames.Length > 0)
			{
				if (projectNames.Length == 1)
				{
					filter &= filterBuilder.Eq(x => x.ProjectName, projectNames[0]);
				}
				else
				{
					filter &= filterBuilder.In(x => x.ProjectName, projectNames);
				}
			}

			if (platforms != null && platforms.Length > 0)
			{
				if (platforms.Length == 1)
				{
					filter &= filterBuilder.AnyEq(x => x.Platforms, platforms[0]);
				}
				else
				{
					filter &= filterBuilder.AnyIn(x => x.Platforms, platforms);
				}
			}

			if (configurations != null && configurations.Length > 0)
			{
				if (configurations.Length == 1)
				{
					filter &= filterBuilder.AnyEq(x => x.Configurations, configurations[0]);
				}
				else
				{
					filter &= filterBuilder.AnyIn(x => x.Configurations, configurations);
				}
			}

			if (buildTargets != null && buildTargets.Length > 0)
			{
				if (buildTargets.Length == 1)
				{
					filter &= filterBuilder.AnyEq(x => x.BuildTargets, buildTargets[0]);
				}
				else
				{
					filter &= filterBuilder.AnyIn(x => x.BuildTargets, buildTargets);
				}
			}

			if (rhi != null && rhi.Length > 0)
			{
				filter &= filterBuilder.Eq(x => x.RHI, rhi);
			}

			if (variation != null && variation.Length > 0)
			{
				filter &= filterBuilder.Eq(x => x.Variation, variation);
			}

			return await _testMeta.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITest>> FindTestsAsync(TestId[] testIds, CancellationToken cancellationToken = default)
		{
			return await _tests.Find(Builders<TestDocument>.Filter.In(x => x.Id, testIds)).ToListAsync(cancellationToken);

		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestSuite>> FindTestSuitesAsync(TestSuiteId[] suiteIds, CancellationToken cancellationToken = default)
		{
			return await _testSuites.Find(Builders<TestSuiteDocument>.Filter.In(x => x.Id, suiteIds)).ToListAsync(cancellationToken);

		}

		private async Task<List<TestDocument>> FindTestsAsync(bool? suiteTests = null, string[]? projectNames = null, string[]? testNames = null, string[]? suiteNames = null, TestMetaId[]? metaIds = null, TestId[]? testIds = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<ITestMeta> metaData = Array.Empty<ITestMeta>();

			if ((projectNames != null && projectNames.Length > 0) || (metaIds != null && metaIds.Length > 0))
			{
				metaData = await FindTestMetaAsync(projectNames, metaIds: metaIds, cancellationToken: cancellationToken);
			}

			TestMetaId[] queryIds = metaData.Select(x => x.Id).ToArray();

			FilterDefinitionBuilder<TestDocument> filterBuilder = Builders<TestDocument>.Filter;
			FilterDefinition<TestDocument> filter = FilterDefinition<TestDocument>.Empty;

			if (queryIds.Length > 0)
			{
				if (queryIds.Length == 1)
				{
					filter &= filterBuilder.AnyEq(x => x.Metadata, queryIds[0]);
				}
				else
				{
					filter &= filterBuilder.AnyIn(x => x.Metadata, queryIds);
				}
			}

			if (testNames != null)
			{
				if (testNames.Length == 1)
				{
					filter &= filterBuilder.Eq(x => x.Name, testNames[0]);
				}
				else
				{
					filter &= filterBuilder.In(x => x.Name, testNames);
				}
			}

			if (testIds != null)
			{
				if (testIds.Length == 1)
				{
					filter &= filterBuilder.Eq(x => x.Id, testIds[0]);
				}
				else
				{
					filter &= filterBuilder.In(x => x.Id, testIds);
				}
			}
			if (suiteTests == false)
			{
				filter &= filterBuilder.Eq(x => x.SuiteName, null);
			}
			else if (suiteTests == true)
			{
				filter &= filterBuilder.Ne(x => x.SuiteName, null);

				if (suiteNames != null)
				{
					if (suiteNames.Length == 1)
					{
						filter &= filterBuilder.Eq(x => x.SuiteName, suiteNames[0]);
					}
					else
					{
						filter &= filterBuilder.In(x => x.SuiteName, suiteNames);
					}
				}
			}

			return await _tests.Find(filter).ToListAsync(cancellationToken);
		}

		private async Task<List<TestSuiteDocument>> FindTestSuitesAsync(string[] suiteNames, TestMetaId[]? metaIds = null, string[]? projectNames = null, CancellationToken cancellationToken = default)
		{
			IReadOnlyList<ITestMeta> metaData = Array.Empty<ITestMeta>();

			if ((projectNames != null && projectNames.Length > 0) || (metaIds != null && metaIds.Length > 0))
			{
				metaData = await FindTestMetaAsync(projectNames, metaIds: metaIds, cancellationToken: cancellationToken);
			}

			TestMetaId[] queryIds = metaData.Select(x => x.Id).ToArray();

			FilterDefinitionBuilder<TestSuiteDocument> filterBuilder = Builders<TestSuiteDocument>.Filter;
			FilterDefinition<TestSuiteDocument> filter = FilterDefinition<TestSuiteDocument>.Empty;

			if (queryIds.Length > 0)
			{
				if (queryIds.Length == 1)
				{
					filter &= filterBuilder.AnyEq(x => x.Metadata, queryIds[0]);
				}
				else
				{
					filter &= filterBuilder.AnyIn(x => x.Metadata, queryIds);
				}
			}

			if (suiteNames.Length == 1)
			{
				filter &= filterBuilder.Eq(x => x.Name, suiteNames[0]);
			}
			else
			{
				filter &= filterBuilder.In(x => x.Name, suiteNames);
			}

			return await _testSuites.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestStream>> FindTestStreamsAsync(StreamId[] streamIds, CancellationToken cancellationToken)
		{
			return await _testStreams.Find(Builders<TestStreamDocument>.Filter.In(x => x.StreamId, streamIds)).ToListAsync(cancellationToken);
		}

		async Task AddTestRefAsync(TestDataRefDocument testRef, CancellationToken cancellationToken)
		{
			StreamId streamId = testRef.StreamId;

			List<TestStreamDocument> streams = await _testStreams.Find(Builders<TestStreamDocument>.Filter.Eq(x => x.StreamId, streamId)).ToListAsync(cancellationToken);

			if (streams.Count == 0)
			{
				TestStreamDocument streamDoc = new TestStreamDocument(streamId);
				if (testRef.TestId != null)
				{
					streamDoc.Tests.Add(testRef.TestId.Value);
				}

				if (testRef.SuiteId != null)
				{
					streamDoc.TestSuites.Add(testRef.SuiteId.Value);
				}

				await _testStreams.InsertOneAsync(streamDoc, null, cancellationToken);
			}
			else
			{
				UpdateDefinitionBuilder<TestStreamDocument> updateBuilder = Builders<TestStreamDocument>.Update;
				List<UpdateDefinition<TestStreamDocument>> updates = new List<UpdateDefinition<TestStreamDocument>>();

				TestStreamDocument streamDoc = streams[0];

				if (testRef.TestId != null)
				{
					if (!streamDoc.Tests.Contains(testRef.TestId.Value))
					{
						streamDoc.Tests.Add(testRef.TestId.Value);
						updates.Add(updateBuilder.Set(x => x.Tests, streamDoc.Tests));
					}
				}

				if (testRef.SuiteId != null)
				{
					if (!streamDoc.TestSuites.Contains(testRef.SuiteId.Value))
					{
						streamDoc.TestSuites.Add(testRef.SuiteId.Value);
						updates.Add(updateBuilder.Set(x => x.TestSuites, streamDoc.TestSuites));
					}
				}
				if (updates.Count > 0)
				{
					FilterDefinitionBuilder<TestStreamDocument> ebuilder = Builders<TestStreamDocument>.Filter;
					FilterDefinition<TestStreamDocument> efilter = ebuilder.Eq(x => x.StreamId, streamDoc.StreamId);
					await _testStreams.UpdateOneAsync(efilter, updateBuilder.Combine(updates), null, cancellationToken);

				}
			}
			await _testRefs.InsertOneAsync(testRef, null, cancellationToken);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="metaData"></param>
		/// <param name="job"></param>
		/// <param name="step"></param>
		/// <param name="testName"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		private async Task<ITestMeta?> AddTestMetaAsync(Dictionary<string, string> metaData, IJob job, IJobStep step, string testName, CancellationToken cancellationToken)
		{
			string? platform;
			if (!metaData.TryGetValue("Platform", out platform))
			{
				_logger.LogWarning("Test {TestName} is missing platform metadata, jobId: {JobId} stepId: {StepId}", testName, job.Id, step.Id);
				return null;
			}

			List<string> platforms = platform.Split('+', StringSplitOptions.TrimEntries).OrderBy(p => p).ToList();

			string? buildTarget;
			if (!metaData.TryGetValue("BuildTarget", out buildTarget))
			{
				_logger.LogWarning("Test {TestName} is missing build target metadata, jobId: {JobId} stepId: {StepId}", testName, job.Id, step.Id);
				return null;
			}

			List<string> buildTargets = buildTarget.Split('+', StringSplitOptions.TrimEntries).OrderBy(b => b).ToList();

			string? configuration;
			if (!metaData.TryGetValue("Configuration", out configuration))
			{
				_logger.LogWarning("Test {TestName} is missing configuration metadata, jobId: {JobId} stepId: {StepId}", testName, job.Id, step.Id);
				return null;
			}

			List<string> configurations = configuration.Split('+', StringSplitOptions.TrimEntries).OrderBy(c => c).ToList();

			string? projectName;
			if (!metaData.TryGetValue("Project", out projectName))
			{
				_logger.LogWarning("Test {TestName} is missing project metadata, jobId: {JobId} stepId: {StepId}", testName, job.Id, step.Id);
				return null;
			}

			string? rhi;
			metaData.TryGetValue("RHI", out rhi);
			rhi ??= "default";

			string? variation;
			metaData.TryGetValue("Variation", out variation);
			variation ??= "default";

			TestMetaDocument meta = new TestMetaDocument(platforms, configurations, buildTargets, projectName, rhi, variation);

			FilterDefinitionBuilder<TestMetaDocument> filterBuilder = Builders<TestMetaDocument>.Filter;
			FilterDefinition<TestMetaDocument> filter = filterBuilder.Eq(x => x.ProjectName, meta.ProjectName);
			filter &= filterBuilder.Eq(x => x.RHI, meta.RHI);
			filter &= filterBuilder.Eq(x => x.Variation, meta.Variation);
			filter &= filterBuilder.Eq(x => x.Platforms, meta.Platforms);
			filter &= filterBuilder.Eq(x => x.BuildTargets, meta.BuildTargets);
			filter &= filterBuilder.Eq(x => x.Configurations, meta.Configurations);

			TestMetaDocument? result = await _testMeta.Find(filter).FirstOrDefaultAsync(cancellationToken);

			if (result == null)
			{
				result = meta;
				await _testMeta.InsertOneAsync(result, null, cancellationToken);
			}
			else
			{
				UpdateDefinitionBuilder<TestMetaDocument> updateBuilder = Builders<TestMetaDocument>.Update;
				List<UpdateDefinition<TestMetaDocument>> updates = new List<UpdateDefinition<TestMetaDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				await _testMeta.FindOneAndUpdateAsync(x => x.Id == result.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			return result;

		}

		async Task<TestDocument?> AddOrUpdateTestAsync(ITestMeta metaData, string testName, string? suiteName = null, string? displayName = null, CancellationToken cancellationToken = default)
		{

			List<TestDocument> tests;

			if (suiteName != null)
			{
				tests = await FindTestsAsync(true, projectNames: new string[] { metaData.ProjectName }, testNames: new string[] { testName }, suiteNames: new string[] { suiteName }, cancellationToken: cancellationToken);
			}
			else
			{
				tests = await FindTestsAsync(false, projectNames: new string[] { metaData.ProjectName }, testNames: new string[] { testName }, cancellationToken: cancellationToken);
			}

			if (tests.Count > 1)
			{
				throw new Exception($"Duplicate tests found for MetaId: {metaData.Id}, TestName: {testName}, SuiteName: {suiteName}, Count: {tests.Count}");
			}

			TestDocument? test;

			if (tests.Count == 0)
			{
				test = new TestDocument(testName, new List<TestMetaId>() { metaData.Id }, suiteName, displayName);
				await _tests.InsertOneAsync(test, null, cancellationToken);
			}
			else
			{
				test = tests[0];

				UpdateDefinitionBuilder<TestDocument> updateBuilder = Builders<TestDocument>.Update;
				List<UpdateDefinition<TestDocument>> updates = new List<UpdateDefinition<TestDocument>>();

				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));

				if (!test.Metadata.Contains(metaData.Id))
				{
					test.Metadata.Add(metaData.Id);

					updates.Add(updateBuilder.Set(x => x.Metadata, test.Metadata));
				}

				await _tests.FindOneAndUpdateAsync(x => x.Id == test.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			return test;
		}

		private async Task<TestSuiteDocument?> AddOrUpdateTestSuiteAsync(string suiteName, ITestMeta metaData, List<TestId> testIds, CancellationToken cancellationToken)
		{
			TestSuiteDocument? suite;

			List<TestSuiteDocument> suiteTests = await FindTestSuitesAsync(suiteNames: new string[] { suiteName }, projectNames: new string[] { metaData.ProjectName }, cancellationToken: cancellationToken);

			if (suiteTests.Count > 1)
			{
				throw new Exception($"Duplicate tests found for MetaId: {metaData.Id}, SuiteName: {suiteName}, Count: {suiteTests.Count}");
			}

			if (suiteTests.Count == 0)
			{
				suite = new TestSuiteDocument(suiteName, new List<TestMetaId> { metaData.Id }, testIds);
				await _testSuites.InsertOneAsync(suite, null, cancellationToken);
			}
			else
			{
				suite = suiteTests[0];

				UpdateDefinitionBuilder<TestSuiteDocument> updateBuilder = Builders<TestSuiteDocument>.Update;
				List<UpdateDefinition<TestSuiteDocument>> updates = new List<UpdateDefinition<TestSuiteDocument>>();

				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));

				List<TestId> newIds = suite.Tests.Union(testIds).ToList();

				if (newIds.Count != suite.Tests.Count)
				{
					suite.Tests = newIds;
					updates.Add(updateBuilder.Set(x => x.Tests, suite.Tests));
				}

				if (!suite.Metadata.Contains(metaData.Id))
				{
					suite.Metadata.Add(metaData.Id);
					updates.Add(updateBuilder.Set(x => x.Metadata, suite.Metadata));
				}

				await _testSuites.FindOneAndUpdateAsync(x => x.Id == suite.Id, updateBuilder.Combine(updates), null, cancellationToken);

			}

			return suite;
		}

		async Task AddTestReportDataAsync(IJob job, IJobStep step, List<TestDataDocument> documents, CancellationToken cancellationToken = default)
		{

			// do not add preflight to temporal data
			if (job.PreflightCommitId != null)
			{
				return;
			}

			List<AutomatedTestSessionData> sessions = new List<AutomatedTestSessionData>();
			List<UnrealAutomatedTestData> tests = new List<UnrealAutomatedTestData>();

			foreach (TestDataDocument item in documents)
			{
				BsonDocument testData = item.Data;

				BsonValue? value;
				if (!testData.TryGetPropertyValue("Type", BsonType.String, out value))
				{
					continue;
				}

				string type = value.AsString;

				if (type == "Simple Report")
				{
					await AddSimpleReportDataAsync(job, step, item.Id, testData, cancellationToken);
					continue;
				}

				if (type == "Automated Test Session")
				{
					sessions.Add(BsonSerializer.Deserialize<AutomatedTestSessionData>(testData));
				}

				if (type == "Unreal Automated Tests")
				{
					tests.Add(BsonSerializer.Deserialize<UnrealAutomatedTestData>(testData));
				}
			}

			if (sessions.Count > 0)
			{
				await AddTestSessionReportDataAsync(job, step, documents, sessions, tests, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task PruneDataAsync(CancellationToken cancellationToken = default)
		{

			HashSet<TestMetaId> metaIds = new HashSet<TestMetaId>();
			foreach (TestMetaId metaId in await (await _testMeta.DistinctAsync(x => x.Id, Builders<TestMetaDocument>.Filter.Empty, null, cancellationToken)).ToListAsync(cancellationToken))
			{
				metaIds.Add(metaId);
			}

			_logger.LogInformation("Test data pruning to {MetaIdCount} meta ids", metaIds.Count);

			// Update tests
			HashSet<TestId> testIds = new HashSet<TestId>();
			HashSet<TestId> testDeletes = new HashSet<TestId>();
			{
				List<TestDocument> tests = await _tests.Find(Builders<TestDocument>.Filter.Empty).ToListAsync(cancellationToken);
				foreach (TestDocument test in tests)
				{
					List<TestMetaId> metas = test.Metadata.Where(x => metaIds.Contains(x)).ToList();

					if (metas.Count == 0)
					{
						testDeletes.Add(test.Id);
						continue;
					}
					else if (metas.Count != test.Metadata.Count)
					{
						UpdateDefinitionBuilder<TestDocument> updateBuilder = Builders<TestDocument>.Update;
						List<UpdateDefinition<TestDocument>> updates = new List<UpdateDefinition<TestDocument>>();
						updates.Add(updateBuilder.Set(x => x.Metadata, metas));
						await _tests.FindOneAndUpdateAsync(x => x.Id == test.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}

					testIds.Add(test.Id);
				}
			}

			// Update suites
			HashSet<TestSuiteId> suiteIds = new HashSet<TestSuiteId>();
			HashSet<TestSuiteId> suiteDeletes = new HashSet<TestSuiteId>();
			{
				List<TestSuiteDocument> suites = await _testSuites.Find(Builders<TestSuiteDocument>.Filter.Empty).ToListAsync(cancellationToken);
				foreach (TestSuiteDocument suite in suites)
				{
					List<TestMetaId> metas = suite.Metadata.Where(x => metaIds.Contains(x)).ToList();
					List<TestId> tests = suite.Tests.Where(x => testIds.Contains(x)).ToList();

					if (metas.Count == 0 || tests.Count == 0)
					{
						suiteDeletes.Add(suite.Id);
						continue;
					}
					else if (metas.Count != suite.Metadata.Count || tests.Count != suite.Tests.Count)
					{
						UpdateDefinitionBuilder<TestSuiteDocument> updateBuilder = Builders<TestSuiteDocument>.Update;
						List<UpdateDefinition<TestSuiteDocument>> updates = new List<UpdateDefinition<TestSuiteDocument>>();
						if (metas.Count != suite.Metadata.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Metadata, metas));
						}

						if (tests.Count != suite.Tests.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Tests, tests));
						}

						await _testSuites.FindOneAndUpdateAsync(x => x.Id == suite.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}

					suiteIds.Add(suite.Id);
				}
			}

			if (testDeletes.Count > 0)
			{
				FilterDefinitionBuilder<TestDocument> filterBuilder = Builders<TestDocument>.Filter;
				FilterDefinition<TestDocument> filter = filterBuilder.Empty;
				filter &= filterBuilder.In(x => x.Id, testDeletes);
				await _tests.DeleteManyAsync(filter, cancellationToken);
			}

			if (suiteDeletes.Count > 0)
			{
				FilterDefinitionBuilder<TestSuiteDocument> filterBuilder = Builders<TestSuiteDocument>.Filter;
				FilterDefinition<TestSuiteDocument> filter = filterBuilder.Empty;
				filter &= filterBuilder.In(x => x.Id, suiteDeletes);
				await _testSuites.DeleteManyAsync(filter, cancellationToken);
			}

			if (testDeletes.Count > 0 || suiteDeletes.Count > 0)
			{
				_logger.LogInformation("Pruned {TestDeleteCount} tests and {TestSuiteCOunt} suites", testDeletes.Count, suiteDeletes.Count);
			}

			int testStreamsDeleted = 0;

			// Update test streams
			{
				List<TestStreamDocument> testStreams = await _testStreams.Find(Builders<TestStreamDocument>.Filter.Empty).ToListAsync(cancellationToken);
				foreach (TestStreamDocument stream in testStreams)
				{
					List<TestId> tests = stream.Tests.Where(x => testIds.Contains(x)).ToList();
					List<TestSuiteId> suites = stream.TestSuites.Where(x => suiteIds.Contains(x)).ToList();

					if (tests.Count == 0 && suites.Count == 0)
					{
						await _testStreams.DeleteOneAsync(x => x.Id == stream.Id, cancellationToken);
						testStreamsDeleted++;
					}
					else if (tests.Count != stream.Tests.Count || suites.Count != stream.TestSuites.Count)
					{
						UpdateDefinitionBuilder<TestStreamDocument> updateBuilder = Builders<TestStreamDocument>.Update;
						List<UpdateDefinition<TestStreamDocument>> updates = new List<UpdateDefinition<TestStreamDocument>>();

						if (tests.Count != stream.Tests.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Tests, tests));
						}

						if (suites.Count != stream.TestSuites.Count)
						{
							updates.Add(updateBuilder.Set(x => x.TestSuites, suites));
						}

						await _testStreams.FindOneAndUpdateAsync(x => x.Id == stream.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}
				}
			}

			if (testStreamsDeleted > 0)
			{
				_logger.LogInformation("Pruned {TestStreamDeleteCount} test streams", testStreamsDeleted);
			}
		}

		static async Task<long> ExpireCollectionAsync<DocType>(IMongoCollection<DocType> collection, DateTime expireTime, CancellationToken cancellationToken) where DocType : ITestExpire
		{
			FilterDefinitionBuilder<DocType> filterBuilder = Builders<DocType>.Filter;
			FilterDefinition<DocType> filter = filterBuilder.Empty;
			filter &= filterBuilder.Lte(x => x.LastSeenUtc!, expireTime);
			DeleteResult result = await collection.DeleteManyAsync(filter, cancellationToken);

			return result.DeletedCount;

		}

		async Task ExpireTestDataAsync(int retainMonths, CancellationToken cancellationToken)
		{
			if (retainMonths <= 0)
			{
				return;
			}

			DateTime expireTime = DateTime.Now.AddMonths(-retainMonths);

			// expire and prune root test data
			long testsExpired = await ExpireCollectionAsync(_tests, expireTime, cancellationToken);
			long suitesExpired = await ExpireCollectionAsync(_testSuites, expireTime, cancellationToken);
			long metaExpired = await ExpireCollectionAsync(_testMeta, expireTime, cancellationToken);

			// Check whether we need to prune
			if (testsExpired > 0 || suitesExpired > 0 || metaExpired > 0)
			{
				_logger.LogInformation("Expired {TestsExpired} tests, {SuitesExpired} suites, {MetaExpired} meta", testsExpired, suitesExpired, metaExpired);
				await PruneDataAsync(cancellationToken);
			}

			// expire test data documents
			{
				ObjectId dataExpireTime = ObjectId.GenerateNewId(expireTime);

				FilterDefinition<TestDataDocument> filter = FilterDefinition<TestDataDocument>.Empty;
				FilterDefinitionBuilder<TestDataDocument> filterBuilder = Builders<TestDataDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, dataExpireTime);
				DeleteResult result = await _testDataDocuments.DeleteManyAsync(filter, null, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test data documents", result.DeletedCount);
				}
			}

			TestRefId refExpireTime = TestRefId.GenerateNewId(expireTime);

			// expire test refs
			{
				FilterDefinition<TestDataRefDocument> filter = FilterDefinition<TestDataRefDocument>.Empty;
				FilterDefinitionBuilder<TestDataRefDocument> filterBuilder = Builders<TestDataRefDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, refExpireTime);
				DeleteResult result = await _testRefs.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test ref documents", result.DeletedCount);
				}
			}

			// expire test details
			{
				FilterDefinition<TestDataDetailsDocument> filter = FilterDefinition<TestDataDetailsDocument>.Empty;
				FilterDefinitionBuilder<TestDataDetailsDocument> filterBuilder = Builders<TestDataDetailsDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, refExpireTime);
				DeleteResult result = await _testDetails.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test detail documents", result.DeletedCount);
				}
			}
		}

		/// <inheritdoc/>
		public async Task<bool> UpdateAsync(int retainMonths, CancellationToken cancellationToken)
		{
			await ExpireTestDataAsync(retainMonths, cancellationToken);

			return true;
		}

		/// <inheritdoc/>
		public async Task UpgradeAsync(CancellationToken cancellationToken)
		{
			// Upgrade test meta data
			{
				FilterDefinition<TestMetaDocument> filter = Builders<TestMetaDocument>.Filter.Empty;
				filter &= Builders<TestMetaDocument>.Filter.Where(x => x.Version == null || x.Version < TestMetaDocument.CurrentVersion);

				List<TestMetaDocument> results = await _testMeta.Find(filter).ToListAsync(cancellationToken);

				if (results.Count > 0)
				{
					_logger.LogInformation("Found {Count} test meta documents to upgrade", results.Count);
				}

				foreach (TestMetaDocument meta in results)
				{
					// unversioned => 1
					if (meta.Version == null && TestMetaDocument.CurrentVersion == 1)
					{
						UpdateDefinitionBuilder<TestMetaDocument> updateBuilder = Builders<TestMetaDocument>.Update;
						List<UpdateDefinition<TestMetaDocument>> updates = new List<UpdateDefinition<TestMetaDocument>>();

						updates.Add(updateBuilder.Set(x => x.Version, TestMetaDocument.CurrentVersion));
						updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));

						if (meta.Variation == null || meta.Variation.Length == 0)
						{
							updates.Add(updateBuilder.Set(x => x.Variation, "default"));
						}

						await _testMeta.FindOneAndUpdateAsync(x => x.Id == meta.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}
				}
			}

			// Upgrade tests
			{
				FilterDefinition<TestDocument> filter = Builders<TestDocument>.Filter.Empty;
				filter &= Builders<TestDocument>.Filter.Where(x => x.Version == null || x.Version < TestDocument.CurrentVersion);

				List<TestDocument> results = await _tests.Find(filter).ToListAsync(cancellationToken);

				if (results.Count > 0)
				{
					_logger.LogInformation("Found {Count} test documents to upgrade", results.Count);
				}

				foreach (TestDocument test in results)
				{
					// unversioned => 1
					if (test.Version == null && TestDocument.CurrentVersion == 1)
					{
						UpdateDefinitionBuilder<TestDocument> updateBuilder = Builders<TestDocument>.Update;
						List<UpdateDefinition<TestDocument>> updates = new List<UpdateDefinition<TestDocument>>();

						updates.Add(updateBuilder.Set(x => x.Version, TestDocument.CurrentVersion));
						updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));

						await _tests.FindOneAndUpdateAsync(x => x.Id == test.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}
				}
			}

			// Upgrade test suites
			{
				FilterDefinition<TestSuiteDocument> filter = Builders<TestSuiteDocument>.Filter.Empty;
				filter &= Builders<TestSuiteDocument>.Filter.Where(x => x.Version == null || x.Version < TestSuiteDocument.CurrentVersion);

				List<TestSuiteDocument> results = await _testSuites.Find(filter).ToListAsync(cancellationToken);

				if (results.Count > 0)
				{
					_logger.LogInformation("Found {Count} test suite documents to upgrade", results.Count);
				}

				foreach (TestSuiteDocument suite in results)
				{
					// unversioned => 1
					if (suite.Version == null && TestSuiteDocument.CurrentVersion == 1)
					{
						UpdateDefinitionBuilder<TestSuiteDocument> updateBuilder = Builders<TestSuiteDocument>.Update;
						List<UpdateDefinition<TestSuiteDocument>> updates = new List<UpdateDefinition<TestSuiteDocument>>();

						updates.Add(updateBuilder.Set(x => x.Version, TestSuiteDocument.CurrentVersion));
						updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));

						await _testSuites.FindOneAndUpdateAsync(x => x.Id == suite.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}
				}
			}
		}

		// --- (Minimal) Parsing of Automation framework tests, geared towards generating indexes for aggregate queries

		class AutomatedTestSessionData
		{
			public class SessionTest
			{
				public string Name { get; set; } = String.Empty;
				public string TestUID { get; set; } = String.Empty;
				public string Suite { get; set; } = String.Empty;
				public string State { get; set; } = String.Empty;
				public List<string> DeviceAppInstanceName { get; set; } = new List<string>();
				public int ErrorCount { get; set; } = 0;
				public int WarningCount { get; set; } = 0;
				public double TimeElapseSec { get; set; } = 0;
			}

			public class TestSessionInfoType
			{
				public double TimeElapseSec { get; set; } = 0;
				public Dictionary<string, SessionTest> Tests { get; set; } = new Dictionary<string, SessionTest>();
				public string TestResultsTestDataUID { get; set; } = String.Empty;
			}

			public string Type { get; set; } = String.Empty;
			public string Name { get; set; } = String.Empty;

			public TestSessionInfoType? TestSessionInfo { get; set; }
			public Dictionary<string, string> Metadata { get; set; } = new Dictionary<string, string>();

		}

		class UnrealAutomatedTestData
		{
			public class UnrealTest
			{
				public string TestDisplayName { get; set; } = String.Empty;
				public string FullTestPath { get; set; } = String.Empty;
				public string State { get; set; } = String.Empty;
				public string DeviceInstance { get; set; } = String.Empty;
				public int Errors { get; set; } = 0;
				public int Warnings { get; set; } = 0;
			}

			public string Type { get; set; } = String.Empty;
			public double TotalDurationSeconds { get; set; } = 0;
			public List<UnrealTest> Tests { get; set; } = new List<UnrealTest>();
			public Dictionary<string, string> Metadata { get; set; } = new Dictionary<string, string>();

		}

		// Simple Test Reports

		class TestRole
		{
			public string Type { get; set; } = String.Empty;
			public string Platform { get; set; } = String.Empty;
			public string Configuration { get; set; } = String.Empty;
		}

		class SimpleTestData
		{
			public string TestName { get; set; } = String.Empty;
			public string Description { get; set; } = String.Empty;
			public string ReportCreatedOn { get; set; } = String.Empty;
			public double TotalDurationSeconds { get; set; } = 0;
			public bool HasSucceeded { get; set; } = false;
			public string Status { get; set; } = String.Empty;
			public string URLLink { get; set; } = String.Empty;
			public int BuildChangeList { get; set; } = 0;
			public TestRole? MainRole { get; set; }
			public List<TestRole>? Roles { get; set; } = new List<TestRole>();
			public string TestResult { get; set; } = "Invalid";
			public List<string> Logs { get; set; } = new List<string>();
			public List<string> Errors { get; set; } = new List<string>();
			public List<string> Warnings { get; set; } = new List<string>();
			public Dictionary<string, string> Metadata { get; set; } = new Dictionary<string, string>();
		}

		async Task AddSimpleReportDataAsync(IJob job, IJobStep step, ObjectId testDataId, BsonDocument testDoc, CancellationToken cancellationToken)
		{
			try
			{
				SimpleTestData testData = BsonSerializer.Deserialize<SimpleTestData>(testDoc);

				ITestMeta? metaData = await AddTestMetaAsync(testData.Metadata, job, step, testData.TestName, cancellationToken);

				if (metaData == null)
				{
					throw new Exception($"Unable to add or update simple test report for {testData.TestName}, unable to generate meta data");
				}

				TestDocument? test = await AddOrUpdateTestAsync(metaData!, testData.TestName, cancellationToken: cancellationToken);

				if (test == null)
				{
					throw new Exception($"Unable to add or update simple test report for {testData.TestName}, unable to generate test document");
				}

				TestDataRefDocument testRef = new TestDataRefDocument(job.StreamId);
				testRef.StreamId = job.StreamId;
				testRef.JobId = job.Id;
				testRef.StepId = step.Id;
				testRef.Metadata = metaData.Id;
				testRef.BuildCommitId = testData.BuildChangeList <= 0 ? job.CommitId : CommitIdWithOrder.FromPerforceChange(testData.BuildChangeList);
				testRef.Duration = TimeSpan.FromSeconds(testData.TotalDurationSeconds);
				testRef.TestId = test.Id;
				testRef.Outcome = testData.HasSucceeded ? TestOutcome.Success : TestOutcome.Unspecified;

				if (testRef.Outcome == TestOutcome.Unspecified)
				{
					switch (testData.TestResult)
					{
						case "Passed":
							testRef.Outcome = TestOutcome.Success;
							break;
						case "Failed":
						case "Cancelled":
						case "TimedOut":
							testRef.Outcome = TestOutcome.Failure;
							break;
					}
				}

				await AddTestRefAsync(testRef, cancellationToken);
				await _testDetails.InsertOneAsync(new TestDataDetailsDocument(testRef.Id, new List<ObjectId> { testDataId }), null, cancellationToken);

			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception while adding test data simple report, jobId: {JobId} stepId: {StepId}", job.Id, step.Id);
			}
		}

		async Task AddTestSessionReportDataAsync(IJob job, IJobStep step, List<TestDataDocument> documents, List<AutomatedTestSessionData> sessions, List<UnrealAutomatedTestData> tests, CancellationToken cancellationToken)
		{
			HashSet<string> suites = new HashSet<string>();

			for (int i = 0; i < sessions.Count; i++)
			{
				AutomatedTestSessionData session = sessions[i];

				AutomatedTestSessionData.TestSessionInfoType? info = session.TestSessionInfo;
				if (info == null)
				{
					return;
				}

				foreach (KeyValuePair<string, AutomatedTestSessionData.SessionTest> item in info.Tests)
				{
					AutomatedTestSessionData.SessionTest test = item.Value;

					if (!String.IsNullOrEmpty(test.Suite))
					{
						suites.Add(test.Suite);
					}
				}
			}

			if (suites.Count == 0)
			{
				return;
			}

			foreach (string suite in suites)
			{
				for (int i = 0; i < sessions.Count; i++)
				{
					HashSet<TestId> suiteTestIds = new HashSet<TestId>();
					Dictionary<string, TestDocument> suiteTestDocuments = new Dictionary<string, TestDocument>();

					AutomatedTestSessionData session = sessions[i];
					AutomatedTestSessionData.TestSessionInfoType? info = session.TestSessionInfo;
					if (info == null)
					{
						continue;
					}

					ITestMeta? metaData = await AddTestMetaAsync(session.Metadata, job, step, "Automated Test Session", cancellationToken);

					if (metaData == null)
					{
						throw new Exception($"Unable to add or update simple test report for automated test session, unable to generate meta data");
					}

					List<AutomatedTestSessionData.SessionTest> suiteTests = info.Tests.Where(x => x.Value.Suite == suite).Select(x => x.Value).ToList();

					foreach (AutomatedTestSessionData.SessionTest test in suiteTests)
					{

						for (int j = 0; j < tests.Count; j++)
						{
							UnrealAutomatedTestData testData = tests[j];

							// attempt to find the unreal test, which holds some optional data
							UnrealAutomatedTestData.UnrealTest? utest = null;
							for (int k = 0; k < testData.Tests.Count; k++)
							{
								UnrealAutomatedTestData.UnrealTest u = testData.Tests[k];
								if (test.Name == u.FullTestPath && test.DeviceAppInstanceName.Contains(u.DeviceInstance, StringComparer.OrdinalIgnoreCase))
								{
									utest = u;
									break;
								}
							}

							string? displayName = null;
							if (utest != null && !String.IsNullOrEmpty(utest.TestDisplayName))
							{
								displayName = utest.TestDisplayName;
							}

							TestDocument? testDoc = await AddOrUpdateTestAsync(metaData, test.Name, suite, displayName, cancellationToken);

							if (testDoc == null)
							{
								_logger.LogWarning("Skipping adding suite test {TestName} metaId: {MetaId} jobId: {JobId} stepId: {StepId}", test.Name, metaData.Id, job.Id, step.Id);
								continue;
							}

							suiteTestDocuments.Add(test.Name, testDoc);
							suiteTestIds.Add(testDoc.Id);
						}
					}

					// register test suite
					if (suiteTestIds.Count > 0)
					{
						TestSuiteDocument? testSuite = await AddOrUpdateTestSuiteAsync(suite, metaData, suiteTestIds.ToList(), cancellationToken);

						if (testSuite == null)
						{
							continue;
						}

						TestDataRefDocument testRef = new TestDataRefDocument(job.StreamId);
						testRef.StreamId = job.StreamId;
						testRef.JobId = job.Id;
						testRef.StepId = step.Id;
						testRef.Metadata = metaData.Id;
						testRef.BuildCommitId = job.CommitId;
						testRef.Duration = TimeSpan.FromSeconds(session.TestSessionInfo!.TimeElapseSec);
						testRef.Outcome = TestOutcome.Unspecified;
						testRef.SuiteId = testSuite.Id;

						int skipCount = 0;
						int warningCount = 0;
						int errorCount = 0;
						int successCount = 0;

						// populate the suite tests
						List<SuiteTestData> suiteTestData = new List<SuiteTestData>();
						foreach (AutomatedTestSessionData.SessionTest test in suiteTests)
						{
							bool counted = false;
							if (test.WarningCount > 0)
							{
								counted = true;
								warningCount++;
							}

							if (test.ErrorCount > 0)
							{
								counted = true;
								errorCount++;
							}

							TestDocument document = suiteTestDocuments[test.Name];
							TestOutcome outcome = TestOutcome.Unspecified;

							switch (test.State)
							{
								case "Success":
									outcome = TestOutcome.Success;
									if (!counted)
									{
										successCount++;
									}
									break;
								case "Skipped":
									outcome = TestOutcome.Skipped;
									if (!counted)
									{
										skipCount++;
									}
									break;
								case "Fail":
									outcome = TestOutcome.Failure;
									if (!counted)
									{
										errorCount++;
									}
									break;
							}

							suiteTestData.Add(new SuiteTestData(document.Id, outcome, TimeSpan.FromSeconds(test.TimeElapseSec), test.TestUID, test.WarningCount != 0 ? test.WarningCount : null, test.ErrorCount != 0 ? test.ErrorCount : null));
						}

						testRef.SuiteSkipCount = skipCount;
						testRef.SuiteErrorCount = errorCount;
						testRef.SuiteWarningCount = warningCount;
						testRef.SuiteSuccessCount = successCount;

						// and add to collections
						await AddTestRefAsync(testRef, cancellationToken);
						await _testDetails.InsertOneAsync(new TestDataDetailsDocument(testRef.Id, documents.Select(x => x.Id).ToList(), suiteTestData), null, cancellationToken);
					}
					else
					{
						_logger.LogWarning("Skipping suite {SuiteName} with not test ids, jobId: {JobId} stepId: {StepId}", suite, job.Id, step.Id);
					}
				}
			}
		}
	}
}
