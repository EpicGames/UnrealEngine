// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Jobs.TestData;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Collection of test data documents v2
	/// </summary>
	public class TestDataCollectionV2 : ITestDataCollectionV2
	{
		/// <summary>
		/// Interface to handle expiration
		/// </summary>
		internal interface ITestExpire
		{
			DateTime LastSeenUtc { get; }
		}

		/// <summary>
		/// Metadata key value pair
		/// </summary>
		class TestMetaEntry : ITestMetaEntry
		{
			[BsonRequired, BsonElement("k")]
			public string Key { get; set; }

			[BsonRequired, BsonElement("v")]
			public string Value { get; set; }

			private TestMetaEntry()
			{
				Key = String.Empty;
				Value = String.Empty;
			}

			public TestMetaEntry(string key, string data)
			{
				Key = key;
				Value = data;
			}
		}

		/// <summary>
		/// Test metadata document
		/// </summary>
		class TestMetadataDocument : ITestMetaRef, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestMetaId Id { get; set; }

			[BsonRequired, BsonElement("m")]
			public List<TestMetaEntry> Entries { get; set; }
			IReadOnlyList<ITestMetaEntry> ITestMetaRef.Entries => Entries;

			[BsonRequired, BsonElement("s")]
			public DateTime LastSeenUtc { get; set; }

			private TestMetadataDocument()
			{
				Entries = new();
			}

			public TestMetadataDocument(IReadOnlyDictionary<string, string> entries)
			{
				Id = TestMetaId.GenerateNewId();
				Entries = entries.Select(i => new TestMetaEntry(i.Key, i.Value)).ToList();
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Param key value pair
		/// </summary>
		class TestParamEntry : ITestParamEntry
		{
			[BsonRequired, BsonElement("k")]
			public string Key { get; set; }

			[BsonRequired, BsonElement("v")]
			public object Value { get; set; }

			private TestParamEntry()
			{
				Key = String.Empty;
				Value = false;
			}

			public TestParamEntry(string key, object data)
			{
				Key = key;
				Value = data;
			}
		}

		/// <summary>
		/// Test recipe reference document
		/// </summary>
		class TestRecipeDocument : ITestRecipeRef, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestRecipeId Id { get; set; }

			[BsonRequired, BsonElement("t")]
			public TestId NameRef { get; set; }

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("n")]
			public string NodeClass { get; set; }

			[BsonRequired, BsonElement("v")]
			public string Version { get; set; }

			[BsonRequired, BsonElement("o")]
			public float TimeoutMinutes { get; set; }

			[BsonRequired, BsonElement("jt")]
			public TemplateId JobTemplateId { get; set; }
			
			[BsonRequired, BsonElement("js")]
			public string JobStep { get; set; }

			[BsonIgnoreIfNull, BsonElement("w")]
			public WorkflowId? WorkflowId { get; set; }

			[BsonIgnoreIfNull, BsonElement("p")]
			public List<TestParamEntry>? Params { get; set; }
			IReadOnlyList<ITestParamEntry>? ITestRecipeRef.Params => Params;

			[BsonRequired, BsonElement("s")]
			public DateTime LastSeenUtc { get; set; }

			private TestRecipeDocument()
			{
				NodeClass = String.Empty;
				Version = String.Empty;
				JobStep = String.Empty;
			}

			public TestRecipeDocument(TestId testId, StreamId streamId, TemplateId jobTemplate, string jobStep, string nodeClass, string version, float timeoutMinutes, WorkflowId? workflow = null)
			{
				Id = TestRecipeId.GenerateNewId();
				NameRef = testId;
				StreamId = streamId;
				JobTemplateId = jobTemplate;
				JobStep = jobStep;
				NodeClass = nodeClass;
				Version = version;
				TimeoutMinutes = timeoutMinutes;
				WorkflowId = workflow;
				LastSeenUtc = DateTime.UtcNow;
			}

			public IReadOnlyList<ITestParamEntry> UpdateParams(IReadOnlyDictionary<string, object> @params)
			{
				Params = @params.Select(i => new TestParamEntry(i.Key, i.Value)).ToList();

				return Params;
			}
		}

		/// <summary>
		/// Test tag reference document
		/// </summary>
		class TestTagDocument : ITestTagRef, ITestExpire
		{
			[BsonRequired, BsonId] 
			public TestTagId Id { get; set; }

			[BsonRequired, BsonElement("n")] 
			public string Name { get; set; }

			[BsonRequired, BsonElement("s")]
			public DateTime LastSeenUtc { get; set; }

			private TestTagDocument()
			{
				Name = String.Empty;
			}

			public TestTagDocument(string name)
			{
				Id = TestTagId.GenerateNewId();
				Name = name;
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Test name reference document
		/// </summary>
		class TestNameDocument : ITestNameRef, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestId Id { get; set; }

			[BsonRequired, BsonElement("n")]
			public string Name { get; set; }

			[BsonRequired, BsonElement("k")]
			public string Key { get; set; }

			[BsonIgnoreIfNull, BsonElement("t")]
			public string? Team { get; set; }

			[BsonIgnoreIfNull, BsonElement("i")]
			public string? Intent { get; set; }

			[BsonIgnoreIfNull, BsonElement("h")]
			public string? Harness { get; set;  }

			[BsonIgnoreIfNull, BsonElement("p")]
			public string? Profile { get; set; }

			[BsonRequired, BsonElement("s")]
			public DateTime LastSeenUtc { get; set; }

			private TestNameDocument()
			{
				Key = String.Empty;
				Name = String.Empty;
			}

			public TestNameDocument(string key, string name)
			{
				Id = TestId.GenerateNewId();
				Key = key;
				Name = name;
				LastSeenUtc = DateTime.UtcNow;
			}

			public void AddDescription(string? team, string? intent, string? harness, string? profile)
			{
				Team = team;
				Intent = intent;
				Harness = harness;
				Profile = profile;
			}
		}

		/// <summary>
		/// Test phase reference document
		/// </summary>
		class TestPhaseDocument : ITestPhaseRef, ITestExpire
		{
			[BsonRequired, BsonId]
			public TestPhaseId Id { get; set; }

			[BsonRequired, BsonElement("tid")]
			public TestId TestNameRef { get; set; }

			[BsonRequired, BsonElement("n")]
			public string Name { get; set; }

			[BsonRequired, BsonElement("k")]
			public string Key { get; set; }

			[BsonRequired, BsonElement("s")]
			public DateTime LastSeenUtc { get; set; }

			private TestPhaseDocument()
			{
				Name = String.Empty;
				Key = String.Empty;
			}

			public TestPhaseDocument(TestId testId, string key, string name)
			{
				Id = TestPhaseId.GenerateNewId();
				TestNameRef = testId;
				Name = name;
				Key = key;
				LastSeenUtc = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Stores the tests running in a stream
		/// </summary>
		class TestSessionStreamDocument : ITestSessionStream
		{
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("t")]
			public List<TestId> Tests { get; set; } = new List<TestId>();
			IReadOnlyList<TestId> ITestSessionStream.Tests => Tests;

			[BsonRequired, BsonElement("m")]
			public List<TestMetaId> Metadata { get; set; } = new List<TestMetaId>();
			IReadOnlyList<TestMetaId> ITestSessionStream.Metadata => Metadata;

			[BsonRequired, BsonElement("tg")]
			public List<TestTagId> Tags { get; set; } = new List<TestTagId>();
			IReadOnlyList<TestTagId> ITestSessionStream.Tags => Tags;

			private TestSessionStreamDocument()
			{

			}

			public TestSessionStreamDocument(StreamId streamId)
			{
				Id = ObjectId.GenerateNewId();
				StreamId = streamId;
			}
		}

		/// <summary>
		/// Store the error fingerprint entries
		/// </summary>
		class TestErrorfingerprintEntry : IErrorFingerprintEntry
		{
			[BsonRequired, BsonElement("k")]

			public string Key { get; set; }

			[BsonRequired, BsonElement("p")]
			public IReadOnlyList<string> Phases { get; set; }

			private TestErrorfingerprintEntry()
			{
				Key = String.Empty;
				Phases = new List<string>();
			}

			public TestErrorfingerprintEntry(string key, IReadOnlyList<string> phases)
			{
				Key = key;
				Phases = phases;
			}
		}

		/// <summary>
		/// Store the test session overall result
		/// </summary>
		class TestSessionDocument : ITestSession
		{
			[BsonRequired, BsonId]
			public TestSessionId Id { get; set; }

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("m")]
			public TestMetaId Metadata { get; set; }

			[BsonIgnoreIfNull, BsonElement("r")]
			public TestRecipeId? Recipe { get; set; }

			[BsonIgnoreIfNull, BsonElement("tg")]
			public IReadOnlyList<TestTagId>? Tags { get; set; }

			[BsonIgnoreIfNull, BsonElement("ef")]
			public List<TestErrorfingerprintEntry>? ErrorFingerprints { get; set; }
			IReadOnlyList<IErrorFingerprintEntry>? ITestSession.ErrorFingerprints => ErrorFingerprints;

			[BsonRequired, BsonElement("did")]
			public ObjectId TestDataId { get; set; }

			public CommitIdWithOrder BuildCommitId
			{
				get => (BuildCommitName != null) ? new CommitIdWithOrder(BuildCommitName, BuildCommitOrder) : CommitIdWithOrder.FromPerforceChange(BuildCommitOrder);
				set => (BuildCommitName, BuildCommitOrder) = (value.Name, value.Order);
			}

			[BsonIgnoreIfNull, BsonElement("bcn")]
			public string? BuildCommitName { get; set; }

			[BsonRequired, BsonElement("bcl")]
			public int BuildCommitOrder { get; set; }

			[BsonRequired, BsonElement("d")]
			public TimeSpan Duration { get; set; }

			[BsonRequired, BsonElement("s")]
			public DateTime StartDateTime { get; set; }

			[BsonRequired, BsonElement("tid")]
			public TestId NameRef { get; set; }

			[BsonRequired, BsonElement("o")]
			public TestOutcome Outcome { get; set; }

			[BsonRequired, BsonElement("ptc")]
			public int PhasesTotalCount { get; set; }

			[BsonRequired, BsonElement("psc")]
			public int PhasesSucceededCount { get; set; }

			[BsonRequired, BsonElement("puc")]
			public int PhasesUndefinedCount { get; set; }

			[BsonRequired, BsonElement("pfc")]
			public int PhasesFailedCount { get; set; }

			[BsonRequired, BsonElement("jid")]
			public JobId JobId { get; set; }

			[BsonRequired, BsonElement("sjid")]
			public JobStepId StepId { get; set; }

			private TestSessionDocument()
			{

			}

			public TestSessionDocument(StreamId streamId, ObjectId testDataId, TestId testId, TestMetaId metaId)
			{
				Id = TestSessionId.GenerateNewId();
				StreamId = streamId;
				TestDataId = testDataId;
				NameRef = testId;
				Metadata = metaId;
			}
		}

		/// <summary>
		/// Store the test phase result
		/// </summary>
		class TestPhaseSessionDocument : ITestPhaseSession
		{
			[BsonRequired, BsonId]
			public TestPhaseSessionId Id { get; set; }

			[BsonRequired, BsonElement("tid")]
			public TestPhaseId PhaseRef { get; set; }

			[BsonRequired, BsonElement("tsid")]
			public TestSessionId SessionId { get; set; }

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("m")]
			public TestMetaId Metadata { get; set; }

			public CommitIdWithOrder BuildCommitId
			{
				get => (BuildCommitName != null) ? new CommitIdWithOrder(BuildCommitName, BuildCommitOrder) : CommitIdWithOrder.FromPerforceChange(BuildCommitOrder);
				set => (BuildCommitName, BuildCommitOrder) = (value.Name, value.Order);
			}

			[BsonIgnoreIfNull, BsonElement("bcn")]
			public string? BuildCommitName { get; set; }

			[BsonRequired, BsonElement("bcl")]
			public int BuildCommitOrder { get; set; }

			[BsonRequired, BsonElement("d")]
			public TimeSpan Duration { get; set; }

			[BsonRequired, BsonElement("s")]
			public DateTime StartDateTime { get; set; }

			[BsonRequired, BsonElement("o")]
			public TestPhaseOutcome Outcome { get; set; }

			[BsonIgnoreIfNull, BsonElement("esp")]
			public string? EventStreamPath { get; set; }

			[BsonIgnoreIfNull, BsonElement("hw")]
			public bool? HasWarning { get; set; }

			[BsonIgnoreIfNull, BsonElement("ef")]
			public string? ErrorFingerprint { get; set; }

			[BsonIgnoreIfNull, BsonElement("tg")]
			public IReadOnlyList<TestTagId>? Tags { get; set; }

			[BsonRequired, BsonElement("jid")]
			public JobId JobId { get; set; }

			[BsonRequired, BsonElement("sjid")]
			public JobStepId StepId { get; set; }

			private TestPhaseSessionDocument()
			{

			}

			public TestPhaseSessionDocument(StreamId streamId, TestSessionId sessionId, TestPhaseId phaseId, TestMetaId metaId)
			{
				Id = TestPhaseSessionId.GenerateNewId();
				StreamId = streamId;
				SessionId = sessionId;
				PhaseRef = phaseId;
				Metadata = metaId;
			}
		}

		/// <summary>
		/// Store test health report info
		/// </summary>
		class TestHealthReportDocument : ITestHealthReport
		{
			[BsonRequired, BsonId]
			public ObjectId Id { get; set; }

			[BsonRequired, BsonElement("tid")]
			public TestId TestId { get; set; }

			[BsonRequired, BsonElement("tn")]
			public string TestName { get; set; } = String.Empty;

			[BsonRequired, BsonElement("sid")]
			public StreamId StreamId { get; set; }

			[BsonRequired, BsonElement("lud")]
			public DateTime LastUpdateDateUtc { get; set; }

			[BsonRequired, BsonElement("h")]
			public bool IsHealthy { get; set; }

			[BsonRequired, BsonElement("s")]
			public string State { get; set; } = String.Empty;

			[BsonIgnoreIfNull, BsonElement("ps")]
			public string? PreviousState { get; set; }

			[BsonRequired, BsonElement("sr")]
			public int SuccessRate { get; set; }

			[BsonRequired, BsonElement("fr")]
			public int FailureRate { get; set; }

			[BsonRequired, BsonElement("cr")]
			public int CatastrophicFailureRate { get; set; }

			[BsonRequired, BsonElement("rer")]
			public int RedundantErrorRate { get; set; }

			[BsonIgnoreIfNull, BsonElement("nld")]
			public DateTime? NotificationLastDateUtc { get; set; }

			private TestHealthReportDocument() { }

			public TestHealthReportDocument(TestId testId, string testName, StreamId streamId)
			{
				Id = ObjectId.GenerateNewId();
				TestId = testId;
				TestName = testName;
				StreamId = streamId;
				LastUpdateDateUtc = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Store test notification settings for a stream
		/// </summary>
		class TestNotificationSettingsEntry : ITestNotificationSettings
		{
			[BsonRequired, BsonElement("s")]
			public StreamId Stream { get; set; }

			[BsonRequired, BsonElement("w")]
			public WorkflowId Workflow { get; set; }
		}

		/// <summary>
		/// Store a test audit entry
		/// </summary>
		class TestAuditDocument : ITestAudit
		{
			[BsonRequired, BsonId]
			public TestAuditId Id { get; set; }

			[BsonRequired, BsonElement("tid")]
			public TestId TestId { get; set; }

			[BsonRequired, BsonElement("ui")]
			public DateTime UserInputDate { get; set; }

			[BsonRequired, BsonElement("a")]
			public bool Archived { get; set; }

			[BsonRequired, BsonElement("uid")]
			public UserId UserId { get; set; }

			[BsonRequired, BsonElement("e")]
			public bool EnableSchedule { get; set; }

			[BsonIgnoreIfNull, BsonElement("la")]
			public DateTime? LastAuditDate { get; set; }

			[BsonIgnoreIfNull, BsonElement("ua")]
			public bool? IsUnderAudit { get; set; }

			[BsonIgnoreIfNull, BsonElement("o")]
			public UserId? Owner { get; set; }

			[BsonIgnoreIfNull, BsonElement("c")]
			public IReadOnlyList<UserId>? Customers { get; set; }

			[BsonIgnoreIfNull, BsonElement("ns")]
			public IReadOnlyList<TestNotificationSettingsEntry>? Notifications { get; set; }
			IReadOnlyList<ITestNotificationSettings>? ITestAudit.Notifications => Notifications;

			[BsonIgnoreIfNull, BsonElement("t")]
			public string? Team { get; set; }

			[BsonIgnoreIfNull, BsonElement("h")]
			public string? Harness { get; set; }

			[BsonIgnoreIfNull, BsonElement("i")]
			public string? Intent { get; set; }

			[BsonIgnoreIfNull, BsonElement("p")]
			public string? Profile { get; set; }

			[BsonIgnoreIfNull, BsonElement("n")]
			public string? Notes { get; set; }

			private TestAuditDocument() { }

			public TestAuditDocument(UserId userId, TestId testId)
			{
				Id = TestAuditId.GenerateNewId();
				TestId = testId;
				UserId = userId;
				UserInputDate = DateTime.UtcNow;
				Archived = false;
				EnableSchedule = true;
			}

			public void UpdateOptions(TestAuditOptions options)
			{
				EnableSchedule = options.EnableSchedule;
				LastAuditDate = options.LastAuditDate;
				IsUnderAudit = options.IsUnderAudit;
				Owner = options.Owner;
				Customers = options.Customers;
				Notifications = options.Notifications?.Select(
					n => new TestNotificationSettingsEntry()
					{
						Stream = n.Stream,
						Workflow = n.Workflow
					}
				).ToList();
				Team = options.Team;
				Harness = options.Harness;
				Intent = options.Intent;
				Profile = options.Profile;
				Notes = options.Notes;
			}
		}

		/// <summary>
		/// Information about a test data document
		/// </summary>
		class TestDataDocument : ITestData
		{
			public ObjectId Id { get; set; }
			public StreamId StreamId { get; set; }
			public TemplateId TemplateRefId { get; set; }
			public JobId JobId { get; set; }
			public JobStepId StepId { get; set; }

			[BsonIgnore]
			public CommitIdWithOrder CommitId
			{
				get => (CommitName != null) ? new CommitIdWithOrder(CommitName, CommitOrder) : CommitIdWithOrder.FromPerforceChange(CommitOrder);
				set => (CommitName, CommitOrder) = (value.Name, value.Order);
			}

			public string? CommitName { get; set; }

			[BsonElement("Change")]
			public int CommitOrder { get; set; }

			[BsonIgnore]
			public CommitId? PreflightCommitId
			{
				get => PreflightCommitName != null ? new CommitId(PreflightCommitName) : null;
				set => PreflightCommitName = value?.Name;
			}

			public string? PreflightCommitName { get; set; }

			public string Key { get; set; }
			public BsonDocument Data { get; set; }

			private TestDataDocument()
			{
				Key = String.Empty;
				Data = new BsonDocument();
			}

			public TestDataDocument(IJob job, IJobStep jobStep, string key, BsonDocument value)
			{
				Id = ObjectId.GenerateNewId();
				StreamId = job.StreamId;
				TemplateRefId = job.TemplateId;
				JobId = job.Id;
				StepId = jobStep.Id;
				CommitId = job.CommitId;
				PreflightCommitId = job.PreflightCommitId;
				Key = key;
				Data = value;
			}
		}

		readonly IMongoCollection<TestDataDocument> _testDataDocuments;
		readonly IMongoCollection<TestMetadataDocument> _testMeta;
		readonly IMongoCollection<TestRecipeDocument> _testRecipes;
		readonly IMongoCollection<TestTagDocument> _testTags;
		readonly IMongoCollection<TestNameDocument> _tests;
		readonly IMongoCollection<TestPhaseDocument> _testPhases;
		readonly IMongoCollection<TestSessionDocument> _testSessions;
		readonly IMongoCollection<TestPhaseSessionDocument> _testPhaseSessions;
		readonly IMongoCollection<TestSessionStreamDocument> _testStreams;
		readonly IMongoCollection<TestAuditDocument> _testAudits;
		readonly IMongoCollection<TestHealthReportDocument> _testHealthReports;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly IAuditLog<TestId> _auditLog;

		/// <summary>
		/// Constructor
		/// </summary>
		public TestDataCollectionV2(IMongoService mongoService, Tracer tracer, ILogger<TestDataCollection> logger, IAuditLog<TestId> auditLog)
		{
			_tracer = tracer;
			_logger = logger;
			_auditLog = auditLog;

			List<MongoIndex<TestDataDocument>> indexes = new List<MongoIndex<TestDataDocument>>();
			indexes.Add(keys => keys.Descending(x => x.JobId).Descending(x => x.StepId));
			_testDataDocuments = mongoService.GetCollection<TestDataDocument>("TestDataV2", indexes);

			List<MongoIndex<TestMetadataDocument>> metaIndexes = new List<MongoIndex<TestMetadataDocument>>();
			metaIndexes.Add(keys => keys.Ascending("m.k").Ascending("m.v"));
			_testMeta = mongoService.GetCollection<TestMetadataDocument>("TestDataV2.Metadata", metaIndexes);

			List<MongoIndex<TestRecipeDocument>> recipeIndexes = new List<MongoIndex<TestRecipeDocument>>();
			recipeIndexes.Add(keys => keys.Ascending(x => x.NameRef).Ascending(x => x.StreamId).Ascending(x => x.JobTemplateId).Ascending(x => x.JobStep), unique: true);
			_testRecipes = mongoService.GetCollection<TestRecipeDocument>("TestDataV2.TestRecipes", recipeIndexes);

			List<MongoIndex<TestTagDocument>> tagIndexes = new List<MongoIndex<TestTagDocument>>();
			tagIndexes.Add(keys => keys.Ascending(x => x.Name), unique: true);
			_testTags = mongoService.GetCollection<TestTagDocument>("TestDataV2.TestTags", tagIndexes);

			List<MongoIndex<TestNameDocument>> testIndexes = new List<MongoIndex<TestNameDocument>>();
			testIndexes.Add(keys => keys.Ascending(x => x.Key), unique: true);
			_tests = mongoService.GetCollection<TestNameDocument>("TestDataV2.TestNames", testIndexes);

			List<MongoIndex<TestPhaseDocument>> testPhaseIndexes = new List<MongoIndex<TestPhaseDocument>>();
			testPhaseIndexes.Add(keys => keys.Ascending(x => x.TestNameRef).Ascending(x => x.Key), unique: true);
			_testPhases = mongoService.GetCollection<TestPhaseDocument>("TestDataV2.TestPhases", testPhaseIndexes);

			List<MongoIndex<TestSessionDocument>> testSessionIndexes = new List<MongoIndex<TestSessionDocument>>();
			testSessionIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.NameRef).Descending(x => x.BuildCommitOrder));
			_testSessions = mongoService.GetCollection<TestSessionDocument>("TestDataV2.TestSessions", testSessionIndexes);

			List<MongoIndex<TestPhaseSessionDocument>> testPhaseSessionIndexes = new List<MongoIndex<TestPhaseSessionDocument>>();
			testPhaseSessionIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.PhaseRef).Descending(x => x.BuildCommitOrder));
			_testPhaseSessions = mongoService.GetCollection<TestPhaseSessionDocument>("TestDataV2.TestPhaseSessions", testPhaseSessionIndexes);

			List<MongoIndex<TestSessionStreamDocument>> streamIndexes = new List<MongoIndex<TestSessionStreamDocument>>();
			streamIndexes.Add(keys => keys.Ascending(x => x.StreamId), unique: true);
			_testStreams = mongoService.GetCollection<TestSessionStreamDocument>("TestDataV2.TestStreams", streamIndexes);

			List<MongoIndex<TestAuditDocument>> auditIndexes = new List<MongoIndex<TestAuditDocument>>();
			auditIndexes.Add(keys => keys.Ascending(x => x.TestId).Descending(x => x.Archived));
			_testAudits = mongoService.GetCollection<TestAuditDocument>("TestDataV2.TestAudits", auditIndexes);

			List<MongoIndex<TestHealthReportDocument>> healthReporIndexes = new List<MongoIndex<TestHealthReportDocument>>();
			healthReporIndexes.Add(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.TestId), unique: true);
			_testHealthReports = mongoService.GetCollection<TestHealthReportDocument>("TestDataV2.TestHealthReports", healthReporIndexes);			
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestSessionStream>> FindTestSessionStreamsAsync(StreamId[] streamIds, CancellationToken cancellationToken = default)
		{
			return await _testStreams.Find(Builders<TestSessionStreamDocument>.Filter.In(x => x.StreamId, streamIds)).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestSession>> FindTestSessionsAsync(StreamId[] streamIds, TestId[]? testIds = null, TestMetaId[]? metaIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestSessionDocument> filter = FilterDefinition<TestSessionDocument>.Empty;
			FilterDefinitionBuilder<TestSessionDocument> filterBuilder = Builders<TestSessionDocument>.Filter;
			SortDefinition<TestSessionDocument> sortDefinition = Builders<TestSessionDocument>.Sort.Descending(x => x.BuildCommitOrder);

			filter &= filterBuilder.In(x => x.StreamId, streamIds);

			if (minCreateTime != null)
			{
				TestSessionId minTime = TestSessionId.GenerateNewId(minCreateTime.Value);
				filter &= filterBuilder.Gte(x => x.Id!, minTime);

			}
			if (maxCreateTime != null)
			{
				TestSessionId maxTime = TestSessionId.GenerateNewId(maxCreateTime.Value);
				filter &= filterBuilder.Lte(x => x.Id!, maxTime);
			}

			if (metaIds != null && metaIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Metadata, metaIds);
			}

			if (testIds != null && testIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.NameRef!, testIds);
			}

			if (minCommitId != null)
			{
				int minCommitOrder = minCommitId.GetPerforceChange();
				filter &= filterBuilder.Gte(x => x.BuildCommitOrder, minCommitOrder);
			}
			if (maxCommitId != null)
			{
				int maxCommitOrder = maxCommitId.GetPerforceChange();
				filter &= filterBuilder.Lte(x => x.BuildCommitOrder, maxCommitOrder);
			}

			List<TestSessionDocument> results;

			using (TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(TestDataCollectionV2)}.{nameof(FindTestSessionsAsync)}"))
			{
				results = await _testSessions.Find(filter).Sort(sortDefinition).ToListAsync(cancellationToken);
			}

			return results.ConvertAll<ITestSession>(x => x);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestNameRef>> FindTestNameRefsAsync(TestId[]? testIds = null, string[]? keys = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestNameDocument> filter = FilterDefinition<TestNameDocument>.Empty;
			FilterDefinitionBuilder<TestNameDocument> filterBuilder = Builders<TestNameDocument>.Filter;

			if (testIds != null && testIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Id, testIds);
			}

			if (keys != null && keys.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Key, keys);
			}

			return await _tests.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestMetaRef>> FindTestMetaAsync(TestMetaId[]? metaIds = null, IReadOnlyDictionary<string, string>? keyValues = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestMetadataDocument> filter = FilterDefinition<TestMetadataDocument>.Empty;
			FilterDefinitionBuilder<TestMetadataDocument> filterDocBuilder = Builders<TestMetadataDocument>.Filter;
			FilterDefinitionBuilder<TestMetaEntry> metaFilterBuilder = Builders<TestMetaEntry>.Filter;

			if (keyValues != null && keyValues.Count > 0)
			{
				foreach (KeyValuePair<string, string> item in keyValues)
				{
					filter &= filterDocBuilder.ElemMatch(x => x.Entries, metaFilterBuilder.Eq(i => i.Key, item.Key) & metaFilterBuilder.Eq(i => i.Value, item.Value));
				}
			}

			if (metaIds != null && metaIds.Length > 0)
			{
				filter &= filterDocBuilder.In(x => x.Id, metaIds);
			}

			return await _testMeta.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestPhaseRef>> FindTestPhasesAsync(TestId[] testIds, string[]? keys = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestPhaseDocument> filter = FilterDefinition<TestPhaseDocument>.Empty;
			FilterDefinitionBuilder<TestPhaseDocument> filterBuilder = Builders<TestPhaseDocument>.Filter;

			filter &= filterBuilder.In(x => x.TestNameRef, testIds);

			if (keys != null && keys.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Key, keys);
			}

			return await _testPhases.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestTagRef>> FindTestTagsAsync(TestTagId[]? tagIds = null, string[]? names = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestTagDocument> filter = FilterDefinition<TestTagDocument>.Empty;
			FilterDefinitionBuilder<TestTagDocument> filterBuilder = Builders<TestTagDocument>.Filter;

			if (tagIds != null && tagIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Id, tagIds);
			}

			if (names != null && names.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Name, names);
			}

			return await _testTags.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestRecipeRef>> FindTestRecipesAsync(TestRecipeId[]? recipeIds = null, TestId[]? testIds = null, StreamId[]? streamIds = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestRecipeDocument> filter = FilterDefinition<TestRecipeDocument>.Empty;
			FilterDefinitionBuilder<TestRecipeDocument> filterBuilder = Builders<TestRecipeDocument>.Filter;

			if (recipeIds != null && recipeIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.Id, recipeIds);
			}

			if (testIds != null && testIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.NameRef, testIds);
			}

			if (streamIds != null && streamIds.Length > 0)
			{
				filter &= filterBuilder.In(x => x.StreamId, streamIds);
			}

			return await _testRecipes.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestPhaseSession>> FindTestPhaseSessionsAsync(StreamId[] streamIds, TestPhaseId[] phaseIds, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestPhaseSessionDocument> filter = FilterDefinition<TestPhaseSessionDocument>.Empty;
			FilterDefinitionBuilder<TestPhaseSessionDocument> filterBuilder = Builders<TestPhaseSessionDocument>.Filter;
			SortDefinition<TestPhaseSessionDocument> sortDefinition = Builders<TestPhaseSessionDocument>.Sort.Descending(x => x.BuildCommitOrder);

			filter &= filterBuilder.In(x => x.StreamId, streamIds);
			filter &= filterBuilder.In(x => x.PhaseRef, phaseIds);

			if (minCreateTime != null)
			{
				TestPhaseSessionId minTime = TestPhaseSessionId.GenerateNewId(minCreateTime.Value);
				filter &= filterBuilder.Gte(x => x.Id!, minTime);

			}
			if (maxCreateTime != null)
			{
				TestPhaseSessionId maxTime = TestPhaseSessionId.GenerateNewId(maxCreateTime.Value);
				filter &= filterBuilder.Lte(x => x.Id!, maxTime);
			}

			if (minCommitId != null)
			{
				int minCommitOrder = minCommitId.GetPerforceChange();
				filter &= filterBuilder.Gte(x => x.BuildCommitOrder, minCommitOrder);
			}
			if (maxCommitId != null)
			{
				int maxCommitOrder = maxCommitId.GetPerforceChange();
				filter &= filterBuilder.Lte(x => x.BuildCommitOrder, maxCommitOrder);
			}

			List<TestPhaseSessionDocument> results;

			using (TelemetrySpan _ = _tracer.StartActiveSpan($"{nameof(TestDataCollectionV2)}.{nameof(FindTestPhaseSessionsAsync)}"))
			{
				results = await _testPhaseSessions.Find(filter).Sort(sortDefinition).ToListAsync(cancellationToken);
			}

			return results.ConvertAll<ITestPhaseSession>(x => x);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestData>> AddAsync(IJob job, IJobStep step, (string key, BsonDocument value)[] data, CancellationToken cancellationToken = default)
		{
			// detailed test data
			List <TestDataDocument> documents = new List<TestDataDocument>();
			for (int i = 0; i < data.Length; i++)
			{
				(string key, BsonDocument document) = data[i];

					int version;
				if (document.TryGetInt32("version", out version) || document.TryGetInt32("Version", out version))
				{
					if (version > 1)
					{
						documents.Add(new TestDataDocument(job, step, key, document));
					}
				}
			}

			if (documents.Count == 0)
			{
				return documents;
			}

			await _testDataDocuments.InsertManyAsync(documents, null, cancellationToken);

			try
			{
				await AddTestReportDataAsync(job, step, documents, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception while adding test data  report, jobId: {JobId} stepId: {StepId}", job.Id, step.Id);
			}

			return documents;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestData>> GetJobStepTestDataAsync(JobId jobId, JobStepId? stepId = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestDataDocument> filter = FilterDefinition<TestDataDocument>.Empty;
			FilterDefinitionBuilder<TestDataDocument> filterBuilder = Builders<TestDataDocument>.Filter;

			filter &= filterBuilder.Eq(x => x.JobId, jobId);

			if (stepId != null)
			{
				filter &= filterBuilder.Eq(x => x.StepId, stepId);
			}

			return await _testDataDocuments.Find(filter).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<ITestData?> GetAsync(ObjectId id, CancellationToken cancellationToken)
		{
			return await _testDataDocuments.Find<TestDataDocument>(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
		}

		// --- (Minimal) Parsing of Automation framework tests, geared towards generating indexes for aggregate queries
		class AutomatedTestSessionData
		{
			public class TestSessionSummary
			{
				[BsonRequired, BsonElement("testName")]
				public string TestName { get; set; } = String.Empty;
				[BsonIgnoreIfNull, BsonElement("dateTime")]
				public string? DateTime { get; set; } = String.Empty;
				[BsonRequired, BsonElement("timeElapseSec")]
				public double TimeElapseSec { get; set; }
				[BsonIgnoreIfNull, BsonElement("commandline")]
				public string? CommandLine { get; set; }
				[BsonRequired, BsonElement("phasesTotalCount")]
				public int PhasesTotalCount { get; set; }
				[BsonRequired, BsonElement("phasesSucceededCount")]
				public int PhasesSucceededCount { get; set; }
				[BsonRequired, BsonElement("phasesUndefinedCount")]
				public int PhasesUndefinedCount { get; set; }
				[BsonRequired, BsonElement("phasesFailedCount")]
				public int PhasesFailedCount { get; set; }
			}

			public class TestPhase
			{
				[BsonRequired, BsonElement("name")]
				public string Name { get; set; } = String.Empty;
				[BsonRequired, BsonElement("key")]
				public string Key { get; set; } = String.Empty;
				[BsonIgnoreIfNull, BsonElement("dateTime")]
				public string? DateTime { get; set; } = String.Empty;
				[BsonRequired, BsonElement("timeElapseSec")]
				public double TimeElapseSec { get; set; }
				[BsonRequired, BsonElement("outcome")]
				public TestPhaseOutcome Outcome { get; set; }
				[BsonIgnoreIfNull, BsonElement("errorFingerprint")]
				public string? ErrorFingerprint { get; set; }
				[BsonIgnoreIfNull, BsonElement("hasWarning")]
				public bool? HasWarning { get; set; }
				[BsonIgnoreIfNull, BsonElement("eventStreamPath")]
				public string? EventStreamPath { get; set; }
				[BsonIgnoreIfNull, BsonElement("tags")]
				public List<string>? Tags { get; set; }
			}

			public class TestDescription
			{
				[BsonIgnoreIfNull, BsonElement("team")]
				public string? Team { get; set; }
				[BsonIgnoreIfNull, BsonElement("intent")]
				public string? Intent { get; set; }
				[BsonIgnoreIfNull, BsonElement("profile")]
				public string? Profile { get; set; }
				[BsonIgnoreIfNull, BsonElement("harness")]
				public string? Harness { get; set; }
			}

			public class TestRecipe
			{
				[BsonRequired, BsonElement("nodeClass")]
				public string NodeClass { get; set; } = String.Empty;
				[BsonRequired, BsonElement("version")]
				public string Version { get; set; } = String.Empty;
				[BsonIgnoreIfNull, BsonElement("timeoutMinutes")]
				public float? TimeoutMinutes { get; set; } = 0;
				[BsonIgnoreIfNull, BsonElement("params")]
				public Dictionary<string, object>? Params { get; set; } = new Dictionary<string, object>();
			}

			[BsonRequired, BsonElement("summary")]
			public TestSessionSummary? Summary { get; set; }
			[BsonRequired, BsonElement("phases")]
			public List<TestPhase> Phases { get; set; } = new List<TestPhase>();
			[BsonRequired, BsonElement("metadata")]
			public Dictionary<string, string> Metadata { get; set; } = new Dictionary<string, string>();
			[BsonIgnoreIfNull, BsonElement("tags")]
			public List<string>? Tags { get; set; }
			[BsonIgnoreIfNull, BsonElement("description")]
			public TestDescription? Description { get; set; }
			[BsonIgnoreIfNull, BsonElement("recipe")]
			public TestRecipe? Recipe { get; set; }
		}

		private async Task AddTestReportDataAsync(IJob job, IJobStep step, IReadOnlyList<TestDataDocument> data, CancellationToken cancellationToken = default)
		{
			// do not add preflight to temporal data
			if (job.PreflightCommitId != null)
			{
				return;
			}

			foreach (TestDataDocument document in data)
			{
				AutomatedTestSessionData testSession = BsonSerializer.Deserialize<AutomatedTestSessionData>(document.Data);

				if (testSession.Summary == null)
				{
					throw new Exception($"Missing summary field in test session {document.Key}.");
				}

				if (testSession.Phases == null)
				{
					throw new Exception($"Missing phases field in test session {testSession.Summary.TestName}.");
				}

				if (testSession.Metadata == null)
				{
					throw new Exception($"Missing metadata field in test session {testSession.Summary.TestName}.");
				}

				IReadOnlyList<ITestTagRef>? tags = await AddTestTagsAsync(testSession.Tags, cancellationToken);
				ITestMetaRef metadata = await AddTestMetadataAsync(testSession.Metadata, cancellationToken);
				ITestNameRef testRef = await AddTestAsync(document.Key, testSession.Summary.TestName, metadata.Id, tags?.Select(t => t.Id).ToList(), testSession.Description, document.StreamId, cancellationToken);

				// test recipe
				ITestRecipeRef? recipe = null;
				if (testSession.Recipe != null)
				{
					recipe = await AddTestRecipeAsync(testRef.Id, testSession.Recipe, job, step, document.StreamId, cancellationToken);
				}

				// test session
				TestSessionDocument sessionDoc = new TestSessionDocument(document.StreamId, document.Id, testRef.Id, metadata.Id);
				// job and commit
				sessionDoc.JobId = job.Id;
				sessionDoc.StepId = step.Id;
				sessionDoc.BuildCommitId = document.CommitId;
				// phase outcomes and summary
				sessionDoc.Outcome = testSession.Summary.PhasesUndefinedCount > 0 ? TestOutcome.Unspecified :
					testSession.Summary.PhasesFailedCount > 0 ? TestOutcome.Failure :
						testSession.Summary.PhasesTotalCount > 0 && testSession.Summary.PhasesSucceededCount > 0 ? TestOutcome.Success : TestOutcome.Skipped;
				sessionDoc.PhasesFailedCount = testSession.Summary.PhasesFailedCount;
				sessionDoc.PhasesSucceededCount = testSession.Summary.PhasesSucceededCount;
				sessionDoc.PhasesTotalCount = testSession.Summary.PhasesTotalCount;
				sessionDoc.PhasesUndefinedCount = testSession.Summary.PhasesUndefinedCount;
				// timing
				DateTime startDateTime;
				if (DateTime.TryParse(testSession.Summary.DateTime, out startDateTime))
				{
					sessionDoc.StartDateTime = startDateTime;
				}
				else
				{
					sessionDoc.StartDateTime = DateTime.UtcNow;
				}
				sessionDoc.Duration = TimeSpan.FromSeconds(testSession.Summary.TimeElapseSec);
				// tags
				sessionDoc.Tags = tags?.Select(t => t.Id).ToList();
				// recipe
				sessionDoc.Recipe = recipe?.Id;
				// error fingerprints
				Dictionary<string, List<string>> errorFingerprints = new();

				// phase session
				List<TestPhaseSessionDocument> phaseSessions = new();
				foreach (AutomatedTestSessionData.TestPhase phase in testSession.Phases)
				{
					ITestPhaseRef phaseRef = await AddTestPhaseAsync(testRef.Id, phase.Key, phase.Name, cancellationToken);

					TestPhaseSessionDocument phaseDoc = new TestPhaseSessionDocument(document.StreamId, sessionDoc.Id, phaseRef.Id, metadata.Id);
					// job and commit
					phaseDoc.JobId = job.Id;
					phaseDoc.StepId = step.Id;
					phaseDoc.BuildCommitId = document.CommitId;
					// outcome and event summary
					phaseDoc.Outcome = phase.Outcome;
					phaseDoc.ErrorFingerprint = phase.ErrorFingerprint;
					phaseDoc.EventStreamPath = phase.EventStreamPath;
					phaseDoc.HasWarning = phase.HasWarning;
					// timing
					if (DateTime.TryParse(phase.DateTime, out startDateTime))
					{
						phaseDoc.StartDateTime = startDateTime;
					}
					else
					{
						phaseDoc.StartDateTime = DateTime.UtcNow;
					}
					phaseDoc.Duration = TimeSpan.FromSeconds(phase.TimeElapseSec);
					// tags
					phaseDoc.Tags = tags?.Where(tag => phase.Tags?.Contains(tag.Name) ?? false).Select(tag => tag.Id).ToList();
					// collect any error fingerprint
					if (phase.ErrorFingerprint != null && errorFingerprints.Count < 50)
					{
						if (!errorFingerprints.ContainsKey(phase.ErrorFingerprint))
						{
							errorFingerprints.Add(phase.ErrorFingerprint, new List<string>() { phase.Key });
						}
						else
						{
							List<string>? phases = errorFingerprints.GetValueOrDefault(phase.ErrorFingerprint);
							if (phases != null && phases.Count < 50)
							{
								phases.Add(phase.Key);
							}
						}
					}

					phaseSessions.Add(phaseDoc);
				}
				if (phaseSessions.Count > 0)
				{
					await _testPhaseSessions.InsertManyAsync(phaseSessions, null, cancellationToken);
				}

				if (errorFingerprints.Count > 0)
				{
					sessionDoc.ErrorFingerprints = errorFingerprints.Select(e => new TestErrorfingerprintEntry(e.Key, e.Value)).ToList();
				}

				await _testSessions.InsertOneAsync(sessionDoc, null, cancellationToken);
			}
		}

		private async Task<ITestMetaRef> AddTestMetadataAsync(IReadOnlyDictionary<string, string> metaData, CancellationToken cancellationToken)
		{
			FilterDefinition<TestMetadataDocument> filter = FilterDefinition<TestMetadataDocument>.Empty;
			FilterDefinitionBuilder<TestMetadataDocument> filterDocBuilder = Builders<TestMetadataDocument>.Filter;
			FilterDefinitionBuilder<TestMetaEntry> metaFilterBuilder = Builders<TestMetaEntry>.Filter;

			foreach (KeyValuePair<string, string> item in metaData)
			{
				filter &= filterDocBuilder.ElemMatch(x => x.Entries, metaFilterBuilder.Eq(i => i.Key, item.Key) & metaFilterBuilder.Eq(i => i.Value, item.Value));
			}
			// Match the exact number of entries
			filter &= filterDocBuilder.Size(x => x.Entries, metaData.Count);

			TestMetadataDocument? meta = await _testMeta.Find(filter).FirstOrDefaultAsync(cancellationToken);

			if (meta == null)
			{
				meta = new TestMetadataDocument(metaData);
				await _testMeta.InsertOneAsync(meta, null, cancellationToken);
			}
			// update only once per day
			else if (meta.LastSeenUtc < DateTime.UtcNow.AddDays(-1))
			{
				UpdateDefinitionBuilder<TestMetadataDocument> updateBuilder = Builders<TestMetadataDocument>.Update;
				List<UpdateDefinition<TestMetadataDocument>> updates = new List<UpdateDefinition<TestMetadataDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				await _testMeta.FindOneAndUpdateAsync(x => x.Id == meta.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			return meta;
		}

		private async Task<ITestRecipeRef?> AddTestRecipeAsync(TestId nameRef, AutomatedTestSessionData.TestRecipe recipe, IJob job, IJobStep jobStep, StreamId streamId, CancellationToken cancellationToken)
		{
			WorkflowId? workflow = jobStep.Annotations?.WorkflowId;
			TemplateId jobTemplate = job.TemplateId;
			string stepName = jobStep.Name;

			FilterDefinition<TestRecipeDocument> filter = FilterDefinition<TestRecipeDocument>.Empty;
			FilterDefinitionBuilder<TestRecipeDocument> filterBuilder = Builders<TestRecipeDocument>.Filter;

			filter &= filterBuilder.Eq(i => i.NameRef, nameRef);
			filter &= filterBuilder.Eq(i => i.StreamId, streamId);
			filter &= filterBuilder.Eq(i => i.JobTemplateId, jobTemplate);
			filter &= filterBuilder.Eq(i => i.JobStep, stepName);

			TestRecipeDocument? recipeDoc = await _testRecipes.Find(filter).FirstOrDefaultAsync(cancellationToken);

			if (recipeDoc == null)
			{
				recipeDoc = new TestRecipeDocument(nameRef, streamId, jobTemplate, stepName, recipe.NodeClass, recipe.Version, recipe.TimeoutMinutes ?? 0, workflow);
				if (recipe.Params != null)
				{
					recipeDoc.UpdateParams(recipe.Params);
				}
				await _testRecipes.InsertOneAsync(recipeDoc, null, cancellationToken);
			}
			// update only once per day
			else if (recipeDoc.LastSeenUtc < DateTime.UtcNow.AddDays(-1))
			{
				UpdateDefinitionBuilder<TestRecipeDocument> updateBuilder = Builders<TestRecipeDocument>.Update;
				List<UpdateDefinition<TestRecipeDocument>> updates = new List<UpdateDefinition<TestRecipeDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				updates.Add(updateBuilder.Set(x => x.NodeClass, recipe.NodeClass));
				updates.Add(updateBuilder.Set(x => x.Version, recipe.Version));
				updates.Add(updateBuilder.Set(x => x.TimeoutMinutes, recipe.TimeoutMinutes ?? 0));
				updates.Add(updateBuilder.Set(x => x.WorkflowId, workflow));
				if (recipe.Params != null)
				{
					recipeDoc.UpdateParams(recipe.Params);
					updates.Add(updateBuilder.Set(x => x.Params, recipeDoc.Params));
				}
				await _testRecipes.FindOneAndUpdateAsync(x => x.Id == recipeDoc.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			return recipeDoc;
		}

		private async Task<IReadOnlyList<ITestTagRef>?> AddTestTagsAsync(IReadOnlyList<string>? tags, CancellationToken cancellationToken)
		{
			if (tags == null)
			{
				return null;
			}

			FilterDefinitionBuilder<TestTagDocument> filterBuilder = Builders<TestTagDocument>.Filter;
			FilterDefinition<TestTagDocument> filter = filterBuilder.In(i => i.Name, tags);

			List<TestTagDocument> storedTags = await _testTags.Find(filter).ToListAsync(cancellationToken);
			if (storedTags.Count > 0)
			{
				// update only once per day
				List<TestTagId> needUpdateTags = storedTags.Where(i => i.LastSeenUtc < DateTime.UtcNow.AddDays(-1)).Select(i => i.Id).ToList();
				if (needUpdateTags.Count > 0)
				{
					UpdateDefinitionBuilder<TestTagDocument> updateBuilder = Builders<TestTagDocument>.Update;
					List<UpdateDefinition<TestTagDocument>> updates = new List<UpdateDefinition<TestTagDocument>>();
					updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
					FilterDefinition<TestTagDocument> filter2 = filterBuilder.In(i => i.Id, needUpdateTags);
					await _testTags.UpdateManyAsync(filter2, updateBuilder.Combine(updates), null, cancellationToken);
				}
			}
			// add the missing ones
			List<TestTagDocument> missingTags = tags.Except(storedTags.Select(i => i.Name)).Select(t => new TestTagDocument(t)).ToList();
			if (missingTags.Count > 0 )
			{
				await _testTags.InsertManyAsync(missingTags, null, cancellationToken);
				storedTags.AddRange(missingTags);
			}

			return storedTags;
		}

		private async Task<ITestNameRef> AddTestAsync(string key, string name, TestMetaId metadataId, IReadOnlyList<TestTagId>? tags, AutomatedTestSessionData.TestDescription? description, StreamId streamId, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<TestNameDocument> filterBuilder = Builders<TestNameDocument>.Filter;

			TestNameDocument? test = await _tests.Find(filterBuilder.Eq(i => i.Key, key)).FirstOrDefaultAsync(cancellationToken);

			if (test == null)
			{
				test = new TestNameDocument(key, name);
				if (description != null)
				{
					test.AddDescription(description.Team, description.Intent, description.Harness, description.Profile);
				}
				await _tests.InsertOneAsync(test, null, cancellationToken);
			}
			// update only once per day
			else if (test.LastSeenUtc < DateTime.UtcNow.AddDays(-1))
			{
				UpdateDefinitionBuilder<TestNameDocument> updateBuilder = Builders<TestNameDocument>.Update;
				List<UpdateDefinition<TestNameDocument>> updates = new List<UpdateDefinition<TestNameDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				updates.Add(updateBuilder.Set(x => x.Name, name));
				if (description != null)
				{
					IReadOnlyList<ITestAudit> queryResult =  await FindTestAuditsAsync([test.Id], null, cancellationToken);
					bool hasAudit = queryResult.Any();
					// update only if no user override was done through audit
					if (!hasAudit || queryResult[0].Team == null)
					{
						updates.Add(updateBuilder.Set(x => x.Team, description.Team));
					}
					if (!hasAudit || queryResult[0].Intent == null)
					{
						updates.Add(updateBuilder.Set(x => x.Intent, description.Intent));
					}
					if (!hasAudit || queryResult[0].Harness == null)
					{
						updates.Add(updateBuilder.Set(x => x.Harness, description.Harness));
					}
					if (!hasAudit || queryResult[0].Profile == null)
					{
						updates.Add(updateBuilder.Set(x => x.Profile, description.Profile));
					}
				}
				await _tests.FindOneAndUpdateAsync(x => x.Id == test.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			await AddTestToStreamSessionAsync(test.Id, metadataId, tags, streamId, cancellationToken);

			return test;
		}

		private async Task AddTestToStreamSessionAsync(TestId testRef, TestMetaId metadataId, IReadOnlyList<TestTagId>? tags, StreamId streamId, CancellationToken cancellationToken)
		{
			List<TestSessionStreamDocument> streams = await _testStreams.Find(Builders<TestSessionStreamDocument>.Filter.Eq(x => x.StreamId, streamId)).ToListAsync(cancellationToken);

			if (streams.Count == 0)
			{
				TestSessionStreamDocument streamDoc = new TestSessionStreamDocument(streamId);
				streamDoc.Tests.Add(testRef);
				streamDoc.Metadata.Add(metadataId);
				if (tags != null)
				{
					streamDoc.Tags.AddRange(tags);
				}
				await _testStreams.InsertOneAsync(streamDoc, null, cancellationToken);
			}
			else
			{
				UpdateDefinitionBuilder<TestSessionStreamDocument> updateBuilder = Builders<TestSessionStreamDocument>.Update;
				List<UpdateDefinition<TestSessionStreamDocument>> updates = new List<UpdateDefinition<TestSessionStreamDocument>>();

				TestSessionStreamDocument streamDoc = streams[0];

				if (!streamDoc.Tests.Contains(testRef))
				{
					streamDoc.Tests.Add(testRef);
					updates.Add(updateBuilder.Set(x => x.Tests, streamDoc.Tests));
				}
				if (!streamDoc.Metadata.Contains(metadataId))
				{
					streamDoc.Metadata.Add(metadataId);
					updates.Add(updateBuilder.Set(x => x.Metadata, streamDoc.Metadata));
				}
				if (tags != null)
				{
					List<TestTagId> missingTags = tags.Except(streamDoc.Tags).ToList();
					if (missingTags.Count > 0)
					{
						streamDoc.Tags.AddRange(missingTags);
						updates.Add(updateBuilder.Set(x => x.Tags, streamDoc.Tags));
					}
				}

				if (updates.Count > 0)
				{
					FilterDefinitionBuilder<TestSessionStreamDocument> ebuilder = Builders<TestSessionStreamDocument>.Filter;
					FilterDefinition<TestSessionStreamDocument> efilter = ebuilder.Eq(x => x.StreamId, streamDoc.StreamId);
					await _testStreams.UpdateOneAsync(efilter, updateBuilder.Combine(updates), null, cancellationToken);
				}
			}
		}

		private async Task<ITestPhaseRef> AddTestPhaseAsync(TestId testId, string key, string name, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<TestPhaseDocument> filterBuilder = Builders<TestPhaseDocument>.Filter;
			FilterDefinition<TestPhaseDocument> filter = FilterDefinition<TestPhaseDocument>.Empty;

			filter &= filterBuilder.Eq(i => i.TestNameRef, testId);
			filter &= filterBuilder.Eq(i => i.Key, key);

			TestPhaseDocument? phase = await _testPhases.Find(filter).FirstOrDefaultAsync(cancellationToken);

			if (phase == null)
			{
				phase = new TestPhaseDocument(testId, key, name);
				await _testPhases.InsertOneAsync(phase, null, cancellationToken);
			}
			// update only once per day
			else if (phase.LastSeenUtc < DateTime.UtcNow.AddDays(-1))
			{
				UpdateDefinitionBuilder<TestPhaseDocument> updateBuilder = Builders<TestPhaseDocument>.Update;
				List<UpdateDefinition<TestPhaseDocument>> updates = new List<UpdateDefinition<TestPhaseDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				updates.Add(updateBuilder.Set(x => x.Name, name));
				await _testPhases.FindOneAndUpdateAsync(x => x.Id == phase.Id, updateBuilder.Combine(updates), null, cancellationToken);
			}

			return phase;
		}

		/// <inheritdoc/>
		public async Task<ITestAudit> AddTestAuditAsync(UserId userId, TestId testId, TestAuditOptions options, CancellationToken cancellationToken = default)
		{
			// update previous one(s) and flag them as archive
			UpdateDefinitionBuilder<TestAuditDocument> updateBuilder = Builders<TestAuditDocument>.Update;
			List<UpdateDefinition<TestAuditDocument>> updates = new List<UpdateDefinition<TestAuditDocument>>();
			updates.Add(updateBuilder.Set(x => x.Archived, true));

			await _testAudits.UpdateManyAsync(x => x.TestId == testId && x.Archived == false, updateBuilder.Combine(updates), null, cancellationToken);

			// add new entry
			TestAuditDocument audit = new TestAuditDocument(userId, testId);
			audit.UpdateOptions(options);

			await _testAudits.InsertOneAsync(audit, null, cancellationToken);

			// update test with user override
			if (audit.Team != null || audit.Intent != null || audit.Harness != null || audit.Profile != null)
			{
				UpdateDefinitionBuilder<TestNameDocument> testUpdateBuilder = Builders<TestNameDocument>.Update;
				List<UpdateDefinition<TestNameDocument>> testUpdates = new List<UpdateDefinition<TestNameDocument>>();
				testUpdates.Add(testUpdateBuilder.Set(x => x.LastSeenUtc, DateTime.UtcNow));
				if (audit.Team != null)
				{
					testUpdates.Add(testUpdateBuilder.Set(x => x.Team, audit.Team));
				}
				if (audit.Intent != null)
				{
					testUpdates.Add(testUpdateBuilder.Set(x => x.Intent, audit.Intent));
				}
				if (audit.Harness != null)
				{
					testUpdates.Add(testUpdateBuilder.Set(x => x.Harness, audit.Harness));
				}
				if (audit.Profile != null)
				{
					testUpdates.Add(testUpdateBuilder.Set(x => x.Profile, audit.Profile));
				}
				await _tests.FindOneAndUpdateAsync(x => x.Id == testId, testUpdateBuilder.Combine(testUpdates), null, cancellationToken);
			}

			return audit;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestAudit>> FindTestAuditsAsync(TestId[] testIds, DateTime? minCreateTime = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestAuditDocument> filter = FilterDefinition<TestAuditDocument>.Empty;
			FilterDefinitionBuilder<TestAuditDocument> filterBuilder = Builders<TestAuditDocument>.Filter;
			SortDefinition<TestAuditDocument> sortDefinition = Builders<TestAuditDocument>.Sort.Descending(x => x.Id);

			filter &= filterBuilder.In(x => x.TestId, testIds);

			if (minCreateTime != null)
			{
				TestAuditId minTime = TestAuditId.GenerateNewId(minCreateTime.Value);
				filter &= filterBuilder.Eq(x => x.Archived, false) | filterBuilder.Gte(x => x.Id, minTime);
			}
			else
			{
				filter &= filterBuilder.Eq(x => x.Archived, false);
			}

			List<TestAuditDocument> results = await _testAudits.Find(filter).Sort(sortDefinition).ToListAsync(cancellationToken);

			return results.ConvertAll<ITestAudit>(x => x);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestAudit>> FindTestWithAuditNotificationWorkflowAsync(CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestAuditDocument> filter = FilterDefinition<TestAuditDocument>.Empty;
			FilterDefinitionBuilder<TestAuditDocument> filterBuilder = Builders<TestAuditDocument>.Filter;
			SortDefinition<TestAuditDocument> sortDefinition = Builders<TestAuditDocument>.Sort.Descending(x => x.Id);

			filter &= filterBuilder.Eq(x => x.Archived, false);
			filter &= filterBuilder.Ne(x => x.Notifications, null);

			List<TestAuditDocument> results = await _testAudits.Find(filter).Sort(sortDefinition).ToListAsync(cancellationToken);

			return results.ConvertAll<ITestAudit>(x => x);
		}

		/// <inheritdoc/>
		public IAuditLogChannel<TestId> GetTestAuditLogger(TestId testId)
		{
			return _auditLog[testId];
		}

		/// <inheritdoc/>
		public async Task<ITestHealthReport> AddOrUpdateTestHealthReportAsync(TestId testId, StreamId streamId, string testName, ComputedTestHealth testHealth, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<TestHealthReportDocument> filterBuilder = Builders<TestHealthReportDocument>.Filter;
			FilterDefinition<TestHealthReportDocument> filter = FilterDefinition<TestHealthReportDocument>.Empty;

			filter &= filterBuilder.Eq(i => i.TestId, testId);
			filter &= filterBuilder.Eq(i => i.StreamId, streamId);

			TestHealthReportDocument? healthReport = await _testHealthReports.Find(filter).FirstOrDefaultAsync(cancellationToken);

			if (healthReport == null)
			{
				healthReport = new TestHealthReportDocument(testId, testName, streamId);
				healthReport.State = testHealth.State;
				healthReport.IsHealthy = testHealth.IsHealthy;
				healthReport.SuccessRate = testHealth.SuccessRate;
				healthReport.FailureRate = testHealth.FailureRate;
				healthReport.CatastrophicFailureRate = testHealth.CatastrophicFailureRate;
				healthReport.RedundantErrorRate = testHealth.RedundantErrorRate;
				await _testHealthReports.InsertOneAsync(healthReport, null, cancellationToken);
			}
			else
			{
				UpdateDefinitionBuilder<TestHealthReportDocument> updateBuilder = Builders<TestHealthReportDocument>.Update;
				List<UpdateDefinition<TestHealthReportDocument>> updates = new List<UpdateDefinition<TestHealthReportDocument>>();
				updates.Add(updateBuilder.Set(x => x.LastUpdateDateUtc, DateTime.UtcNow));
				updates.Add(updateBuilder.Set(x => x.TestName, testName));
				updates.Add(updateBuilder.Set(x => x.IsHealthy, testHealth.IsHealthy));
				updates.Add(updateBuilder.Set(x => x.State, testHealth.State));
				updates.Add(updateBuilder.Set(x => x.PreviousState, healthReport.State));
				updates.Add(updateBuilder.Set(x => x.SuccessRate, testHealth.SuccessRate));
				updates.Add(updateBuilder.Set(x => x.FailureRate, testHealth.FailureRate));
				updates.Add(updateBuilder.Set(x => x.CatastrophicFailureRate, testHealth.CatastrophicFailureRate));
				updates.Add(updateBuilder.Set(x => x.RedundantErrorRate, testHealth.RedundantErrorRate));
				FindOneAndUpdateOptions<TestHealthReportDocument> options = new FindOneAndUpdateOptions<TestHealthReportDocument> { IsUpsert = true, ReturnDocument = ReturnDocument.After };
				healthReport = await _testHealthReports.FindOneAndUpdateAsync(filterBuilder.Eq(x => x.Id, healthReport.Id), updateBuilder.Combine(updates), options, cancellationToken);
			}

			return healthReport;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ITestHealthReport>> FindTestHealthReportsAsync(StreamId[] streamIds, TestId[]? testIds, CancellationToken cancellationToken = default)
		{
			FilterDefinition<TestHealthReportDocument> filter = FilterDefinition<TestHealthReportDocument>.Empty;
			FilterDefinitionBuilder<TestHealthReportDocument> filterBuilder = Builders<TestHealthReportDocument>.Filter;

			filter &= filterBuilder.In(i => i.StreamId, streamIds);
			if (testIds is not null)
			{
				filter &= filterBuilder.In(i => i.TestId, testIds);
			}
			
			List<TestHealthReportDocument> healthReports = await _testHealthReports.Find(filter).ToListAsync(cancellationToken);

			return healthReports;
		}

		/// <inheritdoc/>
		public async Task UpdateTestHealthReportNotificationDateAsync(ObjectId reportId, bool resetDate, CancellationToken cancellationToken = default)
		{
			UpdateDefinitionBuilder<TestHealthReportDocument> updateBuilder = Builders<TestHealthReportDocument>.Update;
			List<UpdateDefinition<TestHealthReportDocument>> updates = new List<UpdateDefinition<TestHealthReportDocument>>();
			updates.Add(updateBuilder.Set(x => x.NotificationLastDateUtc, resetDate ? DateTime.UtcNow : null));
			await _testHealthReports.FindOneAndUpdateAsync(x => x.Id == reportId, updateBuilder.Combine(updates), null, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<bool> UpdateAsync(int retainMonths, CancellationToken cancellationToken)
		{
			await ExpireTestDataAsync(retainMonths, cancellationToken);

			return true;
		}

		static async Task<long> ExpireCollectionAsync<DocType>(IMongoCollection<DocType> collection, DateTime expireTime, CancellationToken cancellationToken) where DocType : ITestExpire
		{
			FilterDefinitionBuilder<DocType> filterBuilder = Builders<DocType>.Filter;
			FilterDefinition<DocType> filter = filterBuilder.Empty;
			filter &= filterBuilder.Lte(x => x.LastSeenUtc, expireTime);
			DeleteResult result = await collection.DeleteManyAsync(filter, cancellationToken);

			return result.DeletedCount;
		}

		private async Task ExpireTestDataAsync(int retainMonths, CancellationToken cancellationToken)
		{
			if (retainMonths <= 0)
			{
				return;
			}

			DateTime expireTime = DateTime.UtcNow.AddMonths(-retainMonths);

			// expire and prune root test data
			long testsExpired = await ExpireCollectionAsync(_tests, expireTime, cancellationToken);
			long phasesExpired = await ExpireCollectionAsync(_testPhases, expireTime, cancellationToken);
			long metadataExpired = await ExpireCollectionAsync(_testMeta, expireTime, cancellationToken);
			long tagsExpired = await ExpireCollectionAsync(_testTags, expireTime, cancellationToken);
			long recipesExpired = await ExpireCollectionAsync(_testRecipes, expireTime, cancellationToken);

			// Check whether we need to prune
			if (testsExpired > 0 || phasesExpired > 0 || metadataExpired > 0 || tagsExpired > 0 || recipesExpired > 0)
			{
				_logger.LogInformation("Expired {TestsExpired} tests, {PhasesExpired} suites, {MetadataExpired} meta, {TagsExpired} tags, {RecipesExpired} recipes", testsExpired, phasesExpired, metadataExpired, tagsExpired, recipesExpired);
				await PruneDataAsync(cancellationToken);
			}

			// expire test data documents
			{
				ObjectId dataExpireTime = ObjectId.GenerateNewId(expireTime);

				FilterDefinition<TestDataDocument> filter = FilterDefinition<TestDataDocument>.Empty;
				FilterDefinitionBuilder<TestDataDocument> filterBuilder = Builders<TestDataDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, dataExpireTime);
				DeleteResult result = await _testDataDocuments.DeleteManyAsync(filter, null, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test data documents", result.DeletedCount);
				}
			}

			// expire test sessions
			{
				TestSessionId sessionExpireTime = TestSessionId.GenerateNewId(expireTime);

				FilterDefinition<TestSessionDocument> filter = FilterDefinition<TestSessionDocument>.Empty;
				FilterDefinitionBuilder<TestSessionDocument> filterBuilder = Builders<TestSessionDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, sessionExpireTime);
				DeleteResult result = await _testSessions.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test session documents", result.DeletedCount);
				}
			}

			// expire test phase sessions
			{
				TestPhaseSessionId sessionExpireTime = TestPhaseSessionId.GenerateNewId(expireTime);
				FilterDefinition<TestPhaseSessionDocument> filter = FilterDefinition<TestPhaseSessionDocument>.Empty;
				FilterDefinitionBuilder<TestPhaseSessionDocument> filterBuilder = Builders<TestPhaseSessionDocument>.Filter;
				filter &= filterBuilder.Lte(x => x.Id!, sessionExpireTime);
				DeleteResult result = await _testPhaseSessions.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test phase session documents", result.DeletedCount);
				}
			}

			// expire test audits
			{
				TestAuditId auditExpireTime = TestAuditId.GenerateNewId(expireTime);
				FilterDefinition<TestAuditDocument> filter = FilterDefinition<TestAuditDocument>.Empty;
				FilterDefinitionBuilder<TestAuditDocument> filterBuilder = Builders<TestAuditDocument>.Filter;
				filter &= filterBuilder.Eq(x => x.Archived, true);
				filter &= filterBuilder.Lte(x => x.Id!, auditExpireTime);
				DeleteResult result = await _testAudits.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Expired {NumDeleted} test audit documents", result.DeletedCount);
				}
			}
		}

		/// <inheritdoc/>
		public async Task PruneDataAsync(CancellationToken cancellationToken = default)
		{
			HashSet<TestId> testIds = new HashSet<TestId>();
			IAsyncCursor<TestId> distinctTests = await _tests.DistinctAsync(x => x.Id, Builders<TestNameDocument>.Filter.Empty, null, cancellationToken);
			foreach (TestId testId in await distinctTests.ToListAsync(cancellationToken))
			{
				testIds.Add(testId);
			}
			_logger.LogInformation("Test data pruning to {TestIdCount} test name ids", testIds.Count);

			HashSet<TestMetaId> metadataIds = new HashSet<TestMetaId>();
			IAsyncCursor<TestMetaId> distinctMetadata = await _testMeta.DistinctAsync(x => x.Id, Builders<TestMetadataDocument>.Filter.Empty, null, cancellationToken);
			foreach (TestMetaId metaId in await distinctMetadata.ToListAsync(cancellationToken))
			{
				metadataIds.Add(metaId);
			}
			_logger.LogInformation("Test data pruning to {MetadataIdCount} test metadata ids", metadataIds.Count);

			HashSet<TestTagId> tagIds = new HashSet<TestTagId>();
			IAsyncCursor<TestTagId> distinctTags = await _testTags.DistinctAsync(x => x.Id, Builders<TestTagDocument>.Filter.Empty, null, cancellationToken);
			foreach (TestTagId tagId in await distinctTags.ToListAsync(cancellationToken))
			{
				tagIds.Add(tagId);
			}
			_logger.LogInformation("Test data pruning to {TagIdCount} test tag ids", tagIds.Count);

			HashSet<TestRecipeId> recipeIds = new HashSet<TestRecipeId>();
			IAsyncCursor<TestRecipeId> distinctRecipes = await _testRecipes.DistinctAsync(x => x.Id, Builders<TestRecipeDocument>.Filter.Empty, null, cancellationToken);
			foreach (TestRecipeId recipeId in await distinctRecipes.ToListAsync(cancellationToken))
			{
				recipeIds.Add(recipeId);
			}
			_logger.LogInformation("Test data pruning to {RecipeIdCount} test recipe ids", recipeIds.Count);

			int testStreamsDeleted = 0;

			// update test session streams
			{
				List<TestSessionStreamDocument> testStreams = await _testStreams.Find(Builders<TestSessionStreamDocument>.Filter.Empty).ToListAsync(cancellationToken);
				foreach (TestSessionStreamDocument stream in testStreams)
				{
					List<TestId> tests = stream.Tests.Where(x => testIds.Contains(x)).ToList();
					if (tests.Count == 0)
					{
						await _testStreams.DeleteOneAsync(x => x.Id == stream.Id, cancellationToken);
						testStreamsDeleted++;
						continue;
					}

					List<TestMetaId> metadata = stream.Metadata.Where(x => metadataIds.Contains(x)).ToList();
					List<TestTagId> tags = stream.Tags.Where(x => tagIds.Contains(x)).ToList();
					if (tests.Count != stream.Tests.Count || metadata.Count != stream.Metadata.Count || tags.Count != stream.Tags.Count)
					{
						UpdateDefinitionBuilder<TestSessionStreamDocument> updateBuilder = Builders<TestSessionStreamDocument>.Update;
						List<UpdateDefinition<TestSessionStreamDocument>> updates = new List<UpdateDefinition<TestSessionStreamDocument>>();

						if (tests.Count != stream.Tests.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Tests, tests));
						}
						if (metadata.Count != stream.Metadata.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Metadata, metadata));
						}
						if (tags.Count != stream.Tags.Count)
						{
							updates.Add(updateBuilder.Set(x => x.Tags, tags));
						}

						await _testStreams.FindOneAndUpdateAsync(x => x.Id == stream.Id, updateBuilder.Combine(updates), null, cancellationToken);
					}
				}
			}

			if (testStreamsDeleted > 0)
			{
				_logger.LogInformation("Pruned {TestStreamDeleteCount} test session streams", testStreamsDeleted);
			}

			// pruned all audits related to tests that were NOT kept
			{
				FilterDefinition<TestAuditDocument> filter = FilterDefinition<TestAuditDocument>.Empty;
				FilterDefinitionBuilder<TestAuditDocument> filterBuilder = Builders<TestAuditDocument>.Filter;
				filter &= filterBuilder.Nin(x => x.TestId, testIds);
				DeleteResult result = await _testAudits.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Pruned {NumDeleted} test audit documents", result.DeletedCount);
				}
			}
			// pruned all test health reports related to tests that were NOT kept
			{
				FilterDefinition<TestHealthReportDocument> filter = FilterDefinition<TestHealthReportDocument>.Empty;
				FilterDefinitionBuilder<TestHealthReportDocument> filterBuilder = Builders<TestHealthReportDocument>.Filter;
				filter &= filterBuilder.Nin(x => x.TestId, testIds);
				DeleteResult result = await _testHealthReports.DeleteManyAsync(filter, cancellationToken);
				if (result.DeletedCount > 0)
				{
					_logger.LogInformation("Pruned {NumDeleted} test health report documents", result.DeletedCount);
				}
			}
		}
	}
}
