// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.TestData;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Controller for the /api/v1/testdata endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class TestDataController : ControllerBase
	{
		/// <summary>
		/// Collection of job documents
		/// </summary>
		private readonly JobService _jobService;

		/// <summary>
		/// Collection of test data documents
		/// </summary>
		private readonly ITestDataCollection _testDataCollection;

		private readonly IUserCollection _userCollection;

		readonly TestDataService _testDataService;

		readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public TestDataController(TestDataService testDataService, JobService jobService, ITestDataCollection testDataCollection, IOptionsSnapshot<BuildConfig> buildConfig, IUserCollection userCollection)
		{
			_jobService = jobService;
			_testDataCollection = testDataCollection;
			_testDataService = testDataService;
			_buildConfig = buildConfig;
			_userCollection = userCollection;
		}

		/// <summary>
		/// Get metadata 
		/// </summary>
		/// <param name="projects"></param>
		/// <param name="platforms"></param>
		/// <param name="targets"></param>
		/// <param name="configurations"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v2/testdata/metadata")]
		[ProducesResponseType(typeof(List<GetTestMetaResponse>), 200)]
		public async Task<ActionResult<List<GetTestMetaResponse>>> GetTestMetaAsync(
			[FromQuery(Name = "project")] string[]? projects = null,
			[FromQuery(Name = "platform")] string[]? platforms = null,
			[FromQuery(Name = "target")] string[]? targets = null,
			[FromQuery(Name = "configuration")] string[]? configurations = null)
		{
			IReadOnlyList<ITestMeta> metaData = await _testDataService.FindTestMetaAsync(projects, platforms, configurations, targets);
			return metaData.ConvertAll(m => new GetTestMetaResponse
			{
				Id = m.Id.ToString(),
				Platforms = m.Platforms.Select(p => p).ToList(),
				Configurations = m.Configurations.Select(p => p).ToList(),
				BuildTargets = m.BuildTargets.Select(p => p).ToList(),
				ProjectName = m.ProjectName,
				RHI = m.RHI,
				Variation = m.Variation

			});
		}

		/// <summary>
		/// Get test details from provided refs
		/// </summary>
		/// <param name="ids"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v2/testdata/details")]
		[ProducesResponseType(typeof(List<GetTestDataDetailsResponse>), 200)]
		public async Task<ActionResult<List<GetTestDataDetailsResponse>>> GetTestDetailsAsync([FromQuery(Name = "id")] string[] ids)
		{
			TestRefId[] idValues = Array.ConvertAll(ids, x => TestRefId.Parse(x));
			IReadOnlyList<ITestDataDetails> details = await _testDataService.FindTestDetailsAsync(idValues);
			return details.Select(d => new GetTestDataDetailsResponse
			{
				Id = d.Id.ToString(),
				TestDataIds = d.TestDataIds.Select(x => x.ToString()).ToList(),
				SuiteTests = d.SuiteTests?.Select(x => new GetSuiteTestDataResponse
				{
					TestId = x.TestId.ToString(),
					Outcome = x.Outcome,
					Duration = x.Duration,
					UID = x.UID,
					WarningCount = x.WarningCount,
					ErrorCount = x.ErrorCount
				}).ToList()
			}).ToList();
		}

		/// <summary>
		/// Get test details from provided refs
		/// </summary>
		/// <param name="request"></param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v2/testdata/tests")]
		[ProducesResponseType(typeof(List<GetTestResponse>), 200)]
		public async Task<ActionResult<List<GetTestResponse>>> GetTestsAsync([FromBody] GetTestsRequest request)
		{
			HashSet<string> testIds = new HashSet<string>(request.TestIds);

			IReadOnlyList<ITest> testValues = await _testDataService.FindTestsAsync(testIds.Select(x => TestId.Parse(x)).ToArray());

			return testValues.Select(x => new GetTestResponse
			{
				Id = x.Id.ToString(),
				Name = x.Name,
				DisplayName = x.DisplayName,
				SuiteName = x.SuiteName?.ToString(),
				Metadata = x.Metadata.Select(m => m.ToString()).ToList(),
			}).ToList();
		}

		/// <summary>
		/// Get stream test data for the provided ids
		/// </summary>
		/// <param name="streamIds"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v2/testdata/streams")]
		[ProducesResponseType(typeof(List<GetTestStreamResponse>), 200)]
		public async Task<ActionResult<List<GetTestStreamResponse>>> GetTestStreamsAsync([FromQuery(Name = "Id")] string[] streamIds)
		{
			StreamId[] streamIdValues = Array.ConvertAll(streamIds, x => new StreamId(x));

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestStreamResponse> responses = new List<GetTestStreamResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			HashSet<TestId> testIds = new HashSet<TestId>();
			HashSet<TestMetaId> metaIds = new HashSet<TestMetaId>();

			IReadOnlyList<ITestStream> streams = await _testDataService.FindTestStreamsAsync(queryStreams.ToArray());

			// flatten requested streams to single service queries		
			HashSet<TestSuiteId> suiteIds = new HashSet<TestSuiteId>();
			for (int i = 0; i < streams.Count; i++)
			{
				foreach (TestId testId in streams[i].Tests)
				{
					testIds.Add(testId);
				}

				foreach (TestSuiteId suiteId in streams[i].TestSuites)
				{
					suiteIds.Add(suiteId);
				}
			}

			IReadOnlyList<ITestSuite> suites = new List<ITestSuite>();
			if (suiteIds.Count > 0)
			{
				suites = await _testDataService.FindTestSuitesAsync(suiteIds.ToArray());
			}

			IReadOnlyList<ITest> tests = new List<ITest>();
			if (testIds.Count > 0)
			{
				tests = await _testDataService.FindTestsAsync(testIds.ToArray());
			}

			// gather all meta data
			IReadOnlyList<ITestMeta> metaData = new List<ITestMeta>();
			foreach (ITest test in tests)
			{
				foreach (TestMetaId metaId in test.Metadata)
				{
					metaIds.Add(metaId);
				}
			}

			foreach (ITestSuite suite in suites)
			{
				foreach (TestMetaId metaId in suite.Metadata)
				{
					metaIds.Add(metaId);
				}
			}

			if (metaIds.Count > 0)
			{
				metaData = await _testDataService.FindTestMetaAsync(metaIds: metaIds.ToArray());
			}

			// generate individual stream responses
			foreach (ITestStream s in streams)
			{
				List<ITest> streamTests = tests.Where(x => s.Tests.Contains(x.Id)).ToList();

				List<ITestSuite> streamSuites = new List<ITestSuite>();
				foreach (TestSuiteId suiteId in s.TestSuites)
				{
					ITestSuite? suite = suites.FirstOrDefault(x => x.Id == suiteId);
					if (suite != null)
					{
						streamSuites.Add(suite);
					}
				}

				HashSet<TestMetaId> streamMetaIds = new HashSet<TestMetaId>();
				foreach (ITest test in streamTests)
				{
					foreach (TestMetaId id in test.Metadata)
					{
						streamMetaIds.Add(id);
					}
				}

				foreach (ITestSuite suite in streamSuites)
				{
					foreach (TestMetaId id in suite.Metadata)
					{
						streamMetaIds.Add(id);
					}
				}

				List<ITestMeta> streamMetaData = metaData.Where(x => streamMetaIds.Contains(x.Id)).ToList();

				responses.Add(new GetTestStreamResponse
				{
					StreamId = s.StreamId.ToString(),
					Tests = tests.Select(test => new GetTestResponse
					{
						Id = test.Id.ToString(),
						Name = test.Name,
						DisplayName = test.DisplayName,
						SuiteName = test.SuiteName?.ToString(),
						Metadata = test.Metadata.Select(m => m.ToString()).ToList()
					}).ToList(),
					TestSuites = suites.Select(suite => new GetTestSuiteResponse
					{
						Id = suite.Id.ToString(),
						Name = suite.Name,
						Metadata = suite.Metadata.Select(x => x.ToString()).ToList()
					}).ToList(),
					TestMetadata = metaData.Select(meta => new GetTestMetaResponse
					{
						Id = meta.Id.ToString(),
						Platforms = meta.Platforms.Select(p => p).ToList(),
						Configurations = meta.Configurations.Select(p => p).ToList(),
						BuildTargets = meta.BuildTargets.Select(p => p).ToList(),
						ProjectName = meta.ProjectName,
						RHI = meta.RHI,
						Variation = meta.Variation
					}).ToList()
				});
			}

			return responses;
		}

		/// <summary>
		/// Gets test data refs 
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="testIds"></param>
		/// <param name="suiteIds"></param>
		/// <param name="metaIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minChange"></param>
		/// <param name="maxChange"></param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v2/testdata/refs")]
		[ProducesResponseType(typeof(List<GetTestDataRefResponse>), 200)]
		public async Task<ActionResult<List<GetTestDataRefResponse>>> GetTestDataRefAsync(
			[FromQuery(Name = "Id")] string[] streamIds,
			[FromQuery(Name = "Mid")] string[] metaIds,
			[FromQuery(Name = "Tid")] string[]? testIds = null,
			[FromQuery(Name = "Sid")] string[]? suiteIds = null,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] CommitId? minChange = null,
			[FromQuery] CommitId? maxChange = null)
		{
			StreamId[] streamIdValues = Array.ConvertAll(streamIds, x => new StreamId(x));

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestDataRefResponse> responses = new List<GetTestDataRefResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			IReadOnlyList<ITestDataRef> dataRefs = await _testDataService.FindTestRefsAsync(queryStreams.ToArray(), metaIds.ConvertAll(x => TestMetaId.Parse(x)).ToArray(), testIds, suiteIds, minCreateTime?.UtcDateTime, maxCreateTime?.UtcDateTime, minChange, maxChange);
			foreach (ITestDataRef testData in dataRefs)
			{
				responses.Add(new GetTestDataRefResponse
				{
					Id = testData.Id.ToString(),
					StreamId = testData.StreamId.ToString(),
					JobId = testData.JobId?.ToString(),
					StepId = testData.StepId?.ToString(),
					Duration = testData.Duration,
					BuildCommitId = testData.BuildCommitId,
					MetaId = testData.Metadata.ToString(),
					TestId = testData.TestId?.ToString(),
					Outcome = testData.TestId != null ? testData.Outcome : null,
					SuiteId = testData.SuiteId?.ToString(),
					SuiteSkipCount = testData.SuiteSkipCount,
					SuiteWarningCount = testData.SuiteWarningCount,
					SuiteErrorCount = testData.SuiteErrorCount,
					SuiteSuccessCount = testData.SuiteSuccessCount
				});
			}

			return responses;
		}

		/// <summary>
		/// Creates a new TestData document
		/// </summary>
		/// <returns>The stream document</returns>
		[HttpPost]
		[Route("/api/v1/testdata")]
		public async Task<ActionResult<CreateTestDataResponse>> CreateAsync(CreateTestDataRequest request)
		{
			IJob? job = await _jobService.GetJobAsync(request.JobId);
			if (job == null)
			{
				return NotFound();
			}
			if (!_buildConfig.Value.Authorize(job, JobAclAction.UpdateJob, User))
			{
				return Forbid();
			}

			IJobStep? jobStep;
			if (!job.TryGetStep(request.StepId, out jobStep))
			{
				return NotFound();
			}

			string dataJson = JsonSerializer.Serialize(request.Data); 
			BsonDocument dataBson = BsonSerializer.Deserialize<BsonDocument>(dataJson);
			IReadOnlyList<ITestData> testData = await _testDataCollection.AddAsync(job, jobStep, new (string key, BsonDocument value)[] { (request.Key, dataBson) });
			return new CreateTestDataResponse(testData[0].Id.ToString());
		}

		/// <summary>
		/// Searches for test data that matches a set of criteria
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="minChange">The minimum changelist number to return (inclusive)</param>
		/// <param name="maxChange">The maximum changelist number to return (inclusive)</param>
		/// <param name="jobId">The job id</param>
		/// <param name="jobStepId">The unique step id</param>
		/// <param name="key">Key identifying the result to return</param>
		/// <param name="index">Offset within the results to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The stream document</returns>
		[HttpGet]
		[Route("/api/v1/testdata")]
		[ProducesResponseType(typeof(List<GetTestDataResponse>), 200)]
		public async Task<ActionResult<List<object>>> FindTestDataAsync([FromQuery] string? streamId = null, [FromQuery] CommitId? minChange = null, [FromQuery] CommitId? maxChange = null, JobId? jobId = null, JobStepId? jobStepId = null, string? key = null, int index = 0, int count = 10, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamId? streamIdValue = null;
			if (streamId != null)
			{
				streamIdValue = new StreamId(streamId);
			}

			List<object> results = new List<object>();

			IReadOnlyList<ITestData> documents = await _testDataCollection.FindAsync(streamIdValue, minChange, maxChange, jobId, jobStepId, key, index, count, cancellationToken);
			foreach (ITestData testData in documents)
			{
				if (await _jobService.AuthorizeAsync(testData.JobId, JobAclAction.ViewJob, User, _buildConfig.Value, cancellationToken))
				{
					results.Add(PropertyFilter.Apply(new GetTestDataResponse
					{
						Id = testData.Id.ToString(),
						StreamId = testData.StreamId.ToString(),
						TemplateRefId = testData.TemplateRefId.ToString(),
						JobId = testData.JobId.ToString(),
						StepId = testData.StepId.ToString(),
						CommitId = testData.CommitId,
						Key = testData.Key,
						Data = BsonSerializer.Deserialize<Dictionary<string, object>>(testData.Data)
					}
				, filter));
				}
			}

			return results;
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="testDataId">Id of the document to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/testdata/{testDataId}")]
		[ProducesResponseType(typeof(GetTestDataResponse), 200)]
		public async Task<ActionResult<object>> GetTestDataAsync(string testDataId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ITestData? testData = await _testDataCollection.GetAsync(ObjectId.Parse(testDataId), cancellationToken);
			if (testData == null)
			{
				return NotFound();
			}
			if (!await _jobService.AuthorizeAsync(testData.JobId, JobAclAction.ViewJob, User, _buildConfig.Value, cancellationToken))
			{
				return Forbid();
			}

			return PropertyFilter.Apply(new GetTestDataResponse
			{
				Id = testData.Id.ToString(),
				StreamId = testData.StreamId.ToString(),
				TemplateRefId = testData.TemplateRefId.ToString(),
				JobId = testData.JobId.ToString(),
				StepId = testData.StepId.ToString(),
				CommitId = testData.CommitId,
				Key = testData.Key,
				Data = BsonSerializer.Deserialize<Dictionary<string, object>>(testData.Data)
			}, filter);
		}

		/// <summary>
		/// Retrieve list of testdata ids run within a job or job step
		/// </summary>
		/// <param name="jobId">job Id from which the test was run in</param>
		/// <param name="stepId">step Id from which the test was run in</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of testdata ids</returns>
		[HttpGet]
		[Route("/api/v3/testdata/")]
		[ProducesResponseType(typeof(List<GetJobStepTestDataIDResponse>), 200)]
		public async Task<ActionResult<List<GetJobStepTestDataIDResponse>>> GetJobStepTestDataV2Async([FromQuery] string jobId, [FromQuery] string? stepId = null, CancellationToken cancellationToken = default)
		{
			if (!await _jobService.AuthorizeAsync(JobId.Parse(jobId), JobAclAction.ViewJob, User, _buildConfig.Value, cancellationToken))
			{
				return Forbid();
			}

			IReadOnlyList<ITestData> testData = await _testDataService.GetJobStepTestDataV2Async(JobId.Parse(jobId), stepId != null ? JobStepId.Parse(stepId) : null, cancellationToken);
			IReadOnlyList<ITestNameRef> tests = await _testDataService.FindTestNameRefsV2Async(keys: testData.Select(x => x.Key).ToArray(), cancellationToken: cancellationToken);

			return testData.Select(item => new GetJobStepTestDataIDResponse() {
				TestDataId = item.Id.ToString(),
				TestKey = item.Key,
				TestName = tests.FirstOrDefault(t => t.Key == item.Key)?.Name ?? item.Key,
				TestNameRef = tests.FirstOrDefault(t => t.Key == item.Key)?.Id.ToString() ?? item.Id.ToString(),
			}).ToList();
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="testDataId">Id of the document to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>the testdata information</returns>
		[HttpGet]
		[Route("/api/v3/testdata/{testDataId}")]
		[ProducesResponseType(typeof(GetTestDataResponse), 200)]
		public async Task<ActionResult<object>> GetTestDataV2Async(string testDataId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ITestData? testData = await _testDataService.GetTestDataV2Async(ObjectId.Parse(testDataId), cancellationToken);
			if (testData == null)
			{
				return NotFound();
			}
			if (!await _jobService.AuthorizeAsync(testData.JobId, JobAclAction.ViewJob, User, _buildConfig.Value, cancellationToken))
			{
				return Forbid();
			}

			return PropertyFilter.Apply(new GetTestDataResponse
			{
				Id = testData.Id.ToString(),
				StreamId = testData.StreamId.ToString(),
				TemplateRefId = testData.TemplateRefId.ToString(),
				JobId = testData.JobId.ToString(),
				StepId = testData.StepId.ToString(),
				CommitId = testData.CommitId,
				PreflightCommitId = testData.PreflightCommitId,
				Key = testData.Key,
				Data = BsonSerializer.Deserialize<Dictionary<string, object>>(testData.Data)
			}, filter);
		}

		/// <summary>
		/// Get stream test data for the provided ids
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/streams")]
		[ProducesResponseType(typeof(List<GetTestSessionStreamResponse>), 200)]
		public async Task<ActionResult<List<GetTestSessionStreamResponse>>> GetTestStreamsV2Async([FromBody] GetStreamTestsRequest request, CancellationToken cancellationToken = default)
		{
			StreamId[] streamIdValues = request.StreamIds.Select(x => new StreamId(x)).ToArray();

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestSessionStreamResponse> responses = new List<GetTestSessionStreamResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			IReadOnlyList<ITestSessionStream> streams = await _testDataService.FindTestSessionStreamsV2Async(queryStreams.ToArray(), cancellationToken);

			// flatten requested streams to single service queries		
			HashSet<TestId> testIds = new HashSet<TestId>();
			HashSet<TestMetaId> metaIds = new HashSet<TestMetaId>();
			HashSet<TestTagId> tagIds = new HashSet<TestTagId>();
			foreach (ITestSessionStream stream in streams)
			{
				testIds.UnionWith(stream.Tests);
				metaIds.UnionWith(stream.Metadata);
				tagIds.UnionWith(stream.Tags);
			}

			// Gather all relevant test names
			IReadOnlyList<ITestNameRef> tests = new List<ITestNameRef>();
			if (testIds.Count > 0)
			{
				tests = await _testDataService.FindTestNameRefsV2Async(testIds.ToArray(), null, cancellationToken);
			}

			// Gather all relevant metadata
			IReadOnlyList<ITestMetaRef> metadata = new List<ITestMetaRef>();
			if (metaIds.Count > 0)
			{
				metadata = await _testDataService.FindTestMetaV2Async(metaIds.ToArray(), null, cancellationToken);
			}

			// Gather all relevant tags
			IReadOnlyList<ITestTagRef> tags = new List<ITestTagRef>();
			if (tagIds.Count > 0)
			{
				tags = await _testDataService.FindTestTagsV2Async(tagIds.ToArray(), null, cancellationToken);
			}

			// generate individual stream responses
			foreach (ITestSessionStream s in streams)
			{
				IEnumerable<ITestNameRef> filteredTests = tests.Where(x => s.Tests.Contains(x.Id));
				IEnumerable<ITestMetaRef> filteredMetadata = metadata.Where(x => s.Metadata.Contains(x.Id));
				IEnumerable<ITestTagRef> filteredTags = tags.Where(x => s.Tags.Contains(x.Id));

				responses.Add(new GetTestSessionStreamResponse
				{
					StreamId = s.StreamId.ToString(),
					Tests = filteredTests.Select(test => new GetTestNameResponse
					{
						Id = test.Id.ToString(),
						Name = test.Name,
						Key = test.Key,
						Team = test.Team,
						Harness = test.Harness,
						Profile =  test.Profile
						// we purposefully skip Intent has it is not going to be a search-able field for now
					}).ToList(),
					TestMetadata = filteredMetadata.Select(meta => new GetTestMetadataResponse
					{
						Id = meta.Id.ToString(),
						Entries = meta.Entries.ToDictionary(i => i.Key, i => i.Value)
					}).ToList(),
					TestTags = filteredTags.Select(tag => new GetTestTagResponse
					{
						Id = tag.Id.ToString(),
						Name = new (tag.Name)
					}).ToList()
				});
			}

			return responses;
		}

		/// <summary>
		/// Get test names from provided ids
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/tests")]
		[ProducesResponseType(typeof(List<GetTestNameResponse>), 200)]
		public async Task<ActionResult<List<GetTestNameResponse>>> GetTestNameRefV2Async([FromBody] GetTestsRequest request, CancellationToken cancellationToken = default)
		{
			HashSet<string> testIds = new HashSet<string>(request.TestIds);

			IReadOnlyList<ITestNameRef> testValues = await _testDataService.FindTestNameRefsV2Async(testIds.Select(x => TestId.Parse(x)).ToArray(), null, cancellationToken);

			return testValues.Select(x => new GetTestNameResponse
			{
				Id = x.Id.ToString(),
				Name = x.Name,
				Key = x.Key,
				Team = x.Team,
				Intent = x.Intent,
				Harness = x.Harness,
				Profile = x.Profile,
			}).ToList();
		}

		/// <summary>
		/// Get test metadata from provided ids
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/metadata")]
		[ProducesResponseType(typeof(List<GetTestMetadataResponse>), 200)]
		public async Task<ActionResult<List<GetTestMetadataResponse>>> GetTestMetadataV2Async([FromBody] GetMetadataRequest request, CancellationToken cancellationToken = default)
		{
			TestMetaId[]? metaIds = request.MetadataIds?.Select(x => TestMetaId.Parse(x)).ToHashSet().ToArray();

			IReadOnlyList<ITestMetaRef> metadataValues = await _testDataService.FindTestMetaV2Async(metaIds, request.Entries, cancellationToken);

			return metadataValues.Select(x => new GetTestMetadataResponse
			{
				Id = x.Id.ToString(),
				Entries = x.Entries.ToDictionary(i => i.Key, i => i.Value)
			}).ToList();
		}

		/// <summary>
		/// Get test tags from provided ids
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/tags")]
		[ProducesResponseType(typeof(List<GetTestTagResponse>), 200)]
		public async Task<ActionResult<List<GetTestTagResponse>>> GetTestTagRefV2Async([FromBody] GetTestTagRequest request, CancellationToken cancellationToken = default)
		{
			TestTagId[]? tagIds = request.TagIds?.Select(x => TestTagId.Parse(x)).ToHashSet().ToArray();

			IReadOnlyList<ITestTagRef> testValues = await _testDataService.FindTestTagsV2Async(tagIds, request.TagNames?.ToArray(), cancellationToken);

			return testValues.Select(x => new GetTestTagResponse
			{
				Id = x.Id.ToString(),
				Name = x.Name
			}).ToList();
		}

		/// <summary>
		/// Get test recipes from provided ids
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/recipes")]
		[ProducesResponseType(typeof(List<GetTestRecipeResponse>), 200)]
		public async Task<ActionResult<List<GetTestRecipeResponse>>> GetTestRecipeRefV2Async([FromBody] GetTestRecipesRequest request, CancellationToken cancellationToken = default)
		{
			TestRecipeId[]? recipeIds = request.RecipeIds?.Select(x => TestRecipeId.Parse(x)).ToHashSet().ToArray();
			TestId[]? testIds = request.TestIds?.Select(x => TestId.Parse(x)).ToHashSet().ToArray();
			StreamId[]? streamIds = request.StreamIds?.Select(x => new StreamId(x)).ToHashSet().ToArray();

			IReadOnlyList<ITestRecipeRef> testValues = await _testDataService.FindTestRecipesV2Async(recipeIds, testIds, streamIds, cancellationToken);

			return testValues.Where(
				x => _buildConfig.Value.TryGetStream(x.StreamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User)
			).Select(x => new GetTestRecipeResponse
			{
				Id = x.Id.ToString(),
				NameRef = x.NameRef.ToString(),
				StreamId = x.StreamId.ToString(),
				JobTemplateId = x.JobTemplateId.ToString(),
				JobStep = x.JobStep,
				NodeClass = x.NodeClass,
				Version = x.Version,
				TimeoutMinutes = x.TimeoutMinutes,
				WorkflowId = x.WorkflowId?.ToString(),
				Params = x.Params?.ToDictionary(i => i.Key, i => i.Value)
			}).ToList();
		}

		/// <summary>
		/// Get test phases from provide test keys and refs
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/phases")]
		[ProducesResponseType(typeof(List<GetTestPhaseResponse>), 200)]
		public async Task<ActionResult<List<GetTestPhaseResponse>>> GetTestPhaseRefFromTestV2Async([FromBody] GetTestPhasesRequest request, CancellationToken cancellationToken = default)
		{
			string[]? testKeys = request.TestKeys?.ToHashSet().ToArray();
			TestId[]? testIds = request.TestIds?.Select(x => TestId.Parse(x)).ToHashSet().ToArray();
			IReadOnlyList<ITestNameRef> testNameRefs = await _testDataService.FindTestNameRefsV2Async(testIds, testKeys, cancellationToken);
			IReadOnlyList<ITestPhaseRef> phaseValues = await _testDataService.FindTestPhasesV2Async(testNameRefs.Select(t => t.Id).ToArray(), null, cancellationToken);

			List<GetTestPhaseResponse> response = new List<GetTestPhaseResponse>();

			foreach(ITestNameRef test in testNameRefs)
			{
				response.Add(new GetTestPhaseResponse
				{
					TestId = test.Id.ToString(),
					TestKey = test.Key.ToString(),
					TestName = test.Name.ToString(),
					Phases = phaseValues.Where(p => p.TestNameRef == test.Id)
						.Select(p => new GetPhaseResponse
							{
								Id = p.Id.ToString(),
								Key = p.Key,
								Name = p.Name
							}).ToList()
				});
			}

			return response;
		}

		/// <summary>
		/// Gets test data sessions 
		/// </summary>
		/// <param name="request"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minChange"></param>
		/// <param name="maxChange"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/session/tests")]
		[ProducesResponseType(typeof(List<GetTestSessionResponse>), 200)]
		public async Task<ActionResult<List<GetTestSessionResponse>>> GetTestSessionsV2Async(
			[FromBody] GetTestSessionsRequest request,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] CommitId? minChange = null,
			[FromQuery] CommitId? maxChange = null,
			CancellationToken cancellationToken = default)
		{
			StreamId[] streamIdValues = request.StreamIds.Select(x => new StreamId(x)).ToArray();

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestSessionResponse> responses = new List<GetTestSessionResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			TestMetaId[]? queryMetaIds = request.MetaIds?.Select(x => TestMetaId.Parse(x)).ToArray();
			TestId[]? queryTestIds = request.TestIds?.Select(x => TestId.Parse(x)).ToArray();

			IReadOnlyList<ITestSession> sessions = await _testDataService.FindTestSessionsV2Async(queryStreams.ToArray(), queryTestIds, queryMetaIds, minCreateTime?.UtcDateTime, maxCreateTime?.UtcDateTime, minChange, maxChange, cancellationToken);
			foreach (ITestSession testData in sessions)
			{
				responses.Add(new GetTestSessionResponse
				{
					Id = testData.Id.ToString(),
					StreamId = testData.StreamId.ToString(),
					JobId = testData.JobId.ToString(),
					StepId = testData.StepId.ToString(),
					Duration = testData.Duration.TotalSeconds,
					StartDateTime = testData.StartDateTime,
					CommitId = testData.BuildCommitId.ToString(),
					CommitOrder = testData.BuildCommitId.Order,
					MetadataId = testData.Metadata.ToString(),
					NameRef = testData.NameRef.ToString(),
					RecipeId = testData.Recipe?.ToString(),
					TestDataId = testData.TestDataId.ToString(),
					TagIds = testData.Tags?.Select(t => t.ToString()).ToArray(),
					Outcome = testData.Outcome,
					PhasesTotalCount = testData.PhasesTotalCount,
					PhasesSucceededCount = testData.PhasesSucceededCount,
					PhasesUndefinedCount = testData.PhasesUndefinedCount,
					PhasesFailedCount = testData.PhasesFailedCount,
					ErrorFingerprints = testData.ErrorFingerprints?.Select(
						e => new GetErrorFingerprintResponse()
						{
							Key = e.Key,
							Phases = e.Phases.ToList()
						} ).ToList()
				});
			}

			return responses;
		}

		/// <summary>
		/// Gets test data phase sessions 
		/// </summary>
		/// <param name="request"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minChange"></param>
		/// <param name="maxChange"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/session/phases")]
		[ProducesResponseType(typeof(List<GetTestPhaseSessionResponse>), 200)]
		public async Task<ActionResult<List<GetTestPhaseSessionResponse>>> GetTestPhaseSessionsV2Async(
			[FromBody] GetPhaseSessionsRequest request,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			[FromQuery] DateTimeOffset? maxCreateTime = null,
			[FromQuery] CommitId? minChange = null,
			[FromQuery] CommitId? maxChange = null,
			CancellationToken cancellationToken = default)
		{
			StreamId[] streamIdValues = request.StreamIds.Select(x => new StreamId(x)).ToArray();

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestPhaseSessionResponse> responses = new List<GetTestPhaseSessionResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			TestPhaseId[] queryPhaseIds = request.PhaseIds.Select(x => TestPhaseId.Parse(x)).ToArray();

			IReadOnlyList<ITestPhaseSession> sessions = await _testDataService.FindTestPhaseSessionsV2Async(queryStreams.ToArray(), queryPhaseIds, minCreateTime?.UtcDateTime, maxCreateTime?.UtcDateTime, minChange, maxChange, cancellationToken);
			foreach (ITestPhaseSession testData in sessions)
			{
				responses.Add(new GetTestPhaseSessionResponse
				{
					Id = testData.Id.ToString(),
					StreamId = testData.StreamId.ToString(),
					JobId = testData.JobId.ToString(),
					StepId = testData.StepId.ToString(),
					Duration = testData.Duration.TotalSeconds,
					StartDateTime = testData.StartDateTime,
					CommitId = testData.BuildCommitId.ToString(),
					CommitOrder = testData.BuildCommitId.Order,
					MetadataId = testData.Metadata.ToString(),
					PhaseRef = testData.PhaseRef.ToString(),
					SessionId = testData.SessionId.ToString(),
					Outcome = testData.Outcome,
					HasWarning = testData.HasWarning,
					TagIds = testData.Tags?.Select(t => t.ToString()).ToArray(),
					ErrorFingerprint = testData.ErrorFingerprint,
					EventStreamPath = testData.EventStreamPath
				});
			}

			return responses;
		}

		/// <summary>
		/// Post a test audit
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/audit/post")]
		[ProducesResponseType(typeof(GetTestAuditResponse), 200)]
		public async Task<ActionResult<GetTestAuditResponse>> AddTestAuditV2Async([FromBody] AddTestAuditRequest request, CancellationToken cancellationToken = default)
		{
			UserId? userId = User.GetUserId();
			if (userId is null)
			{
				return BadRequest("No user logged in.");
			}

			TestId testId;
			if (request.TestId is null || request.TestId.Length == 0 || !TestId.TryParse(request.TestId, out testId))
			{
				return BadRequest("Invalid test id.");
			}

			ITestAudit? prevAudit = (await _testDataService.FindTestAuditsV2Async([testId], null, cancellationToken)).ToList().FirstOrDefault();

			// add notification settings hidden from user
			Dictionary<string, string>? notificationsHiddenFromUser = prevAudit?.Notifications?
				.Where(n => _buildConfig.Value.TryGetStream(n.Stream, out StreamConfig? streamConfig) && !streamConfig.Authorize(JobAclAction.ViewJob, User))
				.ToDictionary(n => n.Stream.ToString(), n => n.Workflow.ToString());
			if (notificationsHiddenFromUser is not null && notificationsHiddenFromUser.Count > 0)
			{
				request.Notifications = request.Notifications is null? notificationsHiddenFromUser : request.Notifications.Concat(notificationsHiddenFromUser).ToLookup(n => n.Key, n => n.Value).ToDictionary(n => n.Key, n => n.Last());
			}

			// add audit entry
			TestAuditOptions options = new TestAuditOptions()
			{
				LastAuditDate = request.LastAuditDate?.UtcDateTime,
				IsUnderAudit = request.IsUnderAudit,
				EnableSchedule = request.EnableSchedule ?? true,
				Owner = request.Owner == null || request.Owner.Length == 0 || !UserId.TryParse(request.Owner, out _) ? null : UserId.Parse(request.Owner),
				Customers = request.Customers?.Where(u => u.Length > 0 && UserId.TryParse(u, out _)).Select(u => UserId.Parse(u)).ToList(),
				Notifications = request.Notifications?.Select(
					n => new TestNotificationSettingsOptions()
					{
						Stream = new StreamId(n.Key),
						Workflow = new WorkflowId(n.Value)
					}).ToList(),
				Team = request.Team,
				Harness = request.Harness,
				Intent = request.Intent,
				Profile = request.Profile,
				Notes = request.Notes
			};

			ITestAudit audit = await _testDataService.AddTestAuditAsync(userId!.Value, testId, options, cancellationToken);

			// log changes
			await LogAuditChangesAsync(userId!.Value, testId, options, prevAudit, cancellationToken);

			return GetNewTestAutditResponse(audit);
		}

		private async Task LogAuditChangesAsync(UserId userId, TestId testId, TestAuditOptions options, ITestAudit? prevAudit, CancellationToken cancellationToken = default)
		{
			string userName = (await _userCollection.GetCachedUserAsync(userId, cancellationToken))?.Name ?? userId.ToString();
			IAuditLogChannel<TestId> logger = _testDataService.GetTestAuditLogger(testId);

			if (options.Team != prevAudit?.Team)
			{
				logger.LogInformation("Team changed to {Team} by {UserName}", options.Team ?? "none", userName);
			}
			if (options.Intent != prevAudit?.Intent)
			{
				logger.LogInformation("Intent changed to {Intent} by {UserName}", options.Intent ?? "none", userName);
			}
			if (options.Harness != prevAudit?.Harness)
			{
				logger.LogInformation("Harness changed to {Harness} by {UserName}", options.Harness ?? "none", userName);
			}
			if (options.Profile != prevAudit?.Profile)
			{
				logger.LogInformation("Profile changed to {Profile} by {UserName}", options.Profile ?? "none", userName);
			}
			if (options.EnableSchedule != prevAudit?.EnableSchedule)
			{
				logger.LogInformation("Enable Schedule changed to {EnableSchedule} by {UserName}", options.EnableSchedule, userName);
			}
			if (options.LastAuditDate != prevAudit?.LastAuditDate)
			{
				logger.LogInformation("Last Audit Date changed to {LastAuditDate} by {UserName}", options.LastAuditDate?.ToString() ?? "none", userName);
			}
			if (options.IsUnderAudit != prevAudit?.IsUnderAudit)
			{
				logger.LogInformation("Is Under Audit changed to {IsUnderAudit} by {UserName}", options.IsUnderAudit?.ToString() ?? "none", userName);
			}
			if (options.Owner != prevAudit?.Owner)
			{
				string ownerName = options.Owner is null ? "none" : (await _userCollection.GetCachedUserAsync(options.Owner.Value, cancellationToken))?.Name ?? options.Owner.Value.ToString();
				logger.LogInformation("Owner changed to {Owner} by {UserName}", ownerName, userName);
			}
			if (options.Customers is null && prevAudit?.Customers is not null)
			{
				logger.LogInformation("Customers changed to {Customers} by {UserName}", "none", userName);
			}
			else if (options.Customers is not null && (prevAudit?.Customers is null || !options.Customers.SequenceEqual(prevAudit.Customers)))
			{
				IEnumerable<Task<IUser?>> tasks = options.Customers.Select(c => _userCollection.GetCachedUserAsync(c, cancellationToken).AsTask());
				string[] customerNames = (await Task.WhenAll(tasks)).OfType<IUser>().Select(c => c.Name).ToArray();
				logger.LogInformation("Customers changed to {Customers} by {UserName}", String.Join(", ", customerNames), userName);
			}
			if (options.Notifications is null && prevAudit?.Notifications is not null)
			{
				logger.LogInformation("Notifications changed to {Notifications} by {UserName}", "none", userName);
			}
			else if (options.Notifications is not null
					&& (
						prevAudit?.Notifications is null
						|| options.Notifications.Any(n => !prevAudit.Notifications.Any(p => p.Stream == n.Stream && p.Workflow == n.Workflow))
						|| prevAudit.Notifications.Any(n => !options.Notifications.Any(p => p.Stream == n.Stream && p.Workflow == n.Workflow))
						))
			{
				string[] notifications = options.Notifications.Select(n => $"[{n.Stream}: {n.Workflow}]").ToArray();
				logger.LogInformation("Notifications changed to {Notifications} by {UserName}", String.Join(", ", notifications), userName);
			}
			if (options.Notes != prevAudit?.Notes)
			{
				logger.LogInformation("Notes changed by {UserName} to:\n{Notes}", userName, options.Notes ?? "<none>");
			}
		}

		/// <summary>
		/// Get test audit history
		/// </summary>
		/// <param name="request"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/audit/history")]
		[ProducesResponseType(typeof(List<GetTestAuditResponse>), 200)]
		public async Task<ActionResult<List<GetTestAuditResponse>>> GetTestAuditsV2Async(
			[FromBody] GetTestAuditRequest request,
			[FromQuery] DateTimeOffset? minCreateTime = null,
			CancellationToken cancellationToken = default)
		{
			TestId[] testIds = request.TestIds.Select(x => TestId.Parse(x)).ToHashSet().ToArray();

			IReadOnlyList<ITestAudit> audits = await _testDataService.FindTestAuditsV2Async(testIds, minCreateTime?.UtcDateTime, cancellationToken);

			return audits.Select(a => GetNewTestAutditResponse(a)).ToList();
		}

		private GetTestAuditResponse GetNewTestAutditResponse(ITestAudit audit)
		{
			return new GetTestAuditResponse()
			{
				TestId = audit.TestId.ToString(),
				EnableSchedule = audit.EnableSchedule,
				UserId = audit.UserId.ToString(),
				UserInputDate = audit.UserInputDate,
				LastAuditDate = audit.LastAuditDate,
				IsUnderAudit = audit.IsUnderAudit,
				Owner = audit.Owner?.ToString(),
				Customers = audit.Customers == null ? null : audit.Customers.Select(c => c.ToString()).ToList(),
				Notifications = audit.Notifications?.Where(
						n => _buildConfig.Value.TryGetStream(n.Stream, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User)
					).ToDictionary(n => n.Stream.ToString(), n => n.Workflow.ToString()),
				Team = audit.Team?.ToString(),
				Harness = audit.Harness?.ToString(),
				Intent = audit.Intent?.ToString(),
				Profile = audit.Profile?.ToString(),
				Notes = audit.Notes?.ToString()
			};
		}

		/// <summary>
		/// Retrieve historical information about a specific test audit
		/// </summary>
		/// <param name="testId"></param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested test audit</returns>
		[HttpGet]
		[Route("/api/v3/testdata/audit/log/{testId}")]
		public async Task<ActionResult> GetAuditLogHistoryAsync(string testId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50, CancellationToken cancellationToken = default)
		{
			TestId test;
			if (String.IsNullOrEmpty(testId) || !TestId.TryParse(testId, out test))
			{
				return BadRequest("Invalid test id");
			}

			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync(cancellationToken);
			await _testDataService.GetTestAuditLogger(test).FindAsync(Response.BodyWriter, minTime, maxTime, index, count, cancellationToken);
			await Response.CompleteAsync();

			return Empty;
		}

		/// <summary>
		/// Get test health reports
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v3/testdata/health/reports")]
		[ProducesResponseType(typeof(List<GetTestHealthReportResponse>), 200)]
		public async Task<ActionResult<List<GetTestHealthReportResponse>>> GetTestHealthReportsV2Async([FromBody] GetTestHealthReportRequest request, CancellationToken cancellationToken = default)
		{
			StreamId[] streamIdValues = request.StreamIds.Select(x => new StreamId(x)).ToArray();

			List<StreamId> queryStreams = new List<StreamId>();
			List<GetTestHealthReportResponse> responses = new List<GetTestHealthReportResponse>();

			// authorize streams
			foreach (StreamId streamId in streamIdValues)
			{
				if (_buildConfig.Value.TryGetStream(streamId, out StreamConfig? streamConfig) && streamConfig.Authorize(JobAclAction.ViewJob, User))
				{
					queryStreams.Add(streamId);
				}
			}

			if (queryStreams.Count == 0)
			{
				return responses;
			}

			TestId[]? queryTestIds = request.TestIds?.Select(x => TestId.Parse(x)).ToArray();

			IReadOnlyList<ITestHealthReport> reports = await _testDataService.FindTestHealthReportsV2Async(queryStreams.ToArray(), queryTestIds, cancellationToken);

			return reports.Select(report => new GetTestHealthReportResponse()
			{
				Id = report.Id.ToString(),
				TestId = report.TestId.ToString(),
				StreamId = report.StreamId.ToString(),
				LastUpdateDate = report.LastUpdateDateUtc,
				IsHealthy = report.IsHealthy,
				State = report.State,
				SuccessRate = report.SuccessRate,
				FailureRate	= report.FailureRate,
				CatastrophicFailureRate = report.CatastrophicFailureRate,
				RedundantErrorRate = report.RedundantErrorRate,
				NotificationLastDate = report.NotificationLastDateUtc
			}).ToList();
		}
	}
}
