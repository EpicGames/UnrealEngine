// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// TestData management service
	/// </summary>
	public sealed class TestDataService : IHostedService, IAsyncDisposable
	{

		readonly ITestDataCollection _testData;
		readonly ITestDataCollectionV2 _testDataV2;
		readonly IOptionsMonitor<BuildServerConfig> _settings;
		readonly ITicker _ticker;
		readonly ILogger<TestDataService> _logger;

		/// <summary>
		/// TestData service constructor
		/// </summary>
		public TestDataService(ITestDataCollection testData, ITestDataCollectionV2 testDataV2, IOptionsMonitor<BuildServerConfig> settings, IClock clock, ILogger<TestDataService> logger)
		{
			_testData = testData;
			_testDataV2 = testDataV2;
			_settings = settings;
			_logger = logger;
			_ticker = clock.AddSharedTicker<TestDataService>(TimeSpan.FromMinutes(10.0), TickAsync, logger);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();

			try
			{
				await _testData.UpgradeAsync(cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while upgrading test data collection: {Message}", ex.Message);
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
		}

		/// <summary>
		/// Ticks service
		/// </summary>
		async ValueTask TickAsync(CancellationToken stoppingToken)
		{
			if (!stoppingToken.IsCancellationRequested)
			{
				try
				{
					await _testData.UpdateAsync(_settings.CurrentValue.TestDataRetainMonths, stoppingToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while ticking test data collection: {Message}", ex.Message);
				}
				try
				{
					await _testDataV2.UpdateAsync(_settings.CurrentValue.TestDataRetainMonths, stoppingToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while ticking test data collection v2: {Message}", ex.Message);
				}
			}
		}

		internal async Task TickForTestingAsync()
		{
			await TickAsync(CancellationToken.None);
		}

		/// <summary>
		/// Find test streams
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestStream>> FindTestStreamsAsync(StreamId[] streamIds, CancellationToken cancellationToken = default)
		{
			return await _testData.FindTestStreamsAsync(streamIds, cancellationToken);
		}

		/// <summary>
		/// Find tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITest>> FindTestsAsync(TestId[] testIds, CancellationToken cancellationToken = default)
		{
			return await _testData.FindTestsAsync(testIds, cancellationToken);
		}

		/// <summary>
		/// Find test suites
		/// </summary>
		/// <param name="suiteIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestSuite>> FindTestSuitesAsync(TestSuiteId[] suiteIds, CancellationToken cancellationToken = default)
		{
			return await _testData.FindTestSuitesAsync(suiteIds, cancellationToken);
		}

		/// <summary>
		/// Find test meta data
		/// </summary>
		/// <param name="projectNames"></param>
		/// <param name="platforms"></param>
		/// <param name="configurations"></param>
		/// <param name="buildTargets"></param>
		/// <param name="rhi"></param>
		/// <param name="variation"></param>
		/// <param name="metaIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestMeta>> FindTestMetaAsync(string[]? projectNames = null, string[]? platforms = null, string[]? configurations = null, string[]? buildTargets = null, string? rhi = null, string? variation = null, TestMetaId[]? metaIds = null, CancellationToken cancellationToken = default)
		{
			return await _testData.FindTestMetaAsync(projectNames, platforms, configurations, buildTargets, rhi, variation, metaIds, cancellationToken);
		}

		/// <summary>
		/// Gets test data refs
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="metaIds"></param>
		/// <param name="testIds"></param>
		/// <param name="suiteIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestDataRef>> FindTestRefsAsync(StreamId[] streamIds, TestMetaId[] metaIds, string[]? testIds = null, string[]? suiteIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{
			TestId[]? tids = testIds?.ConvertAll(x => TestId.Parse(x));
			TestSuiteId[]? sids = suiteIds?.ConvertAll(x => TestSuiteId.Parse(x));

			return await _testData.FindTestRefsAsync(streamIds, metaIds, tids, sids, minCreateTime, maxCreateTime, minCommitId, maxCommitId, cancellationToken);
		}

		/// <summary>
		/// Find test details
		/// </summary>
		/// <param name="ids"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestDataDetails>> FindTestDetailsAsync(TestRefId[] ids, CancellationToken cancellationToken = default)
		{
			return await _testData.FindTestDetailsAsync(ids, cancellationToken);
		}

		/// <summary>
		/// Gets testdata ids run within a job step
		/// </summary>
		/// <param name="jobId">Unique id of the job</param>
		/// <param name="stepId">Unique id of the step</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The testdata ids</returns>
		public async Task<IReadOnlyList<ITestData>> GetJobStepTestDataV2Async(JobId jobId, JobStepId? stepId = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.GetJobStepTestDataAsync(jobId, stepId, cancellationToken);
		}

		/// <summary>
		/// Gets a testdata by ID
		/// </summary>
		/// <param name="id">Unique id of the testdata</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The testdata document</returns>
		public async Task<ITestData?> GetTestDataV2Async(ObjectId id, CancellationToken cancellationToken)
		{
			return await _testDataV2.GetAsync(id, cancellationToken);
		}

		/// <summary>
		/// Gets the test sessions running in provided streams
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestSessionStream>> FindTestSessionStreamsV2Async(StreamId[] streamIds, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestSessionStreamsAsync(streamIds, cancellationToken);
		}

		/// <summary>
		/// Find test sessions
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="testIds"></param>
		/// <param name="metaIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestSession>> FindTestSessionsV2Async(StreamId[] streamIds, TestId[]? testIds = null, TestMetaId[]? metaIds = null, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestSessionsAsync(streamIds, testIds, metaIds, minCreateTime, maxCreateTime, minCommitId, maxCommitId, cancellationToken);
		}

		/// <summary>
		/// Find test name references
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="keys"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestNameRef>> FindTestNameRefsV2Async(TestId[]? testIds = null, string[]? keys = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestNameRefsAsync(testIds, keys, cancellationToken);
		}

		/// <summary>
		/// Find test metadata set
		/// </summary>
		/// <param name="metaIds"></param>
		/// <param name="keyValues"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestMetaRef>> FindTestMetaV2Async(TestMetaId[]? metaIds = null, IReadOnlyDictionary<string, string>? keyValues = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestMetaAsync(metaIds, keyValues, cancellationToken);
		}

		/// <summary>
		/// Find test phase references associated with a set of tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="keys"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestPhaseRef>> FindTestPhasesV2Async(TestId[] testIds, string[]? keys = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestPhasesAsync(testIds, keys, cancellationToken);
		}

		/// <summary>
		/// Find test tag references
		/// </summary>
		/// <param name="tagIds"></param>
		/// <param name="names"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestTagRef>> FindTestTagsV2Async(TestTagId[]? tagIds, string[]? names = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestTagsAsync(tagIds, names, cancellationToken);
		}

		/// <summary>
		/// Find test recipe references
		/// </summary>
		/// <param name="recipeIds"></param>
		/// <param name="testIds"></param>
		/// <param name="streamIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestRecipeRef>> FindTestRecipesV2Async(TestRecipeId[]? recipeIds = null, TestId[]? testIds = null, StreamId[]? streamIds = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestRecipesAsync(recipeIds, testIds, streamIds, cancellationToken);
		}

		/// <summary>
		/// Find test phase sessions from a set of test phases
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="phaseIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="maxCreateTime"></param>
		/// <param name="minCommitId"></param>
		/// <param name="maxCommitId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestPhaseSession>> FindTestPhaseSessionsV2Async(StreamId[] streamIds, TestPhaseId[] phaseIds, DateTime? minCreateTime = null, DateTime? maxCreateTime = null, CommitId? minCommitId = null, CommitId? maxCommitId = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestPhaseSessionsAsync(streamIds, phaseIds, minCreateTime, maxCreateTime, minCommitId, maxCommitId, cancellationToken);
		}

		/// <summary>
		/// Add a test audit entry
		/// </summary>
		/// <param name="userId"></param>
		/// <param name="testId"></param>
		/// <param name="options"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<ITestAudit> AddTestAuditAsync(UserId userId, TestId testId, TestAuditOptions options, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.AddTestAuditAsync(userId, testId, options, cancellationToken);
		}

		/// <summary>
		/// Find audit associated with the tests
		/// </summary>
		/// <param name="testIds"></param>
		/// <param name="minCreateTime"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestAudit>> FindTestAuditsV2Async(TestId[] testIds, DateTime? minCreateTime = null, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestAuditsAsync(testIds, minCreateTime, cancellationToken);
		}

		/// <summary>
		/// Add or update test health report
		/// </summary>
		/// <param name="testId"></param>
		/// <param name="streamId"></param>
		/// <param name="testName"></param>
		/// <param name="testHealth"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<ITestHealthReport> AddOrUpdateTestHealthReportV2Async(TestId testId, StreamId streamId, string testName, ComputedTestHealth testHealth, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.AddOrUpdateTestHealthReportAsync(testId, streamId, testName, testHealth, cancellationToken);
		}

		/// <summary>
		/// Get the logger for the specified test id
		/// </summary>
		/// <param name="testId"></param>
		/// <returns></returns>
		public IAuditLogChannel<TestId> GetTestAuditLogger(TestId testId)
		{
			return _testDataV2.GetTestAuditLogger(testId);
		}

		/// <summary>
		/// Find Test health report associated with stream and test ids
		/// </summary>
		/// <param name="streamIds"></param>
		/// <param name="testIds"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task<IReadOnlyList<ITestHealthReport>> FindTestHealthReportsV2Async(StreamId[] streamIds, TestId[]? testIds, CancellationToken cancellationToken = default)
		{
			return await _testDataV2.FindTestHealthReportsAsync(streamIds, testIds, cancellationToken);
		}

		/// <summary>
		/// Update test health report notification information
		/// </summary>
		/// <param name="reportId"></param>
		/// <param name="resetDate"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public async Task UpdateTestHealthReportNotificationDateAsync(ObjectId reportId, bool resetDate, CancellationToken cancellationToken = default)
		{
			await _testDataV2.UpdateTestHealthReportNotificationDateAsync(reportId, resetDate, cancellationToken);
		}
	}
}