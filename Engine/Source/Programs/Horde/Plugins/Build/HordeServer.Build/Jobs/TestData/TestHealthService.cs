// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Streams;
using HordeServer.Auditing;
using HordeServer.Issues;
using HordeServer.Notifications;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Jobs.TestData
{

	[SingletonDocument("test-health-report-state")]
	class TestHealthReportState : SingletonBase
	{
		public DateTime ReportTime { get; set; } = DateTime.MinValue;
	}

	enum TestHealthState
	{
		Healthy = 5,
		Fair = 4,
		Unstable = 3,
		Bad = 2,
		Broken = 1,
	}

	/// <summary>
	/// Test Health Service
	/// </summary>
	public sealed class TestHealthService : IHostedService, IAsyncDisposable
	{
		readonly ILogger<TestHealthService> _logger;
		readonly Tracer _tracer;
		readonly ITicker _ticker;
		readonly IClock _clock;
		readonly ITestDataCollectionV2 _testData;
		readonly SingletonDocument<TestHealthReportState> _state;
		readonly IOptions<BuildServerConfig> _staticBuildConfig;
		readonly IOptions<BuildConfig> _buildConfig;
		readonly INotificationService _notificationService;

		readonly int _reportIntervalMinutes = 12 * 60; // every 12 hours
		private bool _forceReport = false;

		/// <summary>
		/// Test Health Service constructor
		/// </summary>
		/// <param name="mongoService"></param>
		/// <param name="testData"></param>
		/// <param name="notificationService"></param>
		/// <param name="clock"></param>
		/// <param name="logger"></param>
		/// <param name="tracer"></param>
		/// <param name="staticBuildConfig"></param>
		/// <param name="buildConfig"></param>
		public TestHealthService(IMongoService mongoService, ITestDataCollectionV2 testData, INotificationService notificationService, IClock clock, ILogger<TestHealthService> logger, Tracer tracer, IOptions<BuildServerConfig> staticBuildConfig, IOptions<BuildConfig> buildConfig)
		{
			_state = new SingletonDocument<TestHealthReportState>(mongoService);
			_testData = testData;
			_clock = clock;
			_logger = logger;
			_tracer = tracer;
			_ticker = clock.AddSharedTicker<TestHealthService>(TimeSpan.FromMinutes(10.0), TickAsync, logger);
			_staticBuildConfig = staticBuildConfig;
			_buildConfig = buildConfig;
			_notificationService = notificationService;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
		}

		internal async Task TickForTestingAsync(bool bForceReport = false)
		{
			_forceReport = bForceReport;
			await TickAsync(CancellationToken.None);
		}

		/// <summary>
		/// Ticks service
		/// </summary>
		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(TestHealthService)}.{nameof(TickAsync)}");

			TestHealthReportState state = await _state.GetAsync(cancellationToken);
			DateTime currentTime = _clock.UtcNow;
			DateTime lastReportTime = state.ReportTime == DateTime.MinValue ? currentTime.Subtract(TimeSpan.FromMinutes(_reportIntervalMinutes + 1)) : state.ReportTime;

			if (!_forceReport && (currentTime - lastReportTime).TotalMinutes <= _reportIntervalMinutes)
			{
				return;
			}

			IReadOnlyList<ITestAudit> audits = await _testData.FindTestWithAuditNotificationWorkflowAsync(cancellationToken);
			if (audits.Any())
			{
				_logger.LogInformation("Generating test health reports");

				audits = audits.DistinctBy(a => a.TestId).ToList();
				span.SetAttribute("horde.test_health.audit_workflow_count", audits.Count);

				IReadOnlyDictionary<TestId, ITestAudit> testAuditByTestId = audits.ToDictionary(a => a.TestId);

				IReadOnlyList<ITestNameRef> testNameRefs = await _testData.FindTestNameRefsAsync(testAuditByTestId.Keys.ToArray(), cancellationToken: cancellationToken);
				IReadOnlyDictionary<TestId, ITestNameRef> nameRefByTestId = testNameRefs.ToDictionary(t => t.Id);

				IReadOnlyDictionary<StreamId, StreamConfig> configByStreamId = _buildConfig.Value.Projects.SelectMany(p => p.Streams).ToDictionary(s => s.Id);

				int historyEvaluationDays = _staticBuildConfig.Value.TestHealthHistoryEvaluationDays;
				DateTime minCreateTime = currentTime.Subtract(TimeSpan.FromDays(historyEvaluationDays));

				int reportRemindDays = _staticBuildConfig.Value.TestHealthReportRemindDays;
				DateTime reportRemindTime = currentTime.Subtract(TimeSpan.FromDays(reportRemindDays));

				StreamId[] streamIds = audits.SelectMany(a => a.Notifications!.Select(n => n.Stream)).Distinct().ToArray();
				span.SetAttribute("horde.test_health.audit_stream_count", streamIds.Length);
				foreach (StreamId streamId in streamIds)
				{
					StreamConfig? streamConfig = configByStreamId.GetValueOrDefault(streamId);

					if (streamConfig is null)
					{
						_logger.LogWarning("Skipping Test Health reporting for stream '{StreamId}'; reason: No stream config found for this stream", streamId);
						continue;
					}

					TestId[] testIds = audits.Where(a => a.Notifications!.Any(n => n.Stream == streamId)).Select(a => a.TestId).ToArray();
					IReadOnlyList<ITestSession> sessions = await _testData.FindTestSessionsAsync([streamId], testIds, minCreateTime: minCreateTime, cancellationToken: cancellationToken);
					IReadOnlyList<ITestHealthReport> previousReports = await _testData.FindTestHealthReportsAsync([streamId], testIds, cancellationToken);
					IReadOnlyDictionary<TestId, ITestHealthReport> previousRepostsByTestId = previousReports.ToDictionary(r => r.TestId);

					// Re-organize data
					IReadOnlyDictionary<TestId, List<ITestSession>> sessionByTestId = testIds.ToDictionary(t => t, t => new List<ITestSession>());
					foreach (ITestSession session in sessions)
					{
						sessionByTestId.GetValueOrDefault(session.NameRef)!.Add(session);
					}

					// Apply Health Heuristic to each test
					foreach ((TestId testId, List<ITestSession> history) in sessionByTestId)
					{
						IAuditLogChannel<TestId> auditLog = _testData.GetTestAuditLogger(testId);
						ITestNameRef? nameRef = nameRefByTestId.GetValueOrDefault(testId);
						if (nameRef is null)
						{
							auditLog.LogError("No name reference found for test {TestId}", testId);
							continue;
						}

						if (history.Count == 0)
						{
							auditLog.LogInformation("No session history found for stream {Stream} in the last {Days} days", streamConfig.Name, historyEvaluationDays);
							continue;
						}

						ITestAudit testAudit = testAuditByTestId.GetValueOrDefault(testId)!;

						ITestNotificationSettings? notificationSettings = testAudit.Notifications?.FirstOrDefault(n => n.Stream == streamId);
						if (notificationSettings is null)
						{
							auditLog.LogError("No notification settings found for stream {Stream}", streamConfig.Name);
							continue;
						}

						WorkflowConfig? workflowConfig = streamConfig.Workflows.FirstOrDefault(w => w.Id == notificationSettings.Workflow);
						if (workflowConfig?.ReportChannel is null)
						{
							auditLog.LogError("No workflow config or report channel found for '{WorkflowId}' in stream {Stream}", notificationSettings.Workflow, streamConfig.Name);
							continue;
						}

						ITestHealthReport? previousReport = previousRepostsByTestId.GetValueOrDefault(testId);
						if (previousReport is not null && history.First().Id.CreationTime.AddMinutes(1) < previousReport.LastUpdateDateUtc)
						{
							// Nothing new to compute
							continue;
						}

						ComputedTestHealth? computedHealth = ComputeHealthHeuristic(history);
						if (computedHealth is not null)
						{
							ITestHealthReport report = await _testData.AddOrUpdateTestHealthReportAsync(testId, streamId, nameRef.Name, computedHealth, cancellationToken);

							if (testAudit.IsUnderAudit ?? false)
							{
								// No notification if under audit
								continue;
							}

							bool hasOpenNotificationThread = report.NotificationLastDateUtc is not null;
							if ((report.IsHealthy && !hasOpenNotificationThread) /* no open notification thread */
								|| (!report.IsHealthy && hasOpenNotificationThread && report.PreviousState == report.State && report.NotificationLastDateUtc > reportRemindTime)) /* no state change under remind time */
							{
								continue;
							}

							string[]? userIds = GetAuditAttachedUsers(testAudit);
							await _notificationService.NotifyTestHealthReportAsync(report, workflowConfig.ReportChannel, userIds, cancellationToken);
							// Update notification date or remove it if healthy
							await _testData.UpdateTestHealthReportNotificationDateAsync(report.Id, !report.IsHealthy, cancellationToken);
							auditLog.LogInformation("Reported state {State} to Workflow notification [{Stream}: {Workflow}] channel", report.State, streamId, workflowConfig.Id);
						}
					}
				}
			}

			await _state.UpdateAsync(s => s.ReportTime = currentTime, cancellationToken);
			_forceReport = false;
		}

		/// <summary>
		/// Compute Health Heuristic
		/// </summary>
		/// <param name="history"></param>
		/// <returns></returns>
		private static ComputedTestHealth? ComputeHealthHeuristic(List<ITestSession> history)
		{
			double totalPhaseCount = 0;
			double totalUndefinedCount = 0;
			double totalFailureCount = 0;
			double totalSuccessCount = 0;
			HashSet<string> errorFingerprints = new HashSet<string>();

			foreach (ITestSession session in history)
			{
				totalPhaseCount += session.PhasesSucceededCount + session.PhasesUndefinedCount + session.PhasesFailedCount;
				totalFailureCount += session.PhasesUndefinedCount + session.PhasesFailedCount;
				totalUndefinedCount += session.PhasesUndefinedCount;
				totalSuccessCount += session.PhasesSucceededCount;

				session.ErrorFingerprints?.ToList().ForEach(f => errorFingerprints.Add(f.Key));
			}

			if (totalPhaseCount == 0)
			{
				return null;
			}

			int catastrophicFailureRate = totalFailureCount == 0 ? 0 : (int)Math.Round(totalUndefinedCount / totalFailureCount * 100);
			int failureRate = (int)Math.Round(totalFailureCount / totalPhaseCount * 100);
			int successRate = (int)Math.Round(totalSuccessCount / totalPhaseCount * 100);
			int redundantErrorRate = totalFailureCount == 0 ? 0 : (int)Math.Round((1 - (double)errorFingerprints.Count / totalFailureCount) * 100);
			int nStars = (int)TestHealthState.Fair;
			if (failureRate > 15)
			{
				if (catastrophicFailureRate > 40 || successRate == 0)
				{
					nStars -= 2;
				}
				else
				{
					nStars -= 1;
				}
				if (redundantErrorRate < 80)
				{
					nStars += redundantErrorRate < 60 ? 1 : 0;
				}
				else if (failureRate > 40)
				{
					nStars -= 1;
				}
			}
			if (successRate > 60)
			{
				nStars += 1;
			}

			bool isHealthy = nStars >= (int)TestHealthState.Fair;
			string state = Enum.GetName(typeof(TestHealthState), nStars) ?? "Unknown";
			if (!isHealthy)
			{
				state += $"[{nStars} star{(nStars > 1 ? "s" : "")}]";
			}

			return new ComputedTestHealth()
			{
				IsHealthy = isHealthy,
				State = state,
				SuccessRate = successRate,
				FailureRate = failureRate,
				CatastrophicFailureRate = catastrophicFailureRate,
				RedundantErrorRate = redundantErrorRate
			};
		}

		private static string[]? GetAuditAttachedUsers(ITestAudit testAudit)
		{
			List<string>? userIds = null;
			bool hasOwner = testAudit.Owner is not null;
			bool hasCustomers = testAudit.Customers?.Any() ?? false;
			if (hasOwner || hasCustomers)
			{
				userIds = new List<string>();
				if (hasOwner)
				{
					userIds.Add(testAudit.Owner!.Value.ToString());
				}
				if (hasCustomers)
				{
					userIds.AddRange(testAudit.Customers!.Select(c => c.ToString()));
				}
			}

			return userIds?.ToArray();
		}
	}
}
