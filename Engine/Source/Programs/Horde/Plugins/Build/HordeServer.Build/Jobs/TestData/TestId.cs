// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestId, TestIdConverter>))]
	[ObjectIdConverter(typeof(TestIdConverter))]
	public record struct TestId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestId"/>
		/// </summary>
		public static TestId GenerateNewId() => new TestId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestId Parse(string text) => new TestId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestIdConverter : ObjectIdConverter<TestId>
	{
		/// <inheritdoc/>
		public override TestId FromObjectId(ObjectId id) => new TestId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestId value) => value.Id;
	}
}
