// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestMetaId, TestMetaIdConverter>))]
	[ObjectIdConverter(typeof(TestMetaIdConverter))]
	public record struct TestMetaId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestMetaId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestMetaId"/>
		/// </summary>
		public static TestMetaId GenerateNewId() => new TestMetaId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestMetaId Parse(string text) => new TestMetaId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestMetaId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestMetaId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestMetaIdConverter : ObjectIdConverter<TestMetaId>
	{
		/// <inheritdoc/>
		public override TestMetaId FromObjectId(ObjectId id) => new TestMetaId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestMetaId value) => value.Id;
	}
}
