// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a test phase
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestPhaseId, TestPhaseIdConverter>))]
	[ObjectIdConverter(typeof(TestPhaseIdConverter))]
	public record struct TestPhaseId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for phase id
		/// </summary>
		public static TestPhaseId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestPhaseId"/>
		/// </summary>
		public static TestPhaseId GenerateNewId() => new TestPhaseId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestPhaseId Parse(string text) => new TestPhaseId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestPhaseId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestPhaseId(objectId);
				return true;
			}

			id = default;
			return false;
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestPhaseIdConverter : ObjectIdConverter<TestPhaseId>
	{
		/// <inheritdoc/>
		public override TestPhaseId FromObjectId(ObjectId id) => new TestPhaseId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestPhaseId value) => value.Id;
	}
}
