// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a test phase session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestPhaseSessionId, TestPhaseSessionIdConverter>))]
	[ObjectIdConverter(typeof(TestPhaseSessionIdConverter))]
	public record struct TestPhaseSessionId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestPhaseSessionId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestPhaseSessionId"/>
		/// </summary>
		public static TestPhaseSessionId GenerateNewId() => new TestPhaseSessionId(ObjectId.GenerateNewId());

		/// <summary>
		/// Creates a new <see cref="TestPhaseSessionId"/>
		/// </summary>
		public static TestPhaseSessionId GenerateNewId(DateTime time) => new TestPhaseSessionId(ObjectId.GenerateNewId(time));

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestPhaseSessionId Parse(string text) => new TestPhaseSessionId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestPhaseSessionId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestPhaseSessionId(objectId);
				return true;
			}

			id = default;
			return false;
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestPhaseSessionIdConverter : ObjectIdConverter<TestPhaseSessionId>
	{
		/// <inheritdoc/>
		public override TestPhaseSessionId FromObjectId(ObjectId id) => new TestPhaseSessionId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestPhaseSessionId value) => value.Id;
	}
}
