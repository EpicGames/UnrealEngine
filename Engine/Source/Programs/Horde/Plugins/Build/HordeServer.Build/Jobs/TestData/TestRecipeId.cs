// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a test recipe
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestRecipeId, TestRecipeIdConverter>))]
	[ObjectIdConverter(typeof(TestRecipeIdConverter))]
	public record struct TestRecipeId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for test recipe id
		/// </summary>
		public static TestRecipeId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestRecipeId"/>
		/// </summary>
		public static TestRecipeId GenerateNewId() => new TestRecipeId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestRecipeId Parse(string text) => new TestRecipeId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestRecipeId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestRecipeId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestRecipeIdConverter : ObjectIdConverter<TestRecipeId>
	{
		/// <inheritdoc/>
		public override TestRecipeId FromObjectId(ObjectId id) => new TestRecipeId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestRecipeId value) => value.Id;
	}
}
