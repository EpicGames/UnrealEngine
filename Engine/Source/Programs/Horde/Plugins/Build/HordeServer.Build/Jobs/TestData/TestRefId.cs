// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestRefId, TestRefIdConverter>))]
	[ObjectIdConverter(typeof(TestRefIdConverter))]
	public record struct TestRefId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestRefId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestRefId"/>
		/// </summary>
		public static TestRefId GenerateNewId() => new TestRefId(ObjectId.GenerateNewId());

		/// <summary>
		/// Creates a new <see cref="TestRefId"/>
		/// </summary>
		public static TestRefId GenerateNewId(DateTime time) => new TestRefId(ObjectId.GenerateNewId(time));

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestRefId Parse(string text) => new TestRefId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestRefId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestRefId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestRefIdConverter : ObjectIdConverter<TestRefId>
	{
		/// <inheritdoc/>
		public override TestRefId FromObjectId(ObjectId id) => new TestRefId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestRefId value) => value.Id;
	}
}
