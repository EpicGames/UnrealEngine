// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a test session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestSessionId, TestSessionIdConverter>))]
	[ObjectIdConverter(typeof(TestSessionIdConverter))]
	public record struct TestSessionId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestSessionId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestSessionId"/>
		/// </summary>
		public static TestSessionId GenerateNewId() => new TestSessionId(ObjectId.GenerateNewId());

		/// <summary>
		/// Creates a new <see cref="TestSessionId"/>
		/// </summary>
		public static TestSessionId GenerateNewId(DateTime time) => new TestSessionId(ObjectId.GenerateNewId(time));

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestSessionId Parse(string text) => new TestSessionId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestSessionId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestSessionId(objectId);
				return true;
			}

			id = default;
			return false;
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();

		/// <summary>
		/// The creation time of the id (derived from timestamp)
		/// </summary>
		public DateTime CreationTime => Id.CreationTime;
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestSessionIdConverter : ObjectIdConverter<TestSessionId>
	{
		/// <inheritdoc/>
		public override TestSessionId FromObjectId(ObjectId id) => new TestSessionId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestSessionId value) => value.Id;
	}
}
