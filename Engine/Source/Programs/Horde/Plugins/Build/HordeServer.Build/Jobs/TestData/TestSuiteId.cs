// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a session
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestSuiteId, TestSuiteIdConverter>))]
	[ObjectIdConverter(typeof(TestSuiteIdConverter))]
	public record struct TestSuiteId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestSuiteId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestSuiteId"/>
		/// </summary>
		public static TestSuiteId GenerateNewId() => new TestSuiteId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestSuiteId Parse(string text) => new TestSuiteId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestSuiteId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestSuiteId(objectId);
				return true;
			}
			else
			{
				id = default;
				return false;
			}
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestSuiteIdConverter : ObjectIdConverter<TestSuiteId>
	{
		/// <inheritdoc/>
		public override TestSuiteId FromObjectId(ObjectId id) => new TestSuiteId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestSuiteId value) => value.Id;
	}
}
