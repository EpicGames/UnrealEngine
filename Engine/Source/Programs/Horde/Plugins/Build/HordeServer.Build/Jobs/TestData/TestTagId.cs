// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using EpicGames.Core;
using HordeServer.Utilities;
using MongoDB.Bson;

namespace HordeServer.Jobs.TestData
{
	/// <summary>
	/// Identifier for a tag
	/// </summary>
	/// <param name="Id">Id to construct from</param>
	[JsonSchemaString]
	[TypeConverter(typeof(ObjectIdTypeConverter<TestTagId, TestTagIdConverter>))]
	[ObjectIdConverter(typeof(TestTagIdConverter))]
	public record struct TestTagId(ObjectId Id)
	{
		/// <summary>
		/// Default empty value for session id
		/// </summary>
		public static TestTagId Empty { get; } = default;

		/// <summary>
		/// Creates a new <see cref="TestTagId"/>
		/// </summary>
		public static TestTagId GenerateNewId() => new TestTagId(ObjectId.GenerateNewId());

		/// <inheritdoc cref="ObjectId.Parse(System.String)"/>
		public static TestTagId Parse(string text) => new TestTagId(ObjectId.Parse(text));

		/// <inheritdoc cref="ObjectId.TryParse(System.String, out ObjectId)"/>
		public static bool TryParse(string text, out TestTagId id)
		{
			if (ObjectId.TryParse(text, out ObjectId objectId))
			{
				id = new TestTagId(objectId);
				return true;
			}

			id = default;
			return false;
		}

		/// <inheritdoc/>
		public override string ToString() => Id.ToString();
	}

	/// <summary>
	/// Converter to and from <see cref="ObjectId"/> instances.
	/// </summary>
	class TestTagIdConverter : ObjectIdConverter<TestTagId>
	{
		/// <inheritdoc/>
		public override TestTagId FromObjectId(ObjectId id) => new TestTagId(id);

		/// <inheritdoc/>
		public override ObjectId ToObjectId(TestTagId value) => value.Id;
	}
}
