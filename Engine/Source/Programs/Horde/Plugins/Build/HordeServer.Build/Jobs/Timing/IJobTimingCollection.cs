// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Timing;
using MongoDB.Bson.Serialization.Attributes;

namespace HordeServer.Jobs.Timing
{
	/// <summary>
	/// Average timing information for a node
	/// </summary>
	public class JobStepTimingData
	{
		/// <summary>
		/// Name of the node
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Wait time before executing the group containing this node
		/// </summary>
		[BsonIgnoreIfNull]
		public float? AverageWaitTime { get; set; }

		/// <summary>
		/// Time taken for the group containing this node to initialize
		/// </summary>
		[BsonIgnoreIfNull]
		public float? AverageInitTime { get; set; }

		/// <summary>
		/// Time spent executing this node
		/// </summary>
		[BsonIgnoreIfNull]
		public float? AverageDuration { get; set; }

		/// <summary>
		/// Constructor for serialization
		/// </summary>
		[BsonConstructor]
		private JobStepTimingData()
		{
			Name = null!;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the node</param>
		/// <param name="averageWaitTime">Wait time before executing the group containing this node</param>
		/// <param name="averageInitTime">Time taken for the group containing this node to initialize</param>
		/// <param name="averageDuration">Time spent executing this node</param>
		public JobStepTimingData(string name, float? averageWaitTime, float? averageInitTime, float? averageDuration)
		{
			Name = name;
			AverageWaitTime = averageWaitTime;
			AverageInitTime = averageInitTime;
			AverageDuration = averageDuration;
		}
	}

	/// <summary>
	/// Interface for a collection of JobTiming documents
	/// </summary>
	public interface IJobTimingCollection
	{
		/// <summary>
		/// Add timing information for the given job
		/// </summary>
		/// <param name="jobId">The job id</param>
		/// <param name="steps">List of timing info for each step</param>
		/// <returns>New timing document</returns>
		Task<IJobTiming?> TryAddAsync(JobId jobId, List<JobStepTimingData> steps);

		/// <summary>
		/// Attempts to get the timing information for a particular job
		/// </summary>
		/// <param name="jobId">The unique job id</param>
		/// <returns>Timing info for the requested jbo</returns>
		Task<IJobTiming?> TryGetAsync(JobId jobId);

		/// <summary>
		/// Adds timing information for the particular job
		/// </summary>
		/// <param name="jobTiming">The current timing info for the job</param>
		/// <param name="steps">List of steps to add</param>
		/// <returns>New timing document. Null if the document could not be updated.</returns>
		Task<IJobTiming?> TryAddStepsAsync(IJobTiming jobTiming, List<JobStepTimingData> steps);
	}
}
