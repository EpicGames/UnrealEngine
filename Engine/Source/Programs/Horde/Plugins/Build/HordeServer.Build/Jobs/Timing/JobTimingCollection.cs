// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Timing;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Jobs.Timing
{
	/// <summary>
	/// Concrete implementation of IJobTimingCollection
	/// </summary>
	public class JobTimingCollection : IJobTimingCollection
	{
		/// <summary>
		/// Timing information for a particular step
		/// </summary>
		class JobStepTimingDocument : IJobStepTiming
		{
			public string Name { get; set; }
			public float? AverageWaitTime { get; set; }
			public float? AverageInitTime { get; set; }
			public float? AverageDuration { get; set; }

			[BsonConstructor]
			private JobStepTimingDocument()
			{
				Name = null!;
			}

			public JobStepTimingDocument(string name, float? averageWaitTime, float? averageInitTime, float? averageDuration)
			{
				Name = name;
				AverageWaitTime = averageWaitTime;
				AverageInitTime = averageInitTime;
				AverageDuration = averageDuration;
			}
		}

		/// <summary>
		/// Timing information for a particular job
		/// </summary>
		class JobTimingDocument : IJobTiming
		{
			public JobId Id { get; set; }
			public List<JobStepTimingDocument> Steps { get; set; } = new List<JobStepTimingDocument>();
			public int UpdateIndex { get; set; }

			[BsonIgnore]
			public Dictionary<string, IJobStepTiming>? _nameToStep;

			public bool TryGetStepTiming(string name, ILogger logger, [NotNullWhen(true)] out IJobStepTiming? timing)
			{
				if (_nameToStep == null)
				{
					_nameToStep = new Dictionary<string, IJobStepTiming>();
					foreach (JobStepTimingDocument step in Steps)
					{
						if (_nameToStep.ContainsKey(step.Name))
						{
							logger.LogWarning("Step {Name} appears twice in job timing document {Id}", step.Name, Id);
						}
						_nameToStep[step.Name] = step;
					}
				}
				return _nameToStep.TryGetValue(name, out timing);
			}
		}

		/// <summary>
		/// Collection of timing documents
		/// </summary>
		readonly IMongoCollection<JobTimingDocument> _collection;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service singleton</param>
		public JobTimingCollection(IMongoService mongoService)
		{
			_collection = mongoService.GetCollection<JobTimingDocument>("JobTiming");
		}

		/// <inheritdoc/>
		public async Task<IJobTiming?> TryAddAsync(JobId jobId, List<JobStepTimingData> steps)
		{
			JobTimingDocument jobTiming = new JobTimingDocument();
			jobTiming.Id = jobId;
			jobTiming.Steps.AddRange(steps.Select(x => new JobStepTimingDocument(x.Name, x.AverageWaitTime, x.AverageInitTime, x.AverageDuration)));
			jobTiming.UpdateIndex = 1;

			try
			{
				await _collection.InsertOneAsync(jobTiming);
				return jobTiming;
			}
			catch (MongoWriteException ex)
			{
				if (ex.WriteError.Category == ServerErrorCategory.DuplicateKey)
				{
					return null;
				}
				else
				{
					throw;
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IJobTiming?> TryGetAsync(JobId jobId)
		{
			return await _collection.Find(x => x.Id == jobId).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<IJobTiming?> TryAddStepsAsync(IJobTiming jobTiming, List<JobStepTimingData> steps)
		{
			JobTimingDocument jobTimingDocument = (JobTimingDocument)jobTiming;
			List<JobStepTimingDocument> stepDocuments = steps.ConvertAll(x => new JobStepTimingDocument(x.Name, x.AverageWaitTime, x.AverageInitTime, x.AverageDuration));

			FilterDefinition<JobTimingDocument> filter = Builders<JobTimingDocument>.Filter.Expr(x => x.Id == jobTimingDocument.Id && x.UpdateIndex == jobTimingDocument.UpdateIndex);
			UpdateDefinition<JobTimingDocument> update = Builders<JobTimingDocument>.Update.Set(x => x.UpdateIndex, jobTimingDocument.UpdateIndex + 1).PushEach(x => x.Steps, stepDocuments);

			UpdateResult result = await _collection.UpdateOneAsync(filter, update);
			if (result.ModifiedCount > 0)
			{
				jobTimingDocument._nameToStep = null;
				jobTimingDocument.Steps.AddRange(stepDocuments);
				jobTimingDocument.UpdateIndex++;
				return jobTimingDocument;
			}
			return null;
		}
	}
}
