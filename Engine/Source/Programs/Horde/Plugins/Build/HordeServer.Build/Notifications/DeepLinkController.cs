// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Controller for the /api/v1/deeplink endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class DeepLinkController : ControllerBase
	{
		private readonly INotificationService _notificationService;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public DeepLinkController(INotificationService notificationService, ILogger<DeepLinkController> logger)
		{
			_notificationService = notificationService;
			_logger = logger;
		}

		/// <summary>
		/// Get direct message deep link
		/// </summary>
		/// <param name="request"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpPost]
		[Route("/api/v1/deeplink/directmessage")]
		[ProducesResponseType(typeof(GetDirectMessageLinkResponse), 200)]
		public async Task<ActionResult<GetDirectMessageLinkResponse>> GetDirectMessageDeepLinkAsync([FromBody] GetDirectMessageLinkRequest request, CancellationToken cancellationToken = default)
		{
			HashSet<UserId>? userIds = request?.UserIds?.Select(u => TryParseUserId(u)).OfType<UserId>().ToHashSet();
			if (userIds is null || userIds.Count == 0)
			{
				_logger.LogWarning("Invalid user ids: {UserIds}", request?.UserIds == null? "null" : String.Join(", ", request.UserIds));
				return BadRequest("Invalid user ids");
			}

			UserId? userId = User.GetUserId();
			if (userId is null)
			{
				return BadRequest("No user logged in.");
			}

			userIds.Add(userId!.Value);

			string? url = await _notificationService.GetDirectMessageLinkAsync(userIds.ToList(), cancellationToken);
			if (url is null)
			{
				return Problem("The DM could not be deep linked");
			}

			return new GetDirectMessageLinkResponse() { Url = url };
		}

		/// <summary>
		/// Get channel deep link
		/// </summary>
		/// <param name="channelId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/deeplink/channel")]
		[ProducesResponseType(typeof(GetChannelLinkResponse), 200)]
		public async Task<ActionResult<GetChannelLinkResponse>> GetChannelDeepLinkAsync([FromQuery] string channelId, CancellationToken cancellationToken = default)
		{
			if (String.IsNullOrEmpty(channelId))
			{
				return BadRequest("Channel id is invalid");
			}

			string? url = await _notificationService.GetChannelLinkAsync(channelId, cancellationToken);
			if (url is null)
			{
				return NotFound("Could not find the channel");
			}

			return new GetChannelLinkResponse() { Url = url };
		}

		/// <summary>
		/// Parse a user id from a string. Return null if failed to parse.
		/// </summary>
		/// <param name="userId"></param>
		/// <returns></returns>
		static UserId? TryParseUserId(string userId)
		{
			UserId newObjectId;
			if (!String.IsNullOrEmpty(userId) && UserId.TryParse(userId, out newObjectId))
			{
				return newObjectId;
			}
			else
			{
				return null;
			}
		}
	}
}
