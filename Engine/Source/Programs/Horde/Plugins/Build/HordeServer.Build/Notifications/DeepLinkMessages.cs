// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Notifications
{
	/// <summary>
	/// A direct message link request
	/// </summary>
	public class GetDirectMessageLinkRequest
	{
		/// <summary>
		/// the user ids
		/// </summary>
		public List<string> UserIds { get; set; } = new List<string>();
	}

	/// <summary>
	/// The Direct Message link response
	/// </summary>
	public class GetDirectMessageLinkResponse
	{
		/// <summary>
		/// The result deep link
		/// </summary>
		public string? Url { get; set; }
	}

	/// <summary>
	/// The Channel link response
	/// </summary>
	public class GetChannelLinkResponse
	{
		/// <summary>
		/// The result deep link
		/// </summary>
		public string? Url { get; set; }
	}
}