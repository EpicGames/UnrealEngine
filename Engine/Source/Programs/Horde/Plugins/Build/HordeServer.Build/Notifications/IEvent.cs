// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Base interface for events that users can subscribe for notifications to
	/// </summary>
	public interface IEvent
	{
		/// <summary>
		/// Converts this event into an API event record
		/// </summary>
		/// <returns>The API event record</returns>
		public EventRecord ToRecord();
	}

	/// <summary>
	/// Event that occurs when a job completes
	/// </summary>
	public interface IJobCompleteEvent : IEvent
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The template id
		/// </summary>
		TemplateId TemplateId { get; }
	}

	/// <summary>
	/// Event that occurs when a label completes
	/// </summary>
	public interface ILabelCompleteEvent : IEvent
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The template id
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// Name of the label to monitor
		/// </summary>
		string LabelName { get; }
	}

	/// <summary>
	/// Event that occurs when a job step completes
	/// </summary>
	public interface IStepCompleteEvent : IEvent
	{
		/// <summary>
		/// The stream id
		/// </summary>
		StreamId StreamId { get; }

		/// <summary>
		/// The template id
		/// </summary>
		TemplateId TemplateId { get; }

		/// <summary>
		/// Name of the step to monitor
		/// </summary>
		string StepName { get; }
	}
}
