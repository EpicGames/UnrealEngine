// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Jobs.TestData;
using HordeServer.Streams;
using HordeServer.Users;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Marker interface for (serializable) notifications
	/// Implements IEquatable primarily for being IMemoryCache-compatible.
	/// </summary>
	[SuppressMessage("Design", "CA1040:Avoid empty interfaces", Justification = "Marker interface for generic type handling")]
	public interface INotification<T> : IEquatable<T> { }

	/// <summary>
	/// Notification for job scheduled events
	/// </summary>
	public class JobScheduledNotification : INotification<JobScheduledNotification>
	{
		/// <summary>Job ID</summary>
		public string JobId { get; }

		/// <summary>Job name</summary>
		public string JobName { get; }

		/// <summary>Pool name job got scheduled in</summary>
		public string PoolName { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="jobId"></param>
		/// <param name="jobName"></param>
		/// <param name="poolName"></param>
		public JobScheduledNotification(string jobId, string jobName, string poolName)
		{
			JobId = jobId;
			JobName = jobName;
			PoolName = poolName;
		}

		/// <inheritdoc />
		public bool Equals(JobScheduledNotification? other)
		{
			if (ReferenceEquals(null, other))
			{
				return false;
			}
			if (ReferenceEquals(this, other))
			{
				return true;
			}
			return JobId == other.JobId && JobName == other.JobName && PoolName == other.PoolName;
		}

		/// <inheritdoc />
		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}
			if (ReferenceEquals(this, obj))
			{
				return true;
			}
			if (obj.GetType() != GetType())
			{
				return false;
			}
			return Equals((JobScheduledNotification)obj);
		}

		/// <inheritdoc />
		public override int GetHashCode()
		{
			return HashCode.Combine(JobId, JobName, PoolName);
		}
	}

	/// <summary>
	/// Interface for the notification service
	/// </summary>
	public interface INotificationService
	{
		/// <summary>
		/// Updates a subscription to a trigger
		/// </summary>
		/// <param name="triggerId"></param>
		/// <param name="user"></param>
		/// <param name="email">Whether to receive email notifications</param>
		/// <param name="slack">Whether to receive Slack notifications</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<bool> UpdateSubscriptionsAsync(NotificationTriggerId triggerId, ClaimsPrincipal user, bool? email, bool? slack, CancellationToken cancellationToken);

		/// <summary>
		/// Gets the current subscriptions for a user
		/// </summary>
		/// <param name="triggerId"></param>
		/// <param name="user"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Subscriptions for that user</returns>
		Task<INotificationSubscription?> GetSubscriptionsAsync(NotificationTriggerId triggerId, ClaimsPrincipal user, CancellationToken cancellationToken);

		/// <summary>
		/// Notify all subscribers that a job has been cancelled
		/// </summary>
		/// <param name="job">The job that has finished</param>
		/// <param name="graph">Graph for the job</param>
		void NotifyJobAborted(IJob job, IGraph graph);

		/// <summary>
		/// Notify all subscribers that a job step has finished
		/// </summary>
		/// <param name="job">The job containing the step that has finished</param>
		/// <param name="graph"></param>
		/// <param name="batchId">The batch id</param>
		/// <param name="stepId">The step id</param>
		/// <returns>Async task</returns>
		void NotifyJobStepComplete(IJob job, IGraph graph, JobStepBatchId batchId, JobStepId stepId);

		/// <summary>
		/// Notify all subscribers that a job step's outcome has changed
		/// </summary>
		/// <param name="job">The job containing the step that has changed</param>
		/// <param name="oldLabelStates">The old label states for the job</param>
		/// <param name="newLabelStates">The new label states for the job</param>
		/// <returns>Async task</returns>
		void NotifyLabelUpdate(IJob job, IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates);

		/// <summary>
		/// Notify slack channel about a configuration update
		/// </summary>
		/// <param name="info">Information about the config update</param>
		void NotifyConfigUpdate(ConfigUpdateInfo info);

		/// <summary>
		/// Notify slack channel about a stream update failure
		/// </summary>
		/// <param name="errorMessage">Error message passed back</param>
		/// <param name="fileName"></param>
		/// <param name="change">Latest change number of the file</param>
		/// <param name="author">Author of the change, if found from p4 service</param>
		/// <param name="description">Description of the change, if found from p4 service</param>
		void NotifyConfigUpdateFailure(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null);

		/// <summary>
		/// Sends a notification to a user regarding a build health issue.
		/// </summary>
		/// <param name="issue">The new issue that was created</param>
		void NotifyIssueUpdated(IIssue issue);

		/// <summary>
		/// Send a device service notification
		/// </summary>
		/// <param name="message">The message to send</param>
		/// <param name="device">The device</param>
		/// <param name="pool">The pool</param>
		/// <param name="streamConfig"></param>
		/// <param name="job"></param>
		/// <param name="step"></param>
		/// <param name="node"></param>
		/// <param name="user"></param>
		void NotifyDeviceService(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null);

		/// <summary>
		/// Post a notification for device issues
		/// </summary>
		/// <param name="report">The report data to send</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken);

		/// <summary>
		/// Post a notification for any agents encountering issues
		/// </summary>
		/// <param name="report">The report data to send</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken);

		/// <summary>
		/// Post a notification for the open issues in a stream
		/// </summary>
		/// <param name="report">The report data to send</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken);

		/// <summary>
		/// Post a notification for test health report
		/// </summary>
		/// <param name="report">The report data to send</param>
		/// <param name="recipient">The recipient of the report</param>
		/// <param name="carbonCopies">The users to send a carbon copy</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken);

		/// <summary>
		/// Get Direct Message link
		/// </summary>
		/// <param name="userIds">The user ids to start conversation with</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get Channel link
		/// </summary>
		/// <param name="channel">The channel to get the link for</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken = default);
	}
}
