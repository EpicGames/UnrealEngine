// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Users;
using HordeServer.Agents;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Streams;
using HordeServer.Users;

#pragma warning disable CA1716 // Rename parameter x so that it no longer conflicts with the reserved language keyword y

namespace HordeServer.Notifications
{
	/// <summary>
	/// Implements a notification method
	/// </summary>
	public interface INotificationSink
	{
		/// <summary>
		/// Send notifications that a job has been scheduled
		/// </summary>
		/// <param name="notifications">List of notifications to send</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyJobScheduledAsync(List<JobScheduledNotification> notifications, CancellationToken cancellationToken);

		/// <summary>
		/// Send notifications that a job has completed
		/// </summary>
		/// <param name="job">The job containing the step</param>
		/// <param name="graph"></param>
		/// <param name="outcome"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyJobCompleteAsync(IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken);

		/// <summary>
		/// Send notifications that a job has completed
		/// </summary>
		/// <param name="user">User to notify</param>
		/// <param name="job">The job containing the step</param>
		/// <param name="graph"></param>
		/// <param name="outcome"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyJobCompleteAsync(IUser user, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken);

		/// <summary>
		/// Send notifications that a job step has been canceled
		/// </summary>
		/// <param name="usersToNotify">Optional list of users to notify</param>
		/// <param name="job">The job containing the step</param>
		/// <param name="batch">Unique id of the batch</param>
		/// <param name="step">The step id</param>
		/// <param name="node">Corresponding node for the step</param>
		/// <param name="jobStepEventData"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyJobStepAbortedAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken);

		/// <summary>
		/// Send notifications that a job step has completed
		/// </summary>
		/// <param name="usersToNotify">Optional list of users to notify</param>
		/// <param name="job">The job containing the step</param>
		/// <param name="batch">Unique id of the batch</param>
		/// <param name="step">The step id</param>
		/// <param name="node">Corresponding node for the step</param>
		/// <param name="jobStepEventData"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyJobStepCompleteAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken);

		/// <summary>
		/// Send notifications that a job step has completed
		/// </summary>
		/// <param name="user">User to notify</param>
		/// <param name="job">The job containing the step</param>
		/// <param name="label"></param>
		/// <param name="labelIdx"></param>
		/// <param name="outcome"></param>
		/// <param name="stepData"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task NotifyLabelCompleteAsync(IUser user, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> stepData, CancellationToken cancellationToken);

		#region Issues

		/// <summary>
		/// Send notifications that a new issue has been created or assigned
		/// </summary>
		/// <param name="issue"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task NotifyIssueUpdatedAsync(IIssue issue, CancellationToken cancellationToken);

		/// <summary>
		/// Notify the status of issues in a stream
		/// </summary>
		/// <param name="report"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken);

		#endregion

		/// <summary>
		/// Notification that the configuration state has changed
		/// </summary>
		/// <param name="info">Information about the updated config</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task NotifyConfigUpdateAsync(ConfigUpdateInfo info, CancellationToken cancellationToken);

		/// <summary>
		/// Notification that a stream has failed to update
		/// </summary>
		/// <param name="errorMessage"></param>
		/// <param name="fileName"></param>
		/// <param name="change"></param>
		/// <param name="author"></param>
		/// <param name="description"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task NotifyConfigUpdateFailureAsync(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Notification for device service
		/// </summary>
		/// <param name="message"></param>
		/// <param name="device"></param>
		/// <param name="pool"></param>
		/// <param name="streamConfig"></param>
		/// <param name="job"></param>
		/// <param name="step"></param>
		/// <param name="node"></param>
		/// <param name="user"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task NotifyDeviceServiceAsync(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Notify the status of devices
		/// </summary>
		/// <param name="report"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken);

		/// <summary>
		/// Notify the status of agents in the farm
		/// </summary>
		Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken);

		/// <summary>
		/// Send a batched report of session conflicts detected in the last 12 hours
		/// </summary>
		/// <param name="conflicts">Agent IDs paired with their total mismatch count</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task SendSessionConflictReportAsync(IReadOnlyList<(AgentId Id, int Count)> conflicts, CancellationToken cancellationToken);

		/// <summary>
		/// Post a notification for test health report
		/// </summary>
		/// <param name="report"></param>
		/// <param name="recipient"></param>
		/// <param name="carbonCopies"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken);

		/// <summary>
		/// Get Direct Message Link
		/// </summary>
		/// <param name="userIds"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get Channel link
		/// </summary>
		/// <param name="channel"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken = default);
	}
}

