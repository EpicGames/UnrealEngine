// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Notifications;
using EpicGames.Horde.Users;

namespace HordeServer.Notifications
{
	/// <summary>
	/// An individual subscription
	/// </summary>
	public interface INotificationSubscription
	{
		/// <summary>
		/// Name of the user
		/// </summary>
		public UserId UserId { get; }

		/// <summary>
		/// Whether to receive email notifications
		/// </summary>
		public bool Email { get; }

		/// <summary>
		/// Whether to receive Slack notifications
		/// </summary>
		public bool Slack { get; }
	}

	/// <summary>
	/// Trigger for notifications to be sent
	/// </summary>
	public interface INotificationTrigger
	{
		/// <summary>
		/// Unique id for this subscription list
		/// </summary>
		NotificationTriggerId Id { get; }

		/// <summary>
		/// Whether this trigger has been fired
		/// </summary>
		bool Fired { get; }

		/// <summary>
		/// List of subscriptions to this event
		/// </summary>
		public IReadOnlyList<INotificationSubscription> Subscriptions { get; }
	}
}
