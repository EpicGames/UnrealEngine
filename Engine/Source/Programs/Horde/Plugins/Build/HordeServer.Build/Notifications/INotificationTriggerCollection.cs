// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Notifications;
using EpicGames.Horde.Users;

#pragma warning disable CA1030 // Conside making FireAsync an event

namespace HordeServer.Notifications
{
	/// <summary>
	/// Collection of notification triggers
	/// </summary>
	public interface INotificationTriggerCollection
	{
		/// <summary>
		/// Finds or adds a trigger
		/// </summary>
		/// <param name="triggerId">The trigger id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New trigger document</returns>
		Task<INotificationTrigger> FindOrAddAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken);

		/// <summary>
		/// Finds an existing trigger with the given id, or adds one if it does not exist
		/// </summary>
		/// <param name="triggerId"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		Task<INotificationTrigger?> GetAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken);

		/// <summary>
		/// Deletes a trigger
		/// </summary>
		/// <param name="triggerId">The unique trigger id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken);

		/// <summary>
		/// Deletes a trigger
		/// </summary>
		/// <param name="triggerIds">The unique trigger id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(List<NotificationTriggerId> triggerIds, CancellationToken cancellationToken);

		/// <summary>
		/// Fires the trigger, and marks notifications has having been sent
		/// </summary>
		/// <param name="trigger">The trigger to fire</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The trigger document</returns>
		Task<INotificationTrigger?> FireAsync(INotificationTrigger trigger, CancellationToken cancellationToken);

		/// <summary>
		/// Adds a subscriber to a particular trigger
		/// </summary>
		/// <param name="trigger">The trigger to subscribe to</param>
		/// <param name="userId">The user name</param>
		/// <param name="email">Whether to receive email notifications</param>
		/// <param name="slack">Whether to receive Slack notifications</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new trigger state</returns>
		Task<INotificationTrigger?> UpdateSubscriptionsAsync(INotificationTrigger trigger, UserId userId, bool? email, bool? slack, CancellationToken cancellationToken);
	}
}
