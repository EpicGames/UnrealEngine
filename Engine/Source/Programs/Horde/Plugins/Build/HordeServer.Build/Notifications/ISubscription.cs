// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;

#pragma warning disable CA1716 // Rename virtual/interface member ISubscription.Event so that it no longer conflicts with the reserved language keyword 'Event'.

namespace HordeServer.Notifications
{
	/// <summary>
	/// The type of notification to send
	/// </summary>
	public enum NotificationType
	{
		/// <summary>
		/// Send a DM on Slack
		/// </summary>
		Slack,
	}

	/// <summary>
	/// Subscription to a type of event
	/// </summary>
	public interface ISubscription
	{
		/// <summary>
		/// Unique id for this subscription
		/// </summary>
		public string Id { get; }

		/// <summary>
		/// Name of the event to subscribe to
		/// </summary>
		public IEvent Event { get; }

		/// <summary>
		/// User to notify
		/// </summary>
		public UserId UserId { get; }

		/// <summary>
		/// Type of notification to receive
		/// </summary>
		public NotificationType NotificationType { get; }
	}
}
