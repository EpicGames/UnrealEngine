// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Information for creating a new subscription
	/// </summary>
	public class NewSubscription
	{
		/// <summary>
		/// Name of the event
		/// </summary>
		public EventRecord Event { get; set; }

		/// <summary>
		/// The user name
		/// </summary>
		public UserId UserId { get; set; }

		/// <summary>
		/// Type of notification to send
		/// </summary>
		public NotificationType NotificationType { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="eventRecord">Name of the event</param>
		/// <param name="userId">User name</param>
		/// <param name="notificationType">Notification type</param>
		public NewSubscription(EventRecord eventRecord, UserId userId, NotificationType notificationType)
		{
			Event = eventRecord;
			UserId = userId;
			NotificationType = notificationType;
		}
	}

	/// <summary>
	/// Interface for a collection of subscriptions
	/// </summary>
	public interface ISubscriptionCollection
	{
		/// <summary>
		/// Add new subscription documents
		/// </summary>
		/// <param name="subscriptions">The new subscriptions to add</param>
		/// <returns>The subscriptions that were added</returns>
		public Task<List<ISubscription>> AddAsync(IEnumerable<NewSubscription> subscriptions);

		/// <summary>
		/// Remove a set of existing subscriptions
		/// </summary>
		/// <param name="subscriptions">Subscriptions to remove</param>
		/// <returns>Async task</returns>
		public Task RemoveAsync(IEnumerable<ISubscription> subscriptions);

		/// <summary>
		/// Gets a subscription by id
		/// </summary>
		/// <param name="subscriptionId">Subscription to remove</param>
		/// <returns>Async task</returns>
		public Task<ISubscription?> GetAsync(string subscriptionId);

		/// <summary>
		/// Find all subscribers of a certain event
		/// </summary>
		/// <param name="eventRecord">Name of the event</param>
		/// <returns>Name of the event to find subscribers for</returns>
		public Task<List<ISubscription>> FindSubscribersAsync(EventRecord eventRecord);

		/// <summary>
		/// Find subscriptions for a particular user
		/// </summary>
		/// <param name="userId">The user to search for</param>
		/// <returns>List of subscriptions</returns>
		public Task<List<ISubscription>> FindSubscriptionsAsync(UserId userId);
	}
}
