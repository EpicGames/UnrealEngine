// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Notifications
{
	/// <summary>
	/// ACL actions which apply to notifications
	/// </summary>
	public static class NotificationAclAction
	{
		/// <summary>
		/// Ability to subscribe to notifications
		/// </summary>
		public static AclAction CreateSubscription { get; } = new AclAction("CreateSubscription");
	}
}
