// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Notifications
{
	/// <summary>
	/// A request from a user to be notified when something happens.
	/// </summary>
	public class UpdateNotificationsRequest
	{
		/// <summary>
		/// Request notifications by email
		/// </summary>
		public bool? Email { get; set; }

		/// <summary>
		/// Request notifications on Slack
		/// </summary>
		public bool? Slack { get; set; }
	}

	/// <summary>
	/// A request from a user to be notified when something happens.
	/// </summary>
	public class GetNotificationResponse
	{
		/// <summary>
		/// Request notifications by email
		/// </summary>
		public bool Email { get; set; }

		/// <summary>
		/// Request notifications on Slack
		/// </summary>
		public bool Slack { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="subscription">The subscription interface</param>
		internal GetNotificationResponse(INotificationSubscription? subscription)
		{
			if (subscription == null)
			{
				Email = false;
				Slack = false;
			}
			else
			{
				Email = subscription.Email;
				Slack = subscription.Slack;
			}
		}
	}
}
