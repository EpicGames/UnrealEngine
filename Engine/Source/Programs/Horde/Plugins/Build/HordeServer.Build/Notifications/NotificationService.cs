// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Notifications;
using EpicGames.Horde.Users;
using EpicGames.Redis;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.Graphs;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using StackExchange.Redis;
using JsonSerializer = System.Text.Json.JsonSerializer;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Wraps functionality for delivering notifications.
	/// </summary>
	public sealed class NotificationService : IHostedService, INotificationService, IAsyncDisposable
	{
		/// <summary>
		/// The available notification sinks
		/// </summary>
		private readonly List<INotificationSink> _sinks;

		/// <summary>
		/// Collection of subscriptions
		/// </summary>
		private readonly ISubscriptionCollection _subscriptionCollection;

		/// <summary>
		/// Collection of notification request documents.
		/// </summary>
		private readonly INotificationTriggerCollection _triggerCollection;

		/// <summary>
		/// Instance of the <see cref="_graphCollection"/>.
		/// </summary>
		private readonly IGraphCollection _graphCollection;

		/// <summary>
		/// 
		/// </summary>
		private readonly IUserCollection _userCollection;

		/// <summary>
		/// Job service instance
		/// </summary>
		private readonly JobService _jobService;

		/// <summary>
		/// 
		/// </summary>
		private readonly IssueService _issueService;

		/// <summary>
		/// Instance of the <see cref="ILogCollection"/>.
		/// </summary>
		private readonly ILogCollection _logCollection;

		/// <summary>
		/// Cache for de-duplicating queued notifications
		/// </summary>
		private readonly IMemoryCache _cache;

		/// <summary>
		/// Lock object for manipulating the above cache
		/// Used since batch notification queue handling is run async.
		/// </summary>
		private readonly object _cacheLock = new object();

		/// <summary>
		/// Connection pool for Redis databases
		/// </summary>
		private readonly RedisConnectionPool _redisConnectionPool;

		/// <summary>
		/// Settings for the application.
		/// </summary>
		private readonly IOptionsMonitor<BuildServerConfig> _settings;

		/// <summary>
		/// Async notification tasks currently executing. Tracked so they can be awaited before shutting down
		/// </summary>
		private readonly List<Task> _tasks = [];
		private readonly object _tasksLock = new();

		/// <summary>
		/// Settings for the application.
		/// </summary>
		private readonly ILogger<NotificationService> _logger;

		/// <summary>
		/// Ticker for running batch sender method
		/// </summary>
		internal ITicker _ticker;

		/// <summary>
		/// Interval at which queued notifications should be sent as a batch 
		/// </summary>
		internal TimeSpan _notificationBatchInterval = TimeSpan.FromHours(12);

		readonly IServerInfo _serverInfo;
		readonly SessionConflictService _sessionConflictService;
		readonly Counter<int> _jobCounter;
		readonly Histogram<double> _jobDurationHistogram;

		static string RedisQueueListKey(string notificationType) => "NotificationService.queued." + notificationType;

		/// <summary>
		/// Constructor
		/// </summary>
		public NotificationService(
			IEnumerable<INotificationSink> sinks,
			IOptionsMonitor<BuildServerConfig> settings,
			ILogger<NotificationService> logger,
			IGraphCollection graphCollection,
			ISubscriptionCollection subscriptionCollection,
			INotificationTriggerCollection triggerCollection,
			IUserCollection userCollection,
			JobService jobService,
			IssueService issueService,
			ILogCollection logCollection,
			Meter meter,
			IMemoryCache cache,
			IRedisService redisService,
			IConfigService configService,
			IServerInfo serverInfo,
			SessionConflictService sessionConflictService,
			IClock clock)
		{
			_sinks = sinks.ToList();
			_settings = settings;
			_logger = logger;
			_graphCollection = graphCollection;
			_subscriptionCollection = subscriptionCollection;
			_triggerCollection = triggerCollection;
			_userCollection = userCollection;
			_jobService = jobService;
			_issueService = issueService;
			_logCollection = logCollection;
			_cache = cache;
			_redisConnectionPool = redisService.ConnectionPool;
			_serverInfo = serverInfo;
			_sessionConflictService = sessionConflictService;

			issueService.OnIssueUpdated += NotifyIssueUpdated;
			jobService.OnJobAborted += NotifyJobAborted;
			jobService.OnJobStepComplete += NotifyJobStepComplete;
			jobService.OnJobScheduled += NotifyJobScheduled;
			jobService.OnLabelUpdate += NotifyLabelUpdate;
			configService.OnConfigUpdate += NotifyConfigUpdate;

			_ticker = clock.AddSharedTicker<NotificationService>(_notificationBatchInterval, TickEveryTwelveHoursAsync, logger);
			_jobCounter = meter.CreateCounter<int>("horde.notification.job.count");
			_jobDurationHistogram = meter.CreateHistogram<double>("horde.notification.job.duration");
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
			await WaitForAsyncTasksToCompleteAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			_issueService.OnIssueUpdated -= NotifyIssueUpdated;
			_jobService.OnJobAborted -= NotifyJobAborted;
			_jobService.OnJobStepComplete -= NotifyJobStepComplete;
			_jobService.OnJobScheduled -= NotifyJobScheduled;
			_jobService.OnLabelUpdate -= NotifyLabelUpdate;

			await _ticker.DisposeAsync();
		}

		private async Task WaitForAsyncTasksToCompleteAsync(CancellationToken ct)
		{
			Task[] allTasks;
			lock (_tasksLock)
			{
				allTasks = _tasks.ToArray();
			}

			if (allTasks.Length > 0)
			{
				_logger.LogInformation("Waiting for {Count} notification tasks to complete...", allTasks.Length);
				using CancellationTokenSource cts = CancellationTokenSource.CreateLinkedTokenSource(ct);
				cts.CancelAfter(TimeSpan.FromSeconds(20));
				try
				{
					await Task.WhenAll(allTasks).WaitAsync(cts.Token);
				}
				catch (OperationCanceledException)
				{
					int incomplete = allTasks.Count(t => !t.IsCompleted);
					_logger.LogWarning("Stop timeout reached with {Count} notification tasks still in flight", incomplete);
				}
			}
		}

		/// <inheritdoc/>
		public async Task<bool> UpdateSubscriptionsAsync(NotificationTriggerId triggerId, ClaimsPrincipal user, bool? email, bool? slack, CancellationToken cancellationToken)
		{
			UserId? userId = user.GetUserId();
			if (userId == null)
			{
				_logger.LogWarning("Unable to find username for principal {User}", user.Identity?.Name);
				return false;
			}

			INotificationTrigger trigger = await _triggerCollection.FindOrAddAsync(triggerId, cancellationToken);
			await _triggerCollection.UpdateSubscriptionsAsync(trigger, userId.Value, email, slack, cancellationToken);
			return true;
		}

		/// <inheritdoc/>
		public async Task<INotificationSubscription?> GetSubscriptionsAsync(NotificationTriggerId triggerId, ClaimsPrincipal user, CancellationToken cancellationToken)
		{
			UserId? userId = user.GetUserId();
			if (userId == null)
			{
				return null;
			}

			INotificationTrigger? trigger = await _triggerCollection.GetAsync(triggerId, cancellationToken);
			if (trigger == null)
			{
				return null;
			}

			return trigger.Subscriptions.FirstOrDefault(x => x.UserId == userId.Value);
		}

		/// <inheritdoc/>
		public void NotifyJobAborted(IJob job, IGraph graph)
		{
			// Enqueue job complete notifications if needed
			if (job.GetState() == JobState.Complete && job.AbortedByUserId is not null)
			{
				_logger.LogInformation("Queuing job notifications for {JobId}", job.Id);
				EnqueueTask(ctx => SendJobNotificationsAsync(job, graph, ctx));
				EnqueueTask(ctx => RecordJobCompleteMetricsAsync(job, ctx));
			}
		}

		/// <inheritdoc/>
		public void NotifyJobStepComplete(IJob job, IGraph graph, JobStepBatchId batchId, JobStepId stepId)
		{
			// Enqueue job step complete notifications if needed
			if (job.TryGetStep(batchId, stepId, out IJobStep? step))
			{
				_logger.LogInformation("Queuing step notifications for {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);
				EnqueueTask(ctx => SendJobStepNotificationsAsync(job, batchId, stepId, ctx));
			}

			// Enqueue job complete notifications if needed
			if (job.GetState() == JobState.Complete)
			{
				_logger.LogInformation("Queuing job notifications for {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);
				EnqueueTask(ctx => SendJobNotificationsAsync(job, graph, ctx));
				EnqueueTask(ctx => RecordJobCompleteMetricsAsync(job, ctx));
			}
		}

		/// <summary>
		/// Notify all subscribers that a job step was scheduled
		/// </summary>
		/// <param name="pool">The pool job was scheduled in</param>
		/// <param name="poolHasAgentsOnline">Whether the pool has agents online</param>
		/// <param name="job">The job containing the step that has finished</param>
		/// <param name="graph">Graph of the job</param>
		/// <param name="batchId">The batch id</param>
        public void NotifyJobScheduled(IPoolConfig pool, bool poolHasAgentsOnline, IJob job, IGraph graph, JobStepBatchId batchId)
		{
			if (pool.EnableAutoscaling && !poolHasAgentsOnline)
			{
				EnqueueTasks((sink, ctx) => EnqueueNotificationForBatchSendingAsync(new JobScheduledNotification(job.Id.ToString(), job.Name, pool.Name)));
			}
		}

		/// <inheritdoc/>
		public void NotifyLabelUpdate(IJob job, IReadOnlyList<(LabelState, LabelOutcome)> oldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates)
		{
			// If job has any label trigger IDs, send label complete notifications if needed
			for (int idx = 0; idx < oldLabelStates.Count && idx < newLabelStates.Count; idx++)
			{
				if (oldLabelStates[idx] != newLabelStates[idx])
				{
					EnqueueTask(ctx => SendAllLabelNotificationsAsync(job, oldLabelStates, newLabelStates, ctx));
					break;
				}
			}
		}

		/// <inheritdoc/>
		public void NotifyIssueUpdated(IIssue issue)
		{
			_logger.LogInformation("Issue {IssueId} updated", issue.Id);
			EnqueueTasks((sink, ctx) => sink.NotifyIssueUpdatedAsync(issue, ctx));
		}

		/// <inheritdoc/>
		public void NotifyConfigUpdate(ConfigUpdateInfo info)
		{
			_logger.LogInformation(info.Exception, "Configuration updated ({Result})", (info.Exception == null) ? "success" : "failure");
			EnqueueTasks((sink, ctx) => sink.NotifyConfigUpdateAsync(info, ctx));
		}

		/// <inheritdoc/>
		public void NotifyConfigUpdateFailure(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null)
		{
			EnqueueTasks((sink, ctx) => sink.NotifyConfigUpdateFailureAsync(errorMessage, fileName, change, author, description, ctx));
		}

		/// <inheritdoc/>
		public void NotifyDeviceService(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null)
		{
			EnqueueTasks((sink, ctx) => sink.NotifyDeviceServiceAsync(message, device, pool, streamConfig, job, step, node, user, ctx));
		}

		/// <inheritdoc/>
		public async Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken)
		{
			foreach (INotificationSink sink in _sinks)
			{
				try
				{
					await sink.SendDeviceIssueReportAsync(report, cancellationToken);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed sending issue report to {Channel}", report.Channel);
				}
			}
		}

		/// <inheritdoc/>
		public async Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken)
		{
			foreach (INotificationSink sink in _sinks)
			{
				try
				{
					await sink.NotifyTestHealthReportAsync(report, recipient, carbonCopies, cancellationToken);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed sending test health report to {Recipient}", recipient);
				}
			}
		}

		/// <summary>
		/// Enqueues an async task
		/// </summary>
		/// <param name="taskFunc">Function to generate an async task</param>
		void EnqueueTask(Func<CancellationToken, Task> taskFunc)
		{
			Task task = Task.Run(async () =>
			{
				try
				{
					await taskFunc(CancellationToken.None);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Exception while executing notification task: {Message}", e.Message);
				}
			});
			
			// Self-cleanup when done
			_ = task.ContinueWith(_ =>
			{
				lock (_tasksLock)
				{
					_tasks.Remove(task);
				}
			}, TaskScheduler.Default);
			
			lock (_tasksLock)
			{
				_tasks.Add(task);
			}
		}

		/// <summary>
		/// Enqueues an async task
		/// </summary>
		/// <param name="taskFunc">Function to generate an async task</param>
		void EnqueueTasks(Func<INotificationSink, CancellationToken, Task> taskFunc)
		{
			foreach (INotificationSink sink in _sinks)
			{
				EnqueueTask(ctx => taskFunc(sink, ctx));
			}
		}

		/// <summary>
		/// Enqueue a notification in Redis for batch sending later on 
		/// </summary>
		/// <param name="notification">Notification to enqueue</param>
		/// <param name="deduplicate">True if notification should be deduplicated</param>
		/// <typeparam name="T">Any INotification type</typeparam>
		private async Task EnqueueNotificationForBatchSendingAsync<T>(T notification, bool deduplicate = true) where T : INotification<T>
		{
			lock (_cacheLock)
			{
				if (deduplicate)
				{
					// Use cache to deduplicate notifications
					if (_cache.TryGetValue(notification, out object? _))
					{
						return;
					}
					_cache.Set(notification, notification, _notificationBatchInterval / 2);
				}
			}

			try
			{
				byte[] data = JsonSerializer.SerializeToUtf8Bytes(notification);
				await _redisConnectionPool.GetDatabase().ListRightPushAsync(RedisQueueListKey(typeof(T).ToString()), data);
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Unable to serialize and queue notification {Type} for batch sending in Redis", notification.GetType());
			}
		}

		private async ValueTask TickEveryTwelveHoursAsync(CancellationToken cancellationToken)
		{
			List<JobScheduledNotification> jobScheduledNotifications = await GetAllQueuedNotificationsAsync<JobScheduledNotification>();
			if (jobScheduledNotifications.Count > 0)
			{
				foreach (INotificationSink sink in _sinks)
				{
					await sink.NotifyJobScheduledAsync(jobScheduledNotifications, cancellationToken);
				}
			}

			try
			{
				List<(AgentId Id, int Count)> conflicts = await _sessionConflictService.DrainConflictsAsync(cancellationToken);
				if (conflicts.Count > 0)
				{
					_logger.LogInformation("Drained {Count} agents with session conflicts", conflicts.Count);
					foreach (INotificationSink sink in _sinks)
					{
						try
						{
							await sink.SendSessionConflictReportAsync(conflicts, cancellationToken);
						}
						catch (Exception e)
						{
							_logger.LogError(e, "Failed sending session conflict report");
						}
					}
				}
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Failed draining session conflicts from Redis");
			}
		}

		/// <summary>
		/// Get and clear all queued notifications of type T from Redis
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <returns>Deserialized notifications</returns>
		/// <exception cref="Exception"></exception>
		private async Task<List<T>> GetAllQueuedNotificationsAsync<T>() where T : INotification<T>
		{
			IDatabase redis = _redisConnectionPool.GetDatabase();

			// Reading and deleting the entire list from Redis in this way is not thread-safe.
			// But likely not a big deal considering the alternative of distributed locks.
			string redisKey = RedisQueueListKey(typeof(T).ToString());
			RedisValue[] rawNotifications = await redis.ListRangeAsync(redisKey);
			await redis.KeyDeleteAsync(redisKey);

			List<T> notifications = new List<T>();
			foreach (byte[]? data in rawNotifications)
			{
				try
				{
					T? notification = JsonSerializer.Deserialize<T>(data!);
					if (notification == null)
					{
						throw new Exception("Unable to deserialize");
					}
					notifications.Add(notification);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Unable to deserialize notification data of {Type}", typeof(T));
				}
			}

			return notifications;
		}

		/// <summary>
		/// Gets the <see cref="INotificationTrigger"/> for a given trigger ID, if any.
		/// </summary>
		/// <param name="triggerId"></param>
		/// <param name="fireTrigger">If true, the trigger is fired and cannot be reused</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		private async Task<INotificationTrigger?> GetNotificationTriggerAsync(NotificationTriggerId? triggerId, bool fireTrigger, CancellationToken cancellationToken)
		{
			if (triggerId == null)
			{
				return null;
			}

			INotificationTrigger? trigger = await _triggerCollection.GetAsync(triggerId.Value, cancellationToken);
			if (trigger == null)
			{
				return null;
			}

			return fireTrigger ? await _triggerCollection.FireAsync(trigger, cancellationToken) : trigger;
		}

		private async Task SendJobNotificationsAsync(IJob job, IGraph graph, CancellationToken cancellationToken)
		{
			using IDisposable? scope = _logger.BeginScope("Sending notifications for job {JobId}", job.Id);

			job.GetJobState(job.GetStepForNodeMap(), out _, out LabelOutcome outcome);
			JobCompleteEventRecord jobCompleteEvent = new JobCompleteEventRecord(job.StreamId, job.TemplateId, outcome);

			IReadOnlyList<IUser> usersToNotify = await GetUsersToNotifyAsync(jobCompleteEvent, job.NotificationTriggerId, true, cancellationToken);
			foreach (IUser userToNotify in usersToNotify)
			{
				if (job.PreflightCommitId != null)
				{
					if (userToNotify.Id != job.StartedByUserId)
					{
						continue;
					}
				}
				EnqueueTasks((sink, ctx) => sink.NotifyJobCompleteAsync(userToNotify, job, graph, outcome, ctx));
			}

			if (job.PreflightCommitId == null)
			{
				EnqueueTasks((sink, ctx) => sink.NotifyJobCompleteAsync(job, graph, outcome, ctx));
			}

			_logger.LogDebug("Finished sending notifications for job {JobId}", job.Id);
		}

		private Task RecordJobCompleteMetricsAsync(IJob job, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			void RecordMetric(string type, JobStepOutcome outcome, DateTimeOffset? startTime, DateTimeOffset? finishTime)
			{
				string outcomeStr = outcome switch
				{
					JobStepOutcome.Unspecified => "unspecified",
					JobStepOutcome.Failure => "failure",
					JobStepOutcome.Warnings => "warnings",
					JobStepOutcome.Success => "success",
					_ => "unspecified"
				};

				KeyValuePair<string, object?>[] tags =
				{
					KeyValuePair.Create<string, object?>("stream", job.StreamId.ToString()),
					KeyValuePair.Create<string, object?>("template", job.TemplateId.ToString()),
					KeyValuePair.Create<string, object?>("outcome", outcomeStr),
					KeyValuePair.Create<string, object?>("type", type)
				};

				_jobCounter.Add(1, tags);

				if (startTime == null || finishTime == null)
				{
					_logger.LogDebug("Completed job or step is missing start or finish time, cannot record duration metric. Job ID={JobId}", job.Id);
					return;
				}

				TimeSpan duration = finishTime.Value - startTime.Value;
				_jobDurationHistogram.Record(duration.TotalSeconds, tags);
			}

			try
			{
				JobStepOutcome jobOutcome = job.Batches.SelectMany(x => x.Steps).Min(x => x.Outcome);
				DateTime? startTime = job.Batches.Select(x => x.StartTimeUtc).Min();
				DateTime? finishTime = job.Batches.Select(x => x.FinishTimeUtc).Max();
				RecordMetric("job", jobOutcome, startTime, finishTime);
			}
			catch (Exception e)
			{
				_logger.LogWarning(e, "Failed to Record job metric for {JobId}", job.Id);
			}

			// TODO: record metrics for individual steps
			// foreach (IJobStepBatch Batch in Job.Batches)
			// {
			// 	foreach (IJobStep Step in Batch.Steps)
			// 	{
			// 	}
			// }

			return Task.CompletedTask;
		}

		private async Task<IReadOnlyList<IUser>> GetUsersToNotifyAsync(EventRecord? eventRecord, NotificationTriggerId? notificationTriggerId, bool fireTrigger, CancellationToken cancellationToken)
		{
			List<UserId> userIds = new List<UserId>();

			// Find the notifications for all steps of this type
			if (eventRecord != null)
			{
				List<ISubscription> subscriptions = await _subscriptionCollection.FindSubscribersAsync(eventRecord);
				foreach (ISubscription subscription in subscriptions)
				{
					if (subscription.NotificationType == NotificationType.Slack)
					{
						userIds.Add(subscription.UserId);
					}
				}
			}

			// Find the notifications for this particular step
			if (notificationTriggerId != null)
			{
				INotificationTrigger? trigger = await GetNotificationTriggerAsync(notificationTriggerId, fireTrigger, cancellationToken);
				if (trigger != null)
				{
					foreach (INotificationSubscription subscription in trigger.Subscriptions)
					{
						if (subscription.Email)
						{
							// TODO?
						}
						if (subscription.Slack)
						{
							userIds.Add(subscription.UserId);
						}
					}
				}
			}
			return await _userCollection.FindUsersAsync(userIds, cancellationToken: cancellationToken);
		}

		private async Task SendJobStepNotificationsAsync(IJob job, JobStepBatchId batchId, JobStepId stepId, CancellationToken cancellationToken)
		{
			using IDisposable? scope = _logger.BeginScope("Sending notifications for step {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);

			IJobStepBatch? batch;
			if (!job.TryGetBatch(batchId, out batch))
			{
				_logger.LogError("Unable to find batch {BatchId} in job {JobId}", batchId, job.Id);
				return;
			}

			IJobStep? step;
			if (!batch.TryGetStep(stepId, out step))
			{
				_logger.LogError("Unable to find step {StepId} in batch {JobId}:{BatchId}", stepId, job.Id, batchId);
				return;
			}

			IGraph jobGraph = await _graphCollection.GetAsync(job.GraphHash, cancellationToken);
			INode node = jobGraph.GetNode(new NodeRef(batch.GroupIdx, step.NodeIdx));

			// Find the notifications for this particular step
			EventRecord eventRecord = new StepCompleteEventRecord(job.StreamId, job.TemplateId, node.Name, step.Outcome);

			IReadOnlyList<IUser> usersToNotify = await GetUsersToNotifyAsync(eventRecord, step.NotificationTriggerId, true, cancellationToken);

			// If this is not a success notification and the author isn't in the list to notify, add them manually if this is the outcome has gotten worse.
			int failures = job.Batches.Sum(x => x.Steps.Count(y => y.Outcome == JobStepOutcome.Failure));
			int warnings = job.Batches.Sum(x => x.Steps.Count(y => y.Outcome == JobStepOutcome.Warnings));
			bool firstFailure = step.Outcome == JobStepOutcome.Failure && failures == 1;
			bool firstWarning = step.Outcome == JobStepOutcome.Warnings && failures == 0 && warnings == 1;
			if (job.StartedByUserId.HasValue && !usersToNotify.Any(x => x.Id == job.StartedByUserId))
			{
				if (firstFailure || firstWarning)
				{					
					IUser? authorUser = await _userCollection.GetUserAsync(job.StartedByUserId.Value, cancellationToken);
					if (authorUser != null)
					{
						_logger.LogInformation("Author {AuthorUserId} is not in notify list but step outcome is {JobStepOutcome}, adding them to the list...", job.StartedByUserId, step.Outcome);
						List<IUser> newUsersToNotify = usersToNotify.ToList();
						newUsersToNotify.Add(authorUser);
						usersToNotify = newUsersToNotify;
					}
					else
					{
						_logger.LogWarning("Author {AuthorUserId} is not in notify list, step outcome is {JobStepOutcome}, though unable to find in user collection", job.StartedByUserId, step.Outcome);
					}
				}
				else
				{
					_logger.LogInformation("Author {AuthorUserId} is not in notify list, step outcome is {JobStepOutcome}, not adding to list with failure count: {JobStepFailures} and warning count: {JobStepWarnings}...", job.StartedByUserId, step.Outcome, failures, warnings);
				}
			}

			List<IUser> filteredUsersToNotify = new List<IUser>();
			foreach (IUser slackUser in usersToNotify)
			{
				if (job.PreflightCommitId != null)
				{
					if (slackUser.Id != job.StartedByUserId)
					{
						continue;
					}
				}
				filteredUsersToNotify.Add(slackUser);
			}

			// Fetch our event data which is mandatory for a completed step, but optional for canceled steps
			ILog? log = null;
			List<ILogEventData> jobStepEventData = new List<ILogEventData>();
			if (step.LogId is not null)
			{
				log = await _logCollection.GetAsync(step.LogId.Value, cancellationToken);
				if (log is not null)
				{
					List<ILogAnchor> jobStepEvents = await log.GetAnchorsAsync(cancellationToken: cancellationToken);
					foreach (ILogAnchor logEvent in jobStepEvents)
					{
						ILogEventData eventData = await logEvent.GetDataAsync(cancellationToken);
						jobStepEventData.Add(eventData);
					}
				}
			}

			// Treat a canceled job step separately from a completed step as we may have different logic for each
			if (step.State == JobStepState.Aborted)
			{
				if (filteredUsersToNotify.Count == 0)
				{
					_logger.LogInformation("No users to notify for step {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);
				}
				else
				{
					EnqueueTasks((sink, ctx) => sink.NotifyJobStepAbortedAsync(filteredUsersToNotify, job, batch, step, node, jobStepEventData, ctx));
				}

				if (job.PreflightCommitId is null)
				{
					EnqueueTasks((sink, ctx) => sink.NotifyJobStepAbortedAsync(null, job, batch, step, node, jobStepEventData, ctx));
				}
			}
			else
			{
				if (step.LogId == null)
				{
					_logger.LogError("Step does not have a log file");
					return;
				}

				if (log == null)
				{
					_logger.LogError("Step does not have a log file");
					return;
				}

				if (filteredUsersToNotify.Count == 0)
				{
					_logger.LogInformation("No users to notify for step {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);
				}
				else
				{
					EnqueueTasks((sink, ctx) => sink.NotifyJobStepCompleteAsync(filteredUsersToNotify, job, batch, step, node, jobStepEventData, ctx));
				}

				if (job.PreflightCommitId is null)
				{
					EnqueueTasks((sink, ctx) => sink.NotifyJobStepCompleteAsync(null, job, batch, step, node, jobStepEventData, ctx));
				}
			}

			_logger.LogDebug("Finished sending notifications for step {JobId}:{BatchId}:{StepId}", job.Id, batchId, stepId);
		}

		private async Task SendAllLabelNotificationsAsync(IJob job, IReadOnlyList<(LabelState State, LabelOutcome Outcome)> oldLabelStates, IReadOnlyList<(LabelState, LabelOutcome)> newLabelStates, CancellationToken cancellationToken)
		{
			IGraph? graph = await _graphCollection.GetAsync(job.GraphHash, cancellationToken);
			if (graph == null)
			{
				_logger.LogError("Unable to find graph {GraphHash} for job {JobId}", job.GraphHash, job.Id);
				return;
			}

			IReadOnlyDictionary<NodeRef, IJobStep> stepForNode = job.GetStepForNodeMap();
			for (int labelIdx = 0; labelIdx < graph.Labels.Count; ++labelIdx)
			{
				(LabelState State, LabelOutcome Outcome) oldLabel = oldLabelStates[labelIdx];
				(LabelState State, LabelOutcome Outcome) newLabel = newLabelStates[labelIdx];
				if (oldLabel != newLabel)
				{
					// If the state transitioned from Unspecified to Running, don't update unless the outcome also changed.
					if (oldLabel.State == LabelState.Unspecified && newLabel.State == LabelState.Running && oldLabel.Outcome == newLabel.Outcome)
					{
						continue;
					}

					// If the label isn't complete, don't report on outcome changing to success, this will be reported when the label state becomes complete.
					if (newLabel.State != LabelState.Complete && newLabel.Outcome == LabelOutcome.Success)
					{
						return;
					}

					ILabel label = graph.Labels[labelIdx];

					EventRecord? eventId;
					if (String.IsNullOrEmpty(label.DashboardName))
					{
						eventId = null;
					}
					else
					{
						eventId = new LabelCompleteEventRecord(job.StreamId, job.TemplateId, label.DashboardCategory, label.DashboardName, newLabel.Outcome);
					}

					NotificationTriggerId? triggerId;
					if (job.LabelIdxToTriggerId.TryGetValue(labelIdx, out NotificationTriggerId newTriggerId))
					{
						triggerId = newTriggerId;
					}
					else
					{
						triggerId = null;
					}

					bool fireTrigger = newLabel.State == LabelState.Complete;

					IReadOnlyList<IUser> usersToNotify = await GetUsersToNotifyAsync(eventId, triggerId, fireTrigger, cancellationToken);

					// filter preflight label notifications to only include initiator
					if (usersToNotify.Count > 0 && job.PreflightCommitId != null && job.StartedByUserId != null)
					{
						usersToNotify = usersToNotify.Where(x => x.Id == job.StartedByUserId).ToList();
					}

					if (usersToNotify.Count > 0)
					{
						SendLabelUpdateNotifications(job, graph, stepForNode, graph.Labels[labelIdx], labelIdx, newLabel.Outcome, usersToNotify);
					}
					else
					{
						_logger.LogDebug("No users to notify for label {DashboardName}/{UgsName} in job {JobId}", graph.Labels[labelIdx].DashboardName, graph.Labels[labelIdx].UgsName, job.Id);
					}
				}
			}
		}

		private void SendLabelUpdateNotifications(IJob job, IGraph graph, IReadOnlyDictionary<NodeRef, IJobStep> stepForNode, ILabel label, int labelIdx, LabelOutcome outcome, IReadOnlyList<IUser> slackUsers)
		{
			List<(string, JobStepOutcome, Uri)> stepData = new List<(string, JobStepOutcome, Uri)>();
			if (outcome != LabelOutcome.Success)
			{
				foreach (NodeRef includedNodeRef in label.IncludedNodes)
				{
					INode includedNode = graph.GetNode(includedNodeRef);
					IJobStep includedStep = stepForNode[includedNodeRef];
					stepData.Add((includedNode.Name, includedStep.Outcome, new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}?step={includedStep.Id}")));
				}
			}

			foreach (IUser slackUser in slackUsers)
			{
				EnqueueTasks((sink, ctx) => sink.NotifyLabelCompleteAsync(slackUser, job, label, labelIdx, outcome, stepData, ctx));
			}

			_logger.LogDebug("Finished sending label notifications for label {DashboardName}/{UgsName} in job {JobId}", label.DashboardName, label.UgsName, job.Id);
		}

		/// <inheritdoc/>
		public async Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken)
		{
			foreach (INotificationSink sink in _sinks)
			{
				try
				{
					await sink.SendAgentReportAsync(report, cancellationToken);
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed sending agent report to {Channel}", _settings.CurrentValue.AgentNotificationChannel);
				}
			}
		}

		/// <inheritdoc/>
		public async Task SendIssueReportAsync(IssueReportGroup report, CancellationToken cancellationToken)
		{
			foreach (INotificationSink sink in _sinks)
			{
				try
				{
					await sink.SendIssueReportAsync(report, cancellationToken);
				}
				catch (Exception e)
				{
					string streamsWithChannel = String.Join(", ", report.Reports.Select(x => $"{x.StreamId} {x.TriageChannel}"));
					_logger.LogError(e, "Failed sending issue report to {Channel} for streams/channels {StreamsWithChannels})", report.Channel, streamsWithChannel);
				}
			}
		}

		/// <inheritdoc/>
		/// <remarks>It will return the first non null output from the sinks ignoring the rest</remarks>
		public async Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken)
		{
			int sinksLeft = _sinks.Count;
			foreach (INotificationSink sink in _sinks)
			{
				sinksLeft--;
				try
				{
					string? url = await sink.GetDirectMessageLinkAsync(userIds, cancellationToken);
					if (url is not null)
					{
						if (sinksLeft > 0)
						{
							_logger.LogInformation("A DM link was generated from {Sink} ignoring the remaining {Count} sink(s)", sink.GetType().Name, sinksLeft);
						}
						return url;
					}
				}
				catch (Exception e)
				{
					string users = String.Join(", ", userIds.Select(x => x.ToString()));
					_logger.LogError(e, "Failed getting direct message link for users {Users}", users);
				}
			}

			return null;
		}

		/// <inheritdoc/>
		/// <remarks>It will return the first non null output from the sinks ignoring the rest</remarks>
		public async Task<string?> GetChannelLinkAsync(string channel, CancellationToken cancellationToken)
		{
			int sinksLeft = _sinks.Count;
			foreach (INotificationSink sink in _sinks)
			{
				sinksLeft--;
				try
				{
					string? url = await sink.GetChannelLinkAsync(channel, cancellationToken);
					if (url is not null)
					{
						if (sinksLeft > 0)
						{
							_logger.LogInformation("A Channel link was generated from {Sink} ignoring the remaining {Count} sink(s)", sink.GetType().Name, sinksLeft);
						}
						return url;
					}
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed getting channel link for channel {Channel}", channel);
				}
			}

			return null;
		}
	}
}
