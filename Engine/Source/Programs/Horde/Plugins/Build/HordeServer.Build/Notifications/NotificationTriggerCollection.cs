// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Notifications;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Collection of notification triggers
	/// </summary>
	public class NotificationTriggerCollection : INotificationTriggerCollection
	{
		class SubscriptionDocument : INotificationSubscription
		{
			[BsonIgnoreIfNull]
			public string? User { get; set; }

			public UserId UserId { get; set; }
			public bool Email { get; set; }
			public bool Slack { get; set; }

			[BsonConstructor]
			private SubscriptionDocument()
			{
				User = null!;
			}

			public SubscriptionDocument(UserId userId, bool email, bool slack)
			{
				UserId = userId;
				Email = email;
				Slack = slack;
			}

			public SubscriptionDocument(INotificationSubscription subscription)
			{
				UserId = subscription.UserId;
				Email = subscription.Email;
				Slack = subscription.Slack;
			}
		}

		class TriggerDocument : INotificationTrigger
		{
			public ObjectId Id { get; set; }
			public bool Fired { get; set; }
			public List<SubscriptionDocument> Subscriptions { get; set; } = new List<SubscriptionDocument>();
			public int UpdateIndex { get; set; }

			NotificationTriggerId INotificationTrigger.Id => new NotificationTriggerId(BinaryIdUtils.FromObjectId(Id));
			IReadOnlyList<INotificationSubscription> INotificationTrigger.Subscriptions => Subscriptions;
		}

		readonly IMongoCollection<TriggerDocument> _triggers;
		readonly IUserCollection _userCollection;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database singleton</param>
		/// <param name="userCollection"></param>
		public NotificationTriggerCollection(IMongoService mongoService, IUserCollection userCollection)
		{
			_triggers = mongoService.GetCollection<TriggerDocument>("NotificationTriggers");
			_userCollection = userCollection;
		}

		/// <inheritdoc/>
		public async Task<INotificationTrigger?> GetAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken)
		{
			ObjectId triggerObjectId = BinaryIdUtils.ToObjectId(triggerId.Id);

			TriggerDocument? trigger = await _triggers.Find(x => x.Id == triggerObjectId).FirstOrDefaultAsync(cancellationToken);
			if (trigger != null)
			{
				for (int idx = 0; idx < trigger.Subscriptions.Count; idx++)
				{
					SubscriptionDocument subscription = trigger.Subscriptions[idx];
					if (subscription.User != null)
					{
						IUser? user = await _userCollection.FindUserByLoginAsync(subscription.User, cancellationToken);
						if (user == null)
						{
							trigger.Subscriptions.RemoveAt(idx);
							idx--;
						}
						else
						{
							subscription.UserId = user.Id;
							subscription.User = null;
						}
					}
				}
			}
			return trigger;
		}

		/// <inheritdoc/>
		public async Task<INotificationTrigger> FindOrAddAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				// Find an existing trigger
				INotificationTrigger? existing = await GetAsync(triggerId, cancellationToken);
				if (existing != null)
				{
					return existing;
				}

				// Try to insert a new document
				try
				{
					TriggerDocument newDocument = new TriggerDocument();
					newDocument.Id = BinaryIdUtils.ToObjectId(triggerId.Id);
					await _triggers.InsertOneAsync(newDocument, (InsertOneOptions?)null, cancellationToken);
					return newDocument;
				}
				catch (MongoWriteException ex)
				{
					if (ex.WriteError.Category != ServerErrorCategory.DuplicateKey)
					{
						throw;
					}
				}
			}
		}

		/// <summary>
		/// Updates an existing document
		/// </summary>
		/// <param name="trigger">The trigger to update</param>
		/// <param name="transaction">The update definition</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated document</returns>
		async Task<INotificationTrigger?> TryUpdateAsync(INotificationTrigger trigger, TransactionBuilder<TriggerDocument> transaction, CancellationToken cancellationToken)
		{
			TriggerDocument document = (TriggerDocument)trigger;
			int nextUpdateIndex = document.UpdateIndex + 1;

			ObjectId triggerObjectId = BinaryIdUtils.ToObjectId(trigger.Id.Id);
			FilterDefinition<TriggerDocument> filter = Builders<TriggerDocument>.Filter.Expr(x => x.Id == triggerObjectId && x.UpdateIndex == document.UpdateIndex);
			UpdateDefinition<TriggerDocument> update = transaction.ToUpdateDefinition().Set(x => x.UpdateIndex, nextUpdateIndex);

			UpdateResult result = await _triggers.UpdateOneAsync(filter, update, null, cancellationToken);
			if (result.ModifiedCount > 0)
			{
				transaction.ApplyTo(document);
				document.UpdateIndex = nextUpdateIndex;
				return document;
			}

			return null;
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(NotificationTriggerId triggerId, CancellationToken cancellationToken)
		{
			ObjectId triggerObjectId = BinaryIdUtils.ToObjectId(triggerId.Id);
			await _triggers.DeleteOneAsync(x => x.Id == triggerObjectId, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(List<NotificationTriggerId> triggerIds, CancellationToken cancellationToken)
		{
			List<ObjectId> triggerObjectIds = triggerIds.ConvertAll(x => BinaryIdUtils.ToObjectId(x.Id));
			FilterDefinition<TriggerDocument> filter = Builders<TriggerDocument>.Filter.In(x => x.Id, triggerObjectIds);
			await _triggers.DeleteManyAsync(filter, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<INotificationTrigger?> FireAsync(INotificationTrigger trigger, CancellationToken cancellationToken)
		{
			if (trigger.Fired)
			{
				return null;
			}

			for (; ; )
			{
				TransactionBuilder<TriggerDocument> transaction = new TransactionBuilder<TriggerDocument>();
				transaction.Set(x => x.Fired, true);

				INotificationTrigger? newTrigger = await TryUpdateAsync(trigger, transaction, cancellationToken);
				if (newTrigger != null)
				{
					return newTrigger;
				}

				newTrigger = await FindOrAddAsync(trigger.Id, cancellationToken); // Need to add to prevent race condition on triggering vs adding
				if (newTrigger == null || newTrigger.Fired)
				{
					return null;
				}
			}
		}

		/// <inheritdoc/>
		public async Task<INotificationTrigger?> UpdateSubscriptionsAsync(INotificationTrigger trigger, UserId userId, bool? email, bool? slack, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				// If the trigger has already fired, don't add a new subscription to it
				if (trigger.Fired)
				{
					return trigger;
				}

				// Try to update the trigger
				List<SubscriptionDocument> newSubscriptions = new List<SubscriptionDocument>();
				newSubscriptions.AddRange(trigger.Subscriptions.Select(x => new SubscriptionDocument(x)));

				SubscriptionDocument? newSubscription = newSubscriptions.FirstOrDefault(x => x.UserId == userId);
				if (newSubscription == null)
				{
					newSubscription = new SubscriptionDocument(userId, email ?? false, slack ?? false);
					newSubscriptions.Add(newSubscription);
				}
				else
				{
					newSubscription.Email = email ?? newSubscription.Email;
					newSubscription.Slack = slack ?? newSubscription.Slack;
				}

				TransactionBuilder<TriggerDocument> transaction = new TransactionBuilder<TriggerDocument>();
				transaction.Set(x => x.Subscriptions, newSubscriptions);

				INotificationTrigger? newTrigger = await TryUpdateAsync(trigger, transaction, cancellationToken);
				if (newTrigger != null)
				{
					return newTrigger;
				}

				newTrigger = await GetAsync(trigger.Id, cancellationToken);
				if (newTrigger == null)
				{
					return null;
				}
			}
		}
	}
}
