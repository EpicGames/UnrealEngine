// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Globalization;
using System.Net.WebSockets;
using System.Text;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using EpicGames.Redis;
using EpicGames.Redis.Utility;
using EpicGames.Slack;
using EpicGames.Slack.Blocks;
using EpicGames.Slack.Elements;
using HordeServer.Agents;
using HordeServer.Commits;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Issues;
using HordeServer.Issues.External;
using HordeServer.Jobs.TestData;
using HordeServer.Logs;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Users;
using HordeServer.Utilities;
using HordeServer.VersionControl.Perforce;
using Microsoft.AspNetCore.Hosting;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using StackExchange.Redis;

using HordeServer.RCA;

namespace HordeServer.Notifications.Sinks
{
	/// <summary>
	/// Maintains a connection to Slack, in order to receive socket-mode notifications of user interactions
	/// </summary>
	public sealed class SlackNotificationSink : IHostedService, INotificationSink, IAvatarService, IRcaNotifier, IAsyncDisposable
	{
		const bool DefaultAllowMentions = true;

		const int MaxLineLength = 2048;

		// The color to use for error messages.
		const string ErrorColor = "#ec4c47";

		// The color to use for warning messages.
		const string WarningColor = "#f7d154";

		// The color to use for success messages.
		const string SuccessColor = "#4db507";

		// Max attempts when looking up an existing triage thread for auto-RCA posting.
		const int RcaThreadLookupMaxAttempts = 4;

		// Base delay in seconds for exponential backoff between thread lookup retries (5s, 10s, 20s).
		const int RcaThreadLookupBaseDelaySecs = 5;

		// Badges to add color accents
		const string YellowgBadge = ":large_yellow_square:";
		const string OrangeBadge = ":large_orange_square:";
		const string RedBadge = ":large_red_square:";
		const string BlueBadge = ":large_blue_square:";
		const string BlackBadge = ":black_square:";

		class SocketResponse
		{
			[JsonPropertyName("ok")]
			public bool Ok { get; set; }

			[JsonPropertyName("url")]
			public Uri? Url { get; set; }
		}

		class EventMessage
		{
			[JsonPropertyName("envelope_id")]
			public string? EnvelopeId { get; set; }

			[JsonPropertyName("type")]
			public string? Type { get; set; }

			[JsonPropertyName("payload")]
			public EventPayload? Payload { get; set; }
		}

		class EventResponse
		{
			[JsonPropertyName("envelope_id")]
			public string? EnvelopeId { get; set; }

			[JsonPropertyName("payload")]
			public object? Payload { get; set; }
		}

		class EventPayload
		{
			[JsonPropertyName("type")]
			public string? Type { get; set; }

			[JsonPropertyName("trigger_id")]
			public string? TriggerId { get; set; }

			[JsonPropertyName("user")]
			public SlackUser? User { get; set; }

			[JsonPropertyName("response_url")]
			public string? ResponseUrl { get; set; }

			[JsonPropertyName("actions")]
			public List<ActionInfo> Actions { get; set; } = new List<ActionInfo>();

			[JsonPropertyName("view")]
			public SlackView? View { get; set; }
		}

		class ActionInfo
		{
			[JsonPropertyName("blockId")]
			public string BlockId { get; set; } = String.Empty;

			[JsonPropertyName("action_id")]
			public string ActionId { get; set; } = String.Empty;

			[JsonPropertyName("value")]
			public string Value { get; set; } = String.Empty;
		}

		class MessageStateDocument
		{
			[BsonId]
			public ObjectId Id { get; set; }

			[BsonElement("uid")]
			public string Recipient { get; set; } = String.Empty;

			[BsonElement("usr")]
			public UserId? UserId { get; set; }

			[BsonElement("eid")]
			public string EventId { get; set; } = String.Empty;

			[BsonElement("ch")]
			public string Channel { get; set; } = String.Empty;

			[BsonElement("ts")]
			public string Ts { get; set; } = String.Empty;

			[BsonElement("dig")]
			public string Digest { get; set; } = String.Empty;

			[BsonElement("lnk"), BsonIgnoreIfNull]
			public string? Permalink { get; set; }

			[BsonIgnore]
			public SlackMessageId MessageId => new SlackMessageId(Channel, null, Ts);
		}

		class SlackUserDocument : IAvatar
		{
			public const int CurrentVersion = 2;

			public UserId Id { get; set; }

			[BsonElement("u")]
			public string? SlackUserId { get; set; }

			[BsonElement("i24")]
			public string? Image24 { get; set; }

			[BsonElement("i32")]
			public string? Image32 { get; set; }

			[BsonElement("i48")]
			public string? Image48 { get; set; }

			[BsonElement("i72")]
			public string? Image72 { get; set; }

			[BsonElement("t")]
			public DateTime Time { get; set; }

			[BsonElement("v")]
			public int Version { get; set; }

			private SlackUserDocument()
			{
			}

			public SlackUserDocument(UserId id, SlackUser? info)
			{
				Id = id;
				SlackUserId = info?.Id;
				if (info != null && info.Profile != null && info.Profile.IsCustomImage)
				{
					Image24 = info.Profile.Image24;
					Image32 = info.Profile.Image32;
					Image48 = info.Profile.Image48;
					Image72 = info.Profile.Image72;
				}
				Time = DateTime.UtcNow;
				Version = CurrentVersion;
			}
		}

		readonly IRedisService _redisService;
		readonly IssueService _issueService;
		readonly ICommitService _commitService;
		readonly IUserCollection _userCollection;
		readonly ILogCollection _logCollection;
		readonly IWebHostEnvironment _environment;
		readonly BuildServerConfig _settings;
		readonly IMongoCollection<MessageStateDocument> _messageStates;
		readonly IMongoCollection<SlackUserDocument> _slackUsers;
		readonly HashSet<string>? _allowUsers;
		readonly IExternalIssueService _externalIssueService;
		readonly JsonSerializerOptions _jsonSerializerOptions;
		readonly ITicker _escalateTicker;
		static readonly RedisSortedSetKey<int> s_escalateIssues = "slack/escalate";
		static readonly RedisSortedSetKey<int> s_escalateRootCauseAnalysis = "slack/escalate-rca";
		readonly BackgroundTask _backgroundTask;
		readonly IServerInfo _serverInfo;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger _logger;

		readonly ITicker _issueQueueTicker;
		static readonly RedisListKey<int> s_redisIssueQueue = "slack/issue-queue";
		readonly string _redisIssueLockPrefix;

		readonly SlackClient _slackClient;
		readonly SlackClient? _adminSlackClient;

		/// <summary>
		/// Map of email address to Slack user ID.
		/// </summary>
		private readonly MemoryCache _userCache = new MemoryCache(new MemoryCacheOptions());

		/// <summary>
		/// Map of Slack user IDs to DM channel.
		/// </summary>
		private readonly MemoryCache _dmChannelCache = new MemoryCache(new MemoryCacheOptions() { SizeLimit = 1024, ExpirationScanFrequency = TimeSpan.FromMinutes(10) });

		/// <summary>
		/// Constructor
		/// </summary>
		public SlackNotificationSink(IMongoService mongoService, IRedisService redisService, IssueService issueService, ICommitService commitService, IUserCollection userCollection, ILogCollection logCollection, IExternalIssueService externalIssueService, IWebHostEnvironment environment, IOptions<BuildServerConfig> settings, IClock clock, IServerInfo serverInfo, IOptionsMonitor<BuildConfig> buildConfig, ILogger<SlackNotificationSink> logger)
		{
			_redisService = redisService;
			_issueService = issueService;
			_commitService = commitService;
			_userCollection = userCollection;
			_logCollection = logCollection;
			_externalIssueService = externalIssueService;
			_environment = environment;
			_settings = settings.Value;
			_messageStates = mongoService.GetCollection<MessageStateDocument>("SlackV2", keys => keys.Ascending(x => x.Recipient).Ascending(x => x.EventId), unique: true);
			_slackUsers = mongoService.GetCollection<SlackUserDocument>("Slack.UsersV2");
			_backgroundTask = new BackgroundTask(ExecuteAsync);
			_serverInfo = serverInfo;
			_buildConfig = buildConfig;
			_logger = logger;

			_escalateTicker = clock.AddSharedTicker<SlackNotificationSink>(TimeSpan.FromMinutes(1.0), EscalateAsync, _logger);

			_issueQueueTicker = clock.AddSharedTicker("slack-issues", TimeSpan.FromMinutes(1.0), ProcessIssueQueueAsync, _logger);
			_redisIssueLockPrefix = "slack/issues/";

			_slackClient = SlackClient.Create(_settings.SlackToken, _logger);

			if (!String.IsNullOrEmpty(_settings.SlackAdminToken))
			{
				_adminSlackClient = SlackClient.Create(_settings.SlackAdminToken, _logger);
			}

			_jsonSerializerOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonSerializerOptions);

			if (!String.IsNullOrEmpty(settings.Value.SlackUsers))
			{
				_allowUsers = new HashSet<string>(settings.Value.SlackUsers.Split(','), StringComparer.OrdinalIgnoreCase);
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
			_userCache.Dispose();
			_dmChannelCache.Dispose();
			_slackClient.Dispose();
			_adminSlackClient?.Dispose();
			await _issueQueueTicker.DisposeAsync();
			await _escalateTicker.DisposeAsync();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_backgroundTask.Start();
			await _issueQueueTicker.StartAsync();
			await _escalateTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _escalateTicker.StopAsync();
			await _issueQueueTicker.StopAsync();
			await _backgroundTask.StopAsync(cancellationToken);
		}

		#region Avatars

		/// <inheritdoc/>
		public async Task<IAvatar?> GetAvatarAsync(IUser user, CancellationToken cancellationToken)
		{
			return await GetSlackUserAsync(user, cancellationToken);
		}

		#endregion

		#region Message state 

		async Task<(MessageStateDocument, bool)> AddOrUpdateMessageStateAsync(string recipient, string eventId, UserId? userId, string digest, SlackMessageId? messageId, CancellationToken cancellationToken)
		{
			ObjectId newId = ObjectId.GenerateNewId();

			FilterDefinition<MessageStateDocument> filter = Builders<MessageStateDocument>.Filter.Eq(x => x.Recipient, recipient) & Builders<MessageStateDocument>.Filter.Eq(x => x.EventId, eventId);
			UpdateDefinition<MessageStateDocument> update = Builders<MessageStateDocument>.Update.SetOnInsert(x => x.Id, newId).Set(x => x.UserId, userId).Set(x => x.Digest, digest);

			if (messageId != null)
			{
				update = update.Set(x => x.Channel, messageId.Channel).Set(x => x.Ts, messageId.Ts);
			}

			MessageStateDocument state = await _messageStates.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<MessageStateDocument> { IsUpsert = true, ReturnDocument = ReturnDocument.After }, cancellationToken);
			if (state.Id == newId)
			{
				_logger.LogInformation("Posted message {StateId} (recipient: {Recipient}, user: {UserId}, event: {EventId}, messageId: {MessageId}, digest: {Digest})", state.Id, recipient, userId ?? UserId.Empty, eventId, state.MessageId, digest);
			}
			else
			{
				_logger.LogInformation("Updated message {StateId} (recipient: {Recipient}, user: {UserId}, event: {EventId}, messageId: {MessageId}, digest: {Digest})", state.Id, recipient, userId ?? UserId.Empty, eventId, state.MessageId, digest);
			}

			return (state, state.Id == newId);
		}

		async Task<MessageStateDocument?> GetMessageStateAsync(string recipient, string eventId, CancellationToken cancellationToken)
		{
			FilterDefinition<MessageStateDocument> filter = Builders<MessageStateDocument>.Filter.Eq(x => x.Recipient, recipient) & Builders<MessageStateDocument>.Filter.Eq(x => x.EventId, eventId);
			return await _messageStates.Find(filter).FirstOrDefaultAsync(cancellationToken);
		}

		async Task<bool> DeleteMessageStateAsync(string recipient, string eventId, CancellationToken cancellationToken)
		{
			FilterDefinition<MessageStateDocument> filter = Builders<MessageStateDocument>.Filter.Eq(x => x.Recipient, recipient) & Builders<MessageStateDocument>.Filter.Eq(x => x.EventId, eventId);
			DeleteResult result = await _messageStates.DeleteOneAsync(filter, cancellationToken);
			return result.DeletedCount > 0;
		}

		async Task UpdateMessageStateAsync(ObjectId stateId, SlackMessageId id, string? permalink = null, CancellationToken cancellationToken = default)
		{
			FilterDefinition<MessageStateDocument> filter = Builders<MessageStateDocument>.Filter.Eq(x => x.Id, stateId);
			UpdateDefinition<MessageStateDocument> update = Builders<MessageStateDocument>.Update.Set(x => x.Channel, id.Channel).Set(x => x.Ts, id.Ts);

			if (permalink == null)
			{
				update = update.Unset(x => x.Permalink);
			}
			else
			{
				update = update.Set(x => x.Permalink, permalink);
			}

			await _messageStates.FindOneAndUpdateAsync(filter, update, cancellationToken: cancellationToken);
			_logger.LogInformation("Updated message {StateId} (messageId: {MessageId}, permalink: {Permalink})", stateId, id, permalink ?? "(n/a)");
		}

		#endregion

		/// <inheritdoc/>
		public async Task NotifyJobScheduledAsync(List<JobScheduledNotification> notifications, CancellationToken cancellationToken)
		{
			if (_settings.JobNotificationChannel != null)
			{
				string jobIds = String.Join(", ", notifications.Select(x => x.JobId));
				_logger.LogInformation("Sending Slack notification for scheduled job IDs {JobIds} to channel {SlackChannel}", jobIds, _settings.JobNotificationChannel);
				foreach (string channel in _settings.JobNotificationChannel.Split(';'))
				{
					await SendJobScheduledOnEmptyAutoScaledPoolMessageAsync($"#{channel}", notifications, cancellationToken);
				}
			}
		}

		private async Task SendJobScheduledOnEmptyAutoScaledPoolMessageAsync(string recipient, List<JobScheduledNotification> notifications, CancellationToken cancellationToken)
		{
			const int MaxItems = 10;
			string jobIds = StringUtils.FormatList(notifications.Select(x => x.JobId.ToString()).ToArray(), MaxItems);

			StringBuilder sb = new();
			for (int idx = 0; idx < notifications.Count; idx++)
			{
				if (idx >= MaxItems && notifications.Count > MaxItems + 2)
				{
					sb.AppendLine($"...and {notifications.Count - idx} others.");
					break;
				}

				JobScheduledNotification notification = notifications[idx];
				string jobUrl = new Uri(_serverInfo.DashboardUrl, $"job/{notification.JobId}").ToString();
				sb.AppendLine($"Job `{notification.JobName}` with ID <{jobUrl}|{notification.JobId}> in pool `{notification.PoolName}`");
			}

			string outcomeColor = WarningColor;
			SlackAttachment attachment = new SlackAttachment();
			attachment.Color = outcomeColor;
			attachment.FallbackText = $"Job(s) scheduled in an auto-scaled pool with no agents online. Job IDs {jobIds}";

			attachment.AddHeader($"Jobs scheduled in empty pool", true);
			attachment.AddSection($"One or more jobs were scheduled in an auto-scaled pool but with no current agents online.");
			if (sb.Length > 0)
			{
				attachment.AddSection(sb.ToString());
			}

			await SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#region Job Complete

		/// <inheritdoc/>
		public async Task NotifyJobCompleteAsync(IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.CurrentValue.TryGetStream(job.StreamId, out streamConfig))
			{
				return;
			}
			if (job.NotificationChannel != null)
			{
				await SendJobCompleteNotificationToChannelAsync(job.NotificationChannel, job.NotificationChannelFilter, streamConfig, job, graph, outcome, cancellationToken);
			}
			if (streamConfig.NotificationChannel != null)
			{
				await SendJobCompleteNotificationToChannelAsync(streamConfig.NotificationChannel, streamConfig.NotificationChannelFilter, streamConfig, job, graph, outcome, cancellationToken);
			}
		}

		async Task SendJobCompleteNotificationToChannelAsync(string notificationChannel, string? notificationFilter, StreamConfig streamConfig, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			if (notificationFilter != null)
			{
				List<LabelOutcome> outcomes = new List<LabelOutcome>();
				foreach (string filterOption in notificationFilter.Split('|'))
				{
					LabelOutcome result;
					if (Enum.TryParse(filterOption, out result))
					{
						outcomes.Add(result);
					}
					else
					{
						_logger.LogWarning("Invalid filter option {Option} specified in job filter {NotificationChannelFilter} in job {JobId} or stream {StreamId}", filterOption, notificationFilter, job.Id, job.StreamId);
					}
				}
				if (!outcomes.Contains(outcome))
				{
					return;
				}
			}
			foreach (string channel in notificationChannel.Split(';'))
			{
				await SendJobCompleteMessageAsync(channel, streamConfig, job, graph, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task NotifyJobCompleteAsync(IUser slackUser, IJob job, IGraph graph, LabelOutcome outcome, CancellationToken cancellationToken)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.CurrentValue.TryGetStream(job.StreamId, out streamConfig))
			{
				return;
			}

			string? slackUserId = await GetSlackUserIdAsync(slackUser, cancellationToken);
			if (slackUserId != null && slackUser.Id != job.AbortedByUserId)
			{
				await SendJobCompleteMessageAsync(slackUserId, streamConfig, job, graph, cancellationToken);
			}
		}

		private async Task SendJobCompleteMessageAsync(string recipient, StreamConfig streamConfig, IJob job, IGraph graph, CancellationToken cancellationToken)
		{
			JobStepOutcome jobOutcome = job.Batches.SelectMany(x => x.Steps).Min(x => x.Outcome);
			if (job.AbortedByUserId != null)
			{
				jobOutcome = JobStepOutcome.Failure;
			}

			_logger.LogInformation("Sending Slack notification for job {JobId} outcome {Outcome} to {SlackUser}", job.Id, jobOutcome, recipient);

			Uri jobLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}");

			string outcomeColor = jobOutcome == JobStepOutcome.Failure ? ErrorColor : jobOutcome == JobStepOutcome.Warnings ? WarningColor : SuccessColor;

			SlackAttachment attachment = new SlackAttachment();
			attachment.FallbackText = $"{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {jobOutcome}";
			attachment.Color = outcomeColor;
			attachment.AddSection($"*<{jobLink}|{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name}>*");

			if (!String.IsNullOrEmpty(job.PreflightDescription))
			{
				string description = job.PreflightDescription.TrimEnd();
				if (jobOutcome == JobStepOutcome.Success)
				{
					description += $"\n#preflight {job.Id}";
				}
				attachment.AddSection($"```{description}```");
			}

			if (job.AbortedByUserId != null)
			{
				attachment.AddSection($"*Job cancelled by {await FormatMentionAsync(job.AbortedByUserId.Value, true, cancellationToken)}*");
			}
			else if (jobOutcome == JobStepOutcome.Success)
			{
				attachment.AddSection($"*Job Succeeded*");
			}
			else
			{
				List<string> failedStepStrings = new List<string>();
				List<string> warningStepStrings = new List<string>();

				IReadOnlyDictionary<NodeRef, IJobStep> nodeToStep = job.GetStepForNodeMap();
				foreach ((NodeRef nodeRef, IJobStep step) in nodeToStep)
				{
					if (step.State == JobStepState.Completed)
					{
						INode stepNode = graph.GetNode(nodeRef);
						string stepName = $"<{jobLink}?step={step.Id}|{stepNode.Name}>";
						if (step.Outcome == JobStepOutcome.Failure)
						{
							failedStepStrings.Add(stepName);
						}
						else if (step.Outcome == JobStepOutcome.Warnings)
						{
							warningStepStrings.Add(stepName);
						}
					}
				}

				if (failedStepStrings.Any())
				{
					string msg = $"*Errors*\n{String.Join(", ", failedStepStrings)}";
					attachment.AddSection(msg.Substring(0, Math.Min(msg.Length, 3000)));
				}
				else if (warningStepStrings.Any())
				{
					string msg = $"*Warnings*\n{String.Join(", ", warningStepStrings)}";
					attachment.AddSection(msg.Substring(0, Math.Min(msg.Length, 3000)));
				}
			}

			if (job.AutoSubmit)
			{
				attachment.AddDivider();
				if (job.AutoSubmitChange != null)
				{
					attachment.AddSection($"Shelved files were submitted in CL {job.AutoSubmitChange}.");
				}
				else
				{
					attachment.Color = WarningColor;

					string autoSubmitMessage = String.Empty;
					if (!String.IsNullOrEmpty(job.AutoSubmitMessage))
					{
						autoSubmitMessage = $"\n\n```{job.AutoSubmitMessage}```";
					}

					attachment.AddSection($"Files in CL *{job.PreflightCommitId}* were *not submitted*. Please resolve these issues and submit manually.{autoSubmitMessage}");
				}
			}

			await SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#endregion

		#region Job step complete

		/// <inheritdoc/>
		public Task NotifyJobStepAbortedAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
			=> Task.CompletedTask;

		/// <inheritdoc/>
		public async Task NotifyJobStepCompleteAsync(IEnumerable<IUser>? usersToNotify, IJob job, IJobStepBatch batch, IJobStep step, INode node, List<ILogEventData> jobStepEventData, CancellationToken cancellationToken)
		{
			// The Slack notification requires the list of user's to notify to be non-null
			if (usersToNotify is null)
			{
				return;
			}

			if (step.Error == JobStepError.TimedOut)
			{
				if (_settings.JobNotificationChannel != null)
				{
					BuildConfig buildConfig = _buildConfig.CurrentValue;
					if (buildConfig.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
					{
						foreach (string channel in _settings.JobNotificationChannel.Split(';'))
						{
							Uri jobStepLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}?step={step.Id}");

							SlackAttachment attachment = new SlackAttachment();
							attachment.FallbackText = $"{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name} - {step.Outcome}";
							attachment.Color = ErrorColor;
							attachment.AddSection($"*<{jobStepLink}|{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name}>*");
							attachment.AddSection($"*Job Step Aborted due to Timeout*");

							await SendMessageAsync($"#{channel}", attachment, cancellationToken);
						}
					}
				}
			}

			foreach (IUser slackUser in usersToNotify)
			{
				_logger.LogInformation("Sending Slack notification for job {JobId}, batch {BatchId}, step {StepId}, outcome {Outcome} to {SlackUser} ({UserId})", job.Id, batch.Id, step.Id, step.Outcome, slackUser.Name, slackUser.Id);

				string? slackUserId = await GetSlackUserIdAsync(slackUser, cancellationToken);
				if (slackUserId != null)
				{
					await SendJobStepCompleteMessageAsync(slackUserId, job, step, node, jobStepEventData, cancellationToken);
				}
			}
		}

		/// <summary>
		/// Creates a Slack message about a completed step job.
		/// </summary>
		/// <param name="recipient"></param>
		/// <param name="job">The job that contains the step that completed.</param>
		/// <param name="step">The job step that completed.</param>
		/// <param name="node">The node for the job step.</param>
		/// <param name="events">Any events that occurred during the job step.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		private Task SendJobStepCompleteMessageAsync(string recipient, IJob job, IJobStep step, INode node, List<ILogEventData> events, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			if (!buildConfig.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				return Task.CompletedTask;
			}

			Uri jobStepLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}?step={step.Id}");
			Uri jobStepLogLink = new Uri($"{_serverInfo.DashboardUrl}log/{step.LogId}");

			string outcomeColor = step.Outcome == JobStepOutcome.Failure ? ErrorColor : step.Outcome == JobStepOutcome.Warnings ? WarningColor : SuccessColor;

			SlackAttachment attachment = new SlackAttachment();
			attachment.FallbackText = $"{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name} - {step.Outcome}";
			attachment.Color = outcomeColor;
			attachment.AddSection($"*<{jobStepLink}|{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name}>*");
			if (step.Outcome == JobStepOutcome.Success)
			{
				attachment.AddSection($"*Job Step Succeeded*");
			}
			else
			{
				List<ILogEventData> errors = events.Where(x => x.Severity == LogEventSeverity.Error).ToList();
				List<ILogEventData> warnings = events.Where(x => x.Severity == LogEventSeverity.Warning).ToList();
				List<string> eventStrings = new List<string>();
				if (errors.Any())
				{
					string errorSummary = errors.Count > MaxJobStepEvents ? $"*Errors (First {MaxJobStepEvents} shown)*" : $"*Errors*";
					attachment.AddSection(errorSummary);
					foreach (ILogEventData error in errors.Take(MaxJobStepEvents))
					{
						if (!String.IsNullOrWhiteSpace(error.Message))
						{
							attachment.AddSection(QuoteText(error.Message));
						}
					}
				}
				else if (warnings.Any())
				{
					string warningSummary = warnings.Count > MaxJobStepEvents ? $"*Warnings (First {MaxJobStepEvents} shown)*" : $"*Warnings*";
					eventStrings.Add(warningSummary);
					foreach (ILogEventData warning in warnings.Take(MaxJobStepEvents))
					{
						if (!String.IsNullOrWhiteSpace(warning.Message))
						{
							attachment.AddSection(QuoteText(warning.Message));
						}
					}
				}

				attachment.AddSection($"<{jobStepLogLink}|View Job Step Log>");
			}

			return SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#endregion

		#region Label complete

		/// <inheritdoc/>
		public async Task NotifyLabelCompleteAsync(IUser user, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> stepData, CancellationToken cancellationToken)
		{
			if (!_buildConfig.CurrentValue.TryGetStream(job.StreamId, out StreamConfig? streamConfig))
			{
				return;
			}

			_logger.LogInformation("Sending Slack notification for job {JobId} outcome {Outcome} to {Name} ({UserId})", job.Id, outcome, user.Name, user.Id);

			string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
			if (slackUserId != null)
			{
				await SendLabelUpdateMessageAsync(slackUserId, streamConfig, job, label, labelIdx, outcome, stepData, cancellationToken);
			}
		}

		Task SendLabelUpdateMessageAsync(string recipient, StreamConfig streamConfig, IJob job, ILabel label, int labelIdx, LabelOutcome outcome, List<(string, JobStepOutcome, Uri)> jobStepData, CancellationToken cancellationToken)
		{
			Uri labelLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}?label={labelIdx}");

			string outcomeColor = outcome == LabelOutcome.Failure ? ErrorColor : outcome == LabelOutcome.Warnings ? WarningColor : SuccessColor;

			SlackAttachment attachment = new SlackAttachment();
			attachment.FallbackText = $"{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - Label {label.DashboardName} - {outcome}";
			attachment.Color = outcomeColor;
			attachment.AddSection($"*<{labelLink}|{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - Label {label.DashboardName}>*");
			if (outcome == LabelOutcome.Success)
			{
				attachment.AddSection($"*Label Succeeded*");
			}
			else
			{
				List<string> failedStepStrings = new List<string>();
				List<string> warningStepStrings = new List<string>();
				foreach ((string Name, JobStepOutcome StepOutcome, Uri Link) jobStep in jobStepData)
				{
					string stepString = $"<{jobStep.Link}|{jobStep.Name}>";
					if (jobStep.StepOutcome == JobStepOutcome.Failure)
					{
						failedStepStrings.Add(stepString);
					}
					else if (jobStep.StepOutcome == JobStepOutcome.Warnings)
					{
						warningStepStrings.Add(stepString);
					}
				}

				if (failedStepStrings.Any())
				{
					attachment.AddSection($"*Errors*\n{String.Join(", ", failedStepStrings)}");
				}
				else if (warningStepStrings.Any())
				{
					attachment.AddSection($"*Warnings*\n{String.Join(", ", warningStepStrings)}");
				}
			}

			return SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#endregion

		#region Issues

		/// <inheritdoc/>
		public async Task NotifyIssueUpdatedAsync(IIssue issue, CancellationToken cancellationToken)
		{
			// Do not send notifications for quarantined issues
			if (issue.QuarantinedByUserId != null)
			{
				_logger.LogInformation("Skipping notifications for quarantined issue {IssueId}", issue.Id);
				return;
			}

			// Otherwise add it to the redis queue, and attempt to process the queue immediately.
			await _redisService.GetDatabase().ListRightPushAsync(s_redisIssueQueue, issue.Id);
			await ProcessIssueQueueAsync(CancellationToken.None);
		}

		/// <summary>
		/// Processes the issue queue
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		async ValueTask ProcessIssueQueueAsync(CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			HashSet<int> testedIssueIds = new HashSet<int>();

			// Execute loop number of times based on the length of the queue at the start. This should bound the number of iterations while allowing us
			// to re-queue items that we're unable to process now, without having to track whether we've reached the end of the list in its original state.
			long count = await _redisService.GetDatabase().ListLengthAsync(s_redisIssueQueue);
			for (; count > 0; count--)
			{
				int issueId = await _redisService.GetDatabase().ListLeftPopAsync(s_redisIssueQueue);
				if (!testedIssueIds.Add(issueId) || !await TryUpdateIssueAsync(buildConfig, issueId, cancellationToken))
				{
					await _redisService.GetDatabase().ListRightPushAsync(s_redisIssueQueue, issueId);
				}
				cancellationToken.ThrowIfCancellationRequested();
			}
		}

		async ValueTask<bool> TryUpdateIssueAsync(BuildConfig buildConfig, int issueId, CancellationToken cancellationToken)
		{
			using (RedisLock issueLock = new RedisLock(_redisService.GetDatabase(), $"{_redisIssueLockPrefix}/{issueId}"))
			{
				if (await issueLock.AcquireAsync(TimeSpan.FromSeconds(30.0)))
				{
					await NotifyIssueUpdatedInternalAsync(buildConfig, issueId, cancellationToken);
					return true;
				}
				else
				{
					_logger.LogDebug("Unable to aquire lock for updating issue {IssueId}.", issueId);
					return false;
				}
			}
		}

		async Task NotifyIssueUpdatedInternalAsync(BuildConfig buildConfig, int issueId, CancellationToken cancellationToken)
		{
			using IDisposable? scope = _logger.BeginScope("Slack notifications for issue {IssueId}", issueId);
			_logger.LogInformation("Updating Slack notifications for issue {IssueId}", issueId);

			IIssueDetails? details = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (details == null)
			{
				return;
			}

			IIssue issue = details.Issue;

			bool notifyOwner = true;
			bool notifySuspects = issue.Promoted || details.Spans.Any(x => x.LastFailure.Annotations.NotifySubmitters ?? false);

			WorkflowConfig? workflow = null;
			if (details.Spans.Count > 0)
			{
				IIssueSpan span = details.Spans[0];

				WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
				if (workflowId != null)
				{
					StreamConfig? streamConfig;
					if (buildConfig.TryGetStream(span.StreamId, out streamConfig) && streamConfig.TryGetWorkflow(workflowId.Value, out workflow) && !String.IsNullOrEmpty(workflow.TriageChannel))
					{
						bool skip = (issue.Severity == IssueSeverity.Warning && !workflow.TriageWarnings) || (issue.Severity == IssueSeverity.Error && !workflow.TriageErrors);
						if (!skip)
						{
							await CreateOrUpdateWorkflowThreadAsync(workflow.TriageChannel, issue, span, details.Spans, workflow, cancellationToken);
							notifyOwner = notifySuspects = false;
						}
					}
				}
			}

			HashSet<UserId> userIds = new HashSet<UserId>();
			if (notifySuspects)
			{
				if (details.Suspects.Count > 5)
				{
					_issueService.Collection.GetLogger(issueId).LogInformation("Not notifying suspects for issue; too many users ({Count}).", details.Suspects.Count);
				}
				else
				{
					userIds.UnionWith(details.Suspects.Select(x => x.AuthorId));
				}
			}
			if (notifyOwner && issue.OwnerId.HasValue)
			{
				userIds.Add(issue.OwnerId.Value);
			}

			HashSet<string> channels = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

			List<MessageStateDocument> existingMessages = await _messageStates.Find(x => x.EventId == GetIssueEventId(issue)).ToListAsync(cancellationToken);
			foreach (MessageStateDocument existingMessage in existingMessages)
			{
				if (existingMessage.UserId != null)
				{
					userIds.Add(existingMessage.UserId.Value);
				}
				else
				{
					channels.Add(existingMessage.Recipient);
				}
			}

			foreach (IIssueSpan span in details.Spans)
			{
				StreamConfig? streamConfig;
				if (buildConfig.TryGetStream(span.StreamId, out streamConfig))
				{
					TemplateRefConfig? templateRefConfig;
					if (streamConfig.TryGetTemplate(span.TemplateRefId, out templateRefConfig) && !String.IsNullOrEmpty(templateRefConfig.TriageChannel))
					{
						channels.Add(templateRefConfig.TriageChannel);
					}
					else if (!String.IsNullOrEmpty(streamConfig.TriageChannel))
					{
						channels.Add(streamConfig.TriageChannel);
					}
				}
			}

			if (userIds.Count > 0)
			{
				foreach (UserId userId in userIds)
				{
					IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
					if (user == null)
					{
						_logger.LogWarning("Unable to find user {UserId}", userId);
					}
					else
					{
						await NotifyIssueUpdatedAsync(buildConfig, user, issue, details, cancellationToken);
					}
				}
			}

			if (channels.Count > 0)
			{
				foreach (string channel in channels)
				{
					await SendIssueMessageAsync(buildConfig, channel, issue, details, null, DefaultAllowMentions, cancellationToken);
				}
			}

			await UpdateReportsAsync(buildConfig, issue, details.Spans, cancellationToken);
		}

		async Task<IIssueStep?> GetFixFailedStepAsync(IIssue issue, IEnumerable<IIssueSpan> spans, CancellationToken cancellationToken)
		{
			if (issue.FixCommitId != null)
			{
				foreach (IGrouping<StreamId, IIssueSpan> group in spans.GroupBy(x => x.StreamId))
				{
					IIssueStream? stream = issue.Streams.FirstOrDefault(x => x.StreamId == group.Key);
					if (stream != null && (stream.ContainsFix ?? false))
					{
						CommitIdWithOrder fixedCommitIdWithOrder = await _commitService.GetOrderedAsync(group.Key, issue.FixCommitId, cancellationToken);
						foreach (IIssueSpan span in group)
						{
							if (span.LastFailure.CommitId >= fixedCommitIdWithOrder)
							{
								return span.LastFailure;
							}
						}
					}
				}
			}
			return null;
		}

		async Task<IIssueSpan?> GetFixFailedSpanAsync(IIssue issue, IReadOnlyList<IIssueSpan> spans, CancellationToken cancellationToken)
		{
			if (issue.FixCommitId != null)
			{
				HashSet<StreamId> originStreams = new HashSet<StreamId>(issue.Streams.Where(x => x.MergeOrigin ?? false).Select(x => x.StreamId));
				foreach (IGrouping<StreamId, IIssueSpan> group in spans.GroupBy(x => x.StreamId).OrderBy(x => x.Key.ToString()))
				{
					if (originStreams.Contains(group.Key))
					{
						CommitIdWithOrder fixedCommitIdWithOrder = await _commitService.GetOrderedAsync(group.Key, issue.FixCommitId, cancellationToken);
						foreach (IIssueSpan span in group)
						{
							if (span.LastFailure.CommitId > fixedCommitIdWithOrder)
							{
								return span;
							}
						}
					}
				}
			}
			return null;
		}

		async Task InviteUsersAsync(string channel, IEnumerable<UserId> userIds, bool inviteUsersAsAdmin, CancellationToken cancellationToken)
		{
			List<string> slackUserIds = new List<string>();
			foreach (UserId userId in userIds)
			{
				IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
				if (user != null)
				{
					string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
					if (slackUserId != null)
					{
						slackUserIds.Add(slackUserId);
					}
				}
			}

			foreach (string slackUserId in slackUserIds)
			{
				string? errorCode = await _slackClient.TryInviteUsersAsync(channel, new[] { slackUserId }, cancellationToken);
				if (errorCode != null)
				{
					if (errorCode.Equals("user_is_restricted", StringComparison.Ordinal) && inviteUsersAsAdmin && _adminSlackClient != null)
					{
						try
						{
							await _adminSlackClient.AdminInviteUsersAsync(channel, new[] { slackUserId }, cancellationToken);
						}
						catch (SlackException ex)
						{
							if (!String.Equals(ex.Code, "already_in_channel", StringComparison.Ordinal))
							{
								_logger.LogWarning(ex, "Unable to invite user {UserId} to {Channel} (as admin): {Error}", slackUserId, channel, ex.Code);
							}
						}
					}
					else if (!errorCode.Equals("already_in_channel", StringComparison.Ordinal))
					{
						_logger.LogWarning("Unable to invite user {UserId} to {Channel}: {Error}", slackUserId, channel, errorCode);
					}
				}
			}
		}

		static string GetTriageThreadEventId(int issueId) => $"issue_triage_{issueId}";

		[Flags]
		enum ReactionFlags
		{
			None = 0,
			Acknowledged = 1,
			Quarantined = 2,
			FixFailed = 4,
			Resolved = 8,
			AssignedButNotAcknowledged = 16,
			NoSuspects = 32,
		}

#pragma warning disable CA1505
		async Task CreateOrUpdateWorkflowThreadAsync(string triageChannel, IIssue issue, IIssueSpan span, IReadOnlyList<IIssueSpan> spans, WorkflowConfig workflow, CancellationToken cancellationToken)
#pragma warning restore CA1505
		{
			Uri issueUrl = GetIssueUrl(issue, span.FirstFailure);

			string eventId = GetTriageThreadEventId(issue.Id);

			string prefix = workflow.TriagePrefix ?? String.Empty;
			if (issue.Severity == IssueSeverity.Error && (span.LastFailure.Annotations.BuildBlocker ?? false))
			{
				prefix = $"{prefix}*[BUILD BLOCKER]* ";
			}

			string issueSummary = issue.UserSummary ?? issue.Summary;
			string text = $"{workflow.TriagePrefix}{GetSeverityPrefix(issue.Severity)}Issue <{issueUrl}|{issue.Id}>: {issueSummary}{workflow.TriageSuffix}";
			bool closed = !spans.Any(x => x.NextSuccess == null && x.LastFailure.Annotations.WorkflowId != null);
			if (closed) // Thread may be shared by multiple workflows
			{
				text = $"~{text}~";
			}

			// Get the suspects for the issue
			IReadOnlyList<IIssueSuspect> suspects = await _issueService.Collection.FindSuspectsAsync(issue, cancellationToken);

			(MessageStateDocument state, bool isNew) = await SendOrUpdateMessageAsync(triageChannel, eventId, null, text, cancellationToken);
			SlackMessageId threadId = state.MessageId;

			bool isSecondThread = false;
			if (isNew)
			{
				// Create the summary text
				List<ILogAnchor> anchors = new List<ILogAnchor>();
				List<ILogEventData> eventDataItems = new List<ILogEventData>();

				if (span.FirstFailure.LogId != null)
				{
					LogId logId = span.FirstFailure.LogId.Value;
					ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
					if (log != null)
					{
						anchors = await log.GetAnchorsAsync(span.Id, 0, 50, cancellationToken);
						if (anchors.Any(x => x.Severity == LogEventSeverity.Error))
						{
							anchors.RemoveAll(x => x.Severity == LogEventSeverity.Warning);
						}

						List<string> eventStrings = new List<string>();
						for (int idx = 0; idx < Math.Min(anchors.Count, 3); idx++)
						{
							ILogEventData data = await anchors[idx].GetDataAsync(cancellationToken);
							eventDataItems.Add(data);
						}
					}
				}

				SlackMessage message = new SlackMessage();

				message.Blocks.Add(new SectionBlock(new TextObject($"From {FormatJobStep(span.FirstFailure, span.NodeName)}:")));
				foreach (ILogEventData eventDataItem in eventDataItems)
				{
					message.Blocks.Add(new SectionBlock(new TextObject(QuoteText(eventDataItem.Message, MaxLineLength))));
				}
				if (anchors.Count > eventDataItems.Count)
				{
					message.Blocks.Add(new SectionBlock(new TextObject("```...```")));
				}

				SlackMessageId summaryId = await _slackClient.PostMessageToThreadAsync(threadId, message, cancellationToken: cancellationToken);

				// Permalink to the summary text so we link inside the thread rather than just to the original message
				string permalink = await _slackClient.GetPermalinkAsync(summaryId, cancellationToken);
				await UpdateMessageStateAsync(state.Id, state.MessageId, permalink, cancellationToken);

				_issueService.Collection.GetLogger(issue.Id).LogInformation("Created Slack thread under workflow {WorkflowId}: {SlackLink}", workflow.Id, permalink);
				try
				{
					for (IIssue? updateIssue = issue; updateIssue != null;)
					{
						if (updateIssue.WorkflowThreadUrl != null)
						{
							await _slackClient.PostMessageToThreadAsync(threadId, $"Existing thread here: {updateIssue.WorkflowThreadUrl}", cancellationToken: cancellationToken);
							isSecondThread = true;
							break;
						}

						updateIssue = await _issueService.Collection.TryUpdateIssueAsync(updateIssue, null, new UpdateIssueOptions { WorkflowThreadUrl = new Uri(permalink) }, cancellationToken: cancellationToken);
						if (updateIssue != null)
						{
							break;
						}

						updateIssue = await _issueService.Collection.GetIssueAsync(issue.Id, cancellationToken);
					}
				}
				catch (Exception ex)
				{
					_issueService.Collection.GetLogger(issue.Id).LogInformation(ex, "Error associating workflow thread with issue, bad URI format? {ErrorMessage}", ex.Message);
				}
			}

			IIssueSpan? fixFailedSpan = null;
			if (issue.FixCommitId != null)
			{
				fixFailedSpan = await GetFixFailedSpanAsync(issue, spans, cancellationToken);
			}

			if (!isSecondThread)
			{
				// Post a message containing the controls and status
				{
					SlackMessage message = new SlackMessage();

					if (workflow.TriageInstructions != null)
					{
						message.AddSection(workflow.TriageInstructions);
					}

					if (!closed)
					{
						ActionsBlock actions = message.AddActions();
						actions.AddButton("Assign to Me", value: $"issue_{issue.Id}_ack", style: ButtonStyle.Primary);
						actions.AddButton("Not Me", value: $"issue_{issue.Id}_decline", style: ButtonStyle.Danger);
						actions.AddButton("Mark Fixed", value: $"issue_{issue.Id}_markfixed");

						string? context = null;
						if (issue.OwnerId != null)
						{
							string user = await FormatNameAsync(issue.OwnerId.Value, cancellationToken);
							if (issue.AcknowledgedAt == null)
							{
								context = $"Assigned to {user} (unacknowledged).";
							}
							else
							{
								context = $"Acknowledged by {user} at {FormatSlackTime(issue.AcknowledgedAt.Value)}.";
							}
						}
						else if (suspects.Any(x => x.DeclinedAt != null))
						{
							HashSet<UserId> userIds = new HashSet<UserId>();
							foreach (IIssueSuspect suspect in suspects)
							{
								if (suspect.DeclinedAt != null)
								{
									userIds.Add(suspect.AuthorId);
								}
							}

							List<string> users = new List<string>();
							foreach (UserId userId in userIds)
							{
								users.Add(await FormatNameAsync(userId, cancellationToken));
							}
							users.Sort(StringComparer.OrdinalIgnoreCase);

							context = $"Declined by {StringUtils.FormatList(users)}.";
						}

						if (context != null)
						{
							message.AddContext(context);
						}
					}

					if (message.Blocks.Count == 0)
					{
						message.AddSection("Issue has been closed.");
					}

					await SendOrUpdateMessageToThreadAsync(triageChannel, $"{eventId}_buttons", null, threadId, message, cancellationToken);
				}

				bool notifyTriageAlias = false;
				if (isNew)
				{
					// If it has an owner, show that
					HashSet<UserId> inviteUserIds = new HashSet<UserId>();
					if (issue.OwnerId != null)
					{
						string mention = await FormatMentionAsync(issue.OwnerId.Value, workflow.AllowMentions, cancellationToken);

						string changes = String.Join(", ", suspects.Where(x => x.AuthorId == issue.OwnerId).Select(x => FormatChange(x.CommitId)));
						if (changes.Length > 0)
						{
							mention += $" ({changes})";
						}

						await _slackClient.PostMessageToThreadAsync(threadId, $"Assigned to {mention}", cancellationToken);
						inviteUserIds.Add(issue.OwnerId.Value);
					}
					else
					{
						IGrouping<UserId, IIssueSuspect>[] suspectGroups = suspects.GroupBy(x => x.AuthorId).ToArray();

						List<string> lowContextTypes = _buildConfig.CurrentValue.LowContextFingerprintTypes;
						bool isLowContextType = lowContextTypes.Any(prefix => span.Fingerprint.Type.StartsWith(prefix, StringComparison.Ordinal));

						if (suspectGroups.Length > 0
							&& suspectGroups.Length <= workflow.MaxMentions
							&& isLowContextType
							&& span.MaxSuspectRank != null
							&& span.MaxSuspectRank.Value == 0)
						{
							// PATH 0: Low-context type with all suspects tied at rank 0.
							// List suspects without @ mentions; post explanatory note.
							_issueService.Collection.GetLogger(issue.Id).LogInformation(
								"Suppressing @ mentions for issue with MaxSuspectRank={MaxSuspectRank} ({SuspectCount} suspects, fingerprint type: {FingerprintType})",
								span.MaxSuspectRank, suspectGroups.Length, span.Fingerprint.Type);

							List<string> suspectList = new List<string>();
							foreach (IGrouping<UserId, IIssueSuspect> suspectGroup in suspectGroups)
							{
								string name = await FormatNameAsync(suspectGroup.Key, cancellationToken);
								string changes = String.Join(", ", suspectGroup.Select(x => FormatChange(x.CommitId)));
								suspectList.Add($"{name} ({changes})");
							}

							string suspectMessage = $"Possibly {StringUtils.FormatList(suspectList, "or")}.";

							Uri? swarmLink = GetSwarmLinkForSpan(span);
							if (swarmLink != null)
							{
								suspectMessage += $" (<{swarmLink}|View changes>)";
							}

							Uri? robomergeTrackChangesLink = GetRobomergeTrackChangesUrlForSpan(span);
							if (robomergeTrackChangesLink != null)
							{
								suspectMessage += $" (<{robomergeTrackChangesLink}|Track changes>)";
							}

							await _slackClient.PostMessageToThreadAsync(threadId, suspectMessage, cancellationToken: cancellationToken);
							await _slackClient.PostMessageToThreadAsync(threadId,
								$"_This issue does not have enough context (files, symbols) to identify a specific suspect. Mentions suppressed._",
								cancellationToken: cancellationToken);
						}
						else if (suspectGroups.Length > 0 && suspectGroups.Length <= workflow.MaxMentions)
						{
							// PATH 1: @ mentions via FormatMentionAsync
							List<string> suspectList = new List<string>();
							foreach (IGrouping<UserId, IIssueSuspect> suspectGroup in suspectGroups)
							{
								string mention = await FormatMentionAsync(suspectGroup.Key, workflow.AllowMentions, cancellationToken);
								string changes = String.Join(", ", suspectGroup.Select(x => FormatChange(x.CommitId)));
								suspectList.Add($"{mention} ({changes})");
								inviteUserIds.Add(suspectGroup.Key);
							}

							string suspectMessage = $"Possibly {StringUtils.FormatList(suspectList, "or")}.";

							Uri? swarmLink = GetSwarmLinkForSpan(span);
							if (swarmLink != null)
							{
								suspectMessage += $" (<{swarmLink}|View changes>)";
							}

							Uri? robomergeTrackChangesLink = GetRobomergeTrackChangesUrlForSpan(span);
							if (robomergeTrackChangesLink != null)
							{
								suspectMessage += $" (<{robomergeTrackChangesLink}|Track changes>)";
							}

							await _slackClient.PostMessageToThreadAsync(threadId, suspectMessage, cancellationToken: cancellationToken);
						}
						else
						{
							// PATH 2: Swarm/Commits links only, notifyTriageAlias = true
							string suspectMessage = String.Empty;

							Uri? swarmLink = GetSwarmLinkForSpan(span);
							if (swarmLink != null)
							{
								suspectMessage += $"<{swarmLink}|View changes in Swarm>";
							}
							else if (span.LastSuccess == null)
							{
								suspectMessage += $"No previous successes for span. See <{GetStepUrl(span.FirstFailure.JobId, span.FirstFailure.StepId)}|step history panel> and click on change buttons for CL ranges.";
							}

							Uri? commitsViewerLink = GetCommitsViewerLinkForSpan(span);
							if (commitsViewerLink != null)
							{
								suspectMessage += $" (<{commitsViewerLink}|View changes in Commits Viewer>)";
							}

							Uri? robomergeTrackChangesLink = GetRobomergeTrackChangesUrlForSpan(span);
							if (robomergeTrackChangesLink != null)
							{
								suspectMessage += $" (<{robomergeTrackChangesLink}|Track changes>)";
							}

							if (!String.IsNullOrWhiteSpace(suspectMessage))
							{
								await _slackClient.PostMessageToThreadAsync(threadId, suspectMessage, cancellationToken);
							}

							notifyTriageAlias = true;
						}
					}

					if (_environment.IsProduction() && workflow.AllowMentions)
					{
						await InviteUsersAsync(state.Channel, inviteUserIds, workflow.InviteRestrictedUsers, cancellationToken);
					}
				}

				if (workflow.EscalateAlias != null && workflow.EscalateTimes.Count > 0)
				{
					DateTime escalateTime = issue.CreatedAt.AddMinutes(workflow.EscalateTimes[0]);
					if (await _redisService.GetDatabase().SortedSetAddAsync(s_escalateIssues, issue.Id, (escalateTime - DateTime.UnixEpoch).TotalSeconds, SortedSetWhen.NotExists))
					{
						_logger.LogInformation("First escalation time for issue {IssueId} is {Time}", issue.Id, escalateTime);
					}
				}

				if ((issue.ResolvedAt != null && issue.DuplicateIssueId == null) && workflow.EscalateRootCauseAnalysisTimes.Count > 0)
				{
					DateTime escalateTime = issue.ResolvedAt.Value.AddMinutes(workflow.EscalateRootCauseAnalysisTimes[0]);
					if (await _redisService.GetDatabase().SortedSetAddAsync(s_escalateRootCauseAnalysis, issue.Id, (escalateTime - DateTime.UnixEpoch).TotalSeconds, SortedSetWhen.NotExists))
					{
						_logger.LogInformation("First escalation time for root cause analysis {IssueId} is {Time}", issue.Id, escalateTime);
					}
				}

				if ((workflow.TriageAlias != null || workflow.TriageTypeAliases != null) && issue.OwnerId == null && (suspects.All(x => x.DeclinedAt != null) || notifyTriageAlias) && !closed)
				{
					string? triageAlias;

					if (workflow.TriageTypeAliases == null || issue.Fingerprints.Count == 0 || !workflow.TriageTypeAliases.TryGetValue(issue.Fingerprints[0].Type, out triageAlias))
					{
						triageAlias = workflow.TriageAlias;
					}

					if (triageAlias != null)
					{
						string triageMessage = $"(cc {FormatUserOrGroupMention(triageAlias)} for triage).";
						await SendOrUpdateMessageToThreadAsync(triageChannel, eventId + "_triage", null, threadId, triageMessage, cancellationToken);
					}
				}

				if (issue.FixCommitId != null)
				{
					if (fixFailedSpan == null)
					{
						string fixedEventId = $"issue_{issue.Id}_fixed_{issue.FixCommitId}";
						string fixedMessage = $"Marked as fixed in {FormatChange(issue.FixCommitId)}";

						Uri? robomergeTrackChangesLink = GetRobomergeTrackChangesUrlForSpan(span);
						if (robomergeTrackChangesLink != null)
						{
							fixedMessage += $" (<{robomergeTrackChangesLink}|Track changes>)";
						}

						await PostSingleMessageToThreadAsync(triageChannel, fixedEventId, threadId, fixedMessage, cancellationToken);
					}
					else
					{
						string fixFailedEventId = $"issue_{issue.Id}_fixfailed_{issue.FixCommitId}";

						IIssueStream? fixStream = issue.Streams.FirstOrDefault(x => x.ContainsFix ?? false);

						string fixFailedMessage = $"Issue not fixed by {FormatChange(issue.FixCommitId)}; see {FormatJobStep(fixFailedSpan.LastFailure, fixFailedSpan.NodeName)} at CL {fixFailedSpan.LastFailure.CommitId} in {fixFailedSpan.StreamName}.";

						if (fixStream != null && fixStream.StreamId != fixFailedSpan.StreamId)
						{
							string fixStreamName = spans.FirstOrDefault(x => x.StreamId == fixStream.StreamId)?.StreamName ?? fixStream.StreamId.ToString();
							fixFailedMessage = $"Issue possibly not fixed by {FormatChange(issue.FixCommitId)}.  The fix CL may be in the process of automatically or manually merging from {fixStreamName} to {fixFailedSpan.StreamName}; see {FormatJobStep(fixFailedSpan.LastFailure, fixFailedSpan.NodeName)} at CL {fixFailedSpan.LastFailure.CommitId} in {fixFailedSpan.StreamName}.";
						}
						
						if (issue.OwnerId.HasValue)
						{
							string mention = await FormatMentionAsync(issue.OwnerId.Value, workflow.AllowMentions, cancellationToken);
							fixFailedMessage += $" ({mention})";
						}						
						await PostSingleMessageToThreadAsync(triageChannel, fixFailedEventId, threadId, fixFailedMessage, cancellationToken);
					}

					if (fixFailedSpan == null && workflow.ShowMergeWarnings)
					{
						foreach (IIssueStream stream in issue.Streams)
						{
							if ((stream.MergeOrigin ?? false) && !(stream.ContainsFix ?? false))
							{
								string streamName = spans.FirstOrDefault(x => x.StreamId == stream.StreamId)?.StreamName ?? stream.StreamId.ToString();
								string missingEventId = $"issue_{issue.Id}_fixmissing_{issue.FixCommitId}_{stream.StreamId}";
								string missingMessage = $"Note: Fix may need manually merging to {streamName}";
								await PostSingleMessageToThreadAsync(triageChannel, missingEventId, threadId, missingMessage, cancellationToken);
							}
						}
					}
				}
				else if (issue.FixSystemic)
				{
					string fixedEventId = $"issue_{issue.Id}_fixed_systemic";
					string fixedMessage = $"Marked fixed as a systemic issue";
					await PostSingleMessageToThreadAsync(triageChannel, fixedEventId, threadId, fixedMessage, cancellationToken);
				}

				// Assignment notifications
				if (issue.OwnerId != null && issue.NominatedById != null && issue.NominatedById != issue.OwnerId)
				{
					string assignmentEventId = $"issue_{issue.Id}_nominated";
					string assignmentMessage = $"{await FormatMentionAsync(issue.OwnerId.Value, workflow.AllowMentions, cancellationToken)} was nominated to fix by {await FormatNameAsync(issue.NominatedById.Value, cancellationToken)}.";
					await PostSingleMessageToThreadAsync(triageChannel, assignmentEventId, threadId, assignmentMessage, cancellationToken);

					// Invite the nominated user to the channel so they receive the notification
					if (_environment.IsProduction() && workflow.AllowMentions)
					{
						await InviteUsersAsync(state.Channel, new[] { issue.OwnerId.Value }, workflow.InviteRestrictedUsers, cancellationToken);
					}
				}
			}

			await UpdateReactionsAsync(triageChannel, issue, fixFailedSpan, suspects, eventId, threadId, cancellationToken);

			if (issue.ExternalIssueKey != null)
			{
				string extIssueEventId = $"issue_{issue.Id}_ext_{issue.ExternalIssueKey}";
				string extIssueMessage = $"Linked to issue {FormatExternalIssue(issue.ExternalIssueKey)}";
				await PostSingleMessageToThreadAsync(triageChannel, extIssueEventId, threadId, extIssueMessage, cancellationToken);
			}
		}

		private async Task UpdateReactionsAsync(string triageChannel, IIssue issue, IIssueSpan? fixFailedSpan, IReadOnlyList<IIssueSuspect> suspects, string eventId, SlackMessageId threadId, CancellationToken cancellationToken)
		{
			ReactionFlags reactions = ReactionFlags.None;
			if (issue.AcknowledgedAt != null)
			{
				reactions |= ReactionFlags.Acknowledged;
			}
			if (issue.QuarantinedByUserId != null)
			{
				reactions |= ReactionFlags.Quarantined;
			}
			if (fixFailedSpan != null)
			{
				reactions |= ReactionFlags.FixFailed;
			}
			if (issue.ResolvedAt != null && fixFailedSpan == null)
			{
				reactions |= ReactionFlags.Resolved;
			}
			if (issue.AcknowledgedAt == null && issue.OwnerId != null)
			{
				reactions |= ReactionFlags.AssignedButNotAcknowledged;
			}
			if (issue.OwnerId == null && (suspects.Count == 0 || suspects.Count > 5))
			{
				reactions |= ReactionFlags.NoSuspects;
			}

			string reactionEventId = $"{eventId}_reactions";
			string reactionStateDigest = ((int)reactions).ToString();

			MessageStateDocument? reactionState = await GetMessageStateAsync(triageChannel, reactionEventId, cancellationToken);
			if ((reactionState == null && reactions != ReactionFlags.None) || (reactionState != null && reactionState.Digest != reactionStateDigest))
			{
				if ((reactions & ReactionFlags.Acknowledged) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "eyes", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "eyes", cancellationToken);
				}

				if ((reactions & ReactionFlags.Quarantined) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "mask", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "mask", cancellationToken);
				}

				if ((reactions & ReactionFlags.FixFailed) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "x", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "x", cancellationToken);
				}

				if ((reactions & ReactionFlags.Resolved) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "tick", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "tick", cancellationToken);
				}

				if ((reactions & ReactionFlags.AssignedButNotAcknowledged) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "mailbox", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "mailbox", cancellationToken);
				}

				if ((reactions & ReactionFlags.NoSuspects) != 0)
				{
					await _slackClient.AddReactionAsync(threadId, "question", cancellationToken);
				}
				else
				{
					await _slackClient.RemoveReactionAsync(threadId, "question", cancellationToken);
				}

				await AddOrUpdateMessageStateAsync(triageChannel, reactionEventId, null, reactionStateDigest, threadId, cancellationToken);
			}
		}

		Uri? GetSwarmLinkForSpan(IIssueSpan span)
		{
			if (_settings.P4SwarmUrl == null || span.LastSuccess == null)
			{
				return null;
			}
			else
			{
				return new Uri(_settings.P4SwarmUrl, $"files/{span.StreamName.TrimStart('/')}?range=@{span.LastSuccess.CommitId.GetPerforceChange() + 1},@{span.FirstFailure.CommitId}#commits");
			}
		}

		Uri? GetRobomergeTrackChangesUrlForSpan(IIssueSpan span)
		{
			if (_settings.RobomergeUrl == null || span.LastSuccess == null)
			{
				return null;
			}
			else
			{
				return new Uri(_settings.RobomergeUrl, $"trackchange/{span.FirstFailure.CommitId}");
			}
		}

		Uri? GetCommitsViewerLinkForSpan(IIssueSpan span)
		{
			if (_settings.CommitsViewerUrl == null || span.LastSuccess == null)
			{
				return null;
			}
			else
			{
				string depot_stream = span.StreamName.TrimStart('/');
				string[] depot_stream_split = depot_stream.Split(new[] { '/' }, 2);

				// If for some reason the stream is not split correctly for CV, return null
				if (depot_stream_split.Length != 2)
				{
					return null;
				}

				string query_string = $"?depot={depot_stream_split[0]}&stream={depot_stream_split[1]}" +
					$"&query_range[CL][start_cl]={span.LastSuccess.CommitId.GetPerforceChange() + 1}&query_range[CL][end_cl]={span.FirstFailure.CommitId}" +
					"&auto_load=true";

				return new Uri(_settings.CommitsViewerUrl, query_string);
			}
		}

		async Task PostSingleMessageToThreadAsync(string recipient, string eventId, SlackMessageId threadId, string message, CancellationToken cancellationToken)
		{
			(MessageStateDocument state, bool isNew) = await AddOrUpdateMessageStateAsync(recipient, eventId, null, "", null, cancellationToken);
			if (isNew)
			{
				SlackMessageId messageId = await _slackClient.PostMessageToThreadAsync(threadId, message, cancellationToken);
				await UpdateMessageStateAsync(state.Id, messageId, cancellationToken: cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task NotifyRcaCompleteAsync(int issueId, RcaResult result, CancellationToken cancellationToken)
		{
			// Global kill switch check
			if (!_buildConfig.CurrentValue.EnableAutoRcaSlackNotifications)
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: EnableAutoRcaSlackNotifications is disabled globally", issueId);
				return;
			}

			// Get issue details to find the workflow
			IIssueDetails? details = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (details == null || details.Spans.Count == 0)
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: issue not found or has no spans", issueId);
				return;
			}

			IIssueSpan span = details.Spans[0];
			WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
			if (workflowId == null)
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: no workflow annotation on span", issueId);
				return;
			}

			BuildConfig buildConfig = _buildConfig.CurrentValue;
			if (!buildConfig.TryGetStream(span.StreamId, out StreamConfig? streamConfig))
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: stream {StreamId} not found in config", issueId, span.StreamId);
				return;
			}
			if (!streamConfig.TryGetWorkflow(workflowId.Value, out WorkflowConfig? workflow))
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: workflow {WorkflowId} not found in stream {StreamId}", issueId, workflowId.Value, span.StreamId);
				return;
			}
			if (!workflow.AutoRcaSlackEnabled || String.IsNullOrEmpty(workflow.TriageChannel))
			{
				_logger.LogDebug("Skipping auto-RCA notification for issue {IssueId}: AutoRcaSlackEnabled={Enabled}, TriageChannel={Channel}",
					issueId, workflow.AutoRcaSlackEnabled, workflow.TriageChannel ?? "(empty)");
				return;
			}

			// Find the existing Slack thread (with exponential backoff in case thread creation is delayed)
			MessageStateDocument? state = null;
			for (int attempt = 0; attempt < RcaThreadLookupMaxAttempts && state == null; attempt++)
			{
				if (attempt > 0)
				{
					int delaySecs = RcaThreadLookupBaseDelaySecs * (1 << (attempt - 1));
					_logger.LogDebug("Auto-RCA thread lookup for issue {IssueId}: attempt {Attempt}/{MaxAttempts}, waiting {DelaySecs}s",
						issueId, attempt + 1, RcaThreadLookupMaxAttempts, delaySecs);
					await Task.Delay(TimeSpan.FromSeconds(delaySecs), cancellationToken);
				}
				state = await GetMessageStateAsync(workflow.TriageChannel, GetTriageThreadEventId(issueId), cancellationToken);
			}
			if (state == null)
			{
				_logger.LogWarning("No Slack thread found for issue {IssueId} after {MaxAttempts} attempts, skipping auto-RCA notification",
					issueId, RcaThreadLookupMaxAttempts);
				return;
			}

			// Post via idempotent one-time pattern
			string rcaEventId = $"issue_{issueId}_auto_rca";
			string rcaMessage = FormatAutoRcaMessage(result);
			await PostSingleMessageToThreadAsync(workflow.TriageChannel, rcaEventId, state.MessageId, rcaMessage, cancellationToken);
		}

		const int MaxRunnerUpsInSlack = 3;
		const int MaxReasonLength = 700;

		static string TruncateReason(string reason)
			=> reason.Length > MaxReasonLength ? reason[..MaxReasonLength] + "..." : reason;

		string FormatAutoRcaMessage(RcaResult result)
		{
			StringBuilder sb = new();
			sb.AppendLine(":robot_face: *Auto-RCA Suggestion*");
			sb.AppendLine("_:warning: This analysis was generated by AI and may be inaccurate. Use as a starting point for triage only._");

			if (result.CulpritCommitId != null)
			{
				sb.AppendLine($"Suspected culprit: {FormatChange(result.CulpritCommitId)}");
				if (!String.IsNullOrEmpty(result.Reason))
				{
					sb.AppendLine($"> {TruncateReason(result.Reason)}");
				}
			}

			if (result.RunnerUps != null && result.RunnerUps.Count > 0)
			{
				sb.AppendLine("Other suspects:");
				foreach (RcaRunnerUp runnerUp in result.RunnerUps.Take(MaxRunnerUpsInSlack))
				{
					string line = $"- {FormatChange(runnerUp.Commit)}";
					if (!String.IsNullOrEmpty(runnerUp.BriefReason))
					{
						line += $": {TruncateReason(runnerUp.BriefReason)}";
					}
					sb.AppendLine(line);
				}
				if (result.RunnerUps.Count > MaxRunnerUpsInSlack)
				{
					sb.AppendLine($"_...and {result.RunnerUps.Count - MaxRunnerUpsInSlack} more (see dashboard)_");
				}
			}

			return sb.ToString().TrimEnd();
		}

		async Task NotifyIssueUpdatedAsync(BuildConfig buildConfig, IUser user, IIssue issue, IIssueDetails details, CancellationToken cancellationToken)
		{
			string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
			if (slackUserId == null)
			{
				return;
			}

			await SendIssueMessageAsync(buildConfig, slackUserId, issue, details, user.Id, DefaultAllowMentions, cancellationToken);
		}

		Uri GetJobUrl(JobId jobId)
		{
			return new Uri(_serverInfo.DashboardUrl, $"job/{jobId}");
		}

		Uri GetStepUrl(JobId jobId, JobStepId stepId)
		{
			return new Uri(_serverInfo.DashboardUrl, $"job/{jobId}?step={stepId}");
		}

		Uri GetIssueUrl(IIssue issue, IIssueStep step)
		{
			return new Uri(_serverInfo.DashboardUrl, $"job/{step.JobId}?step={step.StepId}&issue={issue.Id}");
		}

		async Task SendIssueMessageAsync(BuildConfig buildConfig, string recipient, IIssue issue, IIssueDetails details, UserId? userId, bool allowMentions, CancellationToken cancellationToken)
		{
			using IDisposable? scope = _logger.BeginScope("SendIssueMessageAsync (User: {SlackUser}, Issue: {IssueId})", recipient, issue.Id);

			SlackAttachment attachment = new SlackAttachment();
			attachment.Color = ErrorColor;

			Uri issueUrl = _serverInfo.DashboardUrl;
			if (details.Steps.Count > 0)
			{
				issueUrl = GetIssueUrl(issue, details.Steps[0]);
			}
			attachment.AddSection($"*<{issueUrl}|Issue #{issue.Id}: {issue.Summary}>*");

			string streamList = StringUtils.FormatList(details.Spans.Select(x => $"*{x.StreamName}*").Distinct(StringComparer.OrdinalIgnoreCase).OrderBy(x => x, StringComparer.OrdinalIgnoreCase));
			StringBuilder summaryBuilder = new StringBuilder($"Occurring in {streamList}.");

			IIssueSpan span = details.Spans[0];
			WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
			if (workflowId != null)
			{
				StreamConfig? streamConfig;
				if (buildConfig.TryGetStream(span.StreamId, out streamConfig) && streamConfig.TryGetWorkflow(workflowId.Value, out WorkflowConfig? workflow) && !String.IsNullOrEmpty(workflow.TriageChannel) && workflow.AllowMentions)
				{
					bool skip = (issue.Severity == IssueSeverity.Warning && !workflow.TriageWarnings) || (issue.Severity == IssueSeverity.Error && !workflow.TriageErrors);
					if (!skip)
					{ 
						MessageStateDocument? state = await GetMessageStateAsync(workflow.TriageChannel, GetTriageThreadEventId(issue.Id), cancellationToken);
						if (state != null)
						{
							summaryBuilder.Append($" See *<{state.Permalink}|discussion thread>*.");
						}
					}
				}
			}
			attachment.AddSection(summaryBuilder.ToString());

			IIssueSpan? lastSpan = details.Spans.OrderByDescending(x => x.LastFailure.StepTime).FirstOrDefault();
			if (lastSpan != null && lastSpan.LastFailure.LogId != null)
			{
				LogId logId = lastSpan.LastFailure.LogId.Value;
				ILog? log = await _logCollection.GetAsync(logId, cancellationToken);
				if (log != null)
				{
					List<ILogAnchor> events = await log.GetAnchorsAsync(lastSpan.Id, 0, 20, cancellationToken);
					if (events.Any(x => x.Severity == LogEventSeverity.Error))
					{
						events.RemoveAll(x => x.Severity == LogEventSeverity.Warning);
					}

					for (int idx = 0; idx < Math.Min(events.Count, 3); idx++)
					{
						ILogEventData data = await events[idx].GetDataAsync(cancellationToken);
						attachment.AddSection(QuoteText(data.Message));
					}
					if (events.Count > 3)
					{
						attachment.AddSection("```...```");
					}
				}
			}

			if (issue.FixCommitId != null)
			{
				IIssueStep? fixFailedStep = await GetFixFailedStepAsync(issue, details.Spans, cancellationToken);

				string text;
				if (fixFailedStep != null)
				{
					Uri fixFailedUrl = new Uri(_serverInfo.DashboardUrl, $"job/{fixFailedStep.JobId}?step={fixFailedStep.StepId}&issue={issue.Id}");
					text = $":cross: Marked fixed in *CL {issue.FixCommitId}*, but seen again at *<{fixFailedUrl}|CL {fixFailedStep.CommitId}>*";

					string rbTrackChangeText = FormatRobomergeTrackChangeLinkChange(issue.FixCommitId);
					if (!String.IsNullOrWhiteSpace(rbTrackChangeText))
					{
						text += rbTrackChangeText;
					}
				}
				else
				{
					text = $":tick: Marked fixed in *CL {issue.FixCommitId}*.";
				}
				attachment.AddSection(text);
			}
			else if (issue.FixSystemic)
			{
				attachment.AddSection(":tick: Marked as a systemic issue.");
			}
			else if (userId != null && issue.OwnerId == userId)
			{
				if (issue.AcknowledgedAt.HasValue)
				{
					attachment.AddSection($":+1: Acknowledged at {FormatSlackTime(issue.AcknowledgedAt.Value)}");
				}
				else
				{
					if (issue.NominatedById != null)
					{
						string mention = await FormatMentionAsync(issue.NominatedById.Value, allowMentions, cancellationToken);
						string text = $"You were nominated to fix this issue by {mention} at {FormatSlackTime(issue.NominatedAt ?? DateTime.UtcNow)}";
						attachment.AddSection(text);
					}
					else
					{
						List<int> changes = details.Suspects.Where(x => x.AuthorId == userId).Select(x => x.CommitId.GetPerforceChange()).OrderBy(x => x).ToList();
						if (changes.Count > 0)
						{
							string text = $"Horde has determined that {StringUtils.FormatList(changes.Select(x => $"CL {x}"), "or")} is the most likely cause for this issue.";
							attachment.AddSection(text);
						}
					}

					ActionsBlock actions = new ActionsBlock();
					actions.AddButton("Acknowledge", value: $"issue_{issue.Id}_ack_{userId}", style: ButtonStyle.Primary);
					actions.AddButton("Not Me", value: $"issue_{issue.Id}_decline_{userId}", style: ButtonStyle.Danger);
					attachment.Blocks.Add(actions);
				}
			}
			else if (issue.OwnerId != null)
			{
				string ownerMention = await FormatMentionAsync(issue.OwnerId.Value, allowMentions, cancellationToken);
				if (issue.AcknowledgedAt.HasValue)
				{
					attachment.AddSection($":+1: Acknowledged by {ownerMention} at {FormatSlackTime(issue.AcknowledgedAt.Value)}");
				}
				else if (issue.NominatedById == null)
				{
					attachment.AddSection($"Assigned to {ownerMention}");
				}
				else if (issue.NominatedById == userId)
				{
					attachment.AddSection($"You nominated {ownerMention} to fix this issue.");
				}
				else
				{
					attachment.AddSection($"{ownerMention} was nominated to fix this issue by {await FormatMentionAsync(issue.NominatedById.Value, allowMentions, cancellationToken)}");
				}
			}
			else if (userId != null)
			{
				IIssueSuspect? suspect = details.Suspects.FirstOrDefault(x => x.AuthorId == userId);
				if (suspect != null)
				{
					if (suspect.DeclinedAt != null)
					{
						attachment.AddSection($":downvote: Declined at {FormatSlackTime(suspect.DeclinedAt.Value)}");
					}
					else
					{
						attachment.AddSection("Please check if any of your recently submitted changes have caused this issue.");

						ActionsBlock actions = attachment.AddActions();
						actions.AddButton("Will Fix", value: $"issue_{issue.Id}_accept_{userId}", style: ButtonStyle.Primary);
						actions.AddButton("Not Me", value: $"issue_{issue.Id}_decline_{userId}", style: ButtonStyle.Danger);
					}
				}
			}
			else if (details.Suspects.Count > 0)
			{
				List<string> declinedLines = new List<string>();
				foreach (IIssueSuspect suspect in details.Suspects)
				{
					if (!details.Issue.Promoted)
					{
						declinedLines.Add($"Possibly {await FormatNameAsync(suspect.AuthorId, cancellationToken)} (CL {suspect.CommitId})");
					}
					else if (suspect.DeclinedAt == null)
					{
						declinedLines.Add($":heavy_minus_sign: Ignored by {await FormatNameAsync(suspect.AuthorId, cancellationToken)} (CL {suspect.CommitId})");
					}
					else
					{
						declinedLines.Add($":downvote: Declined by {await FormatNameAsync(suspect.AuthorId, cancellationToken)} at {FormatSlackTime(suspect.DeclinedAt.Value)} (CL {suspect.CommitId})");
					}
				}
				attachment.AddSection(String.Join("\n", declinedLines));
			}

			if (IsRecipientAllowed(recipient, "issue update"))
			{
				await SendOrUpdateMessageAsync(recipient, GetIssueEventId(issue, recipient), userId, attachment, cancellationToken);
			}
		}

		static string GetIssueEventId(IIssue issue)
		{
			return $"issue_{issue.Id}";
		}

		static string GetIssueEventId(IIssue issue, string recipient)
		{
			return $"issue_{issue.Id}_for_{recipient}";
		}

		async Task<string> FormatNameAsync(UserId userId, CancellationToken cancellationToken)
		{
			IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
			if (user == null)
			{
				return GetDefaultUserName(userId);
			}
			return user.Name;
		}

		static string GetDefaultUserName(UserId userId)
		{
			if (userId == KnownUsers.System)
			{
				return "Horde";
			}
			else if (userId == IIssue.ResolvedByUnknownId)
			{
				return "Horde (Unknown)";
			}
			else if (userId == IIssue.ResolvedByTimeoutId)
			{
				return "Horde (Timeout)";
			}
			else
			{
				return $"User {userId}";
			}
		}

		string FormatChange(CommitId change)
		{
			string output = _settings.P4SwarmUrl != null
				? $"<{new Uri(_settings.P4SwarmUrl, $"changes/{change}")}|CL {change}>"
				: $"CL {change}";

			if (_settings.RobomergeUrl != null)
			{
				output += FormatRobomergeTrackChangeLinkChange(change);
			}

			return output;
		}

		string FormatRobomergeTrackChangeLinkChange(CommitId change)
		{
			return _settings.RobomergeUrl != null
				? $" (<{new Uri(_settings.RobomergeUrl, $"trackchange/{change}")}|Track changes>)"
				: String.Empty;
		}

		string FormatJobStep(IIssueStep step, string name) => $"<{GetJobUrl(step.JobId)}|{step.JobName}> / <{GetStepUrl(step.JobId, step.StepId)}|{name}>";

		string FormatExternalIssue(string key)
		{
			string? url = _externalIssueService.GetIssueUrl(key);
			if (url == null)
			{
				return key;
			}
			else
			{
				return $"<{_externalIssueService.GetIssueUrl(key)}|{key}>";
			}
		}

		static string FormatUserOrGroupMention(string userOrGroup)
		{
			if (userOrGroup.StartsWith("S", StringComparison.OrdinalIgnoreCase))
			{
				return $"<!subteam^{userOrGroup}>";
			}
			else
			{
				return $"<@{userOrGroup}>";
			}
		}

		async Task<string> FormatMentionAsync(UserId userId, bool allowMentions, CancellationToken cancellationToken)
		{
			IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
			if (user == null)
			{
				return GetDefaultUserName(userId);
			}

			string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
			if (slackUserId == null)
			{
				return user.Login;
			}

			if (_allowUsers == null || !_allowUsers.Contains(slackUserId))
			{
				if (!_environment.IsProduction() || !allowMentions)
				{
					return $"{user.Name} [{slackUserId}]";
				}
			}

			return $"<@{slackUserId}>";
		}

		static string FormatSlackTime(DateTimeOffset time)
		{
			return $"<!date^{time.ToUnixTimeSeconds()}^{{time}}|{time}>";
		}

		class ReportBlock
		{
			[JsonPropertyName("h")]
			public bool TemplateHeader { get; set; }

			[JsonPropertyName("s")]
			public StreamId StreamId { get; set; }

			[JsonPropertyName("t")]
			public TemplateId TemplateId { get; set; }

			[JsonPropertyName("i")]
			public List<int> IssueIds { get; set; } = new List<int>();
		}

		class ReportState
		{
			[JsonPropertyName("tm")]
			public DateTime Time { get; set; }

			[JsonPropertyName("msg")]
			public List<ReportBlock> Blocks { get; set; } = new List<ReportBlock>();
		}

		static string GetReportEventId(StreamId streamId, WorkflowId workflowId) => $"issue-report:{streamId}:{workflowId}";

		static string GetReportBlockEventId(string ts, int idx) => $"issue-report-block:{ts}:{idx}";

		/// <inheritdoc/>
		public async Task SendIssueReportAsync(IssueReportGroup group, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			foreach (IssueReport report in group.Reports.OrderBy(x => x.WorkflowId.Id.Text).ThenBy(x => x.StreamId.Id.Text))
			{
				await SendIssueReportForStreamAsync(buildConfig, group.Channel, group.Time, report, cancellationToken);
			}

			SlackMessage message = new SlackMessage();
			message.AddDivider();

			await SendMessageAsync(group.Channel, message, cancellationToken);
		}

		async Task SendIssueReportForStreamAsync(BuildConfig buildConfig, string channel, DateTime time, IssueReport report, CancellationToken cancellationToken)
		{
			const int MaxIssuesPerMessage = 8;

			StreamConfig? streamConfig;
			if (!buildConfig.TryGetStream(report.StreamId, out streamConfig))
			{
				return;
			}

			ReportState state = new ReportState();
			state.Time = time;

			List<List<(IIssue, IIssueSpan?, bool)>> issuesByBlock = new List<List<(IIssue, IIssueSpan?, bool)>>();
			if (report.GroupByTemplate)
			{
				Dictionary<int, IIssue> issueIdToInfo = new Dictionary<int, IIssue>();
				foreach (IIssue issue in report.Issues)
				{
					issueIdToInfo[issue.Id] = issue;
				}

				foreach (IGrouping<TemplateId, IIssueSpan> group in report.IssueSpans.GroupBy(x => x.TemplateRefId).OrderBy(x => x.Key.ToString()))
				{
					TemplateRefConfig? templateConfig;
					if (!streamConfig.TryGetTemplate(group.Key, out templateConfig))
					{
						continue;
					}
					if (!IsIssueOpenForWorkflow(report.StreamId, group.Key, report.WorkflowId, group))
					{
						continue;
					}

					Dictionary<IIssue, IIssueSpan> pairs = new Dictionary<IIssue, IIssueSpan>();
					foreach (IIssueSpan span in group)
					{
						if (issueIdToInfo.TryGetValue(span.IssueId, out IIssue? issue))
						{
							pairs[issue] = span;
						}
					}

					if (pairs.Count > 0)
					{
						bool templateHeader = true;
						foreach (IReadOnlyList<KeyValuePair<IIssue, IIssueSpan>> batch in pairs.OrderBy(x => x.Key.Id).Batch(MaxIssuesPerMessage))
						{
							ReportBlock block = new ReportBlock();
							block.TemplateHeader = templateHeader;
							block.StreamId = report.StreamId;
							block.TemplateId = group.Key;
							block.IssueIds.AddRange(batch.Select(x => x.Key.Id));
							state.Blocks.Add(block);

							issuesByBlock.Add(batch.Select(x => (x.Key, (IIssueSpan?)x.Value, true)).ToList());
							templateHeader = false;
						}
					}
				}
			}
			else
			{
				foreach (IReadOnlyList<IIssue> batch in report.Issues.OrderByDescending(x => x.Id).Batch(MaxIssuesPerMessage))
				{
					ReportBlock block = new ReportBlock();
					block.StreamId = report.StreamId;
					block.IssueIds.AddRange(batch.Select(x => x.Id));
					state.Blocks.Add(block);

					issuesByBlock.Add(batch.Select(x => (x, report.IssueSpans.FirstOrDefault(y => y.IssueId == x.Id), true)).ToList());
				}
			}

			SlackMessage headerMessage = new SlackMessage();
			headerMessage.AddHeader($"Summary for {streamConfig.Name}");

			TimeSpan rateLimitDelay = TimeSpan.FromSeconds(1.0);

			SlackMessageId? messageId = await SendMessageAsync(channel, headerMessage, cancellationToken);
			if (messageId != null)
			{
				string reportEventId = GetReportEventId(streamConfig.Id, report.WorkflowId);
				string json = JsonSerializer.Serialize(state, _jsonSerializerOptions);
				await AddOrUpdateMessageStateAsync(channel, reportEventId, null, json, messageId, cancellationToken);

				if (state.Blocks.Count == 0)
				{
					string header = ":tick: No issues open.";
					await SendMessageAsync(channel, header, cancellationToken);
				}

				for (int idx = 0; idx < state.Blocks.Count; idx++)
				{
					await Task.Delay(rateLimitDelay, cancellationToken);
					string blockEventId = GetReportBlockEventId(messageId.Ts, idx);
					await UpdateReportBlockAsync(channel, blockEventId, time, streamConfig, state.Blocks[idx].TemplateId, issuesByBlock[idx], report.TriageChannel, state.Blocks[idx].TemplateHeader, cancellationToken);
				}

				if (report.WorkflowStats.NumSteps > 0)
				{
					double totalPct = (report.WorkflowStats.NumPassingSteps * 100.0) / report.WorkflowStats.NumSteps;
					string header = $"*{totalPct:0.0}%* of build steps ({report.WorkflowStats.NumPassingSteps:n0}/{report.WorkflowStats.NumSteps:n0}) succeeded since last status update.";
					await SendMessageAsync(channel, header, cancellationToken);
				}

				await Task.Delay(rateLimitDelay, cancellationToken);
			}
		}

		static bool IsIssueOpenForWorkflow(StreamId streamId, TemplateId templateId, WorkflowId workflowId, IEnumerable<IIssueSpan> spans)
		{
			foreach (IIssueSpan span in spans)
			{
				if (span.NextSuccess == null && span.LastFailure.Annotations.WorkflowId == workflowId)
				{
					if ((streamId.IsEmpty || span.StreamId == streamId) && (templateId.IsEmpty || span.TemplateRefId == templateId))
					{
						return true;
					}
				}
			}
			return false;
		}

		async Task UpdateReportsAsync(BuildConfig buildConfig, IIssue issue, IReadOnlyList<IIssueSpan> spans, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Checking for report updates to issue {IssueId}", issue.Id);

			HashSet<string> reportEventIds = new HashSet<string>(StringComparer.Ordinal);
			foreach (IIssueSpan span in spans)
			{
				WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
				if (workflowId == null)
				{
					continue;
				}

				string reportEventId = GetReportEventId(span.StreamId, workflowId.Value);
				if (!reportEventIds.Add(reportEventId))
				{
					continue;
				}

				StreamConfig? streamConfig;
				if (!buildConfig.TryGetStream(span.StreamId, out streamConfig))
				{
					continue;
				}

				WorkflowConfig? workflowConfig;
				if (!streamConfig.TryGetWorkflow(workflowId.Value, out workflowConfig) || workflowConfig.ReportChannel == null)
				{
					continue;
				}

				MessageStateDocument? messageState = await GetMessageStateAsync(workflowConfig.ReportChannel, reportEventId, cancellationToken);
				if (messageState == null)
				{
					continue;
				}

				ReportState? state;
				try
				{
					state = JsonSerializer.Deserialize<ReportState>(messageState.Digest, _jsonSerializerOptions);
				}
				catch
				{
					continue;
				}

				if (state == null)
				{
					continue;
				}

				_logger.LogInformation("Updating report {EventId}", reportEventId);

				for (int idx = 0; idx < state.Blocks.Count; idx++)
				{
					ReportBlock block = state.Blocks[idx];
					if (block.IssueIds.Contains(issue.Id))
					{
						string blockEventId = GetReportBlockEventId(messageState.Ts, idx);
						_logger.LogInformation("Updating report block {EventId}", blockEventId);

						List<(IIssue, IIssueSpan?, bool)> issues = new List<(IIssue, IIssueSpan?, bool)>();
						foreach (int issueId in block.IssueIds)
						{
							IIssueDetails? details = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
							if (details != null)
							{
								IIssueSpan? otherSpan = details.Spans.FirstOrDefault(x => x.TemplateRefId == block.TemplateId);
								if (otherSpan == null && details.Spans.Count > 0)
								{
									otherSpan = details.Spans[0];
								}

								bool open = IsIssueOpenForWorkflow(streamConfig.Id, block.TemplateId, workflowId.Value, details.Spans);
								issues.Add((details.Issue, otherSpan, open));
							}
						}

						await UpdateReportBlockAsync(workflowConfig.ReportChannel, blockEventId, state.Time, streamConfig, block.TemplateId, issues, workflowConfig.TriageChannel, block.TemplateHeader, cancellationToken);
					}
				}
			}
		}

		async Task UpdateReportBlockAsync(string channel, string eventId, DateTime reportTime, StreamConfig streamConfig, TemplateId templateId, List<(IIssue, IIssueSpan?, bool)> issues, string? triageChannel, bool templateHeader, CancellationToken cancellationToken)
		{
			StringBuilder body = new StringBuilder();

			if (templateHeader && !templateId.IsEmpty)
			{
				TemplateRefConfig? templateConfig;
				if (streamConfig.TryGetTemplate(templateId, out templateConfig))
				{
					TabConfig? tab = streamConfig.Tabs.FirstOrDefault(x => x.Templates != null && x.Templates.Contains(templateId));
					if (tab != null)
					{
						Uri templateUrl = new Uri(_serverInfo.DashboardUrl, $"stream/{streamConfig.Id}?tab={tab.Title}&template={templateId}");
						body.Append($"*<{templateUrl}|{templateConfig.Name}>*:");
					}
				}
			}

			foreach ((IIssue issue, IIssueSpan? span, bool open) in issues)
			{
				if (body.Length > 0)
				{
					body.Append('\n');
				}

				string text = await FormatIssueAsync(issue, span, triageChannel, reportTime, open, cancellationToken);
				body.Append(text);
			}

			SlackMessage message = body.ToString();
			message.UnfurlLinks = false;
			message.UnfurlMedia = false;
			await SendOrUpdateMessageAsync(channel, eventId, null, message, cancellationToken);
		}

		string GetSeverityPrefix(IssueSeverity severity)
		{
			return (severity == IssueSeverity.Warning) ? _settings.SlackWarningPrefix : _settings.SlackErrorPrefix;
		}

		async ValueTask<string> FormatIssueAsync(IIssue issue, IIssueSpan? span, string? triageChannel, DateTime reportTime, bool open, CancellationToken cancellationToken)
		{
			Uri issueUrl = _serverInfo.DashboardUrl;
			if (span != null)
			{
				IIssueStep lastFailure = span.LastFailure;
				issueUrl = new Uri(issueUrl, $"job/{lastFailure.JobId}?step={lastFailure.StepId}&issue={issue.Id}");
			}

			IUser? owner = null;
			if (issue.OwnerId != null)
			{
				owner = await _userCollection.GetCachedUserAsync(issue.OwnerId.Value, cancellationToken);
			}

			string status = "*Unassigned*";
			if (issue.FixCommitId != null)
			{
				if (issue.Streams.Any(x => x.FixFailed ?? false))
				{
					status = "*Fix failed*";
				}
				else if (span != null && issue.Streams.Any(x => x.StreamId == span.StreamId && (x.MergeOrigin ?? false) && !(x.ContainsFix ?? false)))
				{
					status = "*Fix needs merging*";
				}
				else
				{
					status = $"Fixed in CL {issue.FixCommitId}";
				}
			}
			else if (issue.FixSystemic)
			{
				status = "Marked fixed as systemic";
			}
			else if (issue.OwnerId != null)
			{
				if (owner == null)
				{
					status = $"Assigned to {GetDefaultUserName(issue.OwnerId.Value)}";
				}
				else
				{
					status = $"Assigned to {owner.Name}";
				}
				if (issue.AcknowledgedAt == null)
				{
					status = $"{status} (pending)";
				}
			}

			if (issue.QuarantinedByUserId != null)
			{
				status = $"{status} - *Quarantined*";
			}

			string prefix = GetSeverityPrefix(issue.Severity);
			StringBuilder body = new StringBuilder($"{prefix}Issue *<{issueUrl}|{issue.Id}>");

			if (!String.IsNullOrEmpty(triageChannel))
			{
				MessageStateDocument? state = await GetMessageStateAsync(triageChannel, GetTriageThreadEventId(issue.Id), cancellationToken);
				if (state != null && state.Permalink != null)
				{
					body.Append($" (<{state.Permalink}|Thread>)");
				}
			}

			string? summary = issue.UserSummary ?? issue.Summary;
			body.Append($"*: {summary} [{FormatReadableTimeSpan(reportTime - issue.CreatedAt)}]");

			if (!String.IsNullOrEmpty(issue.ExternalIssueKey))
			{
				body.Append($" ({FormatExternalIssue(issue.ExternalIssueKey)})");
			}
			if (open)
			{
				body.Append($" - {status}");
			}

			string text = body.ToString();
			if (!open)
			{
				text = $"~{text}~";
			}
			return text;
		}

		static string FormatReadableTimeSpan(TimeSpan timeSpan)
		{
			double totalDays = timeSpan.TotalDays;
			if (totalDays > 1.0)
			{
				return $"{totalDays:n0}d";
			}

			double totalHours = timeSpan.TotalHours;
			if (totalHours > 1.0)
			{
				return $"{totalHours:n0}h";
			}

			return $"{timeSpan.TotalMinutes:n0}m";
		}

		#endregion

		#region Stream updates

		/// <inheritdoc/>
		public async Task NotifyConfigUpdateAsync(ConfigUpdateInfo info, CancellationToken cancellationToken)
		{
			if (String.IsNullOrEmpty(_settings.ConfigNotificationChannel))
			{
				return;
			}

			const string EventId = "config-update";

			Exception? ex = info.Exception;
			if (ex != null)
			{
				// Exception stacktraces can be really deep, which can lead to parsing issues on the log receiving end (OpenTelemetry, Datadog, etc)
				_logger.LogInformation(
					"Sending config update failure notification: {Message}, Exception: {ExceptionType} Stacktrace: {Stacktrace}", 
					ex.Message,
					ex.GetType().FullName,
					ex.StackTrace?.Length > 500 ? ex.StackTrace[..500] : ex.StackTrace);

				List<string> details = new List<string>();
				if (ex is ConfigException configEx)
				{
					ConfigContext context = configEx.GetContext();
					if (context.IncludeStack.TryPeek(out IConfigFile? blame))
					{
						string line = String.Empty;
						if (configEx.InnerException is JsonException jsonEx && jsonEx.LineNumber != null)
						{
							line = $"({jsonEx.LineNumber})";
						}

						string file = FormatConfigFileUri(blame.Uri);
						details.Add($"Error parsing `{file}{line}`:");
						details.Add($"```{ex.Message}```");
						details.Add("Include stack:\n```" + String.Join("\n", context.IncludeStack.Select(x => FormatConfigFileRevision(x))) + "```");

						if (blame.Uri.Scheme == PerforceConfigSource.Scheme)
						{
							string blameMessage = $"Possibly due to CL {blame.Revision}";
							if (blame.Author != null)
							{
								string userId = await FormatMentionAsync(blame.Author.Id, true, cancellationToken);
								blameMessage += $" ({userId})";
							}
							details.Add(blameMessage.ToString());
						}
					}
				}

				if (details.Count == 0)
				{
					details.Add(QuoteText(ex.Message));
				}

				string message = String.Join("\n", details);
				string digest = GetMessageDigest(message);

				MessageStateDocument? state = await GetMessageStateAsync(_settings.ConfigNotificationChannel, EventId, cancellationToken);
				if (state == null || state.Digest != digest)
				{
					SlackMessage header = new SlackMessage();
					header.AddHeader($"Config Update Error");
					await SendMessageAsync(_settings.ConfigNotificationChannel, header, cancellationToken);

					SlackMessageId? messageId = await SendMessageAsync(_settings.ConfigNotificationChannel, message, cancellationToken);
					await AddOrUpdateMessageStateAsync(_settings.ConfigNotificationChannel, EventId, null, digest, messageId, cancellationToken);
				}
			}
			else
			{
				SlackMessage message = new SlackMessage();
				message.AddSection($"*Config Update Succeeded*");
				if (info.Status.Count > 0)
				{
					message.AddSection(String.Join("\n", info.Status));
				}

				// Send a notification to the channel
				if (await DeleteMessageStateAsync(_settings.ConfigNotificationChannel, EventId, cancellationToken))
				{
					await SendMessageAsync(_settings.ConfigNotificationChannel, message, cancellationToken);
				}

				// Notify individual authors that their updates have succeeded.
				foreach (UserId authorId in info.Authors)
				{
					IUser? user = await _userCollection.GetUserAsync(authorId, cancellationToken);
					if (user != null)
					{
						string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
						if (slackUserId != null)
						{
							await SendMessageAsync(slackUserId, message, cancellationToken);
						}
					}
				}
			}
		}

		static string FormatConfigFileUri(Uri uri)
		{
			if (uri.Scheme == FileConfigSource.Scheme)
			{
				return Uri.UnescapeDataString(uri.AbsolutePath).Replace('/', Path.DirectorySeparatorChar);
			}
			else if (uri.Scheme == PerforceConfigSource.Scheme)
			{
				return uri.AbsolutePath;
			}
			else
			{
				return uri.ToString();
			}
		}

		static string FormatConfigFileRevision(IConfigFile file)
		{
			Uri uri = file.Uri;
			if (uri.Scheme == PerforceConfigSource.Scheme)
			{
				return $"{FormatConfigFileUri(uri)}@{file.Revision}";
			}
			else
			{
				return FormatConfigFileUri(uri);
			}
		}

		/// <inheritdoc/>
		public async Task NotifyConfigUpdateFailureAsync(string errorMessage, string fileName, int? change = null, IUser? author = null, string? description = null, CancellationToken cancellationToken = default)
		{
			_logger.LogInformation("Sending config update failure notification for {FileName} (change: {Change}, author: {UserId})", fileName, change ?? -1, author?.Id ?? UserId.Empty);

			string? slackUserId = null;
			if (author != null)
			{
				slackUserId = await GetSlackUserIdAsync(author, cancellationToken);
				if (slackUserId == null)
				{
					_logger.LogWarning("Unable to identify Slack user id for {UserId}", author.Id);
				}
				else
				{
					_logger.LogInformation("Mappsed user {UserId} to Slack user {SlackUserId}", author.Id, slackUserId);
				}
			}

			if (slackUserId != null)
			{
				await SendConfigUpdateFailureMessageAsync(slackUserId, errorMessage, fileName, change, slackUserId, description, cancellationToken);
			}
			if (_settings.UpdateStreamsNotificationChannel != null)
			{
				await SendConfigUpdateFailureMessageAsync($"#{_settings.UpdateStreamsNotificationChannel}", errorMessage, fileName, change, slackUserId, description, cancellationToken);
			}
		}

		private async Task SendConfigUpdateFailureMessageAsync(string recipient, string errorMessage, string fileName, int? change = null, string? author = null, string? description = null, CancellationToken cancellationToken = default)
		{
			string outcomeColor = ErrorColor;
			SlackAttachment attachment = new SlackAttachment();
			attachment.Color = outcomeColor;
			attachment.FallbackText = $"Update Failure: {fileName}";

			attachment.AddHeader($"Config Update Failure :rip:", true);

			attachment.AddSection($"Horde was unable to update {fileName}");
			attachment.AddSection(QuoteText(errorMessage));
			if (change != null)
			{
				if (author != null)
				{
					attachment.AddSection($"Possibly due to CL: {change.Value} by <@{author}>");
				}
				else
				{
					attachment.AddSection($"Possibly due to CL: {change.Value} - (Could not determine author from P4 user)");
				}
				if (description != null)
				{
					attachment.AddSection(QuoteText(description));
				}
			}

			await SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#endregion

		#region Device notifications

		/// <inheritdoc/>
		public async Task SendDeviceIssueReportAsync(DeviceIssueReport report, CancellationToken cancellationToken)
		{
			if (report.PoolReports.Count > 0)
			{
				int count = 0;

				foreach (DevicePoolReport pool in report.PoolReports)
				{
					List<SectionBlock> blocks = new List<SectionBlock>();
					foreach (DevicePoolMetrics metrics in pool.Metrics)
					{
						StringBuilder builder = new StringBuilder();

						string badge = YellowgBadge;

						if (metrics.AverageLoadPercentage >= 40 || metrics.MaxConcurrentProblemsPercentage >= 50)
						{
							badge = RedBadge;
						}
						else if (metrics.AverageLoadPercentage >= 20 || metrics.MaxConcurrentProblemsPercentage >= 30)
						{
							badge = OrangeBadge;
						}
						else if (metrics.Total == (metrics.Disabled + metrics.Maintenance))
						{
							badge = BlackBadge;
						}
						else if (metrics.Problems == 0 && metrics.SaturationSpikes == 0)
						{
							continue;
						}
						builder.Append($"{badge} ");

						count++;

						builder.Append($"<{metrics.PlatformURL}|*{metrics.PlatformName}*>\n> ");
						builder.Append($"Average Load: {metrics.AverageLoadPercentage}% | ");

						if (metrics.SaturationSpikes > 0)
						{
							builder.Append($"Saturation Spikes: {metrics.SaturationSpikes} | ");
							builder.Append($"Average Spike Duration: {metrics.SpikeDurationAverage.ToString(@"hh\:mm")}({metrics.SpikeDurationPercentage}%) | ");
						}

						if (metrics.Problems > 0)
						{
							builder.Append($"Max Concurrent Problems: {metrics.MaxConcurrentProblems}({metrics.MaxConcurrentProblemsPercentage}%) | ");
						}

						builder.Append($"Devices: {metrics.Total} | ");
						if (metrics.Disabled > 0)
						{
							builder.Append($"Disabled: {metrics.Disabled} | ");
						}
						if (metrics.Maintenance > 0)
						{
							builder.Append($"Maintenance: {metrics.Maintenance} | ");
						}
						builder.Append($"Streams: {metrics.Streams.Count}");

						SectionBlock section = new SectionBlock(builder.ToString());
						blocks.Add(section);
					}
					if (blocks.Count > 0)
					{
						SlackMessage poolReportMessage = new SlackMessage();
						poolReportMessage.Blocks.Add(new SectionBlock($"<{pool.PoolURL}|*{pool.PoolName}*> *| Device Pool Health*"));
						poolReportMessage.Blocks.AddRange(blocks);
						poolReportMessage.AddDivider();
						await SendMessageAsync(report.Channel, poolReportMessage, cancellationToken);
					}
				}

				if (count == 0)
				{
					SlackMessage message = new SlackMessage();
					message.Blocks.Add(new SectionBlock("*Device Pool Health* | no outstanding usage to report"));
					message.AddDivider();
					await SendMessageAsync(report.Channel, message, cancellationToken);
				}
			}

			if (report.PlatformReports.Count > 0)
			{
				foreach (DevicePlatformReport platform in report.PlatformReports)
				{
					if (platform.DeviceReports.Count == 0)
					{
						continue;
					}

					SlackMessage message = new SlackMessage();
					message.Blocks.Add(new SectionBlock($"*{platform.PlatformName} | Device Problems Since Last Update*"));

					foreach (DeviceReport device in platform.DeviceReports.OrderByDescending(r => r.ProblemDelta))
					{
						string badge = device.CleaningTime != null ? BlueBadge : RedBadge;
						StringBuilder builder = new StringBuilder($"{badge} <{device.DevicePoolURL}|*{device.DeviceName}*> | {device.DeviceAddress} | {device.PoolName}");
						if (device.CleaningTime != null)
						{
							builder.Append($"\n> Cleaning for {(int) (device.CleaningTime.Value.TotalHours)} hours");
						}
						else
						{
							builder.Append($"\n> {device.ProblemDelta} Problem(s) | {device.ProblemPercent}% Rate");
						}

						if (!String.IsNullOrEmpty(device.LastProblemURL))
						{
							builder.Append($"\n> <{device.LastProblemURL}|{device.LastProblemDesc ?? "??"}>");
						}

						message.Blocks.Add(new SectionBlock(builder.ToString()));
					}

					message.AddDivider();
					await SendMessageAsync(report.Channel, message, cancellationToken);
				}
			}
		}

		/// <inheritdoc/>
		public async Task NotifyDeviceServiceAsync(string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default)
		{
			string? recipient = null;

			if (user != null)
			{
				string? slackRecipient = await GetSlackUserIdAsync(user, cancellationToken);

				if (slackRecipient == null)
				{
					_logger.LogError("NotifyDeviceServiceAsync - Unable to send user slack notification, user {UserId} slack user id not found", user.Id);
					return;
				}

				recipient = slackRecipient;
			}

			if (recipient != null)
			{
				_logger.LogDebug("Sending device service notification to {Recipient}", recipient);
				await SendDeviceServiceMessageAsync(recipient, message, device, pool, streamConfig, job, step, node, user, cancellationToken);
			}
		}

		/// <summary>
		/// Creates a Slack message for a device service notification
		/// </summary>
		/// <param name="recipient"></param>
		/// <param name="message"></param>
		/// <param name="device"></param>
		/// <param name="pool"></param>
		/// <param name="streamConfig"></param>
		/// <param name="job">The job that contains the step that completed.</param>
		/// <param name="step">The job step that completed.</param>
		/// <param name="node">The node for the job step.</param>
		/// <param name="user">The user to notify.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		private Task SendDeviceServiceMessageAsync(string recipient, string message, IDevice? device = null, IDevicePool? pool = null, StreamConfig? streamConfig = null, IJob? job = null, IJobStep? step = null, INode? node = null, IUser? user = null, CancellationToken cancellationToken = default)
		{
			if (user != null)
			{
				return SendMessageAsync(recipient, message, cancellationToken);
			}

			// truncate message to avoid slack error on message length
			if (message.Length > 150)
			{
				message = message.Substring(0, 146) + "...";
			}

			SlackAttachment attachment = new SlackAttachment();

			attachment.FallbackText = $"{message}";

			if (device != null && pool != null)
			{
				attachment.FallbackText += $" - Device: {device.Name} Pool: {pool.Name}";
			}

			attachment.AddHeader(message, false);

			if (streamConfig != null && job != null && step != null && node != null)
			{
				Uri jobStepLink = new Uri($"{_serverInfo.DashboardUrl}job/{job.Id}?step={step.Id}");
				Uri jobStepLogLink = new Uri($"{_serverInfo.DashboardUrl}log/{step.LogId}");

				attachment.FallbackText += $" - {streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name}";
				attachment.AddSection($"*<{jobStepLink}|{streamConfig.Name} - {GetJobChangeText(job)} - {job.Name} - {node.Name}>*");
				attachment.AddSection($"<{jobStepLogLink}|View Job Step Log>");
			}
			else
			{
				attachment.FallbackText += " - No job information (Gauntlet might need to be updated in stream)";
				attachment.AddSection("*No job information (Gauntlet might need to be updated in stream)*");
			}

			return SendMessageAsync(recipient, attachment, cancellationToken);
		}

		#endregion

		#region test health notification

		/// <inheritdoc/>
		public async Task NotifyTestHealthReportAsync(ITestHealthReport report, string recipient, string[]? carbonCopies, CancellationToken cancellationToken)
		{
			string eventId = GetTestHealthEventId(report);
			if (report.IsHealthy)
			{
				// make sure there is a thread to update if test is now healthy
				MessageStateDocument? prevState = await GetMessageStateAsync(recipient, eventId, cancellationToken);
				if (prevState is null)
				{
					return;
				}
			}

			BuildConfig buildConfig = _buildConfig.CurrentValue;
			StreamConfig? streamConfig;
			if (!buildConfig.TryGetStream(report.StreamId, out streamConfig))
			{
				return;
			}

			List<string> ccUsers = new List<string>();
			if (carbonCopies?.Any() ?? false)
			{
				foreach (string user in carbonCopies)
				{
					UserId userId;
					if (UserId.TryParse(user, out userId))
					{
						ccUsers.Add(await FormatMentionAsync(userId, true, cancellationToken));
					}
				}
			}

			SlackMessage message = new SlackMessage();
			string testHealthDetailsUrlLink = new Uri(_serverInfo.DashboardUrl, $"test-automation?stream={report.StreamId}&view=1&health={report.TestId}").ToString();

			string title = GenerateTestHealthSlackMessageTitle(report, streamConfig, testHealthDetailsUrlLink);
			message.Blocks.Add(new SectionBlock(title));
			if (ccUsers.Any())
			{
				message.Blocks.Add(new SectionBlock(String.Join(" ", ccUsers)));
			}

			(MessageStateDocument state, bool isNew) = await SendOrUpdateMessageAsync(recipient, eventId, null, message, cancellationToken);
			SlackMessageId threadId = state.MessageId;
			if (!isNew)
			{
				if (report.IsHealthy)
				{
					// mark fixed
					SlackMessage fixedMessage = new SlackMessage();
					fixedMessage.Text = $"Test Health was restored to {report.State}.";
					await _slackClient.PostMessageToThreadAsync(threadId, fixedMessage, cancellationToken);
					await _slackClient.AddReactionAsync(threadId, "tick", cancellationToken);

					// delete message state
					await DeleteMessageStateAsync(recipient, eventId, cancellationToken);
				}
				else
				{
					// ping
					int daysSinceLastUpdate = (int)(DateTime.UtcNow - report.LastUpdateDateUtc).TotalDays;
					SlackMessage pingMessage = new SlackMessage();
					pingMessage.Text = report.State == report.PreviousState ?
							$"No health improvement has been recorded for {daysSinceLastUpdate} days. Consider disabling the Test or set it under Audit." : $"Test health has changed from {report.PreviousState} to {report.State}.";
					await _slackClient.PostMessageToThreadAsync(threadId, pingMessage, cancellationToken);
				}
			}
		}

		static string GenerateTestHealthSlackMessageTitle(ITestHealthReport report, StreamConfig streamConfig, string testHealthDetailsUrlLink)
		{
			string state = report.PreviousState is not null && report.IsHealthy ? report.PreviousState : report.State;
			string title = $"*[{streamConfig.Name}]* {RedBadge} test <{testHealthDetailsUrlLink}|*{report.TestName}* health> has degraded to *{state}*";
			if (report.IsHealthy)
			{
				// strike title
				title = $"~{title}~";
			}
			return title;
		}

		static string GetTestHealthEventId(ITestHealthReport report) => $"test-health-{report.Id}";

		#endregion

		static string QuoteText(string text, int maxLength = MaxLineLength)
		{
			maxLength -= 6;
			if (text.Length > maxLength)
			{
				int length = text.LastIndexOf('\n', maxLength);
				if (length == 0)
				{
					text = text.Substring(0, maxLength - 7) + "...\n...";
				}
				else
				{
					text = text.Substring(0, length + 1) + "...";
				}
			}
			return $"```{text}```";
		}

		const int MaxJobStepEvents = 5;

		static string GetJobChangeText(IJob job)
		{
			if (job.PreflightCommitId == null)
			{
				return $"CL {job.CommitId}";
			}
			else
			{
				return $"Preflight CL {job.PreflightCommitId} against CL {job.CommitId}";
			}
		}

		static bool ShouldUpdateUser(SlackUserDocument? document)
		{
			if (document == null || document.Version < SlackUserDocument.CurrentVersion)
			{
				return true;
			}

			TimeSpan expiryTime;
			if (document.SlackUserId == null)
			{
				expiryTime = TimeSpan.FromMinutes(10.0);
			}
			else
			{
				expiryTime = TimeSpan.FromDays(1.0);
			}
			return document.Time + expiryTime < DateTime.UtcNow;
		}

		private async Task<string?> GetSlackUserIdAsync(IUser user, CancellationToken cancellationToken)
		{
			return (await GetSlackUserAsync(user, cancellationToken))?.SlackUserId;
		}

		private readonly HashSet<UserId> _usersWithoutEmail = new HashSet<UserId>();

		private async Task<SlackUserDocument?> GetSlackUserAsync(IUser user, CancellationToken cancellationToken)
		{
			string? email = user.Email;
			if (email == null)
			{
				if (_usersWithoutEmail.Add(user.Id))
				{
					_logger.LogInformation("Unable to find Slack user id for {UserId} ({Name}): No email address in user profile", user.Id, user.Name);
				}
				return null;
			}

			SlackUserDocument? userDocument;
			if (!_userCache.TryGetValue(email, out userDocument))
			{
				userDocument = await _slackUsers.Find(x => x.Id == user.Id).FirstOrDefaultAsync(cancellationToken);
				if (userDocument == null || ShouldUpdateUser(userDocument))
				{
					SlackUser? userInfo = await _slackClient.FindUserByEmailAsync(email, cancellationToken);
					if (userDocument == null || userInfo != null)
					{
						userDocument = new SlackUserDocument(user.Id, userInfo);
						await _slackUsers.ReplaceOneAsync(x => x.Id == user.Id, userDocument, new ReplaceOptions { IsUpsert = true }, cancellationToken);
					}
				}
				using (ICacheEntry entry = _userCache.CreateEntry(email))
				{
					entry.SlidingExpiration = TimeSpan.FromMinutes(10.0);
					entry.Value = userDocument;
				}
			}

			return userDocument;
		}

		private async Task<SlackMessageId?> SendMessageAsync(string recipient, SlackMessage message, CancellationToken cancellationToken)
		{
			if (!IsRecipientAllowed(recipient, message.Text))
			{
				return null;
			}
			else
			{
				return await _slackClient.PostMessageAsync(recipient, message, cancellationToken);
			}
		}

		bool IsRecipientAllowed(string recipient, string? description)
		{
			if (_allowUsers != null && !_allowUsers.Contains(recipient) && !recipient.StartsWith("#", StringComparison.Ordinal) && !recipient.StartsWith("C", StringComparison.Ordinal))
			{
				_logger.LogDebug("Suppressing message to {Recipient}: {Description}", recipient, description);
				return false;
			}
			return true;
		}

		private async Task<(MessageStateDocument, bool)> SendOrUpdateMessageAsync(string recipient, string eventId, UserId? userId, SlackMessage message, CancellationToken cancellationToken)
		{
			return await SendOrUpdateMessageToThreadAsync(recipient, eventId, userId, null, message, cancellationToken);
		}

		private static string GetMessageDigest(SlackMessage message)
		{
			return ContentHash.MD5(JsonSerializer.Serialize(message, new JsonSerializerOptions { DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull })).ToString();
		}

		private async Task<(MessageStateDocument, bool)> SendOrUpdateMessageToThreadAsync(string recipient, string eventId, UserId? userId, SlackMessageId? threadId, SlackMessage message, CancellationToken cancellationToken)
		{
			string requestDigest = GetMessageDigest(message);

			MessageStateDocument? prevState = await GetMessageStateAsync(recipient, eventId, cancellationToken);
			if (prevState != null && prevState.Digest == requestDigest)
			{
				return (prevState, false);
			}

			(MessageStateDocument state, bool isNew) = await AddOrUpdateMessageStateAsync(recipient, eventId, userId, requestDigest, null, cancellationToken);
			if (isNew)
			{
				_logger.LogInformation("Sending new slack message to {SlackUser} (state: {StateId}, threadMessageId: {ThreadTs})", recipient, state.Id, threadId?.ToString() ?? "n/a");

				SlackMessageId id;
				if (threadId == null)
				{
					id = await _slackClient.PostMessageAsync(recipient, message, cancellationToken);
				}
				else
				{
					id = await _slackClient.PostMessageToThreadAsync(threadId, message, cancellationToken);
				}

				state.Channel = id.Channel;
				state.Ts = id.Ts;

				await UpdateMessageStateAsync(state.Id, id, null, cancellationToken);
			}
			else if (!String.IsNullOrEmpty(state.Ts))
			{
				_logger.LogInformation("Updating existing slack message {StateId} for user {SlackUser} (messageId: {MessageId}, threadId: {ThreadTs})", state.Id, recipient, state.MessageId, threadId?.ToString() ?? "n/a");

				if (threadId == null)
				{
					await _slackClient.UpdateMessageAsync(state.MessageId, message, cancellationToken);
				}
				else
				{
					await _slackClient.UpdateMessageAsync(new SlackMessageId(threadId.Channel, threadId.Ts, state.Ts), message, cancellationToken);
				}
			}
			return (state, isNew);
		}

		async ValueTask EscalateAsync(CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			DateTime utcNow = DateTime.UtcNow;
			double time = (utcNow - DateTime.UnixEpoch).TotalSeconds;

			int[] issueIds = await _redisService.GetDatabase().SortedSetRangeByScoreAsync(s_escalateIssues, 0, time);
			if (issueIds.Length > 0)
			{
				_logger.LogInformation("Escalating issues for {Time} ({TimeSecs})", utcNow, time);
				foreach (int issueId in issueIds)
				{
					cancellationToken.ThrowIfCancellationRequested();
					try
					{
						double? nextTime = await EscalateSingleIssueAsync(buildConfig, issueId, utcNow, cancellationToken);
						if (nextTime == null)
						{
							_logger.LogInformation("Cancelling escalation for issue {IssueId}", issueId);
							await _redisService.GetDatabase().SortedSetRemoveAsync(s_escalateIssues, issueId);
						}
						else
						{
							_logger.LogInformation("Next escalation for issue {IssueId} is at timestamp {Time}", issueId, nextTime.Value);
							await _redisService.GetDatabase().SortedSetAddAsync(s_escalateIssues, issueId, nextTime.Value);
						}
					}
					catch (SlackException ex)
					{
						if (ex.Message.Equals("channel_not_found", StringComparison.Ordinal))
						{
							_logger.LogWarning(ex, "Slack channel not found while escalating issue {IssueId}; cancelling.", issueId);
						}
						else
						{
							_logger.LogError(ex, "Slack exception while escalating issue {IssueId}; cancelling.", issueId);
						}
							
						await _redisService.GetDatabase().SortedSetRemoveAsync(s_escalateIssues, issueId);
					}
				}
			}

			await EscalateIssuesForKeyAsync(s_escalateRootCauseAnalysis, time, utcNow, "root cause analysis",
				EscalateSingleRootCauseAnalysisAsync, cancellationToken);
		}

		async ValueTask EscalateIssuesForKeyAsync(RedisSortedSetKey<int> key, double time, DateTime utcNow, string notificationName,
			Func<int, DateTime, string, CancellationToken, Task<double?>> escalateSingleIssueAsync, CancellationToken cancellationToken)
		{
			int[] issueIds = await _redisService.GetDatabase().SortedSetRangeByScoreAsync(key, 0, time);
			if (issueIds.Length > 0)
			{
				_logger.LogInformation("Escalating {NotificationName} for {Time} ({TimeSecs})", notificationName, utcNow, time);
				foreach (int issueId in issueIds)
				{
					cancellationToken.ThrowIfCancellationRequested();
					try
					{
						double? nextTime = await escalateSingleIssueAsync(issueId, utcNow, notificationName, cancellationToken);
						if (nextTime == null)
						{
							_logger.LogInformation("Cancelling escalation for {NotificationName} {IssueId}", notificationName, issueId);
							await _redisService.GetDatabase().SortedSetRemoveAsync(key, issueId);
						}
						else
						{
							_logger.LogInformation("Next escalation for {NotificationName} {IssueId} is at timestamp {Time}", notificationName, issueId, nextTime.Value);
							await _redisService.GetDatabase().SortedSetAddAsync(key, issueId, nextTime.Value);
						}
					}
					catch (SlackException ex)
					{
						if (ex.Message.Equals("channel_not_found", StringComparison.Ordinal))
						{
							_logger.LogWarning(ex, "Slack channel not found while escalating {NotificationName} {IssueId}; cancelling.", notificationName, issueId);
						}
						else
						{
							_logger.LogError(ex, "Slack exception while escalating {NotificationName} {IssueId}; cancelling.", notificationName, issueId);
						}

						await _redisService.GetDatabase().SortedSetRemoveAsync(key, issueId);
					}
				}
			}
		}

		async Task<double?> EscalateSingleIssueAsync(BuildConfig buildConfig, int issueId, DateTime utcNow, CancellationToken cancellationToken)
		{
			IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
			if (issue == null)
			{
				return null;
			}

			IReadOnlyList<IIssueSpan> spans = await _issueService.Collection.FindSpansAsync(issueId, cancellationToken);
			if (spans.Count == 0)
			{
				return null;
			}

			if (issue.ResolvedById != null)
			{
				return null;
			}

			IIssueSpan span = spans[0];

			WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
			if (workflowId == null)
			{
				return null;
			}

			StreamConfig? streamConfig;
			if (!buildConfig.TryGetStream(span.StreamId, out streamConfig))
			{
				return null;
			}
			if (!streamConfig.TryGetWorkflow(workflowId.Value, out WorkflowConfig? workflow))
			{
				return null;
			}
			if (String.IsNullOrEmpty(workflow.TriageChannel) || workflow.EscalateAlias == null || workflow.EscalateTimes.Count == 0 || (!workflow.TriageWarnings && issue.Severity == IssueSeverity.Warning) || (!workflow.TriageErrors && issue.Severity == IssueSeverity.Error))
			{
				return null;
			}

			if (!IsIssueOpenForWorkflow(streamConfig.Id, span.TemplateRefId, workflow.Id, spans))
			{
				return null;
			}
			if (issue.IsMarkedFixed() && await GetFixFailedSpanAsync(issue, spans, cancellationToken) == null)
			{
				return null;
			}

			if (issue.QuarantineTimeUtc == null)
			{
				MessageStateDocument? state = await GetMessageStateAsync(workflow.TriageChannel, GetTriageThreadEventId(issueId), cancellationToken);
				if (state == null)
				{
					return null;
				}

				TimeSpan openTime = utcNow - issue.CreatedAt;

				string openTimeStr;
				if (openTime < TimeSpan.FromHours(1.0))
				{
					openTimeStr = $"{(int)openTime.TotalMinutes}m";
				}
				else if (openTime < TimeSpan.FromDays(1.0))
				{
					openTimeStr = $"{(int)openTime.TotalHours}h";
				}
				else
				{
					openTimeStr = $"{(int)openTime.TotalDays}d";
				}

				Uri issueUrl = GetIssueUrl(issue, span.FirstFailure);

				string message;
				if (issue.OwnerId == null)
				{
					message = $"{FormatUserOrGroupMention(workflow.EscalateAlias)} - Issue <{issueUrl}|{issue.Id}> has not been assigned after {openTimeStr}.";
				}
				else
				{
					message = $"{await FormatMentionAsync(issue.OwnerId.Value, workflow.AllowMentions, cancellationToken)} this issue requires your attention ({FormatUserOrGroupMention(workflow.EscalateAlias)} for vis).";
				}

				await _slackClient.PostMessageToThreadAsync(state.MessageId, message, cancellationToken);
			}

			DateTime nextEscalationTime = issue.CreatedAt;
			for (int idx = 0; ; idx++)
			{
				if (idx >= workflow.EscalateTimes.Count)
				{
					nextEscalationTime = utcNow + TimeSpan.FromMinutes(workflow.EscalateTimes[^1]);
					_logger.LogInformation("Next escalation time for issue {IssueId} is {NextEscalationTime} (now + {Time}m [{Idx}/{Count}])", issue.Id, nextEscalationTime, workflow.EscalateTimes[^1], workflow.EscalateTimes.Count, workflow.EscalateTimes.Count);
					break;
				}

				DateTime prevEscalationTime = nextEscalationTime;
				nextEscalationTime += TimeSpan.FromMinutes(workflow.EscalateTimes[idx]);

				if (nextEscalationTime > utcNow + TimeSpan.FromMinutes(5.0))
				{
					_logger.LogInformation("Next escalation time for issue {IssueId} is {NextEscalationTime} ({PrevEscalationTime} + {Time}m [{Idx}/{Count}])", issue.Id, nextEscalationTime, prevEscalationTime, workflow.EscalateTimes[idx], idx + 1, workflow.EscalateTimes.Count);
					break;
				}
			}

			return (nextEscalationTime - DateTime.UnixEpoch).TotalSeconds;
		}

		async Task<double?> EscalateSingleRootCauseAnalysisAsync(int issueId, DateTime utcNow, string notificationName, CancellationToken cancellationToken)
		{
			const string CanceledMessage = "Canceling escalation for {NotificationName} {IssueId} {Reason}";

			IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
			if (issue == null)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue");
				return null;
			}

			IReadOnlyList<IIssueSpan> spans = await _issueService.Collection.FindSpansAsync(issueId, cancellationToken);
			if (spans.Count == 0)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue Spans");
				return null;
			}

			if (issue.ResolvedAt == null)
			{
				_logger.LogInformation(CanceledMessage, notificationName , issueId, "Issue Not Resolved");
				return null;
			}

			if (issue.DuplicateIssueId != null)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Duplicate Issue");
				return null;
			}

			UserId? ownerId = issue.RootCauseOwnerId ?? issue.OwnerId;
			if (ownerId == null || ownerId == UserId.Empty)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "No Root Cause Owner");
				return null;
			}

			// RootCauseCategory is an optional input in the UI because the issue closer may not have the information 
			// when marking as fixed, but it is required to have a value for Root Cause Analysis to be considered closed.
			if (!String.IsNullOrEmpty(issue.RootCauseCategory))
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "RCA Completed");
				return null;
			}

			IIssueSpan span = spans[0];

			WorkflowId? workflowId = span.LastFailure.Annotations.WorkflowId;
			if (workflowId == null)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue WorkflowID");
				return null;
			}

			if (!_buildConfig.CurrentValue.TryGetStream(span.StreamId, out StreamConfig? streamConfig))
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue StreamConfig");
				return null;
			}
			if (!streamConfig.TryGetWorkflow(workflowId.Value, out WorkflowConfig? workflow))
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue StreamConfig Workflow");
				return null;
			}
			if (String.IsNullOrEmpty(workflow.TriageChannel) || workflow.EscalateRootCauseAnalysisTimes.Count == 0)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "No Channel or Escalation Times");
				return null;
			}

			if (!issue.IsMarkedFixed())
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Issue Not Fixed");
				return null;
			}

			MessageStateDocument? state = await GetMessageStateAsync(workflow.TriageChannel, GetTriageThreadEventId(issueId), cancellationToken);
			if (state == null)
			{
				_logger.LogInformation(CanceledMessage, notificationName, issueId, "Cannot Find Issue Message State");
				return null;
			}

			if (!_buildConfig.CurrentValue.EnableIssueRootCauseAnalysisNotifications)
			{
				// When notifications are disabled still allow notifications to resume for issues if enabled later
				return GetNextEscalationTime(issue, issue.ResolvedAt.Value, utcNow, workflow.EscalateRootCauseAnalysisTimes, notificationName);
			}

			if (DateTime.Now.DayOfWeek == DayOfWeek.Saturday || DateTime.Now.DayOfWeek == DayOfWeek.Sunday)
			{
				_logger.LogInformation("Skipping escalation message for {NotificationName} {IssueId} since it's a weekend", notificationName, issueId);
			}
			else
			{
				string mention = await FormatMentionAsync(ownerId.Value, workflow.AllowMentions, cancellationToken);

				_logger.LogInformation("Sending escalation message for {NotificationName} {IssueId} {User} {Channel}", notificationName, issueId, mention, state.MessageId);

				Uri issueUrl = GetIssueUrl(issue, span.FirstFailure);
				string message = $"{mention} this root cause analysis requires your attention - Issue <{issueUrl}|{issue.Id}> - Select ...More > Modify Issue and choose a category to complete the root cause analysis.";
				await _slackClient.PostMessageToThreadAsync(state.MessageId, message, cancellationToken);
			}

			return GetNextEscalationTime(issue, issue.ResolvedAt.Value, utcNow, workflow.EscalateRootCauseAnalysisTimes, notificationName);
		}

		double GetNextEscalationTime(IIssue issue, DateTime whence, DateTime utcNow, List<int> escalationTimes, string notificationName)
		{
			DateTime nextEscalationTime = whence;
			for (int idx = 0; ; idx++)
			{
				if (idx >= escalationTimes.Count)
				{
					nextEscalationTime = utcNow + TimeSpan.FromMinutes(escalationTimes[^1]);
					_logger.LogInformation("Next escalation time for {NotificationName} {IssueId} is {NextEscalationTime} (now + {Time}m [{Idx}/{Count}])",
						notificationName, issue.Id, nextEscalationTime, escalationTimes[^1], escalationTimes.Count, escalationTimes.Count);
					break;
				}

				DateTime prevEscalationTime = nextEscalationTime;
				nextEscalationTime += TimeSpan.FromMinutes(escalationTimes[idx]);

				if (nextEscalationTime > utcNow + TimeSpan.FromMinutes(5.0))
				{
					_logger.LogInformation("Next escalation time for {NotificationName} {IssueId} is {NextEscalationTime} ({PrevEscalationTime} + {Time}m [{Idx}/{Count}])",
						notificationName, issue.Id, nextEscalationTime, prevEscalationTime, escalationTimes[idx], idx + 1, escalationTimes.Count);
					break;
				}
			}

			return (nextEscalationTime - DateTime.UnixEpoch).TotalSeconds;
		}

		/// <inheritdoc/>
		async Task ExecuteAsync(CancellationToken stoppingToken)
		{
			if (String.IsNullOrEmpty(_settings.SlackSocketToken))
			{
				_logger.LogInformation("No Slack socket token configured; will not be able to respond to interactive messages.");
				return;
			}

			int exceptionCount = 0;
			while (!stoppingToken.IsCancellationRequested)
			{
				Stopwatch timer = Stopwatch.StartNew();
				try
				{
					Uri? webSocketUrl = await GetWebSocketUrlAsync(stoppingToken);
					if (webSocketUrl == null)
					{
						_logger.LogWarning("Unable to get Slack websocket URL. Pausing before retry.");
						await Task.Delay(TimeSpan.FromSeconds(5.0), stoppingToken);
						continue;
					}
					await HandleSocketAsync(webSocketUrl, stoppingToken);
				}
				catch (OperationCanceledException)
				{
					break;
				}
				catch (Exception ex)
				{
					if (timer.Elapsed.TotalSeconds > 60)
					{
						exceptionCount = 0;
					}
					else
					{
						exceptionCount++;
					}

					if (exceptionCount == 0)
					{
						_logger.LogInformation(ex, "Exception while updating Slack socket: {Message}", ex.Message);
					}
					else if (exceptionCount < 3)
					{
						_logger.LogWarning(ex, "Exception while updating Slack socket: {Message}", ex.Message);
					}
					else
					{
						_logger.LogError(ex, "Exception while updating Slack socket: {Message}", ex.Message);
					}

					await Task.Delay(TimeSpan.FromSeconds(Math.Max(0.0, 60.0 - timer.Elapsed.TotalSeconds)), stoppingToken);
				}
			}
		}

		/// <summary>
		/// Get the url for opening a websocket to Slack
		/// </summary>
		/// <param name="stoppingToken"></param>
		/// <returns></returns>
		private async Task<Uri?> GetWebSocketUrlAsync(CancellationToken stoppingToken)
		{
			using HttpClient client = new HttpClient();
			client.DefaultRequestHeaders.Add("Authorization", $"Bearer {_settings.SlackSocketToken}");

			using FormUrlEncodedContent content = new FormUrlEncodedContent(Array.Empty<KeyValuePair<string?, string?>>());
			HttpResponseMessage response = await client.PostAsync(new Uri("https://slack.com/api/apps.connections.open"), content, stoppingToken);

			byte[] responseData = await response.Content.ReadAsByteArrayAsync(stoppingToken);

			SocketResponse socketResponse = JsonSerializer.Deserialize<SocketResponse>(responseData)!;
			if (!socketResponse.Ok)
			{
				_logger.LogWarning("Unable to get websocket url: {Response}", Encoding.UTF8.GetString(responseData));
				return null;
			}

			_logger.LogInformation("Using Slack websocket url: {Url}", socketResponse.Url);
			return socketResponse.Url;
		}

		/// <summary>
		/// Handle the lifetime of a websocket connection
		/// </summary>
		/// <param name="socketUrl"></param>
		/// <param name="stoppingToken"></param>
		/// <returns></returns>
		private async Task HandleSocketAsync(Uri socketUrl, CancellationToken stoppingToken)
		{
			_logger.LogInformation("Opening Slack socket {Url}", socketUrl);

			using ClientWebSocket socket = new ClientWebSocket();
			await socket.ConnectAsync(socketUrl, stoppingToken);

			byte[] buffer = new byte[2048];

			bool disconnect = false;
			while (!disconnect)
			{
				stoppingToken.ThrowIfCancellationRequested();

				// Read the next message
				int length = 0;
				while (!disconnect)
				{
					if (length == buffer.Length)
					{
						Array.Resize(ref buffer, buffer.Length + 2048);
					}

					WebSocketReceiveResult result = await socket.ReceiveAsync(new ArraySegment<byte>(buffer, length, buffer.Length - length), stoppingToken);
					if (result.MessageType == WebSocketMessageType.Close)
					{
						return;
					}
					length += result.Count;

					if (result.EndOfMessage)
					{
						break;
					}
				}

				// Get the message data
				_logger.LogInformation("Slack event: {Message}", Encoding.UTF8.GetString(buffer, 0, length));
				EventMessage eventMessage = JsonSerializer.Deserialize<EventMessage>(buffer.AsSpan(0, length))!;

				// Prepare the response
				EventResponse response = new EventResponse();
				response.EnvelopeId = eventMessage.EnvelopeId;

				// Handle the message type
				if (eventMessage.Type != null)
				{
					string type = eventMessage.Type;
					if (type.Equals("hello", StringComparison.Ordinal))
					{
						// nop
					}
					else if (type.Equals("disconnect", StringComparison.Ordinal))
					{
						disconnect = true;
					}
					else if (type.Equals("interactive", StringComparison.Ordinal) && eventMessage.Payload != null)
					{
						try
						{
							response.Payload = await HandleInteractionMessageAsync(eventMessage.Payload, stoppingToken);
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Exception handling Slack interaction message: {Message}", ex.Message);
						}
					}
					else
					{
						_logger.LogDebug("Unhandled event type ({Type})", type);
					}
				}

				// Send the response
				await socket.SendAsync(JsonSerializer.SerializeToUtf8Bytes(response), WebSocketMessageType.Text, true, stoppingToken);
			}
		}
		/// <summary>
		/// Handle a button being clicked
		/// </summary>
		private async Task<object?> HandleInteractionMessageAsync(EventPayload payload, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			if (payload.User != null && payload.User.Id != null)
			{
				if (String.Equals(payload.Type, "block_actions", StringComparison.Ordinal))
				{
					foreach (ActionInfo action in payload.Actions)
					{
						if (action.Value != null)
						{
							Match? match;
							if (TryMatch(action.Value, @"^issue_(\d+)_([a-zA-Z]+)_([a-fA-F0-9]{24})$", out match))
							{
								int issueId = Int32.Parse(match.Groups[1].Value, NumberStyles.Integer, CultureInfo.InvariantCulture);
								string verb = match.Groups[2].Value;
								UserId userId = new UserId(BinaryId.Parse(match.Groups[3].Value));
								await HandleIssueDmResponseAsync(issueId, verb, userId, payload.User.Id, cancellationToken);
							}
							else if (TryMatch(action.Value, @"^issue_(\d+)_([a-zA-Z]+)$", out match) && payload.TriggerId != null)
							{
								int issueId = Int32.Parse(match.Groups[1].Value, NumberStyles.Integer, CultureInfo.InvariantCulture);
								string verb = match.Groups[2].Value;
								await HandleIssueChannelResponseAsync(issueId, verb, payload.User.Id, payload.TriggerId, cancellationToken);
							}
							else
							{
								_logger.LogWarning("Unrecognized action value: {Value}", action.Value);
							}
						}
					}
				}
				else if (String.Equals(payload.Type, "view_submission", StringComparison.Ordinal))
				{
					if (payload.View != null && payload.View.State != null && payload.View.CallbackId != null)
					{
						Match? match;
						if (TryMatch(payload.View.CallbackId, @"^issue_(\d+)_markfixed_([a-fA-F0-9]{24})$", out match))
						{
							int issueId = Int32.Parse(match.Groups[1].Value, NumberStyles.Integer, CultureInfo.InvariantCulture);

							UserId userId = UserId.Parse(match.Groups[2].Value);

							UserId? resolvedById = userId;
							if (payload.View.State.TryGetValue("fixed_by", "fixed_by_action", out string? fixedByStr))
							{
								UserId fixedById;
								if (UserId.TryParse(fixedByStr, out fixedById))
								{
									resolvedById = fixedById;
								}
							}

							string? fixChangeStr;
							CommitId? fixCommitId = null;
							if (payload.View.State.TryGetValue("fix_cl", "fix_cl_action", out fixChangeStr))
							{
								fixCommitId = new CommitId(fixChangeStr);
								_logger.LogInformation("Marking issue {IssueId} fixed by user {UserId} in {Change}", issueId, resolvedById, fixCommitId);
							}

							UserId? rootCauseOwnerId = null;
							if (payload.View.State.TryGetValue("rootcause_owner", "rootcause_owner_action", out string? rootCauseOwnerStr))
							{
								if (UserId.TryParse(rootCauseOwnerStr, out UserId rcaId))
								{
									rootCauseOwnerId = rcaId;
								}
							}

							string? rootChange;
							CommitId rootCommit = new CommitId("");
							if (payload.View.State.TryGetValue("rootcause_cl", "rootcause_cl_action", out rootChange))
							{
								if (rootChange != null && rootChange.Length > 0)
								{
									if (Int32.TryParse(rootChange.Trim(), out int result))
									{
										rootCommit = CommitId.FromPerforceChange(result);
										_logger.LogInformation("Setting issue {IssueId} root cause change by user {UserId} to {Change}", issueId, userId, rootCommit);
									}
									else
									{
										_logger.LogInformation("Setting issue {IssueId} root cause change by user {UserId} not set to invalid {Change}", issueId, userId, rootChange);
									}
								}
							}

							string? rootCategory;
							if (payload.View.State.TryGetValue("rootcause_category", "rootcause_category_action", out rootCategory))
							{
								if (rootCategory != null && rootCategory.Length > 0)
								{
									_logger.LogInformation("Setting issue {IssueId} root cause category by user {UserId} to {Category}", issueId, userId, rootCategory);
								}
								else
								{
									rootCategory = "";
								}
							}
							else
							{
								rootCategory = "";
							}

							string? rootSummary;
							if (payload.View.State.TryGetValue("rootcause_summary", "rootcause_summary_action", out rootSummary))
							{
								if (rootSummary != null && rootSummary.Length > 0)
								{
									_logger.LogInformation("Setting issue {IssueId} root cause summary by user {UserId} to {Summary}", issueId, userId, rootSummary);
								}
								else
								{
									rootSummary = "";
								}
							}
							else
							{
								rootSummary = "";
							}

							int? dupeIssueId = null;
							if (payload.View.State.TryGetValue("rootcause_dupeid", "rootcause_dupeid_action", out string? dupeIdStr))
							{
								if (dupeIdStr != null && dupeIdStr.Length > 0)
								{
									if (Int32.TryParse(dupeIdStr.Trim(), out int result))
									{
										dupeIssueId = result;
										_logger.LogInformation("Setting issue {IssueId} dupliicate issue id by user {UserId} to {DupeId}", issueId, userId, dupeIssueId);
									}
									else
									{
										_logger.LogInformation("Setting issue {IssueId} duplicate issue id by user {UserId} not set to invalid {DupeId}, clearing", issueId, userId, dupeIdStr);
										dupeIssueId = -1;
									}
								}
							}
							else
							{
								dupeIssueId = -1;
							}

							IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);

							string? summary = null;

							if (issue != null)
							{
								string curSummary = issue.Summary;
								bool markedDupe = curSummary.StartsWith("(Dupe) ", StringComparison.OrdinalIgnoreCase);
								if (!markedDupe && dupeIssueId.HasValue && dupeIssueId != -1)
								{
									curSummary = "(Dupe) " + curSummary;
								}
								else if (markedDupe && (!dupeIssueId.HasValue || dupeIssueId == -1))
								{
									curSummary = curSummary.Substring(7);
								}

								summary = curSummary;
							}

							// Note that the current issue OwnerId is passed in this payload to avoid reassigning the issue to the user resolving it, unintentionally.
							await _issueService.UpdateIssueAsync(issueId, summary: summary, ownerId: issue?.OwnerId ?? null, fixCommitId: fixCommitId, resolvedById: fixCommitId != null ? resolvedById : null, initiatedById: userId, duplicateIssueId: dupeIssueId, rootCommitId: rootCommit, rootCauseOwnerId: rootCauseOwnerId, rootCauseCategory: rootCategory, rootCauseSummary: rootSummary, cancellationToken: cancellationToken);
						}
						else if (TryMatch(payload.View.CallbackId, @"^issue_(\d+)_ack_([a-fA-F0-9]{24})$", out match))
						{
							int issueId = Int32.Parse(match.Groups[1].Value, NumberStyles.Integer, CultureInfo.InvariantCulture);
							UserId userId = UserId.Parse(match.Groups[2].Value);
							await _issueService.UpdateIssueAsync(issueId, acknowledged: true, ownerId: userId, initiatedById: userId, cancellationToken: cancellationToken);
						}
					}
				}
			}
			return null;
		}

		static bool TryMatch(string input, string pattern, [NotNullWhen(true)] out Match? match)
		{
			Match result = Regex.Match(input, pattern);
			if (result.Success)
			{
				match = result;
				return true;
			}
			else
			{
				match = null;
				return false;
			}
		}

		async Task HandleIssueDmResponseAsync(int issueId, string verb, UserId userId, string userName, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Issue {IssueId}: {Action} from {SlackUser} ({UserId})", issueId, verb, userName, userId);

			if (String.Equals(verb, "ack", StringComparison.Ordinal))
			{
				await _issueService.UpdateIssueAsync(issueId, acknowledged: true, initiatedById: userId, cancellationToken: cancellationToken);
			}
			else if (String.Equals(verb, "accept", StringComparison.Ordinal))
			{
				await _issueService.UpdateIssueAsync(issueId, ownerId: userId, nominatedById: userId, acknowledged: true, initiatedById: userId, cancellationToken: cancellationToken);
			}
			else if (String.Equals(verb, "decline", StringComparison.Ordinal))
			{
				await _issueService.UpdateIssueAsync(issueId, declinedById: userId, initiatedById: userId, cancellationToken: cancellationToken);
			}

			IIssue? newIssue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
			if (newIssue != null)
			{
				IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
				if (user != null)
				{
					string? recipient = await GetSlackUserIdAsync(user, cancellationToken);
					if (recipient != null)
					{
						IIssueDetails details = await _issueService.GetIssueDetailsAsync(newIssue, cancellationToken);
						await SendIssueMessageAsync(_buildConfig.CurrentValue, recipient, newIssue, details, userId, DefaultAllowMentions, cancellationToken);
					}
				}
			}
		}

		async Task HandleIssueChannelResponseAsync(int issueId, string verb, string slackUserId, string triggerId, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Issue {IssueId}: {Action} from {SlackUser}", issueId, verb, slackUserId);

			SlackUser? slackUser = await _slackClient.GetUserAsync(slackUserId, cancellationToken);
			if (slackUser == null || slackUser.Profile == null || slackUser.Profile.Email == null)
			{
				_logger.LogWarning("Unable to find Slack user profile for {UserId}", slackUserId);
				return;
			}

			IUser? user = await _userCollection.FindUserByEmailAsync(slackUser.Profile.Email, cancellationToken);
			if (user == null)
			{
				_logger.LogWarning("Unable to find Horde user profile for {Email}", slackUser.Profile.Email);
				return;
			}

			if (String.Equals(verb, "ack", StringComparison.Ordinal))
			{
				IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
				if (issue == null)
				{
					_logger.LogWarning("Unable to find issue {IssueId}", issueId);
					return;
				}

				if (issue.OwnerId != null && issue.OwnerId != user.Id)
				{
					IUser? owner = await _userCollection.GetUserAsync(issue.OwnerId.Value, cancellationToken);
					if (owner != null)
					{
						SlackView view = new SlackView($"Issue {issueId}");
						view.CallbackId = $"issue_{issueId}_ack_{user.Id}";
						view.AddSection($"This issue is current assigned to {owner.Name}. Assign it to yourself?");
						view.Close = "Cancel";
						view.Submit = "Assign to Me";

						await _slackClient.OpenViewAsync(triggerId, view, cancellationToken);
						return;
					}
				}

				await _issueService.UpdateIssueAsync(issueId, acknowledged: true, ownerId: user.Id, nominatedById: user.Id, initiatedById: user.Id, cancellationToken: cancellationToken);
			}
			else if (String.Equals(verb, "decline", StringComparison.Ordinal))
			{
				IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);
				if (issue == null)
				{
					_logger.LogWarning("Unable to find issue {IssueId}", issueId);
					return;
				}

				if (issue.OwnerId != user.Id)
				{
					IReadOnlyList<IIssueSuspect> suspects = await _issueService.Collection.FindSuspectsAsync(issue, cancellationToken);
					if (!suspects.Any(x => x.AuthorId == user.Id))
					{
						SlackView view = new SlackView($"Issue {issueId}");
						view.AddSection("You are not currently listed as a suspect for this issue.");
						view.Close = "Cancel";

						await _slackClient.OpenViewAsync(triggerId, view, cancellationToken);
						return;
					}
				}

				await _issueService.UpdateIssueAsync(issueId, declinedById: user.Id, initiatedById: user.Id, cancellationToken: cancellationToken);
			}
			else if (String.Equals(verb, "markfixed", StringComparison.Ordinal))
			{
				SlackView view = new SlackView($"Issue {issueId}");
				view.CallbackId = $"issue_{issueId}_markfixed_{user.Id}";

				IIssue? issue = await _issueService.Collection.GetIssueAsync(issueId, cancellationToken);

				if (issue == null)
				{
					return;
				}

				if (issue.OwnerId != null && issue.OwnerId != user.Id)
				{
					IUser? owner = await _userCollection.GetCachedUserAsync(issue.OwnerId.Value, cancellationToken);
					if (owner != null)
					{
						List<SlackOption> ownerOptions = new List<SlackOption>();
						ownerOptions.Add(new SlackOption("Me", user.Id.ToString()));
						ownerOptions.Add(new SlackOption($"Current owner ({owner.Name})", owner.Id.ToString()));

						RadioButtonGroupElement ownership = new RadioButtonGroupElement("fixed_by_action", ownerOptions);
						ownership.InitialOption = ownerOptions[0];
						view.AddInput("Fixed By:", ownership).BlockId = "fixed_by";
					}
				}

				InputBlock fixBlock = view.AddInput("Fix Changelist:", new PlainTextInputElement("fix_cl_action", placeholder: "Fix Changelist"));
				fixBlock.BlockId = "fix_cl";
				fixBlock.Hint = "Change which fixed this issue. Enter 0 to mark fixed, without a change";

				view.AddDivider();

				List<SlackOption> categoryOptions = new List<SlackOption>
				{
				   new SlackOption( "Asset/Content", "Asset/Content"),
				   new SlackOption( "Code", "Code"),
				   new SlackOption( "UI/Localization", "UI/Localization"),
				   new SlackOption( "Build/Deployment/Version", "Build/Deployment/Version"),
				   new SlackOption( "Configuration/Data", "Configuration/Data"),
				   new SlackOption( "Gameplay/Engine", "Gameplay/Engine"),
				   new SlackOption( "Backend/Server/Network", "Backend/Server/Network"),
				   new SlackOption( "Monitoring/No Code Change", "Monitoring/No Code Change"),
				   new SlackOption( "Unintended Side Effect", "Unintended Side Effect"),
				   new SlackOption( "Build Agent", "Build Agent"),
				   new SlackOption( "Change Backout", "Change Backout"),
				   new SlackOption( "Other", "Other"),
				};

				StaticSelectMenuElement categorySelect = new StaticSelectMenuElement("rootcause_category_action", "Select an Issue Category", categoryOptions);
				if (!String.IsNullOrEmpty(issue.RootCauseCategory))
				{
					SlackOption? existing = categoryOptions.FirstOrDefault(i => i.Value == issue.RootCauseCategory);
					if (existing != null)
					{
						// Only can get, issue with Epic Slack
						//categorySelect.InitialOption = existing;
					}
				}

				if (issue.DuplicateIssueId == null)
				{
					List<SlackOption> rootCauseOwnerOptions =
					[
						new SlackOption("Me", user.Id.ToString())
					];
					if (issue.RootCauseOwnerId != null && issue.RootCauseOwnerId != user.Id)
					{
						IUser? rootCauseOwner = await _userCollection.GetCachedUserAsync(issue.RootCauseOwnerId.Value, cancellationToken);
						if (rootCauseOwner != null)
						{
							rootCauseOwnerOptions.Add(new SlackOption($"Current root cause owner ({rootCauseOwner.Name})", rootCauseOwner.Id.ToString()));
						}
					}
					if (issue.OwnerId != null && issue.OwnerId != user.Id && issue.OwnerId != issue.RootCauseOwnerId)
					{
						IUser? owner = await _userCollection.GetCachedUserAsync(issue.OwnerId.Value, cancellationToken);
						if (owner != null)
						{
							rootCauseOwnerOptions.Add(new SlackOption($"Current owner ({owner.Name})", owner.Id.ToString()));
						}
					}
					if (issue.ResolvedById != null && issue.ResolvedById != user.Id && issue.ResolvedById != issue.OwnerId && issue.ResolvedById != issue.RootCauseOwnerId)
					{
						IUser? resolvedBy = await _userCollection.GetCachedUserAsync(issue.ResolvedById.Value, cancellationToken);
						if (resolvedBy != null)
						{
							rootCauseOwnerOptions.Add(new SlackOption($"Fixed by ({resolvedBy.Name})", resolvedBy.Id.ToString()));
						}
					}

					RadioButtonGroupElement rootCauseOwnership = new("rootcause_owner_action", rootCauseOwnerOptions);
					//rootCauseOwnership.InitialOption = rootCauseOwnerOptions[0];
					InputBlock rootCauseOwnerBlock = view.AddInput("Root Cause Owner", rootCauseOwnership);
					rootCauseOwnerBlock.BlockId = "rootcause_owner";
					rootCauseOwnerBlock.Optional = true;

					InputBlock selectBlock = view.AddInput("Root Cause Category:", categorySelect);
					selectBlock.BlockId = "rootcause_category";
					selectBlock.Optional = true;

					InputBlock summaryBlock = view.AddInput("Root Cause Summary:", new PlainTextInputElement("rootcause_summary_action", multiLine: true, placeholder: "Please enter a brief summary describing the root cause of this issue and any suggestions to prevent it", initialValue: issue?.RootCauseSummary));
					summaryBlock.BlockId = "rootcause_summary";
					summaryBlock.Optional = true;

					InputBlock changeBlock = view.AddInput("Root Cause Changelist:", new PlainTextInputElement("rootcause_cl_action", placeholder: "Root Cause Changelist", initialValue: issue?.RootCommitId?.Name));
					changeBlock.BlockId = "rootcause_cl";
					changeBlock.Hint = "Change which caused this issue, please only enter if certain";
					changeBlock.Optional = true;
				}

				InputBlock dupeBlock = view.AddInput("Duplicate Issue Id:", new PlainTextInputElement("rootcause_dupeid_action", placeholder: "Duplicate Issue Id", initialValue: issue?.DuplicateIssueId?.ToString()));
				dupeBlock.BlockId = "rootcause_dupeid";
				dupeBlock.Optional = true;

				if (issue?.DuplicateIssueId != null)
				{
					dupeBlock.Hint = "Issue ID of the source issue this issue duplicates.  Root cause inputs hidden for duplicate issue, leave blank and update to clear";
				}
				else
				{
					dupeBlock.Hint = "Issue ID of the source issue this issue duplicate. Enter 0 or an issue id to set this as a duplicate issue";
				}

				view.Close = "Cancel";
				view.Submit = "Update";

				await _slackClient.OpenViewAsync(triggerId, view, cancellationToken);
				return;
			}
		}

		/// <inheritdoc/>
		public async Task SendAgentReportAsync(AgentReport report, CancellationToken cancellationToken)
		{
			if (report.ConformLoop.Count == 0 && report.UpgradeLoop.Count == 0)
			{
				_logger.LogInformation("Skipping sending agent report to {Channel}; no current issues.", _settings.AgentNotificationChannel);
				return;
			}

			if (!String.IsNullOrEmpty(_settings.AgentNotificationChannel))
			{
				_logger.LogInformation("Sending agent report to {Channel}", _settings.AgentNotificationChannel);

				SlackMessage headerMessage = new SlackMessage();
				headerMessage.AddHeader($"Agent status");
				await _slackClient.PostMessageAsync(_settings.AgentNotificationChannel, headerMessage, cancellationToken);

				{
					StringBuilder conformMessage = new StringBuilder("*Conform issues:*\n");
					if (report.ConformLoop.Count == 0)
					{
						conformMessage.Append("None.\n");
					}
					else
					{
						const int NumItems = 10;
						foreach ((AgentId agentId, int conformCount) in report.ConformLoop.OrderByDescending(x => x.Item2).ThenBy(x => x.Item1.ToString()).Take(NumItems))
						{
							Uri agentUrl = new Uri(_serverInfo.DashboardUrl, $"agents?agentId={agentId}");
							conformMessage.Append($"\u2022 *<{agentUrl}|{agentId}>* has run conform {conformCount} times\n");
						}
						if (report.ConformLoop.Count > NumItems)
						{
							conformMessage.Append($"+ {report.ConformLoop.Count - NumItems} other(s)\n");
						}
					}
					await _slackClient.PostMessageAsync(_settings.AgentNotificationChannel, conformMessage.ToString(), cancellationToken: cancellationToken);
				}

				{
					StringBuilder upgradeMessage = new StringBuilder("*Upgrade issues:*\n");
					if (report.UpgradeLoop.Count == 0)
					{
						upgradeMessage.Append("None.\n");
					}
					else
					{
						const int NumItems = 10;
						foreach ((AgentId agentId, int upgradeCount) in report.UpgradeLoop.OrderByDescending(x => x.Item2).ThenBy(x => x.Item1.ToString()).Take(NumItems))
						{
							Uri agentUrl = new Uri(_serverInfo.DashboardUrl, $"agents?agentId={agentId}");
							upgradeMessage.Append($"\u2022 *<{agentUrl}|{agentId}>* has attempted to upgrade {upgradeCount} times\n");
						}
						if (report.UpgradeLoop.Count > NumItems)
						{
							upgradeMessage.Append($"+ {report.UpgradeLoop.Count - NumItems} other(s)\n");
						}
					}
					await _slackClient.PostMessageAsync(_settings.AgentNotificationChannel, upgradeMessage.ToString(), cancellationToken);
				}
			}
		}

		/// <inheritdoc/>
		public async Task SendSessionConflictReportAsync(IReadOnlyList<(AgentId Id, int Count)> conflicts, CancellationToken cancellationToken)
		{
			if (conflicts.Count == 0 || String.IsNullOrEmpty(_settings.AgentNotificationChannel))
			{
				return;
			}

			_logger.LogInformation("Sending session conflict report ({Count} agents) to {Channel}", conflicts.Count, _settings.AgentNotificationChannel);

			const int NumItems = 10;
			StringBuilder message = new StringBuilder("*Session conflicts (last 12h):*\n");
			foreach ((AgentId agentId, int count) in conflicts.OrderByDescending(x => x.Count).ThenBy(x => x.Id.ToString()).Take(NumItems))
			{
				Uri agentUrl = new Uri(_serverInfo.DashboardUrl, $"agents?agentId={agentId}");
				message.Append($"\u2022 *<{agentUrl}|{agentId}>* \u2014 {count:N0} mismatches\n");
			}
			if (conflicts.Count > NumItems)
			{
				message.Append($"+ {conflicts.Count - NumItems} other(s)\n");
			}

			await _slackClient.PostMessageAsync(_settings.AgentNotificationChannel, message.ToString(), cancellationToken);
		}

		const string DeepLinkUrl = "https://slack.com/app_redirect";
		const int MaxDirectMessageUsers = 8;

		/// <inheritdoc/>
		public async Task<string?> GetDirectMessageLinkAsync(IReadOnlyList<UserId> userIds, CancellationToken cancellationToken)
		{
			if (userIds.Count == 0 || userIds.Count > MaxDirectMessageUsers)
			{
				_logger.LogWarning("Number of users is out of bound: {Count}. Slack supports 1 to 8 users for DM conversation.", userIds.Count);
				return null;
			}

			string?[] taskResults = await Task.WhenAll(userIds.Distinct().Select(u => GetSlackUserIdAsync(u, cancellationToken)));
			List<string> slackUserIds = taskResults.OfType<string>().ToList();
			if (slackUserIds.Count == 0)
			{
				_logger.LogWarning("Could not retrieve enough valid Slack user ids for DM conversation.");
				return null;
			}
			slackUserIds.Sort();
			string slackUserIdsString = String.Join(",", slackUserIds);

			string? channelId;
			if (!_dmChannelCache.TryGetValue(slackUserIdsString, out channelId))
			{
				channelId = await _slackClient.OpenDirectMessageAsync(slackUserIds, cancellationToken);
				if (channelId is null)
				{
					_logger.LogWarning("Could not open DM conversation for users {Users}.", slackUserIdsString);
					return null;
				}
				using (ICacheEntry entry = _dmChannelCache.CreateEntry(slackUserIdsString))
				{
					entry.SlidingExpiration = TimeSpan.FromMinutes(10.0);
					entry.Value = channelId;
					entry.Size = 1;
				}
			}

			return $"{DeepLinkUrl}?channel={channelId}";
		}

		async Task<string?> GetSlackUserIdAsync(UserId userId, CancellationToken cancellationToken)
		{
			IUser? user = await _userCollection.GetUserAsync(userId, cancellationToken);
			if (user == null)
			{
				return null;
			}

			string? slackUserId = await GetSlackUserIdAsync(user, cancellationToken);
			if (slackUserId == null)
			{
				return null;
			}

			return slackUserId;
		}

		/// <inheritdoc/>
		public Task<string?> GetChannelLinkAsync(string channelId, CancellationToken cancellationToken)
		{
			return Task.FromResult<string?>($"{DeepLinkUrl}?channel={UrlEncoder.Default.Encode(channelId)}");
		}
	}
}
