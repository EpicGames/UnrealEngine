// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Collection of subscription documents
	/// </summary>
	public class SubscriptionCollection : ISubscriptionCollection
	{
		[BsonKnownTypes(typeof(JobCompleteEvent), typeof(LabelCompleteEvent), typeof(StepCompleteEvent))]
		abstract class Event : IEvent
		{
			public static Event FromRecord(EventRecord record)
			{
				JobCompleteEventRecord? jobCompleteRecord = record as JobCompleteEventRecord;
				if (jobCompleteRecord != null)
				{
					return new JobCompleteEvent(jobCompleteRecord);
				}

				LabelCompleteEventRecord? labelCompleteRecord = record as LabelCompleteEventRecord;
				if (labelCompleteRecord != null)
				{
					return new LabelCompleteEvent(labelCompleteRecord);
				}

				StepCompleteEventRecord? stepCompleteRecord = record as StepCompleteEventRecord;
				if (stepCompleteRecord != null)
				{
					return new StepCompleteEvent(stepCompleteRecord);
				}

				throw new ArgumentException("Invalid record type");
			}

			public abstract EventRecord ToRecord();
		}

		class JobCompleteEvent : Event, IJobCompleteEvent
		{
			public StreamId StreamId { get; set; }
			public TemplateId TemplateId { get; set; }
			public LabelOutcome Outcome { get; set; }

			public JobCompleteEvent()
			{
			}

			public JobCompleteEvent(JobCompleteEventRecord record)
			{
				StreamId = new StreamId(record.StreamId);
				TemplateId = new TemplateId(record.TemplateId);
				Outcome = record.Outcome;
			}

			public override EventRecord ToRecord()
			{
				return new JobCompleteEventRecord(StreamId, TemplateId, Outcome);
			}

			public override string ToString()
			{
				return $"stream={StreamId}, template={TemplateId}, outcome={Outcome}";
			}
		}

		class LabelCompleteEvent : Event, ILabelCompleteEvent
		{
			public StreamId StreamId { get; set; }
			public TemplateId TemplateId { get; set; }
			public string? CategoryName { get; set; }
			public string LabelName { get; set; }
			public LabelOutcome Outcome { get; set; }

			public LabelCompleteEvent()
			{
				LabelName = String.Empty;
			}

			public LabelCompleteEvent(LabelCompleteEventRecord record)
			{
				StreamId = new StreamId(record.StreamId);
				TemplateId = new TemplateId(record.TemplateId);
				CategoryName = record.CategoryName;
				LabelName = record.LabelName;
				Outcome = record.Outcome;
			}

			public override EventRecord ToRecord()
			{
				return new LabelCompleteEventRecord(StreamId, TemplateId, CategoryName, LabelName, Outcome);
			}

			public override string ToString()
			{
				StringBuilder result = new StringBuilder();
				result.Append(CultureInfo.InvariantCulture, $"stream={StreamId}, template={TemplateId}");
				if (CategoryName != null)
				{
					result.Append(CultureInfo.InvariantCulture, $", category={CategoryName}");
				}
				result.Append(CultureInfo.InvariantCulture, $", label={LabelName}, outcome={Outcome}");
				return result.ToString();
			}
		}

		class StepCompleteEvent : Event, IStepCompleteEvent
		{
			public StreamId StreamId { get; set; }
			public TemplateId TemplateId { get; set; }
			public string StepName { get; set; }
			public JobStepOutcome Outcome { get; set; }

			public StepCompleteEvent()
			{
				StepName = String.Empty;
			}

			public StepCompleteEvent(StepCompleteEventRecord record)
			{
				StreamId = new StreamId(record.StreamId);
				TemplateId = new TemplateId(record.TemplateId);
				StepName = record.StepName;
				Outcome = record.Outcome;
			}

			public override EventRecord ToRecord()
			{
				return new StepCompleteEventRecord(StreamId, TemplateId, StepName, Outcome);
			}

			public override string ToString()
			{
				return $"stream={StreamId}, template={TemplateId}, step={StepName}, outcome={Outcome}";
			}
		}

		class Subscription : ISubscription
		{
			public string Id { get; set; }
			public Event Event { get; set; }
			public UserId UserId { get; set; }
			public NotificationType NotificationType { get; set; }

			IEvent ISubscription.Event => Event;

			[BsonConstructor]
			private Subscription()
			{
				Id = String.Empty;
				Event = null!;
			}

			public Subscription(NewSubscription subscription)
			{
				Event = Event.FromRecord(subscription.Event);
				UserId = subscription.UserId;
				NotificationType = subscription.NotificationType;
				Id = ContentHash.SHA1($"{Event}, user={UserId}, type={NotificationType}").ToString();
			}
		}

		readonly IMongoCollection<Subscription> _collection;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="mongoService">The database service</param>
		public SubscriptionCollection(IMongoService mongoService)
		{
			List<MongoIndex<Subscription>> indexes = new List<MongoIndex<Subscription>>();
			indexes.Add(keys => keys.Ascending(x => x.Event));
			indexes.Add(keys => keys.Ascending(x => x.UserId));
			_collection = mongoService.GetCollection<Subscription>("SubscriptionsV2", indexes);
		}

		/// <inheritdoc/>
		public async Task<List<ISubscription>> AddAsync(IEnumerable<NewSubscription> newSubscriptions)
		{
			List<Subscription> newDocuments = newSubscriptions.Select(x => new Subscription(x)).ToList();
			try
			{
				await _collection.InsertManyAsync(newDocuments, new InsertManyOptions { IsOrdered = false });
			}
			catch (MongoBulkWriteException ex)
			{
				foreach (WriteError writeError in ex.WriteErrors)
				{
					if (writeError.Category != ServerErrorCategory.DuplicateKey)
					{
						throw;
					}
				}
			}
			return newDocuments.ConvertAll<ISubscription>(x => x);
		}

		/// <inheritdoc/>
		public async Task RemoveAsync(IEnumerable<ISubscription> subscriptions)
		{
			FilterDefinition<Subscription> filter = Builders<Subscription>.Filter.In(x => x.Id, subscriptions.Select(x => ((Subscription)x).Id));
			await _collection.DeleteManyAsync(filter);
		}

		/// <inheritdoc/>
		public async Task<ISubscription?> GetAsync(string subscriptionId)
		{
			FilterDefinition<Subscription> filter = Builders<Subscription>.Filter.Eq(x => x.Id, subscriptionId);
			return await _collection.Find(filter).FirstOrDefaultAsync();
		}

		/// <inheritdoc/>
		public async Task<List<ISubscription>> FindSubscribersAsync(EventRecord eventRecord)
		{
			FilterDefinition<Subscription> filter = Builders<Subscription>.Filter.Eq(x => x.Event, Event.FromRecord(eventRecord));
			List<Subscription> results = await _collection.Find(filter).ToListAsync();
			return results.ConvertAll<ISubscription>(x => x);
		}

		/// <inheritdoc/>
		public async Task<List<ISubscription>> FindSubscriptionsAsync(UserId userId)
		{
			List<Subscription> results = await _collection.Find(x => x.UserId == userId).ToListAsync();
			return results.ConvertAll<ISubscription>(x => x);
		}
	}
}
