// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Request object for creating a new subscription
	/// </summary>
	public class CreateSubscriptionRequest
	{
		/// <summary>
		/// The event to create a subscription for
		/// </summary>
		[Required]
		public EventRecord Event { get; set; } = null!;

		/// <summary>
		/// The user to subscribe. Defaults to the current user if unspecified.
		/// </summary>
		[Required]
		public string UserId { get; set; } = null!;

		/// <summary>
		/// Type of notification to send
		/// </summary>
		public NotificationType NotificationType { get; set; }
	}

	/// <summary>
	/// Response from creating a subscription
	/// </summary>
	public class CreateSubscriptionResponse
	{
		/// <summary>
		/// The subscription id
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="subscription">The subscription to construct from</param>
		public CreateSubscriptionResponse(ISubscription subscription)
		{
			Id = subscription.Id;
		}
	}

	/// <summary>
	/// Response describing a subscription
	/// </summary>
	public class GetSubscriptionResponse
	{
		/// <summary>
		/// The subscription id
		/// </summary>
		public string Id { get; set; }

		/// <summary>
		/// Information about the specific event
		/// </summary>
		public EventRecord Event { get; set; }

		/// <summary>
		/// Unique id of the user subscribed to the event
		/// </summary>
		public string UserId { get; set; }

		/// <summary>
		/// The type of notification to receive
		/// </summary>
		public NotificationType NotificationType { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="subscription">Subscription to construct from</param>
		public GetSubscriptionResponse(ISubscription subscription)
		{
			Id = subscription.Id;
			Event = subscription.Event.ToRecord();
			UserId = subscription.UserId.ToString();
			NotificationType = subscription.NotificationType;
		}
	}

	/// <summary>
	/// Base class for event records
	/// </summary>
	[JsonKnownTypes(typeof(JobCompleteEventRecord), typeof(LabelCompleteEventRecord), typeof(StepCompleteEventRecord))]
	public class EventRecord
	{
	}

	/// <summary>
	/// Event that occurs when a job completes
	/// </summary>
	[JsonDiscriminator("Job")]
	public class JobCompleteEventRecord : EventRecord
	{
		/// <summary>
		/// The stream id
		/// </summary>
		[Required]
		public string StreamId { get; set; }

		/// <summary>
		/// The template id
		/// </summary>
		[Required]
		public string TemplateId { get; set; }

		/// <summary>
		/// Outcome of the job
		/// </summary>
		public LabelOutcome Outcome { get; set; } = LabelOutcome.Success;

		/// <summary>
		/// Default constructor
		/// </summary>
		public JobCompleteEventRecord()
		{
			StreamId = String.Empty;
			TemplateId = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="templateId">The template id</param>
		/// <param name="outcome">Outcome of the job</param>
		public JobCompleteEventRecord(StreamId streamId, TemplateId templateId, LabelOutcome outcome)
		{
			StreamId = streamId.ToString();
			TemplateId = templateId.ToString();
			Outcome = outcome;
		}
	}

	/// <summary>
	/// Event that occurs when a label completes
	/// </summary>
	[JsonDiscriminator("Label")]
	public class LabelCompleteEventRecord : EventRecord
	{
		/// <summary>
		/// The stream id
		/// </summary>
		[Required]
		public string StreamId { get; set; }

		/// <summary>
		/// The template id
		/// </summary>
		[Required]
		public string TemplateId { get; set; }

		/// <summary>
		/// Name of the category for this label
		/// </summary>
		public string? CategoryName { get; set; }

		/// <summary>
		/// Name of the label to monitor
		/// </summary>
		[Required]
		public string LabelName { get; set; }

		/// <summary>
		/// Outcome of the job
		/// </summary>
		public LabelOutcome Outcome { get; set; } = LabelOutcome.Success;

		/// <summary>
		/// Default constructor
		/// </summary>
		public LabelCompleteEventRecord()
		{
			StreamId = String.Empty;
			TemplateId = String.Empty;
			LabelName = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="templateId">The template id</param>
		/// <param name="categoryName">Name of the category</param>
		/// <param name="labelName">The label name</param>
		/// <param name="outcome">Outcome of the label</param>
		public LabelCompleteEventRecord(StreamId streamId, TemplateId templateId, string? categoryName, string labelName, LabelOutcome outcome)
		{
			StreamId = streamId.ToString();
			TemplateId = templateId.ToString();
			CategoryName = categoryName;
			LabelName = labelName;
			Outcome = outcome;
		}
	}

	/// <summary>
	/// Event that occurs when a step completes
	/// </summary>
	[JsonDiscriminator("Step")]
	public class StepCompleteEventRecord : EventRecord
	{
		/// <summary>
		/// The stream id
		/// </summary>
		[Required]
		public string StreamId { get; set; }

		/// <summary>
		/// The template id
		/// </summary>
		[Required]
		public string TemplateId { get; set; }

		/// <summary>
		/// Name of the step to monitor
		/// </summary>
		[Required]
		public string StepName { get; set; }

		/// <summary>
		/// Outcome of the job step
		/// </summary>
		public JobStepOutcome Outcome { get; set; } = JobStepOutcome.Success;

		/// <summary>
		/// Default constructor
		/// </summary>
		public StepCompleteEventRecord()
		{
			StreamId = String.Empty;
			TemplateId = String.Empty;
			StepName = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="templateId">The template id</param>
		/// <param name="stepName">The label name</param>
		/// <param name="outcome">Outcome of the step</param>
		public StepCompleteEventRecord(StreamId streamId, TemplateId templateId, string stepName, JobStepOutcome outcome)
		{
			StreamId = streamId.ToString();
			TemplateId = templateId.ToString();
			StepName = stepName;
			Outcome = outcome;
		}
	}
}
