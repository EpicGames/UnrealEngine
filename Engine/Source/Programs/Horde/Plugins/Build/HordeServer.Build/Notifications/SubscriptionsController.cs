// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Users;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Notifications
{
	/// <summary>
	/// Controller for the /api/v1/agents endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class SubscriptionsController : ControllerBase
	{
		readonly ISubscriptionCollection _subscriptionCollection;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public SubscriptionsController(ISubscriptionCollection subscriptionCollection, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_subscriptionCollection = subscriptionCollection;
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Find subscriptions matching a criteria
		/// </summary>
		/// <param name="userId">Name of the user</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <returns>List of subscriptions</returns>
		[HttpGet]
		[Route("/api/v1/subscriptions")]
		[ProducesResponseType(typeof(List<GetSubscriptionResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetSubscriptionsAsync([FromQuery] string userId, [FromQuery] PropertyFilter? filter = null)
		{
			UserId userIdValue;
			if (!TryParseUserId(userId, out userIdValue))
			{
				return BadRequest("Invalid user id");
			}
			if (!_buildConfig.Value.Acl.AuthorizeAsUser(User, userIdValue))
			{
				return Forbid();
			}

			List<ISubscription> results = await _subscriptionCollection.FindSubscriptionsAsync(userIdValue);
			return results.ConvertAll(x => PropertyFilter.Apply(new GetSubscriptionResponse(x), filter));
		}

		/// <summary>
		/// Find subscriptions matching a criteria
		/// </summary>
		/// <param name="subscriptionId">The subscription id</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <returns>List of subscriptions</returns>
		[HttpGet]
		[Route("/api/v1/subscriptions/{subscriptionId}")]
		[ProducesResponseType(typeof(GetSubscriptionResponse), 200)]
		public async Task<ActionResult<object>> GetSubscriptionAsync(string subscriptionId, [FromQuery] PropertyFilter? filter = null)
		{
			ISubscription? subscription = await _subscriptionCollection.GetAsync(subscriptionId);
			if (subscription == null)
			{
				return NotFound();
			}
			if (!_buildConfig.Value.Acl.AuthorizeAsUser(User, subscription.UserId))
			{
				return Forbid();
			}

			return PropertyFilter.Apply(new GetSubscriptionResponse(subscription), filter);
		}

		/// <summary>
		/// Remove a subscription
		/// </summary>
		/// <param name="subscriptionId">The subscription id</param>
		/// <returns>Async task</returns>
		[HttpDelete]
		[Route("/api/v1/subscriptions/{subscriptionId}")]
		[ProducesResponseType(typeof(List<GetSubscriptionResponse>), 200)]
		public async Task<ActionResult> DeleteSubscriptionAsync(string subscriptionId)
		{
			ISubscription? subscription = await _subscriptionCollection.GetAsync(subscriptionId);
			if (subscription == null)
			{
				return NotFound();
			}
			if (!_buildConfig.Value.Acl.AuthorizeAsUser(User, subscription.UserId))
			{
				return Forbid();
			}

			await _subscriptionCollection.RemoveAsync(new[] { subscription });
			return Ok();
		}

		/// <summary>
		/// Find subscriptions matching a criteria
		/// </summary>
		/// <param name="subscriptions">The new subscriptions to create</param>
		/// <returns>List of subscriptions</returns>
		[HttpPost]
		[Route("/api/v1/subscriptions")]
		public async Task<ActionResult<List<CreateSubscriptionResponse>>> CreateSubscriptionsAsync(List<CreateSubscriptionRequest> subscriptions)
		{
			HashSet<UserId> authorizedUsers = new HashSet<UserId>();

			UserId? currentUserId = User.GetUserId();
			if (currentUserId != null)
			{
				authorizedUsers.Add(currentUserId.Value);
			}

			List<NewSubscription> newSubscriptions = new List<NewSubscription>();
			foreach (CreateSubscriptionRequest subscription in subscriptions)
			{
				UserId newUserId;
				if (!TryParseUserId(subscription.UserId, out newUserId))
				{
					return BadRequest($"Invalid user id: '{subscription.UserId}'.");
				}
				if (authorizedUsers.Add(newUserId) && !_buildConfig.Value.Authorize(ServerAclAction.Impersonate, User))
				{
					return Forbid();
				}
				newSubscriptions.Add(new NewSubscription(subscription.Event, newUserId, subscription.NotificationType));
			}

			List<ISubscription> results = await _subscriptionCollection.AddAsync(newSubscriptions);
			return results.ConvertAll(x => new CreateSubscriptionResponse(x));
		}

		/// <summary>
		/// Parse a user id from a string. Allows passing the user's name as well as their objectid value.
		/// </summary>
		/// <param name="userName"></param>
		/// <param name="objectId"></param>
		/// <returns></returns>
		static bool TryParseUserId(string userName, out UserId objectId)
		{
			UserId newObjectId;
			if (UserId.TryParse(userName, out newObjectId))
			{
				objectId = newObjectId;
				return true;
			}
			else
			{
				objectId = default;
				return false;
			}
		}
	}
}
