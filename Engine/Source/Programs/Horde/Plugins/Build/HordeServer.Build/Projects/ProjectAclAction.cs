// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Projects
{
	/// <summary>
	/// ACL actions that can be performed on projects
	/// </summary>
	public static class ProjectAclAction
	{
		/// <summary>
		/// Allows the creation of new projects
		/// </summary>
		public static AclAction CreateProject { get; } = new AclAction("CreateProject");

		/// <summary>
		/// Allows deletion of projects.
		/// </summary>
		public static AclAction DeleteProject { get; } = new AclAction("DeleteProject");

		/// <summary>
		/// Modify attributes of a project (name, categories, etc...)
		/// </summary>
		public static AclAction UpdateProject { get; } = new AclAction("UpdateProject");

		/// <summary>
		/// View information about a project
		/// </summary>
		public static AclAction ViewProject { get; } = new AclAction("ViewProject");
	}
}
