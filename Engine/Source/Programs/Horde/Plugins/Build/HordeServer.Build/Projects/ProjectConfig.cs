// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Telemetry;
using HordeServer.Acls;
using HordeServer.Agents.Pools;
using HordeServer.Artifacts;
using HordeServer.Configuration;
using HordeServer.Devices;
using HordeServer.Logs;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Projects
{
	/// <summary>
	/// Stores configuration for a project
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/project")]
	[JsonSchemaCatalog("Horde Project", "Horde project configuration file", new[] { "*.project.json", "Projects/*.json" })]
	[ConfigDoc("*.project.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Projects.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	[DebuggerDisplay("{Id}")]
	public class ProjectConfig
	{
		/// <summary>
		/// The project id
		/// </summary>
		public ProjectId Id { get; set; }

		/// <summary>
		/// Name for the new project
		/// </summary>
		public string Name { get; set; } = null!;

		/// <summary>
		/// Direct include path for the project config. For backwards compatibility with old config files when including from a GlobalConfig object.
		/// </summary>
		[ConfigInclude, ConfigRelativePath]
		public string? Path { get; set; }

		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Macros within the global scope
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

		/// <summary>
		/// Order of this project on the dashboard
		/// </summary>
		public int Order { get; set; } = 128;

		/// <summary>
		/// Path to the project logo
		/// </summary>
		public ConfigResource? Logo { get; set; }

		/// <summary>
		/// Optional path to the project logo for the dark theme 
		/// </summary>
		public ConfigResource? LogoDarkTheme { get; set; }

		/// <summary>
		/// List of pools for this project
		/// </summary>
		public List<PoolConfig> Pools { get; set; } = new List<PoolConfig>();

		/// <summary>
		/// Categories to include in this project
		/// </summary>
		public List<ProjectCategoryConfig> Categories { get; set; } = new List<ProjectCategoryConfig>();

		/// <summary>
		/// Default settings for executing jobs
		/// </summary>
		public JobOptions JobOptions { get; set; } = new JobOptions();

		/// <summary>
		/// Default storage retention options for all streams in this project
		/// </summary>
		public RetentionOptions RetentionOptions { get; set; } = new RetentionOptions();

		/// <summary>
		/// Default workspace types for streams
		/// These are added to the list of each stream's workspace types.
		/// </summary>
		public Dictionary<string, WorkspaceConfig> WorkspaceTypes { get; set; } = [];

		/// <summary>
		/// Additional ignore pattern files applied to all streams in this project.
		/// Streams can append additional files via their own IgnorePatternFiles property.
		/// Paths are relative to workspace root.
		/// </summary>
		public List<string>? IgnorePatternFiles { get; set; }

		/// <summary>
		/// Telemetry store for Horde data for this project
		/// </summary>
		public TelemetryStoreId TelemetryStoreId { get; set; }

		/// <summary>
		/// List of streams
		/// </summary>
		public List<StreamConfig> Streams { get; set; } = new List<StreamConfig>();

		/// <summary>
		/// Permissions for artifact types
		/// </summary>
		public List<ArtifactTypeConfig> ArtifactTypes { get; set; } = new List<ArtifactTypeConfig>();

		/// <summary>
		/// Acl entries
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		readonly Dictionary<ArtifactType, ArtifactTypeConfig> _artifactTypeLookup = new Dictionary<ArtifactType, ArtifactTypeConfig>();

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);

		/// <summary>
		/// Callback after this configuration has been read
		/// </summary>
		/// <param name="id">Id of this project</param>
		/// <param name="parentAcl">The owning global config object</param>
		/// <param name="parentArtifactTypes">Set of artifact types</param>
		/// <param name="logger">Logger for messages</param>
		public void PostLoad(ProjectId id, AclConfig parentAcl, IEnumerable<ArtifactTypeConfig> parentArtifactTypes, ILogger? logger)
		{
			Id = id;
			AclAction[] aclActions = AclConfig.GetActions([typeof(ArtifactAclAction), typeof(DeviceAclAction), typeof(LogAclAction), typeof(ProjectAclAction)]);
			Acl.PostLoad(parentAcl, $"project:{Id}", aclActions);
			Acl.LegacyScopeNames = new AclScopeName[] { parentAcl.ScopeName.Append($"p:{Id}") };

			_artifactTypeLookup.Clear();
			foreach (ArtifactTypeConfig artifactTypeConfig in parentArtifactTypes)
			{
				_artifactTypeLookup[artifactTypeConfig.Type] = artifactTypeConfig;
			}
			foreach (ArtifactTypeConfig artifactTypeConfig in ArtifactTypes)
			{
				artifactTypeConfig.PostLoad(Acl);
				_artifactTypeLookup[artifactTypeConfig.Type] = artifactTypeConfig;
			}

			foreach (StreamConfig stream in Streams)
			{
				stream.PostLoad(stream.Id, this, _artifactTypeLookup.Values, logger);
			}
		}
	}

	/// <summary>
	/// Information about a category to display for a stream
	/// </summary>
	public class ProjectCategoryConfig
	{
		/// <summary>
		/// Name of this category
		/// </summary>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Index of the row to display this category on
		/// </summary>
		public int Row { get; set; }

		/// <summary>
		/// Whether to show this category on the nav menu
		/// </summary>
		public bool ShowOnNavMenu { get; set; }

		/// <summary>
		/// Patterns for stream names to include
		/// </summary>
		public List<string> IncludePatterns { get; set; } = new List<string>();

		/// <summary>
		/// Patterns for stream names to exclude
		/// </summary>
		public List<string> ExcludePatterns { get; set; } = new List<string>();
	}
}
