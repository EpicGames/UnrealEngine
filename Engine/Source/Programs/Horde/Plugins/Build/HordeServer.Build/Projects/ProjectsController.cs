// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.RegularExpressions;
using EpicGames.Horde.Projects;
using HordeServer.Configuration;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.AspNetCore.StaticFiles;
using Microsoft.Extensions.Options;

namespace HordeServer.Projects
{
	/// <summary>
	/// Controller for the /api/v1/projects endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[CppApi]
	[Route("[controller]")]
	public class ProjectsController : HordeControllerBase
	{
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		private static readonly FileExtensionContentTypeProvider s_contentTypeProvider = new FileExtensionContentTypeProvider();

		/// <summary>
		/// Constructor
		/// </summary>
		public ProjectsController(IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Query all the projects
		/// </summary>
		/// <param name="includeStreams">Whether to include streams in the response</param>
		/// <param name="includeCategories">Whether to include categories in the response</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Information about all the projects</returns>
		[HttpGet]
		[Route("/api/v1/projects")]
		[ProducesResponseType(typeof(List<GetProjectResponse>), 200)]
		public ActionResult<List<object>> GetProjects([FromQuery(Name = "Streams")] bool includeStreams = false, [FromQuery(Name = "Categories")] bool includeCategories = false, [FromQuery] PropertyFilter? filter = null)
		{
			BuildConfig buildConfig = _buildConfig.Value;

			List<object> responses = new List<object>();
			foreach (ProjectConfig projectConfig in buildConfig.Projects)
			{
				if (projectConfig.Authorize(ProjectAclAction.ViewProject, User))
				{
					List<StreamConfig>? visibleStreams = null;
					if (includeStreams || includeCategories)
					{
						visibleStreams = projectConfig.Streams.Where(x => x.Authorize(StreamAclAction.ViewStream, User)).ToList();
					}
					responses.Add(CreateGetProjectResponse(projectConfig, includeStreams, includeCategories, visibleStreams).ApplyFilter(filter));
				}
			}
			return responses;
		}

		/// <summary>
		/// Retrieve information about a specific project
		/// </summary>
		/// <param name="projectId">Id of the project to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/projects/{projectId}")]
		[ProducesResponseType(typeof(List<GetProjectResponse>), 200)]
		public ActionResult<object> GetProject(ProjectId projectId, [FromQuery] PropertyFilter? filter = null)
		{
			ProjectConfig? projectConfig;
			if (!_buildConfig.Value.TryGetProject(projectId, out projectConfig))
			{
				return NotFound(projectId);
			}
			if (!projectConfig.Authorize(ProjectAclAction.ViewProject, User))
			{
				return Forbid(ProjectAclAction.ViewProject, projectId);
			}

			bool includeStreams = PropertyFilter.Includes(filter, nameof(GetProjectResponse.Streams));
			bool includeCategories = PropertyFilter.Includes(filter, nameof(GetProjectResponse.Categories));

			List<StreamConfig>? visibleStreams = null;
			if (includeStreams || includeCategories)
			{
				visibleStreams = projectConfig.Streams.Where(x => x.Authorize(StreamAclAction.ViewStream, User)).ToList();
			}

			return CreateGetProjectResponse(projectConfig, includeStreams, includeCategories, visibleStreams).ApplyFilter(filter);
		}

		/// <summary>
		/// Retrieve information about a specific project
		/// </summary>
		/// <param name="projectId">Id of the project to get information about</param>
		/// <param name="darkTheme">Whether to prefer darktheme logo, if available</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/projects/{projectId}/logo")]
		public ActionResult<object> GetProjectLogo(ProjectId projectId, [FromQuery] bool darkTheme = false)
		{
			ProjectConfig? projectConfig;
			if (!_buildConfig.Value.TryGetProject(projectId, out projectConfig))
			{
				return NotFound(projectId);
			}
			if (!projectConfig.Authorize(ProjectAclAction.ViewProject, User))
			{
				return Forbid(ProjectAclAction.ViewProject, projectId);
			}

			ConfigResource? logoResource = darkTheme ? projectConfig.LogoDarkTheme ?? projectConfig.Logo : projectConfig.Logo;
			if (logoResource == null || logoResource.Path == null || logoResource.Data.Length == 0)
			{
				return NotFound("Missing logo resource data");
			}

			string? contentType;
			if (!s_contentTypeProvider.TryGetContentType(logoResource.Path, out contentType))
			{
				contentType = "application/octet-stream";
			}

			return new FileContentResult(logoResource.Data.ToArray(), contentType);
		}

		#region Messages

		internal static GetProjectResponse CreateGetProjectResponse(ProjectConfig projectConfig, bool includeStreams, bool includeCategories, List<StreamConfig>? streamConfigs)
		{
			GetProjectResponse response = new GetProjectResponse(projectConfig.Id, projectConfig.Name, projectConfig.Order);

			if (includeStreams)
			{
				response.Streams = streamConfigs!.ConvertAll(x => new GetProjectStreamResponse(x.Id.ToString(), x.Name));
			}

			if (includeCategories)
			{
				List<GetProjectCategoryResponse> categoryResponses = projectConfig.Categories.ConvertAll(x => CreateGetProjectCategoryResponse(x));
				if (streamConfigs != null)
				{
					foreach (StreamConfig streamConfig in streamConfigs)
					{
						GetProjectCategoryResponse? categoryResponse = categoryResponses.FirstOrDefault(x => MatchCategory(streamConfig.Name, x));
						if (categoryResponse == null)
						{
							int row = (categoryResponses.Count > 0) ? categoryResponses.Max(x => x.Row) : 0;
							if (categoryResponses.Count(x => x.Row == row) >= 3)
							{
								row++;
							}

							ProjectCategoryConfig otherCategory = new ProjectCategoryConfig();
							otherCategory.Name = "Other";
							otherCategory.Row = row;
							otherCategory.IncludePatterns.Add(".*");

							categoryResponse = CreateGetProjectCategoryResponse(otherCategory);
							categoryResponses.Add(categoryResponse);
						}
						categoryResponse.Streams!.Add(streamConfig.Id.ToString());
					}
				}
				response.Categories = categoryResponses;
			}

			return response;
		}

		internal static GetProjectCategoryResponse CreateGetProjectCategoryResponse(ProjectCategoryConfig streamCategory)
		{
			GetProjectCategoryResponse response = new GetProjectCategoryResponse(streamCategory.Name, streamCategory.Row);
			response.ShowOnNavMenu = streamCategory.ShowOnNavMenu;
			response.IncludePatterns.AddRange(streamCategory.IncludePatterns);
			response.ExcludePatterns.AddRange(streamCategory.ExcludePatterns);
			return response;
		}

		// Tests if a category response matches a given stream name
		static bool MatchCategory(string name, GetProjectCategoryResponse category)
		{
			if (category.IncludePatterns.Any(x => Regex.IsMatch(name, x)))
			{
				if (!category.ExcludePatterns.Any(x => Regex.IsMatch(name, x)))
				{
					return true;
				}
			}
			return false;
		}

		#endregion
	}
}
