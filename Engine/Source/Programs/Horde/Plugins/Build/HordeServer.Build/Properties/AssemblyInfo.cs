// Copyright Epic Games, Inc. All Rights Reserved.

using System.Resources;
using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("HordeServer.Tests")]
[assembly: InternalsVisibleTo("HordeServer.Build.Tests")]
[assembly: InternalsVisibleTo("HordeServer.Analytics.Integration.Tests")]
[assembly: NeutralResourcesLanguage("en-US")]
