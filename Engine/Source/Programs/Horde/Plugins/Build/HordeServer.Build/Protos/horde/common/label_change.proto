// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

package Horde.LabelChangeType;

option csharp_namespace = "HordeCommon";

enum RpcLabelChange {

	// The current changelist
	Current = 0;

	// The last code changelist
	Code = 1;
}
