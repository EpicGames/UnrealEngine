// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/empty.proto";
import "horde/job_rpc_messages.proto";
import "horde/log_rpc_messages.proto";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

service JobRpc {

	rpc UpdateAgentWorkspaces(RpcUpdateAgentWorkspacesRequest) returns (RpcUpdateAgentWorkspacesResponse);

	rpc GetJob(RpcGetJobRequest) returns (RpcGetJobResponse);
	rpc UpdateJob(RpcUpdateJobRequest) returns (google.protobuf.Empty);

	rpc BeginBatch(RpcBeginBatchRequest) returns (RpcBeginBatchResponse);
	rpc FinishBatch(RpcFinishBatchRequest) returns (google.protobuf.Empty);

	rpc BeginStep(RpcBeginStepRequest) returns (RpcBeginStepResponse);
	rpc UpdateStep(RpcUpdateStepRequest) returns (google.protobuf.Empty);
	rpc GetStep(RpcGetStepRequest) returns (RpcGetStepResponse);

	rpc UpdateGraph(RpcUpdateGraphRequest) returns (RpcUpdateGraphResponse);

	rpc CreateEvents(RpcCreateLogEventsRequest) returns (google.protobuf.Empty) {
		// Deprecated; use LogRpc.CreateLogEvents instead.
		option deprecated = true;
	}

	rpc UploadTestData(stream RpcUploadTestDataRequest) returns (RpcUploadTestDataResponse);

	rpc CreateArtifactV2(RpcCreateJobArtifactRequestV2) returns (RpcCreateJobArtifactResponseV2);
	rpc GetArtifact(RpcGetJobArtifactRequest) returns (RpcGetJobArtifactResponse);

	rpc CreateReport(RpcCreateReportRequest) returns (RpcCreateReportResponse);
}

