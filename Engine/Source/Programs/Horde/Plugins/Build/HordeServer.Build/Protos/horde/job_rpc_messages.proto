// Copyright Epic Games, Inc. All Rights Reserved.

syntax = "proto3";

import "google/protobuf/timestamp.proto";
import "google/protobuf/wrappers.proto";
import "horde/common/label_change.proto";
import "horde/messages/agent_workspace.proto";

package Horde;

option csharp_namespace = "Horde.Common.Rpc";

////// Agent Workspace Operations //////

message RpcUpdateAgentWorkspacesRequest {

	// The unique agent id
	string AgentId = 1;

	// The new list of workspaces for this agent
	repeated RpcAgentWorkspace Workspaces = 2;

	// Whether or not to remove untracked files
	bool RemoveUntrackedFiles = 3;
}

message RpcUpdateAgentWorkspacesResponse {

	// Whether there are still pending updates
	bool Retry = 1;

	// An updated list of workspaces for this agent
	repeated RpcAgentWorkspace PendingWorkspaces = 2;

	// Whether or not to remove untracked files
	bool RemoveUntrackedFiles = 3;
}

////// Job Operations //////

enum JobArtifactType {

	// Untyped artifact value
	Unknown = 0;

	// Buildgraph temp storage
	TempStorage = 1;

	// Saved trace data from the machine (logs, etc...)
	Saved = 2;

	// Test data
	TestData = 3;

	// Trace data
	Trace = 4;
}

message RpcCreateJobArtifactRequest {

	// The job identifier
	string JobId = 1;

	// The job step identifier
	string StepId = 2;

	// Artifact type
	JobArtifactType Type = 4;

	// (Optional) Namespace containing the artifact, if it already exists
	string NamespaceId = 5;

	// (Optional) Ref name for the artifact, if it already exists
	string RefName = 6;
}

message RpcCreateJobArtifactResponse {

	// Id for the new artifact
	string Id = 1;

	// Namespace for uploading data
	string NamespaceId = 2;

	// The ref name for the final root node
	string RefName = 3;

	// Token allowing the ref to be written
	string Token = 4;
}

message RpcCreateJobArtifactRequestV2 {

	// The job identifier
	string JobId = 1;

	// The job step identifier
	string StepId = 2;

	// Name for the artifact
	string Name = 7;

	// Artifact type
	string Type = 8;

	// Description for the artifact
	string Description = 9;

	// Keys for the new artifact
	repeated string Keys = 10;

	// Metadata for the new artifact
	repeated string Metadata = 11;
}

message RpcCreateJobArtifactResponseV2 {

	// Id for the new artifact
	string Id = 1;

	// Namespace for uploading data
	string NamespaceId = 2;

	// The ref name for the final root node
	string RefName = 3;

	// Token allowing the ref to be written
	string Token = 4;
}

message RpcGetJobArtifactRequest {

	// The job identifier
	string JobId = 3;

	// The job step identifier
	string StepId = 4;

	// Name for the artifact
	string Name = 1;

	// Artifact type
	string Type = 5;
}

message RpcGetJobArtifactResponse {

	// Id for the new artifact
	string Id = 1;

	// Namespace for uploading data
	string NamespaceId = 2;

	// The ref name for the final root node
	string RefName = 3;

	// Access token for reading the artifact
	string Token = 4;
}

////// Job Operations //////

message RpcGetJobRequest {

	// Unique id of the job to fetch properties for.
	string JobId = 1;
}

message RpcGetJobResponse {

	// The stream containing this job
	string StreamId = 1;

	// The changelist number to build. If zero, builds at head.
	int32 Change = 2;

	// The code changelist number
	int32 CodeChange = 6;

	// The user's preflight changelist
	int32 PreflightChange = 3;

	// String commit identifier (for non-Perforce version control systems)
	string CommitId = 4;

	// Cloned changelist to preflight
	reserved 7;
	reserved "ClonedPreflightChange";

	// String preflight commit identifier (for non-Perforce version control systems)
	string PreflightCommitId = 8;

	// List of arguments for this job
	repeated string Arguments = 5;
}

message RpcUpdateJobRequest {

	// Unique id of the stream
	string JobId = 1;

	// New name for the job
	google.protobuf.StringValue Name = 2;

	// New changelist number for the job
	int32 Change = 3;
}

////// Batch Operations //////

message RpcBeginBatchRequest {

	// Unique id of the job being executed.
	string JobId = 1;

	// Unique id of the batch being executed.
	string BatchId = 2;

	// Unique id of the lease held by the agent.
	string LeaseId = 3;
}

message RpcBeginBatchResponse {

	// Unique id of the log file for this batch
	string LogId = 1;

	// String commit identifier (for non-Perforce version control systems)
	string CommitId = 2;

	// The agent type to use for executing this batch
	string AgentType = 3;

	// String preflight commit identifier (for non-Perforce version control systems)
	string PreflightCommitId = 4;

	// Identifier for the stream containing the job
	string StreamId = 15;

	// Name of the stream for the job
	string StreamName = 6;

	// The changelist number to build. If zero, builds at head.
	int32 Change = 7;

	// The code changelist number
	int32 CodeChange = 8;

	// The user's preflight changelist
	int32 PreflightChange = 9;

	// Cloned changelist to preflight
	reserved 10;
	reserved "ClonedPreflightChange";

	// List of arguments for this job
	repeated string Arguments = 11;

	// Path to use for the temp storage directory
	string TempStorageDir = 12;

	// Environment variables to set
	map<string, string> Environment = 13;

	// Valid agent types for this stream.
	repeated string ValidAgentTypes = 14;

	// Path to the engine directory within the workspace
	string EnginePath = 16;

	// Additional ignore pattern files (relative to workspace root)
	repeated string IgnorePatternFiles = 17;

	// Whether expanded ignore pattern discovery is enabled (feature flag)
	bool EnableExpandedIgnorePatterns = 18;
}

message RpcFinishBatchRequest {

	// Unique id of the job being executed.
	string JobId = 1;

	// Unique id of the batch being executed.
	string BatchId = 2;

	// Unique id of the lease held by the agent.
	string LeaseId = 3;
}

////// Step Operations //////

message RpcBeginStepRequest {

	// Unique id of the job being executed.
	string JobId = 2;

	// Unique id of the batch being executed.
	string BatchId = 3;

	// Unique id of the lease held by the agent.
	string LeaseId = 4;
}

message RpcBeginStepResponse {

	enum Result {
		Unspecified = 0;
		Ready = 1;
		Waiting = 2;
		Complete = 3;
	}

	// Current state of the batch
	Result State = 1;

	// Unique id of the step being executed
	string StepId = 2;

	// Unique id of the log file for this batch
	string LogId = 3;

	// Name of this step
	string Name = 4;

	// Credentials for this step. Maps from environment variable name to secret information.
	map<string, string> Credentials = 5;

	// Properties on this step
	map<string, string> Properties = 6;

	// Additional environment variables specific to this step
	map<string, string> EnvVars = 8;

	// Whether to log warnings from this step
	google.protobuf.BoolValue Warnings = 7;

	// List of input names, in the form "node/output"
	repeated string Inputs = 9;

	// List of output names
	repeated string OutputNames = 10;

	// List of output indices required by other nodes
	repeated int32 PublishOutputs = 11;

	// List of artifacts produced by this step
	repeated RpcCreateGraphArtifactRequest Artifacts = 12;

	// List of optional input names, in the form "node/output"
	repeated string OptionalInputs = 13;
}

message RpcUpdateStepRequest {

	// Unique id of the job being executed.
	string JobId = 1;

	// Unique id of the batch being executed.
	string BatchId = 2;

	// Unique id of the step being executed.
	string StepId = 3;

	// The current state of the step (JobStepState)
	int32 State = 4;

	// The current outcome of the step (JobStepOutcome)
	int32 Outcome = 5;
}

message RpcGetStepRequest {

	// Unique id of the job being executed.
	string JobId = 1;

	// Unique id of the batch being executed.
	string BatchId = 2;

	// Unique id of the step being executed.
	string StepId = 3;
}

message RpcGetStepResponse {
	
	// The current state of the step (JobStepState)
	int32 State = 1;

	// The current outcome of the step (JobStepOutcome)
	int32 Outcome = 2;

	// True if the step should be aborted
	bool AbortRequested = 3;
}

/////// NODES //////

message RpcCreateNodeRequest {

	// The name of this node 
	string Name = 1;

	// Indices of nodes which must have succeeded for this node to run
	repeated string InputDependencies = 2;

	// Indices of nodes which must have completed for this node to run
	repeated string OrderDependencies = 3;

	// The priority of this node
	int32 Priority = 5;

	// This node can be run multiple times
	google.protobuf.BoolValue AllowRetry = 7;

	// This node can start running early, before dependencies of other nodes in the same group are complete
	google.protobuf.BoolValue RunEarly = 8;

	// Whether to ignore all warnings from this step
	google.protobuf.BoolValue Warnings = 12;

	// Credentials requested for this node
	map<string, string> Credentials = 10;

	// Properties for this node
	map<string, string> Properties = 11;

	// User annotations for this node from the BuildGraph script
	map<string, string> Annotations = 13;

	// List of inputs
	repeated string Inputs = 14;

	// List of outputs
	repeated string Outputs = 15;

	// List of optional inputs
	repeated string OptionalInputs = 16;

	// Indices of nodes which may have succeeded but not needed for this node to run
	repeated string OptionalInputDependencies = 17;
}

message RpcCreateGroupRequest {

	// The agent type for executing this group
	string AgentType = 2;

	// The nodes for this group
	repeated RpcCreateNodeRequest Nodes = 3;
}

message RpcCreateAggregateRequest {

	// Name of the aggregate
	string Name = 7;

	// Nodes that are required for this label to be shown
	repeated string Nodes = 8;
}

message RpcCreateLabelRequest {

	// Name to show for this label in the dashboard
	string DashboardName = 2;

	// Category for this label in the dashboard
	string DashboardCategory = 1;

	// Name to show for this badge in UGS
	string UgsName = 5;

	// Project that this badge applies to
	string UgsProject = 6;

	// The change to display for this label
	LabelChangeType.RpcLabelChange Change = 7;

	// Nodes that are required for this label to be shown
	repeated string RequiredNodes = 3;

	// Nodes which will be included in this label if present
	repeated string IncludedNodes = 4;
}

message RpcCreateGraphArtifactRequest {

	// Name of the artifact
	string Name = 1;

	// Type of the artifact
	string Type = 2;

	// Description for the artifact
	string Description = 3;

	// Base path for files in the artifact
	string BasePath = 4;

	// Keys for finding the artifact
	repeated string Keys = 6;

	// Metadata for the artifact
	repeated string Metadata = 7;

	// Name of the node producing this artifact
	string NodeName = 8;

	// Name of the output to include in the artifact
	string OutputName = 5;
}

message RpcUpdateGraphRequest {

	// Unique id of the job to update
	string JobId = 1;

	// The nodes to create
	repeated RpcCreateGroupRequest Groups = 2;

	// The aggregates to create
	repeated RpcCreateAggregateRequest Aggregates = 3;

	// The labels to create
	repeated RpcCreateLabelRequest Labels = 4;

	// The artifacts to create
	repeated RpcCreateGraphArtifactRequest Artifacts = 5;
}

message RpcUpdateGraphResponse {

}

////// Stream Operations //////

message RpcGetAgentTypeResponse {

	// Path to use for the temp storage directory
	string TempStorageDir = 3;

	// Environment variables to set
	map<string, string> Environment = 4;
}

message RpcGetStreamRequest {

	// Unique id of the stream to fetch properties for.
	string StreamId = 1;
}

message RpcGetStreamResponse {

	// Name of the stream
	string Name = 1;

	// Map of names to agent types
	map<string, RpcGetAgentTypeResponse> AgentTypes = 2;

	// List of properties for this stream
	map<string, string> Properties = 4;

	// Last time that we queried for commits.
	google.protobuf.Timestamp LastCommitTime = 5;
}

/// ARTIFACTS

message RpcUploadArtifactMetadata {

	// Job Id for the artifact
	string JobId = 1;

	// Unique id of the batch
	string BatchId = 2;

	// Step id for the artifact
	string StepId = 3;

	// Name for the artifact
	string Name = 4;

	// MimeType for the artifact
	string MimeType = 5;

	// Length of this file
	int64 Length = 6;
}

message RpcUploadArtifactRequest {

	oneof ArtifactChunk {

		// Job metadata
		RpcUploadArtifactMetadata Metadata = 1;

		// Chunk of data	
		bytes Data = 2;
	}
}

message RpcUploadArtifactResponse {

	// Id of the artifact
	string Id = 1;
}

/// TEST DATA

message RpcUploadTestDataRequest {

	// The job id
	string JobId = 1;

	// The job step id
	string JobStepId = 2;

	// The key for the data
	string Key = 3;

	// Encoded JSON data
	bytes Value = 4;
}

message RpcUploadTestDataResponse {
}

/// REPORTS

enum RpcReportScope {

	// Shown at step scope
	Step = 0;

	// Shown at job scope
	Job = 1;
}

enum RpcReportPlacement {

	// On a panel of its own
	Panel = 0;

	// In the summary panel
	Summary = 1;
}

message RpcCreateReportRequest {

	// The job id
	string JobId = 1;

	// The batch id
	string BatchId = 2;

	// The job step id
	string StepId = 3;

	// Scope to show the report at
	RpcReportScope Scope = 4;

	// Where to render the report
	RpcReportPlacement Placement = 7;

	// Name of the report
	string Name = 5;

	// The artifact id
	reserved 6;
	reserved "ArtifactId";

	// Content for this report
	string Content = 8;

	// The report message
	string Message = 9;

	// The report message severity
	string Severity = 10;
}

message RpcCreateReportResponse {
}
