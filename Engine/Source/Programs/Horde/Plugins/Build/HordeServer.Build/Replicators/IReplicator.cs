// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Replicators;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Snapshot of the state for a replicator; a process which transfers data from an external source control provider into Horde.
	/// </summary>
	public interface IReplicator
	{
		/// <summary>
		/// Identifier for this replicator
		/// </summary>
		ReplicatorId Id { get; }

		#region User requests

		/// <summary>
		/// Whether replication is paused
		/// </summary>
		bool Pause { get; }

		/// <summary>
		/// Whether to take a clean snapshot from the current change rather than intermentally syncing from the previous change
		/// </summary>
		bool Clean { get; }

		/// <summary>
		/// Whether to reset the change history from the current change
		/// </summary>
		bool Reset { get; }

		/// <summary>
		/// Pauses replication after the current change
		/// </summary>
		bool SingleStep { get; }

		/// <summary>
		/// The next change to replicate, requested by the user.
		/// </summary>
		int? NextChange { get; }

		#endregion

		#region Internal state

		/// <summary>
		/// The last change that was replicated
		/// </summary>
		int? LastChange { get; }

		/// <summary>
		/// Time at which the last change was replicated
		/// </summary>
		DateTime? LastChangeFinishTime { get; }

		/// <summary>
		/// The current change being replicated. May be null if no change is currently being replicated.
		/// </summary>
		int? CurrentChange { get; }

		/// <summary>
		/// Time at which the current change was replicated
		/// </summary>
		DateTime? CurrentChangeStartTime { get; }

		/// <summary>
		/// Total size of data to be replicated for the current change
		/// </summary>
		long? CurrentSize { get; }

		/// <summary>
		/// Size of data that has been replicated for the current change
		/// </summary>
		long? CurrentCopiedSize { get; }

		/// <summary>
		/// Last error with replication, if there is one.
		/// </summary>
		string? CurrentError { get; }

		#endregion

		/// <summary>
		/// Refreshes the replicator, ensuring it's the latest version
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New instance of the replicator, or null if it's been deleted</returns>
		Task<IReplicator?> RefreshAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to delete the replicator.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation.</param>
		/// <returns>True if the replicator was deleted</returns>
		Task<bool> TryDeleteAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to update the replicator
		/// </summary>
		/// <param name="options">Options for the update</param>
		/// <param name="cancellationToken">Cancellation token for the operation.</param>
		/// <returns>Updated replicator if the operation succeeded, null otherwise.</returns>
		Task<IReplicator?> TryUpdateAsync(UpdateReplicatorOptions options, CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Parameters for updating <see cref="IReplicator"/>
	/// </summary>
	public record class UpdateReplicatorOptions(bool? Pause = null, bool? Clean = null, bool? Reset = null, bool? SingleStep = null, int? LastChange = null, int? NextChange = null, int? CurrentChange = null, long? CurrentSize = null, long? CurrentCopiedSize = null, string? CurrentError = null);
}
