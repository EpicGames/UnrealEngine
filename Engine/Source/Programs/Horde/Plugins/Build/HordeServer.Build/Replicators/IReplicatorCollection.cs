// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Replicators;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Interface for a collection of replicators
	/// </summary>
	public interface IReplicatorCollection
	{
		/// <summary>
		/// Finds all replicators
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>All the currently active replicators</returns>
		Task<List<IReplicator>> FindAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the state of a replicator from the collection
		/// </summary>
		/// <param name="id">Identifier for the replicator</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The state of the requested replicator</returns>
		Task<IReplicator?> GetAsync(ReplicatorId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the state of a replicator or adds a new one to the collection
		/// </summary>
		/// <param name="id">Identifier for the replicator</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New replicator instance</returns>
		Task<IReplicator> GetOrAddAsync(ReplicatorId id, CancellationToken cancellationToken = default);
	}
}
