// Copyright Epic Games, Inc. All Rights Reserved.

using System.Buffers.Binary;
using System.Diagnostics;
using System.Security.Cryptography;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Replicators;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using EpicGames.Perforce;
using HordeServer.VersionControl.Perforce;
using HordeServer.Storage;
using HordeServer.Streams;
using Microsoft.Extensions.Logging;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Exception thrown to indicate that the state of a replicator has been modified externally
	/// </summary>
	public class ReplicatorModifiedException : Exception
	{
		internal ReplicatorModifiedException() : base("The replicator has been modified externally")
		{
		}
	}

	/// <summary>
	/// Replicates commits from Perforce into Horde's internal storage
	/// </summary>
	public class PerforceReplicator
	{
		[BlobConverter(typeof(StateNodeConverter))]
		class StateNode
		{
			public int Change { get; }
			public int ParentChange { get; }
			public IHashedBlobRef<CommitNode>? ParentHandle { get; }
			public long CopiedSize { get; set; }
			public IHashedBlobRef<DirectoryNode>? Contents { get; set; }
			public List<string> Paths { get; }

			public StateNode(int number, int parentNumber, IHashedBlobRef<CommitNode>? parentHandle, long copiedSize, IHashedBlobRef<DirectoryNode>? contents, List<string>? paths = null)
			{
				Change = number;
				ParentChange = parentNumber;
				ParentHandle = parentHandle;
				CopiedSize = copiedSize;
				Contents = contents;
				Paths = paths ?? new List<string>();
			}
		}

		class StateNodeConverter : BlobConverter<StateNode>
		{
			static readonly BlobType s_blobType = new BlobType("{8C874966-4273-2E89-9FAC-ABA46DC89154}", 2);

			public override StateNode Read(IBlobReader reader, BlobSerializerOptions options)
			{
				int change = (int)reader.ReadUnsignedVarInt();
				int parentChange = (int)reader.ReadUnsignedVarInt();

				IHashedBlobRef<CommitNode>? parentHandle = null;
				if (reader.ReadBoolean())
				{
					parentHandle = reader.ReadBlobRef<CommitNode>();
				}

				long copiedSize = 0;
				if (reader.Version >= 2)
				{
					copiedSize = (long)reader.ReadUnsignedVarInt();
				}

				IHashedBlobRef<DirectoryNode>? contents = null;
				if (reader.ReadBoolean())
				{
					contents = reader.ReadBlobRef<DirectoryNode>();
				}

				List<string> paths = reader.ReadList(() => reader.ReadString());
				return new StateNode(change, parentChange, parentHandle, copiedSize, contents, paths);
			}

			/// <inheritdoc/>
			public override BlobType Write(IBlobWriter writer, StateNode value, BlobSerializerOptions options)
			{
				writer.WriteUnsignedVarInt(value.Change);
				writer.WriteUnsignedVarInt(value.ParentChange);

				writer.WriteBoolean(value.ParentHandle != null);
				if (value.ParentHandle != null)
				{
					writer.WriteBlobRef(value.ParentHandle);
				}

				writer.WriteUnsignedVarInt((ulong)value.CopiedSize);

				writer.WriteBoolean(value.Contents != null);
				if (value.Contents != null)
				{
					writer.WriteBlobRef(value.Contents);
				}

				writer.WriteList(value.Paths, x => writer.WriteString(x));

				return s_blobType;
			}
		}

		// Partial mirror of FileSysType from P4 API (filesys.h)
		//		const uint FST_TEXT = 0x0001;
		const uint FST_BINARY = 0x0002;
		//		const uint FST_UNICODE = 0x000c;
		const uint FST_UTF16 = 0x000e;
		//		const uint FST_UTF8 = 0x000f;
		const uint FST_MASK = 0x000f;

		// Mirrors FilePerm from P4 API (filesys.h)
		const uint FPM_RO = 0;     // leave file read-only
								   //		const uint FPM_RW = 1;     // leave file read-write
		const uint FPM_ROO = 2;    // leave file read-only (owner)
		const uint FPM_RXO = 3;    // set file read-execute (owner) NO W
								   //		const uint FPM_RWO = 4;    // set file read-write (owner) NO X
		const uint FPM_RWXO = 5;   // set file read-write-execute (owner)

		[DebuggerDisplay("{Path}")]
		class DirectoryToSync
		{
			public readonly string Path;
			public readonly Dictionary<string, long> FileNameToSize;
			public long _size;

			public DirectoryToSync(string path, StringComparer comparer)
			{
				Path = path;
				FileNameToSize = new Dictionary<string, long>(comparer);
			}
		}

		record class FileInfo(string Path, FileEntryFlags Flags, long Length, byte[] Md5, LeafChunkedData LeafChunkedData);

		class FileWriter : IDisposable
		{
			class Handle : IDisposable
			{
				public string? _path;
				public FileEntryFlags _flags;
				public readonly LeafChunkedDataWriter FileWriter;
				public long _size;
				public long _sizeWritten;
				public readonly IncrementalHash Hash;

				public Handle(IBlobWriter writer, LeafChunkedDataNodeOptions options)
				{
					Hash = IncrementalHash.CreateHash(HashAlgorithmName.MD5);
					FileWriter = new LeafChunkedDataWriter(writer, options);
				}

				public void Dispose()
				{
					Hash.Dispose();
					FileWriter.Dispose();
				}
			}

			readonly IBlobWriter _writer;
			readonly LeafChunkedDataNodeOptions _options;
			readonly Stack<Handle> _freeHandles = new Stack<Handle>();
			readonly Dictionary<int, Handle> _openHandles = new Dictionary<int, Handle>();
			readonly ILogger _logger;

			public FileWriter(IBlobWriter writer, LeafChunkedDataNodeOptions options, ILogger logger)
			{
				_writer = writer;
				_options = options;
				_logger = logger;
			}

			public void Dispose()
			{
				foreach (Handle handle in _freeHandles)
				{
					handle.Dispose();
				}
				foreach (Handle handle in _openHandles.Values)
				{
					handle.Dispose();
				}
			}

			public void Open(int fd, string path, long size, FileEntryFlags flags)
			{
				Handle? handle;
				if (!_freeHandles.TryPop(out handle))
				{
					handle = new Handle(_writer, _options);
				}

				handle.FileWriter.Reset();
				handle._path = path;
				handle._size = size;
				handle._sizeWritten = 0;
				handle._flags = flags;

				_openHandles.Add(fd, handle);
			}

			public async Task AppendAsync(int fd, ReadOnlyMemory<byte> data, CancellationToken cancellationToken)
			{
				Handle handle = _openHandles[fd];
				handle.Hash.AppendData(data.Span);

				await handle.FileWriter.AppendAsync(data, cancellationToken);
				handle._sizeWritten += data.Length;
			}

			public async Task<FileInfo> CloseAsync(int fd, CancellationToken cancellationToken)
			{
				Handle handle = _openHandles[fd];
				if (handle._sizeWritten != handle._size)
				{
					_logger.LogWarning("Invalid size for replicated file '{Path}'. Expected {Size}, got {SizeWritten}.", handle._path, handle._size, handle._sizeWritten);
				}

				LeafChunkedData chunkedData = await handle.FileWriter.CompleteAsync(cancellationToken);
				byte[] hash = handle.Hash.GetHashAndReset();
				FileInfo info = new FileInfo(handle._path!, handle._flags, handle._size, hash, chunkedData);

				_openHandles.Remove(fd);
				_freeHandles.Push(handle);

				return info;
			}
		}

		class ReplicationClient
		{
			public PerforceSettings Settings { get; }
			public string ClusterName { get; }
			public InfoRecord ServerInfo { get; }
			public ClientRecord Client { get; }
			public int Change { get; set; }

			public ReplicationClient(PerforceSettings settings, string clusterName, InfoRecord serverInfo, ClientRecord client, int change)
			{
				Settings = settings;
				ClusterName = clusterName;
				ServerInfo = serverInfo;
				Client = client;
				Change = change;
			}
		}

		readonly Dictionary<StreamId, ReplicationClient> _cachedPerforceClients = new Dictionary<StreamId, ReplicationClient>();

		readonly IPerforceService _perforceService;
		readonly StorageService _storageService;
		readonly IReplicatorCollection _replicatorCollection;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceReplicator(IPerforceService perforceService, StorageService storageService, IReplicatorCollection replicatorCollection, ILogger<PerforceReplicator> logger)
		{
			_perforceService = perforceService;
			_storageService = storageService;
			_replicatorCollection = replicatorCollection;
			_logger = logger;
		}

		/// <summary>
		/// Gets the ref name for a given stream
		/// </summary>
		/// <param name="replicatorId">The stream to get a ref for</param>
		/// <returns>Ref name for the stream</returns>
		public static RefName GetRefName(ReplicatorId replicatorId) => new RefName($"{replicatorId.StreamId}/{replicatorId.StreamReplicatorId}");

		/// <summary>
		/// Gets the ref name for a given stream
		/// </summary>
		/// <param name="replicatorId">The stream to get a ref for</param>
		/// <returns>Ref name for the stream</returns>
		static RefName GetIncrementalRefName(ReplicatorId replicatorId) => new RefName($"{replicatorId.StreamId}/{replicatorId.StreamReplicatorId}/incremental");

		/// <summary>
		/// Runs a replication loop for a stream
		/// </summary>
		public async Task RunAsync(ReplicatorId replicatorId, BuildConfig buildConfig, StreamConfig streamConfig, ReplicatorConfig replicatorConfig, CancellationToken cancellationToken = default)
		{
			_logger.LogInformation("Starting replication background task for {ReplicatorId}", replicatorId);

			IReplicator replicator = await _replicatorCollection.GetOrAddAsync(replicatorId, cancellationToken);
			while (replicator.Pause)
			{
				await Task.Delay(TimeSpan.FromSeconds(30.0), cancellationToken);
				replicator = await _replicatorCollection.GetOrAddAsync(replicatorId, cancellationToken);
			}

			while (!cancellationToken.IsCancellationRequested)
			{
				replicator = await RunOnceAsync(replicator, buildConfig, streamConfig, replicatorConfig, cancellationToken);
			}
		}

		/// <summary>
		/// Runs the replicator for a single change
		/// </summary>
		public async Task<IReplicator> RunOnceAsync(IReplicator replicator, BuildConfig buildConfig, StreamConfig streamConfig, ReplicatorConfig replicatorConfig, CancellationToken cancellationToken)
		{
			RefName refName = GetRefName(replicator.Id);
			RefName incRefName = GetIncrementalRefName(replicator.Id);

			IStorageNamespace store = _storageService.GetNamespace(replicatorConfig.NamespaceId);

			ICommitCollection commits = _perforceService.GetCommits(streamConfig);

			if (replicator.Reset)
			{
				_logger.LogInformation("Resetting replication for {ReplicatorId}", replicator.Id);

				await store.RemoveRefAsync(refName, cancellationToken);
				await store.RemoveRefAsync(incRefName, cancellationToken);

				UpdateReplicatorOptions updateOptions = new UpdateReplicatorOptions { Reset = false, LastChange = 0, CurrentChange = 0 };
				replicator = await UpdateReplicatorAsync(replicator, updateOptions, cancellationToken);
			}

			if (replicator.NextChange != null)
			{
				_logger.LogInformation("Forcing current change for {ReplicatorId} to {Change}", replicator.Id, replicator.NextChange.Value);

				await store.RemoveRefAsync(incRefName, cancellationToken);

				UpdateReplicatorOptions updateOptions = new UpdateReplicatorOptions { NextChange = 0, CurrentChange = replicator.NextChange.Value };
				replicator = await UpdateReplicatorAsync(replicator, updateOptions, cancellationToken);
			}

			if (replicator.CurrentChange == null)
			{
				ICommit? nextCommit;
				if (replicator.LastChange != null)
				{
					nextCommit = await commits.SubscribeAsync(CommitIdWithOrder.FromPerforceChange(replicator.LastChange.Value), cancellationToken: cancellationToken).FirstAsync(cancellationToken);
					_logger.LogInformation("Replicating next change for {ReplicatorId} at CL {Change}", replicator.Id, nextCommit.Id);
				}
				else
				{
					nextCommit = await commits.GetLatestAsync(cancellationToken);
					_logger.LogInformation("Starting {ReplicatorId} replication from latest CL {Change}", replicator.Id, nextCommit.Id);
				}

				await store.RemoveRefAsync(incRefName, cancellationToken);

				UpdateReplicatorOptions updateOptions = new UpdateReplicatorOptions { CurrentChange = nextCommit.Id.GetPerforceChange() };
				replicator = await UpdateReplicatorAsync(replicator, updateOptions, cancellationToken);
			}

			int change = replicator.CurrentChange!.Value;
			_logger.LogInformation("Replicating {ReplicatorId} change {Change}", replicator.Id, change);

			BlobSerializerOptions blobOptions = new BlobSerializerOptions();
			try
			{
				replicator = await WriteInternalAsync(replicator, change, buildConfig, streamConfig, replicatorConfig, blobOptions, cancellationToken);
				return replicator;
			}
			catch (OperationCanceledException ex)
			{
				_logger.LogInformation(ex, "Replication task cancelled.");
				throw;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Replication error for {ReplicatorId}: {Message}", replicator.Id, ex.Message);

				for (; ; )
				{
					IReplicator? nextReplicator = await replicator.TryUpdateAsync(new UpdateReplicatorOptions { CurrentError = ex.Message }, cancellationToken);
					if (nextReplicator != null)
					{
						break;
					}
					replicator = await _replicatorCollection.GetOrAddAsync(replicator.Id, cancellationToken);
				}

				throw;
			}
		}

		async Task<IReplicator> WriteInternalAsync(IReplicator replicator, int change, BuildConfig buildConfig, StreamConfig streamConfig, ReplicatorConfig replicatorConfig, BlobSerializerOptions blobOptions, CancellationToken cancellationToken = default)
		{
			ChunkingOptions chunkingOptions = new();
			IStorageNamespace store = _storageService.GetNamespace(replicatorConfig.NamespaceId);

			// Find the parent node
			RefName refName = GetRefName(replicator.Id);

			// Read the current incremental state or create a new node to track the incremental state
			RefName incRefName = GetIncrementalRefName(replicator.Id);

			StateNode? stateNode = null;
			if (replicator.Clean)
			{
				stateNode = new StateNode(change, 0, null, 0, null, null);
			}
			else
			{
				stateNode = await store.TryReadRefTargetAsync<StateNode>(incRefName, options: blobOptions, cancellationToken: cancellationToken);
				if (stateNode != null)
				{
					if (stateNode.Change != change)
					{
						_logger.LogInformation("Incremental sync ref {RefName} has different changelist number {OldChange} vs {NewChange}", incRefName, stateNode.Change, change);
						stateNode = null;
					}
					else
					{
						_logger.LogInformation("Using incremental sync node {RefName}", incRefName);
					}
				}
				if (stateNode == null)
				{
					RedirectNode<CommitNode>? lastCommit = await store.TryReadRefTargetAsync<RedirectNode<CommitNode>>(refName, options: blobOptions, cancellationToken: cancellationToken);

					CommitNode? parent = null;
					IHashedBlobRef<CommitNode>? parentHandle = lastCommit?.Target;
					while (parentHandle != null)
					{
						CommitNode parentBlob = await parentHandle.ReadBlobAsync(cancellationToken);
						if (parentBlob.Number < change)
						{
							parent = parentBlob;
							break;
						}
						parentHandle = parentBlob.Parent;
					}

					stateNode = new StateNode(change, parent?.Number ?? 0, parentHandle, 0, parent?.Contents?.Handle, null);
				}
			}

			if (stateNode.Paths.Count > 0)
			{
				_logger.LogInformation("Current sync paths: {Paths}", String.Join("\n", stateNode.Paths));
			}

			// Get the root node
			DirectoryNode root;
			if (stateNode.Contents != null)
			{
				root = await stateNode.Contents.ReadBlobAsync(cancellationToken: cancellationToken);
			}
			else
			{
				root = new DirectoryNode();
			}

			// Create a client to replicate from this stream
			ReplicationClient clientInfo = await FindOrAddReplicationClientAsync(buildConfig, streamConfig, cancellationToken);

			// Connect to the server and flush the workspace
			using IPerforceConnection perforce = await PerforceConnection.CreateAsync(clientInfo.Settings, _logger);

			// Apply all the updates
			_logger.LogInformation("Syncing client {Client} from changelist {BaseChange} to {Change}", clientInfo.Client.Name, stateNode.ParentChange, change);
			await FlushWorkspaceAsync(clientInfo, perforce, stateNode.ParentChange, cancellationToken);
			clientInfo.Change = -1;

			string clientRoot = clientInfo.Client.Root;
			string queryPath = $"//{clientInfo.Client.Name}/...";

			// Replay the files that have already been synced
			foreach (string path in stateNode.Paths)
			{
				string flushPath = $"//{clientInfo.Client.Name}/{path}@{change}";
				_logger.LogInformation("Flushing {FlushPath}", flushPath);
				await perforce.SyncQuietAsync(SyncOptions.Force | SyncOptions.KeepWorkspaceFiles, -1, flushPath, cancellationToken);
			}

			// Add the root directory from the filter to the list of files to sync. This prevents traversing above it.
			Dictionary<string, DirectoryToSync> pathToDirectory = new Dictionary<string, DirectoryToSync>(clientInfo.ServerInfo.PathComparer);

			// Do a sync preview to find everything that's left, and sort the remaining list of paths
			await foreach (PerforceResponse<SyncRecord> response in perforce.StreamCommandAsync<SyncRecord>("sync", new[] { "-n" }, new string[] { $"{queryPath}@{change}" }, null, cancellationToken))
			{
				PerforceError? error = response.Error;
				if (error != null)
				{
					_logger.LogWarning("Perforce: {Message}", error.Data);
					continue;
				}

				string path = response.Data.Path.ToString();
				if (!path.StartsWith(clientRoot, StringComparison.Ordinal))
				{
					throw new ArgumentException($"Unable to make path {path} relative to client root {clientRoot}");
				}

				int fileIdx = path.Length;
				while (fileIdx > 0 && path[fileIdx - 1] != '/' && path[fileIdx - 1] != '\\')
				{
					fileIdx--;
				}

				string directoryPath = path.Substring(clientRoot.Length, fileIdx - clientRoot.Length);

				DirectoryToSync directory = FindOrAddDirectoryTree(pathToDirectory, directoryPath, clientInfo.ServerInfo.PathComparer);
				directory.FileNameToSize.Add(path.Substring(fileIdx), response.Data.FileSize);
				directory._size += response.Data.FileSize;
			}

			// Sort the directories by name to ensure that they are consistent between runs
			List<DirectoryToSync> directories = pathToDirectory.Values.OrderBy(x => x.Path, clientInfo.ServerInfo.PathComparer).ToList();

			// Output some stats for the sync
			long syncedSize = stateNode.CopiedSize;
			long remainingSize = directories.Sum(x => x._size);
			_logger.LogInformation("Remaining sync size: {Size:n1}mb", remainingSize / (1024.0 * 1024.0));
			long totalSize = syncedSize + remainingSize;
			_logger.LogInformation("Total sync size: {Size:n1}mb", totalSize / (1024.0 * 1024.0));

			// Create the tree writer
			Func<IBlobWriter> createWriter = () => store.CreateBlobWriter(refName);

			await using IBlobWriter directoryWriter = createWriter();
			await using IBlobWriter interiorChunkWriter = createWriter();
			await using IBlobWriter leafNodeWriter = createWriter();

			// Sync incrementally
			while (directories.Count > 0)
			{
				// Find the next paths to sync
				const long MaxBatchSize = 1L * 1024 * 1024 * 1024;

				int dirIdx = directories.Count - 1;
				long size = directories[dirIdx]._size;

				for (; dirIdx > 0; dirIdx--)
				{
					long nextSize = size + directories[dirIdx - 1]._size;
					if (size > 0 && nextSize > MaxBatchSize)
					{
						break;
					}
					size = nextSize;
				}

				syncedSize += size;
				double syncPct = (totalSize == 0) ? 100.0 : (syncedSize * 100.0) / totalSize;
				_logger.LogInformation("Syncing {StreamId} to {Change} [{SyncPct:n1}%] ({Size:n1}mb)", streamConfig.Id, change, syncPct, size / (1024.0 * 1024.0));

				// Update the replicator state
				UpdateReplicatorOptions progressUpdateOptions = new UpdateReplicatorOptions { CurrentSize = totalSize, CurrentCopiedSize = syncedSize };
				replicator = await UpdateReplicatorAsync(replicator, progressUpdateOptions, cancellationToken);

				// Keep track of changes to make to the directory structure
				DirectoryUpdate rootUpdate = new DirectoryUpdate();

				// Copy them to a separate list and remove any redundant paths
				List<string> syncPaths = new List<string>();
				for (int idx = dirIdx; idx < directories.Count; idx++)
				{
					string basePath = directories[idx].Path;
					syncPaths.Add($"//{clientInfo.Client.Name}/{basePath}...@{change}");

					long dirSize = directories[idx]._size;
					while (idx + 1 < directories.Count && directories[idx + 1].Path.StartsWith(basePath, clientInfo.ServerInfo.PathComparison))
					{
						dirSize += directories[idx + 1]._size;
						idx++;
					}

					_logger.LogInformation("  {Directory} ({Size:n1}mb)", directories[idx].Path + "...", dirSize / (1024.0 * 1024.0));
				}

				Stopwatch syncTimer = Stopwatch.StartNew();
				Stopwatch processTimer = new Stopwatch();
				Stopwatch gcTimer = new Stopwatch();

				using FileWriter fileWriter = new FileWriter(leafNodeWriter, chunkingOptions.LeafOptions, _logger);
				await foreach (PerforceResponse response in perforce.StreamCommandAsync("sync", Array.Empty<string>(), syncPaths, null, typeof(SyncRecord), true, default))
				{
					PerforceError? error = response.Error;
					if (error != null)
					{
						if (error.Generic == PerforceGenericCode.Empty)
						{
							continue;
						}
						else
						{
							throw new ReplicationException($"Perforce error while replicating content - {error}");
						}
					}

					processTimer.Start();

					PerforceIo? io = response.Io;
					if (io != null)
					{
						if (io.Command == PerforceIoCommand.Open)
						{
							UnpackOpenPayload(io.Payload, out Utf8String path, out int type, out int mode, out int perms);

							FileEntryFlags flags = 0;
							if ((type & FST_MASK) != FST_BINARY)
							{
								flags |= FileEntryFlags.Text;
							}
							if ((type & FST_UTF16) != 0)
							{
								flags |= FileEntryFlags.Utf16;
							}
							if (perms == FPM_RO || perms == FPM_ROO || perms == FPM_RXO)
							{
								flags |= FileEntryFlags.ReadOnly;
							}
							if (perms == FPM_RWXO || perms == FPM_RXO)
							{
								flags |= FileEntryFlags.Executable;
							}

							string file = GetClientRelativePath(path.ToString(), clientInfo.Client.Root);
							int offset = GetFileOffset(file);

							long fileSize = 0;
							if (!pathToDirectory.TryGetValue(file.Substring(0, offset), out DirectoryToSync? directory))
							{
								throw new ReplicationException($"Unable to find directory for {file}");
							}
							if (!directory.FileNameToSize.TryGetValue(file.Substring(offset), out fileSize))
							{
								throw new ReplicationException($"Unable to find file entry for {file}");
							}

							fileWriter.Open(io.File, file, fileSize, flags);
						}
						else if (io.Command == PerforceIoCommand.Write)
						{
							await fileWriter.AppendAsync(io.File, io.Payload, cancellationToken);
						}
						else if (io.Command == PerforceIoCommand.Close)
						{
							FileInfo info = await fileWriter.CloseAsync(io.File, cancellationToken);
							rootUpdate.AddFile(info.Path.ToString(), info.Flags, info.Length, info.LeafChunkedData, customData: info.Md5);
						}
						else if (io.Command == PerforceIoCommand.Unlink)
						{
							UnpackUnlinkPayload(io.Payload, out string path);
							string file = GetClientRelativePath(path, clientInfo.Client.Root);
							await root.DeleteFileByPathAsync(file, cancellationToken);
						}
						else
						{
							_logger.LogWarning("Unhandled command code {Code}", io.Command);
						}
					}

					processTimer.Stop();
				}

				TimeSpan stallTime = (perforce as NativePerforceConnection)?.StallTime ?? TimeSpan.Zero;
				_logger.LogInformation("Completed batch in {TimeSeconds:n1}s ({ProcessTimeSeconds:n1}s processing, {StallTimeSeconds:n1}s stalled, {Throughput:n1}mb/s)", syncTimer.Elapsed.TotalSeconds, processTimer.Elapsed.TotalSeconds, stallTime.TotalSeconds, size / (1024.0 * 1024.0 * syncTimer.Elapsed.TotalSeconds));

				// Combine all the existing sync paths together with a new wildcard.
				while (directories.Count > dirIdx)
				{
					string nextPath = directories[^1].Path;
					if (stateNode.Paths.Count > 0)
					{
						string lastPath = stateNode.Paths[^1];
						for (int endIdx = 0; endIdx < lastPath.Length; endIdx++)
						{
							if (lastPath[endIdx] == '/')
							{
								string prefix = lastPath.Substring(0, endIdx + 1);
								if (!nextPath.StartsWith(prefix, clientInfo.ServerInfo.PathComparison))
								{
									// Remove any paths that start with this prefix
									while (stateNode.Paths.Count > 0 && stateNode.Paths[^1].StartsWith(prefix, clientInfo.ServerInfo.PathComparison))
									{
										stateNode.Paths.RemoveAt(stateNode.Paths.Count - 1);
									}

									// Replace it with a wildcard
									stateNode.Paths.Add(prefix + "...");
									break;
								}
							}
						}
					}
					stateNode.Paths.Add(nextPath + "...");
					directories.RemoveAt(directories.Count - 1);
				}

				// Save the incremental state
				Stopwatch flushTimer = Stopwatch.StartNew();

				await leafNodeWriter.FlushAsync(cancellationToken);

				await rootUpdate.WriteInteriorNodesAsync(interiorChunkWriter, chunkingOptions.InteriorOptions, cancellationToken);
				await interiorChunkWriter.FlushAsync(cancellationToken);

				await root.UpdateAsync(rootUpdate, directoryWriter, cancellationToken);
				stateNode.Contents = await directoryWriter.WriteBlobAsync(root, cancellationToken);

				IHashedBlobRef<StateNode> stateNodeRef = await directoryWriter.WriteBlobAsync(stateNode, cancellationToken);
				await directoryWriter.FlushAsync(cancellationToken);

				await store.AddRefAsync(incRefName, stateNodeRef, cancellationToken: cancellationToken);
				rootUpdate.Clear();

				if (replicator.Clean || !String.IsNullOrEmpty(replicator.CurrentError))
				{
					UpdateReplicatorOptions cleanUpdateOptions = new UpdateReplicatorOptions { Clean = false, CurrentError = String.Empty };
					replicator = await UpdateReplicatorAsync(replicator, cleanUpdateOptions, cancellationToken);
				}

				flushTimer.Stop();
			}

			// Create the commit node
			await using IBlobWriter commitWriter = createWriter();

			Trace.Assert(stateNode.Contents != null);
			DirectoryNodeRef rootRef = new DirectoryNodeRef(root.Length, stateNode.Contents!);

			ChangeRecord changeRecord = await perforce.GetChangeAsync(GetChangeOptions.None, change, cancellationToken);
			CommitNode commitNode = new CommitNode(change, stateNode.ParentHandle, changeRecord.User ?? "Unknown", null, null, null, changeRecord.Description ?? String.Empty, changeRecord.Date, rootRef, new Dictionary<Guid, IHashedBlobRef>());
			IHashedBlobRef<CommitNode> commitNodeRef = await commitWriter.WriteBlobAsync(commitNode, cancellationToken);

			RedirectNode<CommitNode> redirectNode = new RedirectNode<CommitNode>(commitNodeRef);
			IHashedBlobRef<RedirectNode<CommitNode>> redirectNodeRef = await commitWriter.WriteBlobAsync(redirectNode, cancellationToken);

			await commitWriter.FlushAsync(cancellationToken);
			await store.AddRefAsync(refName, redirectNodeRef, cancellationToken: cancellationToken);

			// Update the replicator state
			UpdateReplicatorOptions completeUpdateOptions = new UpdateReplicatorOptions
			{
				Pause = replicator.SingleStep,
				Clean = false,
				SingleStep = false,
				LastChange = change,
				CurrentChange = 0,
			};
			replicator = await UpdateReplicatorAsync(replicator, completeUpdateOptions, cancellationToken);

			// Log the snapshot info
			_logger.LogInformation("Snapshot for {StreamId} CL {Change} is ref {RefName} (commit: {CommitHandle}, root: {RootHandle})", streamConfig.Id, change, refName, commitNodeRef.GetLocator(), rootRef.Handle.GetLocator());
			return replicator;
		}

		static async Task<IReplicator> UpdateReplicatorAsync(IReplicator replicator, UpdateReplicatorOptions options, CancellationToken cancellationToken)
		{
			IReplicator? nextReplicator = await replicator.TryUpdateAsync(options, cancellationToken);
			return nextReplicator ?? throw new ReplicatorModifiedException();
		}

		static int GetFileOffset(string path)
		{
			int fileIdx = path.Length;
			while (fileIdx > 0 && path[fileIdx - 1] != '/' && path[fileIdx - 1] != '\\')
			{
				fileIdx--;
			}
			return fileIdx;
		}

		static DirectoryToSync FindOrAddDirectoryTree(Dictionary<string, DirectoryToSync> pathToDirectory, string directoryPath, StringComparer comparer)
		{
			DirectoryToSync? directory;
			if (!pathToDirectory.TryGetValue(directoryPath, out directory))
			{
				// Add a new path
				string normalizedPath = NormalizePathSeparators(directoryPath);
				directory = new DirectoryToSync(normalizedPath, comparer);
				pathToDirectory.Add(directoryPath, directory);

				// Also add all the parent directories. This makes the logic for combining wildcards simpler.
				for (int idx = normalizedPath.Length - 2; idx > 0; idx--)
				{
					if (normalizedPath[idx] == '/')
					{
						string parentDirectoryPath = directoryPath.Substring(0, idx + 1);
						if (pathToDirectory.ContainsKey(parentDirectoryPath))
						{
							break;
						}
						else
						{
							pathToDirectory.Add(parentDirectoryPath, new DirectoryToSync(normalizedPath.Substring(0, idx + 1), comparer));
						}
					}
				}
			}
			return directory;
		}

		static string NormalizePathSeparators(string path) => path.Replace('\\', '/');

		async Task FlushWorkspaceAsync(ReplicationClient clientInfo, IPerforceConnection perforce, int change, CancellationToken cancellationToken)
		{
			if (clientInfo.Change != change)
			{
				clientInfo.Change = -1;
				if (change == 0)
				{
					_logger.LogInformation("Flushing have table for {Client}", clientInfo.Client.Name);
					await perforce.SyncQuietAsync(SyncOptions.Force | SyncOptions.KeepWorkspaceFiles, -1, $"//{clientInfo.Client.Name}/...#0", cancellationToken);
				}
				else
				{
					_logger.LogInformation("Flushing have table for {Client} to change {Change}", clientInfo.Client.Name, change);
					await perforce.SyncQuietAsync(SyncOptions.Force | SyncOptions.KeepWorkspaceFiles, -1, $"//{clientInfo.Client.Name}/...@{change}", cancellationToken);
				}
				clientInfo.Change = change;
			}
		}

		static void UnpackOpenPayload(ReadOnlyMemory<byte> data, out Utf8String path, out int flags, out int type, out int perms)
		{
			int length = data.Span.IndexOf((byte)0);
			if (length == -1)
			{
				throw new InvalidStreamException("Invalid data returned by Perforce server; open path does not contain null terminator.");
			}

			path = new Utf8String(data.Slice(0, length)).Clone();
			flags = BinaryPrimitives.ReadInt32LittleEndian(data.Span.Slice(length + 1, sizeof(int)));
			type = BinaryPrimitives.ReadInt32LittleEndian(data.Span.Slice(length + 5, sizeof(int)));
			perms = BinaryPrimitives.ReadInt32LittleEndian(data.Span.Slice(length + 9, sizeof(int)));
		}

		static void UnpackUnlinkPayload(ReadOnlyMemory<byte> data, out string path)
		{
			int length = data.Span.IndexOf((byte)0);
			if (length != -1)
			{
				data = data.Slice(0, length);
			}
			path = Encoding.UTF8.GetString(data.Span);
		}

		static string GetClientRelativePath(string path, string clientRoot)
		{
			if (!path.StartsWith(clientRoot, StringComparison.Ordinal))
			{
				throw new ArgumentException($"Unable to make path {path} relative to client root {clientRoot}");
			}
			return path.Substring(clientRoot.Length);
		}

		async Task<ReplicationClient?> FindReplicationClientAsync(StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			ReplicationClient? clientInfo;
			if (_cachedPerforceClients.TryGetValue(streamConfig.Id, out clientInfo))
			{
				if (!String.Equals(clientInfo.ClusterName, streamConfig.ClusterName, StringComparison.Ordinal) && String.Equals(clientInfo.Client.Stream, streamConfig.Name, StringComparison.Ordinal))
				{
					PerforceSettings serverSettings = new PerforceSettings(clientInfo.Settings);
					serverSettings.ClientName = null;

					using IPerforceConnection perforce = await PerforceConnection.CreateAsync(_logger);
					await perforce.DeleteClientAsync(DeleteClientOptions.None, clientInfo.Client.Name, cancellationToken);

					_cachedPerforceClients.Remove(streamConfig.Id);
					clientInfo = null;
				}
			}
			return clientInfo;
		}

		async Task<ReplicationClient> FindOrAddReplicationClientAsync(BuildConfig buildConfig, StreamConfig streamConfig, CancellationToken cancellationToken = default)
		{
			ReplicationClient? clientInfo = await FindReplicationClientAsync(streamConfig, cancellationToken);
			if (clientInfo == null)
			{
				using IPerforceConnection? perforce = await _perforceService.ConnectAsync(streamConfig.ClusterName, cancellationToken: cancellationToken);
				if (perforce == null)
				{
					throw new PerforceException($"Unable to create connection to Perforce server");
				}

				InfoRecord serverInfo = await perforce.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);

				PerforceCluster cluster = buildConfig.GetPerforceCluster(streamConfig.ClusterName);
				bool partitioned = cluster.SupportsPartitionedWorkspaces;

				string partitionedSuffix = partitioned ? "" : "Full_";

				ClientRecord newClient = new ClientRecord($"Horde.Build_Rep_{partitionedSuffix}{serverInfo.ClientHost}_{streamConfig.Id}", perforce.Settings.UserName, "/p4/");
				newClient.Description = "Created to mirror Perforce content to Horde Storage";
				newClient.Owner = perforce.Settings.UserName;
				newClient.Host = serverInfo.ClientHost;
				newClient.Stream = streamConfig.Name;
				newClient.Type = partitioned ? "readonly" : null;

				try
				{
					await perforce.CreateClientAsync(newClient, cancellationToken);
				}
				catch (PerforceException ex) when (ex.Error != null && ex.Error.Generic == PerforceGenericCode.Usage)
				{
					_logger.LogInformation(ex, "Unable to create client {ClientName}; attempting to delete and retry ({Message})", newClient.Name, ex.Message);
					await perforce.DeleteClientAsync(DeleteClientOptions.None, newClient.Name, cancellationToken);

					_logger.LogInformation("Retrying create client {ClientName} after delete", newClient.Name);
					await perforce.CreateClientAsync(newClient, cancellationToken);
				}

				_logger.LogInformation("Created client {ClientName} for {StreamName}", newClient.Name, streamConfig.Name);

				PerforceSettings settings = new PerforceSettings(perforce.Settings);
				settings.ClientName = newClient.Name;
				settings.PreferNativeClient = true;

				clientInfo = new ReplicationClient(settings, streamConfig.ClusterName, serverInfo, newClient, -1);
				_cachedPerforceClients.Add(streamConfig.Id, clientInfo);
			}
			return clientInfo;
		}
	}
}