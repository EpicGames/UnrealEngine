// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Replicators;
using HordeServer.Streams;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using Polly;
using Polly.Retry;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Exception triggered during content replication
	/// </summary>
	public sealed class ReplicationException : Exception
	{
		internal ReplicationException(string message) : base(message)
		{
		}
	}

	/// <summary>
	/// Service which replicates content from Perforce
	/// </summary>
	sealed class ReplicationService : IHostedService
	{
		readonly IReplicatorCollection _replicatorCollection;
		readonly PerforceReplicator _replicator;
		readonly ITicker _ticker;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ReplicationService(IReplicatorCollection replicatorCollection, PerforceReplicator replicator, IClock clock, IOptionsMonitor<BuildConfig> buildConfig, ILogger<ReplicationService> logger)
		{
			_replicatorCollection = replicatorCollection;
			_replicator = replicator;
			_ticker = clock.AddSharedTicker<ReplicationService>(TimeSpan.FromSeconds(20.0), TickSharedAsync, logger);
			_buildConfig = buildConfig;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		async ValueTask TickSharedAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				TaskCompletionSource tcs = new TaskCompletionSource(TaskCreationOptions.RunContinuationsAsynchronously);
				using (IDisposable? registration = _buildConfig.OnChange((_, _) => tcs.TrySetResult()))
				{
					BuildConfig buildConfig = _buildConfig.CurrentValue;

					// Create all the background tasks and wait for the config to change
					Dictionary<ReplicatorId, BackgroundTask> replicatorIdToTask = new Dictionary<ReplicatorId, BackgroundTask>();
					try
					{
						// Enumerate all the existing replicators 
						Dictionary<ReplicatorId, IReplicator> removeReplicators = new Dictionary<ReplicatorId, IReplicator>();
						foreach (IReplicator replicator in await _replicatorCollection.FindAsync(cancellationToken))
						{
							removeReplicators.Add(replicator.Id, replicator);
						}

						// Start tasks to replicate all the active replicators
						foreach (StreamConfig streamConfig in buildConfig.Streams)
						{
							foreach (ReplicatorConfig replicatorConfig in streamConfig.Replicators)
							{
								if (replicatorConfig.Enabled)
								{
									ReplicatorId replicatorId = new ReplicatorId(streamConfig.Id, replicatorConfig.Id);
									replicatorIdToTask.Add(replicatorId, BackgroundTask.StartNew(ctx => RunReplicationGuardedAsync(replicatorId, buildConfig, streamConfig, replicatorConfig, ctx)));
									removeReplicators.Remove(replicatorId);
								}
							}
						}

						// Remove anything not still referenced
						foreach (IReplicator removeReplicator in removeReplicators.Values)
						{
							await removeReplicator.TryDeleteAsync(cancellationToken);
						}

						// Wait for the config to change
						await tcs.Task.WaitAsync(cancellationToken);
					}
					finally
					{
						await Task.WhenAll(replicatorIdToTask.Values.Select(x => x.DisposeAsync().AsTask()));
					}
				}
			}
		}

		async Task RunReplicationGuardedAsync(ReplicatorId replicatorId, BuildConfig buildConfig, StreamConfig streamConfig, ReplicatorConfig replicatorConfig, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Started background task for replication of {ReplicatorId}", replicatorId);

			ValueTask OnRetry(OnRetryArguments<object> args)
			{
				_logger.LogError(args.Outcome.Exception, "Replication for {ReplicatorId} failed (attempt {Count}): {Message}", replicatorId, args.AttemptNumber, args.Outcome.Exception?.Message ?? "(no exception)");
				return default;
			}

			ResiliencePipeline pipeline = new ResiliencePipelineBuilder()
				.AddRetry(new RetryStrategyOptions { MaxRetryAttempts = Int32.MaxValue, OnRetry = OnRetry })
				.Build();

			await pipeline.ExecuteAsync(async ctx => await _replicator.RunAsync(replicatorId, buildConfig, streamConfig, replicatorConfig, ctx), cancellationToken);
		}
	}
}
