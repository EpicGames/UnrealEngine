// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Replicators
{
	/// <summary>
	/// ACL actions that can be performed on replicators
	/// </summary>
	public static class ReplicatorAclAction
	{
		/// <summary>
		/// Allows deletion of projects.
		/// </summary>
		public static AclAction UpdateReplicator { get; } = new AclAction("UpdateReplicator");

		/// <summary>
		/// Allows the creation of new projects
		/// </summary>
		public static AclAction ViewReplicator { get; } = new AclAction("ViewReplicator");
	}
}
