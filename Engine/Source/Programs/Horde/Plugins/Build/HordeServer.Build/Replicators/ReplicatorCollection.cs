// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Replicators;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson.Serialization;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Default implementation of <see cref="IReplicatorCollection"/>.
	/// </summary>
	class ReplicatorCollection : IReplicatorCollection
	{
		class ReplicatorDoc
		{
			public ReplicatorId Id { get; set; }

			[BsonElement("pause"), BsonIgnoreIfDefault]
			public bool Pause { get; set; }

			[BsonElement("clean"), BsonIgnoreIfDefault]
			public bool Clean { get; set; }

			[BsonElement("reset"), BsonIgnoreIfDefault]
			public bool Reset { get; set; }

			[BsonElement("sstep"), BsonIgnoreIfDefault]
			public bool SingleStep { get; set; }

			[BsonElement("lch")]
			public int? LastChange { get; set; }

			[BsonElement("lct")]
			public DateTime? LastChangeFinishTime { get; set; }

			[BsonElement("nch")]
			public int? NextChange { get; set; }

			[BsonElement("cch")]
			public int? CurrentChange { get; set; }

			[BsonElement("cct")]
			public DateTime? CurrentChangeStartTime { get; set; }

			[BsonElement("tsz"), BsonIgnoreIfDefault]
			public long? CurrentSize { get; set; }

			[BsonElement("csz"), BsonIgnoreIfDefault]
			public long? CurrentCopiedSize { get; set; }

			[BsonElement("err")]
			public string? CurrentError { get; set; }

			[BsonElement("idx")]
			public int UpdateIndex { get; set; }
		}

		class Replicator : IReplicator
		{
			readonly ReplicatorCollection _collection;
			readonly ReplicatorDoc _document;

			public Replicator(ReplicatorCollection collection, ReplicatorDoc document)
			{
				_collection = collection;
				_document = document;
			}

			public ReplicatorId Id => _document.Id;
			public bool Pause => _document.Pause;
			public bool Clean => _document.Clean;
			public bool Reset => _document.Reset;
			public bool SingleStep => _document.SingleStep;
			public int? LastChange => _document.LastChange;
			public int? NextChange => _document.NextChange;
			public DateTime? LastChangeFinishTime => _document.LastChangeFinishTime;
			public int? CurrentChange => _document.CurrentChange;
			public DateTime? CurrentChangeStartTime => _document.CurrentChangeStartTime;
			public long? CurrentSize => _document.CurrentSize;
			public long? CurrentCopiedSize => _document.CurrentCopiedSize;
			public string? CurrentError => _document.CurrentError;

			public Task<IReplicator?> RefreshAsync(CancellationToken cancellationToken = default)
				=> _collection.GetAsync(Id, cancellationToken);

			public Task<bool> TryDeleteAsync(CancellationToken cancellationToken = default)
				=> _collection.TryDeleteAsync(_document, cancellationToken);

			public async Task<IReplicator?> TryUpdateAsync(UpdateReplicatorOptions options, CancellationToken cancellationToken = default)
			{
				ReplicatorDoc? newDoc = await _collection.TryUpdateAsync(_document, options, cancellationToken);
				return (newDoc == null) ? null : new Replicator(_collection, newDoc);
			}
		}

		readonly IMongoCollection<ReplicatorDoc> _documents;
		readonly IClock _clock;

		static ReplicatorCollection()
		{
			BsonClassMap.RegisterClassMap<ReplicatorId>(cm =>
			{
				cm.MapProperty(x => x.StreamId).SetElementName("sid");
				cm.MapProperty(x => x.StreamReplicatorId).SetElementName("rid");
			});
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public ReplicatorCollection(IMongoService mongoService, IClock clock)
		{
			_documents = mongoService.GetCollection<ReplicatorDoc>("Replicators");
			_clock = clock;
		}

		/// <inheritdoc/>
		public async Task<List<IReplicator>> FindAsync(CancellationToken cancellationToken = default)
		{
			List<ReplicatorDoc> documents = await _documents.Find(FilterDefinition<ReplicatorDoc>.Empty, null).ToListAsync(cancellationToken);
			return documents.ConvertAll<IReplicator>(x => new Replicator(this, x));
		}

		/// <inheritdoc/>
		public async Task<IReplicator?> GetAsync(ReplicatorId id, CancellationToken cancellationToken = default)
		{
			ReplicatorDoc? document = await _documents.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
			if (document == null)
			{
				return null;
			}
			return new Replicator(this, document);
		}

		/// <inheritdoc/>
		public async Task<IReplicator> GetOrAddAsync(ReplicatorId id, CancellationToken cancellationToken = default)
		{
			DateTime utcNow = _clock.UtcNow;

			ReplicatorDoc? document;
			for (; ; )
			{
				document = await _documents.Find(x => x.Id == id).FirstOrDefaultAsync(cancellationToken);
				if (document != null)
				{
					break;
				}

				document = new ReplicatorDoc();
				document.Id = id;
				document.UpdateIndex = 1;

				if (await _documents.InsertOneIgnoreDuplicatesAsync(document, cancellationToken))
				{
					break;
				}
			}

			return new Replicator(this, document);
		}

		/// <inheritdoc/>
		async Task<bool> TryDeleteAsync(ReplicatorDoc replicator, CancellationToken cancellationToken)
		{
			DeleteResult result = await _documents.DeleteOneAsync(x => x.Id == replicator.Id && x.UpdateIndex == replicator.UpdateIndex, cancellationToken);
			return result.DeletedCount > 0;
		}

		/// <inheritdoc/>
		async Task<ReplicatorDoc?> TryUpdateAsync(ReplicatorDoc current, UpdateReplicatorOptions options, CancellationToken cancellationToken = default)
		{
			List<UpdateDefinition<ReplicatorDoc>> updates = new List<UpdateDefinition<ReplicatorDoc>>();

			if (options.Pause != null)
			{
				updates.Add(Builders<ReplicatorDoc>.Update.SetOrUnsetBool(x => x.Pause, options.Pause.Value));
			}

			if (options.Clean != null)
			{
				updates.Add(Builders<ReplicatorDoc>.Update.SetOrUnsetBool(x => x.Clean, options.Clean.Value));
			}

			if (options.Reset != null)
			{
				updates.Add(Builders<ReplicatorDoc>.Update.SetOrUnsetBool(x => x.Reset, options.Reset.Value));
			}

			if (options.SingleStep != null)
			{
				updates.Add(Builders<ReplicatorDoc>.Update.SetOrUnsetBool(x => x.SingleStep, options.SingleStep.Value));
			}

			if (options.LastChange != null)
			{
				if (options.LastChange.Value == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.LastChange).Unset(x => x.LastChangeFinishTime));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.LastChange, options.LastChange).Set(x => x.LastChangeFinishTime, _clock.UtcNow));
				}
			}

			if (options.NextChange != null)
			{
				if (options.NextChange.Value == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.NextChange));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.NextChange, options.NextChange));
				}
			}

			if (options.CurrentChange != null)
			{
				if (options.CurrentChange.Value == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.CurrentChange).Unset(x => x.CurrentChangeStartTime));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.CurrentChange, options.CurrentChange.Value).Set(x => x.CurrentChangeStartTime, _clock.UtcNow));
				}
			}

			if (options.CurrentSize != null || options.CurrentChange != null)
			{
				if (options.CurrentSize == null || options.CurrentSize.Value == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.CurrentSize));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.CurrentSize, options.CurrentSize.Value));
				}
			}

			if (options.CurrentCopiedSize != null || options.CurrentChange != null)
			{
				if (options.CurrentCopiedSize == null || options.CurrentCopiedSize.Value == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.CurrentCopiedSize));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.CurrentCopiedSize, options.CurrentCopiedSize.Value));
				}
			}

			if (options.CurrentError != null || options.CurrentChange != null)
			{
				if (options.CurrentError == null || options.CurrentError.Length == 0)
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Unset(x => x.CurrentError));
				}
				else
				{
					updates.Add(Builders<ReplicatorDoc>.Update.Set(x => x.CurrentError, options.CurrentError));
				}
			}

			updates.Add(Builders<ReplicatorDoc>.Update.Inc(x => x.UpdateIndex, 1));

			UpdateDefinition<ReplicatorDoc> update = Builders<ReplicatorDoc>.Update.Combine(updates);
			return await _documents.FindOneAndUpdateAsync<ReplicatorDoc>(x => x.Id == current.Id && x.UpdateIndex == current.UpdateIndex, update, new FindOneAndUpdateOptions<ReplicatorDoc, ReplicatorDoc> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}
	}
}
