// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Replicators;
using EpicGames.Horde.Storage;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Configuration for a stream replicator
	/// </summary>
	public class ReplicatorConfig
	{
		/// <summary>
		/// Identifier for the replicator within the current stream
		/// </summary>
		public StreamReplicatorId Id { get; set; }

		/// <summary>
		/// Whether the replicator is enabled
		/// </summary>
		public bool Enabled { get; set; } = true;

		/// <summary>
		/// Minimum change number to replicate
		/// </summary>
		public int? MinChange { get; set; }

		/// <summary>
		/// Maximum change number to replicate
		/// </summary>
		public int? MaxChange { get; set; }

		/// <summary>
		/// Whether to include content in the replication, or just metadata
		/// </summary>
		public bool IncludeContent { get; set; } = true;

		/// <summary>
		/// Namespace to replicate data to
		/// </summary>
		public NamespaceId NamespaceId { get; set; } = new NamespaceId("horde-perforce");
	}
}
