// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Horde.Replicators;
using EpicGames.Horde.Streams;
using HordeServer.Projects;
using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Replicators
{
	/// <summary>
	/// Controller for the /api/v1/projects endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	public class ReplicatorsController : HordeControllerBase
	{
		readonly IReplicatorCollection _replicatorCollection;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public ReplicatorsController(IReplicatorCollection replicatorCollection, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_replicatorCollection = replicatorCollection;
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Query all the replicators
		/// </summary>
		/// <param name="streamId">Stream to query</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Information about all the projects</returns>
		[HttpGet]
		[Route("/api/v1/replicators")]
		[ProducesResponseType(typeof(List<GetReplicatorResponse>), 200)]
		public async Task<ActionResult<object>> GetReplicatorsAsync([FromQuery] StreamId? streamId = null, [FromQuery] PropertyFilter? filter = null)
		{
			List<object> responses = new List<object>();

			BuildConfig buildConfig = _buildConfig.Value;
			foreach (ProjectConfig projectConfig in buildConfig.Projects)
			{
				foreach (StreamConfig streamConfig in projectConfig.Streams)
				{
					if (streamId == null || streamConfig.Id == streamId.Value)
					{
						if (streamConfig.Authorize(ReplicatorAclAction.ViewReplicator, User))
						{
							foreach (ReplicatorConfig replicatorConfig in streamConfig.Replicators)
							{
								ReplicatorId replicatorId = new ReplicatorId(streamConfig.Id, replicatorConfig.Id);
								IReplicator? replicator = await _replicatorCollection.GetAsync(replicatorId);
								GetReplicatorResponse response = CreateGetReplicatorResponse(replicatorId, replicator);
								responses.Add(response.ApplyFilter(filter));
							}
						}
					}
				}
			}

			return responses;
		}

		/// <summary>
		/// Query a replicator state
		/// </summary>
		/// <param name="replicatorId">Repliactor to query</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Information about all the projects</returns>
		[HttpGet]
		[Route("/api/v1/replicator/{replicatorId}")]
		[ProducesResponseType(typeof(GetReplicatorResponse), 200)]
		public async Task<ActionResult<object>> GetReplicatorsAsync(ReplicatorId replicatorId, [FromQuery] PropertyFilter? filter = null)
		{
			BuildConfig buildConfig = _buildConfig.Value;
			if (!buildConfig.TryGetStream(replicatorId.StreamId, out StreamConfig? streamConfig))
			{
				return NotFound();
			}
			if (!streamConfig.Authorize(ReplicatorAclAction.ViewReplicator, User))
			{
				return Forbid(ReplicatorAclAction.ViewReplicator);
			}

			if (!streamConfig.TryGetReplicator(replicatorId.StreamReplicatorId, out _))
			{
				return NotFound();
			}

			IReplicator? replicator = await _replicatorCollection.GetAsync(replicatorId);
			GetReplicatorResponse response = CreateGetReplicatorResponse(replicatorId, replicator);
			return response.ApplyFilter(filter);
		}

		/// <summary>
		/// Update a replicator state
		/// </summary>
		/// <param name="replicatorId">Repliactor to query</param>
		/// <param name="request">Update request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPut]
		[Route("/api/v1/replicator/{replicatorId}")]
		public async Task<ActionResult<object>> UpdateReplicatorAsync(ReplicatorId replicatorId, [FromBody] UpdateReplicatorRequest request, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.Value;
			if (!buildConfig.TryGetStream(replicatorId.StreamId, out StreamConfig? streamConfig))
			{
				return NotFound();
			}
			if (!streamConfig.Authorize(ReplicatorAclAction.UpdateReplicator, User))
			{
				return Forbid(ReplicatorAclAction.UpdateReplicator);
			}

			if (!streamConfig.TryGetReplicator(replicatorId.StreamReplicatorId, out _))
			{
				return NotFound();
			}

			for (; ; )
			{
				IReplicator? replicator = await _replicatorCollection.GetOrAddAsync(replicatorId, cancellationToken: cancellationToken);

				UpdateReplicatorOptions updateOptions = new UpdateReplicatorOptions(Pause: request.Pause, Clean: request.Clean, Reset: request.Reset, SingleStep: request.SingleStep, NextChange: request.NextChange);
				if (await replicator.TryUpdateAsync(updateOptions, cancellationToken) != null)
				{
					break;
				}
			}

			return Ok();
		}

		static GetReplicatorResponse CreateGetReplicatorResponse(ReplicatorId replicatorId, IReplicator? replicator)
		{
			GetReplicatorResponse response = new GetReplicatorResponse();
			response.Id = replicatorId;
			response.StreamId = replicatorId.StreamId;
			response.StreamReplicatorId = replicatorId.StreamReplicatorId;
			response.Status = GetStatusMessage(replicator);

			if (replicator != null)
			{
				response.Pause = replicator.Pause;
				response.Clean = replicator.Clean;
				response.Reset = replicator.Reset;
				response.SingleStep = replicator.SingleStep;
				response.CurrentChange = replicator.CurrentChange;
				response.CurrentChangeStartTime = replicator.CurrentChangeStartTime;
				response.LastChange = replicator.LastChange;
				response.LastChangeFinishTime = replicator.LastChangeFinishTime;
				response.CurrentSize = replicator.CurrentSize;
				response.CurrentCopiedSize = replicator.CurrentCopiedSize;
				response.CurrentError = replicator.CurrentError;
			}

			return response;
		}

		static string GetStatusMessage(IReplicator? replicator)
		{
			if (replicator == null)
			{
				return "Waiting to start...";
			}
			if (replicator.Pause)
			{
				return "Paused";
			}
			if (replicator.CurrentChange == null)
			{
				return "Waiting";
			}
			if (replicator.CurrentError != null)
			{
				return $"Error: {replicator.CurrentError}";
			}

			StringBuilder message = new StringBuilder($"Replicating CL {replicator.CurrentChange.Value}");
			if (replicator.CurrentSize != null && replicator.CurrentSize.Value > 0)
			{
				double copiedMb = (replicator.CurrentCopiedSize ?? 0) / (1024.0 * 1024.0);
				double totalMb = replicator.CurrentSize.Value / (1024.0 * 1024.0);
				message.Append($" ({copiedMb:n1}mb/{totalMb:n1}mb, {(copiedMb * 100.0) / totalMb:n1}%)");
			}

			return message.ToString();
		}
	}
}
