// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Immutable;
using System.Text.Json;
using Microsoft.Extensions.Logging;

namespace HordeServer.RoboMerge
{
	/// <summary>
	/// Parser for RoboMerge branchmap JSON files
	/// </summary>
	internal static class RoboMergeBranchmapParser
	{
		/// <summary>
		/// Reserved branch names that are filtered from flow target arrays
		/// </summary>
		static readonly HashSet<string> s_reservedNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"NONE", "DEFAULT", "IGNORE", "DEADEND", ""
		};

		/// <summary>
		/// Parse branchmap JSON bytes into a MergeGraph.
		/// Bot name is derived from the depot path filename (e.g. "myproject.branchmap.json" → "myproject").
		/// </summary>
		/// <param name="depotPath">Perforce depot path of the branchmap file</param>
		/// <param name="data">Raw JSON bytes from Perforce</param>
		/// <param name="headChange">Perforce head changelist of the file</param>
		/// <param name="logger">Optional logger for validation warnings</param>
		/// <returns>Parsed merge graph</returns>
		/// <exception cref="JsonException">If the JSON is malformed</exception>
		public static MergeGraph Parse(string depotPath, byte[] data, int headChange, ILogger? logger = null)
		{
			string botName = ExtractBotName(depotPath);

			using JsonDocument doc = JsonDocument.Parse(data);
			JsonElement root = doc.RootElement;

			string defaultStreamDepot = GetStringOrDefault(root, "defaultStreamDepot");
			bool isDefaultBot = GetBoolOrDefault(root, "isDefaultBot");
			string? slackChannel = GetStringOrNull(root, "slackChannel");
			bool reportToBuildHealth = GetBoolOrDefault(root, "reportToBuildHealth");

			IReadOnlyList<string> visibility = GetStringArrayOrEmpty(root, "visibility");
			IReadOnlyList<string> excludeAuthors = GetStringArrayOrEmpty(root, "excludeAuthors");

			// Bot alias normalization: check "aliases" (array) first, then "alias" (singular string)
			IReadOnlyList<string> aliases;
			if (root.TryGetProperty("aliases", out JsonElement aliasesElem) && aliasesElem.ValueKind == JsonValueKind.Array)
			{
				aliases = ParseStringArray(aliasesElem);
			}
			else if (root.TryGetProperty("alias", out JsonElement aliasElem) && aliasElem.ValueKind == JsonValueKind.String)
			{
				string? aliasStr = aliasElem.GetString();
				aliases = !String.IsNullOrEmpty(aliasStr) ? new[] { aliasStr } : Array.Empty<string>();
			}
			else
			{
				aliases = Array.Empty<string>();
			}

			// Parse macros
			IReadOnlyDictionary<string, IReadOnlyList<string>> macros = ParseMacros(root);

			// Parse branches
			List<MergeBranch> branches = new List<MergeBranch>();
			if (root.TryGetProperty("branches", out JsonElement branchesElem) && branchesElem.ValueKind == JsonValueKind.Array)
			{
				foreach (JsonElement branchElem in branchesElem.EnumerateArray())
				{
					MergeBranch? branch = ParseBranch(branchElem);
					if (branch != null)
					{
						branches.Add(branch);
					}
				}
			}

			// Build branch name set for cross-reference validation
			HashSet<string> branchNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			foreach (MergeBranch branch in branches)
			{
				branchNames.Add(branch.Name);
			}

			// Validate branch cross-references
			foreach (MergeBranch branch in branches)
			{
				ValidateFlowTargets(branch.Name, "flowsTo", branch.FlowsTo, branchNames, logger);
				ValidateFlowTargets(branch.Name, "forceFlowTo", branch.ForceFlowTo, branchNames, logger);
				ValidateFlowTargets(branch.Name, "defaultFlow", branch.DefaultFlow, branchNames, logger);
				ValidateFlowTargets(branch.Name, "blockAssetFlow", branch.BlockAssetFlow, branchNames, logger);
			}

			// Parse edges
			List<MergeEdge> edges = new List<MergeEdge>();
			if (root.TryGetProperty("edges", out JsonElement edgesElem) && edgesElem.ValueKind == JsonValueKind.Array)
			{
				foreach (JsonElement edgeElem in edgesElem.EnumerateArray())
				{
					MergeEdge? edge = ParseEdge(edgeElem);
					if (edge != null)
					{
						// Validate edge references
						if (!branchNames.Contains(edge.From))
						{
							logger?.LogWarning("Bot {BotName}: edge references unknown 'from' branch '{From}'", botName, edge.From);
						}
						if (!branchNames.Contains(edge.To))
						{
							logger?.LogWarning("Bot {BotName}: edge references unknown 'to' branch '{To}'", botName, edge.To);
						}
						edges.Add(edge);
					}
				}
			}

			return new MergeGraph
			{
				BotName = botName,
				DefaultStreamDepot = defaultStreamDepot,
				IsDefaultBot = isDefaultBot,
				Aliases = aliases,
				Visibility = visibility,
				ReportToBuildHealth = reportToBuildHealth,
				ExcludeAuthors = excludeAuthors,
				SlackChannel = slackChannel,
				Branches = branches,
				Edges = edges,
				Macros = macros,
				HeadChange = headChange,
				LastRefreshedUtc = DateTime.UtcNow
			};
		}

		/// <summary>
		/// Compute the merge chain (release spine) from a MergeGraph.
		/// Returns null if no valid spine can be computed.
		/// </summary>
		public static MergeChain? ComputeMergeChain(MergeGraph graph, ILogger? logger = null)
		{
			// Filter to enabled branches only
			Dictionary<string, MergeBranch> enabledBranches = new Dictionary<string, MergeBranch>(StringComparer.OrdinalIgnoreCase);
			foreach (MergeBranch branch in graph.Branches)
			{
				if (!branch.Disabled)
				{
					enabledBranches[branch.Name] = branch;
				}
			}

			// Build reverse forceFlowTo map: if B.ForceFlowTo contains T, then reverseMapMulti[T] includes B.
			// Multiple branches may target the same branch (e.g. both Dev and Dev-Authoring target Main),
			// so we collect all candidates and resolve conflicts below.
			Dictionary<string, List<string>> reverseMapMulti = new Dictionary<string, List<string>>(StringComparer.OrdinalIgnoreCase);
			HashSet<string> hasForceFlowTo = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

			foreach (MergeBranch branch in enabledBranches.Values)
			{
				if (branch.ForceFlowTo.Count > 0)
				{
					hasForceFlowTo.Add(branch.Name);
					foreach (string target in branch.ForceFlowTo)
					{
						if (s_reservedNames.Contains(target))
						{
							continue;
						}
						if (!enabledBranches.ContainsKey(target))
						{
							logger?.LogWarning("Bot {BotName}: branch '{BranchName}' forceFlowTo references disabled or missing branch '{Target}'",
								graph.BotName, branch.Name, target);
							continue;
						}
						if (!reverseMapMulti.TryGetValue(target, out List<string>? sources))
						{
							sources = new List<string>();
							reverseMapMulti[target] = sources;
						}
						sources.Add(branch.Name);
					}
				}
			}

			if (reverseMapMulti.Count == 0)
			{
				return null;
			}

			// Resolve conflicts: when multiple branches target the same branch, prefer the one that is itself
			// a forceFlowTo target (i.e. part of the spine, not a side branch). For example, if both Dev
			// and Dev-Authoring target Main, prefer Dev because Release targets it.
			HashSet<string> isTargeted = new HashSet<string>(reverseMapMulti.Keys, StringComparer.OrdinalIgnoreCase);
			Dictionary<string, string> reverseMap = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);

			foreach ((string target, List<string> sources) in reverseMapMulti)
			{
				if (sources.Count == 1)
				{
					reverseMap[target] = sources[0];
				}
				else
				{
					// Prefer the source that is itself a target (spine branch over leaf branch)
					List<string> spineCandidates = sources.Where(s => isTargeted.Contains(s)).ToList();
					if (spineCandidates.Count == 1)
					{
						reverseMap[target] = spineCandidates[0];
					}
					else
					{
						// Ambiguous: pick first and warn
						reverseMap[target] = sources[0];
						logger?.LogWarning("Bot {BotName}: multiple branches target '{Target}' via forceFlowTo ({Sources}), using '{Chosen}'",
							graph.BotName, target, String.Join(", ", sources), sources[0]);
					}
				}
			}

			// Find root(s) — branches that appear as forceFlowTo targets but have no forceFlowTo of their own
			List<string> roots = new List<string>();
			foreach (string target in reverseMap.Keys)
			{
				if (!hasForceFlowTo.Contains(target) && enabledBranches.ContainsKey(target))
				{
					roots.Add(target);
				}
			}

			if (roots.Count == 0)
			{
				logger?.LogWarning("Bot {BotName}: no valid root found for merge chain (possible cycle in forceFlowTo)", graph.BotName);
				return null;
			}

			// Pick root: prefer "Main" (case-insensitive), otherwise first alphabetically
			string root;
			string? mainRoot = roots.Find(r => String.Equals(r, "Main", StringComparison.OrdinalIgnoreCase));
			if (mainRoot != null)
			{
				root = mainRoot;
			}
			else
			{
				roots.Sort(StringComparer.OrdinalIgnoreCase);
				root = roots[0];
				if (roots.Count > 1)
				{
					logger?.LogWarning("Bot {BotName}: multiple roots found for merge chain ({Roots}), using '{Root}'",
						graph.BotName, String.Join(", ", roots), root);
				}
			}

			// Walk from root following reverseMap: Main ← Dev ← Release-N ← ...
			List<string> chain = new List<string>();
			HashSet<string> visited = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			string? current = root;
			while (current != null)
			{
				if (!visited.Add(current))
				{
					logger?.LogWarning("Bot {BotName}: cycle detected in forceFlowTo at branch '{Branch}'", graph.BotName, current);
					break;
				}
				chain.Add(current);
				reverseMap.TryGetValue(current, out current);
			}

			// Reverse to get oldest-first order
			chain.Reverse();

			if (chain.Count < 2)
			{
				return null;
			}

			// Build edge lookup for approval gate checks
			Dictionary<string, MergeEdge> edgeMap = new Dictionary<string, MergeEdge>(StringComparer.OrdinalIgnoreCase);
			foreach (MergeEdge edge in graph.Edges)
			{
				edgeMap[$"{edge.From}|{edge.To}"] = edge;
			}

			// Build chain entries
			List<MergeChainEntry> entries = new List<MergeChainEntry>();
			for (int i = 0; i < chain.Count; i++)
			{
				string branchName = chain[i];
				if (!enabledBranches.TryGetValue(branchName, out MergeBranch? branch))
				{
					continue;
				}

				bool hasApprovalGateBefore = false;
				if (i > 0)
				{
					string prevBranchName = chain[i - 1];
					// Check for approval gate on edge from prev → current
					if (edgeMap.TryGetValue($"{prevBranchName}|{branchName}", out MergeEdge? edge))
					{
						if (edge.Approval != null)
						{
							hasApprovalGateBefore = true;
						}
						if (edge.Terminal)
						{
							logger?.LogWarning("Bot {BotName}: terminal edge in merge chain between '{From}' and '{To}'",
								graph.BotName, prevBranchName, branchName);
						}
					}
				}

				entries.Add(new MergeChainEntry
				{
					Branch = branch,
					StreamPath = branch.GetStreamPath(graph.DefaultStreamDepot),
					HasApprovalGateBefore = hasApprovalGateBefore
				});
			}

			if (entries.Count < 2)
			{
				return null;
			}

			return new MergeChain
			{
				BotName = graph.BotName,
				Entries = entries
			};
		}

		/// <summary>
		/// Extract the bot name from a branchmap depot path.
		/// Strips ".branchmap.json" suffix from the last path segment.
		/// </summary>
		public static string ExtractBotName(string depotPath)
		{
			string fileName = depotPath.Split('/').Last();
			int idx = fileName.IndexOf(".branchmap.json", StringComparison.OrdinalIgnoreCase);
			return idx > 0 ? fileName[..idx] : fileName;
		}

		static MergeBranch? ParseBranch(JsonElement elem)
		{
			string name = GetStringOrDefault(elem, "name");
			if (String.IsNullOrEmpty(name))
			{
				return null;
			}

			// Branch-level visibility can be array or string
			IReadOnlyList<string>? visibility = null;
			if (elem.TryGetProperty("visibility", out JsonElement visElem))
			{
				if (visElem.ValueKind == JsonValueKind.Array)
				{
					visibility = ParseStringArray(visElem);
				}
				else if (visElem.ValueKind == JsonValueKind.String)
				{
					string? visStr = visElem.GetString();
					visibility = !String.IsNullOrEmpty(visStr) ? new[] { visStr } : null;
				}
			}

			// Branch-level macros
			IReadOnlyDictionary<string, IReadOnlyList<string>>? macros = null;
			if (elem.TryGetProperty("macros", out JsonElement macrosElem) && macrosElem.ValueKind == JsonValueKind.Object)
			{
				IReadOnlyDictionary<string, IReadOnlyList<string>> parsed = ParseMacrosElement(macrosElem);
				if (parsed.Count > 0)
				{
					macros = parsed;
				}
			}

			return new MergeBranch
			{
				Name = name,
				Aliases = GetStringArrayOrEmpty(elem, "aliases"),
				StreamName = GetStringOrNull(elem, "streamName"),
				StreamDepot = GetStringOrNull(elem, "streamDepot"),
				FlowsTo = FilterReservedNames(GetStringArrayOrEmpty(elem, "flowsTo")),
				ForceFlowTo = FilterReservedNames(GetStringArrayOrEmpty(elem, "forceFlowTo")),
				DefaultFlow = FilterReservedNames(GetStringArrayOrEmpty(elem, "defaultFlow")),
				BlockAssetFlow = FilterReservedNames(GetStringArrayOrEmpty(elem, "blockAssetFlow")),
				DisallowDeadend = GetBoolOrDefault(elem, "disallowDeadend"),
				DisallowSkip = GetBoolOrDefault(elem, "disallowSkip"),
				Disabled = GetBoolOrDefault(elem, "disabled"),
				Resolver = GetStringOrNull(elem, "resolver"),
				BadgeProject = GetStringOrNull(elem, "badgeProject"),
				GraphNodeColor = GetStringOrNull(elem, "graphNodeColor"),
				IncognitoMode = GetBoolOrDefault(elem, "incognitoMode"),
				Visibility = visibility,
				Macros = macros
			};
		}

		static MergeEdge? ParseEdge(JsonElement elem)
		{
			string from = GetStringOrDefault(elem, "from");
			string to = GetStringOrDefault(elem, "to");
			if (String.IsNullOrEmpty(from) || String.IsNullOrEmpty(to))
			{
				return null;
			}

			MergeApprovalGate? approval = null;
			if (elem.TryGetProperty("approval", out JsonElement approvalElem) && approvalElem.ValueKind == JsonValueKind.Object)
			{
				approval = new MergeApprovalGate
				{
					Description = GetStringOrNull(approvalElem, "description"),
					Block = GetBoolOrDefault(approvalElem, "block")
				};
			}

			return new MergeEdge
			{
				From = from,
				To = to,
				Approval = approval,
				IntegrationMethod = GetStringOrNull(elem, "integrationMethod"),
				DisallowSkip = GetBoolOrDefault(elem, "disallowSkip"),
				Resolver = GetStringOrNull(elem, "resolver"),
				Terminal = GetBoolOrDefault(elem, "terminal"),
				ImplicitCommands = GetStringArrayOrEmpty(elem, "implicitCommands"),
				Branchspec = GetStringOrNull(elem, "branchspec")
			};
		}

		static IReadOnlyDictionary<string, IReadOnlyList<string>> ParseMacros(JsonElement root)
		{
			if (!root.TryGetProperty("macros", out JsonElement macrosElem) || macrosElem.ValueKind != JsonValueKind.Object)
			{
				return ImmutableDictionary<string, IReadOnlyList<string>>.Empty;
			}
			return ParseMacrosElement(macrosElem);
		}

		static IReadOnlyDictionary<string, IReadOnlyList<string>> ParseMacrosElement(JsonElement macrosElem)
		{
			Dictionary<string, IReadOnlyList<string>> macros = new Dictionary<string, IReadOnlyList<string>>(StringComparer.OrdinalIgnoreCase);
			foreach (JsonProperty prop in macrosElem.EnumerateObject())
			{
				if (prop.Value.ValueKind == JsonValueKind.Array)
				{
					macros[prop.Name] = ParseStringArray(prop.Value);
				}
			}
			return macros.Count > 0
				? macros.ToImmutableDictionary(StringComparer.OrdinalIgnoreCase)
				: ImmutableDictionary<string, IReadOnlyList<string>>.Empty;
		}

		static void ValidateFlowTargets(string branchName, string fieldName, IReadOnlyList<string> targets, HashSet<string> branchNames, ILogger? logger)
		{
			foreach (string target in targets)
			{
				if (!branchNames.Contains(target))
				{
					logger?.LogWarning("Branch '{BranchName}': {FieldName} references unknown branch '{Target}'", branchName, fieldName, target);
				}
			}
		}

		static IReadOnlyList<string> FilterReservedNames(IReadOnlyList<string> names)
		{
			bool hasReserved = false;
			foreach (string name in names)
			{
				if (s_reservedNames.Contains(name))
				{
					hasReserved = true;
					break;
				}
			}

			if (!hasReserved)
			{
				return names;
			}

			List<string> filtered = new List<string>();
			foreach (string name in names)
			{
				if (!s_reservedNames.Contains(name))
				{
					filtered.Add(name);
				}
			}
			return filtered;
		}

		static string GetStringOrDefault(JsonElement elem, string propertyName)
		{
			if (elem.TryGetProperty(propertyName, out JsonElement prop) && prop.ValueKind == JsonValueKind.String)
			{
				return prop.GetString() ?? string.Empty;
			}
			return string.Empty;
		}

		static string? GetStringOrNull(JsonElement elem, string propertyName)
		{
			if (elem.TryGetProperty(propertyName, out JsonElement prop) && prop.ValueKind == JsonValueKind.String)
			{
				return prop.GetString();
			}
			return null;
		}

		static bool GetBoolOrDefault(JsonElement elem, string propertyName)
		{
			if (elem.TryGetProperty(propertyName, out JsonElement prop))
			{
				if (prop.ValueKind == JsonValueKind.True)
				{
					return true;
				}
				if (prop.ValueKind == JsonValueKind.False)
				{
					return false;
				}
			}
			return false;
		}

		static IReadOnlyList<string> GetStringArrayOrEmpty(JsonElement elem, string propertyName)
		{
			if (elem.TryGetProperty(propertyName, out JsonElement prop) && prop.ValueKind == JsonValueKind.Array)
			{
				return ParseStringArray(prop);
			}
			return Array.Empty<string>();
		}

		static IReadOnlyList<string> ParseStringArray(JsonElement arrayElem)
		{
			List<string> result = new List<string>();
			foreach (JsonElement item in arrayElem.EnumerateArray())
			{
				if (item.ValueKind == JsonValueKind.String)
				{
					string? str = item.GetString();
					if (!String.IsNullOrEmpty(str))
					{
						result.Add(str);
					}
				}
			}
			return result;
		}
	}
}
