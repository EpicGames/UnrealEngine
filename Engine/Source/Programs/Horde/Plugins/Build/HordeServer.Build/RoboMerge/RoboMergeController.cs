// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Streams;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.RoboMerge
{
	/// <summary>
	/// Controller for RoboMerge merge graph data
	/// </summary>
	[Authorize]
	[ApiController]
	[Tags("RoboMerge")]
	public class RoboMergeController : HordeControllerBase
	{
		readonly IMergeGraphService _mergeGraphService;
		readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public RoboMergeController(IMergeGraphService mergeGraphService, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_mergeGraphService = mergeGraphService;
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Check whether the current user can view at least one Horde stream in the given merge graph.
		/// Falls back to allowing access if no Horde streams match any branch (topology-only data).
		/// </summary>
		bool AuthorizeMergeGraphByBotName(string botName)
		{
			MergeGraph? graph = _mergeGraphService.GetMergeGraph(botName);
			return graph != null && AuthorizeMergeGraph(graph);
		}

		bool AuthorizeMergeGraph(MergeGraph graph)
		{
			BuildConfig buildConfig = _buildConfig.Value;
			bool anyHordeStreamMatched = false;

			foreach (MergeBranch branch in graph.Branches)
			{
				string streamPath = branch.GetStreamPath(graph.DefaultStreamDepot);
				foreach (StreamConfig streamConfig in buildConfig.Streams)
				{
					if (String.Equals(streamConfig.Name, streamPath, StringComparison.OrdinalIgnoreCase))
					{
						anyHordeStreamMatched = true;
						if (streamConfig.Authorize(StreamAclAction.ViewStream, User))
						{
							return true;
						}
					}
				}
			}

			return !anyHordeStreamMatched;
		}

		/// <summary>
		/// Get the service status
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/status")]
		[ProducesResponseType(typeof(GetRoboMergeStatusResponse), 200)]
		public ActionResult<GetRoboMergeStatusResponse> GetStatus()
		{
			IReadOnlyList<MergeGraph> graphs = _mergeGraphService.GetMergeGraphs();
			return new GetRoboMergeStatusResponse
			{
				IsConfigured = _buildConfig.Value.Robomerge?.BranchmapFiles?.Count > 0,
				GraphsLoaded = graphs.Count > 0,
				GraphCount = graphs.Count,
				BotNames = graphs.Select(g => g.BotName).ToList(),
				LastTickUtc = _mergeGraphService.LastTickUtc
			};
		}

		/// <summary>
		/// List all merge graphs (summary)
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/graphs")]
		[ProducesResponseType(typeof(List<GetMergeGraphSummaryResponse>), 200)]
		public ActionResult<List<GetMergeGraphSummaryResponse>> GetMergeGraphs()
		{
			return _mergeGraphService.GetMergeGraphs().Where(AuthorizeMergeGraph).Select(g => new GetMergeGraphSummaryResponse
			{
				BotName = g.BotName,
				DefaultStreamDepot = g.DefaultStreamDepot,
				BranchCount = g.Branches.Count,
				EdgeCount = g.Edges.Count,
				ReportToBuildHealth = g.ReportToBuildHealth,
				HeadChange = g.HeadChange,
				LastRefreshedUtc = g.LastRefreshedUtc
			}).ToList();
		}

		/// <summary>
		/// Get a specific bot's full merge graph
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/graphs/{botName}")]
		[ProducesResponseType(typeof(GetMergeGraphResponse), 200)]
		[ProducesResponseType(404)]
		public ActionResult<GetMergeGraphResponse> GetMergeGraph(string botName)
		{
			MergeGraph? graph = _mergeGraphService.GetMergeGraph(botName);
			if (graph == null)
			{
				return NotFound();
			}
			if (!AuthorizeMergeGraph(graph))
			{
				return Forbid();
			}
			return CreateGetMergeGraphResponse(graph);
		}

		/// <summary>
		/// Get the merge chain for a bot
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/graphs/{botName}/chain")]
		[ProducesResponseType(typeof(GetMergeChainResponse), 200)]
		[ProducesResponseType(404)]
		public ActionResult<GetMergeChainResponse> GetMergeChain(string botName)
		{
			MergeGraph? graph = _mergeGraphService.GetMergeGraph(botName);
			if (graph == null)
			{
				return NotFound();
			}
			if (!AuthorizeMergeGraph(graph))
			{
				return Forbid();
			}
			MergeChain? chain = _mergeGraphService.GetMergeChain(botName);
			if (chain == null)
			{
				return NotFound();
			}
			return CreateGetMergeChainResponse(chain);
		}

		/// <summary>
		/// Get all merge chains
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/chains")]
		[ProducesResponseType(typeof(List<GetMergeChainResponse>), 200)]
		public ActionResult<List<GetMergeChainResponse>> GetMergeChains()
		{
			return _mergeGraphService.GetMergeChains()
				.Where(chain => AuthorizeMergeGraphByBotName(chain.BotName))
				.Select(CreateGetMergeChainResponse)
				.ToList();
		}

		/// <summary>
		/// Find branches matching a stream
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/branches")]
		[ProducesResponseType(typeof(List<GetMergeBranchRefResponse>), 200)]
		public ActionResult<List<GetMergeBranchRefResponse>> FindBranches(
			[FromQuery] string? streamPath,
			[FromQuery] string? depot,
			[FromQuery] string? stream)
		{
			IReadOnlyList<MergeGraphBranchRef> refs;
			if (!String.IsNullOrEmpty(streamPath))
			{
				refs = _mergeGraphService.FindBranchesForStreamPath(streamPath);
			}
			else if (!String.IsNullOrEmpty(depot) && !String.IsNullOrEmpty(stream))
			{
				refs = _mergeGraphService.FindBranchesForStream(depot, stream);
			}
			else
			{
				return BadRequest("Either 'streamPath' or both 'depot' and 'stream' query parameters are required.");
			}

			return refs
				.Where(r => AuthorizeMergeGraphByBotName(r.BotName))
				.Select(r => new GetMergeBranchRefResponse
				{
					BotName = r.BotName,
					Branch = CreateGetMergeBranchResponse(r.Branch, r.StreamPath)
				}).ToList();
		}

		/// <summary>
		/// Get branches reachable from a given branch via transitive flowsTo
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/graphs/{botName}/branches/{branchName}/reachable")]
		[ProducesResponseType(typeof(GetReachableBranchesResponse), 200)]
		[ProducesResponseType(404)]
		public ActionResult<GetReachableBranchesResponse> GetReachableBranches(string botName, string branchName)
		{
			MergeGraph? graph = _mergeGraphService.GetMergeGraph(botName);
			if (graph == null)
			{
				return NotFound();
			}
			if (!AuthorizeMergeGraph(graph))
			{
				return Forbid();
			}
			if (!graph.Branches.Any(b => String.Equals(b.Name, branchName, StringComparison.OrdinalIgnoreCase)))
			{
				return NotFound();
			}
			IReadOnlySet<string> reachable = _mergeGraphService.GetReachableBranches(botName, branchName);
			return new GetReachableBranchesResponse
			{
				BotName = botName,
				BranchName = branchName,
				ReachableBranches = reachable.ToList()
			};
		}

		/// <summary>
		/// Get the edge between two branches in a bot
		/// </summary>
		[HttpGet]
		[Route("/api/v1/robomerge/graphs/{botName}/edges")]
		[ProducesResponseType(typeof(GetMergeEdgeResponse), 200)]
		[ProducesResponseType(400)]
		[ProducesResponseType(404)]
		public ActionResult<GetMergeEdgeResponse> GetEdge(
			string botName,
			[FromQuery] string? from,
			[FromQuery] string? to)
		{
			if (String.IsNullOrEmpty(from) || String.IsNullOrEmpty(to))
			{
				return BadRequest("Both 'from' and 'to' query parameters are required.");
			}
			MergeGraph? graph = _mergeGraphService.GetMergeGraph(botName);
			if (graph == null)
			{
				return NotFound();
			}
			if (!AuthorizeMergeGraph(graph))
			{
				return Forbid();
			}
			MergeEdge? edge = _mergeGraphService.GetEdge(botName, from, to);
			if (edge == null)
			{
				return NotFound();
			}
			return CreateGetMergeEdgeResponse(edge);
		}

		static GetMergeGraphResponse CreateGetMergeGraphResponse(MergeGraph graph)
			=> new GetMergeGraphResponse
			{
				BotName = graph.BotName,
				DefaultStreamDepot = graph.DefaultStreamDepot,
				IsDefaultBot = graph.IsDefaultBot,
				Aliases = graph.Aliases.ToList(),
				Visibility = graph.Visibility.ToList(),
				ReportToBuildHealth = graph.ReportToBuildHealth,
				ExcludeAuthors = graph.ExcludeAuthors.ToList(),
				SlackChannel = graph.SlackChannel,
				HeadChange = graph.HeadChange,
				LastRefreshedUtc = graph.LastRefreshedUtc,
				Branches = graph.Branches.Select(b => CreateGetMergeBranchResponse(b, b.GetStreamPath(graph.DefaultStreamDepot))).ToList(),
				Edges = graph.Edges.Select(CreateGetMergeEdgeResponse).ToList(),
				Macros = graph.Macros.Count > 0
					? graph.Macros.ToDictionary(kvp => kvp.Key, kvp => kvp.Value.ToList())
					: null
			};

		static GetMergeBranchResponse CreateGetMergeBranchResponse(MergeBranch branch, string streamPath)
			=> new GetMergeBranchResponse
			{
				Name = branch.Name,
				Aliases = branch.Aliases.ToList(),
				StreamName = branch.StreamName,
				StreamDepot = branch.StreamDepot,
				StreamPath = streamPath,
				FlowsTo = branch.FlowsTo.ToList(),
				ForceFlowTo = branch.ForceFlowTo.ToList(),
				DefaultFlow = branch.DefaultFlow.ToList(),
				BlockAssetFlow = branch.BlockAssetFlow.ToList(),
				DisallowDeadend = branch.DisallowDeadend,
				DisallowSkip = branch.DisallowSkip,
				Disabled = branch.Disabled,
				Resolver = branch.Resolver,
				BadgeProject = branch.BadgeProject,
				GraphNodeColor = branch.GraphNodeColor,
				IncognitoMode = branch.IncognitoMode,
				Visibility = branch.Visibility?.ToList(),
				Macros = branch.Macros?.ToDictionary(kvp => kvp.Key, kvp => kvp.Value.ToList())
			};

		static GetMergeEdgeResponse CreateGetMergeEdgeResponse(MergeEdge edge)
			=> new GetMergeEdgeResponse
			{
				From = edge.From,
				To = edge.To,
				Approval = edge.Approval != null ? new GetMergeApprovalGateResponse
				{
					Description = edge.Approval.Description,
					Block = edge.Approval.Block
				} : null,
				IntegrationMethod = edge.IntegrationMethod,
				DisallowSkip = edge.DisallowSkip,
				Resolver = edge.Resolver,
				Terminal = edge.Terminal,
				ImplicitCommands = edge.ImplicitCommands.ToList(),
				Branchspec = edge.Branchspec
			};

		static GetMergeChainResponse CreateGetMergeChainResponse(MergeChain chain)
			=> new GetMergeChainResponse
			{
				BotName = chain.BotName,
				Entries = chain.Entries.Select(e => new GetMergeChainEntryResponse
				{
					BranchName = e.Branch.Name,
					StreamPath = e.StreamPath,
					HasApprovalGateBefore = e.HasApprovalGateBefore
				}).ToList()
			};
	}
}
