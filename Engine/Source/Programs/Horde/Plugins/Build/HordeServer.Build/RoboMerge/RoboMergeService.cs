// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Immutable;
using System.Diagnostics;
using System.Diagnostics.Metrics;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Perforce;
using HordeServer.VersionControl.Perforce;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.RoboMerge
{
	/// <summary>
	/// Immutable snapshot of all parsed merge graph data. Swapped atomically via volatile write
	/// so readers never see partial updates between graphs and chains.
	/// </summary>
	internal sealed class MergeGraphSnapshot
	{
		/// <summary>All parsed merge graphs, keyed by bot name (case-insensitive)</summary>
		public ImmutableDictionary<string, MergeGraph> Graphs { get; init; }
			= ImmutableDictionary<string, MergeGraph>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>Pre-computed list of all graphs (avoids allocation on every read)</summary>
		public IReadOnlyList<MergeGraph> AllGraphs { get; init; } = Array.Empty<MergeGraph>();

		/// <summary>Bot alias → primary bot name lookup (case-insensitive)</summary>
		public ImmutableDictionary<string, string> AliasIndex { get; init; }
			= ImmutableDictionary<string, string>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>Computed merge chains, keyed by bot name (case-insensitive)</summary>
		public ImmutableDictionary<string, MergeChain> Chains { get; init; }
			= ImmutableDictionary<string, MergeChain>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>Pre-computed list of all chains (avoids allocation on every read)</summary>
		public IReadOnlyList<MergeChain> AllChains { get; init; } = Array.Empty<MergeChain>();

		/// <summary>
		/// Pre-built index mapping normalized stream paths (case-insensitive)
		/// to branch references. Avoids linear scan on every FindBranchesForStream call.
		/// </summary>
		public ImmutableDictionary<string, ImmutableArray<MergeGraphBranchRef>> StreamPathIndex { get; init; }
			= ImmutableDictionary<string, ImmutableArray<MergeGraphBranchRef>>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Pre-built edge lookup: "botName|from|to" → MergeEdge (case-insensitive).
		/// </summary>
		public ImmutableDictionary<string, MergeEdge> EdgeIndex { get; init; }
			= ImmutableDictionary<string, MergeEdge>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>
		/// Pre-computed reachability sets: "botName|branchName" → set of reachable branch names.
		/// </summary>
		public ImmutableDictionary<string, ImmutableHashSet<string>> ReachabilityIndex { get; init; }
			= ImmutableDictionary<string, ImmutableHashSet<string>>.Empty.WithComparers(StringComparer.OrdinalIgnoreCase);

		/// <summary>When the last tick completed (null before first tick).</summary>
		public DateTime? LastTickUtc { get; init; }
	}

	/// <summary>
	/// Ticking service that reads RoboMerge branchmap files from Perforce and serves
	/// them as in-memory merge graph snapshots.
	/// </summary>
	internal sealed class RoboMergeService : IHostedService, IAsyncDisposable, IMergeGraphService
	{
		readonly IPerforceService _perforceService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ILogger<RoboMergeService> _logger;

		readonly ITicker _ticker;

		// Metrics
		readonly Counter<int> _parseFailureCounter;
		readonly Histogram<double> _tickDurationSeconds;

		// Thread-safe atomic snapshot of all parsed data
		volatile MergeGraphSnapshot _snapshot = new MergeGraphSnapshot();

		// Perforce head changelist per depot path — for change detection.
		// Only accessed during tick (single-threaded by AddTicker guarantee), so Dictionary is safe.
		readonly Dictionary<string, int> _headChanges = new Dictionary<string, int>(StringComparer.OrdinalIgnoreCase);

		// Track the previous graph key set to detect changes for logging
		ImmutableSortedSet<string> _previousBotNames = ImmutableSortedSet<string>.Empty;

		/// <summary>
		/// Initial delay before first tick. Short so data is available quickly after startup.
		/// </summary>
		static readonly TimeSpan s_initialDelay = TimeSpan.FromSeconds(15);

		/// <summary>Configured tick interval for subsequent ticks after the initial load.</summary>
		readonly TimeSpan _tickInterval;

		/// <summary>
		/// Constructor
		/// </summary>
		public RoboMergeService(
			IPerforceService perforceService,
			IOptionsMonitor<BuildConfig> buildConfig,
			IClock clock,
			Meter meter,
			ILogger<RoboMergeService> logger)
		{
			_perforceService = perforceService;
			_buildConfig = buildConfig;
			_logger = logger;

			_tickInterval = buildConfig.CurrentValue.Robomerge?.TickInterval ?? TimeSpan.FromMinutes(5);

			_ticker = clock.AddTicker<RoboMergeService>(s_initialDelay, TickInternalAsync, logger);

			_parseFailureCounter = meter.CreateCounter<int>("horde.robomerge.parse_failures");
			_tickDurationSeconds = meter.CreateHistogram<double>("horde.robomerge.tick_duration_seconds");
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
			=> await _ticker.StartAsync();

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
			=> await _ticker.StopAsync();

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
			=> await _ticker.DisposeAsync();

		/// <summary>
		/// Tick wrapper — delegates to TickAsync and returns the configured interval.
		/// </summary>
		async ValueTask<TimeSpan?> TickInternalAsync(CancellationToken cancellationToken)
		{
			await TickAsync(cancellationToken);
			return _tickInterval;
		}

		/// <summary>
		/// Tick callback — reads branchmap files from Perforce and updates in-memory snapshot.
		/// </summary>
		async ValueTask TickAsync(CancellationToken cancellationToken)
		{
			Stopwatch sw = Stopwatch.StartNew();
			try
			{
				BuildConfig config = _buildConfig.CurrentValue;
				List<string>? files = config.Robomerge?.BranchmapFiles;
				if (files == null || files.Count == 0)
				{
					return;
				}

				Dictionary<string, MergeGraph> newGraphs = new Dictionary<string, MergeGraph>(StringComparer.OrdinalIgnoreCase);

				string clusterName = config.Robomerge?.PerforceCluster ?? "default";

				try
				{
					using IPooledPerforceConnection connection = await _perforceService.ConnectAsync(clusterName, cancellationToken: cancellationToken);

					List<string> depotPaths = files.Distinct(StringComparer.OrdinalIgnoreCase).ToList();

					List<FStatRecord> allRecords = await connection.FStatAsync(
						FStatOptions.ShortenOutput, depotPaths, cancellationToken).ToListAsync(cancellationToken);

					Dictionary<string, FStatRecord> depotPathToRecord = allRecords
						.Where(x => x.DepotFile != null)
						.ToDictionary(x => x.DepotFile!, x => x, StringComparer.OrdinalIgnoreCase);

					foreach (string depotPath in depotPaths)
					{
						try
						{
							if (!depotPathToRecord.TryGetValue(depotPath, out FStatRecord? record))
							{
								_logger.LogWarning("FStatAsync returned no results for {DepotPath}", depotPath);
								PreserveExistingGraph(newGraphs, depotPath);
								continue;
							}

							int headChange = record.HeadChange;
							_headChanges.TryGetValue(depotPath, out int cachedChange);

							if (headChange == cachedChange)
							{
								PreserveExistingGraph(newGraphs, depotPath);
								continue;
							}

							PerforceResponse<PrintRecord<byte[]>> printResponse =
								await connection.TryPrintDataAsync($"{depotPath}@{headChange}", cancellationToken);

							if (!printResponse.Succeeded)
							{
								_logger.LogWarning("TryPrintDataAsync failed for {DepotPath}: {Error}",
									depotPath, printResponse.Error);
								PreserveExistingGraph(newGraphs, depotPath);
								continue;
							}

							byte[]? contents = printResponse.Data.Contents;
							if (contents == null || contents.Length == 0)
							{
								_logger.LogWarning("Empty file contents for {DepotPath}", depotPath);
								continue;
							}

							MergeGraph graph = RoboMergeBranchmapParser.Parse(depotPath, contents, headChange, _logger);
							newGraphs[graph.BotName] = graph;
							_headChanges[depotPath] = headChange;
						}
						catch (PerforceException ex)
						{
							_parseFailureCounter.Add(1);
							_logger.LogWarning(ex, "Error reading branchmap file {DepotPath}", depotPath);
							PreserveExistingGraph(newGraphs, depotPath);
						}
						catch (JsonException ex)
						{
							_parseFailureCounter.Add(1);
							_logger.LogWarning(ex, "Error parsing branchmap JSON for {DepotPath}", depotPath);
							PreserveExistingGraph(newGraphs, depotPath);
						}
					}
				}
				catch (PerforceException ex)
				{
					_logger.LogWarning(ex, "Error connecting to Perforce cluster {ClusterName}", clusterName);
					foreach (string depotPath in files)
					{
						PreserveExistingGraph(newGraphs, depotPath);
					}
				}

				// Build immutable snapshot atomically
				ImmutableDictionary<string, MergeGraph> graphsDict =
					newGraphs.ToImmutableDictionary(StringComparer.OrdinalIgnoreCase);

				// Build alias → primary name index
				ImmutableDictionary<string, string>.Builder aliasBuilder =
					ImmutableDictionary.CreateBuilder<string, string>(StringComparer.OrdinalIgnoreCase);
				foreach (MergeGraph graph in newGraphs.Values)
				{
					foreach (string alias in graph.Aliases)
					{
						aliasBuilder[alias] = graph.BotName;
					}
				}

				// Compute merge chains eagerly
				ImmutableDictionary<string, MergeChain>.Builder chainsBuilder =
					ImmutableDictionary.CreateBuilder<string, MergeChain>(StringComparer.OrdinalIgnoreCase);
				foreach (MergeGraph graph in newGraphs.Values)
				{
					MergeChain? chain = RoboMergeBranchmapParser.ComputeMergeChain(graph, _logger);
					if (chain != null)
					{
						chainsBuilder[graph.BotName] = chain;
					}
				}

				// Build stream path index
				Dictionary<string, List<MergeGraphBranchRef>> streamPathGroups =
					new Dictionary<string, List<MergeGraphBranchRef>>(StringComparer.OrdinalIgnoreCase);
				foreach (MergeGraph graph in newGraphs.Values)
				{
					foreach (MergeBranch branch in graph.Branches)
					{
						string streamPath = branch.GetStreamPath(graph.DefaultStreamDepot);
						if (!streamPathGroups.TryGetValue(streamPath, out List<MergeGraphBranchRef>? list))
						{
							list = new List<MergeGraphBranchRef>();
							streamPathGroups[streamPath] = list;
						}
						list.Add(new MergeGraphBranchRef
						{
							BotName = graph.BotName,
							Branch = branch,
							StreamPath = streamPath
						});
					}
				}
				ImmutableDictionary<string, ImmutableArray<MergeGraphBranchRef>> streamPathIndex =
					streamPathGroups.ToImmutableDictionary(
						kvp => kvp.Key,
						kvp => kvp.Value.ToImmutableArray(),
						StringComparer.OrdinalIgnoreCase);

				// Build edge index
				ImmutableDictionary<string, MergeEdge>.Builder edgeIndexBuilder =
					ImmutableDictionary.CreateBuilder<string, MergeEdge>(StringComparer.OrdinalIgnoreCase);
				foreach (MergeGraph graph in newGraphs.Values)
				{
					foreach (MergeEdge edge in graph.Edges)
					{
						string key = $"{graph.BotName}|{edge.From}|{edge.To}";
						edgeIndexBuilder[key] = edge;
					}
				}

				// Build reachability index
				ImmutableDictionary<string, ImmutableHashSet<string>>.Builder reachabilityBuilder =
					ImmutableDictionary.CreateBuilder<string, ImmutableHashSet<string>>(StringComparer.OrdinalIgnoreCase);
				foreach (MergeGraph graph in newGraphs.Values)
				{
					Dictionary<string, MergeBranch> branchMap = graph.Branches
						.ToDictionary(b => b.Name, b => b, StringComparer.OrdinalIgnoreCase);

					Dictionary<string, MergeEdge> localEdgeMap = new Dictionary<string, MergeEdge>(StringComparer.OrdinalIgnoreCase);
					foreach (MergeEdge edge in graph.Edges)
					{
						localEdgeMap[$"{edge.From}|{edge.To}"] = edge;
					}

					foreach (MergeBranch branch in graph.Branches)
					{
						HashSet<string> visited = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
						Queue<string> queue = new Queue<string>();
						queue.Enqueue(branch.Name);
						while (queue.Count > 0)
						{
							string current = queue.Dequeue();
							if (!visited.Add(current))
							{
								continue;
							}
							if (branchMap.TryGetValue(current, out MergeBranch? b))
							{
								foreach (string target in b.FlowsTo)
								{
									if (!visited.Contains(target))
									{
										bool isTerminal = localEdgeMap.TryGetValue($"{current}|{target}", out MergeEdge? e) && e.Terminal;
										if (isTerminal)
										{
											visited.Add(target);
										}
										else
										{
											queue.Enqueue(target);
										}
									}
								}
							}
						}
						visited.Remove(branch.Name);
						string reachKey = $"{graph.BotName}|{branch.Name}";
						reachabilityBuilder[reachKey] = visited.ToImmutableHashSet(StringComparer.OrdinalIgnoreCase);
					}
				}

				// Swap entire snapshot atomically
				ImmutableDictionary<string, MergeChain> chainsDict = chainsBuilder.ToImmutable();
				_snapshot = new MergeGraphSnapshot
				{
					Graphs = graphsDict,
					AllGraphs = graphsDict.Values.ToArray(),
					AliasIndex = aliasBuilder.ToImmutable(),
					Chains = chainsDict,
					AllChains = chainsDict.Values.ToArray(),
					StreamPathIndex = streamPathIndex,
					EdgeIndex = edgeIndexBuilder.ToImmutable(),
					ReachabilityIndex = reachabilityBuilder.ToImmutable(),
					LastTickUtc = DateTime.UtcNow
				};

				// Log only on first load or when the set of loaded bots changes
				ImmutableSortedSet<string> currentBotNames = newGraphs.Keys.ToImmutableSortedSet(StringComparer.OrdinalIgnoreCase);
				if (!currentBotNames.SetEquals(_previousBotNames))
				{
					_logger.LogInformation("Loaded {Count} merge graph(s) from Perforce: {BotNames}",
						newGraphs.Count, String.Join(", ", currentBotNames));
					_previousBotNames = currentBotNames;
				}
			}
			finally
			{
				_tickDurationSeconds.Record(sw.Elapsed.TotalSeconds);
			}
		}

		/// <summary>
		/// Helper to preserve a previously-loaded graph when a file can't be read.
		/// </summary>
		void PreserveExistingGraph(Dictionary<string, MergeGraph> newGraphs, string depotPath)
		{
			string botName = RoboMergeBranchmapParser.ExtractBotName(depotPath);
			if (_snapshot.Graphs.TryGetValue(botName, out MergeGraph? existing))
			{
				newGraphs[existing.BotName] = existing;
			}
		}

		// --- IMergeGraphService implementation (synchronous in-memory reads) ---

		/// <inheritdoc/>
		public DateTime? LastTickUtc => _snapshot.LastTickUtc;

		/// <inheritdoc/>
		public IReadOnlyList<MergeGraph> GetMergeGraphs()
			=> _snapshot.AllGraphs;

		/// <inheritdoc/>
		public MergeGraph? GetMergeGraph(string botName)
		{
			MergeGraphSnapshot snapshot = _snapshot;
			if (snapshot.Graphs.TryGetValue(botName, out MergeGraph? graph))
			{
				return graph;
			}
			if (snapshot.AliasIndex.TryGetValue(botName, out string? primaryName))
			{
				snapshot.Graphs.TryGetValue(primaryName, out graph);
			}
			return graph;
		}

		/// <inheritdoc/>
		public IReadOnlyList<MergeGraphBranchRef> FindBranchesForStreamPath(string streamPath)
		{
			if (!streamPath.StartsWith("//", StringComparison.Ordinal))
			{
				return Array.Empty<MergeGraphBranchRef>();
			}
			string path = streamPath[2..];
			int slashIdx = path.IndexOf('/', StringComparison.Ordinal);
			if (slashIdx <= 0)
			{
				return Array.Empty<MergeGraphBranchRef>();
			}
			string depot = path[..slashIdx];
			string stream = path[(slashIdx + 1)..];
			return FindBranchesForStream(depot, stream);
		}

		/// <inheritdoc/>
		public IReadOnlyList<MergeGraphBranchRef> FindBranchesForStream(string streamDepot, string streamName)
		{
			string streamPath = $"//{streamDepot}/{streamName}";
			MergeGraphSnapshot snapshot = _snapshot;
			if (snapshot.StreamPathIndex.TryGetValue(streamPath, out ImmutableArray<MergeGraphBranchRef> refs))
			{
				return refs;
			}
			return Array.Empty<MergeGraphBranchRef>();
		}

		/// <inheritdoc/>
		public MergeChain? GetMergeChain(string botName)
		{
			MergeGraphSnapshot snapshot = _snapshot;
			if (snapshot.Chains.TryGetValue(botName, out MergeChain? chain))
			{
				return chain;
			}
			if (snapshot.AliasIndex.TryGetValue(botName, out string? primaryName))
			{
				snapshot.Chains.TryGetValue(primaryName, out chain);
			}
			return chain;
		}

		/// <inheritdoc/>
		public IReadOnlyList<MergeChain> GetMergeChains()
			=> _snapshot.AllChains;

		/// <inheritdoc/>
		public IReadOnlySet<string> GetReachableBranches(string botName, string branchName)
		{
			MergeGraphSnapshot snapshot = _snapshot;
			string resolvedBotName = botName;
			if (!snapshot.Graphs.ContainsKey(botName) &&
				snapshot.AliasIndex.TryGetValue(botName, out string? primaryName))
			{
				resolvedBotName = primaryName;
			}

			string key = $"{resolvedBotName}|{branchName}";
			if (snapshot.ReachabilityIndex.TryGetValue(key, out ImmutableHashSet<string>? reachable))
			{
				return reachable;
			}
			return ImmutableHashSet<string>.Empty;
		}

		/// <inheritdoc/>
		public MergeEdge? GetEdge(string botName, string fromBranch, string toBranch)
		{
			MergeGraphSnapshot snapshot = _snapshot;
			string resolvedBotName = botName;
			if (!snapshot.Graphs.ContainsKey(botName) &&
				snapshot.AliasIndex.TryGetValue(botName, out string? primaryName))
			{
				resolvedBotName = primaryName;
			}

			string key = $"{resolvedBotName}|{fromBranch}|{toBranch}";
			snapshot.EdgeIndex.TryGetValue(key, out MergeEdge? edge);
			return edge;
		}
	}
}
