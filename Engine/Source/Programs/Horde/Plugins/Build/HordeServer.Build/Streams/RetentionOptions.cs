// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.Streams
{
	/// <summary>
	/// Options controlling storage retention and GC behavior for a stream.
	/// Boolean flags are safety valves: only <c>false</c> is meaningful (disables behavior).
	/// <c>null</c> means "not disabled" (default). Setting <c>true</c> is identical to <c>null</c>.
	/// </summary>
	public class RetentionOptions
	{
		/// <summary>
		/// Whether job expiration is enabled for this stream. When null (default),
		/// expiration runs normally if ExpireAfterDays is greater than 0. When explicitly false,
		/// disables expiration regardless of ExpireAfterDays (safety valve).
		/// </summary>
		public bool? EnableJobExpiration { get; set; }

		/// <summary>
		/// Override for artifact retention behavior. When false, artifact expiry
		/// is paused for this stream (useful during investigations).
		/// </summary>
		public bool? EnableArtifactExpiration { get; set; }

		/// <summary>
		/// Maximum number of jobs to delete per hourly expiration tick, per stream.
		/// Prevents thundering-herd behavior when ExpireAfterDays is first enabled
		/// on a stream with a large backlog. Default null, treated as 1000 by consumers
		/// via null-coalescing (safe-by-default cap). Set to 0 for unlimited.
		/// </summary>
		public int? MaxJobDeletionsPerTick { get; set; }

		/// <summary>
		/// Merge defaults from another options object
		/// </summary>
		public void MergeDefaults(RetentionOptions other)
		{
			EnableJobExpiration ??= other.EnableJobExpiration;
			EnableArtifactExpiration ??= other.EnableArtifactExpiration;
			MaxJobDeletionsPerTick ??= other.MaxJobDeletionsPerTick;
		}
	}
}
