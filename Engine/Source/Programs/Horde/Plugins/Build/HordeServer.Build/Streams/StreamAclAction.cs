// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Streams
{
	/// <summary>
	/// ACL actions that can be performed on streams
	/// </summary>
	public static class StreamAclAction
	{
		/// <summary>
		/// Allows the creation of new streams within a project
		/// </summary>
		public static AclAction CreateStream { get; } = new AclAction("CreateStream");

		/// <summary>
		/// Allows updating a stream (agent types, templates, schedules)
		/// </summary>
		public static AclAction UpdateStream { get; } = new AclAction("UpdateStream");

		/// <summary>
		/// Allows deleting a stream
		/// </summary>
		public static AclAction DeleteStream { get; } = new AclAction("DeleteStream");

		/// <summary>
		/// Ability to view a stream
		/// </summary>
		public static AclAction ViewStream { get; } = new AclAction("ViewStream");

		/// <summary>
		/// View changes submitted to a stream. NOTE: this returns responses from the server's Perforce account, which may be a priviledged user.
		/// </summary>
		public static AclAction ViewChanges { get; } = new AclAction("ViewChanges");

		/// <summary>
		/// View template associated with a stream
		/// </summary>
		public static AclAction ViewTemplate { get; } = new AclAction("ViewTemplate");
	}
}
