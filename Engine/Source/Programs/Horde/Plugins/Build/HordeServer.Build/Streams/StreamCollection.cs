// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Schedules;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Commits;
using HordeServer.Jobs.Schedules;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Streams
{
	/// <summary>
	/// Collection of stream documents
	/// </summary>
	public class StreamCollection : IStreamCollection
	{
		class Stream : IStream
		{
			readonly StreamCollection _collection;
			readonly StreamConfig _config;
			readonly StreamDoc _document;
			readonly Dictionary<TemplateId, ITemplateRef> _templateRefs = new Dictionary<TemplateId, ITemplateRef>();

			public StreamDoc Document => _document;

			public StreamId Id => _config.Id;
			public StreamConfig Config => _config;
			public IReadOnlyDictionary<TemplateId, ITemplateRef> Templates => _templateRefs;
			public DateTime? PausedUntil => _document.PausedUntil;
			public string? PauseComment => _document.PauseComment;

			ProjectId IStream.ProjectId => _config.ProjectConfig.Id;
			string IStream.Name => _config.Name;
			string? IStream.ConfigPath => _config.Path;
			string IStream.ConfigRevision => _config.Revision;
			int IStream.Order => _config.Order;
			string? IStream.NotificationChannel => _config.NotificationChannel;
			string? IStream.NotificationChannelFilter => _config.NotificationChannelFilter;
			string? IStream.TriageChannel => _config.TriageChannel;
			IReadOnlyList<IStreamTab> IStream.Tabs => _config.Tabs;
			IDefaultPreflight? IStream.DefaultPreflight => _config.DefaultPreflight;

			IReadOnlyDictionary<string, IAgentType>? _cachedAgentTypes;
			IReadOnlyDictionary<string, IAgentType> IStream.AgentTypes
				=> _cachedAgentTypes ??= _config.AgentTypes.ToDictionary(x => x.Key, x => (IAgentType)x.Value);

			IReadOnlyDictionary<string, IWorkspaceType>? _cachedWorkspaceTypes;
			IReadOnlyDictionary<string, IWorkspaceType> IStream.WorkspaceTypes
				=> _cachedWorkspaceTypes ??= _config.WorkspaceTypes.ToDictionary(x => x.Key, x => (IWorkspaceType)x.Value);

			IReadOnlyList<IWorkflow> IStream.Workflows => _config.Workflows;

			IReadOnlyList<IStreamAlias>? _cachedAliases;
			IReadOnlyList<IStreamAlias> IStream.Aliases
				=> _cachedAliases ??= _config.Aliases.Select(x => (IStreamAlias)x).ToList();

			ICommitCollection IStream.Commits => _collection._commitService.GetCollection(_config);

			public Stream(StreamCollection collection, StreamConfig config, StreamDoc document)
			{
				_collection = collection;
				_config = config;
				_document = document;

				foreach (TemplateRefConfig templateRefConfig in config.Templates)
				{
					TemplateRefDoc templateRefDoc = _document.Templates[templateRefConfig.Id];
					_templateRefs[templateRefConfig.Id] = new TemplateRef(templateRefConfig, templateRefDoc);
				}
			}

			public Task<IStream?> RefreshAsync(CancellationToken cancellationToken = default)
				=> _collection.GetAsync(Id, cancellationToken);

			public async Task<IStream?> TryUpdatePauseStateAsync(DateTime? newPausedUntil, string? newPauseComment, CancellationToken cancellationToken = default)
			{
				StreamDoc? newDocument = await _collection.TryUpdatePauseStateAsync(_document, newPausedUntil, newPauseComment, cancellationToken);
				return (newDocument == null) ? null : new Stream(_collection, _config, newDocument);
			}

			public async Task<IStream?> TryUpdateScheduleTriggerAsync(TemplateId templateRefId, DateTime? lastTriggerTimeUtc, CommitIdWithOrder? lastTriggerCommitId, List<JobId> newActiveJobs, CancellationToken cancellationToken = default)
			{
				StreamDoc? newDocument = await _collection.TryUpdateScheduleTriggerAsync(_document, templateRefId, lastTriggerTimeUtc, lastTriggerCommitId, newActiveJobs, cancellationToken);
				return (newDocument == null) ? null : new Stream(_collection, _config, newDocument);
			}

			public async Task<IStream?> TryUpdateTemplateRefAsync(TemplateId templateRefId, List<UpdateStepStateRequest>? stepStates = null, CancellationToken cancellationToken = default)
			{
				StreamDoc? newDocument = await _collection.TryUpdateTemplateRefAsync(_document, templateRefId, stepStates, cancellationToken);
				return (newDocument == null) ? null : new Stream(_collection, _config, newDocument);
			}
		}

		class TemplateRef : ITemplateRef
		{
			readonly TemplateRefConfig _config;
			readonly TemplateRefDoc _document;
			readonly Schedule? _schedule;

			public TemplateRef(TemplateRefConfig config, TemplateRefDoc document)
			{
				_config = config;
				_document = document;

				if (_config.Schedule != null)
				{
					_schedule = new Schedule(_config.Schedule, _document.Schedule ?? new ScheduleDoc());
				}
			}

			public TemplateId Id => _config.Id;
			public ITemplate Template => throw new NotImplementedException();
			public TemplateRefConfig Config => _config;
			public ISchedule? Schedule => _schedule;
			public IReadOnlyList<ITemplateStep> StepStates => (IReadOnlyList<ITemplateStep>?)_document.StepStates ?? Array.Empty<ITemplateStep>();

			bool ITemplateRef.ShowUgsBadges => _config.ShowUgsBadges;
			bool ITemplateRef.ShowUgsAlerts => _config.ShowUgsAlerts;
			string? ITemplateRef.NotificationChannel => _config.NotificationChannel;
			string? ITemplateRef.NotificationChannelFilter => _config.NotificationChannelFilter;
			string? ITemplateRef.TriageChannel => _config.TriageChannel;
			IReadOnlyList<IChainedJobTemplate>? ITemplateRef.ChainedJobs => _config.ChainedJobs;
			IReadOnlyList<IChangeQuery>? ITemplateRef.DefaultChange => _config.DefaultChange;
		}

		class Schedule : ISchedule
		{
			readonly ScheduleConfig _config;
			readonly ScheduleDoc _document;

			public IReadOnlyList<IScheduleClaim>? Claims => _config.Claims;
			public bool Enabled => _config.Enabled;
			public int MaxActive => _config.MaxActive;
			public int MaxChanges => _config.MaxChanges;
			public bool RequireSubmittedChange => _config.RequireSubmittedChange;
			public IScheduleGate? Gate => _config.Gate;
			public IReadOnlyList<CommitTag> Commits => _config.Commits;
			public CommitIdWithOrder? LastTriggerCommitId => _document.LastTriggerCommitId;
			public DateTime LastTriggerTimeUtc => _document.LastTriggerTimeUtc;
			public IReadOnlyList<JobId> ActiveJobs => _document.ActiveJobs;
			public IReadOnlyList<ISchedulePattern> Patterns => _config.Patterns;
			public IReadOnlyList<string>? Files => _config.Files;
			public IReadOnlyDictionary<string, string> TemplateParameters => _config.TemplateParameters;

			public Schedule(ScheduleConfig config, ScheduleDoc document)
			{
				_config = config;
				_document = document;
			}
		}

		/// <summary>
		/// Information about a stream
		/// </summary>
		class StreamDoc
		{
			[BsonRequired, BsonId]
			public StreamId Id { get; set; }

			public Dictionary<TemplateId, TemplateRefDoc> Templates { get; set; } = new Dictionary<TemplateId, TemplateRefDoc>();
			public DateTime? PausedUntil { get; set; }
			public string? PauseComment { get; set; }

			public int UpdateIndex { get; set; }

			[BsonConstructor]
			private StreamDoc()
			{
			}

			public StreamDoc(StreamId id)
			{
				Id = id;
			}

			public bool PostLoad(StreamConfig config, DateTime utcNow)
			{
				bool replaceDocument = false;

				Dictionary<TemplateId, TemplateRefDoc> templateRefDocs = new Dictionary<TemplateId, TemplateRefDoc>(config.Templates.Count);
				foreach (TemplateRefConfig templateRefConfig in config.Templates)
				{
					TemplateRefDoc? templateRefDoc;
					if (!Templates.TryGetValue(templateRefConfig.Id, out templateRefDoc))
					{
						templateRefDoc = new TemplateRefDoc();
						replaceDocument = true;
					}
					replaceDocument |= templateRefDoc.PostLoad(templateRefConfig, utcNow);

					templateRefDocs.Add(templateRefConfig.Id, templateRefDoc);
				}

				replaceDocument |= Templates.Count != templateRefDocs.Count;
				Templates = templateRefDocs;

				return replaceDocument;
			}
		}

		class TemplateRefDoc
		{
			[BsonIgnoreIfNull]
			public ScheduleDoc? Schedule { get; set; }

			[BsonIgnoreIfNull]
			public List<TemplateStepDoc>? StepStates { get; set; }

			public bool PostLoad(TemplateRefConfig config, DateTime utcNow)
			{
				bool replaceDocument = false;

				if (config.Schedule == null)
				{
					if (Schedule != null)
					{
						Schedule = null;
						replaceDocument = true;
					}
				}
				else
				{
					if (Schedule == null)
					{
						Schedule = new ScheduleDoc();
						Schedule.LastTriggerTimeUtc = utcNow;
						replaceDocument = true;
					}
				}

				Schedule?.PostLoad();

				if (StepStates != null)
				{
					int count = StepStates.RemoveAll(x => x.PausedByUserId == null);
					replaceDocument |= count > 0;
				}

				return replaceDocument;
			}
		}

		class ScheduleDoc
		{
			public CommitIdWithOrder LastTriggerCommitId
			{
				get => (LastTriggerCommitName != null) ? new CommitIdWithOrder(LastTriggerCommitName, LastTriggerCommitOrder) : CommitIdWithOrder.FromPerforceChange(LastTriggerCommitOrder);
				set => (LastTriggerCommitName, LastTriggerCommitOrder) = (value.Name, value.Order);
			}

			public string? LastTriggerCommitName { get; set; }

			[BsonElement("LastTriggerChange")]
			public int LastTriggerCommitOrder { get; set; }

			[BsonIgnoreIfNull, Obsolete("Use LastTriggerTimeUtc instead")]
			public DateTimeOffset? LastTriggerTime { get; set; }

			public DateTime LastTriggerTimeUtc { get; set; }
			public List<JobId> ActiveJobs { get; set; } = new List<JobId>();

			public void PostLoad()
			{
#pragma warning disable CS0618 // Type or member is obsolete
				if (LastTriggerTime.HasValue)
				{
					LastTriggerTimeUtc = LastTriggerTime.Value.UtcDateTime;
					LastTriggerTime = null;
				}
#pragma warning restore CS0618 // Type or member is obsolete
			}
		}

		class TemplateStepDoc : ITemplateStep
		{
			public string Name { get; set; } = String.Empty;
			public UserId? PausedByUserId { get; set; }
			public DateTime? PauseTimeUtc { get; set; }

			UserId ITemplateStep.PausedByUserId => PausedByUserId ?? UserId.Empty;
			DateTime ITemplateStep.PauseTimeUtc => PauseTimeUtc ?? DateTime.MinValue;

			public TemplateStepDoc()
			{
			}

			public TemplateStepDoc(string name, UserId pausedByUserId, DateTime pauseTimeUtc)
			{
				Name = name;
				PausedByUserId = pausedByUserId;
				PauseTimeUtc = pauseTimeUtc;
			}
		}

		readonly IMongoCollection<StreamDoc> _streams;
		readonly ICommitService _commitService;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IClock _clock;

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamCollection(IMongoService mongoService, ICommitService commitService, IOptionsMonitor<BuildConfig> buildConfig, IClock clock)
		{
			_streams = mongoService.GetCollection<StreamDoc>("Streams");
			_commitService = commitService;
			_buildConfig = buildConfig;
			_clock = clock;
		}

		/// <inheritdoc/>
		public async Task<IStream?> GetAsync(StreamId id, CancellationToken cancellationToken)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.CurrentValue.TryGetStream(id, out streamConfig))
			{
				return null;
			}

			StreamDoc document = await GetInternalAsync(streamConfig, cancellationToken);
			return new Stream(this, streamConfig, document);
		}

		async Task<StreamDoc> GetInternalAsync(StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			FindOneAndUpdateOptions<StreamDoc, StreamDoc> options = new FindOneAndUpdateOptions<StreamDoc, StreamDoc>();
			options.IsUpsert = true;
			options.ReturnDocument = ReturnDocument.After;

			for (; ; )
			{
				StreamDoc stream = await _streams.FindOneAndUpdateAsync<StreamDoc>(x => x.Id == streamConfig.Id, Builders<StreamDoc>.Update.SetOnInsert(x => x.UpdateIndex, 0), options, cancellationToken);
				if (await PostLoadAsync(stream, streamConfig, cancellationToken))
				{
					return stream;
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IStream>> FindAsync(IReadOnlyList<StreamId>? ids = null, CancellationToken cancellationToken = default)
		{
			List<StreamConfig> configs = new List<StreamConfig>();
			foreach (StreamConfig config in _buildConfig.CurrentValue.Streams)
			{
				if (ids == null || ids.Contains(config.Id))
				{
					configs.Add(config);
				}
			}

			List<StreamDoc> documents = await GetInternalAsync(configs, cancellationToken);

			Stream[] streams = new Stream[configs.Count];
			for (int idx = 0; idx < configs.Count; idx++)
			{
				streams[idx] = new Stream(this, configs[idx], documents[idx]);
			}

			return streams;
		}

		async Task<List<StreamDoc>> GetInternalAsync(IReadOnlyList<StreamConfig> streamConfigs, CancellationToken cancellationToken)
		{
			FilterDefinition<StreamDoc> filter = Builders<StreamDoc>.Filter.In(x => x.Id, streamConfigs.Select(x => x.Id));
			List<StreamDoc> matches = await _streams.Find(filter).ToListAsync(cancellationToken);

			List<StreamDoc> results = new List<StreamDoc>(streamConfigs.Count);
			foreach (StreamConfig streamConfig in streamConfigs)
			{
				StreamDoc? stream = matches.FirstOrDefault(x => x.Id == streamConfig.Id);
				if (stream == null)
				{
					stream = await GetInternalAsync(streamConfig, cancellationToken);
				}
				else
				{
					stream.PostLoad(streamConfig, _clock.UtcNow);
				}
				results.Add(stream);
			}
			return results;
		}

		async ValueTask<bool> PostLoadAsync(StreamDoc streamDoc, StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			if (streamDoc.PostLoad(streamConfig, _clock.UtcNow))
			{
				int updateIndex = streamDoc.UpdateIndex++;

				ReplaceOneResult result = await _streams.ReplaceOneAsync(x => x.Id == streamDoc.Id && x.UpdateIndex == updateIndex, streamDoc, cancellationToken: cancellationToken);
				if (result.MatchedCount != 1)
				{
					return false;
				}
			}
			return true;
		}

		/// <inheritdoc/>
		async Task<StreamDoc?> TryUpdateTemplateRefAsync(StreamDoc document, TemplateId templateId, List<UpdateStepStateRequest>? stepStates = null, CancellationToken cancellationToken = default)
		{
			TemplateRefDoc? templateRef;
			if (!document.Templates.TryGetValue(templateId, out templateRef))
			{
				return null;
			}

			UpdateDefinitionBuilder<StreamDoc> updateBuilder = Builders<StreamDoc>.Update;
			List<UpdateDefinition<StreamDoc>> updates = new List<UpdateDefinition<StreamDoc>>();

			Dictionary<TemplateId, TemplateRefDoc> newTemplates = new Dictionary<TemplateId, TemplateRefDoc>(document.Templates);

			// clear
			if (stepStates != null && stepStates.Count == 0)
			{
				bool hasUpdates = false;
				foreach (KeyValuePair<TemplateId, TemplateRefDoc> entry in newTemplates)
				{
					if (entry.Value.StepStates != null)
					{
						hasUpdates = true;
						entry.Value.StepStates = null;
					}
				}

				if (hasUpdates)
				{
					updates.Add(updateBuilder.Set(x => x.Templates, newTemplates));
				}
			}
			else if (stepStates != null)
			{
				// get currently valid step states
				List<TemplateStepDoc> newStepStates = templateRef.StepStates?.ToList() ?? new List<TemplateStepDoc>();

				// generate update list
				foreach (UpdateStepStateRequest updateState in stepStates)
				{
					int stateIndex = newStepStates.FindIndex(x => x.Name == updateState.Name);

					UserId? pausedByUserId = updateState.PausedByUserId != null ? UserId.Parse(updateState.PausedByUserId) : null;

					if (stateIndex == -1)
					{
						// if this is a new state without anything set, ignore it
						if (pausedByUserId != null)
						{
							newStepStates.Add(new TemplateStepDoc(updateState.Name, pausedByUserId.Value, _clock.UtcNow));
						}
					}
					else
					{
						if (pausedByUserId == null)
						{
							newStepStates.RemoveAt(stateIndex);
						}
						else
						{
							newStepStates[stateIndex].PausedByUserId = pausedByUserId.Value;
						}
					}
				}

				if (newStepStates.Count == 0)
				{
					templateRef.StepStates = null;
				}
				else
				{
					templateRef.StepStates = newStepStates;
				}

				updates.Add(updateBuilder.Set(x => x.Templates, newTemplates));
			}

			if (updates.Count == 0)
			{
				return document;
			}

			return await TryUpdateStreamAsync(document, updateBuilder.Combine(updates), cancellationToken);

		}

		/// <inheritdoc/>
		async Task<StreamDoc?> TryUpdatePauseStateAsync(StreamDoc document, DateTime? newPausedUntil, string? newPauseComment, CancellationToken cancellationToken)
		{
			UpdateDefinitionBuilder<StreamDoc> updateBuilder = Builders<StreamDoc>.Update;

			List<UpdateDefinition<StreamDoc>> updates = new List<UpdateDefinition<StreamDoc>>();
			document.PausedUntil = newPausedUntil;
			document.PauseComment = newPauseComment;
			updates.Add(updateBuilder.Set(x => x.PausedUntil, newPausedUntil));
			updates.Add(updateBuilder.Set(x => x.PauseComment, newPauseComment));

			return await TryUpdateStreamAsync(document, updateBuilder.Combine(updates), cancellationToken);
		}

		/// <inheritdoc/>
		async Task<StreamDoc?> TryUpdateScheduleTriggerAsync(StreamDoc streamDoc, TemplateId templateId, DateTime? lastTriggerTimeUtc, CommitIdWithOrder? lastTriggerCommitId, List<JobId> newActiveJobs, CancellationToken cancellationToken)
		{
			TemplateRefDoc templateRefDoc = streamDoc.Templates[templateId];
			ScheduleDoc scheduleDoc = templateRefDoc.Schedule!;

			// Build the updates. MongoDB driver cannot parse TemplateRefId in expression tree; need to specify field name explicitly
			List<UpdateDefinition<StreamDoc>> updates = new List<UpdateDefinition<StreamDoc>>();
			if (lastTriggerTimeUtc.HasValue && lastTriggerTimeUtc.Value != scheduleDoc.LastTriggerTimeUtc)
			{
				FieldDefinition<StreamDoc, DateTime> lastTriggerTimeField = $"{nameof(streamDoc.Templates)}.{templateId}.{nameof(templateRefDoc.Schedule)}.{nameof(scheduleDoc.LastTriggerTimeUtc)}";
				updates.Add(Builders<StreamDoc>.Update.Set(lastTriggerTimeField, lastTriggerTimeUtc.Value));

				scheduleDoc.LastTriggerTimeUtc = lastTriggerTimeUtc.Value;
			}
			if (lastTriggerCommitId != null && lastTriggerCommitId > scheduleDoc.LastTriggerCommitId)
			{
				FieldDefinition<StreamDoc, string> lastTriggerCommitNameField = $"{nameof(streamDoc.Templates)}.{templateId}.{nameof(templateRefDoc.Schedule)}.{nameof(scheduleDoc.LastTriggerCommitName)}";
				updates.Add(Builders<StreamDoc>.Update.Set(lastTriggerCommitNameField, lastTriggerCommitId.Name));

				FieldDefinition<StreamDoc, int> lastTriggerChangeField = $"{nameof(streamDoc.Templates)}.{templateId}.{nameof(templateRefDoc.Schedule)}.LastTriggerChange";
				updates.Add(Builders<StreamDoc>.Update.Set(lastTriggerChangeField, lastTriggerCommitId.Order));

				scheduleDoc.LastTriggerCommitId = lastTriggerCommitId;
			}
			if (newActiveJobs != null)
			{
				FieldDefinition<StreamDoc, List<JobId>> field = $"{nameof(streamDoc.Templates)}.{templateId}.{nameof(templateRefDoc.Schedule)}.{nameof(scheduleDoc.ActiveJobs)}";
				updates.Add(Builders<StreamDoc>.Update.Set(field, newActiveJobs));

				scheduleDoc.ActiveJobs = newActiveJobs;
			}

			if (updates.Count == 0)
			{
				return streamDoc;
			}

			return await TryUpdateStreamAsync(streamDoc, Builders<StreamDoc>.Update.Combine(updates), cancellationToken);
		}

		/// <summary>
		/// Update a stream
		/// </summary>
		/// <param name="streamDoc">The stream to update</param>
		/// <param name="update">The update definition</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated document, or null the update failed</returns>
		private async Task<StreamDoc?> TryUpdateStreamAsync(StreamDoc streamDoc, UpdateDefinition<StreamDoc> update, CancellationToken cancellationToken)
		{
			FilterDefinition<StreamDoc> filter = Builders<StreamDoc>.Filter.Expr(x => x.Id == streamDoc.Id && x.UpdateIndex == streamDoc.UpdateIndex);
			update = update.Set(x => x.UpdateIndex, streamDoc.UpdateIndex + 1);

			FindOneAndUpdateOptions<StreamDoc> options = new FindOneAndUpdateOptions<StreamDoc> { ReturnDocument = ReturnDocument.After };
			return await _streams.FindOneAndUpdateAsync(filter, update, options, cancellationToken);
		}

		/// <summary>
		/// Checks the stream definition for consistency
		/// </summary>
		public static void Validate(StreamId streamId, StreamConfig config)
		{
			HashSet<TemplateId> remainingTemplates = new HashSet<TemplateId>(config.Templates.Select(x => x.Id));

			// Check the default preflight template is valid
			if (config.DefaultPreflight != null)
			{
				if (config.DefaultPreflight.TemplateId != null && !remainingTemplates.Contains(config.DefaultPreflight.TemplateId.Value))
				{
					throw new InvalidStreamException($"Default preflight template was listed as '{config.DefaultPreflight.TemplateId.Value}', but no template was found by that name");
				}
			}

			// Check the chained jobs are valid
			foreach (TemplateRefConfig templateRef in config.Templates)
			{
				if (templateRef.ChainedJobs != null)
				{
					foreach (ChainedJobTemplateConfig chainedJob in templateRef.ChainedJobs)
					{
						if (!remainingTemplates.Contains(chainedJob.TemplateId))
						{
							throw new InvalidDataException($"Invalid template ref id '{chainedJob.TemplateId}");
						}
					}
				}
			}

			// Check that all the templates are referenced by a tab
			HashSet<TemplateId> undefinedTemplates = new();
			foreach (TabConfig tab in config.Tabs)
			{
				if (tab.Templates != null)
				{
					remainingTemplates.ExceptWith(tab.Templates);
					foreach (TemplateId templateId in tab.Templates)
					{
						if (config.Templates.Find(x => x.Id == templateId) == null)
						{
							undefinedTemplates.Add(templateId);
						}
					}
				}
			}
			if (remainingTemplates.Count > 0)
			{
				throw new InvalidStreamException(String.Join("\n", remainingTemplates.Select(x => $"Template '{x}' is not listed on any tab for {streamId}")));
			}

			if (undefinedTemplates.Count > 0)
			{
				throw new InvalidStreamException(String.Join("\n", undefinedTemplates.Select(x => $"Template '{x}' is not defined for {streamId}")));
			}

			// Check that all the agent types reference valid workspace names
			foreach (KeyValuePair<string, AgentConfig> pair in config.AgentTypes)
			{
				string? workspaceTypeName = pair.Value.Workspace;
				if (workspaceTypeName != null && !config.WorkspaceTypes.ContainsKey(workspaceTypeName))
				{
					throw new InvalidStreamException($"Agent type '{pair.Key}' references undefined workspace type '{pair.Value.Workspace}' in {streamId}");
				}
			}
		}
	}
}
