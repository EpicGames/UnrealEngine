// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Security.Claims;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Text.RegularExpressions;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Acls;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Common;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Graphs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Replicators;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Telemetry;
using EpicGames.Perforce;
using Horde.Common.Rpc;
using HordeServer.Acls;
using HordeServer.Artifacts;
using HordeServer.Configuration;
using HordeServer.Issues;
using HordeServer.Jobs;
using HordeServer.Jobs.Bisect;
using HordeServer.Jobs.Schedules;
using HordeServer.Jobs.Templates;
using HordeServer.Notifications;
using HordeServer.Projects;
using HordeServer.Replicators;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Streams
{
	/// <summary>
	/// Flags identifying content of a changelist
	/// </summary>
	[Flags]
	[Obsolete("Use tags instead")]
	public enum ChangeContentFlags
	{
		/// <summary>
		/// The change contains code
		/// </summary>
		ContainsCode = 1,

		/// <summary>
		/// The change contains content
		/// </summary>
		ContainsContent = 2,
	}

	/// <summary>
	/// How to replicate data for this stream
	/// </summary>
	public enum ContentReplicationMode
	{
		/// <summary>
		/// No content will be replicated for this stream
		/// </summary>
		None,

		/// <summary>
		/// Only replicate depot path and revision data for each file
		/// </summary>
		RevisionsOnly,

		/// <summary>
		/// Replicate full stream contents to storage
		/// </summary>
		Full,
	}

	/// <summary>
	/// Tags for a stream
	/// </summary>
	public class StreamTag
	{
		/// <summary>
		/// Name of the tag
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Whether the tag should be enabled
		/// </summary>
		public bool Enabled { get; set; }
	}

	/// <summary>
	/// Config for a stream
	/// </summary>
	[JsonSchema("https://unrealengine.com/horde/stream")]
	[JsonSchemaCatalog("Horde Stream", "Horde stream configuration file", new[] { "*.stream.json", "Streams/*.json" })]
	[ConfigDoc("*.stream.json", "[Horde](../../../README.md) > [Configuration](../../Config.md)", "Config/Schema/Streams.md")]
	[ConfigIncludeRoot]
	[ConfigMacroScope]
	[DebuggerDisplay("{Id}")]
	public class StreamConfig
	{
		/// <summary>
		/// Accessor for the project containing this stream
		/// </summary>
		[JsonIgnore]
		public ProjectConfig ProjectConfig { get; private set; } = null!;

		/// <summary>
		/// Identifier for the stream
		/// </summary>
		public StreamId Id { get; set; }

		/// <summary>
		/// Version control service for stream commits
		/// </summary>
		public string VCS { get; set; } = "Perforce";

		/// <summary>
		/// Repository name for version control systems that support multiple repositories
		/// </summary>
		public string? RepositoryName { get; set; }

		/// <summary>
		/// Default branch name for version control systems that use branches (e.g., "main").
		/// If not set, defaults to "main".
		/// </summary>
		public string? DefaultBranchName { get; set; }

		/// <summary>
		/// Direct include path for the stream config. For backwards compatibility with old config files when including from a ProjectConfig object.
		/// </summary>
		[ConfigInclude, ConfigRelativePath]
		public string? Path { get; set; }

		/// <summary>
		/// Includes for other configuration files
		/// </summary>
		public List<ConfigInclude> Include { get; set; } = new List<ConfigInclude>();

		/// <summary>
		/// Macros within this stream
		/// </summary>
		public List<ConfigMacro> Macros { get; set; } = new List<ConfigMacro>();

		/// <summary>
		/// Revision identifier for this configuration object
		/// </summary>
		[JsonIgnore]
		public string Revision { get; set; } = String.Empty;

		/// <summary>
		/// Name of the stream
		/// </summary>
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Path to the engine directory within the workspace. Used for launching UAT.
		/// </summary>
		public string EnginePath { get; set; } = "Engine";

		/// <summary>
		/// Additional file paths (relative to workspace root) containing ignore patterns for the log parser.
		/// Each file should contain one regex per line, with lines starting with '#' treated as comments.
		/// Inherits from ProjectConfig and appends stream-level entries.
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<string>? IgnorePatternFiles { get; set; }

		/// <summary>
		/// The perforce cluster containing the stream
		/// </summary>
		public string ClusterName { get; set; } = PerforceCluster.DefaultName;

		/// <summary>
		/// Order for this stream
		/// </summary>
		public int Order { get; set; } = 128;

		/// <summary>
		/// Default initial agent type for templates
		/// </summary>
		public string? InitialAgentType { get; set; }

		/// <summary>
		/// Notification channel for all jobs in this stream
		/// </summary>
		public string? NotificationChannel { get; set; }

		/// <summary>
		/// Notification channel filter for this template. Can be Success, Failure, or Warnings.
		/// </summary>
		public string? NotificationChannelFilter { get; set; }

		/// <summary>
		/// Channel to post issue triage notifications
		/// </summary>
		public string? TriageChannel { get; set; }

		/// <summary>
		/// Default settings for executing jobs
		/// </summary>
		public JobOptions JobOptions { get; set; } = new JobOptions();

		/// <summary>
		/// Storage retention options for this stream
		/// </summary>
		public RetentionOptions RetentionOptions { get; set; } = new RetentionOptions();

		/// <summary>
		/// Telemetry store for Horde data for this stream
		/// </summary>
		public TelemetryStoreId TelemetryStoreId { get; set; }

		/// <summary>
		/// View for the AutoSDK paths to sync. If null, the whole thing will be synced.
		/// </summary>
		public List<string>? AutoSdkView { get; set; }

		/// <summary>
		/// Legacy name for the default preflight template
		/// </summary>
		[Obsolete("Use DefaultPreflight instead")]
		public string? DefaultPreflightTemplate
		{
			get => DefaultPreflight?.TemplateId?.ToString();
			set => DefaultPreflight = (value == null) ? null : new DefaultPreflightConfig { TemplateId = new TemplateId(value) };
		}

		/// <summary>
		/// Default template for running preflights
		/// </summary>
		public DefaultPreflightConfig? DefaultPreflight { get; set; }

		/// <summary>
		/// List of tags to apply to commits. Allows fast searching and classification of different commit types (eg. code vs content).
		/// </summary>
		public List<CommitTagConfig> CommitTags { get; set; } = new List<CommitTagConfig>();

		/// <summary>
		/// List of tabs to show for the new stream
		/// </summary>
		public List<TabConfig> Tabs { get; set; } = new List<TabConfig>();

		/// <summary>
		/// Global environment variables for all agents in this stream
		/// </summary>
		public Dictionary<string, string>? Environment { get; set; }

		/// <summary>
		/// Map of agent name to type
		/// </summary>
		public Dictionary<string, AgentConfig> AgentTypes { get; set; } = new Dictionary<string, AgentConfig>();

		/// <summary>
		/// Map of workspace name to type
		/// </summary>
		public Dictionary<string, WorkspaceConfig> WorkspaceTypes { get; set; } = new Dictionary<string, WorkspaceConfig>();

		/// <summary>
		/// List of templates to create
		/// </summary>
		public List<TemplateRefConfig> Templates { get; set; } = new List<TemplateRefConfig>();

		/// <summary>
		/// Stream aliases configured for this stream.
		/// Each alias provides a named subset of the stream's templates.
		/// </summary>
		public List<StreamAliasConfig> Aliases { get; set; } = new List<StreamAliasConfig>();

		/// <summary>
		/// Custom permissions for this object
		/// </summary>
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <summary>
		/// Pause stream builds until specified date
		/// </summary>
		public DateTime? PausedUntil { get; set; }

		/// <summary>
		/// Reason for pausing builds of the stream
		/// </summary>
		public string? PauseComment { get; set; }

		/// <summary>
		/// Configuration for workers to replicate commit data into Horde Storage.
		/// </summary>
		public List<ReplicatorConfig> Replicators { get; set; } = new List<ReplicatorConfig>();

		/// <summary>
		/// Workflows for dealing with new issues
		/// </summary>
		public List<WorkflowConfig> Workflows { get; set; } = new List<WorkflowConfig>();

		/// <summary>
		/// Tokens to create for each job step
		/// </summary>
		public List<TokenConfig> Tokens { get; set; } = new List<TokenConfig>();

		/// <summary>
		/// Permissions for artifact types
		/// </summary>
		public List<ArtifactTypeConfig> ArtifactTypes { get; set; } = new List<ArtifactTypeConfig>();

		/// <summary>
		/// Tags for the stream
		/// </summary>
		public List<StreamTag> StreamTags { get; set; } = [];

		readonly Dictionary<ArtifactType, ArtifactTypeConfig> _artifactTypeLookup = new Dictionary<ArtifactType, ArtifactTypeConfig>();

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);

		/// <summary>
		/// Callback after reading this stream configuration
		/// </summary>
		/// <param name="id">The stream id</param>
		/// <param name="projectConfig">Owning project</param>
		/// <param name="parentArtifactTypes">Parent artifact types</param>
		/// <param name="logger">Logger for messages</param>
		public void PostLoad(StreamId id, ProjectConfig projectConfig, IEnumerable<ArtifactTypeConfig> parentArtifactTypes, ILogger? logger)
		{
			Id = id;
			ProjectConfig = projectConfig;

			AclAction[] aclActions = AclConfig.GetActions(
				[
					typeof(BisectTaskAclAction),
					typeof(JobAclAction),
					typeof(NotificationAclAction),
					typeof(ReplicatorAclAction),
					typeof(StreamAclAction)
				]);
			Acl.PostLoad(projectConfig.Acl, $"stream:{Id}", aclActions);
			Acl.LegacyScopeNames = projectConfig.Acl.LegacyScopeNames?.Select(x => x.Append($"s:{Id}")).ToArray();

			JobOptions.MergeDefaults(projectConfig.JobOptions);
			RetentionOptions.MergeDefaults(projectConfig.RetentionOptions);

			if (TelemetryStoreId.IsEmpty)
			{
				TelemetryStoreId = projectConfig.TelemetryStoreId;
			}

			foreach (TemplateRefConfig template in Templates)
			{
				template.PostLoad(this);
			}

			foreach ((string wid, WorkspaceConfig wc) in projectConfig.WorkspaceTypes)
			{
				WorkspaceTypes.TryAdd(wid, wc);
			}

			if (projectConfig.IgnorePatternFiles != null && projectConfig.IgnorePatternFiles.Count > 0)
			{
				if (IgnorePatternFiles == null)
				{
					IgnorePatternFiles = new List<string>(projectConfig.IgnorePatternFiles);
				}
				else
				{
					IgnorePatternFiles.InsertRange(0, projectConfig.IgnorePatternFiles);
				}
			}

			ConfigObject.MergeDefaults(AgentTypes.Select(x => (x.Key, x.Value.Base, x.Value)), logger);
			ConfigObject.MergeDefaults(WorkspaceTypes.Select(x => (x.Key, x.Value.Base, x.Value)), logger);
			ConfigObject.MergeDefaults(Templates.Select(x => (x.Id, x.Base, x)), logger);

			foreach (TemplateRefConfig template in Templates)
			{
				template.JobOptions.MergeDefaults(JobOptions);
			}

			foreach (TemplateRefConfig template in Templates)
			{
				HashSet<ParameterId> parameterIds = new HashSet<ParameterId>();
				foreach (TemplateParameterConfig parameter in template.Parameters)
				{
					parameter.PostLoad(parameterIds);
				}
			}

			Dictionary<string, string> scheduleConditionPropertyLookup = new();
			foreach (StreamTag streamTag in StreamTags)
			{
				scheduleConditionPropertyLookup.TryAdd(streamTag.Name, streamTag.Enabled.ToString());
			}

			foreach (TemplateRefConfig template in Templates)
			{
				ScheduleConfig? schedule = template.Schedule;
				if (schedule != null)
				{
#pragma warning disable CS0618 // Type or member is obsolete
					List<ChangeContentFlags>? filter = schedule.Filter;
					if (filter != null && filter.Count > 0)
					{
						List<CommitTag> commits = schedule.Commits;
						if (filter.Contains(ChangeContentFlags.ContainsCode) && !commits.Contains(CommitTag.Code))
						{
							commits.Add(CommitTag.Code);
						}
						if (filter.Contains(ChangeContentFlags.ContainsContent) && !commits.Contains(CommitTag.Content))
						{
							commits.Add(CommitTag.Content);
						}
					}
#pragma warning restore CS0618 // Type or member is obsolete

					foreach (CommitTag commitTag in schedule.Commits)
					{
						if (!TryGetCommitTag(commitTag, out _))
						{
							throw new InvalidOperationException($"Missing definition for commit tag '{commitTag}' referenced by {Id}:{template.Id}");
						}
					}

					if (schedule.Condition != null)
					{
						bool condition = schedule.Condition.Evaluate(name =>
						{
							if (scheduleConditionPropertyLookup.TryGetValue(name, out string? enabled))
							{
								return [enabled];
							}
							throw new ConditionException($"Unknown property '{name}' in schedule condition '{schedule.Condition}'");
						});

						if (schedule.Enabled)
						{
							logger?.LogInformation("Schedule for template {TemplateId} is {Enabled} given the condition '{Condition}'", template.Id, condition ? "enabled" : "disabled", schedule.Condition);
						}

						schedule.Enabled &= condition;
					}
				}
			}

			if (Environment != null && Environment.Count > 0)
			{
				foreach (AgentConfig agentConfig in AgentTypes.Values)
				{
					agentConfig.Environment ??= new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);
					foreach ((string key, string value) in Environment)
					{
						agentConfig.Environment.TryAdd(key, value);
					}
				}
			}

			// Build a set of all the artifact types
			_artifactTypeLookup.Clear();
			foreach (ArtifactTypeConfig artifactTypeConfig in parentArtifactTypes)
			{
				_artifactTypeLookup[artifactTypeConfig.Type] = artifactTypeConfig;
			}
			foreach (ArtifactTypeConfig artifactTypeConfig in ArtifactTypes)
			{
				artifactTypeConfig.PostLoad(Acl);
				_artifactTypeLookup[artifactTypeConfig.Type] = artifactTypeConfig;
			}

			// Add the default artifact types
			EnsureDefaultArtifactType(ArtifactType.StepOutput);
			EnsureDefaultArtifactType(ArtifactType.StepSaved);
			EnsureDefaultArtifactType(ArtifactType.StepTrace);
			EnsureDefaultArtifactType(ArtifactType.StepTestData);

			// Compute a hash of this stream revision to make it easier to detect changes
			byte[] streamData = JsonSerializer.SerializeToUtf8Bytes(this, JsonUtils.DefaultSerializerOptions);
			Revision = IoHash.Compute(streamData).ToString();
		}

		/// <summary>
		/// Get all artifact types for this stream
		/// </summary>
		public IEnumerable<ArtifactTypeConfig> GetAllArtifactTypes()
			=> _artifactTypeLookup.Values;

		/// <summary>
		/// Finds an artifact type configuration block
		/// </summary>
		/// <param name="type">The artifact type</param>
		/// <param name="artifactTypeConfig">Receives the configuration for the artifact type</param>
		public bool TryGetArtifactType(ArtifactType type, [NotNullWhen(true)] out ArtifactTypeConfig? artifactTypeConfig)
			=> _artifactTypeLookup.TryGetValue(type, out artifactTypeConfig);

		void EnsureDefaultArtifactType(ArtifactType artifactType)
		{
			if (!_artifactTypeLookup.ContainsKey(artifactType))
			{
				ArtifactTypeConfig artifactTypeConfig = new ArtifactTypeConfig { Type = artifactType };
				ArtifactTypes.Add(artifactTypeConfig);
				_artifactTypeLookup.Add(artifactType, artifactTypeConfig);
			}
		}

		/// <summary>
		/// Enumerates all commit tags, including the default tags for code and content.
		/// </summary>
		/// <returns></returns>
		public IEnumerable<CommitTagConfig> GetAllCommitTags()
		{
			if (TryGetCommitTag(CommitTag.Code, out CommitTagConfig? codeConfig))
			{
				yield return codeConfig;
			}
			if (TryGetCommitTag(CommitTag.Content, out CommitTagConfig? contentConfig))
			{
				yield return contentConfig;
			}
			foreach (CommitTagConfig config in CommitTags)
			{
				if (config.Name != CommitTag.Code && config.Name != CommitTag.Content)
				{
					yield return config;
				}
			}
		}

		/// <summary>
		/// Gets the configuration for a commit tag
		/// </summary>
		/// <param name="tag">Tag to search for</param>
		/// <param name="config">Receives the tag configuration</param>
		/// <returns>True if a match was found</returns>
		public bool TryGetCommitTag(CommitTag tag, [NotNullWhen(true)] out CommitTagConfig? config)
		{
			CommitTagConfig? first = CommitTags.FirstOrDefault(x => x.Name == tag);
			if (first != null)
			{
				config = first;
				return true;
			}
			else if (tag == CommitTag.Code)
			{
				config = CommitTagConfig.CodeDefault;
				return true;
			}
			else if (tag == CommitTag.Content)
			{
				config = CommitTagConfig.ContentDefault;
				return true;
			}
			else
			{
				config = null;
				return false;
			}
		}

		/// <summary>
		/// Constructs a <see cref="FileFilter"/> from the rules in this configuration object
		/// </summary>
		/// <returns>Filter object</returns>
		public bool TryGetCommitTagFilter(CommitTag tag, [NotNullWhen(true)] out FileFilter? filter)
		{
			FileFilter newFilter = new FileFilter(FileFilterType.Exclude);
			if (TryGetCommitTagFilter(tag, newFilter, new HashSet<CommitTag>()))
			{
				filter = newFilter;
				return true;
			}
			else
			{
				filter = null;
				return false;
			}
		}

		/// <summary>
		/// Find the filter for a given tag, recursively
		/// </summary>
		bool TryGetCommitTagFilter(CommitTag tag, FileFilter filter, HashSet<CommitTag> visitedTags)
		{
			// Check we don't have a recursive definition for the tag
			if (!visitedTags.Add(tag))
			{
				return false;
			}

			// Get the tag configuration
			CommitTagConfig? config;
			if (!TryGetCommitTag(tag, out config))
			{
				return false;
			}

			// Add rules from the base tag
			if (!config.Base.IsEmpty() && !TryGetCommitTagFilter(config.Base, filter, visitedTags))
			{
				return false;
			}

			// Add rules from this tag
			filter.AddRules(config.Filter);
			return true;
		}

		/// <summary>
		/// Tries to find a replicator with the given id
		/// </summary>
		/// <param name="replicatorId"></param>
		/// <param name="replicatorConfig"></param>
		/// <returns></returns>
		public bool TryGetReplicator(StreamReplicatorId replicatorId, [NotNullWhen(true)] out ReplicatorConfig? replicatorConfig)
		{
			replicatorConfig = Replicators.FirstOrDefault(x => x.Id == replicatorId);
			return replicatorConfig != null;
		}

		/// <summary>
		/// Tries to find a template with the given id
		/// </summary>
		/// <param name="templateRefId"></param>
		/// <param name="templateConfig"></param>
		/// <returns></returns>
		public bool TryGetTemplate(TemplateId templateRefId, [NotNullWhen(true)] out TemplateRefConfig? templateConfig)
		{
			templateConfig = Templates.FirstOrDefault(x => x.Id == templateRefId);
			return templateConfig != null;
		}

		/// <summary>
		/// Tries to find a workflow with the given id
		/// </summary>
		/// <param name="workflowId"></param>
		/// <param name="workflowConfig"></param>
		/// <returns></returns>
		public bool TryGetWorkflow(WorkflowId workflowId, [NotNullWhen(true)] out WorkflowConfig? workflowConfig)
		{
			workflowConfig = Workflows.FirstOrDefault(x => x.Id == workflowId);
			return workflowConfig != null;
		}

		/// <summary>
		/// The escaped name of this stream. Removes all non-identifier characters.
		/// </summary>
		/// <returns>Escaped name for the stream</returns>
		public string GetEscapedName()
		{
			return Regex.Replace(Name, @"[^a-zA-Z0-9_]", "+");
		}

		/// <summary>
		/// Gets the default identifier for workspaces created for this stream. Just includes an escaped depot name.
		/// </summary>
		/// <returns>The default workspace identifier</returns>
		public string GetDefaultWorkspaceIdentifier()
		{
			return Regex.Replace(GetEscapedName(), @"^(\+\+[^+]*).*$", "$1");
		}
	}

	/// <summary>
	/// Information about a page to display in the dashboard for a stream
	/// </summary>
	public class TabConfig : IStreamTab
	{
		/// <inheritdoc/>
		[Required]
		public string Title { get; set; } = null!;

		/// <inheritdoc/>
		public string Type { get; set; } = "Jobs";

		/// <inheritdoc/>
		public TabStyle Style { get; set; }

		/// <inheritdoc/>
		public bool ShowNames
		{
			get => _showNames ?? (Style != TabStyle.Compact);
			set => _showNames = value;
		}
		bool? _showNames;

		/// <inheritdoc/>
		public bool? ShowPreflights { get; set; }

		/// <inheritdoc cref="IStreamTab.JobNames"/>
		public List<string>? JobNames { get; set; }
		IReadOnlyList<string>? IStreamTab.JobNames => JobNames;

		/// <inheritdoc cref="IStreamTab.Templates"/>
		public List<TemplateId>? Templates { get; set; }
		IReadOnlyList<TemplateId>? IStreamTab.Templates => Templates;

		/// <inheritdoc cref="IStreamTab.Columns"/>
		public List<TabColumnConfig>? Columns { get; set; }
		IReadOnlyList<IStreamTabColumn>? IStreamTab.Columns => Columns;
	}

	/// <summary>
	/// Describes a column to display on the jobs page
	/// </summary>
	public class TabColumnConfig : IStreamTabColumn
	{
		/// <inheritdoc/>
		public TabColumnType Type { get; set; } = TabColumnType.Labels;

		/// <inheritdoc/>
		[Required]
		public string Heading { get; set; } = null!;

		/// <inheritdoc/>
		public string? Category { get; set; }

		/// <inheritdoc/>
		public string? Parameter { get; set; }

		/// <inheritdoc/>
		public int? RelativeWidth { get; set; }
	}

	/// <summary>
	/// Mapping from a BuildGraph agent type to a set of machines on the farm
	/// </summary>
	public class AgentConfig : IAgentType
	{
		/// <summary>
		/// Base agent config to inherit settings from
		/// </summary>
		public string? Base { get; set; }

		/// <summary>
		/// Pool of agents to use for this agent type
		/// </summary>
		public PoolId Pool { get; set; }

		/// <summary>
		/// Name of the workspace to sync
		/// </summary>
		public string? Workspace { get; set; }

		/// <summary>
		/// Path to the temporary storage dir
		/// </summary>
		public string? TempStorageDir { get; set; }

		/// <summary>
		/// Environment variables to be set when executing the job
		/// </summary>
		public Dictionary<string, string>? Environment { get; set; }
		IReadOnlyDictionary<string, string>? IAgentType.Environment => Environment;

		/// <summary>
		/// Tokens to allocate for this agent type
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<TokenConfig>? Tokens { get; set; }

		/// <summary>
		/// Creates an API response object from this stream
		/// </summary>
		/// <returns>The response object</returns>
		public RpcGetAgentTypeResponse ToRpcResponse()
		{
			RpcGetAgentTypeResponse response = new RpcGetAgentTypeResponse();
			if (TempStorageDir != null)
			{
				response.TempStorageDir = TempStorageDir;
			}
			if (Environment != null)
			{
				response.Environment.Add(Environment);
			}
			return response;
		}
	}

	/// <summary>
	/// Information about a workspace type
	/// </summary>
	public class WorkspaceConfig : IWorkspaceType
	{
		/// <inheritdoc/>
		public string? Base { get; set; }

		/// <inheritdoc/>
		public string? Cluster { get; set; }

		/// <inheritdoc/>
		public string? ServerAndPort { get; set; }

		/// <inheritdoc/>
		public string? UserName { get; set; }

		/// <inheritdoc/>
		public string? Identifier { get; set; }

		/// <inheritdoc/>
		public string? Stream { get; set; }

		/// <inheritdoc cref="IWorkspaceType.View"/>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<string>? View { get; set; }
		IReadOnlyList<string>? IWorkspaceType.View => View;

		/// <inheritdoc/>
		public bool? Incremental { get; set; }

		/// <inheritdoc/>
		public bool? UseAutoSdk { get; set; }

		/// <inheritdoc cref="IWorkspaceType.AutoSdkView"/>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<string>? AutoSdkView { get; set; }
		IReadOnlyList<string>? IWorkspaceType.AutoSdkView => AutoSdkView;

		/// <inheritdoc/>
		public string? Method { get; set; } = null;

		/// <inheritdoc/>
		public long? MinScratchSpace { get; set; } = null;

		/// <inheritdoc/>
		public long? ConformDiskFreeSpace { get; set; } = null;
	}

	/// <summary>
	/// Specifies defaults for running a preflight
	/// </summary>
	public class DefaultPreflightConfig : IDefaultPreflight
	{
		/// <summary>
		/// The template id to query
		/// </summary>
		public TemplateId? TemplateId { get; set; }

		/// <summary>
		/// Query for the change to use
		/// </summary>
		[Obsolete("Set on template instead")]
		public ChangeQueryConfig? Change { get; set; }

		[Obsolete("Set on template instead")]
		IChangeQuery? IDefaultPreflight.Change => Change;
	}

	/// <summary>
	/// Trigger for another template
	/// </summary>
	public class ChainedJobTemplateConfig : IChainedJobTemplate
	{
		/// <inheritdoc/>
		[Required]
		public string Trigger { get; set; } = String.Empty;

		/// <inheritdoc/>
		[Required]
		public TemplateId TemplateId { get; set; }

		/// <inheritdoc/>
		public bool UseDefaultChangeForTemplate { get; set; }
	}

	/// <summary>
	/// Parameters to create a template within a stream
	/// </summary>
	[DebuggerDisplay("{Id}")]
	public class TemplateRefConfig : TemplateConfig
	{
		/// <summary>
		/// The owning stream config
		/// </summary>
		[JsonIgnore]
		public StreamConfig StreamConfig { get; private set; } = null!;

		/// <summary>
		/// Optional identifier for this ref. If not specified, an id will be generated from the name.
		/// </summary>
		public TemplateId Id { get; set; }

		/// <summary>
		/// Base template id to copy from
		/// </summary>
		public TemplateId Base { get; set; }

		/// <summary>
		/// Whether to show badges in UGS for these jobs
		/// </summary>
		public bool ShowUgsBadges { get; set; }

		/// <summary>
		/// Whether to show alerts in UGS for these jobs
		/// </summary>
		public bool ShowUgsAlerts { get; set; }

		/// <summary>
		/// Notification channel for this template. Overrides the stream channel if set.
		/// </summary>
		public string? NotificationChannel { get; set; }

		/// <summary>
		/// Notification channel filter for this template. Can be a combination of "Success", "Failure" and "Warnings" separated by pipe characters.
		/// </summary>
		public string? NotificationChannelFilter { get; set; }

		/// <summary>
		/// Triage channel for this template. Overrides the stream channel if set.
		/// </summary>
		public string? TriageChannel { get; set; }

		/// <summary>
		/// Workflow to user for this stream
		/// </summary>
		public WorkflowId? WorkflowId
		{
			get => Annotations.WorkflowId;
			set => Annotations.WorkflowId = value;
		}

		/// <summary>
		/// Default annotations to apply to nodes in this template
		/// </summary>
		public NodeAnnotations Annotations { get; set; } = new NodeAnnotations();

		/// <summary>
		/// Schedule to execute this template
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Recursive)]
		public ScheduleConfig? Schedule { get; set; }

		/// <summary>
		/// List of chained job triggers
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Append)]
		public List<ChainedJobTemplateConfig>? ChainedJobs { get; set; }

		/// <summary>
		/// The ACL for this template
		/// </summary>
		[ConfigMergeStrategy(ConfigMergeStrategy.Recursive)]
		public AclConfig Acl { get; set; } = new AclConfig();

		/// <inheritdoc cref="AclConfig.Authorize(AclAction, ClaimsPrincipal)"/>
		public bool Authorize(AclAction action, ClaimsPrincipal user)
			=> Acl.Authorize(action, user);

		/// <summary>
		/// Callback after the config is loaded
		/// </summary>
		/// <param name="streamConfig"></param>
		public void PostLoad(StreamConfig streamConfig)
		{
			StreamConfig = streamConfig;

			InitialAgentType ??= streamConfig.InitialAgentType;

			if (Id.IsEmpty)
			{
				Id = new TemplateId(StringId.Sanitize(Name));
			}

			Acl.PostLoad(streamConfig.Acl, $"template:{Id}", []);
			Acl.LegacyScopeNames = streamConfig.Acl.LegacyScopeNames?.Select(x => x.Append($"t:{Id}")).ToArray();
		}
	}

	/// <summary>
	/// Configuration for allocating access tokens for each job
	/// </summary>
	public class TokenConfig
	{
		/// <summary>
		/// URL to request tokens from
		/// </summary>
		[Required]
		public Uri Url { get; set; } = null!;

		/// <summary>
		/// Client id to use to request a new token
		/// </summary>
		[Required]
		[ResolveSecret]
		public string ClientId { get; set; } = String.Empty;

		/// <summary>
		/// Client secret to request a new access token
		/// </summary>
		[Required]
		[ResolveSecret]
		public string ClientSecret { get; set; } = String.Empty;

		/// <summary>
		/// Environment variable to set with the access token
		/// </summary>
		[Required]
		public string EnvVar { get; set; } = String.Empty;
	}

	/// <summary>
	/// Configuration for a stream alias, which represents a named subset of templates
	/// </summary>
	public class StreamAliasConfig : IStreamAlias
	{
		/// <inheritdoc/>
		[Required]
		public string Name { get; set; } = String.Empty;

		/// <inheritdoc cref="IStreamAlias.Templates"/>
		[Required]
		public List<TemplateId> Templates { get; set; } = new List<TemplateId>();
		IReadOnlyList<TemplateId> IStreamAlias.Templates => Templates;
	}

	/// <summary>
	/// Configuration for custom commit filters
	/// </summary>
	public class CommitTagConfig
	{
		/// <summary>
		/// Name of the tag
		/// </summary>
		[Required]
		public CommitTag Name { get; set; }

		/// <summary>
		/// Base tag to copy settings from
		/// </summary>
		public CommitTag Base { get; set; }

		/// <summary>
		/// List of files to be included in this filter
		/// </summary>
		public List<string> Filter { get; set; } = new List<string>();

		/// <summary>
		/// Default config for code filters
		/// </summary>
		public static CommitTagConfig CodeDefault { get; } = new CommitTagConfig
		{
			Name = CommitTag.Code,
			Filter = PerforceUtils.CodeExtensions.Select(x => $"*{x}").ToList()
		};

		/// <summary>
		/// Default config for content filters
		/// </summary>
		public static CommitTagConfig ContentDefault { get; } = new CommitTagConfig
		{
			Name = CommitTag.Content,
			Filter = Enumerable.Concat(new[] { "*" }, PerforceUtils.CodeExtensions.Select(x => $"-*{x}")).ToList()
		};
	}
}
