// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Projects;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Auditing;
using HordeServer.Commits;
using HordeServer.Jobs;
using HordeServer.Jobs.Schedules;
using HordeServer.Jobs.Templates;
using HordeServer.Projects;
using HordeServer.Server;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Streams
{
	/// <summary>
	/// Controller for the /api/v1/streams endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class StreamsController : HordeControllerBase
	{
		private readonly IStreamCollection _streamCollection;
		private readonly ICommitService _commitService;
		private readonly ITemplateCollectionInternal _templateCollection;
		private readonly IJobStepRefCollection _jobStepRefCollection;
		private readonly IUserCollection _userCollection;
		private readonly ScheduleService _scheduleService;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;
		private readonly TimeZoneInfo _timeZone;

		/// <summary>
		/// Constructor
		/// </summary>
		public StreamsController(IStreamCollection streamCollection, ICommitService commitService, ITemplateCollectionInternal templateCollection, IJobStepRefCollection jobStepRefCollection, IUserCollection userCollection, ScheduleService scheduleService, Tracer tracer, IClock clock, IOptionsSnapshot<BuildConfig> buildConfig, ILogger<StreamsController> logger)
		{
			_streamCollection = streamCollection;
			_commitService = commitService;
			_templateCollection = templateCollection;
			_jobStepRefCollection = jobStepRefCollection;
			_userCollection = userCollection;
			_scheduleService = scheduleService;
			_tracer = tracer;
			_logger = logger;
			_buildConfig = buildConfig;
			_timeZone = clock.TimeZone;
		}

		/// <summary>
		/// Query all the streams for a particular project.
		/// </summary>
		/// <param name="projectIds">Unique id of the project to query</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about all the projects</returns>
		[HttpGet]
		[Route("/api/v1/streams")]
		[ProducesResponseType(typeof(List<GetStreamResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetStreamsAsync([FromQuery(Name = "ProjectId")] string[] projectIds, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			ProjectId[] projectIdValues = Array.ConvertAll(projectIds, x => new ProjectId(x));

			List<StreamConfig> streamConfigs = new List<StreamConfig>();
			foreach (ProjectConfig projectConfig in _buildConfig.Value.Projects)
			{
				if (projectIdValues.Length == 0 || projectIdValues.Contains(projectConfig.Id))
				{
					foreach (StreamConfig streamConfig in projectConfig.Streams)
					{
						if (streamConfig.Authorize(StreamAclAction.ViewStream, User))
						{
							streamConfigs.Add(streamConfig);
						}
					}
				}
			}

			IReadOnlyList<IStream> streams = await _streamCollection.FindAsync(streamConfigs.ConvertAll(x => x.Id), cancellationToken);

			List<GetStreamResponse> responses = new List<GetStreamResponse>();
			foreach (IStream stream in streams)
			{
				StreamConfig streamConfig = streamConfigs.First(x => x.Id == stream.Id);
				GetStreamResponse response = await CreateGetStreamResponseAsync(stream, streamConfig, cancellationToken);
				responses.Add(response);
			}

			return responses.OrderBy(x => x.Id.Id.Text, Utf8StringComparer.Ordinal).Select(x => PropertyFilter.Apply(x, filter)).ToList();
		}

		/// <summary>
		/// Retrieve information about a specific stream.
		/// </summary>
		/// <param name="streamId">Id of the stream to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Information about the requested project</returns>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}")]
		[ProducesResponseType(typeof(GetStreamResponse), 200)]
		public async Task<ActionResult<object>> GetStreamAsync(StreamId streamId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewStream, User))
			{
				return Forbid(StreamAclAction.ViewStream, streamId);
			}

			IStream? stream = await _streamCollection.GetAsync(streamConfig.Id, cancellationToken);
			if (stream == null)
			{
				return NotFound(streamId);
			}

			return PropertyFilter.Apply(await CreateGetStreamResponseAsync(stream, streamConfig, cancellationToken), filter);
		}

		/// <summary>
		/// Create a stream response object, including all the templates
		/// </summary>
		/// <param name="stream">Stream to create response for</param>
		/// <param name="streamConfig">Config object for this stream</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Response object</returns>
		async Task<GetStreamResponse> CreateGetStreamResponseAsync(IStream stream, StreamConfig streamConfig, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(StreamsController)}.{nameof(CreateGetStreamResponseAsync)}");
			span.SetAttribute("streamId", stream.Id);

			List<GetTemplateRefResponse> apiTemplateRefs = new List<GetTemplateRefResponse>();
			foreach (KeyValuePair<TemplateId, ITemplateRef> pair in stream.Templates)
			{
				TemplateRefConfig? templateRefConfig;
				if (!streamConfig.TryGetTemplate(pair.Key, out templateRefConfig))
				{
					continue;
				}

				using TelemetrySpan templateScope = _tracer.StartActiveSpan($"{nameof(StreamsController)}.{nameof(CreateGetStreamResponseAsync)}.Template");
				templateScope.SetAttribute("templateName", templateRefConfig.Name);

				ITemplateRef templateRef = pair.Value;
				if (templateRefConfig.Authorize(StreamAclAction.ViewTemplate, User))
				{
					List<GetTemplateStepStateResponse>? stepStates = null;
					if (templateRef.StepStates != null)
					{
						for (int i = 0; i < templateRef.StepStates.Count; i++)
						{
							ITemplateStep state = templateRef.StepStates[i];

							stepStates ??= new List<GetTemplateStepStateResponse>();

							GetThinUserInfoResponse? pausedByUserInfo = (await _userCollection.GetCachedUserAsync(state.PausedByUserId, cancellationToken))?.ToThinApiResponse();
							stepStates.Add(new GetTemplateStepStateResponse(state, pausedByUserInfo));
						}
					}

					bool canRun = templateRefConfig.Authorize(JobAclAction.CreateJob, User);

					ITemplate? template = await _templateCollection.GetOrAddAsync(templateRefConfig);
					apiTemplateRefs.Add(new GetTemplateRefResponse(pair.Key, pair.Value, template, stepStates, _timeZone, canRun));
				}
			}

			return new GetStreamResponse(stream, apiTemplateRefs);
		}

		/// <summary>
		/// Headers key utilized for expressing missing commits in <see cref="GetChangesAsync"/> and <see cref="GetBatchedChangesAsync"/>.
		/// </summary>
		public const string MissingCommitsHeader = "X-Missing-Commits";

		private const int MaxConcurrentSCMChangeRequests = 32;

		/// <summary>
		/// Gets a list of changes for a stream
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="min">The starting changelist number</param>
		/// <param name="max">The ending changelist number</param>
		/// <param name="changeNumbers">
		///		Array of specific changelist IDs to retrieve for the stream.
		///		If provided, this parameter is mutually exclusive with <paramref name="min"/>, <paramref name="max"/>, <paramref name="tags"/>,
		///		A <see cref="BadRequestObjectResult"/> will return if any of the above are provided alongside changeNumbers.
		///		<paramref name="results"/> will be ignored.
		/// </param>
		/// <param name="results">Number of results to return</param>
		/// <param name="tags">Tags to filter the changes returned</param>
		/// <param name="filter">The filter to apply to the results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// Returns 200 OK with list of filtered commits. Partial failures for batch retrieval are indicated via the <see cref="MissingCommitsHeader"/> header.
		/// <remarks>Batch request mode executed with <paramref name="changeNumbers"/> will store any commits that were not found or that threw an exception in the <see cref="MissingCommitsHeader"/> response header.</remarks>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}/changes")]
		[ProducesResponseType(typeof(List<GetCommitResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetChangesAsync(StreamId streamId, [FromQuery] CommitId? min = null, [FromQuery] CommitId? max = null, [FromQuery] CommitId[]? changeNumbers = null, [FromQuery] int results = 50, [FromQuery] string? tags = null, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewChanges, User))
			{
				return Forbid(StreamAclAction.ViewChanges, streamId);
			}
			if (changeNumbers != null && (min != null || max != null || tags != null))
			{
				return BadRequest("The changeNumbers option is mutually exclusive with min/max/tag options.");
			}

			List<CommitTag>? commitTags = null;
			if (tags != null)
			{
				commitTags = tags.Split(';', StringSplitOptions.RemoveEmptyEntries).Select(x => new CommitTag(x)).ToList();
			}

			List<ICommit>? commits = null;
			IReadOnlyCollection<string> failedCommits = [];

			if (changeNumbers != null) // Batch commit request path.
			{
				(commits, failedCommits) = await GetBatchedCommitsAsync(streamConfig, changeNumbers, cancellationToken);
			}
			else // Change range & tag request path.
			{
				commits = await _commitService.GetCollection(streamConfig).FindAsync(min, max, results, commitTags, cancellationToken).ToListAsync(cancellationToken);
			}

			// Store any failed commits (only populated through batch commit request path) into the response header.
			if (failedCommits.Count > 0)
			{
				Response.Headers.Append(MissingCommitsHeader, String.Join(",", failedCommits));
			}

			List<GetCommitResponse> responses = new List<GetCommitResponse>(commits.Count);
			foreach (ICommit commit in commits)
			{
				IUser? author = await _userCollection.GetCachedUserAsync(commit.AuthorId, cancellationToken);
				responses.Add(CreateGetCommitResponse(commit, author!, null, null));
			}
			return responses.ConvertAll(x => PropertyFilter.Apply(x, filter));
		}

		/// <summary>
		/// Gets a list of changes for a stream
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="changeNumbers">Array of specific changelist IDs to retrieve for the stream.</param>
		/// <param name="filter">The filter to apply to the results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// Returns 200 OK with list of filtered commits. Partial failures for batch retrieval are indicated via the <see cref="MissingCommitsHeader"/> header.
		/// <remarks>
		/// This endpoint is a workaround to retrieve large sets of changes which would result in 414 reponses from <see cref="GetChangesAsync"/>.
		/// Any commits that were not found or that threw an exception are stored in the <see cref="MissingCommitsHeader"/> response header.
		/// </remarks>
		[HttpPost]
		[Route("/api/v1/streams/{streamId}/changes")]
		[ProducesResponseType(typeof(List<GetCommitResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetBatchedChangesAsync(StreamId streamId, [FromBody] CommitId[]? changeNumbers = null, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewChanges, User))
			{
				return Forbid(StreamAclAction.ViewChanges, streamId);
			}

			List<ICommit>? commits;
			IReadOnlyCollection<string> failedCommits = [];

			if (changeNumbers != null)
			{
				(commits, failedCommits) = await GetBatchedCommitsAsync(streamConfig, changeNumbers, cancellationToken);
			}
			else
			{
				commits = new List<ICommit>();
			}

			// Store any failed commits into the response header.
			if (failedCommits.Count > 0)
			{
				Response.Headers.Append(MissingCommitsHeader, String.Join(",", failedCommits));
			}

			List<GetCommitResponse> responses = new List<GetCommitResponse>(commits.Count);
			foreach (ICommit commit in commits)
			{
				IUser? author = await _userCollection.GetCachedUserAsync(commit.AuthorId, cancellationToken);
				responses.Add(CreateGetCommitResponse(commit, author!, null, null));
			}
			return responses.ConvertAll(x => PropertyFilter.Apply(x, filter));
		}

		async Task<(List<ICommit>, IReadOnlyCollection<string>)> GetBatchedCommitsAsync(StreamConfig streamConfig, CommitId[] changeNumbers, CancellationToken cancellationToken)
		{
			List<ICommit> commits = new List<ICommit>(changeNumbers.Length);
			ConcurrentBag<string> failedCommits = [];

			// This is a pre-emptive measure as to not inundate our SCM with too many concurrent requests.
			using SemaphoreSlim maxSCMActiveRequests = new SemaphoreSlim(MaxConcurrentSCMChangeRequests);
			Task<ICommit?>[] commitServiceTasks = [.. changeNumbers.Select(async id =>
				{
					await maxSCMActiveRequests.WaitAsync(cancellationToken);
					try
					{
						return await _commitService.GetCollection(streamConfig).GetAsync(id, cancellationToken);
					}
					catch (Exception ex) when (ex is not OperationCanceledException)
					{
						_logger.LogWarning(ex, "Failed to retrieve commit {CommitId} for stream {StreamId}", id, streamConfig.Id);
						failedCommits.Add(id.ToString());

						return null;
					}
					finally
					{
						maxSCMActiveRequests.Release();
					}
				})];

			ICommit?[] commitServiceResults = await Task.WhenAll(commitServiceTasks);
			commits = [.. commitServiceResults.Where(c => c != null).Select(c => c!)];

			return ([.. commits.OrderByDescending(x => x.Id)], failedCommits.ToList());
		}

		static GetCommitResponse CreateGetCommitResponse(ICommit commit, IUser author, IReadOnlyList<CommitTag>? tags, IReadOnlyList<string>? files)
		{
			GetCommitResponse response = new GetCommitResponse(author.ToThinApiResponse(), commit.Description, commit.DateUtc) { Id = commit.Id };
			if (tags != null)
			{
				response.Tags = new List<CommitTag>(tags);
			}
			if (files != null)
			{
				response.Files = new List<string>(files);
			}

			return response;
		}

		/// <summary>
		/// Gets a list of changes for a stream
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="changeNumber">The changelist number</param>
		/// <param name="maxFiles">Maximum number of files to return</param>
		/// <param name="filter">The filter to apply to the results</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Http result code</returns>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}/changes/{changeNumber}")]
		[ProducesResponseType(typeof(GetCommitResponse), 200)]
		public async Task<ActionResult<object>> GetChangeDetailsAsync(StreamId streamId, CommitId changeNumber, int maxFiles = 100, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(StreamAclAction.ViewChanges, User))
			{
				return Forbid(StreamAclAction.ViewChanges, streamId);
			}

			ICommit? changeDetails = await _commitService.GetCollection(streamConfig).GetAsync(changeNumber, cancellationToken);
			if (changeDetails == null)
			{
				return NotFound("CL {Change} not found in stream {StreamId}", changeNumber, streamId);
			}

			IUser? author = await _userCollection.GetCachedUserAsync(changeDetails.AuthorId, cancellationToken);
			IReadOnlyList<CommitTag> tags = await changeDetails.GetTagsAsync(cancellationToken);
			IReadOnlyList<string> files = await changeDetails.GetFilesAsync(maxFiles, cancellationToken);

			return PropertyFilter.Apply(CreateGetCommitResponse(changeDetails, author!, tags, files), filter);
		}

		/// <summary>
		/// Gets the history of a step in the stream
		/// </summary>
		/// <param name="streamId">The stream id</param>
		/// <param name="templateId"></param>
		/// <param name="step">Name of the step to search for</param>
		/// <param name="change">Maximum changelist number to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">The filter to apply to the results</param>
		/// <returns>Http result code</returns>
		/// <remarks>This will exclude <see cref="JobStepState.Ready"/>, <see cref="JobStepState.Waiting"/>, <see cref="JobStepState.Skipped"/> states.</remarks>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}/history")]
		[ProducesResponseType(typeof(List<GetJobStepRefResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetStepHistoryAsync(StreamId streamId, [FromQuery] string templateId, [FromQuery] string step, [FromQuery] CommitId? change = null, [FromQuery] int count = 10, [FromQuery] PropertyFilter? filter = null)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, streamId);
			}

			TemplateId templateIdValue = new TemplateId(templateId);

			CommitIdWithOrder? commitIdWithOrder = null;
			if (change != null)
			{
				commitIdWithOrder = await _commitService.GetOrderedAsync(streamId, change, HttpContext.RequestAborted);
			}

			List<IJobStepRef> steps = await _jobStepRefCollection.GetStepsForNodeAsync(streamId, templateIdValue, step, commitIdWithOrder, true, count, false, HttpContext.RequestAborted);
			return steps.ConvertAll(x => PropertyFilter.Apply(CreateGetJobStepRefResponse(x), filter));
		}

		private const int MaxConcurrentStepHistoryRequests = 32;

		/// <summary>
		/// Gets the history of a set of steps given a stream and template context.
		/// </summary>
		/// <param name="streamId">The owning stream's id.</param>
		/// <param name="templateId">The owning template's id.</param>
		/// <param name="steps">The steps to obtain the history of.</param>
		/// <param name="change">Maximum changelist number to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">The filter to apply to the results</param>
		/// <param name="cancellationToken">The cancellation token to use for the job.</param>
		/// <returns>Http result code along with the corresponding filtered list of <see cref="GetJobStepRefResponse"/>.</returns>
		/// <remarks>This will return all states.</remarks>
		[HttpPost]
		[Route("/api/v1/streams/{streamId}/history")]
		[ProducesResponseType(typeof(List<GetJobStepRefResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetStepHistoryAsync(StreamId streamId, [FromQuery] string templateId, [FromQuery] string[] steps, [FromQuery] CommitId? change = null, [FromQuery] int count = 10, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}
			if (!streamConfig.Authorize(JobAclAction.ViewJob, User))
			{
				return Forbid(JobAclAction.ViewJob, streamId);
			}
			if (steps == null)
			{
				return BadRequest("Null step argument provided.");
			}

			TemplateId templateIdValue = new TemplateId(templateId);

			CommitIdWithOrder? commitIdWithOrder = null;
			if (change != null)
			{
				commitIdWithOrder = await _commitService.GetOrderedAsync(streamId, change, cancellationToken);
			}

			List<IJobStepRef> jobStepRefs = [];

			// create a batch
			using SemaphoreSlim maxStepHistoryActiveRequests = new SemaphoreSlim(MaxConcurrentStepHistoryRequests);
			Task<List<IJobStepRef>?>[] jobStepRefCollectionTasks = [.. steps.Select(async step =>
				{
					await maxStepHistoryActiveRequests.WaitAsync(cancellationToken);
					try
					{
						return await _jobStepRefCollection.GetStepsForNodeAsync(streamId, templateIdValue, step, commitIdWithOrder, true, count, true, cancellationToken);
					}
					catch (OperationCanceledException)
					{
						throw;
					}
					catch (Exception)
					{
						// Swallow all error exceptions; the user will obtain no result items.
						return null;
					}
					finally
					{
						maxStepHistoryActiveRequests.Release();
					}
				})];

			List<IJobStepRef>?[] jobStepRefCollectionTasksResults = await Task.WhenAll(jobStepRefCollectionTasks);

			jobStepRefs = jobStepRefCollectionTasksResults.Where(kv => kv != null).SelectMany(kv => kv!).ToList();

			List<object> filteredList = jobStepRefs.Select(y => PropertyFilter.Apply(CreateGetJobStepRefResponse(y), filter)).ToList();

			return filteredList;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="jobStepRef">The jobstep ref to construct from</param>
		internal static GetJobStepRefResponse CreateGetJobStepRefResponse(IJobStepRef jobStepRef)
		{
			GetJobStepRefResponse response = new GetJobStepRefResponse();
			response.StepName = jobStepRef.NodeName;
			response.JobId = jobStepRef.Id.JobId;
			response.BatchId = jobStepRef.Id.BatchId;
			response.StepId = jobStepRef.Id.StepId;
			response.CommitId = jobStepRef.CommitId;
			response.LogId = jobStepRef.LogId.ToString();
			response.PoolId = jobStepRef.PoolId?.ToString();
			response.AgentId = jobStepRef.AgentId?.ToString();
			response.State = jobStepRef.State;
			response.Outcome = jobStepRef.Outcome;
			response.IssueIds = jobStepRef.IssueIds?.Select(id => id).ToList();
			response.JobStartTime = jobStepRef.JobStartTimeUtc;
			response.StartTime = jobStepRef.StartTimeUtc;
			response.FinishTime = jobStepRef.FinishTimeUtc;
			response.Metadata = jobStepRef.Metadata?.ToList();
			return response;
		}

		/// <summary>
		/// Gets a template for a stream
		/// </summary>
		/// <param name="streamId">Unique id of the stream to query</param>
		/// <param name="templateId">Unique id of the template to query</param>
		/// <param name="filter">Filter for properties to return</param>
		/// <returns>Information about all the templates</returns>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}/templates/{templateId}")]
		[ProducesResponseType(typeof(List<GetTemplateResponse>), 200)]
		public async Task<ActionResult<object>> GetTemplateAsync(StreamId streamId, TemplateId templateId, [FromQuery] PropertyFilter? filter = null)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}

			TemplateRefConfig? templateConfig;
			if (!streamConfig.TryGetTemplate(templateId, out templateConfig))
			{
				return NotFound(streamId, templateId);
			}
			if (!templateConfig.Authorize(StreamAclAction.ViewTemplate, User))
			{
				return Forbid(StreamAclAction.ViewTemplate, streamId);
			}

			ITemplate template = await _templateCollection.GetOrAddAsync(templateConfig);
			return new GetTemplateResponse(template).ApplyFilter(filter);
		}

		/// <summary>
		/// Retrieve historical information about a specific schedule
		/// </summary>
		/// <param name="streamId">Unique id of the stream to query</param>
		/// <param name="templateId">Unique id of the template to query</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/streams/{streamId}/templates/{templateId}/history")]
		public async Task<ActionResult> GetTemplateHistoryAsync(StreamId streamId, TemplateId templateId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50)
		{
			StreamConfig? streamConfig;
			if (!_buildConfig.Value.TryGetStream(streamId, out streamConfig))
			{
				return NotFound(streamId);
			}

			TemplateRefConfig? templateConfig;
			if (!streamConfig.TryGetTemplate(templateId, out templateConfig))
			{
				return NotFound(streamId, templateId);
			}
			if (!templateConfig.Authorize(StreamAclAction.ViewTemplate, User))
			{
				return Forbid(StreamAclAction.ViewTemplate, streamId);
			}

			IAuditLogChannel channel = _scheduleService.GetAuditLog(streamId, templateId);

			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync();
			await channel.FindAsync(HttpContext.Response.BodyWriter, minTime, maxTime, index, count);
			await Response.CompleteAsync();

			return Empty;
		}

		/// <summary>
		/// Update a stream template ref
		/// </summary>
		[HttpPut]
		[Authorize]
		[Route("/api/v1/streams/{streamId}/templates/{templateRefId}")]
		public async Task<ActionResult> UpdateStreamTemplateRefAsync(StreamId streamId, TemplateId templateRefId, [FromBody] UpdateTemplateRefRequest update)
		{
			if (!_buildConfig.Value.Authorize(AdminAclAction.AdminWrite, User))
			{
				return Forbid(AdminAclAction.AdminWrite);
			}

			IStream? stream = await _streamCollection.GetAsync(streamId);
			if (stream == null)
			{
				return NotFound(streamId);
			}
			if (!stream.Templates.ContainsKey(templateRefId))
			{
				return NotFound(streamId, templateRefId);
			}

			await stream.TryUpdateTemplateRefAsync(templateRefId, update.StepStates);
			return Ok();
		}
	}
}
