// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.VersionControl.Perforce;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.Tasks
{
	/// <summary>
	/// Generates tasks telling agents to sync their workspaces
	/// </summary>
	public sealed class ConformTaskSource : TaskSourceBase<ConformTask>, IHostedService, IAsyncDisposable
	{
		/// <inheritdoc/>
		public override string Type => "Conform";

		/// <inheritdoc/>
		public override TaskSourceFlags Flags => TaskSourceFlags.AllowWhenDisabled;

		readonly IAgentCollection _agentCollection;
		readonly PoolService _poolService;
		readonly SingletonDocument<ConformList> _conformList;
		readonly PerforceLoadBalancer _perforceLoadBalancer;
		readonly ILogCollection _logCollection;
		readonly ILogger _logger;
		readonly IOptionsMonitor<BuildServerConfig> _settings;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITicker _tickConformList;

		/// <summary>
		/// Constructor
		/// </summary>
		public ConformTaskSource(IMongoService mongoService, IAgentCollection agentCollection, PoolService poolService, ILogCollection logCollection, PerforceLoadBalancer perforceLoadBalancer, IClock clock, IOptionsMonitor<BuildServerConfig> settings, IOptionsMonitor<BuildConfig> buildConfig, ILogger<ConformTaskSource> logger)
		{
			_agentCollection = agentCollection;
			_poolService = poolService;
			_conformList = new SingletonDocument<ConformList>(mongoService);
			_perforceLoadBalancer = perforceLoadBalancer;
			_logCollection = logCollection;
			_settings = settings;
			_buildConfig = buildConfig;
			_logger = logger;
			_tickConformList = clock.AddSharedTicker<ConformTaskSource>(TimeSpan.FromMinutes(1.0), CleanConformListAsync, logger);

			OnLeaseStartedProperties.Add(nameof(ConformTask.LogId), x => LogId.Parse(x.LogId));
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _tickConformList.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _tickConformList.StopAsync();

		/// <inheritdoc/>
		public async ValueTask DisposeAsync() => await _tickConformList.DisposeAsync();

		/// <summary>
		/// Clean up the conform list of any outdated entries
		/// </summary>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>Async task</returns>
		async ValueTask CleanConformListAsync(CancellationToken cancellationToken)
		{
			DateTime utcNow = DateTime.UtcNow;
			DateTime lastCheckTimeUtc = utcNow - TimeSpan.FromMinutes(30.0);

			// Get the current state of the conform list
			ConformList list = await _conformList.GetAsync(cancellationToken);

			// Update any leases that are older than LastCheckTimeUtc
			Dictionary<LeaseId, bool> removeLeases = new Dictionary<LeaseId, bool>();
			foreach (ConformListEntry entry in Enumerable.Concat(list.Entries, list.Servers.SelectMany(x => x.Entries)))
			{
				if (entry.LastCheckTimeUtc < lastCheckTimeUtc)
				{
					IAgent? agent = await _agentCollection.GetAsync(entry.AgentId, cancellationToken);

					bool remove = false;
					if (agent == null || !agent.Leases.Any(x => x.Id == entry.LeaseId))
					{
						_logger.LogWarning("Removing invalid lease from conform list: {AgentId} lease {LeaseId}", entry.AgentId, entry.LeaseId);
						remove = true;
					}

					removeLeases[entry.LeaseId] = remove;
				}
			}

			// If there's anything to change, update the list
			if (removeLeases.Count > 0)
			{
				await _conformList.UpdateAsync(list => UpdateConformList(list, utcNow, removeLeases), cancellationToken);
			}
		}

		/// <summary>
		/// Remove items from the conform list, and update timestamps for them
		/// </summary>
		/// <param name="list">The list to update</param>
		/// <param name="utcNow">Current time</param>
		/// <param name="removeLeases">List of leases to update. Entries with values set to true will be removed, entries with values set to false will have their timestamp updated.</param>
		static void UpdateConformList(ConformList list, DateTime utcNow, Dictionary<LeaseId, bool> removeLeases)
		{
			UpdateConformList(list.Entries, utcNow, removeLeases);
			foreach (ConformListServer server in list.Servers)
			{
				UpdateConformList(server.Entries, utcNow, removeLeases);
			}
		}

		/// <summary>
		/// Remove items from the conform list, and update timestamps for them
		/// </summary>
		/// <param name="entries">The list to update</param>
		/// <param name="utcNow">Current time</param>
		/// <param name="removeLeases">List of leases to update. Entries with values set to true will be removed, entries with values set to false will have their timestamp updated.</param>
		static void UpdateConformList(List<ConformListEntry> entries, DateTime utcNow, Dictionary<LeaseId, bool> removeLeases)
		{
			for (int idx = 0; idx < entries.Count; idx++)
			{
				bool remove;
				if (removeLeases.TryGetValue(entries[idx].LeaseId, out remove))
				{
					if (remove)
					{
						entries.RemoveAt(idx--);
					}
					else if (entries[idx].LastCheckTimeUtc < utcNow)
					{
						entries[idx].LastCheckTimeUtc = utcNow;
					}
				}
			}
		}

		/// <inheritdoc/>
		public override async Task<Task<CreateLeaseOptions?>> AssignLeaseAsync(IAgent agent, CancellationToken cancellationToken)
		{
			if (!_settings.CurrentValue.EnableConformTasks || !_buildConfig.CurrentValue.EnableConformTasks || agent.Mode == AgentMode.Workstation)
			{
				return SkipAsync(cancellationToken);
			}

			DateTime utcNow = DateTime.UtcNow;
			if (!await IsConformPendingAsync(agent, utcNow, cancellationToken))
			{
				return SkipAsync(cancellationToken);
			}

			if (agent.Leases.Count == 0)
			{
				ConformTask task = new ConformTask();
				if (await GetWorkspacesAsync(agent, task.Workspaces, cancellationToken))
				{
					LeaseId leaseId = new LeaseId(BinaryIdUtils.CreateNew());
					if (await AllocateConformLeaseAsync(agent.Id, task.Workspaces, leaseId, cancellationToken))
					{
						try
						{
							ILog log = await _logCollection.AddAsync(JobId.Empty, leaseId, agent.SessionId, LogType.Json, cancellationToken: cancellationToken);
							task.LogId = log.Id.ToString();
							task.RemoveUntrackedFiles = agent.RequestFullConform;

							return LeaseAsync(new CreateLeaseOptions(leaseId, null, "Updating workspaces", null, null, log.Id, null, true, task));
						}
						catch
						{
							await ReleaseConformLeaseAsync(leaseId, cancellationToken);
							throw;
						}
					}
				}
			}

			if (agent.RequestConform || agent.RequestFullConform)
			{
				return await DrainAsync(cancellationToken);
			}
			else
			{
				return SkipAsync(cancellationToken);
			}
		}

		/// <inheritdoc/>
		public override Task CancelLeaseAsync(IAgent agent, LeaseId leaseId, ConformTask payload, CancellationToken cancellationToken)
		{
			return ReleaseConformLeaseAsync(leaseId, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<bool> GetWorkspacesAsync(IAgent agent, IList<RpcAgentWorkspace> workspaces, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			HashSet<AgentWorkspaceInfo> conformWorkspaces = await _poolService.GetWorkspacesAsync(agent, DateTime.UtcNow, buildConfig, cancellationToken);
			foreach (AgentWorkspaceInfo conformWorkspace in conformWorkspaces)
			{
				PerforceCluster? cluster = buildConfig.FindPerforceCluster(conformWorkspace.Cluster);
				if (cluster == null || !await agent.TryAddWorkspaceMessageAsync(conformWorkspace, cluster, _perforceLoadBalancer, workspaces, cancellationToken))
				{
					return false;
				}
			}

			return true;
		}

		/// <inheritdoc/>
		public override Task OnLeaseFinishedAsync(IAgent agent, LeaseId leaseId, ConformTask payload, LeaseOutcome outcome, ReadOnlyMemory<byte> output, ILogger logger, CancellationToken cancellationToken)
		{
			return ReleaseConformLeaseAsync(leaseId, cancellationToken);
		}

		/// <summary>
		/// Atempt to allocate a conform resource for the given lease
		/// </summary>
		/// <param name="agentId">The agent id</param>
		/// <param name="workspaces">List of workspaces that are required</param>
		/// <param name="leaseId">The lease id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the resource was allocated, false otherwise</returns>
		private async Task<bool> AllocateConformLeaseAsync(AgentId agentId, IEnumerable<RpcAgentWorkspace> workspaces, LeaseId leaseId, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;
			for (; ; )
			{
				ConformList currentValue = await _conformList.GetAsync(cancellationToken);
				if (buildConfig.MaxConformCount != 0 && currentValue.Entries.Count + currentValue.Servers.Sum(x => x.Entries.Count) >= buildConfig.MaxConformCount)
				{
					return false;
				}

				HashSet<string> servers = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
				foreach (RpcAgentWorkspace workspace in workspaces)
				{
					if (servers.Add(workspace.ServerAndPort))
					{
						PerforceCluster? cluster = buildConfig.FindPerforceCluster(workspace.Cluster);
						if (cluster == null)
						{
							_logger.LogWarning("Unable to find perforce cluster '{Cluster}' for conform", workspace.Cluster);
							return false;
						}

						PerforceServer? server = cluster.Servers.FirstOrDefault(x => x.ServerAndPort.Equals(workspace.BaseServerAndPort, StringComparison.Ordinal));
						if (server == null)
						{
							_logger.LogWarning("Unable to find perforce server '{Server}' in cluster '{Cluster}' for conform", workspace.BaseServerAndPort, workspace.Cluster);
							return false;
						}

						ConformListServer? conformServer = currentValue.Servers.FirstOrDefault(x => x.Cluster.Equals(workspace.Cluster, StringComparison.Ordinal) && x.ServerAndPort.Equals(workspace.ServerAndPort, StringComparison.Ordinal));
						if (conformServer == null)
						{
							conformServer = new ConformListServer { Cluster = workspace.Cluster, ServerAndPort = workspace.ServerAndPort };
							currentValue.Servers.Add(conformServer);
						}

						if (server.MaxConformCount != 0 && conformServer.Entries.Count > server.MaxConformCount)
						{
							return false;
						}

						ConformListEntry entry = new ConformListEntry();
						entry.AgentId = agentId;
						entry.LeaseId = leaseId;
						entry.LastCheckTimeUtc = DateTime.UtcNow;
						conformServer.Entries.Add(entry);
					}
				}

				if (await _conformList.TryUpdateAsync(currentValue, cancellationToken))
				{
					_logger.LogInformation("Added conform lease {LeaseId}", leaseId);
					return true;
				}
			}
		}

		/// <summary>
		/// Terminate a conform lease
		/// </summary>
		/// <param name="leaseId">The lease id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public async Task ReleaseConformLeaseAsync(LeaseId leaseId, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				ConformList currentValue = await _conformList.GetAsync(cancellationToken);
				if (currentValue.Entries.RemoveAll(x => x.LeaseId == leaseId) + currentValue.Servers.Sum(x => x.Entries.RemoveAll(x => x.LeaseId == leaseId)) == 0)
				{
					_logger.LogInformation("Conform lease {LeaseId} is not in singelton", leaseId);
					break;
				}
				if (await _conformList.TryUpdateAsync(currentValue, cancellationToken))
				{
					_logger.LogInformation("Removed conform lease {LeaseId}", leaseId);
					break;
				}
			}
		}

		/// <summary>
		/// Determine if an agent should be conformed
		/// </summary>
		/// <param name="agent">The agent to test</param>
		/// <param name="utcNow">Current time</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the agent should be conformed</returns>
		private async Task<bool> IsConformPendingAsync(IAgent agent, DateTime utcNow, CancellationToken cancellationToken)
		{
			BuildConfig buildConfig = _buildConfig.CurrentValue;

			// If a conform was manually requested, allow it to run even if the agent is disabled
			if (agent.RequestConform || agent.RequestFullConform)
			{
				return !IsConformCoolDownPeriod(agent, utcNow);
			}

			// Otherwise only run if the agent is enabled
			if (agent.Enabled)
			{
				// If we've attempted (and failed) a conform, run again after a certain time
				if (agent.ConformAttemptCount.HasValue)
				{
					return !IsConformCoolDownPeriod(agent, utcNow);
				}

				// Get the current pools for the agent
				List<IPoolConfig> pools = await _poolService.GetPoolsAsync(agent, DateTime.UtcNow - TimeSpan.FromMinutes(2.0), cancellationToken);

				TimeSpan? conformInterval = null;
				foreach (IPoolConfig pool in pools)
				{
					TimeSpan interval = pool.ConformInterval ?? TimeSpan.FromDays(1.0);
					if (interval > TimeSpan.Zero && (conformInterval == null || interval < conformInterval.Value))
					{
						conformInterval = interval;
					}
				}

				// If there is no conform interval, early out
				if (conformInterval == null)
				{
					return false;
				}

				// Always run a conform every 24h
				if (utcNow > agent.LastConformTime + conformInterval.Value)
				{
					return true;
				}

				// Check if the workspaces have changed (first check against a cached list of workspaces, then an accurate one)
				HashSet<AgentWorkspaceInfo> workspaces = await _poolService.GetWorkspacesAsync(agent, utcNow - TimeSpan.FromSeconds(30.0), buildConfig, cancellationToken);
				if (!workspaces.SetEquals(agent.Workspaces))
				{
					workspaces = await _poolService.GetWorkspacesAsync(agent, utcNow, buildConfig, cancellationToken);
					if (!workspaces.SetEquals(agent.Workspaces))
					{
						return true;
					}
				}
			}
			return false;
		}

		/// <summary>
		/// Determines if the given agent is in a cooldown period, waiting to retry after a failed conform
		/// </summary>
		/// <param name="agent">The agent to test</param>
		/// <param name="utcNow">Current time</param>
		/// <returns></returns>
		private static bool IsConformCoolDownPeriod(IAgent agent, DateTime utcNow)
		{
			if (!agent.ConformAttemptCount.HasValue)
			{
				return false;
			}

			TimeSpan retryTime;
			switch (agent.ConformAttemptCount.Value)
			{
				case 1:
					retryTime = TimeSpan.FromMinutes(5.0);
					break;
				case 2:
					retryTime = TimeSpan.FromMinutes(20.0);
					break;
				case 3:
					retryTime = TimeSpan.FromHours(1.0);
					break;
				default:
					retryTime = TimeSpan.FromHours(6.0);
					break;
			}
			return utcNow < agent.LastConformTime + retryTime;
		}
	}
}
