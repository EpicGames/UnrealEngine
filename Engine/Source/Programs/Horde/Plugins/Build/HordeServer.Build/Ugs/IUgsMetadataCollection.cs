// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Ugs;

namespace HordeServer.Ugs
{
	/// <summary>
	/// Collection of stream documents
	/// </summary>
	public interface IUgsMetadataCollection
	{
		/// <summary>
		/// Finds or adds metadata for a given changelist
		/// </summary>
		/// <param name="stream">The stream containing the change</param>
		/// <param name="change">The changelist number</param>
		/// <param name="project">Project identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The metadata instance</returns>
		Task<IUgsMetadata> FindOrAddAsync(string stream, int change, string? project, CancellationToken cancellationToken = default);

		/// <summary>
		/// Adds information to a change
		/// </summary>
		/// <param name="metadata">The existing metadata</param>
		/// <param name="userName">Name of the user posting the change</param>
		/// <param name="synced">Time that the change was synced</param>
		/// <param name="vote">Vote for this change</param>
		/// <param name="investigating">Whether the user is investigating the change</param>
		/// <param name="starred">Whether the change is starred</param>
		/// <param name="comment">New comment from this user</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task task</returns>
		Task<IUgsMetadata> UpdateUserAsync(IUgsMetadata metadata, string userName, bool? synced, UgsUserVote? vote, bool? investigating, bool? starred, string? comment, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates the state of a badge
		/// </summary>
		/// <param name="metadata">The existing metadata</param>
		/// <param name="name">Name of the badge</param>
		/// <param name="url">Url to link to for the badge</param>
		/// <param name="state">State of the badge</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task<IUgsMetadata> UpdateBadgeAsync(IUgsMetadata metadata, string name, Uri? url, UgsBadgeState state, CancellationToken cancellationToken = default);

		/// <summary>
		/// Searches for metadata updates
		/// </summary>
		/// <param name="stream">The stream to search</param>
		/// <param name="changes">List of changelist numbers to retrieve</param>
		/// <param name="minChange">Minimum changelist number</param>
		/// <param name="maxChange">Maximum changelist number</param>
		/// <param name="afterTicks">Last query time</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of metadata updates</returns>
		Task<List<IUgsMetadata>> FindAsync(string stream, IReadOnlyList<int>? changes, int? minChange, int? maxChange = null, long? afterTicks = null, CancellationToken cancellationToken = default);
	}
}
