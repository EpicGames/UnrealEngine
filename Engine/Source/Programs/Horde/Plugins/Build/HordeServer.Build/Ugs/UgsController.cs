// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Issues;
using EpicGames.Horde.Ugs;
using EpicGames.Horde.Users;
using HordeServer.Issues;
using HordeServer.Logs;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;

namespace HordeServer.Ugs
{
	/// <summary>
	/// Controller for the /api/v1/issues endpoint
	/// </summary>
	[ApiController]
	[Route("[controller]")]
	public sealed class UgsController : ControllerBase
	{
		readonly IssueService _issueService;
		readonly IUgsMetadataCollection _ugsMetadataCollection;
		readonly IUserCollection _userCollection;
		readonly ILogCollection _logCollection;
		readonly IServerInfo _serverInfo;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UgsController(IssueService issueService, IUgsMetadataCollection ugsMetadataCollection, IUserCollection userCollection, ILogCollection logCollection, IServerInfo serverInfo, ILogger<UgsController> logger)
		{
			_issueService = issueService;
			_ugsMetadataCollection = ugsMetadataCollection;
			_userCollection = userCollection;
			_logCollection = logCollection;
			_serverInfo = serverInfo;
			_logger = logger;
		}

		/// <summary>
		/// Gets the latest version info
		/// </summary>
		/// <returns>Result code</returns>
		[HttpGet]
		[Route("/ugs/api/latest")]
		public ActionResult<object> GetLatest()
		{
			_ = this;
			return new { Version = 2 };
		}

		/// <summary>
		/// Adds new metadata to the database
		/// </summary>
		/// <param name="request">Request object</param>
		/// <returns>Result code</returns>
		[HttpPost]
		[Route("/ugs/api/metadata")]
		public async Task<ActionResult> AddMetadataAsync(AddUgsMetadataRequest request)
		{
			IUgsMetadata metadata = await _ugsMetadataCollection.FindOrAddAsync(request.Stream, request.Change, request.Project);
			if (request.Synced != null || request.Vote != null || request.Investigating != null || request.Starred != null || request.Comment != null)
			{
				if (request.UserName == null)
				{
					return BadRequest("Missing UserName field on request body");
				}
				metadata = await _ugsMetadataCollection.UpdateUserAsync(metadata, request.UserName, request.Synced, request.Vote, request.Investigating, request.Starred, request.Comment);
			}
			if (request.Badges != null)
			{
				foreach (AddUgsBadgeRequest badge in request.Badges)
				{
					if (!String.IsNullOrEmpty(badge.Name))
					{
						metadata = await _ugsMetadataCollection.UpdateBadgeAsync(metadata, badge.Name, badge.Url, badge.State);
					}
				}
			}
			return Ok();
		}

		/// <summary>
		/// Searches for metadata updates
		/// </summary>
		/// <param name="stream">THe stream to search for</param>
		/// <param name="changes">List of specific changes</param>
		/// <param name="minChange">Minimum changelist number</param>
		/// <param name="maxChange">Maximum changelist number</param>
		/// <param name="projects">The project identifiers to search for</param>
		/// <param name="sequence">Last sequence number</param>
		/// <returns>List of metadata updates</returns>
		[HttpGet]
		[Route("/ugs/api/metadata")]
		public async Task<GetUgsMetadataListResponse> FindMetadataAsync([FromQuery] string stream, [FromQuery(Name = "change")] List<int>? changes = null, [FromQuery] int? minChange = null, [FromQuery] int? maxChange = null, [FromQuery(Name = "project")] List<string>? projects = null, [FromQuery] long? sequence = null)
		{
			List<IUgsMetadata> metadataList = await _ugsMetadataCollection.FindAsync(stream, changes, minChange, maxChange, sequence);

			GetUgsMetadataListResponse response = new GetUgsMetadataListResponse();
			if (sequence != null)
			{
				response.SequenceNumber = sequence.Value;
			}

			HashSet<string>? projectSet = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (projects != null)
			{
				projectSet.UnionWith(projects);
			}

			foreach (IUgsMetadata metadata in metadataList)
			{
				if (metadata.UpdateTicks > response.SequenceNumber)
				{
					response.SequenceNumber = metadata.UpdateTicks;
				}
				if (String.IsNullOrEmpty(metadata.Project) || projectSet.Contains(metadata.Project))
				{
					response.Items.Add(CreateGetUgsMetadataResponse(metadata));
				}
			}
			return response;
		}

		static GetUgsMetadataResponse CreateGetUgsMetadataResponse(IUgsMetadata metadata)
		{
			return new GetUgsMetadataResponse
			{
				Change = metadata.Change,
				Project = metadata.Project,
				Users = metadata.Users?.ConvertAll(x => CreateGetUgsUserResponse(x)),
				Badges = metadata.Badges?.ConvertAll(x => CreateGetUgsBadgeResponse(x))
			};
		}

		static GetUgsUserResponse CreateGetUgsUserResponse(IUgsUserData userData)
		{
			return new GetUgsUserResponse
			{
				User = userData.User,
				SyncTime = userData.SyncTime,
				Vote = (userData.Vote == UgsUserVote.None) ? (UgsUserVote?)null : userData.Vote,
				Comment = userData.Comment,
				Investigating = userData.Investigating,
				Starred = userData.Starred
			};
		}

		static GetUgsBadgeResponse CreateGetUgsBadgeResponse(IUgsBadgeData badgeData)
		{
			return new GetUgsBadgeResponse
			{
				Name = badgeData.Name,
				Url = badgeData.Url,
				State = badgeData.State
			};
		}

		/// <summary>
		/// Retrieve information about open issues
		/// </summary>
		/// <param name="user"></param>
		/// <param name="includeResolved">Whether to include resolved issues</param>
		/// <param name="maxResults">Maximum number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/ugs/api/issues")]
		[ProducesResponseType(typeof(GetUgsIssueResponse), 200)]
		public async Task<ActionResult<List<GetUgsIssueResponse>>> GetIssuesAsync([FromQuery] string? user = null, [FromQuery] bool includeResolved = false, [FromQuery] int maxResults = 100, CancellationToken cancellationToken = default)
		{
			IUser? userInfo = (user != null) ? await _userCollection.FindUserByLoginAsync(user, cancellationToken) : null;

			List<GetUgsIssueResponse> responses = new List<GetUgsIssueResponse>();
			if (includeResolved)
			{
				IReadOnlyList<IIssue> issues = await _issueService.Collection.FindIssuesAsync(null, resolved: null, count: maxResults, cancellationToken: cancellationToken);
				foreach (IIssue issue in issues)
				{
					IIssueDetails details = await _issueService.GetIssueDetailsAsync(issue, cancellationToken);
					bool notify = userInfo != null && details.Suspects.Any(x => x.AuthorId == userInfo.Id);
					responses.Add(await CreateIssueResponseAsync(details, notify, cancellationToken));
				}
			}
			else
			{
				foreach (IIssueDetails cachedOpenIssue in _issueService.CachedOpenIssues)
				{
					if (responses.Count >= maxResults)
					{
						break;
					}

					if (cachedOpenIssue.ShowNotifications())
					{
						bool notify = userInfo != null && cachedOpenIssue.IncludeForUser(userInfo.Id);
						responses.Add(await CreateIssueResponseAsync(cachedOpenIssue, notify, cancellationToken));
					}
				}
			}
			return responses;
		}

		/// <summary>
		/// Retrieve information about a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/ugs/api/issues/{issueId}")]
		[ProducesResponseType(typeof(GetUgsIssueBuildResponse), 200)]
		public async Task<ActionResult<object>> GetIssueAsync(int issueId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IIssueDetails? issue = await _issueService.GetIssueDetailsAsync(issueId, cancellationToken);
			if (issue == null)
			{
				return NotFound();
			}

			return PropertyFilter.Apply(await CreateIssueResponseAsync(issue, false, cancellationToken), filter);
		}

		/// <summary>
		/// Retrieve information about builds for a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/ugs/api/issues/{issueId}/builds")]
		[ProducesResponseType(typeof(List<GetUgsIssueBuildResponse>), 200)]
		public async Task<ActionResult<List<object>>> GetIssueBuildsAsync(int issueId, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			IIssueDetails? issue = await _issueService.GetCachedIssueDetailsAsync(issueId, cancellationToken);
			if (issue == null)
			{
				return NotFound();
			}

			List<GetUgsIssueBuildResponse> responses = new List<GetUgsIssueBuildResponse>();
			foreach (IIssueSpan span in issue.Spans)
			{
				if (span.LastSuccess != null)
				{
					responses.Add(CreateBuildResponse(span, span.LastSuccess, IssueBuildOutcome.Success));
				}
				foreach (IIssueStep step in issue.Steps)
				{
					responses.Add(CreateBuildResponse(span, step, IssueBuildOutcome.Error));
				}
				if (span.NextSuccess != null)
				{
					responses.Add(CreateBuildResponse(span, span.NextSuccess, IssueBuildOutcome.Success));
				}
			}

			return responses.ConvertAll(x => PropertyFilter.Apply(x, filter));
		}

		/// <summary>
		/// Retrieve information about builds for a specific issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="cancellationToken">Cancellation token for the request</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/ugs/api/issues/{issueId}/diagnostics")]
		[ProducesResponseType(typeof(List<GetUgsIssueDiagnosticResponse>), 200)]
		public async Task<ActionResult<List<GetUgsIssueDiagnosticResponse>>> GetIssueDiagnosticsAsync(int issueId, CancellationToken cancellationToken)
		{
			List<GetUgsIssueDiagnosticResponse> diagnostics = new List<GetUgsIssueDiagnosticResponse>();

			IReadOnlyList<IIssueSpan> spans = await _issueService.Collection.FindSpansAsync(issueId, cancellationToken);
			IReadOnlyList<ILogAnchor> events = await _logCollection.FindAnchorsForSpansAsync(spans.Select(x => x.Id), null, 0, count: 10, cancellationToken);

			foreach (ILogAnchor logEvent in events)
			{
				ILogEventData eventData = await logEvent.GetDataAsync(cancellationToken);
				long buildId = logEvent.LogId.GetHashCode();
				Uri url = new Uri(_serverInfo.DashboardUrl, $"log/{logEvent.LogId}?lineindex={logEvent.LineIndex}");

				GetUgsIssueDiagnosticResponse diagnostic = new GetUgsIssueDiagnosticResponse(buildId, eventData.Message, url);
				diagnostics.Add(diagnostic);
			}

			return diagnostics;
		}

		/// <summary>
		/// Gets the URL for a failing step in the
		/// </summary>
		/// <param name="details">The issue to get a URL for</param>
		/// <param name="notify">Whether to show notifications for this issue</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The issue response</returns>
		async Task<GetUgsIssueResponse> CreateIssueResponseAsync(IIssueDetails details, bool notify, CancellationToken cancellationToken)
		{
			Uri? buildUrl = GetIssueBuildUrl(details);

			IUser? owner = details.Issue.OwnerId.HasValue ? await _userCollection.GetCachedUserAsync(details.Issue.OwnerId.Value, cancellationToken) : null;
			IUser? nominatedBy = details.Issue.NominatedById.HasValue ? await _userCollection.GetCachedUserAsync(details.Issue.NominatedById.Value, cancellationToken) : null;

			return CreateGetUgsIssueResponse(details, owner, nominatedBy, notify, buildUrl);
		}

		static GetUgsIssueResponse CreateGetUgsIssueResponse(IIssueDetails details, IUser? owner, IUser? nominatedBy, bool notify, Uri? buildUrl)
		{
			IIssue issue = details.Issue;

			return new GetUgsIssueResponse
			{
				Id = issue.Id,
				CreatedAt = issue.CreatedAt,
				RetrievedAt = DateTime.UtcNow,
				Summary = issue.Summary,
				Owner = owner?.Login,
				NominatedBy = nominatedBy?.Login,
				AcknowledgedAt = issue.AcknowledgedAt,
				FixChange = issue.FixCommitId?.GetPerforceChange(),
				ResolvedAt = issue.ResolvedAt,
				BNotify = notify,
				BWarning = issue.Severity == IssueSeverity.Warning,
				BuildUrl = buildUrl,
				Streams = details.Spans.Select(x => x.StreamName ?? x.StreamId.ToString()).Distinct().ToList()
			};
		}

		/// <summary>
		/// Gets the URL for a failing step in the given issue
		/// </summary>
		/// <param name="issue">The issue details</param>
		/// <returns>The build URL</returns>
		Uri? GetIssueBuildUrl(IIssueDetails issue)
		{
			HashSet<ObjectId> unresolvedSpans = new HashSet<ObjectId>(issue.Spans.Where(x => x.NextSuccess == null).Select(x => x.Id));

			IIssueStep? step = issue.Steps.OrderByDescending(x => unresolvedSpans.Contains(x.SpanId)).ThenByDescending(x => x.CommitId).FirstOrDefault();
			if (step == null)
			{
				return null;
			}

			return new Uri(_serverInfo.DashboardUrl, $"job/{step.JobId}?step={step.StepId}");
		}

		/// <summary>
		/// Creates the response for a particular build
		/// </summary>
		/// <param name="span">Span containing the step</param>
		/// <param name="step">The step to describe</param>
		/// <param name="outcome">Outcome of this step</param>
		/// <returns>Response object</returns>
		GetUgsIssueBuildResponse CreateBuildResponse(IIssueSpan span, IIssueStep step, IssueBuildOutcome outcome)
		{
			GetUgsIssueBuildResponse response = new GetUgsIssueBuildResponse(span.StreamName, step.CommitId.GetPerforceChange(), outcome);
			response.Id = step.LogId.GetHashCode();
			response.JobName = $"{step.JobName}: {span.NodeName}";
			response.JobUrl = new Uri(_serverInfo.DashboardUrl, $"job/{step.JobId}");
			response.JobStepName = span.NodeName;
			response.JobStepUrl = new Uri(_serverInfo.DashboardUrl, $"job/{step.JobId}?step={step.StepId}");
			response.ErrorUrl = response.JobStepUrl;
			return response;
		}

		/// <summary>
		/// Update an issue
		/// </summary>
		/// <param name="issueId">Id of the issue to get information about</param>
		/// <param name="request">The update information</param>
		/// <returns>List of matching agents</returns>
		[HttpPut]
		[Route("/ugs/api/issues/{issueId}")]
		public async Task<ActionResult> UpdateIssueAsync(int issueId, [FromBody] UpdateUgsIssueRequest request)
		{
			UserId? newOwnerId = null;
			if (!String.IsNullOrEmpty(request.Owner))
			{
				newOwnerId = (await _userCollection.FindOrAddUserByLoginAsync(request.Owner))?.Id;
			}

			UserId? newNominatedById = null;
			if (!String.IsNullOrEmpty(request.NominatedBy))
			{
				newNominatedById = (await _userCollection.FindOrAddUserByLoginAsync(request.NominatedBy))?.Id;
			}

			UserId? newDeclinedById = null;
			if (!String.IsNullOrEmpty(request.DeclinedBy))
			{
				newDeclinedById = (await _userCollection.FindOrAddUserByLoginAsync(request.DeclinedBy))?.Id;
			}

			UserId? newResolvedById = null;
			if (!String.IsNullOrEmpty(request.ResolvedBy))
			{
				newResolvedById = (await _userCollection.FindOrAddUserByLoginAsync(request.ResolvedBy))?.Id;
			}
			if (newResolvedById == null && request.Resolved.HasValue)
			{
				newResolvedById = request.Resolved.Value ? IIssue.ResolvedByUnknownId : UserId.Empty;
			}

			if (!await _issueService.UpdateIssueAsync(issueId, ownerId: newOwnerId, nominatedById: newNominatedById, acknowledged: request.Acknowledged, declinedById: newDeclinedById, fixCommitId: CommitId.FromPerforceChange(request.FixChange), resolvedById: newResolvedById, initiatedById: User.GetUserId()))
			{
				return NotFound();
			}
			return Ok();
		}

		/// <summary>
		/// Post information about net core installation. 
		/// </summary>
		[HttpPost]
		[Route("/ugs/api/netcore")]
		public ActionResult<object> PostNetCoreInfo(string? user = null, string? machine = null, bool netCore = false)
		{
			_logger.LogInformation("NetCore: User={User}, Machine={Machine}, NetCore={NetCore}", user, machine, netCore);
			return new { };
		}
	}
}
