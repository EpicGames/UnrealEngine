// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Ugs;

namespace HordeServer.Ugs
{
	/// <summary>
	/// Metadata for a particular user
	/// </summary>
	public interface IUgsUserData
	{
		/// <summary>
		/// The user name
		/// </summary>
		string User { get; }

		/// <summary>
		/// Time that this change was synced
		/// </summary>
		public long? SyncTime { get; }

		/// <summary>
		/// State of the user
		/// </summary>
		public UgsUserVote Vote { get; }

		/// <summary>
		/// Whether the change is starred
		/// </summary>
		bool? Starred { get; }

		/// <summary>
		/// Whether the change is under investigation
		/// </summary>
		bool? Investigating { get; }

		/// <summary>
		/// Comment text
		/// </summary>
		string? Comment { get; }
	}

	/// <summary>
	/// Metadata for a badge
	/// </summary>
	public interface IUgsBadgeData
	{
		/// <summary>
		/// The badge name
		/// </summary>
		string Name { get; }

		/// <summary>
		/// The URL to open when the badge is clicked
		/// </summary>
		Uri? Url { get; }

		/// <summary>
		/// State of the badge
		/// </summary>
		UgsBadgeState State { get; }
	}

	/// <summary>
	/// Metadata for a particular changelist and project
	/// </summary>
	public interface IUgsMetadata
	{
		/// <summary>
		/// The changelist number
		/// </summary>
		int Change { get; }

		/// <summary>
		/// Stream for the metadata
		/// </summary>
		string Stream { get; }

		/// <summary>
		/// The project identifier
		/// </summary>
		string Project { get; }

		/// <summary>
		/// List of user states
		/// </summary>
		IReadOnlyList<IUgsUserData>? Users { get; }

		/// <summary>
		/// List of badge states
		/// </summary>
		IReadOnlyList<IUgsBadgeData>? Badges { get; }

		/// <summary>
		/// Time that this object was last updated
		/// </summary>
		long UpdateTicks { get; }
	}
}
