// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Ugs;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;

namespace HordeServer.Ugs
{
	/// <summary>
	/// Concrete implementation of IUgsMetadataCollection
	/// </summary>
	class UgsMetadataCollection : IUgsMetadataCollection
	{
		class UgsUserDataDocument : IUgsUserData
		{
			public string User { get; set; }

			[BsonIgnoreIfNull]
			public long? SyncTime { get; set; }

			[BsonIgnoreIfNull]
			public UgsUserVote? Vote { get; set; }

			[BsonIgnoreIfNull]
			public bool? Starred { get; set; }

			[BsonIgnoreIfNull]
			public bool? Investigating { get; set; }

			[BsonIgnoreIfNull]
			public string? Comment { get; set; }

			string IUgsUserData.User => User ?? "Unknown";
			long? IUgsUserData.SyncTime => SyncTime;
			UgsUserVote IUgsUserData.Vote => Vote ?? UgsUserVote.None;

			public UgsUserDataDocument(string user)
			{
				User = user;
			}
		}

		class UgsBadgeDataDocument : IUgsBadgeData
		{
			public string Name { get; set; } = String.Empty;

			[BsonIgnoreIfNull]
			public Uri? Url { get; set; }

			public UgsBadgeState State { get; set; }
		}

		class UgsMetadataDocument : IUgsMetadata
		{
			public ObjectId Id { get; set; }
			public string Stream { get; set; }
			public int Change { get; set; }
			public string Project { get; set; } = String.Empty;
			public List<UgsUserDataDocument> Users { get; set; } = new List<UgsUserDataDocument>();
			public List<UgsBadgeDataDocument> Badges { get; set; } = new List<UgsBadgeDataDocument>();
			public int UpdateIndex { get; set; }
			public long UpdateTicks { get; set; }

			IReadOnlyList<IUgsUserData>? IUgsMetadata.Users => (Users.Count > 0) ? Users : null;
			IReadOnlyList<IUgsBadgeData>? IUgsMetadata.Badges => (Badges.Count > 0) ? Badges : null;

			[BsonConstructor]
			private UgsMetadataDocument()
			{
				Stream = String.Empty;
			}

			public UgsMetadataDocument(string stream, int change, string project)
			{
				Id = ObjectId.GenerateNewId();
				Stream = stream;
				Change = change;
				Project = project;
			}
		}

		readonly IMongoCollection<UgsMetadataDocument> _collection;
		readonly ILogger<UgsMetadataCollection> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public UgsMetadataCollection(IMongoService mongoService, ILogger<UgsMetadataCollection> logger)
		{
			List<MongoIndex<UgsMetadataDocument>> indexes = new List<MongoIndex<UgsMetadataDocument>>();
			indexes.Add(keys => keys.Ascending(x => x.Stream).Descending(x => x.Change).Ascending(x => x.Project), unique: true);
			indexes.Add(keys => keys.Ascending(x => x.Stream).Descending(x => x.Change).Descending(x => x.UpdateTicks));
			_collection = mongoService.GetCollection<UgsMetadataDocument>("UgsMetadata", indexes);
			_logger = logger;
		}

		/// <summary>
		/// Find or add a document for the given change
		/// </summary>
		/// <param name="stream">Stream containing the change</param>
		/// <param name="change">The changelist number to add a document for</param>
		/// <param name="project">Arbitrary identifier for this project</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The metadata document</returns>
		public async Task<IUgsMetadata> FindOrAddAsync(string stream, int change, string? project, CancellationToken cancellationToken)
		{
			string normalizedStream = GetNormalizedStream(stream);
			string normalizedProject = GetNormalizedProject(project);
			for (; ; )
			{
				// Find an existing document
				UgsMetadataDocument? existing = await _collection.Find(x => x.Stream == normalizedStream && x.Change == change && x.Project == normalizedProject).FirstOrDefaultAsync(cancellationToken);
				if (existing != null)
				{
					return existing;
				}

				// Try to insert a new document
				try
				{
					UgsMetadataDocument newDocument = new UgsMetadataDocument(normalizedStream, change, normalizedProject);
					await _collection.InsertOneAsync(newDocument, null, cancellationToken);
					return newDocument;
				}
				catch (MongoWriteException ex)
				{
					if (ex.WriteError.Category != ServerErrorCategory.DuplicateKey)
					{
						throw;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IUgsMetadata> UpdateUserAsync(IUgsMetadata metadata, string userName, bool? synced, UgsUserVote? vote, bool? investigating, bool? starred, string? comment, CancellationToken cancellationToken)
		{
			UpdateDefinitionBuilder<UgsMetadataDocument> updateBuilder = Builders<UgsMetadataDocument>.Update;
			for (; ; )
			{
				UgsMetadataDocument document = (UgsMetadataDocument)metadata;

				if (document.Users == null)
				{
					_logger.LogWarning("Empty users collection in UGS metadata: {Contents}", document.ToBsonDocument().ToJson());
					document.Users ??= new();
				}

				int userIdx = document.Users.FindIndex(x => x.User != null && x.User.Equals(userName, StringComparison.OrdinalIgnoreCase));
				if (userIdx == -1)
				{
					// Create a new user entry
					UgsUserDataDocument userData = new UgsUserDataDocument(userName);
					userData.SyncTime = (synced == true) ? (long?)DateTime.UtcNow.Ticks : null;
					userData.Vote = vote;
					userData.Investigating = (investigating == true) ? investigating : null;
					userData.Starred = (starred == true) ? starred : null;
					userData.Comment = comment;

					if (await TryUpdateAsync(document, updateBuilder.Push(x => x.Users, userData), cancellationToken))
					{
						document.Users.Add(userData);
						return metadata;
					}
				}
				else
				{
					// Update an existing entry
					UgsUserDataDocument userData = document.Users[userIdx];

					long? newSyncTime = synced.HasValue ? (synced.Value ? (long?)DateTime.UtcNow.Ticks : null) : userData.SyncTime;
					UgsUserVote? newVote = vote.HasValue ? ((vote.Value != UgsUserVote.None) ? vote : null) : userData.Vote;
					bool? newInvestigating = investigating ?? userData.Investigating;
					bool? newStarred = starred.HasValue ? (starred.Value ? starred : null) : userData.Starred;

					List<UpdateDefinition<UgsMetadataDocument>> updates = new List<UpdateDefinition<UgsMetadataDocument>>();
					if (newSyncTime != userData.SyncTime)
					{
						updates.Add(updateBuilder.Set(x => x.Users![userIdx].SyncTime, newSyncTime));
					}
					if (newVote != userData.Vote)
					{
						updates.Add(updateBuilder.Set(x => x.Users![userIdx].Vote, newVote));
					}
					if (newInvestigating != userData.Investigating)
					{
						updates.Add(updateBuilder.SetOrUnsetNull(x => x.Users![userIdx].Investigating, newInvestigating));
					}
					if (newStarred != userData.Starred)
					{
						updates.Add(updateBuilder.SetOrUnsetNull(x => x.Users![userIdx].Starred, newStarred));
					}
					if (comment != null)
					{
						updates.Add(updateBuilder.Set(x => x.Users![userIdx].Comment, comment));
					}

					if (updates.Count == 0 || await TryUpdateAsync(document, updateBuilder.Combine(updates), cancellationToken))
					{
						userData.SyncTime = newSyncTime;
						userData.Vote = vote;
						userData.Investigating = newInvestigating;
						userData.Starred = newStarred;
						userData.Comment = comment;
						return metadata;
					}
				}

				// Update the document and try again
				metadata = await FindOrAddAsync(metadata.Stream, metadata.Change, metadata.Project, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<IUgsMetadata> UpdateBadgeAsync(IUgsMetadata metadata, string name, Uri? url, UgsBadgeState state, CancellationToken cancellationToken)
		{
			UpdateDefinitionBuilder<UgsMetadataDocument> updateBuilder = Builders<UgsMetadataDocument>.Update;
			for (; ; )
			{
				UgsMetadataDocument document = (UgsMetadataDocument)metadata;

				// Update the document
				int badgeIdx = document.Badges.FindIndex(x => x.Name.Equals(name, StringComparison.OrdinalIgnoreCase));
				if (badgeIdx == -1)
				{
					// Create a new badge
					UgsBadgeDataDocument newBadge = new UgsBadgeDataDocument();
					newBadge.Name = name;
					newBadge.Url = url;
					newBadge.State = state;

					if (await TryUpdateAsync(document, updateBuilder.Push(x => x.Badges, newBadge), cancellationToken))
					{
						document.Badges.Add(newBadge);
						return metadata;
					}
				}
				else
				{
					// Update an existing badge
					List<UpdateDefinition<UgsMetadataDocument>> updates = new List<UpdateDefinition<UgsMetadataDocument>>();

					UgsBadgeDataDocument badgeData = document.Badges[badgeIdx];
					if (url != badgeData.Url)
					{
						updates.Add(updateBuilder.Set(x => x.Badges![badgeIdx].Url, url));
					}
					if (state != badgeData.State)
					{
						updates.Add(updateBuilder.Set(x => x.Badges![badgeIdx].State, state));
					}

					if (updates.Count == 0 || await TryUpdateAsync(document, updateBuilder.Combine(updates), cancellationToken))
					{
						badgeData.Url = url;
						badgeData.State = state;
						return metadata;
					}
				}

				// Update the document and try again
				metadata = await FindOrAddAsync(metadata.Stream, metadata.Change, metadata.Project, cancellationToken);
			}
		}

		/// <inheritdoc/>
		public async Task<List<IUgsMetadata>> FindAsync(string stream, IReadOnlyList<int>? changes = null, int? minChange = null, int? maxChange = null, long? minTime = null, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<UgsMetadataDocument> filterBuilder = Builders<UgsMetadataDocument>.Filter;

			string normalizedStream = GetNormalizedStream(stream);

			FilterDefinition<UgsMetadataDocument> filter = filterBuilder.Eq(x => x.Stream, normalizedStream);
			if (changes != null && changes.Count > 0)
			{
				filter &= filterBuilder.In(x => x.Change, changes);
			}
			if (minChange != null)
			{
				filter &= filterBuilder.Gte(x => x.Change, minChange.Value);
			}
			if (maxChange != null)
			{
				filter &= filterBuilder.Lte(x => x.Change, maxChange.Value);
			}
			if (minTime != null)
			{
				filter &= filterBuilder.Gt(x => x.UpdateTicks, minTime.Value);
			}

			List<UgsMetadataDocument> documents = await _collection.Find(filter).ToListAsync(cancellationToken);
			return documents.ConvertAll<IUgsMetadata>(x =>
			{
				// Remove polluting null entries. Need to find the source of these.
				x.Users = x.Users.Where(y => y != null).ToList();
				x.Badges = x.Badges.Where(y => y != null).ToList();
				return x;
			});
		}

		/// <summary>
		/// Normalize a stream argument
		/// </summary>
		/// <param name="stream">The stream name</param>
		/// <returns>Normalized name</returns>
		[SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase")]
		private static string GetNormalizedStream(string stream)
		{
			return stream.ToLowerInvariant().TrimEnd('/');
		}

		/// <summary>
		/// Normalize a project argument
		/// </summary>
		/// <param name="project">The project name</param>
		/// <returns>Normalized name</returns>
		[SuppressMessage("Globalization", "CA1308:Normalize strings to uppercase")]
		private static string GetNormalizedProject(string? project)
		{
			if (project == null)
			{
				return String.Empty;
			}
			else
			{
				return project.ToLowerInvariant();
			}
		}

		/// <summary>
		/// Try to update a metadata document
		/// </summary>
		/// <param name="document">The document to update</param>
		/// <param name="update">Update definition</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the update succeeded</returns>
		private async Task<bool> TryUpdateAsync(UgsMetadataDocument document, UpdateDefinition<UgsMetadataDocument> update, CancellationToken cancellationToken)
		{
			int newUpdateIndex = document.UpdateIndex + 1;
			update = update.Set(x => x.UpdateIndex, newUpdateIndex);

			long newUpdateTicks = DateTime.UtcNow.Ticks;
			update = update.Set(x => x.UpdateTicks, newUpdateTicks);

			FilterDefinition<UgsMetadataDocument> filter = Builders<UgsMetadataDocument>.Filter.Expr(x => x.Id == document.Id && x.UpdateIndex == document.UpdateIndex);
			try
			{
				UpdateResult result = await _collection.UpdateOneAsync(filter, update, null, cancellationToken);
				if (result.ModifiedCount > 0)
				{
					document.UpdateIndex = newUpdateIndex;
					document.UpdateTicks = newUpdateTicks;
					return true;
				}
				return false;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to update UGS metadata document. State: {State}, Filter: {Filter}, Update: {Update}", document.ToBsonDocument().ToJson(), filter.Render(), update.Render());
				throw;
			}
		}
	}
}
