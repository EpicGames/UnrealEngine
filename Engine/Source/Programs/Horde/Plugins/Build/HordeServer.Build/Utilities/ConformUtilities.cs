// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.VersionControl.Perforce;
using Microsoft.Extensions.Logging;

namespace HordeServer.Utilities;

/// <summary>
/// Utilities for conforming agents
/// </summary>
public static class ConformUtilities
{
	/// <summary>
	/// Conform an agent that is below the disk free conform threshold for a workspace
	/// </summary>
	/// <param name="agent">Agent to query</param>
	/// <param name="poolService">Pool service instance</param>
	/// <param name="config">Current configuration</param>
	/// <param name="logger">Logger instance</param>
	/// <param name="cancellationToken">Cancellation token for the async task</param>
	/// <returns>True if conform was requested for the agent; otherwise false</returns>
	public static async Task<bool> AutoConformAgentAsync(IAgent agent, PoolService poolService, BuildConfig config, ILogger logger, CancellationToken cancellationToken)
	{
		long MegabytesToBytes(long v) => v * 1024 * 1024;
		double BytesToMegabytes(long v) => v / 1024.0 / 1024.0;
		long? freeDiskSpace = agent.GetDiskFreeSpace();
		long conformDiskSpaceNeeded = 0;
		HashSet<AgentWorkspaceInfo> workspaces = await poolService.GetWorkspacesAsync(agent, DateTime.UtcNow - TimeSpan.FromHours(1), config, cancellationToken);

		// Find the largest conform disk space amount needed, if any
		foreach (AgentWorkspaceInfo workspace in workspaces)
		{
			if (workspace.ConformDiskFreeSpace is > 0)
			{
				conformDiskSpaceNeeded = Math.Max(conformDiskSpaceNeeded, MegabytesToBytes(workspace.ConformDiskFreeSpace.Value));
			}
		}

		IAgent? newAgent = null;

		if (freeDiskSpace != null && conformDiskSpaceNeeded > 0 && freeDiskSpace < conformDiskSpaceNeeded &&
		    agent.ConformAttemptCount is null or 0 && !agent.RequestFullConform)
		{
			newAgent = await agent.TryUpdateAsync(new UpdateAgentOptions { RequestFullConform = true }, cancellationToken);
			logger.LogInformation("Auto-conforming {AgentId} as workspace conform disk space needed ({ConformDiskSpace:F1} MB) is greater than free disk space ({FreeDiskSpace:F1} MB)",
				agent.Id.ToString(), BytesToMegabytes(conformDiskSpaceNeeded), BytesToMegabytes(freeDiskSpace.Value));
		}

		return newAgent?.RequestFullConform ?? false;
	}
}