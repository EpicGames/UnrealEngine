// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Jobs.Templates;
using EpicGames.Horde.Streams;
using OpenTelemetry.Trace;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Extensions for OpenTelemetry spans
	/// </summary>
	public static class OpenTelemetryExtensions
	{
		/// <summary>Set a key:value tag on the span</summary>
		/// <returns>This span instance, for chaining</returns>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, ContentHash value)
		{
			span.SetAttribute(key, value.ToString());
			return span;
		}

		/// <summary>Set a key:value tag on the span</summary>
		/// <returns>This span instance, for chaining</returns>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, SubResourceId value)
		{
			span.SetAttribute(key, value.ToString());
			return span;
		}

		/// <inheritdoc cref="TelemetrySpan.SetAttribute(System.String, System.String)"/>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, StreamId? value) => span.SetAttribute(key, value?.ToString());

		/// <inheritdoc cref="TelemetrySpan.SetAttribute(System.String, System.String)"/>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, TemplateId? value) => span.SetAttribute(key, value?.ToString());

		/// <inheritdoc cref="TelemetrySpan.SetAttribute(System.String, System.String)"/>
		public static TelemetrySpan SetAttribute(this TelemetrySpan span, string key, TemplateId[]? values) => span.SetAttribute(key, values != null ? String.Join(',', values.Select(x => x.Id.ToString())) : null);
	}
}
