// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq.Expressions;
using System.Reflection;
using MongoDB.Driver;

namespace HordeServer.Utilities
{
	/// <summary>
	/// Allows building a transactional MongoDB update of several fields, and applying those to an in-memory object. Useful for transactional updates.
	/// </summary>
	/// <typeparam name="TDocument">The type of document to update</typeparam>
	public class TransactionBuilder<TDocument> where TDocument : class
	{
		/// <summary>
		/// Information about an update to a field
		/// </summary>
		class FieldUpdate
		{
			/// <summary>
			/// The update definition
			/// </summary>
			public UpdateDefinition<TDocument> _update;

			/// <summary>
			/// Applies an update to the given object
			/// </summary>
			public Action<object> _apply;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="update">The update definition</param>
			/// <param name="apply">Applies the update to a given object</param>
			public FieldUpdate(UpdateDefinition<TDocument> update, Action<object> apply)
			{
				_update = update;
				_apply = apply;
			}
		}

		/// <summary>
		/// Helper class for determining the indexer for a particular type
		/// </summary>
		/// <typeparam name="T">The type to find the indexer for</typeparam>
		static class Reflection<T>
		{
			/// <summary>
			/// Caches the indexer for this type
			/// </summary>
			public static PropertyInfo Indexer { get; } = GetIndexerProperty();

			/// <summary>
			/// Finds the indexer property from a type
			/// </summary>
			/// <returns>The property info</returns>
			static PropertyInfo GetIndexerProperty()
			{
				foreach (PropertyInfo property in typeof(T).GetProperties())
				{
					if (property.GetIndexParameters().Length > 0)
					{
						return property;
					}
				}
				return null!;
			}
		}

		/// <summary>
		/// List of field updates
		/// </summary>
		readonly List<FieldUpdate> _fieldUpdates = new List<FieldUpdate>();

		/// <summary>
		/// Whether the transaction is currenty empty
		/// </summary>
		public bool IsEmpty => _fieldUpdates.Count == 0;

		/// <summary>
		/// Adds a setting to this transaction
		/// </summary>
		/// <typeparam name="TField">Type of the field to be updated</typeparam>
		/// <param name="expr">The expresion defining the field to update</param>
		/// <param name="value">New value for the field</param>
		public void Set<TField>(Expression<Func<TDocument, TField>> expr, TField value)
		{
			UpdateDefinition<TDocument> update = Builders<TDocument>.Update.Set(expr, value);
			void Apply(object target) => Assign(target, expr, value);
			_fieldUpdates.Add(new FieldUpdate(update, Apply));
		}

		/// <summary>
		/// Adds an update to remove a field
		/// </summary>
		/// <param name="expr">The expresion defining the field to update</param>
		public void Unset(Expression<Func<TDocument, object>> expr)
		{
			UpdateDefinition<TDocument> update = Builders<TDocument>.Update.Unset(expr);
			void Apply(object target) => Unassign(target, expr.Body);
			_fieldUpdates.Add(new FieldUpdate(update, Apply));
		}

		/// <summary>
		/// Updates a property dictionary with a set of adds and removes
		/// </summary>
		/// <param name="expr">Lambda expression defining a field to update</param>
		/// <param name="updates">List of updates</param>
		public void UpdateDictionary<TKey, TValue>(Expression<Func<TDocument, Dictionary<TKey, TValue>>> expr, IEnumerable<KeyValuePair<TKey, TValue?>> updates) where TKey : class where TValue : class
		{
			PropertyInfo indexerProperty = Reflection<Dictionary<TKey, TValue>>.Indexer;
			foreach (KeyValuePair<TKey, TValue?> update in updates)
			{
				MethodCallExpression index = Expression.Call(expr.Body, indexerProperty.GetMethod!, new[] { Expression.Constant(update.Key) });
				if (update.Value == null)
				{
					Expression<Func<TDocument, object>> indexExpression = Expression.Lambda<Func<TDocument, object>>(index, expr.Parameters[0]);
					Unset(indexExpression);
				}
				else
				{
					Expression<Func<TDocument, TValue>> indexExpression = Expression.Lambda<Func<TDocument, TValue>>(index, expr.Parameters[0]);
					Set(indexExpression, update.Value);
				}
			}
		}

		/// <summary>
		/// Applies this transaction to the given update
		/// </summary>
		/// <param name="target">The object to update</param>
		public void ApplyTo(TDocument target)
		{
			foreach (FieldUpdate setting in _fieldUpdates)
			{
				setting._apply(target);
			}
		}

		/// <summary>
		/// Assign a value to a field
		/// </summary>
		/// <param name="target">The target object to be updated</param>
		/// <param name="fieldExpression">Lambda indicating the field to update</param>
		/// <param name="value">New value for the field</param>
		static void Assign(object? target, LambdaExpression fieldExpression, object? value)
		{
			Expression body = fieldExpression.Body;
			if (body.NodeType == ExpressionType.MemberAccess)
			{
				MemberExpression memberExpression = (MemberExpression)body;
				object? obj = Evaluate(memberExpression.Expression!, target);
				PropertyInfo propertyInfo = (PropertyInfo)memberExpression.Member;
				propertyInfo.SetValue(obj, value);
			}
			else if (body.NodeType == ExpressionType.ArrayIndex)
			{
				BinaryExpression binaryExpression = (BinaryExpression)body;
				System.Collections.IList list = (System.Collections.IList)Evaluate(binaryExpression.Left, target)!;
				int index = (int)Evaluate(binaryExpression.Right, target)!;
				list[index] = value;
			}
			else if (body.NodeType == ExpressionType.Call)
			{
				MethodCallExpression callExpression = (MethodCallExpression)body;
				System.Collections.IDictionary dictionary = (System.Collections.IDictionary)Evaluate(callExpression.Object!, target)!;
				object? key = Evaluate(callExpression.Arguments[0], null);
				dictionary[key!] = value;
			}
			else if (body.NodeType == ExpressionType.Index)
			{
				IndexExpression indexExpression = (IndexExpression)body;
				object? obj = Evaluate(indexExpression.Object!, target);

				object?[] arguments = new object?[indexExpression.Arguments.Count];
				for (int idx = 0; idx < indexExpression.Arguments.Count; idx++)
				{
					arguments[idx] = Evaluate(indexExpression.Arguments[idx], null);
				}

				indexExpression.Indexer!.SetValue(obj, value, arguments);
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Removes an entry from a dictionary
		/// </summary>
		/// <param name="target">The target object to be updated</param>
		/// <param name="body">Body of the expression indicating the field to update</param>
		static void Unassign(object? target, Expression body)
		{
			if (body.NodeType == ExpressionType.MemberAccess)
			{
				MemberExpression memberExpression = (MemberExpression)body;
				switch (memberExpression.Member.MemberType)
				{
					case MemberTypes.Field:
						((FieldInfo)memberExpression.Member).SetValue(target, null);
						break;
					case MemberTypes.Property:
						((PropertyInfo)memberExpression.Member).SetValue(target, null);
						break;
					default:
						throw new NotImplementedException();
				}
			}
			else if (body.NodeType == ExpressionType.Call)
			{
				MethodCallExpression callExpression = (MethodCallExpression)body;
				System.Collections.IDictionary? dictionary = (System.Collections.IDictionary?)Evaluate(callExpression.Object!, target);
				object? key = Evaluate(callExpression.Arguments[0], null);
				dictionary!.Remove(key!);
			}
			else if (body.NodeType == ExpressionType.Index)
			{
				IndexExpression indexExpression = (IndexExpression)body;
				System.Collections.IDictionary? dictionary = (System.Collections.IDictionary?)Evaluate(indexExpression.Object!, target);
				object? key = Evaluate(indexExpression.Arguments[0], null);
				dictionary!.Remove(key!);
			}
			else if (body.NodeType == ExpressionType.Convert)
			{
				UnaryExpression unaryExpression = (UnaryExpression)body;
				Unassign(target, unaryExpression.Operand);
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Evaluates an expression
		/// </summary>
		/// <param name="expression">The expression to evaluate</param>
		/// <param name="parameter">Parameter to the unary lambda expression</param>
		/// <returns>Value of the expression</returns>
		static object? Evaluate(Expression expression, object? parameter)
		{
			if (expression.NodeType == ExpressionType.Call)
			{
				MethodCallExpression callExpression = (MethodCallExpression)expression;
				object? obj = Evaluate(callExpression.Object!, parameter);
				object?[] arguments = callExpression.Arguments.Select(x => Evaluate(x, parameter)).ToArray();
				return callExpression.Method.Invoke(obj, arguments);
			}
			else if (expression.NodeType == ExpressionType.Constant)
			{
				ConstantExpression constantExpression = (ConstantExpression)expression;
				return constantExpression.Value;
			}
			else if (expression.NodeType == ExpressionType.MemberAccess)
			{
				MemberExpression memberExpression = (MemberExpression)expression;
				object? target = Evaluate(memberExpression.Expression!, parameter);

				MemberInfo member = memberExpression.Member;
				switch (member.MemberType)
				{
					case MemberTypes.Property:
						return ((PropertyInfo)member).GetValue(target);
					case MemberTypes.Field:
						return ((FieldInfo)member).GetValue(target);
					default:
						throw new NotImplementedException("Unsupported expression type");
				}
			}
			else if (expression.NodeType == ExpressionType.ArrayIndex)
			{
				BinaryExpression binaryExpression = (BinaryExpression)expression;
				System.Collections.IList list = (System.Collections.IList)Evaluate(binaryExpression.Left, parameter)!;
				int index = (int)Evaluate(binaryExpression.Right, null)!;
				return list[index];
			}
			else if (expression.NodeType == ExpressionType.Parameter)
			{
				return parameter;
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		/// <summary>
		/// Converts this transaction to an update definition
		/// </summary>
		public UpdateDefinition<TDocument> ToUpdateDefinition()
		{
			return Builders<TDocument>.Update.Combine(_fieldUpdates.Select(x => x._update));
		}
	}
}
