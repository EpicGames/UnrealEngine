// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using HordeServer.Streams;

namespace HordeServer.VersionControl
{
	/// <summary>
	/// Version Control interface
	/// </summary>
	public interface IVersionControlService
	{
		/// <summary>
		/// Name of the service
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Creates a commit source for the given stream
		/// </summary>
		/// <param name="streamConfig">Stream to create a commit source for</param>
		/// <returns>Commit source instance</returns>
		public ICommitCollection GetCommits(StreamConfig streamConfig);
	}
}
