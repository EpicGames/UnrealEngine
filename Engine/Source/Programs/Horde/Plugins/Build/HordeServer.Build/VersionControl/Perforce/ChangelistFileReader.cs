// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Perforce;
using HordeServer.Configuration;
using Microsoft.Extensions.Logging;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Reads config files from shelved Perforce changelists for diff/validation.
	/// </summary>
	public class ChangelistFileReader : IChangelistFileReader
	{
		readonly IPerforceService _perforceService;
		readonly ILogger<ChangelistFileReader> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ChangelistFileReader(IPerforceService perforceService, ILogger<ChangelistFileReader> logger)
		{
			_perforceService = perforceService;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<Dictionary<Uri, byte[]>> ReadChangelistFilesAsync(
			int changelist, string? cluster, CancellationToken cancellationToken)
		{
			cluster ??= "default";
			using IPooledPerforceConnection perforce = await _perforceService.ConnectAsync(
				cluster, cancellationToken: cancellationToken);

			PerforceResponse<DescribeRecord> describeResponse = await perforce.TryDescribeAsync(
				DescribeOptions.Shelved, -1, changelist, cancellationToken);
			if (!describeResponse.Succeeded)
			{
				throw new ChangelistNotFoundException(
					$"Changelist {changelist} not found or has no shelved files. Ensure the CL is shelved (not just pending).");
			}

			DescribeRecord record = describeResponse.Data;
			Dictionary<Uri, byte[]> files = new Dictionary<Uri, byte[]>();

			foreach (DescribeFileRecord fileRecord in record.Files)
			{
				if (!fileRecord.DepotFile.EndsWith(".json", StringComparison.OrdinalIgnoreCase))
				{
					continue;
				}

				PerforceResponse<PrintRecord<byte[]>> printResponse = await perforce.TryPrintDataAsync(
					$"{fileRecord.DepotFile}@={changelist}", cancellationToken);
				if (!printResponse.Succeeded || printResponse.Data.Contents == null)
				{
					_logger.LogWarning(
						"Skipping file {DepotFile}@={Changelist}: unable to read contents",
						fileRecord.DepotFile, changelist);
					continue;
				}

				PrintRecord<byte[]> printRecord = printResponse.Data;
				Uri uri = new Uri($"perforce://{cluster}{printRecord.DepotFile}");
				files.Add(uri, printRecord.Contents);
			}

			return files;
		}
	}
}
