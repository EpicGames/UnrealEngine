// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using HordeServer.Users;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Record used for Commit telemetry. Models a <see cref="ICommit"/>.
	/// </summary>
	/// <remarks>Modifications to this record's structure should be versioned appropriately through <see cref="AbstractTelemetryRecord.SchemaVersion"/>, and call sites should continue to publish in a backwards compatible manner.</remarks>
	[AnalyticsTableGen]
	[TelemetryEvent(DefaultEventName)]
	[Table("horde.state_change", Schema = "ingest")]
	public record CommitTelemetry : AbstractTelemetryRecord
	{
		/// <summary>
		/// Identifier used when no corresponding user Id can be found when enacting user-to-userId conversion.
		/// </summary>
		public const string UnableToSourceUserId = "NoUserIdFound";

		/// <summary>
		/// Default event name for the CommitTelemetry.
		/// </summary>
		public const string DefaultEventName = "State.Change";

		/// <summary>
		/// Current schema version.
		/// </summary>
		public static readonly int CurrentSchemaVersion = 5;

		/// <summary>
		/// The user who submitted this commit.
		/// </summary>
		[Column("submitted_by")]
		public UserId SubmittedBy { get; init; }

		/// <summary>
		/// The stream this commit belongs to.
		/// </summary>
		[Column("stream_id")]
		public StreamId StreamId { get; init; }

		/// <summary>
		/// The change number.
		/// </summary>
		[Column("change")]
		public int Change { get; init; }

		/// <summary>
		/// The original change number (before any merges).
		/// </summary>
		[Column("original_change")]
		public int OriginalChange { get; init; }

		/// <summary>
		/// The time the commit was submitted.
		/// </summary>
		[Column("submitted_at")]
		public DateTime SubmittedAt { get; init; }

		/// <summary>
		/// The external issue key (e.g., Jira ticket).
		/// </summary>
		[Column("external_issue_key")]
		public string? ExternalIssueKey { get; init; }

		/// <summary>
		/// The Robomerge author tag.
		/// </summary>
		[Column("robomerge_author_tag")]
		public string? RobomergeAuthorTag { get; init; }

		/// <summary>
		/// The Robomerge source tag.
		/// </summary>
		[Column("robomerge_source_tag")]
		public string? RobomergeSourceTag { get; init; }

		/// <summary>
		/// The Robomerge owner tag.
		/// </summary>
		[Column("robomerge_owner_tag")]
		public string? RobomergeOwnerTag { get; init; }

		/// <summary>
		/// The Robomerge bot tag.
		/// </summary>
		[Column("robomerge_bot_tag")]
		public string? RobomergeBotTag { get; init; }

		/// <summary>
		/// The review tag.
		/// </summary>
		[Column("review_tag")]
		public string? ReviewTag { get; init; }

		/// <summary>
		/// The preflight tag.
		/// </summary>
		[Column("preflight_tag")]
		public string? PreflightTag { get; init; }

		/// <summary>
		/// The virtualized tag.
		/// </summary>
		[Column("virtualized")]
		public string? Virtualized { get; init; }

		/// <summary>
		/// Default constructor for an empty telemetry object.
		/// </summary>
		/// <remarks>This constructor is effectively reserved for ORM mapping.</remarks>
		public CommitTelemetry() : base(DefaultEventName, CurrentSchemaVersion)
		{
			SubmittedBy = UserId.Empty;
			StreamId = new StreamId(String.Empty);
			Change = 0;
			OriginalChange = 0;
			SubmittedAt = DateTime.UtcNow;
		}

		/// <summary>
		/// Constructs a CommitTelemetry event based on the provided commit and commit tag container.
		/// </summary>
		/// <param name="commit">The commit to base the telemetry event off of.</param>
		/// <param name="commitTagContainer">The commit description tag container to base the telemetry off of.</param>
		/// <remarks>This should be used when there is no desire to replace all user names with <see cref="UserId"/>, but instead retain plain-text user names.</remarks>
		public CommitTelemetry(ICommit commit, CommitDescriptionTagContainer commitTagContainer) : base(DefaultEventName, CurrentSchemaVersion)
		{
			SubmittedBy = commit.AuthorId;
			StreamId = commit.StreamId;
			Change = commit.Id.Order;
			OriginalChange = commit.OriginalCommitId.Order;
			SubmittedAt = commit.DateUtc;
			ExternalIssueKey = commitTagContainer.JiraTag;
			RobomergeAuthorTag = commitTagContainer.RobomergeAuthorTag;
			RobomergeSourceTag = commitTagContainer.RobomergeSourceTag;
			RobomergeOwnerTag = commitTagContainer.RobomergeOwnerTag;
			RobomergeBotTag = commitTagContainer.RobomergeBotTag;
			ReviewTag = commitTagContainer.ReviewTag;
			PreflightTag = commitTagContainer.PresubmitTag;
			Virtualized = commitTagContainer.VirtualizedTag;
		}

		/// <summary>
		/// Create method used to construct a CommitTelemetry record.
		/// </summary>
		/// <param name="userCollection">The user collection used to derive underlying userId.</param>
		/// <param name="commit">The commit to base the telemetry event off of.</param>
		/// <param name="commitTagContainer">The commit description tag container to base the telemetry off of.</param>
		/// <returns>The CommitTelemetry record, with user names replaced with UserIds (or replaced with <see cref="UnableToSourceUserId"/> if no matching id found.).</returns>
		/// <remarks>This should be used when there is a desire to replace all user names with <see cref="UserId"/>. If no userId is found, replace with <see cref="UnableToSourceUserId"/>.</remarks>
		public static async Task<CommitTelemetry> CreateCommitTelemetryWithUserIdAsync(IUserCollection userCollection, ICommit commit, CommitDescriptionTagContainer commitTagContainer)
		{
			if (commitTagContainer.RobomergeAuthorTag != null)
			{
				IUser? userId = await userCollection.FindUserByLoginAsync(commitTagContainer.RobomergeAuthorTag);
				if (userId != null)
				{
					commitTagContainer.RobomergeAuthorTag = userId.Id.ToString();
				}
				else
				{
					commitTagContainer.RobomergeAuthorTag = UnableToSourceUserId;
				}
			}

			if (commitTagContainer.ReviewTag != null)
			{
				IUser? userId = await userCollection.FindUserByLoginAsync(commitTagContainer.ReviewTag);
				if (userId != null)
				{
					commitTagContainer.ReviewTag = userId.Id.ToString();
				}
				else
				{
					commitTagContainer.ReviewTag = UnableToSourceUserId;
				}
			}

			return new CommitTelemetry(commit, commitTagContainer);
		}
	}
}
