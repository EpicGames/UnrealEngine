// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Commits;
using EpicGames.Perforce;
using HordeServer.Streams;
using HordeServer.Users;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Result from checking a shelved change status
	/// </summary>
	public enum CheckShelfResult
	{
		/// <summary>
		/// The shelf is ok
		/// </summary>
		Ok,

		/// <summary>
		/// Changelist does not exist
		/// </summary>
		NoChange,

		/// <summary>
		/// The change does not contain any shelved files
		/// </summary>
		NoShelvedFiles,

		/// <summary>
		/// The shelf contains *some* files from a different stream
		/// </summary>
		MixedStream,

		/// <summary>
		/// The shelf contains only files from a different stream
		/// </summary>
		WrongStream,
	}

	/// <summary>
	/// Information about a shelved change
	/// </summary>
	public class ShelfInfo
	{
		/// <summary>
		/// Description for the change
		/// </summary>
		public string Description { get; set; }

		/// <summary>
		/// Tags for the change
		/// </summary>
		public List<CommitTag> Tags { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		public ShelfInfo(string description, List<CommitTag> tags)
		{
			Description = description;
			Tags = tags;
		}
	}

	/// <summary>
	/// A connection returned by the Perforce service
	/// </summary>
	public interface IPooledPerforceConnection : IPerforceConnection, IDisposable
	{
		/// <summary>
		/// The client used by the connection
		/// </summary>
		ClientRecord? Client { get; }

		/// <summary>
		/// Gets the server information, returning a cached copy if possible
		/// </summary>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		ValueTask<InfoRecord> GetInfoAsync(CancellationToken cancellationToken);

		/// <summary>
		/// Gets a cached stream view for the given stream
		/// </summary>
		/// <param name="streamConfig">Stream to find the view for</param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		ValueTask<PerforceViewMap> GetCachedStreamViewAsync(StreamConfig streamConfig, CancellationToken cancellationToken);
	}

	/// <summary>
	/// Extension methods for PerforceService
	/// </summary>
	public static class PooledPerforceConnectionExtensions
	{
		/// <summary>
		/// Map files from a <see cref="DescribeRecord"/> to relative paths within a stream
		/// </summary>
		/// <param name="perforce">Perforce connection</param>
		/// <param name="streamConfig">Config for the stream to target</param>
		/// <param name="describeRecord">Description of the change</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of files relative to the stream</returns>
		public static Task<List<string>> GetStreamFilesAsync(this IPooledPerforceConnection perforce, StreamConfig streamConfig, DescribeRecord describeRecord, CancellationToken cancellationToken)
		{
			return GetStreamFilesAsync(perforce, streamConfig, describeRecord.Files.ConvertAll(x => x.DepotFile), cancellationToken);
		}

		/// <summary>
		/// Map files from a <see cref="DescribeRecord"/> to relative paths within a stream
		/// </summary>
		/// <param name="perforce">Perforce connection</param>
		/// <param name="streamConfig">Stream to target</param>
		/// <param name="depotPaths">Paths within the depot</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of files relative to the stream</returns>
		public static async Task<List<string>> GetStreamFilesAsync(this IPooledPerforceConnection perforce, StreamConfig streamConfig, List<string> depotPaths, CancellationToken cancellationToken)
		{
			PerforceViewMap viewMap = await perforce.GetCachedStreamViewAsync(streamConfig, cancellationToken);
			InfoRecord infoRecord = await perforce.GetInfoAsync(cancellationToken);
			return viewMap.MapFiles(depotPaths, infoRecord.PathComparison).ToList();
		}
	}

	/// <summary>
	/// Wrapper around Perforce functionality. Can use a local p4.exe client for development purposes, or a separate HordePerforceBridge instance over REST for deployments.
	/// </summary>
	public interface IPerforceService : IVersionControlService
	{
		/// <summary>
		/// Refreshes any cached metadata for the given change
		/// </summary>
		/// <param name="clusterName">Name of the Perforce cluster</param>
		/// <param name="change">Change to update</param>
		public Task RefreshCachedCommitAsync(string clusterName, int change);

		/// <summary>
		/// Connect to the given Perforce cluster
		/// </summary>
		/// <param name="clusterName">Name of the cluster</param>
		/// <param name="userName">Name of the user to connect as. Uses the service account if null.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Connection information</returns>
		public Task<IPooledPerforceConnection> ConnectAsync(string clusterName, string? userName = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds or adds a user from the given Perforce server, adding the user (and populating their profile with Perforce data) if they do not currently exist
		/// </summary>
		/// <param name="clusterName"></param>
		/// <param name="userName"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public ValueTask<IUser> FindOrAddUserAsync(string clusterName, string userName, CancellationToken cancellationToken = default);

		/// <summary>
		/// Checks a shelf is valid for the given stream
		/// </summary>
		/// <param name="streamConfig">The stream to query</param>
		/// <param name="changeNumber">Shelved changelist number</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Whether the shelf is valid, plus the description for it</returns>
		public Task<(CheckShelfResult, ShelfInfo?)> CheckShelfAsync(StreamConfig streamConfig, int changeNumber, CancellationToken cancellationToken = default);

		/// <summary>
		/// Duplicates a shelved changelist
		/// </summary>
		/// <param name="clusterName">Name of the Perforce cluster</param>
		/// <param name="shelvedChange">The shelved changelist</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The duplicated changelist</returns>
		public Task<int> DuplicateShelvedChangeAsync(string clusterName, int shelvedChange, CancellationToken cancellationToken = default);

		/// <summary>
		/// Submit a shelved changelist
		/// </summary>
		/// <param name="streamConfig">Stream to submit to</param>
		/// <param name="shelvedChange">The shelved changelist number, created by <see cref="DuplicateShelvedChangeAsync(String,Int32,CancellationToken)"/></param>
		/// <param name="originalChange">The original changelist number</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Tuple consisting of the submitted changelist number and message</returns>
		public Task<(int? Change, string Message)> SubmitShelvedChangeAsync(StreamConfig streamConfig, int shelvedChange, int originalChange, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a changelist containing shelved files.
		/// </summary>
		/// <param name="clusterName">Name of the Perforce cluster</param>
		/// <param name="shelvedChange">The changelist containing shelved files</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async tasy</returns>
		public Task DeleteShelvedChangeAsync(string clusterName, int shelvedChange, CancellationToken cancellationToken = default);

		/// <summary>
		/// Updates a changelist description
		/// </summary>
		/// <param name="clusterName">Name of the Perforce cluster</param>
		/// <param name="change">The change to update</param>
		/// <param name="updateFunc">Function to update the description</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public Task UpdateChangelistDescriptionAsync(string clusterName, int change, Func<string, string> updateFunc, CancellationToken cancellationToken = default);
	}
}
