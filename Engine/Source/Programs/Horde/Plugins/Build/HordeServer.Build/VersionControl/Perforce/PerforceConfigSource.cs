// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Users;
using EpicGames.Perforce;
using HordeServer.Configuration;
using HordeServer.Users;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Perforce cluster config file source
	/// </summary>
	public sealed class PerforceConfigSource : IConfigSource
	{
		class ConfigFileImpl : IConfigFile
		{
			public Uri Uri { get; }
			public int Change { get; }
			public string Revision { get; }
			public IUser? Author { get; }

			readonly PerforceConfigSource _owner;

			public ConfigFileImpl(Uri uri, int change, IUser? author, PerforceConfigSource owner)
			{
				Uri = uri;
				Change = change;
				Revision = $"{change}";
				Author = author;
				_owner = owner;
			}

			public ValueTask<ReadOnlyMemory<byte>> ReadAsync(CancellationToken cancellationToken) => _owner.ReadAsync(Uri, Change, cancellationToken);
		}

		/// <summary>
		/// Name of the scheme for this source
		/// </summary>
		public const string Scheme = "perforce";

		/// <inheritdoc/>
		string IConfigSource.Scheme => Scheme;

		/// <inheritdoc/>
		public TimeSpan UpdateInterval => TimeSpan.FromMinutes(1.0);

		readonly IOptionsMonitor<BuildServerConfig> _settings;
		readonly IUserCollection _userCollection;
		readonly IMemoryCache _cache;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceConfigSource(IOptionsMonitor<BuildServerConfig> settings, IUserCollection userCollection, IMemoryCache cache, ILogger<PerforceConfigSource> logger)
		{
			_settings = settings;
			_userCollection = userCollection;
			_cache = cache;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<IConfigFile[]> GetFilesAsync(Uri[] uris, CancellationToken cancellationToken)
		{
			Dictionary<Uri, IConfigFile> results = new Dictionary<Uri, IConfigFile>();
			foreach (IGrouping<string, Uri> group in uris.GroupBy(x => x.Host))
			{
				using (IPerforceConnection perforce = await ConnectAsync(group.Key, cancellationToken))
				{
					try
					{
						FileSpecList fileSpec = group.Select(x => x.AbsolutePath).Distinct(StringComparer.OrdinalIgnoreCase).ToList();

						List<FStatRecord> records = await perforce.FStatAsync(FStatOptions.ShortenOutput, fileSpec, cancellationToken).ToListAsync(cancellationToken);
						records.RemoveAll(x => x.HeadAction == FileAction.Delete || x.HeadAction == FileAction.MoveDelete);

						Dictionary<string, FStatRecord> absolutePathToRecord = records.ToDictionary(x => x.DepotFile ?? String.Empty, x => x, StringComparer.OrdinalIgnoreCase);
						foreach (Uri uri in group)
						{
							FStatRecord? record;
							if (!absolutePathToRecord.TryGetValue(uri.AbsolutePath, out record))
							{
								throw new FileNotFoundException($"Unable to read {uri}. No matching files found.");
							}

							IUser? author = await GetAuthorAsync(perforce, group.Key, record.HeadChange, cancellationToken);
							results[uri] = new ConfigFileImpl(uri, record.HeadChange, author, this);
						}
					}
					catch (PerforceException ex) when (ex.Error != null && ex.Error.Generic == PerforceGenericCode.Config)
					{
						throw new PerforceException($"{ex.Message} [server: {perforce.Settings.ServerAndPort}, user: {perforce.Settings.UserName}]", ex);
					}
				}
			}
			return uris.ConvertAll(x => results[x]);
		}

		async ValueTask<IUser?> GetAuthorAsync(IPerforceConnection perforce, string host, int change, CancellationToken cancellationToken)
		{
			string cacheKey = $"{nameof(PerforceConfigSource)}:author:{host}@{change}";
			if (!_cache.TryGetValue(cacheKey, out string? author))
			{
				ChangeRecord record = await perforce.GetChangeAsync(GetChangeOptions.None, change, cancellationToken);
				using (ICacheEntry entry = _cache.CreateEntry(cacheKey))
				{
					entry.SetSlidingExpiration(TimeSpan.FromHours(1.0));
					entry.SetSize(256);
					entry.SetValue(record.User);
				}
			}
			return (author != null) ? await _userCollection.FindUserByLoginAsync(author, cancellationToken) : null;
		}

		async ValueTask<ReadOnlyMemory<byte>> ReadAsync(Uri uri, int change, CancellationToken cancellationToken)
		{
			string cacheKey = $"{nameof(PerforceConfigSource)}:data:{uri}@{change}";
			if (_cache.TryGetValue(cacheKey, out ReadOnlyMemory<byte> data))
			{
				_logger.LogInformation("Read {Uri}@{Change} from cache ({Key})", uri, change, cacheKey);
			}
			else
			{
				_logger.LogInformation("Reading {Uri} at CL {Change} from Perforce", uri, change);
				using (IPerforceConnection perforce = await ConnectAsync(uri.Host, cancellationToken))
				{
					PerforceResponse<PrintRecord<byte[]>> response = await perforce.TryPrintDataAsync($"{uri.AbsolutePath}@{change}", cancellationToken);
					response.EnsureSuccess();
					data = response.Data.Contents!;
				}
				using (ICacheEntry entry = _cache.CreateEntry(cacheKey))
				{
					entry.SetSlidingExpiration(TimeSpan.FromHours(1.0));
					entry.SetSize(data.Length);
					entry.SetValue(data);
				}
			}
			return data;
		}

		async Task<IPerforceConnection> ConnectAsync(string host, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			BuildServerConfig settings = _settings.CurrentValue;

			PerforceConnectionId connectionId = new PerforceConnectionId();
			if (!String.IsNullOrEmpty(host))
			{
				connectionId = new PerforceConnectionId(host);
			}

			PerforceConnectionSettings? connectionSettings = settings.Perforce.FirstOrDefault(x => x.Id == connectionId);
			if (connectionSettings == null)
			{
				if (connectionId == PerforceConnectionSettings.Default)
				{
					connectionSettings = new PerforceConnectionSettings();
				}
				else
				{
					throw new InvalidOperationException($"No Perforce connection settings defined for '{connectionId}'.");
				}
			}

			IPerforceConnection connection = await PerforceConnection.CreateAsync(connectionSettings.ToPerforceSettings(), _logger);
			if (connectionSettings.Credentials != null && !String.IsNullOrEmpty(connectionSettings.Credentials.Password) && String.IsNullOrEmpty(connectionSettings.Credentials.Ticket))
			{
				await connection.LoginAsync(connectionSettings.Credentials.Password, cancellationToken);
			}
			return connection;
		}

		/// <inheritdoc/>
		public async Task GetUpdateInfoAsync(IReadOnlyDictionary<Uri, string> files, IReadOnlyDictionary<Uri, string>? prevFiles, ConfigUpdateInfo updateInfo, CancellationToken cancellationToken)
		{
			foreach (IGrouping<string, KeyValuePair<Uri, string>> group in files.Where(x => x.Key.Scheme == PerforceConfigSource.Scheme).GroupBy(x => x.Key.Host))
			{
				// Figure out the most recent changelist
				int change = group.Select(x => Int32.Parse(x.Value)).Max();
				updateInfo.Status.Add($"Perforce changelist ({group.Key}): {change}");

				// Find all the users that have committed changes
				if (prevFiles != null)
				{
					using IPerforceConnection perforce = await ConnectAsync(group.Key, cancellationToken);
					foreach ((Uri uri, string revision) in group)
					{
						string? prevRevision;
						if (prevFiles.TryGetValue(uri, out prevRevision) && prevRevision != revision)
						{
							int prevRevisionNum = Int32.Parse(prevRevision);
							string fileSpec = $"{uri.LocalPath}@{prevRevisionNum + 1},{revision}";
							await FindAuthorsAsync(perforce, fileSpec, updateInfo.Authors, cancellationToken);
						}
					}
				}
			}
		}

		async Task FindAuthorsAsync(IPerforceConnection perforce, string fileSpec, HashSet<UserId> authors, CancellationToken cancellationToken)
		{
			List<ChangesRecord> records = await perforce.GetChangesAsync(ChangesOptions.None, -1, ChangeStatus.Submitted, fileSpec, cancellationToken);
			foreach (ChangesRecord record in records)
			{
				IUser? user = await _userCollection.FindUserByLoginAsync(record.User, cancellationToken);
				if (user != null)
				{
					authors.Add(user.Id);
				}
			}
		}
	}
}
