// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json.Serialization;
using HordeServer.Server;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Controller for Perforce triggers and callbacks
	/// </summary>
	[Authorize]
	[ApiController]
	[Route("[controller]")]
	public class PerforceController : ControllerBase
	{
		private readonly PerforceLoadBalancer _perforceLoadBalancer;
		private readonly IOptionsSnapshot<BuildConfig> _buildConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceController(PerforceLoadBalancer perforceLoadBalancer, IOptionsSnapshot<BuildConfig> buildConfig)
		{
			_perforceLoadBalancer = perforceLoadBalancer;
			_buildConfig = buildConfig;
		}

		/// <summary>
		/// Get the current server status
		/// </summary>
		/// <returns></returns>
		[HttpGet]
		[Route("/api/v1/perforce/status")]
		public async Task<ActionResult<List<object>>> GetStatusAsync(CancellationToken cancellationToken)
		{
			List<IPerforceServer> servers = await _perforceLoadBalancer.GetServersAsync(cancellationToken);

			List<object> responses = new List<object>();
			foreach (IPerforceServer server in servers)
			{
				responses.Add(new { server.ServerAndPort, server.BaseServerAndPort, server.Cluster, server.NumLeases, server.Status, server.Detail, server.LastUpdateTime });
			}
			return responses;
		}

		/// <summary>
		/// Gets the current perforce settinsg
		/// </summary>
		/// <returns>List of Perforce clusters</returns>
		[HttpGet]
		[Route("/api/v1/perforce/settings")]
		public ActionResult<List<PerforceCluster>> GetPerforceSettings()
		{
			BuildConfig buildConfig = _buildConfig.Value;

			if (!buildConfig.Authorize(AdminAclAction.AdminRead, User))
			{
				return Forbid();
			}

			return buildConfig.PerforceClusters;
		}
	}

	/// <summary>
	/// Body of Perforce trigger request (deserialized from JSON)
	/// </summary>
	public class PerforceTriggerRequest
	{
		/// <summary>
		/// Type of trigger (change-commit, form-save etc)
		/// </summary>
		public string TriggerType { get; }

		/// <summary>
		/// Triggering user’s client workspace name.
		/// </summary>
		public string Client { get; }

		/// <summary>
		/// Hostname of the user’s workstation (even if connected through a proxy, broker, replica, or an edge server.)
		/// </summary>
		public string ClientHost { get; }

		/// <summary>
		/// The IP address of the user’s workstation (even if connected through a proxy, broker, replica, or an edge server.)
		/// </summary>
		public string ClientIp { get; }

		/// <summary>
		/// The name of the user’s client application. For example, P4V, P4Win
		/// </summary>
		public string ClientProg { get; }

		/// <summary>
		/// The version of the user’s client application.
		/// </summary>
		public string ClientVersion { get; }

		/// <summary>
		/// If the command was sent through a proxy, broker, replica, or edge server, the hostname of the proxy, broker, replica, or edge server.
		/// (If the command was sent directly, %peerhost% matches %clienthost%)
		/// </summary>
		public string PeerHost { get; }

		/// <summary>
		/// If the command was sent through a proxy, broker, replica, or edge server, the IP address of the proxy, broker, replica, or edge server.
		/// (If the command was sent directly, %peerip% matches %clientip%)
		/// </summary>
		public string PeerIp { get; }

		/// <summary>
		/// Hostname of the Helix Core Server.
		/// </summary>
		public string ServerHost { get; }

		/// <summary>
		/// The value of the Helix Core Server’s server.id. See p4 serverid in the Helix Core Command-Line (P4) Reference.
		/// </summary>
		public string ServerId { get; }

		/// <summary>
		/// The IP address of the server.
		/// </summary>
		public string ServerIp { get; }

		/// <summary>
		/// The value of the Helix Core Server’s P4NAME.
		/// </summary>
		public string ServerName { get; }

		/// <summary>
		/// The transport, IP address, and port of the Helix Core Server, in the format prefix:ip_address:port.
		/// </summary>
		public string ServerPort { get; }

		/// <summary>
		/// In a distributed installation, for any change trigger:
		///     If the submit was run on the commit server, %submitserverid% equals %serverid%.
		///     If the submit was run on the edge server, %submitserverid% does not equal %serverid%. In this case, %submitserverid% holds the edge server’s server id.
		/// If this is not a distributed installation, %submitserverid% is always empty.
		/// </summary>
		public string? SubmitServerId { get; }

		/// <summary>
		/// Helix Server username of the triggering user.
		/// </summary>
		public string User { get; }

		/// <summary>
		/// Name of form (for instance, a branch name or a changelist number).
		/// </summary>
		public string? FormName { get; }

		/// <summary>
		/// Type of form (for instance, branch, change, and so on).
		/// </summary>
		public string? FormType { get; }

		/// <summary>
		/// The number of the changelist being submitted. Not set for form-save.
		/// </summary>
		[JsonPropertyName("ChangeNumber")]
		public string? ChangeNumberString { get; }

		/// <summary>
		/// The root path of files submitted.
		/// </summary>
		public string? ChangeRoot { get; }

		/// <summary>
		/// Change number
		/// Normalized as form-save stores the change number in 'formname'
		/// </summary>
		[JsonIgnore]
		public int ChangeNumber { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="triggerType"></param>
		/// <param name="client"></param>
		/// <param name="clientHost"></param>
		/// <param name="clientIp"></param>
		/// <param name="clientProg"></param>
		/// <param name="clientVersion"></param>
		/// <param name="peerHost"></param>
		/// <param name="peerIp"></param>
		/// <param name="serverHost"></param>
		/// <param name="serverId"></param>
		/// <param name="serverIp"></param>
		/// <param name="serverName"></param>
		/// <param name="serverPort"></param>
		/// <param name="submitServerId"></param>
		/// <param name="user"></param>
		/// <param name="formName"></param>
		/// <param name="formType"></param>
		/// <param name="changeNumberString"></param>
		/// <param name="changeRoot"></param>
		public PerforceTriggerRequest(string triggerType, string client, string clientHost, string clientIp, string clientProg, string clientVersion, string peerHost, string peerIp, string serverHost, string serverId, string serverIp, string serverName, string serverPort, string submitServerId, string user, string formName, string formType, string changeNumberString, string changeRoot)
		{
			TriggerType = triggerType;
			Client = client;
			ClientHost = clientHost;
			ClientIp = clientIp;
			ClientProg = clientProg;
			ClientVersion = clientVersion;
			PeerHost = peerHost;
			PeerIp = peerIp;
			ServerHost = serverHost;
			ServerId = serverId;
			ServerIp = serverIp;
			ServerName = serverName;
			ServerPort = serverPort;
			SubmitServerId = submitServerId;
			User = user;
			FormName = formName;
			FormType = formType;
			ChangeNumberString = changeNumberString;
			ChangeRoot = changeRoot;

			try
			{
				ChangeNumber = TriggerType == "form-save" ? Convert.ToInt32(FormName) : Convert.ToInt32(ChangeNumberString);
			}
			catch (Exception)
			{
				// form-save can contain the change number "default"
				ChangeNumber = -1;
			}
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"TriggerType={TriggerType} User={User} ChangeNumber={ChangeNumber}";
		}
	}

	/// <summary>
	/// Controller for Perforce triggers and callbacks
	/// </summary>
	[ApiController]
	[Tags("Perforce")]
	[Route("[controller]")]
	public class PublicPerforceController : ControllerBase
	{
		private readonly IPerforceService _perforceService;
		private readonly Tracer _tracer;
		private readonly ILogger<PerforceController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public PublicPerforceController(IPerforceService perforceService, Tracer tracer, ILogger<PerforceController> logger)
		{
			_perforceService = perforceService;
			_tracer = tracer;
			_logger = logger;
		}

		/// <summary>
		/// Endpoint which trigger scripts in Perforce will call
		/// </summary>
		/// <returns>200 OK on success</returns>
		[HttpPost]
		[Route("/api/v1/perforce/{cluster}/trigger")]
		public async Task<ActionResult> TriggerCallbackAsync(string cluster, [FromBody] PerforceTriggerRequest trigger)
		{
			_logger.LogDebug("Received Perforce trigger callback. Type={Type} CL={Changelist} User={User} Root={Root}",
				trigger.TriggerType, trigger.ChangeNumber, trigger.User, trigger.ChangeRoot);

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PublicPerforceController)}.{nameof(TriggerCallbackAsync)}");
			span.SetAttribute("type", trigger.TriggerType);
			span.SetAttribute("cl", trigger.ChangeNumber);
			span.SetAttribute("user", trigger.User);
			span.SetAttribute("root", trigger.ChangeRoot);
			span.SetAttribute("client", trigger.Client);

			switch (trigger.TriggerType)
			{
				case "change-commit":
				case "shelve-commit":
				case "form-save":
					// For "form-save" triggers, change number can be -1 due to variable "formname" is set to "default" (non-submitted changelist)
					if (trigger.ChangeNumber != -1)
					{
						await _perforceService.RefreshCachedCommitAsync(cluster, trigger.ChangeNumber);
						span.SetAttribute("commitRefreshed", true);
					}
					break;
			}

			string content = "{\"message\": \"Trigger received\"}";
			return new ContentResult { ContentType = "application/json", StatusCode = (int)HttpStatusCode.OK, Content = content };
		}
	}
}
