// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.Net;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Perforce;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Diagnostics.HealthChecks;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson.Serialization.Attributes;
using OpenTelemetry.Trace;
using StackExchange.Redis;

namespace HordeServer.VersionControl.Perforce
{
	using Condition = EpicGames.Horde.Common.Condition;

	/// <summary>
	/// Health of a particular server
	/// </summary>
	public enum PerforceServerStatus
	{
		/// <summary>
		/// Server could not be reached
		/// </summary>
		Unknown,

		/// <summary>
		/// Bad. Do not use.
		/// </summary>
		Unhealthy,

		/// <summary>
		/// Degraded, but functioning
		/// </summary>
		Degraded,

		/// <summary>
		/// Good
		/// </summary>
		Healthy,
	}

	/// <summary>
	/// Perforce server infomation
	/// </summary>
	public interface IPerforceServer
	{
		/// <summary>
		/// The server and port
		/// </summary>
		public string ServerAndPort { get; }

		/// <summary>
		/// The server and port before doing DNS resolution
		/// </summary>
		public string BaseServerAndPort { get; }

		/// <summary>
		/// The cluster this server belongs to
		/// </summary>
		public string Cluster { get; }

		/// <summary>
		/// Whether this server supports partitioned workspaces
		/// </summary>
		public bool SupportsPartitionedWorkspaces { get; }

		/// <summary>
		/// Current status
		/// </summary>
		public PerforceServerStatus Status { get; }

		/// <summary>
		/// Number of leases using this server
		/// </summary>
		public int NumLeases { get; }

		/// <summary>
		/// Error message related to this server
		/// </summary>
		public string? Detail { get; }

		/// <summary>
		/// Last update time for this server
		/// </summary>
		public DateTime? LastUpdateTime { get; }
	}

	/// <summary>
	/// Load balancer for Perforce edge servers
	/// </summary>
	public sealed class PerforceLoadBalancer : IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Information about a resolved Perforce server
		/// </summary>
		class PerforceServerEntry : IPerforceServer
		{
			/// <summary>
			/// Resolved host:port for this Perforce server.
			/// Mapping this separately from <see cref="BaseServerAndPort"/> is useful when a DNS name has multiple IP lookups for load balancing
			/// </summary>
			public string ServerAndPort { get; set; } = String.Empty;
			
			/// <inheritdoc cref="PerforceServer.ServerAndPort" />
			public string BaseServerAndPort { get; set; } = String.Empty;
			public string? HealthCheckUrl { get; set; }
			public string Cluster { get; set; } = String.Empty;
			public bool SupportsPartitionedWorkspaces { get; set; }
			public PerforceServerStatus Status { get; set; }
			public string? Detail { get; set; }
			public int NumLeases { get; set; }
			public DateTime? LastUpdateTime { get; set; }

			[BsonConstructor]
			private PerforceServerEntry()
			{
			}

			public PerforceServerEntry(string serverAndPort, string baseServerAndPort, string? healthCheckUrl, string cluster, bool supportsPartitionedWorkspaces, PerforceServerStatus status, string? detail, DateTime? lastUpdateTime)
			{
				ServerAndPort = serverAndPort;
				BaseServerAndPort = baseServerAndPort;
				HealthCheckUrl = healthCheckUrl;
				Cluster = cluster;
				SupportsPartitionedWorkspaces = supportsPartitionedWorkspaces;
				Status = status;
				Detail = detail;
				LastUpdateTime = lastUpdateTime;
			}
		}

		/// <summary>
		/// 
		/// </summary>
		[SingletonDocument("perforce-server-list", "6046aec374a9283100967ee7")]
		class PerforceServerList : SingletonBase
		{
			public List<PerforceServerEntry> Servers { get; set; } = new List<PerforceServerEntry>();
		}

		readonly IRedisService _redisService;
		readonly ILeaseCollection _leaseCollection;
		readonly SingletonDocument<PerforceServerList> _serverListSingleton;
		readonly Random _random = new Random();
		readonly HttpClient _httpClient;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly IHealthMonitor _health;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly ITicker _ticker;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceLoadBalancer(IMongoService mongoService, IRedisService redisService, ILeaseCollection leaseCollection, IClock clock, HttpClient httpClient, IOptionsMonitor<BuildConfig> buildConfig, IHealthMonitor<PerforceLoadBalancer> health, Tracer tracer, ILogger<PerforceLoadBalancer> logger)
		{
			_redisService = redisService;
			_leaseCollection = leaseCollection;
			_serverListSingleton = new SingletonDocument<PerforceServerList>(mongoService);
			_httpClient = httpClient;
			_buildConfig = buildConfig;
			_tracer = tracer;
			_logger = logger;

			if (mongoService.ReadOnlyMode)
			{
				_ticker = new NullTicker();
			}
			else
			{
				_ticker = clock.AddSharedTicker<PerforceLoadBalancer>(TimeSpan.FromMinutes(1.0), TickInternalAsync, logger);
			}

			_health = health;
			_health.SetName("Perforce");
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken) => _ticker.StartAsync();

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken) => _ticker.StopAsync();

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
		}

		/// <summary>
		/// Get the current server list
		/// </summary>
		/// <returns></returns>
		async Task<PerforceServerList> GetServerListAsync(CancellationToken cancellationToken)
		{
			PerforceServerList serverList = await _serverListSingleton.GetAsync(cancellationToken);

			DateTime minLastUpdateTime = DateTime.UtcNow - TimeSpan.FromMinutes(7.5);
			foreach (PerforceServerEntry server in serverList.Servers)
			{
				if (server.Status == PerforceServerStatus.Healthy && server.LastUpdateTime != null && server.LastUpdateTime.Value < minLastUpdateTime)
				{
					server.Status = PerforceServerStatus.Degraded;
					server.Detail = "Server has not responded to health check";
				}
			}

			return serverList;
		}

		/// <summary>
		/// Get the perforce servers
		/// </summary>
		/// <returns></returns>
		public async Task<List<IPerforceServer>> GetServersAsync(CancellationToken cancellationToken)
		{
			PerforceServerList serverList = await GetServerListAsync(cancellationToken);
			return serverList.Servers.ConvertAll<IPerforceServer>(x => x);
		}

		/// <summary>
		/// Allocates a server for use by a lease
		/// </summary>
		/// <returns>The server to use. Null if there is no healthy server available.</returns>
		public async Task<IPerforceServer?> GetServerAsync(string cluster, CancellationToken cancellationToken)
		{
			PerforceServerList serverList = await GetServerListAsync(cancellationToken);

			List<PerforceServerEntry> candidates = serverList.Servers.Where(x => x.Cluster == cluster && x.Status >= PerforceServerStatus.Healthy).ToList();
			if (candidates.Count == 0)
			{
				int idx = _random.Next(0, candidates.Count);
				return candidates[idx];
			}
			return null;
		}

		/// <summary>
		/// Select a Perforce server to use by the Horde server
		/// </summary>
		/// <param name="cluster"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public Task<IPerforceServer?> SelectServerAsync(PerforceCluster cluster, CancellationToken cancellationToken)
		{
			List<string> properties = new List<string> { "HordeServer=1" };
			return SelectServerAsync("server", cluster, properties, cancellationToken);
		}

		/// <summary>
		/// Select a Perforce server to use
		/// </summary>
		/// <param name="cluster"></param>
		/// <param name="agent"></param>
		/// <param name="cancellationToken"></param>
		/// <returns></returns>
		public Task<IPerforceServer?> SelectServerAsync(PerforceCluster cluster, IAgent agent, CancellationToken cancellationToken)
		{
			return SelectServerAsync($"agent:{agent.Id}", cluster, agent.Properties, cancellationToken);
		}

		/// <summary>
		/// Evaluate a server condition using standard Horde syntax
		/// </summary>
		static bool EvaluateCondition(Condition? condition, IReadOnlyList<string> properties)
		{
			if (condition == null)
			{
				return false;
			}
			if (condition.IsEmpty())
			{
				return false;
			}
			if (properties == null)
			{
				throw new ArgumentException("Properties argument to EvaluateCondition may not be null", nameof(properties));
			}
			return condition.Evaluate(x => GetPropertyValues(properties, x));
		}

		/// <summary>
		/// Get property values from a list starting with the given key
		/// </summary>
		static IEnumerable<string> GetPropertyValues(IReadOnlyList<string> properties, string name)
		{
			foreach (string property in properties)
			{
				if (property.Length > name.Length && property[name.Length] == '=' && property.StartsWith(name, StringComparison.OrdinalIgnoreCase))
				{
					yield return property.Substring(name.Length + 1);
				}
			}
		}

		/// <summary>
		/// Select a Perforce server to use
		/// </summary>
		/// <param name="key"></param>
		/// <param name="cluster"></param>
		/// <param name="properties"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public async Task<IPerforceServer?> SelectServerAsync(string key, PerforceCluster cluster, IReadOnlyList<string> properties, CancellationToken cancellationToken)
		{
			// Find all the valid servers for this agent
			List<PerforceServer> validServers = new List<PerforceServer>();
			if (properties != null)
			{
				validServers.AddRange(cluster.Servers.Where(x => (x.Properties != null && x.Properties.Count > 0 && x.Properties.All(y => properties.Contains(y))) || EvaluateCondition(x.Condition, properties)));
			}
			if (validServers.Count == 0)
			{
				validServers.AddRange(cluster.Servers.Where(x => (x.Properties == null || x.Properties.Count == 0) && (x.Condition == null || x.Condition.IsEmpty())));
			}

			HashSet<string> validServerNames = new HashSet<string>(validServers.Select(x => x.ServerAndPort), StringComparer.OrdinalIgnoreCase);

			// Find all the matching servers.
			PerforceServerList serverList = await GetServerListAsync(cancellationToken);

			List<PerforceServerEntry> candidates = serverList.Servers.Where(x => x.Cluster == cluster.Name && validServerNames.Contains(x.BaseServerAndPort)).ToList();
			if (candidates.Count == 0)
			{
				foreach (PerforceServer validServer in validServers)
				{
					_logger.LogDebug("Fetching server info for {ServerAndPort}", validServer.ServerAndPort);
					await UpdateServerAsync(cluster, validServer, candidates, cancellationToken);
				}
				if (candidates.Count == 0)
				{
					_logger.LogWarning("Unable to resolve any Perforce servers from valid list");
					return null;
				}
			}

			// Count number of servers per status before filtering
			Dictionary<string, object> statusCounts = new();
			foreach (PerforceServerStatus status in System.Enum.GetValues<PerforceServerStatus>())
			{
				statusCounts[status.ToString()] = candidates.Count(x => x.Status == status);
			}

			// Remove any servers that are unhealthy
			if (candidates.Any(x => x.Status == PerforceServerStatus.Healthy))
			{
				candidates.RemoveAll(x => x.Status != PerforceServerStatus.Healthy);
			}
			else
			{
				candidates.RemoveAll(x => x.Status != PerforceServerStatus.Unknown);
			}

			if (candidates.Count == 0)
			{
				using IDisposable? logScope = _logger.BeginScope(statusCounts);
				_logger.LogWarning("Unable to find any healthy Perforce server in cluster {ClusterName}", cluster.Name);
				return null;
			}

			// Get the previously selected server for this key
			RedisKey redisKey = new RedisKey($"perforce-lb:{key}");
			string? prevServerAndPort = await _redisService.GetDatabase().StringGetAsync(redisKey);
			PerforceServerEntry? prevCandidate = candidates.FirstOrDefault(x => x.ServerAndPort.Equals(prevServerAndPort, StringComparison.Ordinal));

			// Use the previous server by default, if it's still available. If we don't have one, select which to use with a weighted average of the number of active leases
			PerforceServerEntry? nextCandidate = prevCandidate;
			if (nextCandidate == null)
			{
				int index = 0;

				int totalLeases = candidates.Sum(x => x.NumLeases);
				int totalWeight = candidates.Sum(x => GetWeight(x, prevCandidate, totalLeases));

				int weight = _random.Next(totalWeight);
				for (; index + 1 < candidates.Count; index++)
				{
					weight -= GetWeight(candidates[index], prevCandidate, totalLeases);
					if (weight < 0)
					{
						break;
					}
				}

				nextCandidate = candidates[index];
			}

			// Update redis with the chosen candidate. Only allow selecting a new server every day.
			await _redisService.GetDatabase().StringSetAsync(redisKey, nextCandidate.ServerAndPort, expiry: TimeSpan.FromDays(1.0), keepTtl: true, flags: CommandFlags.FireAndForget);

			if (prevCandidate == null)
			{
				_logger.LogDebug("Adding new preferred server for {Key}: {ServerAndPort}", key, nextCandidate.ServerAndPort);
			}
			else if (prevCandidate == nextCandidate)
			{
				_logger.LogDebug("Reusing preferred server for {Key}: {ServerAndPort}", key, nextCandidate.ServerAndPort);
			}
			else
			{
				_logger.LogDebug("Changing preferred server for {Key}: {PrevServerAndPort} -> {NextServerAndPort}", key, prevCandidate.ServerAndPort, nextCandidate.ServerAndPort);
			}

			return nextCandidate;
		}

		static int GetWeight(PerforceServerEntry candidate, PerforceServerEntry? prevCandidate, int totalLeases)
		{
			int baseWeight;
			if (candidate == prevCandidate)
			{
				baseWeight = 200;
			}
			else
			{
				baseWeight = 20;
			}
			return baseWeight + (totalLeases - candidate.NumLeases);
		}
		
		internal ValueTask TickInternalTestOnlyAsync(CancellationToken cancellationToken)
		{
			return TickInternalAsync(cancellationToken);
		}
		
		async ValueTask TickInternalAsync(CancellationToken cancellationToken)
		{
			_logger.LogDebug("Ticking server health status");
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceLoadBalancer)}.{nameof(TickInternalAsync)}");

			BuildConfig buildConfig = _buildConfig.CurrentValue;

			// Set of new server entries
			List<PerforceServerEntry> newServers = [];

			// Update the state of all the valid servers
			foreach (PerforceCluster cluster in buildConfig.PerforceClusters)
			{
				foreach (PerforceServer server in cluster.Servers)
				{
					try
					{
						await UpdateServerAsync(cluster, server, newServers, cancellationToken);
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while updating Perforce server status");
					}
				}
			}

			// Update the number of leases for each entry
			List<PerforceServerEntry> newServerEntries = newServers.OrderBy(x => x.Cluster).ThenBy(x => x.BaseServerAndPort).ThenBy(x => x.ServerAndPort).ToList();
			await UpdateLeaseCountsAsync(newServerEntries, cancellationToken);
			PerforceServerList list = await _serverListSingleton.UpdateAsync(currentServerList => MergeServerList(currentServerList, newServerEntries), cancellationToken);
			span.SetAttribute("numServers", list.Servers.Count);

			// Run health checks for all Perforce servers in parallel
			await Task.WhenAll(list.Servers.Select(server => Task.Run(async () => { await UpdateHealthGuardedAsync(server, cancellationToken); }, cancellationToken)));

			// Update Horde's internal health monitor for Perforce subsystem
			list = await _serverListSingleton.GetAsync(cancellationToken);
			(HealthStatus health, string? message) = GetPerforceHealth(list.Servers);
			span.SetAttribute("health.status", health.ToString());
			span.SetAttribute("health.message", message ?? String.Empty);
			await _health.UpdateAsync(health, message);
		}

		static (HealthStatus health, string? message) GetPerforceHealth(List<PerforceServerEntry> servers)
		{
			HealthStatus result = HealthStatus.Healthy;
			foreach (PerforceServerEntry server in servers)
			{
				if (server.Status != PerforceServerStatus.Unknown)
				{
					HealthStatus serverHealth = server.Status switch
					{
						PerforceServerStatus.Unknown => HealthStatus.Degraded,
						PerforceServerStatus.Unhealthy => HealthStatus.Unhealthy,
						PerforceServerStatus.Degraded => HealthStatus.Degraded,
						PerforceServerStatus.Healthy => HealthStatus.Healthy,
						_ => throw new ArgumentOutOfRangeException($"Unknown health status: {server.Status}")
					};
					result = serverHealth < result ? serverHealth : result;
				}
			}

			string? message = result switch
			{
				HealthStatus.Unhealthy => "One or more Perforce servers are unhealthy. Check Perforce servers page for details.",
				HealthStatus.Degraded => "One or more Perforce servers are degraded. Check Perforce servers page for details.",
				HealthStatus.Healthy => null,
				_ => throw new ArgumentOutOfRangeException($"Unknown health status: {result}")
			};

			return (result, message);
		}

		static void MergeServerList(PerforceServerList serverList, List<PerforceServerEntry> newEntries)
		{
			Dictionary<string, PerforceServerEntry> existingEntries = new (StringComparer.OrdinalIgnoreCase);
			foreach (PerforceServerEntry existingEntry in serverList.Servers)
			{
				existingEntries[existingEntry.ServerAndPort] = existingEntry;
			}
			foreach (PerforceServerEntry newEntry in newEntries)
			{
				if (existingEntries.TryGetValue(newEntry.ServerAndPort, out PerforceServerEntry? existingEntry))
				{
					newEntry.Status = existingEntry.Status;
					newEntry.Detail = existingEntry.Detail;
					newEntry.LastUpdateTime = existingEntry.LastUpdateTime;
				}
			}
			serverList.Servers = newEntries;
		}

		async Task UpdateLeaseCountsAsync(IEnumerable<PerforceServerEntry> newServerEntries, CancellationToken cancellationToken)
		{
			Dictionary<string, Dictionary<string, PerforceServerEntry>> newServerLookup = new Dictionary<string, Dictionary<string, PerforceServerEntry>>();
			foreach (PerforceServerEntry newServerEntry in newServerEntries)
			{
				if (newServerEntry.Cluster != null)
				{
					Dictionary<string, PerforceServerEntry>? clusterServers;
					if (!newServerLookup.TryGetValue(newServerEntry.Cluster, out clusterServers))
					{
						clusterServers = new Dictionary<string, PerforceServerEntry>();
						newServerLookup.Add(newServerEntry.Cluster, clusterServers);
					}
					clusterServers[newServerEntry.ServerAndPort] = newServerEntry;
				}
			}

			IReadOnlyList<ILease> leases = await _leaseCollection.FindActiveLeasesAsync(cancellationToken: cancellationToken);
			foreach (ILease lease in leases)
			{
				Any any = Any.Parser.ParseFrom(lease.Payload.ToArray());

				HashSet<(string, string)> servers = new HashSet<(string, string)>();
				if (any.TryUnpack(out ConformTask conformTask))
				{
					foreach (RpcAgentWorkspace workspace in conformTask.Workspaces)
					{
						servers.Add((workspace.Cluster, workspace.ServerAndPort));
					}
				}
				if (any.TryUnpack(out ExecuteJobTask executeJobTask))
				{
					if (executeJobTask.AutoSdkWorkspace != null)
					{
						servers.Add((executeJobTask.AutoSdkWorkspace.Cluster, executeJobTask.AutoSdkWorkspace.ServerAndPort));
					}
					if (executeJobTask.Workspace != null)
					{
						servers.Add((executeJobTask.Workspace.Cluster, executeJobTask.Workspace.ServerAndPort));
					}
				}

				foreach ((string cluster, string serverAndPort) in servers)
				{
					IncrementLeaseCount(cluster, serverAndPort, newServerLookup);
				}
			}
		}

		static void IncrementLeaseCount(string cluster, string serverAndPort, Dictionary<string, Dictionary<string, PerforceServerEntry>> newServers)
		{
			Dictionary<string, PerforceServerEntry>? clusterServers;
			if (cluster != null && newServers.TryGetValue(cluster, out clusterServers))
			{
				PerforceServerEntry? entry;
				if (clusterServers.TryGetValue(serverAndPort, out entry))
				{
					entry.NumLeases++;
				}
			}
		}

		async Task UpdateServerAsync(PerforceCluster cluster, PerforceServer server, List<PerforceServerEntry> newServers, CancellationToken cancellationToken)
		{
			string initialHostName = server.ServerAndPort;
			int port = 1666;

			int portIdx = initialHostName.LastIndexOf(':');
			if (portIdx != -1)
			{
				port = Int32.Parse(initialHostName.Substring(portIdx + 1), NumberStyles.Integer, CultureInfo.InvariantCulture);
				initialHostName = initialHostName.Substring(0, portIdx);
			}

			List<string> hostNames = [];
			if (!server.ResolveDns || IPAddress.TryParse(initialHostName, out IPAddress? _))
			{
				hostNames.Add(initialHostName);
			}
			else
			{
				await ResolveDnsHostnameAsync(initialHostName, hostNames, cancellationToken);
			}

			foreach (string hostName in hostNames)
			{
				string? healthCheckUrl = null;
				if (server.HealthCheck)
				{
					healthCheckUrl = $"http://{hostName}:5000/healthcheck";
				}
				newServers.Add(new PerforceServerEntry($"{hostName}:{port}", server.ServerAndPort, healthCheckUrl, cluster.Name, cluster.SupportsPartitionedWorkspaces, PerforceServerStatus.Unknown, "", DateTime.UtcNow));
			}
		}

		/// <summary>
		/// Find all the IP addresses for a hostname
		/// </summary>
		/// <param name="hostName">DNS hostname</param>
		/// <param name="ipAddresses">List of IP addresses</param>
		/// <param name="cancellationToken">Cancellation token</param>
		async Task ResolveDnsHostnameAsync(string hostName, List<string> ipAddresses, CancellationToken cancellationToken)
		{
			IPHostEntry entry = await Dns.GetHostEntryAsync(hostName, cancellationToken);
			foreach (IPAddress address in entry.AddressList)
			{
				try
				{
					ipAddresses.Add(address.ToString());
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Unable to resolve host name for Perforce server {Address}", address);
				}
			}
		}

		async Task UpdateHealthGuardedAsync(PerforceServerEntry entry, CancellationToken cancellationToken)
		{
			try
			{
				await UpdateHealthAsync(entry, cancellationToken);
			}
			catch (OperationCanceledException)
			{
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error updating health for {ServerAndPort}: {Message}", entry.ServerAndPort, ex.Message);
			}
		}

		/// <summary>
		/// Checks and updates the health of a Perforce server
		/// </summary>
		async Task UpdateHealthAsync(PerforceServerEntry entry, CancellationToken cancellationToken)
		{
			ServerHealth health = await GetServerHealthViaP4InfoAsync(entry.BaseServerAndPort, _logger, cancellationToken);
			if (entry.HealthCheckUrl != null)
			{
				ServerHealth http = await GetServerHealthViaHttpAsync(entry.HealthCheckUrl, cancellationToken);
				if (http.Status != PerforceServerStatus.Unknown && http.Status < health.Status)
				{
					// Unhealthy HTTP status will override any p4 info-based status
					health = new ServerHealth(http.Status, http.Detail);
				}
			}

			await _serverListSingleton.UpdateAsync(x => UpdateHealth(x, entry.ServerAndPort, health.Status, health.Detail, DateTime.UtcNow), cancellationToken);
		}

		internal async Task UpdateHealthTestOnlyAsync(string cluster, string serverAndPort, string? healthCheckUrl, CancellationToken cancellationToken)
		{
			PerforceServerEntry pse = new(serverAndPort, serverAndPort, healthCheckUrl, cluster, false, PerforceServerStatus.Unknown, null, null);
			await _serverListSingleton.UpdateAsync((x) => x.Servers = [pse], cancellationToken);
			await UpdateHealthAsync(pse, cancellationToken);
		}

		static void UpdateHealth(PerforceServerList serverList, string serverAndPort, PerforceServerStatus status, string detail, DateTime? updateTime)
		{
			PerforceServerEntry? entry = serverList.Servers.FirstOrDefault(x => x.ServerAndPort.Equals(serverAndPort, StringComparison.Ordinal));
			if (entry != null)
			{
				if (entry.LastUpdateTime == null || updateTime == null || entry.LastUpdateTime.Value < updateTime.Value)
				{
					entry.Status = status;
					entry.Detail = detail;
					entry.LastUpdateTime = updateTime;
				}
			}
		}

		internal record ServerHealth(PerforceServerStatus Status, string Detail);

		/// <summary>
		/// Queries the server's associated health check URL for status
		/// </summary>
		/// <param name="healthCheckUrl">URL to query</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Status as reported by the URL response</returns>
		async Task<ServerHealth> GetServerHealthViaHttpAsync(string healthCheckUrl, CancellationToken cancellationToken)
		{
			try
			{
				Uri healthCheckUri = new(healthCheckUrl);
				HttpResponseMessage response = await _httpClient.GetAsync(healthCheckUri, cancellationToken);

				byte[] data = await response.Content.ReadAsByteArrayAsync(cancellationToken);
				JsonDocument document = JsonDocument.Parse(data);

				foreach (JsonElement element in document.RootElement.GetProperty("results").EnumerateArray())
				{
					if (element.TryGetProperty("checker", out JsonElement checker) && checker.ValueEquals("edge_traffic_lights"))
					{
						if (element.TryGetProperty("output", out JsonElement output))
						{
							string status = output.GetString() ?? String.Empty;
							return status switch
							{
								"green" => new ServerHealth(PerforceServerStatus.Healthy, "Server is healthy"),
								"yellow" => new ServerHealth(PerforceServerStatus.Degraded, "Degraded service"),
								"red" => new ServerHealth(PerforceServerStatus.Unhealthy, "Server is being drained"),
								_ => new ServerHealth(PerforceServerStatus.Unknown, $"Expected state for health check ({status})")
							};
						}
					}
				}

				string dataStr = Encoding.UTF8.GetString(data).Substring(0, 200);
				_logger.LogWarning("Unable to parse health check output. Data {Data} URL {HealthCheckUrl}", dataStr, healthCheckUrl);
				return new ServerHealth(PerforceServerStatus.Unknown, "Unable to parse health check output");
			}
			catch (Exception ex) when (ex is not OperationCanceledException)
			{
				_logger.LogWarning("Failed checking Perforce server health via HTTP. URL {HealthCheckUrl} Reason {Reason}", healthCheckUrl, ex.Message);
				return new ServerHealth(PerforceServerStatus.Unhealthy, $"Failed to query status at {healthCheckUrl}");
			}
		}

		/// <summary>
		/// Checks the health of given Perforce server by invoking the equivalent of "p4 info"
		/// </summary>
		/// <param name="serverAndPort">Perforce server hostname with port</param>
		/// <param name="logger">Logger</param>
		/// <param name="cancellationToken"></param>
		/// <returns>Health status of server</returns>
		internal static async Task<ServerHealth> GetServerHealthViaP4InfoAsync(string serverAndPort, ILogger logger, CancellationToken cancellationToken)
		{
			PerforceSettings settings = new(serverAndPort, "") { AppName = "Horde.Server", PreferNativeClient = true };

			try
			{
				using IPerforceConnection connection = await PerforceConnection.CreateAsync(settings, logger);
				await connection.GetInfoAsync(InfoOptions.ShortOutput, cancellationToken);
				return new ServerHealth(PerforceServerStatus.Healthy, "Server responded to \"p4 info\"");
			}
			catch (Exception ex) when (ex is not OperationCanceledException)
			{
				logger.LogWarning("Failed checking Perforce server health via info command. Reason {Reason}", ex.Message);
				return new ServerHealth(PerforceServerStatus.Unhealthy, "Server failed \"p4 info\" query");
			}
		}
	}
}
