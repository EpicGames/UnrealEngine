// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Users;
using EpicGames.Perforce;
using EpicGames.Redis;
using HordeServer.Server;
using HordeServer.Streams;
using HordeServer.Telemetry;
using HordeServer.Users;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Bson.Serialization.Options;
using MongoDB.Driver;
using OpenTelemetry.Trace;
using StackExchange.Redis;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Service which mirrors changes from Perforce
	/// </summary>
	class PerforceServiceCache : PerforceService, IHostedService
	{
		[SingletonDocument("commit-cache")]
		class CacheState : SingletonBase
		{
			[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<string, ClusterState> Clusters { get; set; } = new Dictionary<string, ClusterState>();
		}

		class ClusterState
		{
			public int MaxChange { get; set; }

			[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<StreamId, IoHash> Streams { get; set; } = new Dictionary<StreamId, IoHash>();

			[BsonDictionaryOptions(DictionaryRepresentation.ArrayOfDocuments)]
			public Dictionary<StreamId, int> MinChanges { get; set; } = new Dictionary<StreamId, int>();
		}

		internal class CommitTagInfo
		{
			public CommitTag Name { get; }
			public FileFilter Filter { get; }

			public CommitTagInfo(CommitTag name, FileFilter filter)
			{
				Name = name;
				Filter = filter;
			}
		}

		internal class StreamInfo
		{
			public StreamConfig StreamConfig { get; set; }
			public PerforceViewMap View { get; }
			public PerforceChangeView ChangeView { get; }
			public IoHash Hash { get; }

			public IReadOnlyList<CommitTagInfo> CommitTags { get; }

			public StreamInfo(StreamConfig streamConfig, PerforceViewMap view, PerforceChangeView changeView)
			{
				StreamConfig = streamConfig;
				View = view;
				ChangeView = changeView;

				List<CommitTagInfo> commitTags = new List<CommitTagInfo>();
				foreach (CommitTagConfig commitTagConfig in streamConfig.GetAllCommitTags())
				{
					if (streamConfig.TryGetCommitTagFilter(commitTagConfig.Name, out FileFilter? filter))
					{
						commitTags.Add(new CommitTagInfo(commitTagConfig.Name, filter));
					}
				}
				CommitTags = commitTags;

				using (StringWriter writer = new StringWriter())
				{
					writer.WriteLine("View");
					foreach (PerforceViewMapEntry entry in view.Entries)
					{
						writer.WriteLine($"  {entry.Include}|{entry.Source}|{entry.Target}");
					}

					writer.WriteLine("ChangeView");
					foreach (PerforceChangeViewEntry entry in changeView.Entries)
					{
						writer.WriteLine($"  {entry.Path}|{entry.Change}");
					}

					Hash = IoHash.Compute(Encoding.UTF8.GetBytes(writer.ToString()));
				}
			}
		}

		sealed class ClusterTicker : IAsyncDisposable
		{
			public string ClusterName { get; }
			public BackgroundTask? Task { get; set; }

			// Volatile ensures the reference assignment is immediately visible to reader threads. The list contents are never mutated after assignment.
			private volatile List<StreamInfo> _streams;
			public List<StreamInfo> Streams
			{
				get => _streams;
				set => _streams = value;
			}

			public ClusterTicker(string name, List<StreamInfo> streamInfos)
			{
				ClusterName = name;
				_streams = streamInfos;
			}

			public async ValueTask DisposeAsync()
			{
				if (Task != null)
				{
					await Task.DisposeAsync();
					Task = null;
				}
			}
		}

		class CachedCommitDoc : ICommit
		{
			[BsonIgnore]
			PerforceServiceCache _owner = null!;

			[BsonIgnore]
			StreamConfig _streamConfig = null!;

			[BsonIgnoreIfDefault] // Allow upserts
			public ObjectId Id { get; set; }

			public StreamId StreamId { get; set; }

			public int Number { get; set; }
			CommitIdWithOrder ICommit.Id => CommitIdWithOrder.FromPerforceChange(Number);

			public int OriginalChange { get; set; }
			CommitIdWithOrder ICommit.OriginalCommitId => CommitIdWithOrder.FromPerforceChange(OriginalChange);

			public UserId AuthorId { get; set; }
			public UserId OwnerId { get; set; }
			public string Description { get; set; }
			public string BasePath { get; set; }
			public DateTime DateUtc { get; set; }

			public List<CommitTag> CommitTags { get; set; } = [];
			public List<string> Files { get; set; } = [];

			[BsonElement("haf")]
			public bool HasAllFiles { get; set; } = true;

			[BsonConstructor]
			CachedCommitDoc()
			{
				Description = null!;
				BasePath = null!;

				CommitTags = [];
				Files = [];
			}

			public CachedCommitDoc(ICommit commit, List<CommitTag> commitTags, List<string> files, bool hasAllFiles)
			{
				StreamId = commit.StreamId;
				Number = commit.Id.GetPerforceChange();
				OriginalChange = commit.OriginalCommitId.GetPerforceChange();
				AuthorId = commit.AuthorId;
				OwnerId = commit.OwnerId;
				Description = commit.Description;

				const int MaxDescriptionLength = 128 * 1024; // 128k characters is a pretty big limit, but we've seen automated processes submit changes with larger :(
				if (Description.Length > MaxDescriptionLength)
				{
					Description = Description.Substring(0, MaxDescriptionLength);
				}

				BasePath = commit.BasePath;
				DateUtc = commit.DateUtc;

				CommitTags = commitTags;
				Files = files;
				HasAllFiles = hasAllFiles;
			}

			public static async Task<CachedCommitDoc> FromCommitAsync(ICommit commit, CancellationToken cancellationToken)
			{
				List<CommitTag> commitTags = new(await commit.GetTagsAsync(cancellationToken));
				List<string> files = new(await commit.GetFilesAsync(null, MaxFiles, cancellationToken));
				bool hasAllFiles = files.Count < MaxFiles;
				return new CachedCommitDoc(commit, commitTags, files, hasAllFiles);
			}

			public void PostLoad(PerforceServiceCache owner, StreamConfig streamConfig)
			{
				_owner = owner;
				_streamConfig = streamConfig;
			}

			public ValueTask<IReadOnlyList<CommitTag>> GetTagsAsync(CancellationToken cancellationToken)
			{
				return new ValueTask<IReadOnlyList<CommitTag>>(CommitTags);
			}

			public async ValueTask<bool> MatchesFilterAsync(FileFilter filter, CancellationToken cancellationToken)
			{
				if (HasAllFiles)
				{
					return filter.ApplyTo(Files).Any();
				}

				IReadOnlyList<string> files = await GetFilesAsync(null, null, cancellationToken);
				return filter.ApplyTo(files).Any();
			}

			public async ValueTask<IReadOnlyList<string>> GetFilesAsync(int? minFiles, int? maxFiles, CancellationToken cancellationToken)
			{
				if (HasAllFiles)
				{
					return Files;
				}

				// The cached commit document does not have a complete list of files for the CL because it exceeded max files to be saved
				// Query Perforce server for the complete list and return that, but do not store it
				using IPooledPerforceConnection perforce = await _owner.ConnectAsync(_streamConfig.ClusterName, null, cancellationToken);
				DescribeRecord describeRecord = await perforce.DescribeAsync(DescribeOptions.None, maxFiles ?? -1, Number, cancellationToken);
				return await perforce.GetStreamFilesAsync(_streamConfig, describeRecord, cancellationToken);
			}
		}

		readonly IMongoService _mongoService;
		readonly IRedisService _redisService;
		readonly IDowntimeService _downtimeService;
		readonly IMongoCollection<CachedCommitDoc> _commits;
		readonly IOptionsMonitor<BuildConfig> _buildConfig;
		readonly ITelemetryWriter _telemetryWriter;
		readonly Tracer _tracer;
		readonly ILogger _logger;

		static readonly RedisChannel<StreamId> s_commitUpdateChannel = new RedisChannel<StreamId>(RedisChannel.Literal("commit-update"));

		readonly ITicker _updateCommitsTicker;

		/// <summary>
		/// Constructor
		/// </summary>
		public PerforceServiceCache(PerforceLoadBalancer loadBalancer, IMongoService mongoService, IRedisService redisService, IDowntimeService downtimeService, IUserCollection userCollection, IClock clock, IOptions<BuildServerConfig> settings, IOptionsMonitor<BuildConfig> buildConfig, ITelemetryWriter telemetryWriter, Tracer tracer, ILogger<PerforceService> logger)
			: base(loadBalancer, userCollection, settings, buildConfig, tracer, logger)
		{
			_mongoService = mongoService;
			_redisService = redisService;
			_downtimeService = downtimeService;

			List<MongoIndex<CachedCommitDoc>> indexes = new List<MongoIndex<CachedCommitDoc>>();
			indexes.Add(MongoIndex.Create<CachedCommitDoc>(keys => keys.Ascending(x => x.StreamId).Descending(x => x.Number), true));
			indexes.Add(MongoIndex.Create<CachedCommitDoc>(keys => keys.Ascending(x => x.StreamId).Ascending(x => x.CommitTags).Descending(x => x.Number), true));
			_commits = mongoService.GetCollection<CachedCommitDoc>("CommitsV3", indexes);

			_buildConfig = buildConfig;
			_telemetryWriter = telemetryWriter;
			_tracer = tracer;
			_logger = logger;

			_updateCommitsTicker = clock.AddSharedTicker<PerforceServiceCache>(TimeSpan.FromSeconds(10.0), UpdateCommitsAsync, logger);
		}

		/// <inheritdoc/>
		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			await _updateCommitsTicker.DisposeAsync();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _updateCommitsTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _updateCommitsTicker.StopAsync();
		}

		#region Commit updates

		/// <summary>
		/// Polls Perforce for submitted changes
		/// </summary>
		async ValueTask UpdateCommitsAsync(CancellationToken cancellationToken)
		{
			Dictionary<string, ClusterTicker> clusterToTicker = new Dictionary<string, ClusterTicker>();
			try
			{
				_logger.LogDebug("Starting commit metadata replication");
				for (; ; )
				{
					if (!_downtimeService.IsDowntimeActive)
					{
						Dictionary<string, List<StreamInfo>> clusterToStreams = await CreateStreamInfoAsync(cancellationToken);

						// Stop any tickers that no longer exist
						foreach (ClusterTicker ticker in clusterToTicker.Values.ToArray())
						{
							if (!clusterToStreams.ContainsKey(ticker.ClusterName))
							{
								await StopTickerAsync(ticker);
								clusterToTicker.Remove(ticker.ClusterName);
							}
						}

						// Start any new tickers that don't exist yet, or update existing ones
						foreach ((string clusterName, List<StreamInfo> streamInfos) in clusterToStreams)
						{
							if (!clusterToTicker.ContainsKey(clusterName))
							{
								ClusterTicker ticker = StartTicker(clusterName, streamInfos);
								clusterToTicker.Add(ticker.ClusterName, ticker);
							}
							else
							{
								// Update existing ticker's stream list (includes refreshed Views from P4 stream changes)
								if (!clusterToTicker[clusterName].Streams.Select(x => x.Hash).SequenceEqual(streamInfos.Select(x => x.Hash)))
							{
								_logger.LogInformation("Refreshing stream definitions for cluster {ClusterName}", clusterName);
							}
								clusterToTicker[clusterName].Streams = streamInfos;
							}
						}
					}

					// Wait before performing the next update
					await Task.Delay(TimeSpan.FromSeconds(30.0), cancellationToken);
				}
			}
			finally
			{
				_logger.LogDebug("Stopping commit metadata replication");
				foreach (ClusterTicker ticker in clusterToTicker.Values)
				{
					await StopTickerAsync(ticker);
				}
			}
		}

		async Task<Dictionary<string, List<StreamInfo>>> CreateStreamInfoAsync(CancellationToken cancellationToken)
		{
			IReadOnlyList<StreamConfig> streams = _buildConfig.CurrentValue.Streams;

			Dictionary<string, List<StreamInfo>> clusters = new(StringComparer.OrdinalIgnoreCase);
			foreach (IGrouping<string, StreamConfig> group in streams.Where(x => x.VCS.Equals("Perforce", StringComparison.OrdinalIgnoreCase)).GroupBy(x => x.ClusterName, StringComparer.OrdinalIgnoreCase))
			{
				try
				{
					List<StreamInfo> streamInfoList = await CreateStreamInfoForClusterAsync(group.Key, group, cancellationToken);
					clusters[group.Key] = streamInfoList;
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Failed resolving stream infos for Perforce cluster {ClusterId}", group.Key);
				}
			}

			return clusters;
		}

		async Task<List<StreamInfo>> CreateStreamInfoForClusterAsync(string clusterName, IEnumerable<StreamConfig> streams, CancellationToken cancellationToken)
		{
			using IPooledPerforceConnection perforce = await ConnectAsync(clusterName, null, cancellationToken);
			InfoRecord serverInfo = await perforce.GetInfoAsync(cancellationToken);

			List<StreamInfo> streamInfoList = [];
			foreach (StreamConfig stream in streams)
			{
				StreamRecord record = await perforce.GetStreamAsync(stream.Name, true, cancellationToken);
				PerforceViewMap view = PerforceViewMap.Parse(record.View);
				PerforceChangeView changeView = PerforceChangeView.Parse(record.ChangeView, !serverInfo.IsCaseSensitive);
				streamInfoList.Add(new StreamInfo(stream, view, changeView));
			}
			return streamInfoList;
		}

		ClusterTicker StartTicker(string clusterName, List<StreamInfo> streamInfos)
		{
			ClusterTicker ticker = new ClusterTicker(clusterName, streamInfos);
			ticker.Task = BackgroundTask.StartNew(ctx => UpdateClusterAsync(ticker, ctx));
			return ticker;
		}

		async ValueTask StopTickerAsync(ClusterTicker ticker)
		{
			try
			{
				await ticker.DisposeAsync();
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Exception while disposing ticker for cluster '{ClusterName}': {Message}", ticker.ClusterName, ex.Message);
			}
		}

		async Task UpdateClusterAsync(ClusterTicker ticker, CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				// Update the cluster state
				if (!_downtimeService.IsDowntimeActive)
				{
					try
					{
						using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceServiceCache)}.{nameof(UpdateClusterAsync)}");
						span.SetAttribute("ClusterName", ticker.ClusterName);
						await UpdateClusterInternalAsync(ticker.ClusterName, ticker.Streams, span.Context, cancellationToken);
					}
					catch (OperationCanceledException)
					{
						throw;
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while updating cluster state: {Message}", ex.Message);
					}
				}

				// Wait before polling again
				await Task.Delay(TimeSpan.FromSeconds(2.0), cancellationToken);
			}
		}

		internal async Task UpdateClusterInternalAsync(string clusterName, List<StreamInfo> streamInfos, SpanContext? parentContext, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(PerforceServiceCache)}.{nameof(UpdateClusterInternalAsync)}", parentContext: parentContext ?? default);

			// Get the current state of this cluster
			CacheState cacheState = await _mongoService.GetSingletonAsync<CacheState>(cancellationToken);

			ClusterState? clusterState;
			if (!cacheState.Clusters.TryGetValue(clusterName, out clusterState))
			{
				clusterState = new ClusterState();
			}

			// Update the commits
			ClusterState? nextClusterState = await UpdateClusterCommitsAsync(clusterName, streamInfos, clusterState, span.Context, cancellationToken);
			if (nextClusterState != null)
			{
				await _mongoService.UpdateSingletonAsync<CacheState>(state => state.Clusters[clusterName] = nextClusterState, cancellationToken);
			}
		}

		async Task<ClusterState?> UpdateClusterCommitsAsync(string clusterName, List<StreamInfo> streamInfos, ClusterState state, SpanContext parentContext, CancellationToken cancellationToken)
		{
			using TelemetrySpan telemetrySpan = _tracer.StartActiveSpan($"{nameof(PerforceServiceCache)}.{nameof(UpdateClusterCommitsAsync)}", parentContext: parentContext);
			using (IPooledPerforceConnection perforce = await ConnectAsync(clusterName, null, cancellationToken))
			{
				const int MaxChanges = 250;

				// If the hash of any stream definition has changed, invalidate the replicated changes.
				bool modified = false;
				foreach (StreamInfo streamInfo in streamInfos)
				{
					IoHash prevHash;
					if (state.Streams.TryGetValue(streamInfo.StreamConfig.Id, out prevHash) && prevHash != streamInfo.Hash)
					{
						_logger.LogInformation("Invalidating cached commits for stream {StreamId} due to definition change ({OldHash} -> {NewHash})", streamInfo.StreamConfig.Id, prevHash, streamInfo.Hash);
						state.MinChanges.Remove(streamInfo.StreamConfig.Id);
						modified = true;
					}
				}

				// Update the new hashes
				state.Streams = streamInfos.ToDictionary(x => x.StreamConfig.Id, x => x.Hash);

				// Remove any changes we need to update
				int[] refreshNumbers = await _redisService.GetDatabase().SetPopAsync(GetRefreshSetKey(clusterName), 100);

				// Get the changelist range to query
				FileSpecList spec = FileSpecList.Any;
				if (state.MaxChange > 0)
				{
					spec = $"@{state.MaxChange + 1},@now";
				}

				// Find the changes within that range, and abort if there's nothing new
				List<ChangesRecord> changes = await perforce.GetChangesAsync(ChangesOptions.None, MaxChanges, ChangeStatus.Submitted, spec, cancellationToken);
				if (changes.Count > 0)
				{
					telemetrySpan.SetAttribute("MinChange", changes.Min(x => x.Number));
					telemetrySpan.SetAttribute("MaxChange", changes.Max(x => x.Number));
				}

				List<int> changeNumbers = new List<int>();
				changeNumbers.AddRange(refreshNumbers.Where(x => x <= state.MaxChange));
				changeNumbers.AddRange(changes.Select(x => x.Number));

				telemetrySpan.SetAttribute("NumChanges", changes.Count);

				if (changeNumbers.Count == 0)
				{
					return modified ? state : null;
				}

				_logger.LogDebug("Replicating {NumChanges} changes for cluster {ClusterName} matching {FileSpec}, {NumUpdatedChanges} changes to refresh", changes.Count, clusterName, spec, refreshNumbers.Length);

				// If we've retrieved the maximum number of changes from the server, we no longer have a complete chronological cache and need to reset it.
				bool reset = false;
				if (changes.Count >= MaxChanges)
				{
					_logger.LogWarning("Too many commits since last P4 cache update (query: {Spec}). Resetting cache for {Cluster}.", spec, clusterName);
					reset = true;
				}

				// Get the server information
				InfoRecord info = await perforce.GetInfoAsync(cancellationToken);

				// Create a buffer for files
				List<string> files = new List<string>();

				// Describe the changes and create records for them
				PerforceResponseList<DescribeRecord> describeRecordResponses = await perforce.TryDescribeAsync(DescribeOptions.None, MaxFiles, changeNumbers.ToArray(), cancellationToken);
				foreach (StreamInfo streamInfo in streamInfos)
				{
					foreach (PerforceResponse<DescribeRecord> describeRecordResponse in describeRecordResponses)
					{
						if (!describeRecordResponse.Succeeded)
						{
							// We can receive trigger notifications for modified changes that no longer exist. Ignore them.
							continue;
						}

						DescribeRecord describeRecord = describeRecordResponse.Data;
						if (describeRecord.Status != ChangeStatus.Submitted)
						{
							// This can happen because we received a P4 trigger notifying us of a form save. Need to filter out non-committed changes.
							continue;
						}

						files.Clear();
						foreach (DescribeFileRecord describeFile in describeRecord.Files)
						{
							string relativePath;
							if (streamInfo.View.TryMapFile(describeFile.DepotFile, info.PathComparison, out relativePath))
							{
								if (streamInfo.ChangeView.IsVisible(describeFile.DepotFile, describeRecord.Number))
								{
									files.Add(relativePath);
								}
							}
						}

						if (files.Count > 0)
						{
							try
							{
								bool hasAllFiles = describeRecord.Files.Count < MaxFiles;

								ICommit commit = await CreateCommitAsync(perforce, streamInfo.StreamConfig, describeRecord, hasAllFiles, info, cancellationToken, files.ToList());
								_logger.LogInformation("Replicating {StreamId} commit {Change}", streamInfo.StreamConfig.Id, describeRecord.Number);

								IReadOnlyList<CommitTag> commitTags = await commit.GetTagsAsync(cancellationToken);
								List<string> cappedFiles = files.Take(MaxFiles).ToList();
								CachedCommitDoc commitDoc = new(commit, commitTags.ToList(), cappedFiles, hasAllFiles);
								await AddCachedCommitAsync(commitDoc, cancellationToken);

								await _redisService.PublishAsync(s_commitUpdateChannel, streamInfo.StreamConfig.Id, CommandFlags.FireAndForget);

								// Emit telemetry
								if (!streamInfo.StreamConfig.TelemetryStoreId.IsEmpty)
								{
									try
									{
										CommitTelemetry newTelemetry = await CommitTelemetry.CreateCommitTelemetryWithUserIdAsync(UserCollection, commit, CommitDescriptionTagContainer.Create(commitDoc.Description));
										_telemetryWriter.WriteEvent(streamInfo.StreamConfig.TelemetryStoreId, newTelemetry);
									}
									catch (Exception ex)
									{
										_logger.LogError(ex, "Failed to publish Commit telemetry.");
									}
								}
							}
							catch (PerforceException ex)
							{
								// We want to make sure a single perforce exception doesn't create batch abandonment for other streams, and records.
								// We are currently keeping this narrow to PerforceException to prevent too wide a capture, in the event there is something more systemic.
								_logger.LogError(ex, "Skipping CL {Change} for stream {StreamId}: {Message}", describeRecord.Number, streamInfo.StreamConfig.Id, ex.Message);
							}
						}
					}
				}

				// Update the cache state
				if (changes.Count > 0)
				{
					Dictionary<StreamId, int> minChanges = new Dictionary<StreamId, int>(streamInfos.Count);
					foreach (StreamInfo streamInfo in streamInfos)
					{
						int minChange;
						if (reset || !state.MinChanges.TryGetValue(streamInfo.StreamConfig.Id, out minChange))
						{
							minChange = changes[^1].Number;
						}
						minChanges.Add(streamInfo.StreamConfig.Id, minChange);
					}
					state.MinChanges = minChanges;
					state.MaxChange = changes[0].Number;
				}
				return state;
			}
		}

		async Task AddCachedCommitAsync(CachedCommitDoc commitDoc, CancellationToken cancellationToken)
		{
			try
			{
				FilterDefinition<CachedCommitDoc> filter = Builders<CachedCommitDoc>.Filter.Expr(x => x.StreamId == commitDoc.StreamId && x.Number == commitDoc.Number);
				await _commits.ReplaceOneAsync(filter, commitDoc, new ReplaceOptions { IsUpsert = true }, cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to add cached commit for {StreamId} CL {Number}: {Message}", commitDoc.StreamId, commitDoc.Number, ex.Message);
				throw;
			}
		}

#pragma warning disable CA1308 // Normalize strings to uppercase
		static RedisSetKey<int> GetRefreshSetKey(string clusterName) => new RedisSetKey<int>($"perforce/{clusterName.ToLowerInvariant()}/refresh");
#pragma warning restore CA1308 // Normalize strings to uppercase

		/// <inheritdoc/>
		public override async Task RefreshCachedCommitAsync(string clusterName, int change)
		{
			RedisSetKey<int> refreshSetKey = GetRefreshSetKey(clusterName);
			await _redisService.GetDatabase().SetAddAsync(refreshSetKey, change, CommandFlags.FireAndForget);
		}

		#endregion

		#region ICommitSource implementation

		/// <summary>
		/// Returns commits within the cached range
		/// </summary>
		class CachedCommitSource : CommitSource
		{
			readonly PerforceServiceCache _owner;

			public CachedCommitSource(PerforceServiceCache owner, StreamConfig stream, ILogger logger)
				: base(owner, stream, logger)
			{
				_owner = owner;
			}

			public override async IAsyncEnumerable<ICommit> FindAsync(int? minChange = null, bool includeMinChange = true, int? maxChange = null, bool includeMaxChange = true, int? maxResults = null, IReadOnlyList<CommitTag>? tags = null, CommitSortOrder sortOrder = CommitSortOrder.Descending, [EnumeratorCancellation] CancellationToken cancellationToken = default)
			{
				if (minChange != null && !includeMinChange)
				{
					minChange = minChange.Value + 1;
				}
				if (maxChange != null && !includeMaxChange)
				{
					maxChange = maxChange.Value - 1;
				}

				if (minChange != null && maxChange != null && minChange.Value > maxChange.Value)
				{
					yield break;
				}

				int numResults = 0;
				_owner._logger.LogDebug("Querying Perforce cache for {StreamId} commits from {MinChange} to {MaxChange} (max: {MaxResults}, tags: {Tags})", StreamConfig.Id, minChange ?? -2, maxChange ?? -2, maxResults ?? -1, (tags == null || tags.Count == 0) ? "none" : String.Join("/", tags.Select(x => x.ToString())));

				using TelemetrySpan span = _owner._tracer.StartActiveSpan($"{nameof(Perforce.PerforceService)}.{nameof(CachedCommitSource)}.{nameof(FindAsync)}");
				span.SetAttribute("stream", StreamConfig.Name);
				span.SetAttribute("minChange", minChange ?? -2);
				span.SetAttribute("maxChange", maxChange ?? -2);
				span.SetAttribute("maxResults", maxResults ?? -1);
				span.SetAttribute("sortOrder", sortOrder.ToString());

				CacheState state = await _owner._mongoService.GetSingletonAsync<CacheState>(cancellationToken);
				if (state.Clusters.TryGetValue(StreamConfig.ClusterName, out ClusterState? clusterState))
				{
					int minReplicatedChange;
					if (clusterState.MinChanges.TryGetValue(StreamConfig.Id, out minReplicatedChange) && (maxChange == null || maxChange.Value > minReplicatedChange))
					{
						FilterDefinition<CachedCommitDoc> filter = Builders<CachedCommitDoc>.Filter.Eq(x => x.StreamId, StreamConfig.Id);

						if (tags != null && tags.Count > 0)
						{
							if (tags.Count == 1)
							{
								filter &= Builders<CachedCommitDoc>.Filter.AnyEq(x => x.CommitTags, tags[0]);
							}
							else
							{
								filter &= Builders<CachedCommitDoc>.Filter.AnyIn(x => x.CommitTags, tags);
							}
						}

						if (maxChange != null)
						{
							filter &= Builders<CachedCommitDoc>.Filter.Lte(x => x.Number, Math.Min(maxChange.Value, clusterState.MaxChange));
						}
						else
						{
							filter &= Builders<CachedCommitDoc>.Filter.Lte(x => x.Number, clusterState.MaxChange);
						}

						if (minChange != null)
						{
							filter &= Builders<CachedCommitDoc>.Filter.Gte(x => x.Number, Math.Max(minChange.Value, minReplicatedChange));
						}
						else
						{
							filter &= Builders<CachedCommitDoc>.Filter.Gte(x => x.Number, minReplicatedChange);
						}

						IFindFluent<CachedCommitDoc, CachedCommitDoc> query = _owner._commits.Find(filter);
						query = sortOrder == CommitSortOrder.Ascending
							? query.SortBy(x => x.Number)
							: query.SortByDescending(x => x.Number);
						using (IAsyncCursor<CachedCommitDoc> cursor = await query.Limit(maxResults).ToCursorAsync(cancellationToken))
						{
							while (await cursor.MoveNextAsync(cancellationToken))
							{
								foreach (CachedCommitDoc commit in cursor.Current)
								{
									commit.PostLoad(_owner, StreamConfig);
									yield return commit;
									numResults++;
								}
							}
						}

						_owner._logger.LogDebug("Found {NumResults} cached results (min-replicated: {MinReplicatedChange}, max-replicated: {MaxReplicatedChange})", numResults, minReplicatedChange, clusterState.MaxChange);

						if (maxResults != null)
						{
							maxResults = maxResults.Value - numResults;
						}

						if (minChange != null && minChange.Value > minReplicatedChange)
						{
							yield break;
						}

						maxChange = minReplicatedChange - 1;

						// Expand the range of cached changes if necessary
						if ((maxResults == null || maxResults.Value > 0) && maxChange > 0)
						{
							int? maxUnfilteredResults = null;
							if (tags == null || tags.Count == 0)
							{
								maxUnfilteredResults = maxResults;
							}
							else if (maxResults != null)
							{
								maxUnfilteredResults = 2 * maxResults.Value;
							}

							await foreach (ICommit commit in base.FindAsync(minChange, true, maxChange, true, maxUnfilteredResults, null, sortOrder, cancellationToken))
							{
								CachedCommitDoc cachedCommit = await CachedCommitDoc.FromCommitAsync(commit, cancellationToken);
								cachedCommit.PostLoad(_owner, StreamConfig);
								await _owner.AddCachedCommitAsync(cachedCommit, cancellationToken);
								await _owner._mongoService.UpdateSingletonAsync<CacheState>(x => TryUpdateRange(x, StreamConfig, commit.Id.GetPerforceChange(), maxChange), cancellationToken);
								_owner._logger.LogDebug("Adding new cached commit for {StreamId} at change {Change} with {NumFiles} (hasAllFiles={HasAllFiles})",
									StreamConfig.Id, commit.Id, cachedCommit.Files.Count, cachedCommit.HasAllFiles);

								if (tags == null || tags.Any(x => cachedCommit.CommitTags.Contains(x)))
								{
									yield return cachedCommit;

									if (maxResults != null)
									{
										maxResults = maxResults.Value - 1;
										if (maxResults.Value == 0)
										{
											break;
										}
									}
								}
							}
						}
					}
				}

				if (maxResults == null || maxResults.Value > 0)
				{
					_owner._logger.LogDebug("Querying Perforce server for {StreamId} commits from {MinChange} to {MaxChange} (max: {MaxResults}, tags: {Tags})", StreamConfig.Id, minChange ?? -2, maxChange ?? -2, maxResults ?? -1, (tags == null || tags.Count == 0) ? "none" : String.Join("/", tags.Select(x => x.ToString())));

					await foreach (ICommit commit in base.FindAsync(minChange, true, maxChange, true, maxResults, tags, sortOrder, cancellationToken))
					{
						yield return commit;
					}
				}
			}

			static bool TryUpdateRange(CacheState state, StreamConfig streamConfig, int newMinChange, int? maxChange)
			{
				ClusterState? clusterState;
				if (state.Clusters.TryGetValue(streamConfig.ClusterName, out clusterState))
				{
					int streamMinChange;
					if (clusterState.MinChanges.TryGetValue(streamConfig.Id, out streamMinChange))
					{
						if (newMinChange < streamMinChange && (maxChange == null || maxChange.Value >= streamMinChange - 1))
						{
							clusterState.MinChanges[streamConfig.Id] = newMinChange;
							return true;
						}
					}
				}
				return false;
			}

			public override async Task<ICommit> GetAsync(int changeNumber, CancellationToken cancellationToken = default)
			{
				CachedCommitDoc? commit = await _owner._commits.Find(x => x.StreamId == StreamConfig.Id && x.Number == changeNumber).FirstOrDefaultAsync(cancellationToken);
				if (commit != null)
				{
					commit.PostLoad(_owner, StreamConfig);
					return commit;
				}
				return await base.GetAsync(changeNumber, cancellationToken);
			}

			/// <inheritdoc/>
			public override async IAsyncEnumerable<ICommit> SubscribeAsync(int minChange, IReadOnlyList<CommitTag>? tags = null, [EnumeratorCancellation] CancellationToken cancellationToken = default)
			{
				AsyncEvent updateEvent = new AsyncEvent();

				void OnUpdate(StreamId streamId)
				{
					if (streamId == StreamConfig.Id)
					{
						updateEvent.Set();
					}
				}

				RedisSubscription? subscription = null;
				try
				{
					for (; ; )
					{
						cancellationToken.ThrowIfCancellationRequested();

						Task task = updateEvent.Task;

						await foreach (ICommit commit in FindCachedCommitsInAscendingOrderAsync(minChange, tags, cancellationToken))
						{
							yield return commit;
							minChange = commit.Id.GetPerforceChange();
						}

						if (subscription == null)
						{
							subscription = await _owner._redisService.SubscribeAsync(s_commitUpdateChannel, OnUpdate);
							continue;
						}

						await task;
					}
				}
				finally
				{
					if (subscription != null)
					{
						await subscription.DisposeAsync();
					}
				}
			}

			async IAsyncEnumerable<ICommit> FindCachedCommitsInAscendingOrderAsync(int minChange, IReadOnlyList<CommitTag>? tags, [EnumeratorCancellation] CancellationToken cancellationToken)
			{
				FilterDefinition<CachedCommitDoc> filter = Builders<CachedCommitDoc>.Filter.Eq(x => x.StreamId, StreamConfig.Id);
				filter &= Builders<CachedCommitDoc>.Filter.Gt(x => x.Number, minChange);

				if (tags != null && tags.Count > 0)
				{
					if (tags.Count == 1)
					{
						filter &= Builders<CachedCommitDoc>.Filter.AnyEq(x => x.CommitTags, tags[0]);
					}
					else
					{
						filter &= Builders<CachedCommitDoc>.Filter.AnyIn(x => x.CommitTags, tags);
					}
				}

				using (IAsyncCursor<CachedCommitDoc> cursor = await _owner._commits.Find(filter).SortBy(x => x.Number).ToCursorAsync(cancellationToken))
				{
					while (await cursor.MoveNextAsync(cancellationToken))
					{
						foreach (CachedCommitDoc commit in cursor.Current)
						{
							commit.PostLoad(_owner, StreamConfig);
							yield return commit;
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public override ICommitCollection GetCommits(StreamConfig streamConfig)
		{
			return new CachedCommitSource(this, streamConfig, _logger);
		}

		#endregion
	}
}
