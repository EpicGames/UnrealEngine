// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Streams;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Streams;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Helper methods for dealing with Perforce workspaces
	/// </summary>
	public static class PerforceWorkspace
	{
		/// <summary>
		/// Get the AutoSDK workspace required for an agent
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="cluster">The perforce cluster to get a workspace for</param>
		/// <param name="pools">Pools that the agent belongs to</param>
		/// <returns></returns>
		public static AgentWorkspaceInfo? GetAutoSdkWorkspace(this IAgent agent, PerforceCluster cluster, IEnumerable<IPool> pools)
		{
			AutoSdkConfig? autoSdkConfig = null;
			foreach (IPool pool in pools)
			{
				autoSdkConfig = AutoSdkConfig.Merge(autoSdkConfig, pool.AutoSdkConfig);
			}
			if (autoSdkConfig == null)
			{
				return null;
			}

			return GetAutoSdkWorkspace(agent, cluster, autoSdkConfig);
		}

		/// <summary>
		/// Get the AutoSDK workspace required for an agent
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="cluster">The perforce cluster to get a workspace for</param>
		/// <param name="autoSdkConfig">Configuration for autosdk</param>
		/// <returns></returns>
		public static AgentWorkspaceInfo? GetAutoSdkWorkspace(this IAgent agent, PerforceCluster cluster, AutoSdkConfig autoSdkConfig)
		{
			foreach (AutoSdkWorkspace autoSdk in cluster.AutoSdk)
			{
				if (autoSdk.Stream != null && autoSdk.Properties.All(x => agent.Properties.Contains(x)))
				{
					return new AgentWorkspaceInfo(cluster.Name, autoSdk.UserName, autoSdk.Name ?? "AutoSDK", autoSdk.Stream!, autoSdkConfig.View.ToList(), true, null, null);
				}
			}
			return null;
		}

		/// <summary>
		/// Converts this workspace to an RPC message
		/// </summary>
		/// <param name="agent">The agent to get a workspace for</param>
		/// <param name="workspace">The workspace definition</param>
		/// <param name="cluster">The global state</param>
		/// <param name="loadBalancer">The Perforce load balancer</param>
		/// <param name="workspaceMessages">List of messages</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The RPC message</returns>
		public static async Task<bool> TryAddWorkspaceMessageAsync(this IAgent agent, AgentWorkspaceInfo workspace, PerforceCluster cluster, PerforceLoadBalancer loadBalancer, IList<RpcAgentWorkspace> workspaceMessages, CancellationToken cancellationToken)
		{
			// Find a matching server, trying to use a previously selected one if possible
			string? baseServerAndPort;
			string? serverAndPort;
			bool partitioned;

			RpcAgentWorkspace? existingWorkspace = workspaceMessages.FirstOrDefault(x => x.ConfiguredCluster == workspace.Cluster);
			if (existingWorkspace != null)
			{
				baseServerAndPort = existingWorkspace.BaseServerAndPort;
				serverAndPort = existingWorkspace.ServerAndPort;
				partitioned = existingWorkspace.Partitioned;
			}
			else
			{
				if (cluster == null)
				{
					return false;
				}

				IPerforceServer? server = await loadBalancer.SelectServerAsync(cluster, agent, cancellationToken);
				if (server == null)
				{
					return false;
				}

				baseServerAndPort = server.BaseServerAndPort;
				serverAndPort = server.ServerAndPort;
				partitioned = server.SupportsPartitionedWorkspaces;
			}

			// Find the matching credentials for the desired user
			PerforceCredentials? credentials = null;
			if (cluster != null)
			{
				if (workspace.UserName == null)
				{
					credentials = cluster.Credentials.FirstOrDefault();
				}
				else
				{
					credentials = cluster.Credentials.FirstOrDefault(x => String.Equals(x.UserName, workspace.UserName, StringComparison.OrdinalIgnoreCase));
				}
			}

			// Construct the message
			RpcAgentWorkspace result = new RpcAgentWorkspace
			{
				ConfiguredCluster = workspace.Cluster,
				ConfiguredUserName = workspace.UserName,
				Cluster = cluster?.Name,
				BaseServerAndPort = baseServerAndPort,
				ServerAndPort = serverAndPort,
				UserName = credentials?.UserName ?? workspace.UserName,
				Password = credentials?.Password,
				Ticket = credentials?.Ticket,
				Identifier = workspace.Identifier,
				Stream = workspace.Stream,
				Incremental = workspace.Incremental,
				Partitioned = partitioned,
				Method = workspace.Method ?? String.Empty,
				MinScratchSpace = workspace.MinScratchSpace ?? 0,
				ConformDiskFreeSpace = workspace.ConformDiskFreeSpace ?? 0,
			};

			if (workspace.View != null)
			{
				result.View.AddRange(workspace.View);
			}

			workspaceMessages.Add(result);
			return true;
		}

		/// <summary>
		/// Tries to get an agent workspace definition from the given type name
		/// </summary>
		/// <param name="streamConfig">The stream object</param>
		/// <param name="agentType">The agent type</param>
		/// <param name="workspace">Receives the agent workspace definition</param>
		/// <param name="autoSdkConfig">Receives the autosdk workspace config</param>
		/// <returns>True if the agent type was valid, and an agent workspace could be created</returns>
		public static bool TryGetAgentWorkspace(this StreamConfig streamConfig, IAgentType agentType, [NotNullWhen(true)] out AgentWorkspaceInfo? workspace, out AutoSdkConfig? autoSdkConfig)
		{
			// Get the workspace settings
			if (agentType.Workspace == null)
			{
				// Use the default settings (fast switching workspace, clean)
				workspace = new AgentWorkspaceInfo(streamConfig.ClusterName, null, streamConfig.GetDefaultWorkspaceIdentifier(), streamConfig.Name, null, false, null, null);
				autoSdkConfig = AutoSdkConfig.Full;
				return true;
			}
			else
			{
				// Try to get the matching workspace type
				WorkspaceConfig? workspaceConfig;
				if (!streamConfig.WorkspaceTypes.TryGetValue(agentType.Workspace, out workspaceConfig))
				{
					workspace = null;
					autoSdkConfig = null;
					return false;
				}

				// Get the workspace identifier
				string identifier;
				if (workspaceConfig.Identifier != null)
				{
					identifier = workspaceConfig.Identifier;
				}
				else if (workspaceConfig.Incremental ?? false)
				{
					identifier = $"{streamConfig.GetEscapedName()}+{agentType.Workspace}";
				}
				else
				{
					identifier = streamConfig.GetDefaultWorkspaceIdentifier();
				}

				// Create the new workspace
				string cluster = workspaceConfig.Cluster ?? streamConfig.ClusterName;
				workspace = new AgentWorkspaceInfo(
					cluster,
					workspaceConfig.UserName,
					identifier,
					workspaceConfig.Stream ?? streamConfig.Name,
					workspaceConfig.View,
					workspaceConfig.Incremental ?? false,
					workspaceConfig.Method,
					workspaceConfig.MinScratchSpace,
					workspaceConfig.ConformDiskFreeSpace);
				
				autoSdkConfig = GetAutoSdkConfig(workspaceConfig, streamConfig);

				return true;
			}

			static AutoSdkConfig? GetAutoSdkConfig(WorkspaceConfig workspaceConfig, StreamConfig streamConfig)
			{
				AutoSdkConfig? autoSdkConfig = null;
				if (workspaceConfig.UseAutoSdk ?? true)
				{
					List<string> view = new List<string>();
					if (streamConfig.AutoSdkView != null)
					{
						view.AddRange(streamConfig.AutoSdkView);
					}
					if (workspaceConfig.AutoSdkView != null)
					{
						view.AddRange(workspaceConfig.AutoSdkView);
					}
					if (view.Count == 0)
					{
						view.Add("...");
					}
					autoSdkConfig = new AutoSdkConfig(view);
				}
				return autoSdkConfig;
			}
		}

		/// <summary>
		/// Get a list of workspaces for the given agent
		/// </summary>
		/// <param name="poolService">The pool service instance</param>
		/// <param name="agent">The agent to return workspaces for</param>
		/// <param name="validAtTime">Absolute time at which we expect the results to be valid. Values may be cached as long as they are after this time.</param>
		/// <param name="buildConfig">Current configuration</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of workspaces</returns>
		public static async Task<HashSet<AgentWorkspaceInfo>> GetWorkspacesAsync(this PoolService poolService, IAgent agent, DateTime validAtTime, BuildConfig buildConfig, CancellationToken cancellationToken)
		{
			List<IPoolConfig> pools = await poolService.GetPoolsAsync(agent, validAtTime, cancellationToken);

			HashSet<AgentWorkspaceInfo> workspaces = new HashSet<AgentWorkspaceInfo>();
			foreach (IPoolConfig pool in pools)
			{
				workspaces.UnionWith(pool.Workspaces);
			}

			AutoSdkConfig? autoSdkConfig = GetAutoSdkConfig(pools);
			if (autoSdkConfig != null)
			{
				foreach (string? clusterName in workspaces.Select(x => x.Cluster).Distinct().ToList())
				{
					PerforceCluster? cluster = buildConfig.FindPerforceCluster(clusterName);
					if (cluster != null)
					{
						AgentWorkspaceInfo? autoSdkWorkspace = agent.GetAutoSdkWorkspace(cluster, autoSdkConfig);
						if (autoSdkWorkspace != null)
						{
							workspaces.Add(autoSdkWorkspace);
						}
					}
				}
			}

			return workspaces;
		}

		static AutoSdkConfig? GetAutoSdkConfig(IEnumerable<IPoolConfig> pools)
		{
			AutoSdkConfig? autoSdkConfig = null;
			foreach (IPoolConfig pool in pools)
			{
				autoSdkConfig = AutoSdkConfig.Merge(autoSdkConfig, pool.AutoSdkConfig);
			}
			return autoSdkConfig;
		}

		/// <summary>
		/// Gets all the autosdk workspaces required for an agent
		/// </summary>
		/// <param name="poolService">The pool service instance</param>
		/// <param name="agent"></param>
		/// <param name="cluster"></param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns></returns>
		public static async Task<AgentWorkspaceInfo?> GetAutoSdkWorkspaceAsync(this PoolService poolService, IAgent agent, PerforceCluster cluster, CancellationToken cancellationToken)
		{
			List<IPoolConfig> pools = await poolService.GetPoolsAsync(agent, DateTime.UtcNow - TimeSpan.FromSeconds(10.0), cancellationToken);

			AutoSdkConfig? autoSdkConfig = GetAutoSdkConfig(pools);
			if (autoSdkConfig == null)
			{
				return null;
			}

			return agent.GetAutoSdkWorkspace(cluster, autoSdkConfig);
		}

		/// <summary>
		/// Get a list of workspaces for the given agent
		/// </summary>
		/// <param name="poolService">Instance of the pool service</param>
		/// <param name="agent">The agent to return workspaces for</param>
		/// <param name="perforceCluster">The P4 cluster to find a workspace for</param>
		/// <param name="validAtTime">Absolute time at which we expect the results to be valid. Values may be cached as long as they are after this time.</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of workspaces</returns>
		public static async Task<AgentWorkspaceInfo?> GetAutoSdkWorkspaceAsync(this PoolService poolService, IAgent agent, PerforceCluster perforceCluster, DateTime validAtTime, CancellationToken cancellationToken)
		{
			AutoSdkConfig? autoSdkConfig = null;

			IReadOnlyDictionary<PoolId, IPoolConfig> poolMapping = await poolService.GetPoolLookupAsync(validAtTime, cancellationToken);
			foreach (PoolId poolId in agent.Pools)
			{
				IPoolConfig? pool;
				if (poolMapping.TryGetValue(poolId, out pool))
				{
					autoSdkConfig = AutoSdkConfig.Merge(autoSdkConfig, pool.AutoSdkConfig);
				}
			}

			if (autoSdkConfig == null)
			{
				return null;
			}
			else
			{
				return agent.GetAutoSdkWorkspace(perforceCluster, autoSdkConfig);
			}
		}
	}
}
