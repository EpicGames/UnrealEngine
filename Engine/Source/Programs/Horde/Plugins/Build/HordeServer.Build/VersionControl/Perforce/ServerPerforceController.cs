// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Server;
using HordeServer.Configuration;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;

namespace HordeServer.VersionControl.Perforce
{
	/// <summary>
	/// Implements preflight of config changes with Perforce
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class ServerPerforceController : HordeControllerBase
	{
		readonly IConfigService _configService;
		readonly IChangelistFileReader _fileReader;

		/// <summary>
		/// Constructor
		/// </summary>
		public ServerPerforceController(IConfigService configService, IChangelistFileReader fileReader)
		{
			_configService = configService;
			_fileReader = fileReader;
		}

		/// <summary>
		/// Returns settings for automating auth against this server
		/// </summary>
		[HttpPost]
		[Route("/api/v1/server/preflightconfig")]
		public async Task<ActionResult<PreflightConfigResponse>> PreflightConfigAsync(PreflightConfigRequest request, CancellationToken cancellationToken)
		{
			Dictionary<Uri, byte[]> files;
			try
			{
				files = await _fileReader.ReadChangelistFilesAsync(request.ShelvedChange, request.Cluster, cancellationToken);
			}
			catch (ChangelistNotFoundException)
			{
				return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "CL {Change} does not exist.", request.ShelvedChange);
			}

			if (files.Count == 0)
			{
				return BadRequest(KnownLogEvents.Horde_InvalidPreflight, "No config files found in CL {Change}.", request.ShelvedChange);
			}

			ConfigValidationResult validationResult = await _configService.ValidateAsync(files, cancellationToken);

			PreflightConfigResponse response = new PreflightConfigResponse();
			response.Result = validationResult.Success;
			response.Message = validationResult.Error;
			response.Warnings = validationResult.Warnings.ToArray();

			return response;
		}
	}
}
