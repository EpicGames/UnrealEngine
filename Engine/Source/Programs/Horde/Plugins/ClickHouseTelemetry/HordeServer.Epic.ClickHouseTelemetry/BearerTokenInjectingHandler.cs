// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Http.Headers;
using HordeServer.Analytics;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// HTTP message handler that asks the registered <see cref="IAuthenticationProvider{TConfig}"/> for a token on each outbound request and, when one is available, attaches it as <c>Authorization: Bearer &lt;token&gt;</c>. 
	/// When the provider returns null (the inert path — empty <c>OAuthTokenEndpoint</c>), no header is added and the request flies through unchanged so ClickHouse can consume whatever auth is already in the connection string (UID/PWD, or none).
	///
	/// Plugged into <c>ClickHouseConnection</c>'s <c>HttpClientHandler</c> overload so it transparently covers every HTTP call the native client makes — queries, bulk-copy inserts, schema migrations, table existence checks. 
	/// Token caching/refresh happens inside the provider; this handler just reads the current token per request.
	/// </summary>
	internal sealed class BearerTokenInjectingHandler : HttpClientHandler
	{
		readonly IAuthenticationProvider<ClickHouseTelemetryConfig> _authProvider;

		public BearerTokenInjectingHandler(IAuthenticationProvider<ClickHouseTelemetryConfig> authProvider)
		{
			CheckCertificateRevocationList = true;
			AutomaticDecompression = DecompressionMethods.All;

			_authProvider = authProvider;
		}

		/// <inheritdoc/>
		protected override async Task<HttpResponseMessage> SendAsync(HttpRequestMessage request, CancellationToken cancellationToken)
		{
			string? token = await _authProvider.GetTokenAsync();
			if (!String.IsNullOrEmpty(token))
			{
				request.Headers.Authorization = new AuthenticationHeaderValue("Bearer", token);
			}
			return await base.SendAsync(request, cancellationToken);
		}
	}
}
