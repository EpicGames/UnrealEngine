// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Builds the <see cref="HttpClient"/> used by every ClickHouse code path (sink, schema migrator, query-side data source). Centralises construction so the OAuth-bearer-token injection, certificate-revocation check, and decompression behavior live in exactly one place.
	///
	/// The returned <see cref="HttpClient"/> owns its handler (<c>disposeHandler=true</c> default), so the caller need only dispose the client when the owning component shuts down. ClickHouseConnection wraps a long-lived connection around one of these clients.
	/// </summary>
	internal static class ClickHouseHttpClientFactory
	{
		/// <summary>
		/// Builds a fresh <see cref="HttpClient"/> with a <see cref="BearerTokenInjectingHandler"/> attached. Each call returns a new instance — callers own its lifetime.
		/// </summary>
		/// <param name="authProvider">The authentication provider that supplies bearer tokens per outbound request.</param>
		public static HttpClient Create(IAuthenticationProvider<ClickHouseTelemetryConfig> authProvider)
		{
#pragma warning disable CA5399 // CheckCertificateRevocationList is set inside BearerTokenInjectingHandler's constructor

#pragma warning disable CA2000 // Dispose objects before losing scope - HttpClient takes ownership of the handler (disposeHandler=true default)
			BearerTokenInjectingHandler handler = new BearerTokenInjectingHandler(authProvider);
#pragma warning restore CA2000

			return new HttpClient(handler);
#pragma warning restore CA5399
		}
	}
}
