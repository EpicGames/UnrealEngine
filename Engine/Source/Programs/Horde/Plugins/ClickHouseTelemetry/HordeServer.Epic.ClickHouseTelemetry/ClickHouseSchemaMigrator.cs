// Copyright Epic Games, Inc. All Rights Reserved.

using ClickHouse.Client.ADO;
using HordeServer.Analytics;
using HordeServer.Analytics.Schemas;
using HordeServer.Dialects;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Computes and applies ClickHouse-side schema migrations when a new schema
	/// version is approved or created. Runs before the new version is committed
	/// to MongoDB so a DDL failure leaves the system consistent — the metadata
	/// never gets ahead of the actual table shape.
	///
	/// Behaviour per change kind (matches the agreed-on policy):
	///   - Add column           → ALTER TABLE … ADD COLUMN IF NOT EXISTS (additive)
	///   - Drop column          → ALTER TABLE … DROP COLUMN IF EXISTS    (destroys history; warned)
	///   - Rename or type change → DROP old + ADD new                    (destroys history; warned)
	///   - Table/database rename → no DDL; warned that old data is left behind and the
	///                             sink will lazy-create the new table on first event
	///
	/// First-version schemas (no previous Mongo row) need no migration; the sink's
	/// existing CREATE TABLE IF NOT EXISTS handles them.
	/// </summary>
	public class ClickHouseSchemaMigrator : ISchemaMigrator
	{
		readonly IOptionsMonitor<ClickHouseTelemetryConfig> _globalConfig;
		readonly ITelemetrySchemaCollection _schemaCollection;
		readonly IAuthenticationProvider<ClickHouseTelemetryConfig> _authProvider;
		readonly ISqlDialect _dialect = new ClickHouseSqlDialect();
		readonly ILogger<ClickHouseSchemaMigrator> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ClickHouseSchemaMigrator(IOptionsMonitor<ClickHouseTelemetryConfig> globalConfig, ITelemetrySchemaCollection schemaCollection, IAuthenticationProvider<ClickHouseTelemetryConfig> authProvider, ILogger<ClickHouseSchemaMigrator> logger)
		{
			_globalConfig = globalConfig;
			_schemaCollection = schemaCollection;
			_authProvider = authProvider;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<MigrationPlan?> PlanAsync(
			string eventName,
			IReadOnlyList<SchemaColumn> proposedColumns,
			string proposedTableName,
			string? proposedSchemaName,
			CancellationToken cancellationToken = default)
		{
			ITelemetrySchema? current = await _schemaCollection.GetLatestSchemaAsync(eventName, cancellationToken);
			if (current == null)
			{
				// First version — no migration needed. The sink's EnsureTableExistsFromSchemaAsync
				// will CREATE TABLE IF NOT EXISTS on the first event of this schema.
				return null;
			}

			MigrationPlan plan = new()
			{
				EventName = eventName,
				TableName = proposedTableName,
				SchemaName = proposedSchemaName
			};

			// Detect table/database relocation. When either changes, we don't touch the
			// old table — admins were warned and explicitly accept history being left
			// behind. The sink will create the new table on first event of the new version.
			bool tableNameChanged = !String.Equals(current.TableName, proposedTableName, StringComparison.Ordinal);
			bool schemaNameChanged = !String.Equals(current.SchemaName ?? "", proposedSchemaName ?? "", StringComparison.Ordinal);
			if (tableNameChanged || schemaNameChanged)
			{
				string oldQualified = QualifiedName(current.SchemaName, current.TableName);
				string newQualified = QualifiedName(proposedSchemaName, proposedTableName);
				plan.Steps.Add(new MigrationStep
				{
					Kind = MigrationStepKind.NewTableLocation,
					Detail = $"{oldQualified} -> {newQualified}"
				});

				plan.Warnings.Add(
					$"Table location changed ({oldQualified} -> {newQualified}). " +
					$"Existing data at {oldQualified} will be left in place; the new table at " +
					$"{newQualified} starts empty and is created lazily on the first event.");
				plan.IsDestructive = true;

				// Don't compute column-level diff when the location changed: the new
				// table will be created fresh from the proposed shape, no ALTER applies.
				return plan;
			}

			// Index columns by name for O(1) lookup on each side.
			Dictionary<string, SchemaColumn> currentByName = current.Columns.ToDictionary(c => c.ColumnName, StringComparer.OrdinalIgnoreCase);
			Dictionary<string, SchemaColumn> proposedByName = proposedColumns.ToDictionary(c => c.ColumnName, StringComparer.OrdinalIgnoreCase);

			// Type changes (same name, different DataType or IsNullable) — drop + add.
			// Renames produce a removed name on one side and an added name on the other,
			// which fall through to the Added/Dropped branches below; the warnings are
			// the same (history destroyed) so we don't try to detect renames separately.
			foreach (SchemaColumn proposed in proposedColumns)
			{
				if (currentByName.TryGetValue(proposed.ColumnName, out SchemaColumn? existing))
				{
					if (existing.DataType != proposed.DataType || existing.IsNullable != proposed.IsNullable)
					{
						string detail = $"{TypeLabel(existing)} -> {TypeLabel(proposed)}";
						plan.Steps.Add(new MigrationStep
						{
							Kind = MigrationStepKind.DropAndAddColumn,
							ColumnName = proposed.ColumnName,
							Detail = detail,
							Column = proposed
						});
						plan.Warnings.Add(
							$"Column '{proposed.ColumnName}' type change ({detail}) is implemented as drop+add. " +
							$"All stored history for this column will be destroyed.");
						plan.IsDestructive = true;
					}
				}
			}

			// Drops — present in current, absent in proposed. (Rename appears here as the old name.)
			foreach (SchemaColumn existing in current.Columns)
			{
				if (!proposedByName.ContainsKey(existing.ColumnName))
				{
					plan.Steps.Add(new MigrationStep
					{
						Kind = MigrationStepKind.DropColumn,
						ColumnName = existing.ColumnName
					});
					plan.Warnings.Add(
						$"Column '{existing.ColumnName}' will be dropped. " +
						$"All stored history for this column will be destroyed.");
					plan.IsDestructive = true;
				}
			}

			// Adds — present in proposed, absent in current. (Rename appears here as the new name.)
			foreach (SchemaColumn proposed in proposedColumns)
			{
				if (!currentByName.ContainsKey(proposed.ColumnName))
				{
					plan.Steps.Add(new MigrationStep
					{
						Kind = MigrationStepKind.AddColumn,
						ColumnName = proposed.ColumnName,
						Detail = TypeLabel(proposed),
						Column = proposed
					});
					// No warning: ADD COLUMN is non-destructive. Existing rows get NULL/default.
				}
			}

			return plan;
		}

		/// <inheritdoc/>
		public async Task ApplyAsync(MigrationPlan plan, CancellationToken cancellationToken = default)
		{
			if (plan.Steps.Count == 0)
			{
				return;
			}

			// NewTableLocation steps don't touch the old table — admins accepted that
			// the new table starts fresh. The sink will create it on first event.
			List<MigrationStep> ddlSteps = plan.Steps.Where(s => s.Kind != MigrationStepKind.NewTableLocation).ToList();
			if (ddlSteps.Count == 0)
			{
				_logger.LogInformation("Schema migration for {EventName}: no DDL to run (table-location-only change)", plan.EventName);
				return;
			}

			string connectionString = _globalConfig.CurrentValue.ConnectionString;
			if (String.IsNullOrEmpty(connectionString))
			{
				throw new InvalidOperationException("ClickHouse connection string is not configured; cannot apply schema migration.");
			}

			string effectiveSchemaName = plan.SchemaName ?? _globalConfig.CurrentValue.DefaultDatabaseName;
			string quotedTableName = _dialect.FormatTableName(effectiveSchemaName, plan.TableName);

			// We can disable both for the following reasons:
			// - The dispose is handled via the HttpClient when provided a nested handler: https://source.dot.net/#System.Net.Http/System/Net/Http/HttpClient.cs,146
			// - The bearerTokenInjectingHandler sets CheckCertificateRevocationList

#pragma warning disable CA2000 // Dispose objects before losing scope
#pragma warning disable CA5399 // HttpClient is created without enabling CheckCertificateRevocationList
			using HttpClient httpClient = new HttpClient(new BearerTokenInjectingHandler(_authProvider));
#pragma warning restore CA5399
#pragma warning restore CA2000
			await using ClickHouseConnection connection = new ClickHouseConnection(connectionString, httpClient);
			await connection.OpenAsync(cancellationToken);

			if (!await TableExistsAsync(connection, quotedTableName, cancellationToken))
			{
				_logger.LogInformation(
					"Schema migration for {EventName} skipped: target table {Table} does not exist; sink will create it from the new schema on first event.",
					plan.EventName, quotedTableName);
				return;
			}

			foreach (MigrationStep step in ddlSteps)
			{
				string ddl = BuildDdl(step, quotedTableName);
				_logger.LogInformation("Schema migration for {EventName}: {Ddl}", plan.EventName, ddl);

				using ClickHouseCommand cmd = connection.CreateCommand();
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
				cmd.CommandText = ddl;
#pragma warning restore CA2100
				await cmd.ExecuteNonQueryAsync(cancellationToken);
			}

			_logger.LogInformation("Schema migration for {EventName} applied {StepCount} step(s)", plan.EventName, ddlSteps.Count);
		}

		/// <summary>
		/// Checks whether a fully-qualified ClickHouse table identifier exists.
		/// Uses <c>EXISTS TABLE</c> which returns a UInt8 (0 or 1) rather than the
		/// system.tables join — keeps the dependency on system catalogue layout out
		/// of this code path.
		/// </summary>
		static async Task<bool> TableExistsAsync(ClickHouseConnection connection, string quotedTableName, CancellationToken cancellationToken)
		{
			using ClickHouseCommand cmd = connection.CreateCommand();
#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
			cmd.CommandText = $"EXISTS TABLE {quotedTableName}";
#pragma warning restore CA2100
			object? result = await cmd.ExecuteScalarAsync(cancellationToken);
			return result != null && Convert.ToInt32(result) != 0;
		}

		/// <summary>
		/// Translates a single migration step into ClickHouse ALTER TABLE syntax.
		/// The portable <see cref="SchemaColumn"/> on the step is resolved here via
		/// this plugin's own <see cref="ClickHouseTypeMapper"/> — the migration plan
		/// stays backend-agnostic and the DDL translation stays local to this file.
		///
		/// ClickHouse does not support a table-level "ALTER TABLE IF EXISTS" guard,
		/// so callers must pre-check table existence (see <see cref="TableExistsAsync"/>).
		/// The per-column IF EXISTS / IF NOT EXISTS clauses ARE supported and we use
		/// them so individual operations are idempotent on partial failure.
		/// </summary>
		static string BuildDdl(MigrationStep step, string quotedTableName)
		{
			switch (step.Kind)
			{
				case MigrationStepKind.AddColumn:
					{
						SchemaColumn col = RequireColumn(step);
						string ddlType = ClickHouseTypeMapper.MapColumnToClickHouseType(col);
						return $"ALTER TABLE {quotedTableName} ADD COLUMN IF NOT EXISTS {step.ColumnName} {ddlType}";
					}

				case MigrationStepKind.DropColumn:
					return $"ALTER TABLE {quotedTableName} DROP COLUMN IF EXISTS {step.ColumnName}";

				case MigrationStepKind.DropAndAddColumn:
					{
						// ClickHouse has no single "replace column with type change" statement, so
						// we issue two ALTERs separated by a semicolon. The per-column IF EXISTS /
						// IF NOT EXISTS clauses keep this re-runnable on partial failure.
						SchemaColumn col = RequireColumn(step);
						string ddlType = ClickHouseTypeMapper.MapColumnToClickHouseType(col);
						return $"ALTER TABLE {quotedTableName} DROP COLUMN IF EXISTS {step.ColumnName}; " +
							$"ALTER TABLE {quotedTableName} ADD COLUMN IF NOT EXISTS {step.ColumnName} {ddlType}";
					}

				default:
					throw new InvalidOperationException($"BuildDdl received an unexpected step kind: {step.Kind}");
			}
		}

		/// <summary>
		/// Guards against a step that should carry a column definition but doesn't.
		/// Any miss here is a planner bug, not user input — fail loudly.
		/// </summary>
		static SchemaColumn RequireColumn(MigrationStep step)
		{
			return step.Column ?? throw new InvalidOperationException(
				$"Migration step {step.Kind} for column '{step.ColumnName}' is missing its Column definition.");
		}

		/// <summary>Compact human-readable type label for warnings (no ClickHouse DDL syntax).</summary>
		static string TypeLabel(SchemaColumn column)
		{
			string baseLabel = column.DataType.ToString();
			if (column.DataType == SchemaDataType.Array && column.ArrayElementType.HasValue)
			{
				baseLabel = $"Array<{column.ArrayElementType.Value}>";
			}
			return column.IsNullable ? $"{baseLabel}?" : baseLabel;
		}

		/// <summary>Pretty database.table for warning text. Falls back to default DB when schema is null.</summary>
		string QualifiedName(string? schemaName, string tableName)
		{
			string db = schemaName ?? _globalConfig.CurrentValue.DefaultDatabaseName;
			return $"{db}.{tableName}";
		}
	}
}
