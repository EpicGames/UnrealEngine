// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Configuration for the ClickHouseTelemetry plugin. Implements
	/// <see cref="IOAuthClientCredentialsConfig"/> so the sink can optionally
	/// participate in the shared OAuth2 token-exchange machinery for backends
	/// that front ClickHouse with bearer-token auth (e.g. ClickHouse Cloud
	/// behind an API gateway, or a future managed offering with OIDC).
	/// Empty <c>OAuthTokenEndpoint</c> = inert; the connection string is
	/// used verbatim with whatever auth (UID/PWD or none) it carries.
	/// </summary>
	public class ClickHouseTelemetryConfig : IPluginConfig, IOAuthClientCredentialsConfig
	{
		/// <summary>
		/// Connection string for the ClickHouse database.
		/// Example: "Host=localhost;Port=8123;Database=horde_telemetry"
		/// </summary>
		public string ConnectionString { get; set; } = String.Empty;

		/// <summary>
		/// Allow list of event names to process. If empty, all events are processed.
		/// Events not in this list will be ignored.
		/// </summary>
		public List<string> AllowList { get; set; } = new List<string>();

		/// <summary>
		/// Whether to automatically create tables for telemetry record types.
		/// When enabled, tables are created based on AbstractTelemetryRecord attributes.
		/// </summary>
		public bool AutoCreateTables { get; set; } = true;

		/// <summary>
		/// Database name to use for telemetry tables.
		/// </summary>
		public string DefaultDatabaseName { get; set; } = "horde_telemetry";

		/// <summary>
		/// Batch size for bulk inserts.
		/// </summary>
		public int BatchSize { get; set; } = 1000;

		/// <summary>
		/// Flush interval in seconds. Events are batched and flushed at this interval.
		/// </summary>
		public int FlushIntervalSeconds { get; set; } = 30;

		/// <summary>
		/// Whether to enable the unstructured telemetry path.
		/// When true, events with [TelemetryEventAttribute] but without [AnalyticsTableGen]
		/// are stored in a raw staging table for future schematization.
		/// When false, such events are dropped.
		/// </summary>
		public bool EnableUnstructuredPath { get; set; } = false;

		/// <summary>
		/// When true, property names from incoming events and schema PropertyNames are both
		/// normalised (lowercased, '.', '_' and '-' stripped) before being matched against
		/// each other. This lets a column named <c>JobId</c> pick up an incoming property
		/// called <c>job_id</c>, <c>job-id</c>, <c>job.id</c>, or any casing variant — useful
		/// when ingest sources don't agree on naming conventions and rewriting the producer
		/// isn't an option. Off by default: matching stays case-insensitive but otherwise
		/// exact, preserving the original behaviour.
		/// </summary>
		public bool NormalizePropertiesOnIngest { get; set; } = false;

		/// <summary>
		/// HTTPS endpoint for the OAuth2 client-credentials token exchange. When
		/// empty (default), no bearer token is fetched and the ClickHouse
		/// connection string passes through verbatim — same wire behaviour as
		/// before this field existed.
		/// </summary>
		public string OAuthTokenEndpoint { get; set; } = String.Empty;

		/// <summary>
		/// OAuth2 scope sent in the token request. Defaults to <c>all-apis</c>
		/// matching the StructuredAnalytics convention. Set to empty to omit.
		/// </summary>
		public string OAuthScope { get; set; } = "all-apis";

		/// <summary>
		/// Connection-string key the access token would be folded into for ODBC
		/// backends. Unused for the ClickHouse native HTTP path (which uses
		/// Authorization: Bearer headers instead) but kept on the config for
		/// interface symmetry with the rest of the analytics OAuth surface.
		/// </summary>
		public string OAuthAccessTokenFragmentKey { get; set; } = "Auth_AccessToken";

		/// <summary>
		/// Secret-store ID for the OAuth client_id / client_secret pair.
		/// </summary>
		public string OAuthCredentialsSecretId { get; set; } = String.Empty;

		/// <inheritdoc/>
		public void PostLoad(PluginConfigOptions configOptions)
		{
		}
	}
}
