// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Analytics;
using HordeServer.Analytics.Schemas;
using HordeServer.ClickHouseTelemetry;
using HordeServer.DataSources;
using HordeServer.Plugins;
using HordeServer.Telemetry;
using Microsoft.AspNetCore.Builder;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;

namespace HordeServer
{
	/// <summary>
	/// Entry point for the ClickHouseTelemetry plugin
	/// </summary>
	[Plugin("ClickHouseTelemetry", EnabledByDefault = false, GlobalConfigType = typeof(ClickHouseTelemetryConfig), ServerConfigType = typeof(ClickHouseTelemetryServerConfig))]
	public class ClickHouseTelemetryPlugin : IPluginStartup
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public ClickHouseTelemetryPlugin()
		{
		}

		/// <inheritdoc/>
		public void Configure(IApplicationBuilder app)
		{
		}

		/// <inheritdoc/>
		public void ConfigureServices(IServiceCollection serviceCollection)
		{
			// Sink depends on registry for schema lookup
			serviceCollection.AddSingleton<ClickHouseTelemetrySink>();
			serviceCollection.AddSingleton<ITelemetrySink>(sp => sp.GetRequiredService<ClickHouseTelemetrySink>());
			serviceCollection.AddSingleton<IHostedService>(sp => sp.GetRequiredService<ClickHouseTelemetrySink>());

			serviceCollection.AddSingleton<ISchemaMigrator, ClickHouseSchemaMigrator>();

			serviceCollection.AddHttpClient();
			serviceCollection.AddSingleton<IAuthenticationProvider<ClickHouseTelemetryConfig>, OAuthClientCredentialsTokenProvider<ClickHouseTelemetryConfig>>();

			// Query-side data source: enables StructuredAnalytics/PerformanceTrends to run reads against ClickHouse natively (no ODBC).
			// Registered under the keyed-services key "ClickHouse" so consumers can opt in via BackendName in their config.
			serviceCollection.AddKeyedSingleton<IAnalyticsDataSource, ClickHouseAnalyticsDataSource>(BackendKey);
		}

		/// <summary>
		/// Key under which this plugin's <see cref="IAnalyticsDataSource"/> is registered. Consumer plugins reference this in their <c>BackendName</c> config field.
		/// </summary>
		public const string BackendKey = "ClickHouse";
	}

	/// <summary>
	/// Helper methods for ClickHouseTelemetry plugin config
	/// </summary>
	public static class ClickHouseTelemetryExtensions
	{
		/// <summary>
		/// Configures the ClickHouseTelemetry plugin
		/// </summary>
		public static void AddClickHouseTelemetryConfig(this IDictionary<PluginName, IPluginConfig> dictionary, ClickHouseTelemetryConfig config)
			=> dictionary[new PluginName("ClickHouseTelemetry")] = config;

		/// <summary>
		/// Gets configuration for the ClickHouseTelemetry plugin
		/// </summary>
		public static ClickHouseTelemetryConfig GetClickHouseTelemetryConfig(this IDictionary<PluginName, IPluginConfig> dictionary)
			=> (ClickHouseTelemetryConfig)dictionary[new PluginName("ClickHouseTelemetry")];
	}
}
