// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Plugins;

namespace HordeServer
{
	/// <summary>
	/// Server configuration for the ClickHouseTelemetry plugin
	/// </summary>
	public class ClickHouseTelemetryServerConfig : PluginServerConfig
	{
	}
}
