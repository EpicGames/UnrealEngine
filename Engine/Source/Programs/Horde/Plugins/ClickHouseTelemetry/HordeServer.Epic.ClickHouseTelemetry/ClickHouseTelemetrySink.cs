// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Data.Common;
using System.Text.Json;
using ClickHouse.Client.ADO;
using ClickHouse.Client.Copy;
using EpicGames.Analytics;
using EpicGames.Analytics.Telemetry;
using EpicGames.Core;
using EpicGames.Horde.Telemetry;
using HordeServer.Analytics;
using HordeServer.Analytics.Schemas;
using HordeServer.Dialects;
using HordeServer.Telemetry;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Telemetry sink that forwards events to ClickHouse.
	///
	/// Events are routed based on their attributes:
	/// - Events with BOTH <see cref="TelemetryEventAttribute"/> AND <see cref="AnalyticsTableGenAttribute"/> go to structured tables.
	/// - Events with only <see cref="TelemetryEventAttribute"/> go to raw_telemetry_events (if EnableUnstructuredPath is true).
	/// - Events without <see cref="TelemetryEventAttribute"/> are dropped.
	/// </summary>
	public sealed class ClickHouseTelemetrySink : ITelemetrySink, IHostedService, IAsyncDisposable
	{
		readonly IOptionsMonitor<ClickHouseTelemetryConfig> _globalConfig;
		readonly IOptionsMonitor<ClickHouseTelemetryServerConfig> _serverConfig;
		readonly ITelemetrySchemaCache _schemaCache;
		readonly IAuthenticationProvider<ClickHouseTelemetryConfig> _authProvider;
		readonly ISqlDialect _dialect = new ClickHouseSqlDialect();
		readonly ITicker _flushTicker;
		readonly ILogger _logger;
		readonly JsonSerializerOptions _jsonOptions;

		readonly ConcurrentDictionary<string, bool> _tableExistsCache = new();
		readonly ConcurrentDictionary<string, bool> _databaseExistsCache = new();
		readonly ConcurrentQueue<QueuedEvent> _eventQueue = new();
		readonly IDisposable? _configChangeSubscription;

		HashSet<string> _eventAllowList;

		ClickHouseConnection? _connection;
		HttpClient? _connectionHttpClient;
		bool _disposed;

		/// <inheritdoc/>
		public bool Enabled => !String.IsNullOrEmpty(_globalConfig.CurrentValue.ConnectionString) && (_serverConfig.CurrentValue.Enabled.HasValue && _serverConfig.CurrentValue.Enabled.Value);

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="config">The config to use throughout.</param>
		/// <param name="serverConfig">The server config to use throughout.</param>
		/// <param name="schemaCache">The schema cache.</param>
		/// <param name="authProvider">The auth provider to use for for connections.</param>
		/// <param name="clock">The clock for timed events.</param>
		/// <param name="logger">The logger to use throughout.</param>
		public ClickHouseTelemetrySink(IOptionsMonitor<ClickHouseTelemetryConfig> config, IOptionsMonitor<ClickHouseTelemetryServerConfig> serverConfig, ITelemetrySchemaCache schemaCache, IAuthenticationProvider<ClickHouseTelemetryConfig> authProvider, IClock clock, ILogger<ClickHouseTelemetrySink> logger)
		{
			_globalConfig = config;
			_serverConfig = serverConfig;
			_schemaCache = schemaCache;
			_authProvider = authProvider;
			_logger = logger;

			_eventAllowList = BuildAllowList(_globalConfig.CurrentValue.AllowList);
			TimeSpan flushInterval = TimeSpan.FromSeconds(_globalConfig.CurrentValue.FlushIntervalSeconds > 0 ? _globalConfig.CurrentValue.FlushIntervalSeconds : 5);
			_flushTicker = clock.AddTicker<ClickHouseTelemetrySink>(flushInterval, FlushInternalAsync, logger);

			_jsonOptions = new JsonSerializerOptions();
			JsonUtils.ConfigureJsonSerializer(_jsonOptions);

			_configChangeSubscription = _globalConfig.OnChange((ClickHouseTelemetryConfig config) =>
			{
				_eventAllowList = BuildAllowList(config.AllowList);
			});
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			if (!Enabled)
			{
				_logger.LogInformation("ClickHouseTelemetrySink is disabled");
				return;
			}

			try
			{
				string connectionString = _globalConfig.CurrentValue.ConnectionString;

				_connectionHttpClient = ClickHouseHttpClientFactory.Create(_authProvider);
				_connection = new ClickHouseConnection(connectionString, _connectionHttpClient);

				await _connection.OpenAsync(cancellationToken);

				_logger.LogInformation("Connected to ClickHouse at {ConnectionString}", SanitizeConnectionString(connectionString));

				if (_globalConfig.CurrentValue.AutoCreateTables)
				{
					// Ensure the default database exists before creating any table in it.
					// Per-schema databases are handled lazily inside EnsureTableExistsFromSchemaAsync.
					await EnsureUnstructuredTableExistsAsync(cancellationToken);
				}

				await _flushTicker.StartAsync();
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Failed to connect to ClickHouse");
			}
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _flushTicker.StopAsync();
			await FlushAsync(cancellationToken);

			if (_connection != null)
			{
				await _connection.CloseAsync();
			}
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_disposed)
			{
				return;
			}

			_disposed = true;
			_configChangeSubscription?.Dispose();

			await _flushTicker.DisposeAsync();

			if (_connection != null)
			{
				await _connection.DisposeAsync();
			}

			// Dispose the HttpClient (and its inner BearerTokenInjectingHandler) only
			// after the ClickHouseConnection is gone — the connection drains in-flight
			// requests during DisposeAsync and they need the transport alive.
			_connectionHttpClient?.Dispose();
		}

		/// <inheritdoc/>
		public async ValueTask FlushAsync(CancellationToken cancellationToken)
		{
			await FlushInternalAsync(cancellationToken);
		}

		async ValueTask FlushInternalAsync(CancellationToken cancellationToken)
		{
			if (_connection == null || _connection.State != System.Data.ConnectionState.Open)
			{
				return;
			}

			// Drain the queue into batches by table
			Dictionary<string, List<QueuedEvent>> batches = new();
			while (_eventQueue.TryDequeue(out QueuedEvent? evt))
			{
				if (!batches.TryGetValue(evt.TableName, out List<QueuedEvent>? batch))
				{
					batch = new List<QueuedEvent>();
					batches[evt.TableName] = batch;
				}
				batch.Add(evt);
			}

			// Insert each batch
			foreach ((string tableName, List<QueuedEvent> events) in batches)
			{
				try
				{
					// Get schema name from the first event (all events in batch have same table/schema)
					string? schemaName = events.FirstOrDefault()?.SchemaName;
					await InsertBatchAsync(tableName, schemaName, events, cancellationToken);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Failed to insert batch into table {TableName}", tableName);
					// Re-queue failed events
					foreach (QueuedEvent evt in events)
					{
						_eventQueue.Enqueue(evt);
					}
				}
			}
		}

		/// <inheritdoc/>
		public void SendEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{
			if (!Enabled || _connection == null)
			{
				return;
			}

			object payload = telemetryEvent.Payload;
			if (payload == null)
			{
				return;
			}

			// Handle JsonElement payloads (from REST API) - keep existing behavior for now
			if (payload is JsonElement jsonElement)
			{
				SendJsonEvent(telemetryStoreId, telemetryEvent, jsonElement);
				return;
			}

			// Get event name from payload
			string? eventName = GetEventName(payload);
			if (eventName == null)
			{
				return;
			}

			// Optional config allowlist
			if (_eventAllowList.Count > 0 && !_eventAllowList.Contains(eventName))
			{
				return;
			}

			// Warm cache lookup via the shared ITelemetrySchemaCache — no blocking, no
			// MongoDB call on the hot path. Cache freshness is bounded by the cache
			// service's refresh interval (cluster-wide).
			if (_schemaCache.Schemas.TryGetValue(eventName, out ITelemetrySchema? telemetrySchema) && telemetrySchema != null)
			{
				SendEventFromSchema(telemetryStoreId, telemetryEvent, telemetrySchema);
				return;
			}

			// UNSTRUCTURED PATH: Route to raw table (if enabled). Also covers events whose
			// schema is mid-propagation across the cluster — they land in the unstructured
			// table rather than being lost.
			if (_globalConfig.CurrentValue.EnableUnstructuredPath)
			{
				SendUnstructuredEvent(telemetryStoreId, telemetryEvent, eventName, payload);
			}
		}

		/// <summary>
		/// Handles JSON payloads from REST API
		/// </summary>
		void SendJsonEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent, JsonElement jsonElement)
		{
			// Extract event name from JSON
			string? eventName = null;
			if (jsonElement.TryGetProperty("EventName", out JsonElement eventNameElement))
			{
				eventName = eventNameElement.GetString();
			}
			else if (jsonElement.TryGetProperty("eventName", out eventNameElement))
			{
				eventName = eventNameElement.GetString();
			}

			if (String.IsNullOrEmpty(eventName))
			{
				_logger.LogDebug("JSON event missing EventName property, skipping");
				return;
			}

			// Optional config allowlist
			if (_eventAllowList.Count > 0 && !_eventAllowList.Contains(eventName))
			{
				return;
			}

			// Warm cache lookup via the shared ITelemetrySchemaCache.
			if (_schemaCache.Schemas.TryGetValue(eventName, out ITelemetrySchema? telemetrySchema) && telemetrySchema != null)
			{
				SendJsonEventFromSchema(telemetryStoreId, telemetryEvent, jsonElement, telemetrySchema);
				return;
			}

			// UNSTRUCTURED PATH: Route to raw table (if enabled). Also covers events whose
			// schema is mid-propagation across the cluster.
			if (_globalConfig.CurrentValue.EnableUnstructuredPath)
			{
				SendUnstructuredJsonEvent(telemetryStoreId, telemetryEvent, eventName, jsonElement);
			}
		}

		/// <summary>
		/// Sends a JSON event using schema from ITelemetrySchemaCollection.
		/// For JSON payloads from REST API when a MongoDB schema exists.
		/// </summary>
		void SendJsonEventFromSchema(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent, JsonElement jsonElement, ITelemetrySchema schema)
		{
			if (jsonElement.ValueKind != JsonValueKind.Object)
			{
				_logger.LogWarning("JSON payload is not an object: {Kind}", jsonElement.ValueKind);
				return;
			}

			// Snapshot the flag once — the IOptionsMonitor read isn't free and the value can't
			// meaningfully change mid-event.
			bool normalize = _globalConfig.CurrentValue.NormalizePropertiesOnIngest;

			// Build case-insensitive lookup from JSON property names
			// JSON uses camelCase (e.g., "jobId"), schema PropertyName is PascalCase (e.g., "JobId")
			Dictionary<string, JsonElement> jsonProps = new(StringComparer.OrdinalIgnoreCase);
			jsonProps.EnsureCapacity(jsonElement.GetPropertyCount());

			foreach (JsonProperty prop in jsonElement.EnumerateObject())
			{
				jsonProps[normalize ? ClickHouseTelemetrySinkHelpers.NormalizePropertyName(prop.Name) : prop.Name] = prop.Value;
			}

			Dictionary<string, object?> values = new(schema.Columns.Count);

			// Extract values using schema column mappings
			foreach (SchemaColumn col in schema.Columns)
			{
				object? value = null;

				// Match by PropertyName (case-insensitive) since JSON may use camelCase. When
				// NormalizePropertiesOnIngest is on, both sides went through NormalizePropertyName,
				// so this lookup still hits even when separators or casing diverge.
				string lookupKey = normalize ? ClickHouseTelemetrySinkHelpers.NormalizePropertyName(col.PropertyName) : col.PropertyName;
				if (jsonProps.TryGetValue(lookupKey, out JsonElement jsonValue))
				{
					value = ConvertJsonValueBySchemaType(jsonValue, col, normalize);
				}

				values[col.ColumnName] = value;
			}

			// Add metadata columns
			AddMetadataColumns(values, telemetryStoreId, telemetryEvent);

			// Ensure table exists (async, fire-and-forget)
			EnsureTableExistsFromSchema(schema);

			_eventQueue.Enqueue(new QueuedEvent(schema.TableName, schema.SchemaName, values));
		}

		/// <summary>
		/// Sends an event using schema from ITelemetrySchemaCollection.
		/// For object payloads from TelemetryManager.
		///
		/// Uses JSON serialization to convert the payload, which ensures consistency with
		/// other telemetry sinks and respects custom JsonConverter attributes.
		/// </summary>
		void SendEventFromSchema(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent, ITelemetrySchema schema)
		{
			// Serialize payload to JSON - this applies all custom converters and naming policies
			using JsonDocument jsonDoc = JsonSerializer.SerializeToDocument(telemetryEvent.Payload, _jsonOptions);
			JsonElement jsonElement = jsonDoc.RootElement;

			if (jsonElement.ValueKind != JsonValueKind.Object)
			{
				_logger.LogWarning("Payload serialized to non-object JSON: {Kind}", jsonElement.ValueKind);
				return;
			}

			// Snapshot the flag once per event so the loop below doesn't re-read the monitor.
			bool normalize = _globalConfig.CurrentValue.NormalizePropertiesOnIngest;

			// Build case-insensitive lookup from JSON property names
			// JSON uses camelCase (e.g., "jobId"), schema PropertyName is PascalCase (e.g., "JobId")
			Dictionary<string, JsonElement> jsonProps = new(StringComparer.OrdinalIgnoreCase);
			jsonProps.EnsureCapacity(jsonElement.GetPropertyCount());

			foreach (JsonProperty prop in jsonElement.EnumerateObject())
			{
				jsonProps[normalize ? ClickHouseTelemetrySinkHelpers.NormalizePropertyName(prop.Name) : prop.Name] = prop.Value;
			}

			Dictionary<string, object?> values = new(schema.Columns.Count);

			// Extract values using schema column mappings
			foreach (SchemaColumn col in schema.Columns)
			{
				object? value = null;

				// Match by PropertyName (case-insensitive) since JSON serialization preserves property names
				// (with camelCase transformation: "JobId" -> "jobId"). When NormalizePropertiesOnIngest is on,
				// both sides were normalised so naming-convention drift between producer and schema stops mattering.
				string lookupKey = normalize ? ClickHouseTelemetrySinkHelpers.NormalizePropertyName(col.PropertyName) : col.PropertyName;
				if (jsonProps.TryGetValue(lookupKey, out JsonElement jsonValue))
				{
					value = ConvertJsonValueBySchemaType(jsonValue, col, normalize);
				}

				values[col.ColumnName] = value;
			}

			// Add metadata columns
			AddMetadataColumns(values, telemetryStoreId, telemetryEvent);

			// Ensure table exists (async, fire-and-forget)
			EnsureTableExistsFromSchema(schema);

			_eventQueue.Enqueue(new QueuedEvent(schema.TableName, schema.SchemaName, values));
		}

		/// <summary>
		/// Sends an unstructured event to the raw staging table
		/// </summary>
		void SendUnstructuredEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent, string eventName, object payload)
		{
			Dictionary<string, object?> values = new Dictionary<string, object?>
			{
				[UnstructuredColumns.EventName] = eventName,
				[UnstructuredColumns.InsertTime] = DateTime.UtcNow.ToString("o"),
				[UnstructuredColumns.TelemetryStoreId] = telemetryStoreId.ToString(),
				[UnstructuredColumns.AppId] = telemetryEvent.RecordMeta.AppId,
				[UnstructuredColumns.AppVersion] = telemetryEvent.RecordMeta.AppVersion,
				[UnstructuredColumns.AppEnvironment] = telemetryEvent.RecordMeta.AppEnvironment,
				[UnstructuredColumns.SessionId] = telemetryEvent.RecordMeta.SessionId,
				[UnstructuredColumns.Payload] = JsonSerializer.Serialize(payload, _jsonOptions)
			};

			_eventQueue.Enqueue(new QueuedEvent(UnstructuredTelemetry.TableName, null, values));
		}

		/// <summary>
		/// Sends an unstructured JSON event to the raw staging table
		/// </summary>
		void SendUnstructuredJsonEvent(TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent, string eventName, JsonElement jsonElement)
		{
			Dictionary<string, object?> values = new Dictionary<string, object?>
			{
				[UnstructuredColumns.EventName] = eventName,
				[UnstructuredColumns.InsertTime] = DateTime.UtcNow.ToString("o"),
				[UnstructuredColumns.TelemetryStoreId] = telemetryStoreId.ToString(),
				[UnstructuredColumns.AppId] = telemetryEvent.RecordMeta.AppId,
				[UnstructuredColumns.AppVersion] = telemetryEvent.RecordMeta.AppVersion,
				[UnstructuredColumns.AppEnvironment] = telemetryEvent.RecordMeta.AppEnvironment,
				[UnstructuredColumns.SessionId] = telemetryEvent.RecordMeta.SessionId,
				[UnstructuredColumns.Payload] = jsonElement.GetRawText()
			};

			_eventQueue.Enqueue(new QueuedEvent(UnstructuredTelemetry.TableName, null, values));
		}

		/// <summary>
		/// Adds standard metadata columns to the values dictionary
		/// </summary>
		static void AddMetadataColumns(Dictionary<string, object?> values, TelemetryStoreId telemetryStoreId, TelemetryEvent telemetryEvent)
		{
			values["_telemetry_store_id"] = telemetryStoreId.ToString();
			values["_app_id"] = telemetryEvent.RecordMeta.AppId;
			values["_app_version"] = telemetryEvent.RecordMeta.AppVersion;
			values["_app_environment"] = telemetryEvent.RecordMeta.AppEnvironment;
			values["_session_id"] = telemetryEvent.RecordMeta.SessionId;

			// Stamp telemetry_timestamp only if upstream hasn't already provided one — typed records inheriting from AbstractTelemetryRecord set it via their ctor, JSON payloads may carry it explicitly. Sink fills the gap for everything else.
			if (!values.ContainsKey(AbstractTelemetryRecord.TelemetryTimestampColumnName))
			{
				values[AbstractTelemetryRecord.TelemetryTimestampColumnName] = DateTime.UtcNow;
			}
		}

		/// <summary>
		/// Gets the event name from a payload via TelemetryEventAttribute or EventName property
		/// </summary>
		static string? GetEventName(object payload)
		{
			// Check for TelemetryEventAttribute on the type
			TelemetryEventAttribute? attr = payload.GetType().GetCustomAttributes(typeof(TelemetryEventAttribute), false)
				.OfType<TelemetryEventAttribute>()
				.FirstOrDefault();

			if (attr != null)
			{
				return attr.EventName;
			}

			// Fallback: Try to get EventName property
			System.Reflection.PropertyInfo? eventNameProp = payload.GetType().GetProperty("EventName");
			return eventNameProp?.GetValue(payload) as string;
		}

		#region -- Conversion Methods --

		/// <summary>
		/// Converts a JsonElement value based on SchemaColumn.
		/// Used when extracting values from JSON-serialized payloads.
		/// For nested objects, uses the nested schema to serialize with correct column names.
		/// </summary>
		/// <param name="normalize">
		/// Forwarded to the nested-object converter so deep schemas keep the same matching
		/// semantics as the top-level call. Other DataType branches don't do property matching
		/// and ignore it.
		/// </param>
		/// <param name="element">The element to convert.</param>
		/// <param name="col">The column reference to use.</param>
		object? ConvertJsonValueBySchemaType(JsonElement element, SchemaColumn col, bool normalize)
		{
			if (element.ValueKind == JsonValueKind.Null || element.ValueKind == JsonValueKind.Undefined)
			{
				return null;
			}

			return col.DataType switch
			{
				SchemaDataType.String => ClickHouseTelemetrySinkHelpers.ConvertJsonToString(element),
				SchemaDataType.Int64 => ClickHouseTelemetrySinkHelpers.ConvertJsonToInt64(element, _logger),
				SchemaDataType.Double => ClickHouseTelemetrySinkHelpers.ConvertJsonToDouble(element),
				SchemaDataType.Bool => ClickHouseTelemetrySinkHelpers.ConvertToBool(element),
				SchemaDataType.DateTime => ClickHouseTelemetrySinkHelpers.ConvertJsonToDateTime(element),
				SchemaDataType.Object => ClickHouseTelemetrySinkHelpers.ConvertNestedObject(element, col.NestedSchemaEventName, _schemaCache, _logger, normalize),
				SchemaDataType.Array => ClickHouseTelemetrySinkHelpers.ConvertJsonArray(element, col.ArrayElementType, _logger),
				_ => element.GetRawText()
			};
		}

		#endregion -- Conversion Methods --

		/// <summary>
		/// Ensures a specific database exists (with caching)
		/// </summary>
		async Task EnsureDatabaseExistsInternalAsync(string databaseName, CancellationToken cancellationToken)
		{
			if (_connection == null || _databaseExistsCache.ContainsKey(databaseName))
			{
				return;
			}

			string sql = $"CREATE DATABASE IF NOT EXISTS `{databaseName}`";
			using ClickHouseCommand cmd = _connection.CreateCommand();

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
			cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities

			await cmd.ExecuteNonQueryAsync(cancellationToken);

			_databaseExistsCache[databaseName] = true;
		}

		/// <summary>
		/// Ensures the unstructured telemetry table exists
		/// </summary>
		async Task EnsureUnstructuredTableExistsAsync(CancellationToken cancellationToken)
		{
			if (_connection == null || !_globalConfig.CurrentValue.EnableUnstructuredPath)
			{
				return;
			}

			string sql = UnstructuredTelemetry.GetCreateTableSql(_globalConfig.CurrentValue.DefaultDatabaseName);

			using ClickHouseCommand cmd = _connection.CreateCommand();

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
			cmd.CommandText = sql;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities

			await cmd.ExecuteNonQueryAsync(cancellationToken);

			_tableExistsCache[UnstructuredTelemetry.TableName] = true;
			_logger.LogInformation("Ensured unstructured telemetry table exists: {TableName}", UnstructuredTelemetry.TableName);
		}

		/// <summary>
		/// Ensures table exists based on ITelemetrySchema (fire-and-forget)
		/// </summary>
		void EnsureTableExistsFromSchema(ITelemetrySchema schema)
		{
			if (!_globalConfig.CurrentValue.AutoCreateTables)
			{
				return;
			}

			if (_tableExistsCache.ContainsKey(schema.TableName))
			{
				return;
			}

			Task.Run(async () =>
			{
				try
				{
					await EnsureTableExistsFromSchemaAsync(schema, CancellationToken.None);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Failed to create table {TableName}", schema.TableName);
				}
			});
		}

		async Task EnsureTableExistsFromSchemaAsync(ITelemetrySchema schema, CancellationToken cancellationToken)
		{
			if (_connection == null)
			{
				return;
			}

			// Admin-authored schemas leave SchemaName null on purpose (the field is optional
			// in the authoring form). Fall back to the configured default database so those
			// tables land somewhere sensible instead of being silently skipped.
			string databaseName = schema.SchemaName ?? _globalConfig.CurrentValue.DefaultDatabaseName;
			string quotedTableName = _dialect.FormatTableName(databaseName, schema.TableName);

			await EnsureDatabaseExistsInternalAsync(databaseName, cancellationToken);

			string ddl = ClickHouseTypeMapper.GetCreateTableDdl(schema, quotedTableName);

			using ClickHouseCommand cmd = _connection.CreateCommand();

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
			cmd.CommandText = ddl;
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities

			await cmd.ExecuteNonQueryAsync(cancellationToken);

			_tableExistsCache[schema.TableName] = true;
			_logger.LogInformation("Created table {TableName} from schema with {ColumnCount} columns", schema.TableName, schema.Columns.Count);
		}

		/// <summary>
		/// Inserts a batch of events into a table
		/// </summary>
		async Task InsertBatchAsync(string tableName, string? schemaName, List<QueuedEvent> events, CancellationToken cancellationToken)
		{
			if (_connection == null || events.Count == 0)
			{
				return;
			}

			// Get all column names from the events
			HashSet<string> allColumns = new HashSet<string>();
			foreach (QueuedEvent evt in events)
			{
				foreach (string col in evt.Values.Keys)
				{
					allColumns.Add(col);
				}
			}

			List<string> columnList = allColumns.ToList();

			string effectiveSchemaName = schemaName ?? _globalConfig.CurrentValue.DefaultDatabaseName;
			string destinationTable = _dialect.FormatTableName(effectiveSchemaName, tableName);

			// Query the table to get actual column names
			List<string> tableColumns = await GetTableColumnsAsync(destinationTable, cancellationToken);
			HashSet<string> tableColumnSet = new HashSet<string>(tableColumns, StringComparer.OrdinalIgnoreCase);

			// Filter to only columns that exist in the table
			List<string> validColumns = columnList.Where(c => tableColumnSet.Contains(c)).ToList();

			if (validColumns.Count == 0)
			{
				_logger.LogWarning("No matching columns between event data and table {TableName}. Event columns: [{EventCols}], Table columns: [{TableCols}]",
					tableName,
					String.Join(", ", columnList),
					String.Join(", ", tableColumns));
				return;
			}

			using ClickHouseBulkCopy bulkCopy = new ClickHouseBulkCopy(_connection)
			{
				DestinationTableName = destinationTable,
				ColumnNames = validColumns,
				BatchSize = _globalConfig.CurrentValue.BatchSize
			};

			await bulkCopy.InitAsync();

			IEnumerable<object?[]> rows = events.Select(evt =>
			{
				object?[] row = new object?[validColumns.Count];
				for (int i = 0; i < validColumns.Count; i++)
				{
					evt.Values.TryGetValue(validColumns[i], out row[i]);
				}
				return row;
			});

			try
			{
				await bulkCopy.WriteToServerAsync(rows, cancellationToken);
				_logger.LogDebug("Inserted {Count} rows into {TableName}", events.Count, tableName);
			}
			catch (Exception ex)
			{
				// Log detailed info about what we tried to insert
				_logger.LogError(ex, "Failed to insert into {TableName}. Columns: [{Columns}]. First row values: [{Values}]",
					tableName,
					String.Join(", ", validColumns),
					String.Join(", ", events.FirstOrDefault()?.Values.Select(kv => $"{kv.Key}={kv.Value?.GetType().Name ?? "null"}:{kv.Value}") ?? Array.Empty<string>()));
				throw;
			}
		}

		/// <summary>
		/// Gets column names for a table
		/// </summary>
		async Task<List<string>> GetTableColumnsAsync(string quotedTableName, CancellationToken cancellationToken)
		{
			List<string> columns = new List<string>();
			if (_connection == null)
			{
				return columns;
			}

			using ClickHouseCommand cmd = _connection.CreateCommand();

#pragma warning disable CA2100 // Review SQL queries for security vulnerabilities
			cmd.CommandText = $"DESCRIBE TABLE {quotedTableName}";
#pragma warning restore CA2100 // Review SQL queries for security vulnerabilities

			using DbDataReader reader = await cmd.ExecuteReaderAsync(cancellationToken);
			while (await reader.ReadAsync(cancellationToken))
			{
				columns.Add(reader.GetString(0));
			}

			return columns;
		}

		/// <summary>
		/// Sanitizes a connection string for logging (removes password)
		/// </summary>
		static string SanitizeConnectionString(string connectionString)
		{
			return System.Text.RegularExpressions.Regex.Replace(connectionString, @"(Password|Pwd)\s*=\s*[^;]+", "$1=***", System.Text.RegularExpressions.RegexOptions.IgnoreCase);
		}

		private static HashSet<string> BuildAllowList(List<string> allowList)
		{
			return allowList == null ? new HashSet<string>(StringComparer.OrdinalIgnoreCase) : new HashSet<string>(allowList, StringComparer.OrdinalIgnoreCase);
		}

		record QueuedEvent(string TableName, string? SchemaName, Dictionary<string, object?> Values);
	}
}
