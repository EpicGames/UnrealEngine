// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using System.Text.Json;
using HordeServer.Analytics.Schemas;
using Microsoft.Extensions.Logging;

namespace HordeServer.ClickHouseTelemetry
{
	internal static class ClickHouseTelemetrySinkHelpers
	{
		/// <summary>
		/// Normalises a property name for ingest matching when <c>NormalizePropertiesOnIngest</c>
		/// is enabled on the sink config. Lowercases and strips '.', '_' and '-' so producer
		/// naming conventions (camelCase / snake_case / kebab-case / dot.case) all collapse to
		/// the same key. Lives here rather than on the sink so the nested-object helper can use
		/// the same routine when it does its own property matching.
		/// </summary>
		internal static string NormalizePropertyName(string name)
		{
			StringBuilder sb = new StringBuilder(name.Length);
			foreach (char c in name)
			{
				if (c == '.' || c == '_' || c == '-')
				{
					continue;
				}
				sb.Append(Char.ToLowerInvariant(c));
			}
			return sb.ToString();
		}

		/// <summary>
		///
		/// </summary>
		/// <param name="element"></param>
		/// <param name="elementType"></param>
		/// <param name="logger"></param>
		/// <returns></returns>
		internal static object? ConvertJsonArray(JsonElement element, SchemaDataType? elementType, ILogger logger)
		{
			if (element.ValueKind != JsonValueKind.Array)
			{
				return element.GetRawText();
			}

			// When an element type is declared on the schema, validate each element matches
			// before serializing. A single bad element drops the whole value to null with a
			// warning — partial arrays would silently corrupt downstream aggregations.
			// An undeclared elementType preserves the prior pass-through behaviour.
			if (elementType.HasValue)
			{
				int index = 0;
				foreach (JsonElement item in element.EnumerateArray())
				{
					if (!ElementMatchesSchemaType(item, elementType.Value))
					{
						logger.LogWarning(
							"ConvertJsonArray: rejecting array — element[{Index}] kind={Kind} does not match expected {Expected}",
							index, item.ValueKind, elementType.Value);
						return null;
					}
					index++;
				}
			}

			// Serialize arrays as JSON strings for consistency. Writers produce JSON and readers
			// parse JSON — no ClickHouse native-array format conversion needed.
			return element.GetRawText();
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="element"></param>
		/// <returns></returns>
		internal static DateTime? ConvertJsonToDateTime(JsonElement element)
		{
			if (element.ValueKind == JsonValueKind.String)
			{
				string? str = element.GetString();
				if (str != null && DateTime.TryParse(str, out DateTime dt))
				{
					return dt;
				}
			}

			return null;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="element"></param>
		/// <returns></returns>
		internal static double? ConvertJsonToDouble(JsonElement element)
		{
			if (element.ValueKind == JsonValueKind.Number)
			{
				if (element.TryGetDouble(out double d))
				{
					return d;
				}
			}
			else if (element.ValueKind == JsonValueKind.String)
			{
				if (Double.TryParse(element.GetString(), out double parsed))
				{
					return parsed;
				}
			}

			return null;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="element"></param>
		/// <returns></returns>
		internal static bool? ConvertToBool(JsonElement element)
		{
			return element.ValueKind == JsonValueKind.True ? true : element.ValueKind == JsonValueKind.False ? false : null;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="element"></param>
		/// <param name="logger"></param>
		/// <returns></returns>
		internal static long? ConvertJsonToInt64(JsonElement element, ILogger logger)
		{
			if (element.ValueKind == JsonValueKind.Number)
			{
				if (element.TryGetInt64(out long l))
				{
					return l;
				}

				// JSON numbers that don't fit Int64 directly: accept only integral doubles
				// in range. Non-integral (e.g. 1.5) and out-of-range values are rejected
				// with a warning rather than silently truncated, since those losses are
				// not visible to the producer otherwise.
				if (element.TryGetDouble(out double d))
				{
					if (!Double.IsFinite(d) || d < Int64.MinValue || d > Int64.MaxValue)
					{
						logger.LogWarning("ConvertJsonToInt64: rejecting out-of-range value {Value} for Int64 column", d);
						return null;
					}

					long truncated = (long)d;
					if (truncated == d)
					{
						return truncated;
					}

					logger.LogWarning("ConvertJsonToInt64: rejecting non-integral value {Value} for Int64 column (would lose precision)", d);
					return null;
				}
			}
			else if (element.ValueKind == JsonValueKind.String)
			{
				// Handle string-encoded numbers (e.g., TimeSpan serialized as ticks string)
				if (Int64.TryParse(element.GetString(), out long parsed))
				{
					return parsed;
				}
			}

			return null;
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="element"></param>
		/// <returns></returns>
		internal static string? ConvertJsonToString(JsonElement element)
		{
			return element.ValueKind switch
			{
				JsonValueKind.String => element.GetString(),
				JsonValueKind.Number => element.GetRawText(),
				JsonValueKind.True => "true",
				JsonValueKind.False => "false",
				_ => element.GetRawText()
			};
		}

		/// <summary>
		/// Tests whether a single array element's JSON kind is compatible with the declared
		/// schema element type. Object/Array element types accept any object/array shape and
		/// rely on downstream validation; null is always permitted.
		/// </summary>
		internal static bool ElementMatchesSchemaType(JsonElement item, SchemaDataType elementType)
		{
			if (item.ValueKind == JsonValueKind.Null || item.ValueKind == JsonValueKind.Undefined)
			{
				return true;
			}

			return elementType switch
			{
				SchemaDataType.String => item.ValueKind == JsonValueKind.String,
				SchemaDataType.Int64 => item.ValueKind == JsonValueKind.Number && item.TryGetInt64(out _),
				SchemaDataType.Double => item.ValueKind == JsonValueKind.Number,
				SchemaDataType.Bool => item.ValueKind == JsonValueKind.True || item.ValueKind == JsonValueKind.False,
				SchemaDataType.DateTime => item.ValueKind == JsonValueKind.String && DateTime.TryParse(item.GetString(), out _),
				SchemaDataType.Object => item.ValueKind == JsonValueKind.Object,
				SchemaDataType.Array => item.ValueKind == JsonValueKind.Array,
				_ => true
			};
		}

		/// <summary>
		/// Converts a nested object JSON element using schema column names.
		/// Uses the warm schema cache for nested schema resolution (no blocking, no MongoDB).
		/// </summary>
		internal static string ConvertNestedObject(JsonElement element, string? nestedSchemaEventName, ITelemetrySchemaCache schemaCache, ILogger logger, bool normalize = false)
		{
			if (element.ValueKind != JsonValueKind.Object || String.IsNullOrEmpty(nestedSchemaEventName))
			{
				logger.LogDebug("ConvertNestedObject falling back to raw JSON: kind={Kind}, schemaName={SchemaName}", element.ValueKind, nestedSchemaEventName ?? "(null)");
				return element.GetRawText();
			}

			if (!schemaCache.Schemas.TryGetValue(nestedSchemaEventName, out ITelemetrySchema? nestedSchema) || nestedSchema == null)
			{
				logger.LogDebug("ConvertNestedObject: no schema found for {EventName}, falling back to raw JSON", nestedSchemaEventName);
				return element.GetRawText();
			}

			// Build a case-insensitive lookup of incoming JSON properties so PascalCase column
			// PropertyNames match regardless of the producer's JSON naming convention. When
			// `normalize` is on, both sides of the match also collapse separators and casing so
			// snake_case / kebab-case / dot.case producers line up with PascalCase schemas.
			Dictionary<string, JsonElement> jsonProps = new(StringComparer.OrdinalIgnoreCase);
			jsonProps.EnsureCapacity(element.GetPropertyCount());
			foreach (JsonProperty prop in element.EnumerateObject())
			{
				jsonProps[normalize ? NormalizePropertyName(prop.Name) : prop.Name] = prop.Value;
			}

			// Build JSON object with column names as keys instead of property names.
			// Use Utf8JsonWriter to avoid double-encoding nested objects.
			using MemoryStream ms = new();
			using (Utf8JsonWriter writer = new(ms))
			{
				writer.WriteStartObject();
				foreach (SchemaColumn col in nestedSchema.Columns)
				{
					string lookupKey = normalize ? NormalizePropertyName(col.PropertyName) : col.PropertyName;
					if (jsonProps.TryGetValue(lookupKey, out JsonElement propValue))
					{
						writer.WritePropertyName(col.ColumnName);
						if (col.DataType == SchemaDataType.Object && !String.IsNullOrEmpty(col.NestedSchemaEventName))
						{
							// Propagate `normalize` recursively so deep nesting stays consistent.
							string nestedJson = ConvertNestedObject(propValue, col.NestedSchemaEventName, schemaCache, logger, normalize);
							writer.WriteRawValue(nestedJson);
						}
						else
						{
							propValue.WriteTo(writer);
						}
					}
				}
				writer.WriteEndObject();
			}

			return Encoding.UTF8.GetString(ms.ToArray());
		}
	}
}