// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Analytics;
using HordeServer.Analytics.Schemas;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Maps fundamental schema types to ClickHouse types
	/// </summary>
	public static class ClickHouseTypeMapper
	{
		/// <summary>
		/// Maps a SchemaDataType to a ClickHouse type string
		/// </summary>
		/// <param name="dataType">The fundamental schema type</param>
		/// <param name="isNullable">Whether the type is nullable</param>
		/// <returns>The ClickHouse type string</returns>
		public static string MapToClickHouseType(SchemaDataType dataType, bool isNullable)
		{
			string baseType = dataType switch
			{
				SchemaDataType.String => "String",
				SchemaDataType.Int64 => "Int64",
				SchemaDataType.Double => "Float64",
				SchemaDataType.Bool => "Bool",
				SchemaDataType.DateTime => "DateTime64(3)", // millisecond precision
				SchemaDataType.Object => "String", // Complex objects stored as JSON string
				SchemaDataType.Array => "String", // Arrays stored as JSON string for ODBC compatibility
				_ => "String"
			};

			if (isNullable)
			{
				return $"Nullable({baseType})";
			}

			return baseType;
		}

		/// <summary>
		/// Maps a SchemaColumn to its ClickHouse type string
		/// </summary>
		public static string MapColumnToClickHouseType(SchemaColumn column)
		{
			return MapToClickHouseType(column.DataType, column.IsNullable);
		}

		/// <summary>
		/// Appends the standard metadata column definitions to a StringBuilder. These columns are added to every telemetry table the sink manages. The <see cref="AbstractTelemetryRecord.TelemetryTimestampColumnName"/> column is conditionally emitted only when the supplied schema doesn't already declare it (which it does for every <see cref="AbstractTelemetryRecord"/>-derived type via codegen).
		/// </summary>
		/// <param name="sb">The StringBuilder to append to.</param>
		/// <param name="alreadyDeclaredColumns">Column names already present in the schema being CREATE TABLE'd. Used to dedup metadata columns the schema has already provided. Pass null for tables with no schema-side declarations (e.g. the unstructured staging table).</param>
		public static void AppendMetadataColumnsDdl(System.Text.StringBuilder sb, IReadOnlySet<string>? alreadyDeclaredColumns = null)
		{
			sb.AppendLine("    _telemetry_store_id String,");
			sb.AppendLine("    _app_id Nullable(String),");
			sb.AppendLine("    _app_version Nullable(String),");
			sb.AppendLine("    _app_environment Nullable(String),");
			sb.AppendLine("    _session_id Nullable(String),");

			// telemetry_timestamp is the canonical wire-format column carried on AbstractTelemetryRecord. Codegen-derived schemas already declare it; the unstructured / no-schema case doesn't, in which case the sink enforces it here.
			if (alreadyDeclaredColumns == null || !alreadyDeclaredColumns.Contains(AbstractTelemetryRecord.TelemetryTimestampColumnName))
			{
				sb.AppendLine($"    {AbstractTelemetryRecord.TelemetryTimestampColumnName} DateTime64(3),");
			}
		}

		/// <summary>
		/// Appends the standard table options (ENGINE, ORDER BY, PARTITION BY) to a StringBuilder.
		/// </summary>
		/// <param name="sb">The StringBuilder to append to</param>
		public static void AppendTableOptionsDdl(System.Text.StringBuilder sb)
		{
			sb.AppendLine(") ENGINE = MergeTree()");
			sb.AppendLine($"ORDER BY ({AbstractTelemetryRecord.TelemetryTimestampColumnName}, _telemetry_store_id)");
			sb.AppendLine($"PARTITION BY toYYYYMM({AbstractTelemetryRecord.TelemetryTimestampColumnName})");
		}

		/// <summary>
		/// Gets the CREATE TABLE DDL for a schema
		/// </summary>
		public static string GetCreateTableDdl(ITelemetrySchema schema, string quotedTableName)
		{
			System.Text.StringBuilder sb = new System.Text.StringBuilder();
			sb.AppendLine($"CREATE TABLE IF NOT EXISTS {quotedTableName} (");

			// Pre-compute the schema's column names so the metadata DDL can dedup against any column the schema already declares (notably telemetry_timestamp on AbstractTelemetryRecord-derived types).
			HashSet<string> declaredColumnNames = new HashSet<string>(schema.Columns.Select(c => c.ColumnName), StringComparer.OrdinalIgnoreCase);

			// Metadata columns first
			AppendMetadataColumnsDdl(sb, declaredColumnNames);

			// Schema columns
			List<SchemaColumn> columns = schema.Columns.ToList();
			for (int i = 0; i < columns.Count; i++)
			{
				SchemaColumn col = columns[i];
				string clickHouseType = MapColumnToClickHouseType(col);
				sb.Append($"    `{col.ColumnName}` {clickHouseType}");
				if (i < columns.Count - 1)
				{
					sb.Append(',');
				}
				sb.AppendLine();
			}

			AppendTableOptionsDdl(sb);

			return sb.ToString();
		}
	}
}
