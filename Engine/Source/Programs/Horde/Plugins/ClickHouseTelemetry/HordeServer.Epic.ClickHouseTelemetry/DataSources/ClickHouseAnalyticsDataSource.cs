// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data.Common;
using ClickHouse.Client.ADO;
using HordeServer.Analytics;
using HordeServer.ClickHouseTelemetry;
using HordeServer.Dialects;
using Microsoft.Extensions.Options;

namespace HordeServer.DataSources
{
	/// <summary>
	/// ClickHouse-native implementation of <see cref="IAnalyticsDataSource"/>. Mirrors <c>ClickHouseTelemetrySink</c>'s connection pattern: a single long-lived <see cref="HttpClient"/> wraps a <see cref="BearerTokenInjectingHandler"/> (so OAuth tokens are folded as <c>Authorization: Bearer</c> headers when the provider is configured), and each <see cref="OpenAsync"/> creates a fresh <see cref="ClickHouseConnection"/> against that shared HttpClient.
	///
	/// Connection-string source: <see cref="ClickHouseTelemetryConfig.ConnectionString"/> — the same field the sink uses, so query and write paths point at the same cluster by construction.
	/// </summary>
	public sealed class ClickHouseAnalyticsDataSource : IAnalyticsDataSource, IAsyncDisposable, IDisposable
	{
		private readonly IOptionsMonitor<ClickHouseTelemetryConfig> _config;
		private readonly ISqlDialect _dialect = new ClickHouseSqlDialect();
		private readonly HttpClient _httpClient;
		private bool _disposed;

		/// <inheritdoc/>
		public ISqlDialect Dialect => _dialect;

		/// <summary>
		/// Constructor.
		/// </summary>
		public ClickHouseAnalyticsDataSource(IOptionsMonitor<ClickHouseTelemetryConfig> config, IAuthenticationProvider<ClickHouseTelemetryConfig> authProvider)
		{
			_config = config;
			_httpClient = ClickHouseHttpClientFactory.Create(authProvider);
		}

		/// <inheritdoc/>
		public Task<DbConnection> OpenAsync(CancellationToken cancellationToken = default)
		{
			string connectionString = _config.CurrentValue.ConnectionString;
			if (String.IsNullOrEmpty(connectionString))
			{
				throw new InvalidOperationException($"ClickHouseTelemetry is enabled as the analytics data source but {nameof(ClickHouseTelemetryConfig.ConnectionString)} is not configured.");
			}

			ClickHouseConnection connection = new ClickHouseConnection(connectionString, _httpClient);
			return OpenInternalAsync(connection, cancellationToken);
		}

		private static async Task<DbConnection> OpenInternalAsync(ClickHouseConnection connection, CancellationToken cancellationToken)
		{
			try
			{
				await connection.OpenAsync(cancellationToken);
				return connection;
			}
			catch
			{
				connection.Dispose();
				throw;
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			if (_disposed)
			{
				return;
			}

			_httpClient.Dispose();
			_disposed = true;
			GC.SuppressFinalize(this);
		}

		/// <inheritdoc/>
		public ValueTask DisposeAsync()
		{
			Dispose();
			return ValueTask.CompletedTask;
		}
	}
}
