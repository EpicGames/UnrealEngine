// Copyright Epic Games, Inc. All Rights Reserved.

using System.Data;
using System.Data.Common;

namespace HordeServer.Dialects
{
	/// <summary>
	/// SQL dialect for ClickHouse's native ADO.NET driver. Emits parameter placeholders as <c>{name:Type}</c> per ClickHouse's HTTP/native protocol convention, and binds values via the command's own <see cref="DbCommand.CreateParameter"/> factory so the driver materialises <c>ClickHouseParameter</c> instances.
	/// </summary>
	public sealed class ClickHouseSqlDialect : ISqlDialect
	{
		/// <summary>
		/// Standard name of the dialect.
		/// </summary>
		public const string DialectIdentifier = "ClickHouse";

		/// <inheritdoc/>
		public string FormatParameterPlaceholder(string canonicalName, DbType dbType)
		{
			string clickHouseTypeName = MapDbTypeToClickHouseTypeName(dbType);
			return $"{{{canonicalName}:{clickHouseTypeName}}}";
		}

		/// <inheritdoc/>
		public void AddParameter(DbCommand command, string canonicalName, object? value, DbType dbType)
		{
			DbParameter parameter = command.CreateParameter();
			parameter.ParameterName = canonicalName;
			parameter.DbType = dbType;
			parameter.Value = value ?? DBNull.Value;
			command.Parameters.Add(parameter);
		}

		/// <inheritdoc/>
		public string QuoteIdentifier(string identifier) => $"`{identifier}`";

		/// <inheritdoc/>
		public string FormatTableName(string? schemaName, string tableName)
		{
			// Logical table names from the schema document may contain dots (e.g. "horde.state_job_summary"),
			// which ClickHouse parses as a database.table separator and chokes on. The dialect projects the
			// logical name to a physical name by replacing in-name dots with underscores so the FROM clause
			// is plain "schema.table_name" — no backticks, no driver-specific quoting, no parser ambiguity.
			// The schema/database separator dot between schemaName and tableName is preserved.
			string physicalTableName = tableName.Replace('.', '_');
			return String.IsNullOrEmpty(schemaName) ? physicalTableName : $"{schemaName}.{physicalTableName}";
		}

		/// <summary>
		/// Maps a <see cref="DbType"/> to the ClickHouse type name used in <c>{name:Type}</c> placeholders. Covers the types used by today's analytics queries (per the Phase 0 audit); additional types can be added when new queries demand them.
		/// </summary>
		private static string MapDbTypeToClickHouseTypeName(DbType dbType) => dbType switch
		{
			DbType.AnsiString or DbType.String or DbType.AnsiStringFixedLength or DbType.StringFixedLength => "String",
			DbType.Int32 => "Int32",
			DbType.Int64 => "Int64",
			DbType.Int16 => "Int16",
			DbType.Byte => "UInt8",
			DbType.Boolean => "UInt8",
			DbType.Single => "Float32",
			DbType.Double => "Float64",
			DbType.Decimal => "Decimal128(9)",
			DbType.DateTime or DbType.DateTime2 => "DateTime64(3)",
			DbType.DateTimeOffset => "DateTime64(3, 'UTC')",
			DbType.Date => "Date",
			DbType.Guid => "UUID",
			_ => "String"
		};
	}
}
