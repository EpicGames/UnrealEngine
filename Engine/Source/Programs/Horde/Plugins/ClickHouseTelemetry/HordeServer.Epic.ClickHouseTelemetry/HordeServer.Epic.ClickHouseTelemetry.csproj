<Project Sdk="Microsoft.NET.Sdk">

  <PropertyGroup>
    <ImplicitUsings>enable</ImplicitUsings>
    <RootNamespace>HordeServer</RootNamespace>
  </PropertyGroup>

  <ItemGroup>
    <PackageReference Include="ClickHouse.Client" Version="7.8.0" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\..\HordeServer.Shared\HordeServer.Shared.csproj" />
    <ProjectReference Include="..\..\..\Plugins\Analytics\HordeServer.Analytics\HordeServer.Analytics.csproj" />
    <ProjectReference Include="..\..\StructuredAnalytics\HordeServer.StructuredAnalytics\HordeServer.StructuredAnalytics.csproj" />
  </ItemGroup>

</Project>
