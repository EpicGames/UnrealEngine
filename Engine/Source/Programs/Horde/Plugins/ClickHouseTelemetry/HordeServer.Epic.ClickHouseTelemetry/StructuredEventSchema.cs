// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Cached schema metadata for a structured telemetry type.
	/// Built via reflection at startup from types with both
	/// [TelemetryEventAttribute] and [AnalyticsTableGen].
	/// </summary>
	public sealed class StructuredEventSchema
	{
		/// <summary>
		/// Event name from [TelemetryEventAttribute]
		/// </summary>
		public required string EventName { get; init; }

		/// <summary>
		/// Table name from [Table] attribute
		/// </summary>
		public required string TableName { get; init; }

		/// <summary>
		/// Schema/database name from [Table] attribute Schema property
		/// </summary>
		public string? SchemaName { get; init; }

		/// <summary>
		/// The telemetry record type
		/// </summary>
		public required Type RecordType { get; init; }

		/// <summary>
		/// Cached property info with column names
		/// </summary>
		public required IReadOnlyList<ColumnMapping> Columns { get; init; }

		/// <summary>
		/// Lookup from column name to column mapping (case-insensitive).
		/// Used for JSON property to column mapping.
		/// </summary>
		public required IReadOnlyDictionary<string, ColumnMapping> ColumnsByName { get; init; }
	}

	/// <summary>
	/// Maps a property to its ClickHouse column.
	/// </summary>
	public sealed class ColumnMapping
	{
		/// <summary>
		/// The property to read from
		/// </summary>
		public required PropertyInfo Property { get; init; }

		/// <summary>
		/// Column name from [Column] attribute
		/// </summary>
		public required string ColumnName { get; init; }

		/// <summary>
		/// ClickHouse type for this column
		/// </summary>
		public required string ClickHouseType { get; init; }

		/// <summary>
		/// Whether this column is nullable
		/// </summary>
		public required bool IsNullable { get; init; }
	}
}
