// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel.DataAnnotations.Schema;
using System.Reflection;
using EpicGames.Analytics.Telemetry;
using Microsoft.Extensions.Logging;

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Registry of structured telemetry schemas, built at startup via reflection.
	/// Only types with BOTH [TelemetryEventAttribute] AND [AnalyticsTableGen] are registered.
	/// </summary>
	public sealed class StructuredTelemetryRegistry
	{
		readonly Dictionary<string, StructuredEventSchema> _byEventName;
		readonly Dictionary<Type, StructuredEventSchema> _byType;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor. Scans assemblies immediately.
		/// </summary>
		public StructuredTelemetryRegistry(ILogger<StructuredTelemetryRegistry> logger)
		{
			_logger = logger;
			_byEventName = new Dictionary<string, StructuredEventSchema>(StringComparer.OrdinalIgnoreCase);
			_byType = new Dictionary<Type, StructuredEventSchema>();

			ScanAndRegisterTypes();
		}

		/// <summary>
		/// All registered schemas
		/// </summary>
		public IEnumerable<StructuredEventSchema> AllSchemas => _byType.Values;

		/// <summary>
		/// Try to get schema by event name
		/// </summary>
		public bool TryGetByEventName(string eventName, out StructuredEventSchema? schema)
			=> _byEventName.TryGetValue(eventName, out schema);

		/// <summary>
		/// Try to get schema by payload type
		/// </summary>
		public bool TryGetByType(Type type, out StructuredEventSchema? schema)
			=> _byType.TryGetValue(type, out schema);

		/// <summary>
		/// Check if an event name is registered as structured
		/// </summary>
		public bool IsStructuredEvent(string eventName)
			=> _byEventName.ContainsKey(eventName);

		/// <summary>
		/// Scans loaded assemblies for types with both required attributes.
		/// </summary>
		void ScanAndRegisterTypes()
		{
			foreach (Assembly assembly in AppDomain.CurrentDomain.GetAssemblies())
			{
				try
				{
					foreach (Type type in assembly.GetTypes())
					{
						if (type.IsAbstract || type.IsInterface)
						{
							continue;
						}

						// Must have BOTH attributes
						TelemetryEventAttribute? telemetryAttr = type.GetCustomAttribute<TelemetryEventAttribute>();
						AnalyticsTableGenAttribute? tableGenAttr = type.GetCustomAttribute<AnalyticsTableGenAttribute>();

						if (telemetryAttr == null || tableGenAttr == null)
						{
							continue;
						}

						// Get table name from [Table] attribute
						TableAttribute? tableAttr = type.GetCustomAttribute<TableAttribute>();
						if (tableAttr == null || String.IsNullOrEmpty(tableAttr.Name))
						{
							_logger.LogWarning("Type {Type} has [AnalyticsTableGen] but no [Table] attribute, skipping", type.FullName);
							continue;
						}

						// Build column mappings from [Column] attributes
						List<ColumnMapping> columns = BuildColumnMappings(type);

						if (columns.Count == 0)
						{
							_logger.LogWarning("Type {Type} has no properties with [Column] attribute, skipping", type.FullName);
							continue;
						}

						// Build lookup by column name (case-insensitive) - done once at startup
						Dictionary<string, ColumnMapping> columnsByName = new Dictionary<string, ColumnMapping>(StringComparer.OrdinalIgnoreCase);
						foreach (ColumnMapping col in columns)
						{
							columnsByName[col.ColumnName] = col;
						}

						StructuredEventSchema schema = new StructuredEventSchema
						{
							EventName = telemetryAttr.EventName,
							TableName = tableAttr.Name,
							SchemaName = tableAttr.Schema,
							RecordType = type,
							Columns = columns,
							ColumnsByName = columnsByName
						};

						_byEventName[schema.EventName] = schema;
						_byType[type] = schema;

						_logger.LogDebug("Registered structured telemetry: {EventName} -> {TableName} ({ColumnCount} columns)",
							schema.EventName, schema.TableName, columns.Count);
					}
				}
				catch (Exception ex)
				{
					_logger.LogDebug(ex, "Error scanning assembly {Assembly} for telemetry types", assembly.FullName);
				}
			}

			_logger.LogInformation("StructuredTelemetryRegistry: Registered {Count} structured telemetry types", _byType.Count);
		}

		/// <summary>
		/// Builds column mappings by walking the type hierarchy and finding [Column] attributes.
		/// </summary>
		static List<ColumnMapping> BuildColumnMappings(Type type)
		{
			List<ColumnMapping> columns = new List<ColumnMapping>();
			HashSet<string> seenColumns = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

			// Walk type hierarchy to get inherited properties too
			Type? current = type;
			while (current != null && current != typeof(object))
			{
				foreach (PropertyInfo prop in current.GetProperties(BindingFlags.Public | BindingFlags.Instance | BindingFlags.DeclaredOnly))
				{
					ColumnAttribute? columnAttr = prop.GetCustomAttribute<ColumnAttribute>();
					if (columnAttr == null || String.IsNullOrEmpty(columnAttr.Name))
					{
						continue;
					}

					// Avoid duplicates from shadowed properties
					if (seenColumns.Contains(columnAttr.Name))
					{
						continue;
					}

					seenColumns.Add(columnAttr.Name);

					columns.Add(new ColumnMapping
					{
						Property = prop,
						ColumnName = columnAttr.Name,
						ClickHouseType = MapTypeToClickHouse(prop.PropertyType),
						IsNullable = IsNullableType(prop.PropertyType)
					});
				}

				current = current.BaseType;
			}

			return columns;
		}

		/// <summary>
		/// Maps a CLR type to a ClickHouse type string.
		/// </summary>
		static string MapTypeToClickHouse(Type type)
		{
			Type underlyingType = Nullable.GetUnderlyingType(type) ?? type;
			bool isNullable = IsNullableType(type);

			string baseType = underlyingType switch
			{
				Type t when t == typeof(string) => "String",
				Type t when t == typeof(int) => "Int32",
				Type t when t == typeof(long) => "Int64",
				Type t when t == typeof(short) => "Int16",
				Type t when t == typeof(byte) => "UInt8",
				Type t when t == typeof(uint) => "UInt32",
				Type t when t == typeof(ulong) => "UInt64",
				Type t when t == typeof(ushort) => "UInt16",
				Type t when t == typeof(float) => "Float32",
				Type t when t == typeof(double) => "Float64",
				Type t when t == typeof(decimal) => "Decimal128(4)",
				Type t when t == typeof(bool) => "Bool",
				Type t when t == typeof(DateTime) => "String", // Store as ISO 8601
				Type t when t == typeof(DateTimeOffset) => "String", // Store as ISO 8601
				Type t when t == typeof(Guid) => "UUID",
				Type t when t == typeof(TimeSpan) => "Int64", // Store as ticks
				Type t when t.IsEnum => "String",
				Type t when t.IsArray => "String", // Store arrays as JSON
				_ => "String" // Default to JSON string for complex types
			};

			return isNullable && baseType != "String" ? $"Nullable({baseType})" : baseType;
		}

		/// <summary>
		/// Checks if a type is nullable (reference type or Nullable&lt;T&gt;).
		/// </summary>
		static bool IsNullableType(Type type)
		{
			if (!type.IsValueType)
			{
				return true;
			}
			return Nullable.GetUnderlyingType(type) != null;
		}
	}
}
