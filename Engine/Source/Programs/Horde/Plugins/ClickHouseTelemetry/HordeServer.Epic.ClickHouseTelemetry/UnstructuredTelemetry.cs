// Copyright Epic Games, Inc. All Rights Reserved.

namespace HordeServer.ClickHouseTelemetry
{
	/// <summary>
	/// Column names for the raw telemetry table
	/// </summary>
	public static class UnstructuredColumns
	{
		/// <summary>Event name from TelemetryEventAttribute</summary>
		public const string EventName = "event_name";

		/// <summary>Time the event was inserted</summary>
		public const string InsertTime = "insert_time";

		/// <summary>Telemetry store ID</summary>
		public const string TelemetryStoreId = "telemetry_store_id";

		/// <summary>Application ID</summary>
		public const string AppId = "app_id";

		/// <summary>Application version</summary>
		public const string AppVersion = "app_version";

		/// <summary>Application environment</summary>
		public const string AppEnvironment = "app_environment";

		/// <summary>Session ID</summary>
		public const string SessionId = "session_id";

		/// <summary>JSON serialized payload</summary>
		public const string Payload = "payload";
	}

	/// <summary>
	/// Constants and helpers for the unstructured telemetry staging table.
	/// Events with [TelemetryEventAttribute] but without [AnalyticsTableGen] are routed here.
	/// </summary>
	public static class UnstructuredTelemetry
	{
		/// <summary>
		/// Table name for raw/unstructured telemetry events
		/// </summary>
		public const string TableName = "raw_telemetry_events";

		/// <summary>
		/// Gets the CREATE TABLE SQL for the raw telemetry staging table.
		/// </summary>
		/// <param name="databaseName">The database name to create the table in</param>
		/// <returns>SQL CREATE TABLE statement</returns>
		public static string GetCreateTableSql(string databaseName)
		{
			return $@"
CREATE TABLE IF NOT EXISTS {databaseName}.{TableName} (
    {UnstructuredColumns.EventName} String,
    {UnstructuredColumns.InsertTime} DateTime64(3),
    {UnstructuredColumns.TelemetryStoreId} String,
    {UnstructuredColumns.AppId} Nullable(String),
    {UnstructuredColumns.AppVersion} Nullable(String),
    {UnstructuredColumns.AppEnvironment} Nullable(String),
    {UnstructuredColumns.SessionId} Nullable(String),
    {UnstructuredColumns.Payload} String
) ENGINE = MergeTree()
ORDER BY ({UnstructuredColumns.EventName}, {UnstructuredColumns.InsertTime})
PARTITION BY toYYYYMM({UnstructuredColumns.InsertTime})";
		}
	}
}
