// Copyright Epic Games, Inc. All Rights Reserved.

extern alias HordeAgent;
using EpicGames.Horde;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using Google.Protobuf.WellKnownTypes;
using HordeAgent::HordeCommon.Rpc.Tasks;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Agents.Leases;
using System.Security.Cryptography;
using EpicGames.Horde.Tools;
using Google.Protobuf;

namespace HordeServer.Tests.Agents;

[TestClass]
public class AgentCollectionTests : ComputeTestSetup
{
	private IAgent _agent;
	private readonly CreateLeaseOptions _lease1;
	private readonly CreateLeaseOptions _lease2;
	private readonly CreateLeaseOptions _leaseWithParent3;
	private readonly CreateLeaseOptions _leaseWithParent4;

	public AgentCollectionTests()
	{
		_agent = null!;
		_lease1 = new CreateLeaseOptions(LeaseId.Parse("aaaaaaaaaaaaaaaaaaaaaaaa"), null, "lease", null, null, null, null, false, new Empty());
		_lease2 = new CreateLeaseOptions(LeaseId.Parse("bbbbbbbbbbbbbbbbbbbbbbbb"), null, "lease", null, null, null, null, false, new Empty());
		_leaseWithParent3 = new CreateLeaseOptions(LeaseId.Parse("cccccccccccccccccccccccc"), _lease1.Id, "leaseWithParent3", null, null, null, null, false, new Empty());
		_leaseWithParent4 = new CreateLeaseOptions(LeaseId.Parse("dddddddddddddddddddddddd"), _lease1.Id, "leaseWithParent4", null, null, null, null, false, new Empty());
	}

	[TestInitialize]
	public async Task SetupAsync()
	{
		_agent = await AgentCollection.AddAsync(new CreateAgentOptions(new AgentId("test"), AgentMode.Dedicated, true, ""));
	}

	[TestMethod]
	public async Task AddLeaseAsync()
	{
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);
		Assert.AreEqual(1, agent.Leases.Count);
		await UpdateAgentAsync();

		agent = await agent.TryCreateLeaseAsync(_leaseWithParent3);
		Assert.IsNotNull(agent);
		Assert.AreEqual(2, agent.Leases.Count);
		await UpdateAgentAsync();

		agent = await agent.TryCreateLeaseAsync(_leaseWithParent4);
		Assert.IsNotNull(agent);
		Assert.AreEqual(3, agent.Leases.Count);
		await UpdateAgentAsync();

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(3, leases.Length);
		Assert.IsTrue(leases.Contains(_lease1.Id));
		Assert.IsTrue(leases.Contains(_leaseWithParent3.Id));
		Assert.IsTrue(leases.Contains(_leaseWithParent4.Id));
	}

	[TestMethod]
	public async Task StartSessionAsync()
	{
		await _agent.TryCreateLeaseAsync(_lease1);
		await UpdateAgentAsync();

		await _agent.TryCreateLeaseAsync(_lease2);
		await UpdateAgentAsync();

		await _agent.TryCreateSessionAsync(new CreateSessionOptions(
			new RpcAgentCapabilities(), new List<PoolId>(), null));

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(0, leases.Length);
	}

	[TestMethod]
	public async Task UpdateSession_WithEmptyLeasesAsync()
	{
		await _agent.TryCreateLeaseAsync(_lease1);
		await UpdateAgentAsync();

		await _agent.TryCreateLeaseAsync(_lease2);
		await UpdateAgentAsync();

		await _agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>
			{
				new RpcLease{ Id = _lease1.Id, State = RpcLeaseState.Active },
				new RpcLease{ Id = _lease2.Id, State = RpcLeaseState.Active }
			}
		});
		await UpdateAgentAsync();

		await _agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>()
		});

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(0, leases.Length);
	}

	[TestMethod]
	public async Task UpdateSession_WithNewLeasesAsync()
	{
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);

		agent = await agent.TryCreateLeaseAsync(_lease2);
		Assert.IsNotNull(agent);

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(2, leases.Length);
		Assert.IsTrue(leases.Contains(_lease1.Id));
		Assert.IsTrue(leases.Contains(_lease2.Id));
	}

	[TestMethod]
	public async Task UpdateSession_WithOneLeaseRemovedAsync()
	{
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);

		agent = await agent.TryCreateLeaseAsync(_lease2);
		Assert.IsNotNull(agent);

		agent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>
			{
				new RpcLease{ Id = _lease1.Id, State = RpcLeaseState.Active },
				new RpcLease{ Id = _lease2.Id, State = RpcLeaseState.Active }
			}
		});
		Assert.IsNotNull(agent);

		agent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>
			{
				new RpcLease { Id = _lease1.Id }
			}
		});
		Assert.IsNotNull(agent);

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(1, leases.Length);
		Assert.IsTrue(leases.Contains(_lease1.Id));
	}
	
	[TestMethod]
	public async Task UpgradeAttemptCount_ShouldTrackUpgradeHistory_Async()
	{
		const string CurrentVersion = "v1";
		const string NewVersion = "v2";
		
		// First failing upgrade attempt to v2
		IAgent? agent = await CreateSessionAsync(_agent, CurrentVersion);
		agent = await CreateLeaseAsync(agent!, new UpgradeTask { SoftwareId = NewVersion });
		Assert.AreEqual(CurrentVersion, agent!.Version);
		Assert.AreEqual(NewVersion, agent.LastUpgradeVersion);
		Assert.AreEqual(1, agent.UpgradeAttemptCount);
		agent = await agent.TryTerminateSessionAsync();
		Assert.IsNotNull(agent);
		
		// Second failing upgrade attempt to v2
		agent = await CreateSessionAsync(agent, CurrentVersion);
		agent = await CreateLeaseAsync(agent!, new UpgradeTask { SoftwareId = NewVersion });
		Assert.AreEqual(CurrentVersion, agent!.Version);
		Assert.AreEqual(NewVersion, agent.LastUpgradeVersion);
		Assert.AreEqual(2, agent.UpgradeAttemptCount);
		agent = await agent.TryTerminateSessionAsync();
		Assert.IsNotNull(agent);

		// Successful upgrade
		agent = await CreateSessionAsync(agent, NewVersion);
		Assert.AreEqual(NewVersion, agent!.Version);
		Assert.AreEqual(NewVersion, agent.LastUpgradeVersion);
		Assert.IsNull(agent.UpgradeAttemptCount);
	}
	
	private static Task<IAgent?> CreateSessionAsync(IAgent agent, string? version = null)
	{
		return agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), version));
	}
	
	private static Task<IAgent?> CreateLeaseAsync(IAgent agent, IMessage? payload = null)
	{
		LeaseId leaseId = new (new BinaryId(RandomNumberGenerator.GetBytes(12)));
		CreateLeaseOptions lease = new(leaseId, null, "lease", null, null, null, null, false, payload ?? new Empty());
		return agent.TryCreateLeaseAsync(lease);
	}
	
	[TestMethod]
	public async Task CancelLeaseAsync()
	{
		IAgent? agent = _agent;

		agent = await agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);
		Assert.AreEqual(1, agent.Leases.Count);

		agent = await agent.TryCreateLeaseAsync(_lease2);
		Assert.IsNotNull(agent);
		Assert.AreEqual(2, agent.Leases.Count);

		agent = await agent.TryCancelLeaseAsync(0);
		Assert.IsNotNull(agent);
		Assert.AreEqual(1, agent.Leases.Count);

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();
		Assert.AreEqual(1, leases.Length);
		Assert.IsTrue(leases.Contains(_lease2.Id));
	}

	[TestMethod]
	public async Task CancelLease2Async()
	{
		IAgent? agent = _agent;

		agent = await agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);
		Assert.AreEqual(1, agent.Leases.Count);

		agent = await agent.TryCreateLeaseAsync(_lease2);
		Assert.IsNotNull(agent);
		Assert.AreEqual(2, agent.Leases.Count);
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease1.Id && x.State == LeaseState.Pending));
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease2.Id && x.State == LeaseState.Pending));

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();
		Assert.AreEqual(2, leases.Length);
		Assert.IsTrue(leases.Contains(_lease1.Id));
		Assert.IsTrue(leases.Contains(_lease2.Id));

		agent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>
			{
				new RpcLease { Id = _lease1.Id, State = RpcLeaseState.Active },
				new RpcLease { Id = _lease2.Id, State = RpcLeaseState.Active }
			}
		});
		Assert.IsNotNull(agent);
		Assert.AreEqual(2, agent.Leases.Count);
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease1.Id && x.State == LeaseState.Active));
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease2.Id && x.State == LeaseState.Active));

		agent = await agent.TryCancelLeaseAsync(0);
		Assert.IsNotNull(agent);
		Assert.AreEqual(2, agent.Leases.Count);
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease1.Id && x.State == LeaseState.Cancelled));
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease2.Id && x.State == LeaseState.Active));

		agent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions
		{
			Leases = new List<RpcLease>
			{
				new RpcLease { Id = _lease1.Id, State = RpcLeaseState.Completed },
				new RpcLease { Id = _lease2.Id, State = RpcLeaseState.Active }
			}
		});
		Assert.IsNotNull(agent);
		Assert.AreEqual(1, agent.Leases.Count);
		Assert.IsTrue(agent.Leases.Any(x => x.Id == _lease2.Id && x.State == LeaseState.Active));

		leases = await AgentScheduler.FindActiveLeaseIdsAsync();
		Assert.AreEqual(1, leases.Length);
		Assert.IsTrue(leases.Contains(_lease2.Id));
	}

	[TestMethod]
	public async Task TerminateSessionAsync()
	{
		await _agent.TryCreateLeaseAsync(_lease1);
		await UpdateAgentAsync();

		await _agent.TryCreateLeaseAsync(_lease2);
		await UpdateAgentAsync();

		await _agent.TryTerminateSessionAsync();

		LeaseId[] leases = await AgentScheduler.FindActiveLeaseIdsAsync();

		Assert.AreEqual(0, leases.Length);
	}

	[TestMethod]
	public async Task TerminateSession_SetsLeaseOutcomeReasonAsync()
	{
		// Create session first
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		// Create leases
		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);

		agent = await agent.TryCreateLeaseAsync(_lease2);
		Assert.IsNotNull(agent);

		// Terminate session without agent-provided lease data (server-initiated termination)
		await agent.TryTerminateSessionAsync();

		// Verify leases have correct outcome reason
		ILease? lease1 = await LeaseCollection.GetAsync(_lease1.Id);
		ILease? lease2 = await LeaseCollection.GetAsync(_lease2.Id);

		Assert.IsNotNull(lease1);
		Assert.IsNotNull(lease2);

		// Both leases should be cancelled with ServerInitiated | AgentSessionTerminated reason
		Assert.AreEqual(LeaseOutcome.Cancelled, lease1.Outcome);
		Assert.AreEqual(LeaseOutcome.Cancelled, lease2.Outcome);
		Assert.IsTrue(lease1.OutcomeReason.HasFlag(LeaseOutcomeReason.ServerInitiated));
		Assert.IsTrue(lease1.OutcomeReason.HasFlag(LeaseOutcomeReason.AgentSessionTerminated));
		Assert.IsTrue(lease2.OutcomeReason.HasFlag(LeaseOutcomeReason.ServerInitiated));
		Assert.IsTrue(lease2.OutcomeReason.HasFlag(LeaseOutcomeReason.AgentSessionTerminated));
	}
	
	[TestMethod]
	public async Task UpdateSession_MergesOutcomeReasons_WhenLeaseAlreadyFinishedAsync()
	{
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), [], "foo"));
		agent = await agent!.TryCreateLeaseAsync(_lease1);
		agent = await agent!.TryUpdateSessionAsync(new UpdateSessionOptions { Leases = [new() { Id = _lease1.Id, State = RpcLeaseState.Active }] });

		// Simulate race: Another process already finished the lease
		bool result = await LeaseCollection.TrySetOutcomeAsync(_lease1.Id, DateTime.UtcNow, LeaseOutcome.Cancelled,
			LeaseOutcomeReason.ServerInitiated | LeaseOutcomeReason.AgentSessionTerminated, null, null);
		Assert.IsTrue(result);

		// Agent reports lease with additional reason - should merge via TryAddOutcomeReasonsAsync
		RpcLease cancelledLease = new() { Id = _lease1.Id, State = RpcLeaseState.Cancelled, Outcome = RpcLeaseOutcome.Cancelled };
		cancelledLease.OutcomeReasons.Add(RpcLeaseOutcomeReason.NonZeroExitCode);
		agent = await agent!.TryUpdateSessionAsync(new UpdateSessionOptions { Leases = [cancelledLease] });
		Assert.IsNotNull(agent);

		// Verify all reasons merged
		ILease? lease = await LeaseCollection.GetAsync(_lease1.Id);
		Assert.IsTrue(lease!.OutcomeReason.HasFlag(LeaseOutcomeReason.ServerInitiated));
		Assert.IsTrue(lease.OutcomeReason.HasFlag(LeaseOutcomeReason.AgentSessionTerminated));
		Assert.IsTrue(lease.OutcomeReason.HasFlag(LeaseOutcomeReason.NonZeroExitCode));
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesReplacesExistingPropertyAsync()
	{
		// Boot with initial WorkingDiskFreeSpace in capabilities
		List<string> initialProps = [$"{KnownPropertyNames.WorkingDiskFreeSpace}=500"];
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(initialProps), [], "foo")));
		Assert.AreEqual(500, agent.GetDiskFreeSpace());

		// Update replaces (not duplicates) the property via CapabilityOverrides
		RpcAgentCapabilities overrides = new([$"{KnownPropertyNames.WorkingDiskFreeSpace}=250"]);
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { CapabilityOverrides = overrides }));
		Assert.AreEqual(250, agent.GetDiskFreeSpace());
		Assert.HasCount(1, agent.GetPropertyValues(KnownPropertyNames.WorkingDiskFreeSpace));

		// Re-read from DB to verify replacement persisted
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(250, agentFromDb.GetDiskFreeSpace());
		Assert.HasCount(1, agentFromDb.GetPropertyValues(KnownPropertyNames.WorkingDiskFreeSpace));

		// Update without CapabilityOverrides preserves existing value
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { Status = AgentStatus.Ok }));
		Assert.AreEqual(250, agent.GetDiskFreeSpace());

		agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(250, agentFromDb.GetDiskFreeSpace());
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesReplacesExistingResourceAsync()
	{
		// Boot with initial resources
		RpcAgentCapabilities initialCaps = new();
		initialCaps.Resources.Add("LogicalCores", 16);
		initialCaps.Resources.Add("Ram", 32768);
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(initialCaps, [], "foo")));
		Assert.AreEqual(16, agent.Resources["LogicalCores"]);
		Assert.AreEqual(32768, agent.Resources["Ram"]);

		// Override one resource, leave the other unchanged
		RpcAgentCapabilities overrides = new();
		overrides.Resources.Add("Ram", 65536);
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { CapabilityOverrides = overrides }));
		Assert.AreEqual(16, agent.Resources["LogicalCores"]);
		Assert.AreEqual(65536, agent.Resources["Ram"]);

		// Re-read from DB to verify
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(16, agentFromDb.Resources["LogicalCores"]);
		Assert.AreEqual(65536, agentFromDb.Resources["Ram"]);
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesReplacesResourceAsync()
	{
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), [], "foo")));

		RpcAgentCapabilities caps = new();
		caps.Resources.Add("LogicalCores", 16);
		caps.Resources.Add("Ram", 32768);

		RpcAgentCapabilities overrides = new();
		overrides.Resources.Add("Ram", 65536);

		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { Capabilities = caps, CapabilityOverrides = overrides }));
		Assert.AreEqual(16, agent.Resources["LogicalCores"]);
		Assert.AreEqual(65536, agent.Resources["Ram"]);

		// Re-read from DB to verify
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(16, agentFromDb.Resources["LogicalCores"]);
		Assert.AreEqual(65536, agentFromDb.Resources["Ram"]);
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesReplacesMultiPropertiesAsync()
	{
		// Boot with multiple properties
		List<string> initialProps = ["OSFamily=Windows", "OsVersion=10.0", $"{KnownPropertyNames.WorkingDiskFreeSpace}=500"];
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(initialProps), [], "foo")));
		Assert.AreEqual(500, agent.GetDiskFreeSpace());
		Assert.HasCount(1, agent.GetPropertyValues("OSFamily"));

		// Override WorkingDiskFreeSpace and OSFamily, leave OsVersion unchanged
		RpcAgentCapabilities overrides = new([$"{KnownPropertyNames.WorkingDiskFreeSpace}=250", "OSFamily=Linux"]);
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { CapabilityOverrides = overrides }));
		Assert.AreEqual(250, agent.GetDiskFreeSpace());
		Assert.AreEqual("Linux", agent.GetPropertyValues("OSFamily").First());
		Assert.AreEqual("10.0", agent.GetPropertyValues("OsVersion").First());

		// Re-read from DB to verify
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(250, agentFromDb.GetDiskFreeSpace());
		Assert.AreEqual("Linux", agentFromDb.GetPropertyValues("OSFamily").First());
		Assert.AreEqual("10.0", agentFromDb.GetPropertyValues("OsVersion").First());
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesIgnoresNewPropertyAsync()
	{
		// Boot with a single property
		List<string> initialProps = ["OSFamily=Windows"];
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(initialProps), [], "foo")));
		Assert.AreEqual("Windows", agent.GetPropertyValues("OSFamily").First());

		// Override with one existing key and one new key; new key should be ignored
		RpcAgentCapabilities overrides = new(["OSFamily=Linux", "NewProp=ShouldBeIgnored"]);
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { CapabilityOverrides = overrides }));
		Assert.AreEqual("Linux", agent.GetPropertyValues("OSFamily").First());
		Assert.HasCount(0, agent.GetPropertyValues("NewProp"));

		// Re-read from DB to verify
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual("Linux", agentFromDb.GetPropertyValues("OSFamily").First());
		Assert.HasCount(0, agentFromDb.GetPropertyValues("NewProp"));
	}

	[TestMethod]
	public async Task UpdateSession_CapabilityOverridesIgnoresNewResourceAsync()
	{
		// Boot with initial resources
		RpcAgentCapabilities initialCaps = new();
		initialCaps.Resources.Add("LogicalCores", 16);
		IAgent? agent = Deref(await _agent.TryCreateSessionAsync(new CreateSessionOptions(initialCaps, [], "foo")));
		Assert.AreEqual(16, agent.Resources["LogicalCores"]);

		// Override with one existing key and one new key; new key should be ignored
		RpcAgentCapabilities overrides = new();
		overrides.Resources.Add("LogicalCores", 32);
		overrides.Resources.Add("Gpu", 2);
		agent = Deref(await agent.TryUpdateSessionAsync(new UpdateSessionOptions { CapabilityOverrides = overrides }));
		Assert.AreEqual(32, agent.Resources["LogicalCores"]);
		Assert.IsFalse(agent.Resources.ContainsKey("Gpu"));

		// Re-read from DB to verify
		IAgent? agentFromDb = Deref(await AgentCollection.GetAsync(agent.Id));
		Assert.AreEqual(32, agentFromDb.Resources["LogicalCores"]);
		Assert.IsFalse(agentFromDb.Resources.ContainsKey("Gpu"));
	}

	[TestMethod]
	public async Task GetChildLeaseIdsAsync()
	{
		IAgent? agent = _agent;

		agent = await agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(), Array.Empty<PoolId>(), "foo"));
		Assert.IsNotNull(agent);
		Assert.IsNotNull(agent.SessionId);

		agent = await agent.TryCreateLeaseAsync(_lease1);
		Assert.IsNotNull(agent);

		agent = await agent.TryCreateLeaseAsync(_leaseWithParent3);
		Assert.IsNotNull(agent);

		agent = await agent.TryCreateLeaseAsync(_leaseWithParent4);
		Assert.IsNotNull(agent);

		LeaseId[] leases = await AgentScheduler.GetChildLeaseIdsAsync(_leaseWithParent3.ParentId!.Value);

		Assert.AreEqual(2, leases.Length);
		Assert.IsTrue(leases.Contains(_leaseWithParent3.Id));
		Assert.IsTrue(leases.Contains(_leaseWithParent4.Id));
	}
	
	public static IEnumerable<object?[]> GetSoftwareToolIdTestData()
	{
		yield return [ null, null, AgentExtensions.AgentToolId.ToString() ];
		yield return [ "my-tool", null, "my-tool" ];
		yield return [ "my-tool", new AgentSoftwareConfig { ToolId = new ToolId("other-tool"), Condition = "" }, "other-tool" ];
		yield return [ null, new AgentSoftwareConfig { ToolId = new ToolId("other-tool"), Condition = "" }, "other-tool" ];
	}
	
	[TestMethod]
	[DynamicData(nameof(GetSoftwareToolIdTestData))]
	public async Task GetSoftwareToolIdAsync(string? reportedToolId, AgentSoftwareConfig? softwareConfig, string? expectedToolId)
	{
		List<string> props = reportedToolId != null ? [$"{KnownPropertyNames.ToolId}={reportedToolId}"] : [];
		IAgent? agent = await _agent.TryCreateSessionAsync(new CreateSessionOptions(new RpcAgentCapabilities(props), Array.Empty<PoolId>(), "foo"));
		ToolId actual = agent!.GetSoftwareToolId(new ComputeConfig() { Software = softwareConfig != null ? [softwareConfig] : [] });
		Assert.AreEqual(expectedToolId, actual.ToString());
	}

	private async Task UpdateAgentAsync()
	{
		IAgent? agent = await AgentCollection.GetAsync(_agent.Id);
		Assert.IsNotNull(agent);
		_agent = agent;
	}
}