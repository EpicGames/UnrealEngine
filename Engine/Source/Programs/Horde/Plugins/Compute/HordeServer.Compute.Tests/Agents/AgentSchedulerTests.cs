// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeCommon;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Agents.Sessions;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.DependencyInjection;
using StackExchange.Redis;

namespace HordeServer.Tests.Agents
{
	[TestClass]
	public class AgentSchedulerTests : ComputeTestSetup
	{
		[TestMethod]
		public async Task CreateSessionAsync()
		{
			IClock clock = ServiceProvider.GetRequiredService<IClock>();

			RpcAgentCapabilities caps = new RpcAgentCapabilities();
			caps.Properties.Add("foo=bar");

			RpcSession? session = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), caps);
			Assert.IsNotNull(session);

			Assert.AreEqual(session.AgentId, new AgentId("foo"));
			Assert.AreEqual(0, session.Leases.Count);
			Assert.IsTrue(session.ExpiryTime > clock.UtcNow);

			RpcSession? session2 = await AgentScheduler.TryGetSessionAsync(session.SessionId);
			Assert.AreEqual(session, session2);

			RpcAgentCapabilities? caps2 = await AgentScheduler.TryGetCapabilitiesAsync(session);
			Assert.AreEqual(caps2, caps);
		}

		[TestMethod]
		public async Task UpdateCapabilitesAsync()
		{
			// Set and get the initial capabilities
			RpcAgentCapabilities caps = new RpcAgentCapabilities();
			caps.Properties.Add("foo=bar");

			RpcSession? session = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), caps);
			Assert.IsNotNull(session);

			RpcAgentCapabilities? caps2 = await AgentScheduler.TryGetCapabilitiesAsync(session);
			Assert.AreEqual(caps, caps2);

			// Update them
			RpcAgentCapabilities caps3 = new RpcAgentCapabilities(caps);
			caps3.Properties.Add("bar=baz");

			RpcSession? session2 = await AgentScheduler.TryUpdateSessionAsync(session, newCapabilities: caps3);
			Assert.IsNotNull(session2);

			RpcAgentCapabilities? caps5 = await AgentScheduler.TryGetCapabilitiesAsync(session2);
			Assert.AreEqual(caps3, caps5);
		}

		[TestMethod]
		public async Task TerminateSessionAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();

			RpcSession? session = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), new RpcAgentCapabilities());
			Assert.IsNotNull(session);

			await clock.AdvanceAsync(TimeSpan.FromSeconds(30.0));

			RpcSession? session2 = await AgentScheduler.TryUpdateSessionAsync(session, new RpcSession(session) { Status = RpcAgentStatus.Stopped });
			Assert.IsNotNull(session2);
			Assert.AreEqual(clock.UtcNow, session2.ExpiryTime);

			RpcSession? session3 = await AgentScheduler.TryGetSessionAsync(session.SessionId);
			Assert.IsNull(session3);
		}

		[TestMethod]
		public async Task AddSessionsToNewQueueTestAsync()
		{
			AgentScheduler scheduler = (AgentScheduler)AgentScheduler;
			await scheduler.StartAsync(default);

			// First session
			for (int idx = 0; idx < 3; idx++)
			{
				RpcAgentCapabilities caps1 = new RpcAgentCapabilities();
				caps1.Properties.Add("Platform=Windows");

				RpcSession? session1 = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), caps1);
				Assert.IsNotNull(session1);
			}

			// Second session
			for (int idx = 0; idx < 5; idx++)
			{
				RpcAgentCapabilities caps2 = new RpcAgentCapabilities();
				caps2.Properties.Add("Platform=MacOS");

				RpcSession? session2 = await AgentScheduler.TryCreateSessionAsync(new AgentId("bar"), SessionIdUtils.GenerateNewId(), caps2);
				Assert.IsNotNull(session2);
			}

			// Create the first queue
			RpcAgentRequirements reqs1 = new RpcAgentRequirements();
			reqs1.Properties.Add("Platform=Windows");

			IoHash queueHash1 = await scheduler.CreateFilterAsync(reqs1);

			// Create the second queue
			RpcAgentRequirements reqs2 = new RpcAgentRequirements();
			reqs2.Properties.Add("Platform=MacOS");

			IoHash queueHash2 = await scheduler.CreateFilterAsync(reqs2);

			// Update the cached queues
			await scheduler.ForceUpdateFiltersAsync(CancellationToken.None);

			SessionId[] sessionIds1 = await scheduler.EnumerateFilteredSessionsAsync(queueHash1, SessionFilterType.Potential).ToArrayAsync();
			Assert.AreEqual(3, sessionIds1.Length);

			SessionId[] sessionIds2 = await scheduler.EnumerateFilteredSessionsAsync(queueHash2, SessionFilterType.Potential).ToArrayAsync();
			Assert.AreEqual(5, sessionIds2.Length);
		}

		[TestMethod]
		public async Task FindTerminatedSessionsAsync()
		{
			RpcSession? session = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), new RpcAgentCapabilities());
			Assert.IsNotNull(session);
			Assert.AreEqual(0, (await AgentScheduler.RemoveOrphanedSessionsAsync()).Count);

			SessionId orphanedId = SessionIdUtils.GenerateNewId();
			IDatabase database = GetRedisServiceSingleton().GetDatabase();
			string orphanedKey = HordeServer.Agents.AgentScheduler.SessionStateKey(orphanedId.ToString());
			await database.StringSetAsync(orphanedKey, "not-relevant");

			// Ensure only orphaned session are removed and valid ones are left intact
			List<SessionId> orphanedIds = await AgentScheduler.RemoveOrphanedSessionsAsync();
			CollectionAssert.AreEquivalent(new List<SessionId> { orphanedId }, orphanedIds);
			session = await AgentScheduler.TryGetSessionAsync(session.SessionId);
			Assert.IsNotNull(session);
		}

		[TestMethod]
		public async Task AddSessionsToExistingQueueTestAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			AgentScheduler scheduler = (AgentScheduler)AgentScheduler;
			await scheduler.StartAsync(default);

			// Create the first queue
			RpcAgentRequirements reqs1 = new RpcAgentRequirements();
			reqs1.Properties.Add("Platform=Windows");

			IoHash queueHash1 = await scheduler.CreateFilterAsync(reqs1);

			// Create the second queue
			RpcAgentRequirements reqs2 = new RpcAgentRequirements();
			reqs2.Properties.Add("Platform=MacOS");

			IoHash queueHash2 = await scheduler.CreateFilterAsync(reqs2);

			// Update the cached queues
			await clock.AdvanceAsync(TimeSpan.FromMinutes(1.0));

			// First session
			for (int idx = 0; idx < 3; idx++)
			{
				RpcAgentCapabilities caps1 = new RpcAgentCapabilities();
				caps1.Properties.Add("Platform=Windows");

				RpcSession? session1 = await AgentScheduler.TryCreateSessionAsync(new AgentId("foo"), SessionIdUtils.GenerateNewId(), caps1);
				Assert.IsNotNull(session1);
			}

			// Second session
			for (int idx = 0; idx < 5; idx++)
			{
				RpcAgentCapabilities caps2 = new RpcAgentCapabilities();
				caps2.Properties.Add("Platform=MacOS");

				RpcSession? session2 = await AgentScheduler.TryCreateSessionAsync(new AgentId("bar"), SessionIdUtils.GenerateNewId(), caps2);
				Assert.IsNotNull(session2);
			}

			//
			SessionId[] sessionIds1 = await scheduler.EnumerateFilteredSessionsAsync(queueHash1, SessionFilterType.Potential).ToArrayAsync();
			Assert.AreEqual(3, sessionIds1.Length);

			SessionId[] sessionIds2 = await scheduler.EnumerateFilteredSessionsAsync(queueHash2, SessionFilterType.Potential).ToArrayAsync();
			Assert.AreEqual(5, sessionIds2.Length);
		}

		[TestMethod]
		public async Task ExpireQueuesTestAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			AgentScheduler scheduler = (AgentScheduler)AgentScheduler;
			await scheduler.StartAsync(default);

			// Create the first queue
			RpcAgentRequirements reqs1 = new RpcAgentRequirements();
			reqs1.Properties.Add("Platform=Windows");

			IoHash reqsHash1 = await scheduler.CreateFilterAsync(reqs1);
			Assert.AreEqual(1, (await scheduler.GetFiltersAsync()).Length);

			// Check that the queue exists
			await clock.AdvanceAsync(HordeServer.Agents.AgentScheduler.ExpireFiltersTime / 2);
			await scheduler.TouchFilterAsync(reqsHash1);
			Assert.AreEqual(1, (await scheduler.GetFiltersAsync()).Length);

			await clock.AdvanceAsync(HordeServer.Agents.AgentScheduler.ExpireFiltersTime / 2);
			await scheduler.TouchFilterAsync(reqsHash1);
			Assert.AreEqual(1, (await scheduler.GetFiltersAsync()).Length);

			await clock.AdvanceAsync(HordeServer.Agents.AgentScheduler.ExpireFiltersTime / 2);
			Assert.AreEqual(1, (await scheduler.GetFiltersAsync()).Length);

			await clock.AdvanceAsync(HordeServer.Agents.AgentScheduler.ExpireFiltersTime);
			Assert.AreEqual(0, (await scheduler.GetFiltersAsync()).Length);
		}

		[TestMethod]
		public async Task FilterAvailableTestAsync()
		{
			FakeClock clock = ServiceProvider.GetRequiredService<FakeClock>();
			AgentScheduler scheduler = (AgentScheduler)AgentScheduler;
			await scheduler.StartAsync(default);

			// Create the first filter (non-shared windows agents)
			RpcAgentRequirements reqs1 = new RpcAgentRequirements();
			reqs1.Properties.Add("Platform=Windows");
			reqs1.Shared = false;

			IoHash reqsHash1 = await scheduler.CreateFilterAsync(reqs1);

			// Create the second filter (windows agents with 1 core free)
			RpcAgentRequirements reqs2 = new RpcAgentRequirements();
			reqs2.Properties.Add("Platform=Windows");
			reqs2.Resources.Add("LogicalCores", 1);
			reqs2.Shared = true;

			IoHash reqsHash2 = await scheduler.CreateFilterAsync(reqs2);

			// Create the second filter (windows agents with 2 cores free)
			RpcAgentRequirements reqs3 = new RpcAgentRequirements();
			reqs3.Properties.Add("Platform=Windows");
			reqs3.Resources.Add("LogicalCores", 2);
			reqs3.Shared = true;

			IoHash reqsHash3 = await scheduler.CreateFilterAsync(reqs3);

			// Create the third filter (windows agents with at least 16gb RAM)
			RpcAgentRequirements reqs4 = new RpcAgentRequirements();
			reqs4.Properties.Add("Platform=Windows");
			reqs4.Resources.Add("RAM", 16);
			reqs4.Shared = true;

			IoHash reqsHash4 = await scheduler.CreateFilterAsync(reqs4);

			// Create the fourth filter (mac agents)
			RpcAgentRequirements reqs5 = new RpcAgentRequirements();
			reqs5.Properties.Add("Platform=MacOS");

			IoHash reqsHash5 = await scheduler.CreateFilterAsync(reqs5);

			// Update the clock so the scheduler caches the filters
			await clock.AdvanceAsync(TimeSpan.FromMinutes(1.0));

			// Create some sessions
			RpcAgentCapabilities winCaps = new RpcAgentCapabilities();
			winCaps.Properties.Add("Platform=Windows");
			winCaps.Resources.Add("LogicalCores", 2);
			winCaps.Resources.Add("RAM", 16);

			RpcAgentCapabilities macCaps = new RpcAgentCapabilities();
			macCaps.Properties.Add("Platform=MacOS");

			RpcSession? session1 = await AgentScheduler.TryCreateSessionAsync(new AgentId("session1"), SessionIdUtils.GenerateNewId(), winCaps);
			Assert.IsNotNull(session1);

			RpcSession? session2 = await AgentScheduler.TryCreateSessionAsync(new AgentId("session2"), SessionIdUtils.GenerateNewId(), winCaps);
			Assert.IsNotNull(session2);

			RpcSession? session3 = await AgentScheduler.TryCreateSessionAsync(new AgentId("session3"), SessionIdUtils.GenerateNewId(), winCaps);
			Assert.IsNotNull(session3);

			RpcSession? session4 = await AgentScheduler.TryCreateSessionAsync(new AgentId("session4"), SessionIdUtils.GenerateNewId(), macCaps);
			Assert.IsNotNull(session4);

			async Task CheckFilterLengthsAsync(SessionFilterType type, int length1 = 3, int length2 = 3, int length3 = 3, int length4 = 3, int length5 = 1)
			{
				SessionId[] sessionIds1 = await scheduler.EnumerateFilteredSessionsAsync(reqsHash1, type).ToArrayAsync();
				SessionId[] sessionIds2 = await scheduler.EnumerateFilteredSessionsAsync(reqsHash2, type).ToArrayAsync();
				SessionId[] sessionIds3 = await scheduler.EnumerateFilteredSessionsAsync(reqsHash3, type).ToArrayAsync();
				SessionId[] sessionIds4 = await scheduler.EnumerateFilteredSessionsAsync(reqsHash4, type).ToArrayAsync();
				SessionId[] sessionIds5 = await scheduler.EnumerateFilteredSessionsAsync(reqsHash5, type).ToArrayAsync();
				Assert.AreEqual(length1, sessionIds1.Length);
				Assert.AreEqual(length2, sessionIds2.Length);
				Assert.AreEqual(length3, sessionIds3.Length);
				Assert.AreEqual(length4, sessionIds4.Length);
				Assert.AreEqual(length5, sessionIds5.Length);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available);
			await CheckFilterLengthsAsync(SessionFilterType.Potential);

			// Add a lease to session 2, and check it's no longer reported as available in the exclusive queue (1), and two-cpu queue (3)
			{
				RpcSessionLease newLease = new RpcSessionLease();
				newLease.Id = new LeaseId(BinaryIdUtils.CreateNew());
				newLease.Resources.Add("LogicalCores", 1);

				RpcSession newSession2 = new RpcSession(session2);
				newSession2.Leases.Add(newLease);

				session2 = await AgentScheduler.TryUpdateSessionAsync(session2, newSession2);
				Assert.IsNotNull(session2);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available, length1: 2, length3: 2); // filter 1 (exclusive), filter 3 (2 cpus)
			await CheckFilterLengthsAsync(SessionFilterType.Potential);

			// Add another lease to session 2, and check that it's also removed from the one-cpu queue (1)
			{
				RpcSessionLease newLease = new RpcSessionLease();
				newLease.Id = new LeaseId(BinaryIdUtils.CreateNew());
				newLease.Resources.Add("LogicalCores", 1);

				RpcSession newSession2 = new RpcSession(session2);
				newSession2.Leases.Add(newLease);

				session2 = await AgentScheduler.TryUpdateSessionAsync(session2, newSession2);
				Assert.IsNotNull(session2);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available, length1: 2, length2: 2, length3: 2); // filter 1 (exclusive), filter 2 (1 cpu), filter 3 (2 cpus)
			await CheckFilterLengthsAsync(SessionFilterType.Potential);

			// Add a lease to session 3 that also requests ram, and and check that it's also removed from filter 4
			{
				RpcSessionLease newLease = new RpcSessionLease();
				newLease.Id = new LeaseId(BinaryIdUtils.CreateNew());
				newLease.Resources.Add("RAM", 1);

				RpcSession newSession3 = new RpcSession(session3);
				newSession3.Leases.Add(newLease);

				session3 = await AgentScheduler.TryUpdateSessionAsync(session3, newSession3);
				Assert.IsNotNull(session3);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available, length1: 1, length2: 2, length3: 2, length4: 2); // filter 1 (exclusive), filter 2 (1 cpu), filter 3 (2 cpus), filter 4 (16gb)
			await CheckFilterLengthsAsync(SessionFilterType.Potential);

			// Remove the leases from session 2
			{
				RpcSession newSession2 = new RpcSession(session2);
				newSession2.Leases.Clear();

				session2 = await AgentScheduler.TryUpdateSessionAsync(session2, newSession2);
				Assert.IsNotNull(session2);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available, length1: 2, length4: 2); // filter 1 (exclusive), filter 4 (16gb)
			await CheckFilterLengthsAsync(SessionFilterType.Potential);

			// Remove the leases from session 3
			{
				RpcSession newSession3 = new RpcSession(session3);
				newSession3.Leases.Clear();

				session3 = await AgentScheduler.TryUpdateSessionAsync(session3, newSession3);
				Assert.IsNotNull(session3);
			}

			await CheckFilterLengthsAsync(SessionFilterType.Available);
			await CheckFilterLengthsAsync(SessionFilterType.Potential);
		}
	}
}
