// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Agents;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging.Abstractions;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Agents;

[TestClass]
public class AgentsControllerTests : ComputeTestSetup
{
	private class OptionsSnapshotStub<T>(T value) : IOptionsSnapshot<T> where T : class
	{
		public T Value { get; } = value;
		public T Get(string? name)
		{
			return Value;
		}
	}
	
	[TestMethod]
	public async Task FindAgents_Simple_Async()
	{
		// Arrange
		List<string> properties = ["foo=1", "bar=2"];
		AgentsController controller = CreateAgentsController();
		await CreateAgentAsync("myAgent1", properties);
		
		// Act
		ActionResult<List<object>> result = await controller.FindAgentsAsync();
		
		// Assert
		Assert.IsNotNull(result.Value);
		List<GetAgentResponse> getAgentResponses = result.Value!.Cast<GetAgentResponse>().ToList();
		Assert.AreEqual(1, getAgentResponses.Count);
		CollectionAssert.AreEquivalent(properties, getAgentResponses[0].Properties);
	}
	
	[TestMethod]
	public async Task FindAgents_Filter_Async()
	{
		// Arrange
		List<string> properties = ["foo=1", "bar=2"];
		AgentsController controller = CreateAgentsController();
		await CreateAgentAsync("myAgent1", properties);
		
		// Act
		ActionResult<List<object>> result = await controller.FindAgentsAsync(filter: PropertyFilter.Parse("id,name,enabled"));
		
		// Assert
		Assert.IsNotNull(result.Value);
		List<Dictionary<string, object>> getAgentResponses = result.Value!.Cast<Dictionary<string, object>>().ToList();
		Assert.AreEqual(1, getAgentResponses.Count);
		Assert.AreEqual(3, getAgentResponses[0].Count);
		Assert.AreEqual(new AgentId("MYAGENT1"), getAgentResponses[0]["id"]);
		Assert.AreEqual("MYAGENT1", getAgentResponses[0]["name"]);
		Assert.AreEqual(true, getAgentResponses[0]["enabled"]);
	}
	
	[TestMethod]
	public async Task FindAgents_FilterProps_Async()
	{
		// Arrange
		List<string> properties = ["foo=1", "bar=2", "baz=3"];
		AgentsController controller = CreateAgentsController();
		await CreateAgentAsync("myAgent1", properties);
		
		// Act
		ActionResult<List<object>> result = await controller.FindAgentsAsync(filter: PropertyFilter.Parse("id,name,enabled,properties"), propsFilter: "foo,baz");
		
		// Assert
		Assert.IsNotNull(result.Value);
		List<Dictionary<string, object>> getAgentResponses = result.Value!.Cast<Dictionary<string, object>>().ToList();
		Assert.AreEqual(1, getAgentResponses.Count);
		Assert.AreEqual(4, getAgentResponses[0].Count);
		Assert.AreEqual(new AgentId("MYAGENT1"), getAgentResponses[0]["id"]);
		Assert.AreEqual("MYAGENT1", getAgentResponses[0]["name"]);
		Assert.AreEqual(true, getAgentResponses[0]["enabled"]);
		object o = getAgentResponses[0]["properties"];
		
		List<string> actualProps = [..(IEnumerable<string>)o];
		CollectionAssert.AreEquivalent(new List<string>() { "foo=1", "baz=3"  }, actualProps);
	}

	private AgentsController CreateAgentsController()
	{
		ComputeConfig computeConfig = GlobalConfig.CurrentValue.Plugins.GetComputeConfig();
		IOptionsSnapshot<ComputeConfig> configOptions = new OptionsSnapshotStub<ComputeConfig>(computeConfig);
		AgentsController controller = new (AgentService, null!, null!, configOptions, Tracer, NullLogger<AgentsController>.Instance)
		{
			ControllerContext = new ControllerContext()
			{
				HttpContext = new DefaultHttpContext()
				{
					User = new ClaimsPrincipal(new ClaimsIdentity([HordeClaims.AdminClaim.ToClaim()]))
				}
			}
		};
		return controller;
	}
	
	private async Task<IAgent> CreateAgentAsync(string name, List<string> properties)
	{
		IAgent agent = await AgentService.CreateAgentAsync(new CreateAgentOptions(new AgentId(name), AgentMode.Dedicated, false, ""));
		agent = await AgentService.CreateSessionAsync(agent, new RpcAgentCapabilities(properties), null);
		return agent;
	}
}

