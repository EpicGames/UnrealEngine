// Copyright Epic Games, Inc. All Rights Reserved.

extern alias HordeAgent;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeServer.Agents.Leases;

namespace HordeServer.Tests.Agents.Leases;

[TestClass]
public class LeaseCollectionTests : ComputeTestSetup
{
	[TestMethod]
	public async Task SetOutcome_WithMetrics_Async()
	{
		ILease? lease = await CreateLeaseAsync();

		lease = await LeaseCollection.GetAsync(lease.Id);
		Assert.IsNull(lease!.Metrics);
		Assert.AreEqual(LeaseOutcome.Unspecified, lease.Outcome);

		LeaseCpuTime cpuTime = new (TimeSpan.FromSeconds(3), TimeSpan.FromSeconds(2), TimeSpan.FromSeconds(1));
		LeaseMetrics metrics = new (TimeSpan.FromSeconds(10), TimeSpan.FromSeconds(20), 5, 0.59, cpuTime);
		Assert.IsTrue(await LeaseCollection.TrySetOutcomeAsync(lease.Id, DateTime.UtcNow, LeaseOutcome.Success, LeaseOutcomeReason.None, null, metrics));
		
		lease = await LeaseCollection.GetAsync(lease.Id);
		Assert.IsNotNull(lease!.Metrics);
		Assert.AreEqual(LeaseOutcome.Success, lease.Outcome);
		Assert.AreEqual(TimeSpan.FromSeconds(10), lease.Metrics.SetupTime);
		Assert.AreEqual(TimeSpan.FromSeconds(20), lease.Metrics.TeardownTime);
		Assert.AreEqual(5, lease.Metrics.CpuCount);
		Assert.AreEqual(0.59, lease.Metrics.GlobalCpuUtilization, delta: 0.001);
		Assert.AreEqual(TimeSpan.FromSeconds(3), lease.Metrics.GlobalCpuTime.User);
		Assert.AreEqual(TimeSpan.FromSeconds(2), lease.Metrics.GlobalCpuTime.System);
		Assert.AreEqual(TimeSpan.FromSeconds(1), lease.Metrics.GlobalCpuTime.Idle);
	}
	
	[TestMethod]
	public async Task SetOutcome_WithoutMetrics_Async()
	{
		ILease? lease = await CreateLeaseAsync();
		
		Assert.IsTrue(await LeaseCollection.TrySetOutcomeAsync(lease.Id, DateTime.UtcNow, LeaseOutcome.Failed, LeaseOutcomeReason.None, null, null));
		lease = await LeaseCollection.GetAsync(lease.Id);
		Assert.IsNull(lease!.Metrics);
		Assert.AreEqual(LeaseOutcome.Failed, lease.Outcome);
	}

	private Task<ILease> CreateLeaseAsync()
	{
		return LeaseCollection.AddAsync(LeaseId.Parse("aaaaaaaaaaaaaaaaaaaaaaaa"), null, "lease1", new AgentId("agent1"),
			SessionId.Parse("bbbbbbbbbbbbbbbbbbbbbbbb"), null, null, null, DateTime.UtcNow, Array.Empty<byte>());
	}
}