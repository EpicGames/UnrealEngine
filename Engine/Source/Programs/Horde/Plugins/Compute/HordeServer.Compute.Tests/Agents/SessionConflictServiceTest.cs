// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using HordeServer.Agents;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests.Agents;

[TestClass]
public class SessionConflictServiceTest : ComputeTestSetup
{
	SessionConflictService SessionConflictService => ServiceProvider.GetRequiredService<SessionConflictService>();

	[TestMethod]
	public async Task RecordConflictReturnsIncrementingCountAsync()
	{
		AgentId agentId = new("agent-1");
		Assert.AreEqual(1, await SessionConflictService.RecordConflictAsync(agentId));
		Assert.AreEqual(2, await SessionConflictService.RecordConflictAsync(agentId));
		Assert.AreEqual(3, await SessionConflictService.RecordConflictAsync(agentId));
	}

	[TestMethod]
	public async Task MultipleAgentsTrackedIndependentlyAsync()
	{
		AgentId agent1 = new("agent-1");
		AgentId agent2 = new("agent-2");

		Assert.AreEqual(1, await SessionConflictService.RecordConflictAsync(agent1));
		Assert.AreEqual(1, await SessionConflictService.RecordConflictAsync(agent2));
		Assert.AreEqual(2, await SessionConflictService.RecordConflictAsync(agent1));
		Assert.AreEqual(2, await SessionConflictService.RecordConflictAsync(agent2));
	}

	[TestMethod]
	public async Task DrainReturnsAllAgentsWithCountsAsync()
	{
		AgentId agent1 = new("agent-1");
		AgentId agent2 = new("agent-2");

		await SessionConflictService.RecordConflictAsync(agent1);
		await SessionConflictService.RecordConflictAsync(agent1);
		await SessionConflictService.RecordConflictAsync(agent1);
		await SessionConflictService.RecordConflictAsync(agent2);

		List<(AgentId Id, int Count)> conflicts = await SessionConflictService.DrainConflictsAsync();
		Assert.AreEqual(2, conflicts.Count);
		Assert.IsTrue(conflicts.Any(c => c.Id == agent1 && c.Count == 3));
		Assert.IsTrue(conflicts.Any(c => c.Id == agent2 && c.Count == 1));
	}

	[TestMethod]
	public async Task DrainClearsSortedSetAsync()
	{
		AgentId agentId = new("agent-1");
		await SessionConflictService.RecordConflictAsync(agentId);

		List<(AgentId Id, int Count)> first = await SessionConflictService.DrainConflictsAsync();
		Assert.AreEqual(1, first.Count);

		List<(AgentId Id, int Count)> second = await SessionConflictService.DrainConflictsAsync();
		Assert.AreEqual(0, second.Count);
	}

	[TestMethod]
	public async Task DrainEmptyReturnsEmptyListAsync()
	{
		List<(AgentId Id, int Count)> conflicts = await SessionConflictService.DrainConflictsAsync();
		Assert.AreEqual(0, conflicts.Count);
	}
}
