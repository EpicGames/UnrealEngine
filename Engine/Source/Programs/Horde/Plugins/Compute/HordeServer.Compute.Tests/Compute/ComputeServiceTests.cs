// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;

namespace HordeServer.Tests.Compute
{
	[TestClass]
	public class ComputeServiceTests
	{
		[TestMethod]
		[DataRow("192.168.1.1")]
		[DataRow("::ffff:192.168.1.1")]
		public void TryGetNetworkConfigMatchesCidrBlock(string ip)
		{
			ComputeConfig config = new() { Networks = { new NetworkConfig { Id = "test", CidrBlock = "192.168.0.0/16" } } };
			Assert.IsTrue(config.TryGetNetworkConfig(IPAddress.Parse(ip), out _));
		}
	}
}
