// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Compute;
using HordeServer.Server;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Tests.Compute
{
	[TestClass]
	public class RedisTaskSchedulerTest : ComputeTestSetup
	{
		private readonly RedisTaskScheduler<string, string> _scheduler;

		public RedisTaskSchedulerTest()
		{
			using ILoggerFactory loggerFactory = LoggerFactory.Create(builder => { builder.AddConsole(); });
			IRedisService redisService = GetRedisServiceSingleton();
			_scheduler = new RedisTaskScheduler<string, string>(redisService.ConnectionPool, new RedisKey("myBaseKey"), loggerFactory.CreateLogger<RedisTaskScheduler<string, string>>());
		}

		public override async ValueTask DisposeAsync()
		{
			await _scheduler.DisposeAsync();
			await base.DisposeAsync();
			GC.SuppressFinalize(this);
		}

		[TestMethod]
		public async Task EnqueueAndDequeueTaskAsync()
		{
			await _scheduler.EnqueueAsync("queue", "task1", true);
			string? dequeuedTask = await _scheduler.DequeueAsync("queue");
			Assert.AreEqual("task1", dequeuedTask);
		}

		[TestMethod]
		[Ignore("Currently fails as nothing picks up tasks(?)")]
		public async Task DequeueWithPredicateFromMiddleOfQueueAsync()
		{
			using CancellationTokenSource cts = new(2000);
			await _scheduler.EnqueueAsync("queue", "task1", true);
			await _scheduler.EnqueueAsync("queue", "task2", true);
			await _scheduler.EnqueueAsync("queue", "task3", true);
			Task<(string, string)?> dequeueTask = await _scheduler.DequeueAsync((x) => ValueTask.FromResult(x == "task2"), cts.Token);

			(string queueId, string task) = (await dequeueTask.WaitAsync(cts.Token))!.Value;

			Assert.AreEqual("queue", queueId);
			Assert.AreEqual("task2", task);
		}

		[TestMethod]
		public async Task EnqueueTaskAtFrontAsync()
		{
			await _scheduler.EnqueueAsync("queue", "task1", true);
			await _scheduler.EnqueueAsync("queue", "task2", true);
			Assert.AreEqual("task2", await _scheduler.DequeueAsync("queue"));
		}

		[TestMethod]
		public async Task EnqueueTaskAtBackAsync()
		{
			await _scheduler.EnqueueAsync("queue", "task1", false);
			await _scheduler.EnqueueAsync("queue", "task2", false);
			Assert.AreEqual("task1", await _scheduler.DequeueAsync("queue"));
		}

		[TestMethod]
		public async Task EnqueueTasksOnSeparateQueuesAsync()
		{
			await _scheduler.EnqueueAsync("queue1", "task1", true);
			await _scheduler.EnqueueAsync("queue2", "task2", true);
			Assert.AreEqual("task1", await _scheduler.DequeueAsync("queue1"));
			Assert.AreEqual("task2", await _scheduler.DequeueAsync("queue2"));
		}

		[TestMethod]
		public async Task DequeueEmptyQueueAsync()
		{
			Assert.IsNull(await _scheduler.DequeueAsync("queue"));
		}

		[TestMethod]
		public async Task DequeuingMakesQueueActiveAsync()
		{
			await _scheduler.EnqueueAsync("queue", "task1", true);
			Assert.AreEqual(1, (await _scheduler.GetInactiveQueuesAsync()).Count);

			await _scheduler.DequeueAsync("queue");
			Assert.AreEqual(0, (await _scheduler.GetInactiveQueuesAsync()).Count);
		}
	}
}