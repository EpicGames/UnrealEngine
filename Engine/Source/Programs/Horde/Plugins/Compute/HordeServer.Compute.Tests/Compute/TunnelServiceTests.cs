// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Net.Sockets;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Compute.Clients;
using HordeServer.Compute;
using Microsoft.Extensions.Options;

namespace HordeServer.Tests.Compute
{
	[TestClass]
	public sealed class TunnelServiceTests
	{
		private static readonly Random s_random = new(112233);
		private readonly string _hostname = "127.0.0.1";
		private readonly int _echoPort = GetAvailablePort();
		private readonly int _tunnelPort = GetAvailablePort();

		private Task? _tunnelServerTask;
		private Task? _echoServerTask;

		/// <summary>
		/// Find an available TCP/IP port
		/// </summary>
		/// <returns>Port number available</returns>
		public static int GetAvailablePort()
		{
			using TcpListener listener = new(IPAddress.Loopback, 0);
			listener.Start();
			int port = ((IPEndPoint)listener.LocalEndpoint).Port;
			listener.Stop();
			return port;
		}

		[TestCleanup]
		public async Task CleanupAsync()
		{
			if (_echoServerTask != null)
			{
				await _echoServerTask.IgnoreCanceledExceptionsAsync().ConfigureAwait(false);
			}

			if (_tunnelServerTask != null)
			{
				await _tunnelServerTask.IgnoreCanceledExceptionsAsync().ConfigureAwait(false);
			}
		}

		[TestMethod]
		[DataRow(20)]
		[DataRow(1024)]
		[DataRow(4096)]
		[DataRow(4070)]
		[DataRow(20000)]
		public async Task SendRecvTestAsync(int echoServerBufferSize)
		{
			using CancellationTokenSource cts = new(10000);

			_echoServerTask = StartEchoServerAsync(_echoPort, echoServerBufferSize, cts.Token);

			ComputeServerConfig serverSettings = new() { ComputeTunnelPort = _tunnelPort };
			using TunnelService tunnelService = new(new OptionsWrapper<ComputeServerConfig>(serverSettings), ComputeTestSetup.CreateConsoleLogger<TunnelService>());
			tunnelService.Start(IPAddress.Any);
			_tunnelServerTask = tunnelService.ServerTask!;

			using TcpClient client = new();
			await client.ConnectAsync(_hostname, _tunnelPort, cts.Token);
			await using NetworkStream stream = client.GetStream();
			using StreamReader streamReader = new(stream);
			await using StreamWriter streamWriter = new(stream) { AutoFlush = true };

			await streamWriter.WriteLineAsync(new TunnelHandshakeRequest(_hostname, _echoPort).Serialize());

			string? handshakeResponseStr = await ReadLineWithTimeoutAsync(streamReader, TimeSpan.FromSeconds(3));
			TunnelHandshakeResponse handshakeResponse = TunnelHandshakeResponse.Deserialize(handshakeResponseStr);
			if (!handshakeResponse.IsSuccess)
			{
				throw new Exception("Handshake failed! " + handshakeResponse.Message);
			}

			try
			{
				await AssertSendReceiveAsync(streamReader, streamWriter, 1);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize / 3);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize - 1);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize + 1);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize * 10 + 3);
				await AssertSendReceiveAsync(streamReader, streamWriter, echoServerBufferSize * 20);
			}
			finally
			{
				await cts.CancelAsync();
				await tunnelService.StopAsync(CancellationToken.None);
			}
		}

		private static async Task AssertSendReceiveAsync(StreamReader reader, StreamWriter writer, int length)
		{
			string randomString = GenerateRandomString(length);
			await writer.WriteLineAsync(randomString);
			string? response = await ReadLineWithTimeoutAsync(reader, TimeSpan.FromSeconds(3));
			Assert.AreEqual(randomString, response);
		}

		private static string GenerateRandomString(int length)
		{
			const string Chars = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789";
			StringBuilder result = new(length);

			for (int i = 0; i < length; i++)
			{
				result.Append(Chars[s_random.Next(Chars.Length)]);
			}

			return result.ToString();
		}

		private static async Task StartEchoServerAsync(int port, int bufferSize, CancellationToken cancellationToken)
		{
			using TcpListener listener = new(IPAddress.Any, port);
			listener.Start();

			Console.WriteLine($"Echo server started on port {port}...");
			List<Task> tasks = new();

			while (!cancellationToken.IsCancellationRequested)
			{
				TcpClient client = await listener.AcceptTcpClientAsync(cancellationToken);
				Console.WriteLine("Accepted connection from client...");
				tasks.Add(HandleEchoClientAsync(client, bufferSize, cancellationToken));
			}

			await Task.WhenAll(tasks);
		}

		private static async Task HandleEchoClientAsync(TcpClient client, int bufferSize, CancellationToken cancellationToken)
		{
			await using (NetworkStream stream = client.GetStream())
			{
				Console.WriteLine("Echo client connected!");
				byte[] buffer = new byte[bufferSize];
				int bytesRead;

				while ((bytesRead = await stream.ReadAsync(buffer, 0, buffer.Length, cancellationToken)) > 0)
				{
					await stream.WriteAsync(buffer, 0, bytesRead, cancellationToken);
				}
			}

			client.Close();
			Console.WriteLine("Client connection closed.");
		}

		private static async Task<string?> ReadLineWithTimeoutAsync(StreamReader reader, TimeSpan timeout)
		{
			Task<string?> readLineTask = reader.ReadLineAsync();
			Task delayTask = Task.Delay(timeout);
			Task completedTask = await Task.WhenAny(readLineTask, delayTask);
			if (completedTask == readLineTask)
			{
				return await readLineTask;
			}
			throw new TimeoutException("Read operation timed out.");
		}
	}
}
