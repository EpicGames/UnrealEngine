// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Agents;
using HordeServer.Agents.Leases;
using HordeServer.Compute;
using HordeServer.Logs;
using HordeServer.Storage;
using Microsoft.Extensions.DependencyInjection;

namespace HordeServer.Tests
{
	public class ComputeTestSetup : ServerTestSetup
	{
		public ComputeService ComputeService => ServiceProvider.GetRequiredService<ComputeService>();
		public ILeaseCollection LeaseCollection => ServiceProvider.GetRequiredService<ILeaseCollection>();
		public IAgentCollection AgentCollection => ServiceProvider.GetRequiredService<IAgentCollection>();
		public IAgentScheduler AgentScheduler => ServiceProvider.GetRequiredService<IAgentScheduler>();
		public AgentService AgentService => ServiceProvider.GetRequiredService<AgentService>();
		public StorageService StorageService => ServiceProvider.GetRequiredService<StorageService>();
		public ILogCollection LogCollection => ServiceProvider.GetRequiredService<ILogCollection>();

		public ComputeTestSetup()
		{
			AddPlugin<StoragePlugin>();
			AddPlugin<ComputePlugin>();
		}
	}
}
