// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Storage;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Storage;
using Microsoft.Extensions.Logging.Abstractions;

namespace HordeServer.Tests.Logs
{
	class TestLogWriter : IAsyncDisposable
	{
		ILog _logFile;
		readonly IStorageNamespace _storageNamespace;
		readonly IBlobWriter _blobWriter;
		readonly LogBuilder _builder;
		int _lineCount;

		public TestLogWriter(ILog logFile, StorageService storageService)
		{
			_logFile = logFile;
			_storageNamespace = storageService.GetNamespace(Namespace.Logs);
			_blobWriter = _storageNamespace.CreateBlobWriter(logFile.RefName);
			_builder = new LogBuilder((logFile.Type == LogType.Text) ? LogFormat.Text : LogFormat.Json, NullLogger.Instance);
		}

		public async ValueTask DisposeAsync()
		{
			await _blobWriter.DisposeAsync();
		}

		public async Task<ILog> WriteDataAsync(ReadOnlyMemory<byte> data)
		{
			_builder.WriteData(data);
			_lineCount += data.Span.Count((byte)'\n');
			_logFile = await _logFile.UpdateLineCountAsync(_lineCount, false, CancellationToken.None);
			return await FlushAsync(false);
		}

		public async Task<ILog> FlushAsync(bool complete = true)
		{
			IHashedBlobRef<LogNode> handle = await _builder.FlushAsync(_blobWriter, complete, CancellationToken.None);
			await _storageNamespace.AddRefAsync(_logFile.RefName, handle);
			_logFile = await _logFile.UpdateLineCountAsync(_lineCount, complete, CancellationToken.None);
			return _logFile;
		}
	}

	[TestClass]
	public class LogIndexingTests : ComputeTestSetup
	{
		private readonly byte[] _data = HordeServer.Compute.Tests.Properties.Resources.TextFile;

		[TestMethod]
		public async Task IndexTestsAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog logFile = await LogCollection.AddAsync(jobId, null, null, LogType.Text);

			// Write the test data to the log file in blocks
			await using (TestLogWriter logWriter = new TestLogWriter(logFile, StorageService))
			{
				int offset = 0;
				int lineIndex = 0;
				while (offset < _data.Length)
				{
					int length = 0;
					int lineCount = 0;

					for (int idx = offset; idx < Math.Min(_data.Length, offset + 18830); idx++)
					{
						if (_data[idx] == '\n')
						{
							length = (idx + 1) - offset;
							lineCount++;
						}
					}

					await logWriter.WriteDataAsync(_data.AsMemory(offset, length));

					offset += length;
					lineIndex += lineCount;
				}

				await logWriter.FlushAsync();
			}

			// Read the data back out and check it's the same
			byte[] readData = new byte[_data.Length];
			using (Stream stream = await logFile.OpenRawStreamAsync())
			{
				int readSize = await stream.ReadAsync(readData, 0, readData.Length);
				Assert.AreEqual(readData.Length, readSize);

				int equalSize = 0;
				while (equalSize < _data.Length && _data[equalSize] == readData[equalSize])
				{
					equalSize++;
				}

				Assert.AreEqual(equalSize, readSize);
			}

			// Test some searches
			await SearchLogDataTestAsync(logFile.Id);
		}

		[TestMethod]
		public void TrieTests()
		{
			ReadOnlyTrieBuilder builder = new ReadOnlyTrieBuilder();

			ulong[] values = { 1, 2, 3, 5, 8, 13, 21, 34, 55, 89, 144, 233, 377, 610, 987, 1597, 2584, 4181, 6765, 10946, 0xfedcba9876543210UL };
			foreach (ulong value in values)
			{
				builder.Add(value);
			}

			ReadOnlyTrie trie = builder.Build();
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(0, UInt64.MaxValue), values));
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(0, 90), values.Where(x => x <= 90)));
			Assert.IsTrue(Enumerable.SequenceEqual(trie.EnumerateRange(2, 89), values.Where(x => x >= 2 && x <= 89)));
		}

		[TestMethod]
		public async Task PartialTokenTestsAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog logFile = await LogCollection.AddAsync(jobId, null, null, LogType.Text);

			string[] lines =
			{
				"abcdefghi\n",
				"jklmno123\n",
				"pqrst99uv\n",
				"wx\n"
			};

			int length = 0;
			await using (TestLogWriter logWriter = new TestLogWriter(logFile, StorageService))
			{
				for (int lineIdx = 0; lineIdx < lines.Length; lineIdx++)
				{
					logFile = await logWriter.WriteDataAsync(Encoding.UTF8.GetBytes(lines[lineIdx]));
					length += lines[lineIdx].Length;
				}
				logFile = await logWriter.FlushAsync(true);
			}

			for (int lineIdx = 0; lineIdx < lines.Length; lineIdx++)
			{
				for (int strLen = 1; strLen < 7; strLen++)
				{
					for (int strOfs = 0; strOfs + strLen < lines[lineIdx].Length - 1; strOfs++)
					{
						string str = lines[lineIdx].Substring(strOfs, strLen);

						SearchStats stats = new SearchStats();
						List<int> results = await logFile.SearchLogDataAsync(str, 0, 5, stats, CancellationToken.None);
						Assert.AreEqual(1, results.Count);
						Assert.AreEqual(lineIdx, results[0]);

						Assert.AreEqual(1, stats.NumScannedBlocks);
						Assert.AreEqual(3, stats.NumSkippedBlocks);
						Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
					}
				}
			}
		}

		[TestMethod]
		public async Task AppendIndexTestsAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog logFile = await LogCollection.AddAsync(jobId, null, null, LogType.Text);

			await using (TestLogWriter writer = new TestLogWriter(logFile, StorageService))
			{
				logFile = await writer.WriteDataAsync(Encoding.UTF8.GetBytes("abc\n"));
				logFile = await writer.WriteDataAsync(Encoding.UTF8.GetBytes("def\n"));
				logFile = await writer.WriteDataAsync(Encoding.UTF8.GetBytes("ghi\n"));
				logFile = await writer.FlushAsync(true);
			}

			{
				SearchStats stats = new SearchStats();
				List<int> results = await logFile.SearchLogDataAsync("abc", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(0, results[0]);

				Assert.AreEqual(1, stats.NumScannedBlocks); // abc
				Assert.AreEqual(2, stats.NumSkippedBlocks); // def + ghi
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
			{
				SearchStats stats = new SearchStats();
				List<int> results = await logFile.SearchLogDataAsync("def", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(1, results[0]);

				Assert.AreEqual(1, stats.NumScannedBlocks); // def
				Assert.AreEqual(2, stats.NumSkippedBlocks); // abc + ghi
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
			{
				SearchStats stats = new SearchStats();
				List<int> results = await logFile.SearchLogDataAsync("ghi", 0, 5, stats, CancellationToken.None);
				Assert.AreEqual(1, results.Count);
				Assert.AreEqual(2, results[0]);

				Assert.AreEqual(1, stats.NumScannedBlocks); // ghi
				Assert.AreEqual(2, stats.NumSkippedBlocks); // abc + def
				Assert.AreEqual(0, stats.NumFalsePositiveBlocks);
			}
		}

		async Task SearchLogDataTestAsync(LogId logId)
		{
			ILog? logFile = await LogCollection.GetAsync(logId, CancellationToken.None);
			Assert.IsNotNull(logFile);

			await SearchLogDataTestAsync(logFile, "HISPANIOLA", 0, 4, new[] { 1503, 1520, 1525, 1595 });
			await SearchLogDataTestAsync(logFile, "Hispaniola", 0, 4, new[] { 1503, 1520, 1525, 1595 });
			await SearchLogDataTestAsync(logFile, "HizpaniolZ", 0, 4, Array.Empty<int>());
			await SearchLogDataTestAsync(logFile, "Pieces of eight!", 0, 100, new[] { 2227, 2228, 5840, 5841, 7520 });
			await SearchLogDataTestAsync(logFile, "NEWSLETTER", 0, 100, new[] { 7886 });
		}

		static async Task SearchLogDataTestAsync(ILog logFile, string text, int firstLine, int count, int[] expectedLines)
		{
			SearchStats stats = new SearchStats();
			List<int> lines = await logFile.SearchLogDataAsync(text, firstLine, count, stats, CancellationToken.None);
			Assert.IsTrue(lines.SequenceEqual(expectedLines));
		}
	}
}
