// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Logs;
using HordeServer.Agents.Sessions;
using HordeServer.Jobs;
using HordeServer.Logs;
using HordeServer.Utilities;

namespace HordeServer.Tests.Logs
{
	[TestClass]
	public sealed class LogServiceTest : ComputeTestSetup
	{
		[TestMethod]
		public void PlainTextDecoder()
		{
			string jsonText = "{\"time\":\"2022-01-26T14:18:29\",\"level\":\"Error\",\"message\":\"       \\tdepends on: /mnt/horde/U5\\u002BR5.0\\u002BInc\\u002BMin/Sync/Engine/Binaries/Linux/libUnrealEditor-Core.so\",\"id\":1,\"line\":5,\"lineCount\":1028}";
			byte[] jsonData = Encoding.UTF8.GetBytes(jsonText);

			byte[] outputData = new byte[jsonData.Length];
			int outputLength = HordeServer.Logs.LogCollection.ConvertToPlainText(jsonData, outputData, 0);
			string outputText = Encoding.UTF8.GetString(outputData, 0, outputLength);

			string expectText = "       \tdepends on: /mnt/horde/U5+R5.0+Inc+Min/Sync/Engine/Binaries/Linux/libUnrealEditor-Core.so\n";
			Assert.AreEqual(outputText, expectText);
		}

		[TestMethod]
		public async Task WriteLogLifecycleOldTestAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog logFile = await LogCollection.AddAsync(jobId, null, null, LogType.Text, logId: null, cancellationToken: CancellationToken.None);

			await using (TestLogWriter writer = new TestLogWriter(logFile, StorageService))
			{
				await writer.WriteDataAsync(Encoding.ASCII.GetBytes("hello\n"));
				await writer.FlushAsync(false);

				await writer.WriteDataAsync(Encoding.ASCII.GetBytes("foo\nbar\n"));
				await writer.FlushAsync(false);

				await writer.WriteDataAsync(Encoding.ASCII.GetBytes("baz\n"));
				await writer.FlushAsync(true);
			}

			logFile = (await LogCollection.GetAsync(logFile.Id, CancellationToken.None))!;

			Assert.AreEqual("hello", await ReadLogAsync(logFile, 0, 5));
			Assert.AreEqual("foo\nbar\nbaz\n", await ReadLogAsync(logFile, 6, 12));

			List<Utf8String> lines = await logFile.ReadLinesAsync(0, 100, CancellationToken.None);
			Assert.AreEqual("hello", lines[0].ToString());
			Assert.AreEqual("foo", lines[1].ToString());
			Assert.AreEqual("bar", lines[2].ToString());
			Assert.AreEqual("baz", lines[3].ToString());
		}

		private static async Task AssertLineOffsetAsync(ILogCollection logCollection, LogId logId, int lineIndex, int clampedLineIndex, long offset)
		{
			ILog? logFile = await logCollection.GetAsync(logId, CancellationToken.None);
			Assert.IsNotNull(logFile);

			List<Utf8String> lines = await logFile.ReadLinesAsync(0, lineIndex + 1, CancellationToken.None);
			Assert.AreEqual(clampedLineIndex, Math.Min(lineIndex, lines.Count));
			Assert.AreEqual(offset, lines.Take(lineIndex).Sum(x => x.Length + 1));
		}

		static async Task<string> ReadLogAsync(ILog log, long offset, long length)
		{
			using Stream stream = await log.OpenRawStreamAsync();

			byte[] prefix = new byte[(int)offset];
			await stream.ReadGreedyAsync(prefix);

			byte[] data = new byte[(int)length];
			int actualLength = await stream.ReadGreedyAsync(data);

			return Encoding.UTF8.GetString(data.AsSpan(0, actualLength));
		}

		[TestMethod]
		public async Task WriteLogLifecycleAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog log = await LogCollection.AddAsync(jobId, null, null, LogType.Text);

			string str1 = "hello\n";
			string str2 = "foo\nbar\n";
			string str3 = "baz\nqux\nquux\n";
			string str4 = "quuz\n";

			int lineIndex = 0;
			int offset = 0;

			await using TestLogWriter logWriter = new TestLogWriter(log, StorageService);

			// First write with flush. Will become chunk #1
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(str1));
			Assert.AreEqual(str1, await ReadLogAsync(log, 0, str1.Length));
			Assert.AreEqual(str1, await ReadLogAsync(log, 0, str1.Length + 100)); // Reading too far is valid?
			await AssertLineOffsetAsync(LogCollection, log.Id, 0, 0, 0);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);

			// Second write without flushing. Will become chunk #2
			offset += str1.Length;
			lineIndex += str1.Count(f => f == '\n');
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(str2));

			Assert.AreEqual(str1 + str2, await ReadLogAsync(log, 0, str1.Length + str2.Length));
			await AssertLineOffsetAsync(LogCollection, log.Id, 0, 0, 0);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);
			await AssertLineOffsetAsync(LogCollection, log.Id, 2, 2, 10);
			await AssertLineOffsetAsync(LogCollection, log.Id, 3, 3, 14);

			// Third write without flushing. Will become chunk #2
			offset += str2.Length;
			lineIndex += str3.Count(f => f == '\n');
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(str3));
			Assert.AreEqual(str1 + str2 + str3, await ReadLogAsync(log, 0, str1.Length + str2.Length + str3.Length));
			await AssertLineOffsetAsync(LogCollection, log.Id, 0, 0, 0);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);
			await AssertLineOffsetAsync(LogCollection, log.Id, 2, 2, 10);
			await AssertLineOffsetAsync(LogCollection, log.Id, 3, 3, 14);
			await AssertLineOffsetAsync(LogCollection, log.Id, 4, 4, 18);

			// Fourth write with flush. Will become chunk #2
			offset += str3.Length;
			lineIndex += str4.Count(f => f == '\n');
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(str4));
			Assert.AreEqual(str1 + str2 + str3 + str4,
			await ReadLogAsync(log, 0, str1.Length + str2.Length + str3.Length + str4.Length));
			Assert.AreEqual(str3 + str4,
			await ReadLogAsync(log, str1.Length + str2.Length, str3.Length + str4.Length));
			await AssertLineOffsetAsync(LogCollection, log.Id, 0, 0, 0);
			await AssertLineOffsetAsync(LogCollection, log.Id, 1, 1, 6);
			await AssertLineOffsetAsync(LogCollection, log.Id, 2, 2, 10);
			await AssertLineOffsetAsync(LogCollection, log.Id, 3, 3, 14);
			await AssertLineOffsetAsync(LogCollection, log.Id, 4, 4, 18);
			await AssertLineOffsetAsync(LogCollection, log.Id, 5, 5, 22);
			await AssertLineOffsetAsync(LogCollection, log.Id, 6, 6, 27);
			await AssertLineOffsetAsync(LogCollection, log.Id, 7, 7, 32);

			// Fifth write with flush and data that will span more than chunk. Will become chunk #3
			string a = "Lorem ipsum dolor sit amet\n";
			string b = "consectetur adipiscing\n";
			string str5 = a + b;

			offset += str4.Length;
			lineIndex += str5.Count(f => f == '\n');

			// Using this single write below will fail the ReadLogFile assert below. A bug?
			//await LogFileService.WriteLogDataAsync(LogFile, Offset, LineIndex, Encoding.ASCII.GetBytes(Str5), true);

			// Dividing it in two like this will work however
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(a));
			log = await logWriter.WriteDataAsync(Encoding.ASCII.GetBytes(b));

			Assert.AreEqual(str5, await ReadLogAsync(log, offset, str5.Length));
		}

		[TestMethod]
		public async Task GetLogFileTestAsync()
		{
			// Will implicitly test GetLogFileAsync(), AddCachedLogFile()
			JobId jobId = JobIdUtils.GenerateNewId();
			SessionId sessionId = SessionIdUtils.GenerateNewId();
			ILog a = await LogCollection.AddAsync(jobId, null, sessionId, LogType.Text, logId: null, cancellationToken: CancellationToken.None);
			ILog b = (await LogCollection.GetAsync(a.Id, CancellationToken.None))!;
			Assert.AreEqual(a.JobId, b.JobId);
			Assert.AreEqual(a.SessionId, b.SessionId);
			Assert.AreEqual(a.Type, b.Type);

			ILog? notFound = await LogCollection.GetAsync(LogIdUtils.GenerateNewId(), CancellationToken.None);
			Assert.IsNull(notFound);
		}

		[TestMethod]
		public async Task AuthorizeForSessionAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			SessionId sessionId = SessionIdUtils.GenerateNewId();
			ILog log = await LogCollection.AddAsync(jobId, null, sessionId, LogType.Text, logId: null, cancellationToken: CancellationToken.None);
			ILog logNoSession = await LogCollection.AddAsync(jobId, null, null, LogType.Text, logId: null, cancellationToken: CancellationToken.None);

			ClaimsPrincipal hasClaim = new ClaimsPrincipal(new ClaimsIdentity(new List<Claim>
			{
				new Claim(HordeClaimTypes.AgentSessionId, sessionId.ToString()),
			}, "TestAuthType"));
			ClaimsPrincipal hasNoClaim = new ClaimsPrincipal(new ClaimsIdentity(new List<Claim>
			{
				new Claim(HordeClaimTypes.AgentSessionId, "invalid-session-id"),
			}, "TestAuthType"));

			Assert.IsTrue(log.AuthorizeForSession(hasClaim));
			Assert.IsFalse(log.AuthorizeForSession(hasNoClaim));
			Assert.IsFalse(logNoSession.AuthorizeForSession(hasClaim));
		}

		[TestMethod]
		public async Task ChunkSplittingAsync()
		{
			JobId jobId = JobIdUtils.GenerateNewId();
			ILog logFile = await LogCollection.AddAsync(jobId, null, null, LogType.Text, logId: null, cancellationToken: CancellationToken.None);
			await using TestLogWriter logWriter = new TestLogWriter(logFile, StorageService);

			byte[] line1 = Encoding.UTF8.GetBytes("hello world\n");
			await logWriter.WriteDataAsync(line1);

			byte[] line2 = Encoding.UTF8.GetBytes("ab\n");
			await logWriter.WriteDataAsync(line2);

			byte[] line3 = Encoding.UTF8.GetBytes("a\n");
			await logWriter.WriteDataAsync(line3);

			byte[] line4 = Encoding.UTF8.GetBytes("b\n");
			await logWriter.WriteDataAsync(line4);

			byte[] line5 = Encoding.UTF8.GetBytes("a\nb\n");
			await logWriter.WriteDataAsync(line5);

			logFile = await logWriter.FlushAsync();
			LogMetadata metadata = await logFile.GetMetadataAsync(CancellationToken.None);
			Assert.AreEqual(6, metadata.MaxLineIndex);
		}
	}
}