// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Logs;
using EpicGames.Redis;
using HordeServer.Logs;
using HordeServer.Server;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace HordeServer.Tests.Logs
{
	[TestClass]
	public class LogTailServiceTests : ComputeTestSetup
	{
		readonly LogId _logId = LogId.Parse("637809e571f04e1d1ba311f1");

		[TestMethod]
		public async Task TruncatedAsync()
		{
			LogTailService tailService = ServiceProvider.GetRequiredService<LogTailService>();
			await tailService.EnableTailingAsync(_logId, 30);
			await tailService.AppendAsync(_logId, 30, Encoding.UTF8.GetBytes("foo\nbar\nbaz\n"));

			for (int idx = 25; idx <= 29; idx++)
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, idx, 10);
				Assert.AreEqual(0, lines.Count);
			}

			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 30, 10);
				Assert.AreEqual(3, lines.Count);
				Assert.AreEqual("foo", lines[0].ToString());
				Assert.AreEqual("bar", lines[1].ToString());
				Assert.AreEqual("baz", lines[2].ToString());
			}
		}

		[TestMethod]
		public async Task TruncatedMultiLineAsync()
		{
			LogTailService tailService = ServiceProvider.GetRequiredService<LogTailService>();
			await tailService.EnableTailingAsync(_logId, 30);
			await tailService.AppendAsync(_logId, 30, Encoding.UTF8.GetBytes("foo\n"));
			await tailService.AppendAsync(_logId, 31, Encoding.UTF8.GetBytes("bar\n"));
			await tailService.AppendAsync(_logId, 32, Encoding.UTF8.GetBytes("baz\n"));

			for (int idx = 25; idx <= 29; idx++)
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, idx, 10);
				Assert.AreEqual(0, lines.Count);
			}

			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 30, 10);
				Assert.AreEqual(3, lines.Count);
				Assert.AreEqual("foo", lines[0].ToString());
				Assert.AreEqual("bar", lines[1].ToString());
				Assert.AreEqual("baz", lines[2].ToString());
			}
		}

		[TestMethod]
		public async Task SplittingAsync()
		{
			IRedisService redisService = ServiceProvider.GetRequiredService<IRedisService>();

			LogTailService tailService = new LogTailService(redisService, ServiceProvider.GetRequiredService<IClock>(), 4, ServiceProvider.GetRequiredService<IServerInfo>(), ServiceProvider.GetRequiredService<ILogger<LogTailService>>());
			await tailService.StartAsync(CancellationToken.None);

			await tailService.EnableTailingAsync(_logId, 30);
			await tailService.AppendAsync(_logId, 30, Encoding.UTF8.GetBytes("line30\nline31\nline32\nline33\nline34\nline35\nline36\nline37\nline38\nline39\nline40\n"));

			Assert.AreEqual(4, await redisService.GetDatabase().SortedSetLengthAsync(LogTailService.TailDataKey(_logId)));

			await tailService.FlushAsync(_logId, 33);

			Assert.AreEqual(4, await redisService.GetDatabase().SortedSetLengthAsync(LogTailService.TailDataKey(_logId)));

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(1.0));

			Assert.AreEqual(3, await redisService.GetDatabase().SortedSetLengthAsync(LogTailService.TailDataKey(_logId)));

			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 25, 20);
				Assert.AreEqual(0, lines.Count);
			}
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 31, 20);
				Assert.AreEqual(0, lines.Count);
			}
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 32, 20);
				Assert.AreEqual("line32", lines[0].ToString());
				Assert.AreEqual("line33", lines[1].ToString());
				Assert.AreEqual("line34", lines[2].ToString());
				Assert.AreEqual("line35", lines[3].ToString());
				Assert.AreEqual("line36", lines[4].ToString());
				Assert.AreEqual("line37", lines[5].ToString());
				Assert.AreEqual("line38", lines[6].ToString());
				Assert.AreEqual("line39", lines[7].ToString());
				Assert.AreEqual("line40", lines[8].ToString());
			}
		}

		[TestMethod]
		public async Task ExpiryAsync()
		{
			IRedisService redisService = GetRedisServiceSingleton();

			LogTailService tailService = ServiceProvider.GetRequiredService<LogTailService>();

			await tailService.StartAsync(CancellationToken.None);
			await tailService.EnableTailingAsync(_logId, 30);
			await tailService.AppendAsync(_logId, 30, Encoding.UTF8.GetBytes("foo\n"));

			Assert.IsTrue(await redisService.GetDatabase().KeyExistsAsync(LogTailService.TailNextKey(_logId)));
			Assert.IsTrue(await redisService.GetDatabase().KeyExistsAsync(LogTailService.TailDataKey(_logId)));

			await Clock.AdvanceAsync(TimeSpan.FromMinutes(5.0));

			Assert.IsFalse(await redisService.GetDatabase().KeyExistsAsync(LogTailService.TailNextKey(_logId)));
			Assert.IsFalse(await redisService.GetDatabase().KeyExistsAsync(LogTailService.TailDataKey(_logId)));
		}

		[TestMethod]
		public async Task MissingDataAsync()
		{
			LogTailService tailService = ServiceProvider.GetRequiredService<LogTailService>();

			await tailService.StartAsync(CancellationToken.None);
			await tailService.EnableTailingAsync(_logId, 30);
			await tailService.AppendAsync(_logId, 31, Encoding.UTF8.GetBytes("line31\nline32\nline33\nline34\nline35\n"));

			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 29, 4);
				Assert.AreEqual(0, lines.Count);
			}
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 29, 4);
				Assert.AreEqual(0, lines.Count);
			}

			await tailService.AppendAsync(_logId, 30, Encoding.UTF8.GetBytes("line30\n"));
			await tailService.AppendAsync(_logId, 32, Encoding.UTF8.GetBytes("line32\nline33\nline34\nline35\n"));

			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 29, 4);
				Assert.AreEqual(0, lines.Count);
			}
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 30, 4);
				Assert.AreEqual(1, lines.Count);
				Assert.AreEqual("line30", lines[0].ToString());
			}
			{
				List<Utf8String> lines = await tailService.ReadAsync(_logId, 32, 4);
				Assert.AreEqual(0, lines.Count);
			}
		}

		[TestMethod]
		public async Task AsyncEnableAsync()
		{
			LogTailService tailService = ServiceProvider.GetRequiredService<LogTailService>();
			await tailService.StartAsync(CancellationToken.None);

			LogTailService tailService2 = new LogTailService(ServiceProvider.GetRequiredService<IRedisService>(), ServiceProvider.GetRequiredService<IClock>(), 4, ServiceProvider.GetRequiredService<IServerInfo>(), ServiceProvider.GetRequiredService<ILogger<LogTailService>>());

			Task<int> task = tailService.WaitForTailNextAsync(_logId, CancellationToken.None);
			Assert.IsTrue(!task.IsCompleted);

			await tailService2.EnableTailingAsync(_logId, 30);

			await Task.WhenAny(task, Task.Delay(1000));
			Assert.IsTrue(task.IsCompletedSuccessfully);

			Assert.AreEqual(30, await task);
		}
	}
}
