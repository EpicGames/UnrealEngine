// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Agents
{
	/// <summary>
	/// ACL actions which apply to agents
	/// </summary>
	public static class AgentAclAction
	{
		/// <summary>
		/// Ability to create an agent. This may be done explicitly, or granted to agents to allow them to self-register.
		/// </summary>
		public static AclAction CreateAgent { get; } = new AclAction("CreateAgent");
		
		/// <summary>
		/// Ability to create a workstation agent. Granted to users with hardware to offer.
		/// </summary>
		public static AclAction CreateWorkstationAgent { get; } = new AclAction("CreateWorkstationAgent");

		/// <summary>
		/// Update an agent's name, pools, etc...
		/// </summary>
		public static AclAction UpdateAgent { get; } = new AclAction("UpdateAgent");

		/// <summary>
		/// Soft-delete an agent
		/// </summary>
		public static AclAction DeleteAgent { get; } = new AclAction("DeleteAgent");

		/// <summary>
		/// View an agent
		/// </summary>
		public static AclAction ViewAgent { get; } = new AclAction("ViewAgent");

		/// <summary>
		/// List the available agents
		/// </summary>
		public static AclAction ListAgents { get; } = new AclAction("ListAgents");
	}
}
