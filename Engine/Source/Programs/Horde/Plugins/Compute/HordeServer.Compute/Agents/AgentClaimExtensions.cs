// Copyright Epic Games, Inc. All Rights Reserved.

using System.Security.Claims;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeServer.Utilities;

namespace HordeServer.Agents
{
	/// <summary>
	/// Extension methods for getting agent-related claims from principals
	/// </summary>
	public static class ClaimExtensions
	{
		/// <summary>
		/// Test whether a principal is the given agent
		/// </summary>
		public static bool HasAgentClaim(this ClaimsPrincipal user, AgentId agentId)
		{
			return user.HasClaim(HordeClaimTypes.Agent, agentId.ToString());
		}

		/// <summary>
		/// Test whether a principal is the given lease
		/// </summary>
		public static bool HasLeaseClaim(this ClaimsPrincipal user, LeaseId leaseId)
		{
			return user.HasClaim(HordeClaimTypes.Lease, leaseId.ToString());
		}

		/// <summary>
		/// Test whether a principal is the given session
		/// </summary>
		public static bool HasSessionClaim(this ClaimsPrincipal user, SessionId sessionId)
		{
			return user.HasClaim(HordeClaimTypes.AgentSessionId, sessionId.ToString());
		}

		/// <summary>
		/// Attempts to get the lease id from the given principal
		/// </summary>
		public static LeaseId? GetLeaseClaim(this ClaimsPrincipal user)
		{
			Claim? claim = user.FindFirst(HordeClaimTypes.Lease);
			if (claim == null || !LeaseId.TryParse(claim.Value, out LeaseId leaseIdValue))
			{
				return null;
			}
			else
			{
				return leaseIdValue;
			}
		}

		/// <summary>
		/// Attempts to get the session id from the given principal
		/// </summary>
		public static SessionId? GetSessionClaim(this ClaimsPrincipal user)
		{
			Claim? claim = user.FindFirst(HordeClaimTypes.AgentSessionId);
			if (claim == null || !SessionId.TryParse(claim.Value, out SessionId sessionIdValue))
			{
				return null;
			}
			else
			{
				return sessionIdValue;
			}
		}
	}
}
