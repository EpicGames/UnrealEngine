// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Concurrent;
using System.Collections.ObjectModel;
using System.Diagnostics.CodeAnalysis;
using System.Linq.Expressions;
using System.Runtime.CompilerServices;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using Google.Protobuf;
using Google.Protobuf.Collections;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Sessions;
using HordeServer.Auditing;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Agents
{
	/// <summary>
	/// Collection of agent documents
	/// </summary>
	public sealed class AgentCollection : IAgentCollection, IHostedService
	{
		class Agent : IAgent
		{
			static IReadOnlyList<string> DefaultProperties { get; } = new List<string>();
			static IReadOnlyDictionary<string, int> DefaultResources { get; } = new Dictionary<string, int>();

			public AgentDocument Document => _document;
			public RpcSession? Session => _session;

			readonly AgentCollection _collection;
			readonly AgentDocument _document;
			readonly RpcSession? _session;

			public AgentId Id => _document.Id;
			SessionId? IAgent.SessionId => _session?.SessionId;
			DateTime? IAgent.SessionExpiresAt => _session?.ExpiryTime;
			AgentStatus IAgent.Status => (AgentStatus?)_session?.Status ?? AgentStatus.Stopped;
			DateTime? IAgent.LastOnlineTime => _document.LastOnlineTime;
			bool IAgent.Enabled => _document.Enabled;
			bool IAgent.Ephemeral => _document.Ephemeral;
			AgentMode IAgent.Mode => _document.Mode;
			bool IAgent.Deleted => _document.Deleted;
			string? IAgent.Version => _document.Version;
			string? IAgent.Comment => _document.Comment;
			IReadOnlyList<string> IAgent.ServerDefinedProperties => _document.ServerDefinedProperties ?? DefaultProperties;
			IReadOnlyList<string> IAgent.Properties => _document.Properties ?? DefaultProperties;
			IReadOnlyDictionary<string, int> IAgent.Resources => _document.Resources ?? DefaultResources;
			string? IAgent.LastUpgradeVersion => _document.LastUpgradeVersion;
			DateTime? IAgent.LastUpgradeTime => _document.LastUpgradeTime;
			int? IAgent.UpgradeAttemptCount => _document.UpgradeAttemptCount;
			IReadOnlyList<PoolId> IAgent.Pools => _document.Pools;
			IReadOnlyList<PoolId> IAgent.DynamicPools => _document.DynamicPools;
			IReadOnlyList<PoolId> IAgent.ExplicitPools => _document.ExplicitPools;
			bool IAgent.RequestConform => _document.RequestConform;
			bool IAgent.RequestFullConform => _document.RequestFullConform;
			bool IAgent.RequestRestart => _document.RequestRestart;
			bool IAgent.RequestShutdown => _document.RequestShutdown;
			bool IAgent.RequestForceRestart => _document.RequestForceRestart;
			string? IAgent.LastShutdownReason => _document.LastShutdownReason;
			IReadOnlyList<AgentWorkspaceInfo> IAgent.Workspaces => _document.Workspaces;
			DateTime IAgent.LastConformTime => _document.LastConformTime;
			int? IAgent.ConformAttemptCount => _document.ConformAttemptCount;
			IReadOnlyList<IAgentLease> IAgent.Leases => (IReadOnlyList<IAgentLease>?)_session?.Leases ?? Array.Empty<IAgentLease>();
			string IAgent.EnrollmentKey => _document.EnrollmentKey;
			DateTime IAgent.UpdateTime => _document.UpdateTime;
			uint IAgent.UpdateIndex => _document.UpdateIndex;

			public Agent(AgentCollection collection, AgentDocument document, RpcSession? session)
			{
				_collection = collection;
				_document = document;
				_session = session;
			}

			/// <inheritdoc/>
			public async Task<ISession?> GetSessionAsync(SessionId sessionId, CancellationToken cancellationToken = default)
				=> await _collection.GetSessionAsync(_document.Id, sessionId, cancellationToken);

			/// <inheritdoc/>
			public async Task<IReadOnlyList<ISession>> FindSessionsAsync(DateTime? startTime, DateTime? finishTime, int index, int count, CancellationToken cancellationToken = default)
				=> await _collection.FindSessionsAsync(_document.Id, startTime, finishTime, index, count, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryCreateLeaseAsync(CreateLeaseOptions options, CancellationToken cancellationToken = default)
				=> await _collection.TryCreateLeaseAsync(this, options, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryCancelLeaseAsync(int leaseIdx, LeaseOutcomeReason? reason = null, CancellationToken cancellationToken = default)
				=> await _collection.TryCancelLeaseAsync(this, leaseIdx, reason, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryDeleteAsync(CancellationToken cancellationToken = default)
				=> await _collection.TryDeleteAsync(this, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryResetAsync(bool ephemeral, string enrollmentKey, CancellationToken cancellationToken = default)
				=> await _collection.TryResetAsync(this, ephemeral, enrollmentKey, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryCreateSessionAsync(CreateSessionOptions options, CancellationToken cancellationToken = default)
				=> await _collection.TryCreateSessionAsync(this, options, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryTerminateSessionAsync(CancellationToken cancellationToken = default)
				=> await _collection.TryTerminateSessionAsync(this, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryUpdateSessionAsync(UpdateSessionOptions options, CancellationToken cancellationToken = default)
				=> await _collection.TryUpdateSessionAsync(this, options, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryUpdateAsync(UpdateAgentOptions options, CancellationToken cancellationToken = default)
				=> await _collection.TryUpdateSettingsAsync(this, options, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> TryUpdateWorkspacesAsync(List<AgentWorkspaceInfo> workspaces, bool requestConform, CancellationToken cancellationToken = default)
				=> await _collection.TryUpdateWorkspacesAsync(this, workspaces, requestConform, cancellationToken);

			/// <inheritdoc/>
			public async Task<IAgent?> WaitForUpdateAsync(CancellationToken cancellationToken = default)
				=> await _collection.WaitForSessionUpdateAsync(this, cancellationToken);
		}

		/// <summary>
		/// Concrete implementation of an agent document
		/// </summary>
		class AgentDocument
		{
			[BsonRequired, BsonId]
			public AgentId Id { get; set; }

			public SessionId? SessionId { get; set; }

			[BsonIgnoreIfNull]
			[Obsolete("Session state now stored in Redis")]
			public AgentStatus? Status { get; set; }

			public DateTime? LastOnlineTime { get; set; }

			[BsonRequired]
			public bool Enabled { get; set; } = true;

			public bool Ephemeral { get; set; }
			
			public AgentMode Mode { get; set; } = AgentMode.Dedicated;

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool Deleted { get; set; }

			[BsonElement("Version2")]
			public string? Version { get; set; }
			
			public List<string>? ServerDefinedProperties { get; set; }
			public List<string>? Properties { get; set; }
			public Dictionary<string, int>? Resources { get; set; }

			[BsonIgnoreIfNull]
			public string? LastUpgradeVersion { get; set; }

			[BsonIgnoreIfNull]
			public DateTime? LastUpgradeTime { get; set; }

			[BsonIgnoreIfNull]
			public int? UpgradeAttemptCount { get; set; }

			public List<PoolId> Pools { get; set; } = new List<PoolId>();
			public List<PoolId> DynamicPools { get; set; } = new List<PoolId>();
			public List<PoolId> ExplicitPools { get; set; } = new List<PoolId>();

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool RequestConform { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool RequestFullConform { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool RequestRestart { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool RequestShutdown { get; set; }

			[BsonIgnoreIfDefault, BsonDefaultValue(false)]
			public bool RequestForceRestart { get; set; }

			[BsonIgnoreIfNull]
			public string? LastShutdownReason { get; set; }

			public List<AgentWorkspaceInfo> Workspaces { get; set; } = new List<AgentWorkspaceInfo>();
			public DateTime LastConformTime { get; set; }

			[BsonIgnoreIfNull]
			public int? ConformAttemptCount { get; set; }

			[BsonIgnoreIfNull]
			[Obsolete("Session state now stored in Redis")]
			public List<AgentLease>? Leases { get; set; }

			public DateTime UpdateTime { get; set; }
			public uint UpdateIndex { get; set; }
			public string EnrollmentKey { get; set; } = String.Empty;
			public string? Comment { get; set; }

			[BsonElement("dv")]
			public int DocumentVersion { get; set; } = 0;

			[BsonConstructor]
			private AgentDocument()
			{
			}

			public AgentDocument(AgentId id, bool ephemeral, string enrollmentKey)
			{
				Id = id;
				Ephemeral = ephemeral;
				EnrollmentKey = enrollmentKey;
			}
		}

		/// <summary>
		/// Concrete implementation of ISession
		/// </summary>
		class SessionDocument : ISession
		{
			[BsonRequired, BsonId]
			public SessionId Id { get; set; }

			[BsonRequired]
			public AgentId AgentId { get; set; }

			public DateTime StartTime { get; set; }
			public DateTime? FinishTime { get; set; }
			public string Version { get; set; } = String.Empty;

			[BsonConstructor]
			private SessionDocument()
			{
			}

			public SessionDocument(SessionId id, AgentId agentId, DateTime startTime, string? version)
			{
				Id = id;
				AgentId = agentId;
				StartTime = startTime;
				if (version != null)
				{
					Version = version;
				}
			}
		}

		readonly IMongoCollection<AgentDocument> _agentCollection;
		readonly IMongoCollection<SessionDocument> _sessionCollection;
		readonly ILeaseCollection _leaseCollection;
		readonly IAgentScheduler _scheduler;
		readonly IAuditLog<AgentId> _auditLog;
		readonly IClock _clock;
		readonly ITicker _sharedTicker;
		readonly ConcurrentDictionary<SessionId, TaskCompletionSource> _sessionIdToTcs = new ConcurrentDictionary<SessionId, TaskCompletionSource>();
		readonly Tracer _tracer;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentCollection(IMongoService mongoService, ILeaseCollection leaseCollection, IAgentScheduler scheduler, IClock clock, IAuditLog<AgentId> auditLog, Tracer tracer, ILogger<AgentCollection> logger)
		{
			List<MongoIndex<AgentDocument>> agentIndexes = new List<MongoIndex<AgentDocument>>();
			agentIndexes.Add(keys => keys.Ascending(x => x.Deleted).Ascending(x => x.Id).Ascending(x => x.Pools));
			_agentCollection = mongoService.GetCollection<AgentDocument>("Agents", agentIndexes);

			List<MongoIndex<SessionDocument>> sessionIndexes = new List<MongoIndex<SessionDocument>>();
			sessionIndexes.Add(keys => keys.Ascending(x => x.AgentId).Ascending(x => x.StartTime).Ascending(x => x.FinishTime));
			sessionIndexes.Add(keys => keys.Ascending(x => x.FinishTime));
			_sessionCollection = mongoService.GetCollection<SessionDocument>("Sessions", sessionIndexes);

			_leaseCollection = leaseCollection;
			_scheduler = scheduler;
			_clock = clock;
			_sharedTicker = clock.AddSharedTicker($"{nameof(AgentCollection)}.{nameof(TickSharedAsync)}", TimeSpan.FromSeconds(30.0), TickSharedAsync, logger);
			_auditLog = auditLog;
			_tracer = tracer;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _sharedTicker.DisposeAsync();
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_scheduler.SessionUpdated += OnSessionUpdate;
			await _sharedTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _sharedTicker.StopAsync();
			_scheduler.SessionUpdated -= OnSessionUpdate;
		}

		internal async ValueTask TickSharedAsync(CancellationToken stoppingToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentCollection)}.{nameof(TickSharedAsync)}");

			await TerminateExpiredSessionsAsync(stoppingToken);
			await DeleteExpiredEphemeralAgentsAsync(stoppingToken);
		}

		internal async Task TerminateExpiredSessionsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentCollection)}.{nameof(TerminateExpiredSessionsAsync)}");

			int c = 0;
			await foreach (RpcSession session in _scheduler.FindExpiredSessionsAsync(cancellationToken))
			{
				AgentDocument? document = await GetDocumentAsync(session.AgentId, cancellationToken);
				if (document != null)
				{
					Agent? agent = await CreateAgentObjectAsync(document, session, cancellationToken);
					if (agent != null && agent.Session != null && agent.Session.UpdateTicks == session.UpdateTicks)
					{
						_logger.LogDebug("Terminating session {SessionId} for agent {Agent} (expiry time: {Time})", session.SessionId, session.AgentId, agent.Session?.ExpiryTime);
						await TryUpdateSessionAsync(agent, new UpdateSessionOptions { Status = AgentStatus.Stopped, DefaultLeaseOutcomeReason = LeaseOutcomeReason.ServerInitiated | LeaseOutcomeReason.AgentSessionTerminated }, cancellationToken);
						c++;
					}
				}
			}
			span.SetAttribute("NumAgentsTerminated", c);
		}
		
		internal async Task DeleteExpiredEphemeralAgentsAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentCollection)}.{nameof(DeleteExpiredEphemeralAgentsAsync)}");
			int numDeleted = 0;
			TimeSpan maxAge = TimeSpan.FromHours(1);
			
			FilterDefinition<AgentDocument> filter = Builders<AgentDocument>.Filter.Or(
				Builders<AgentDocument>.Filter.Eq(x => x.Deleted, true),
				Builders<AgentDocument>.Filter.Lt(x => x.LastOnlineTime, _clock.UtcNow - maxAge)
			);
			
			List<AgentDocument> deletedDocuments = await _agentCollection.Find(filter).ToListAsync(cancellationToken);
			deletedDocuments = await PostLoadAsync(deletedDocuments, cancellationToken);

			foreach (AgentDocument agent in deletedDocuments)
			{
				cancellationToken.ThrowIfCancellationRequested();
				if (agent is { Ephemeral: true, LastOnlineTime: not null } && _clock.UtcNow > agent.LastOnlineTime.Value + maxAge)
				{
					_logger.LogDebug("Deleting ephemeral agent {Agent}", agent.Id);

					DeleteResult result = await _agentCollection.DeleteOneAsync(x => x.Id == agent.Id && x.UpdateIndex == agent.UpdateIndex, cancellationToken);
					if (result.DeletedCount > 0)
					{
						await _sessionCollection.DeleteManyAsync(x => x.AgentId == agent.Id, CancellationToken.None);
					}

					numDeleted++;
				}
			}

			span.SetAttribute("NumAgentsDeleted", numDeleted);
		}

#pragma warning disable CS0618
		async ValueTask<AgentDocument?> PostLoadAsync(AgentDocument? document, CancellationToken cancellationToken)
		{
			while (document != null)
			{
				AgentDocument? newDocument = null;
				if (document.DocumentVersion == 0)
				{
					UpdateDefinition<AgentDocument> updateDefinition = Builders<AgentDocument>.Update
						.Set(x => x.Pools, CreatePoolsList(document.Pools, document.DynamicPools, document.Properties))
						.Set(x => x.DynamicPools, CreatePoolsList(document.DynamicPools))
						.Set(x => x.ExplicitPools, CreatePoolsList(document.Pools))
						.Set(x => x.DocumentVersion, 1);

					newDocument = await TryUpdateAsync(document, updateDefinition, cancellationToken);
				}
				else if (document.DocumentVersion == 1)
				{
					if (await MoveLeasesToSchedulerAsync(document, cancellationToken))
					{
						UpdateDefinition<AgentDocument> updateDefinition = Builders<AgentDocument>.Update
							.Unset(x => x.Leases)
							.Set(x => x.DocumentVersion, 2);

						newDocument = await TryUpdateAsync(document, updateDefinition, cancellationToken);
					}
				}
				else
				{
					break;
				}
				document = newDocument ?? await _agentCollection.Find<AgentDocument>(x => x.Id == document.Id).FirstOrDefaultAsync(cancellationToken);
			}
			return document;
		}

		async Task<bool> MoveLeasesToSchedulerAsync(AgentDocument document, CancellationToken cancellationToken)
		{
			if (document.SessionId != null)
			{
				// Create the session with this id, or get the existing session document
				RpcSession? session = await _scheduler.TryCreateSessionAsync(document.Id, document.SessionId.Value, new RpcAgentCapabilities(), cancellationToken);
				if (session == null)
				{
					session = await _scheduler.TryGetSessionAsync(document.SessionId.Value, cancellationToken);
					if (session == null)
					{
						return false;
					}
				}

				// Update the session with the leases in the document
				RpcSession newSession = new RpcSession(session);
				if (document.Status != null)
				{
					newSession.Status = (RpcAgentStatus)document.Status.Value;
				}
				if (document.Leases != null)
				{
					foreach (AgentLease agentLease in document.Leases)
					{
						if (!newSession.Leases.Any(x => x.Id == agentLease.Id))
						{
							RpcSessionLease sessionLease = new RpcSessionLease();
							sessionLease.Id = agentLease.Id;
							sessionLease.ParentId = agentLease.ParentId;
							sessionLease.Payload = (agentLease.Payload == null) ? null : Any.Parser.ParseFrom(agentLease.Payload);
							sessionLease.State = (RpcLeaseState)agentLease.State;
							sessionLease.Resources.Add(agentLease.Resources ?? Enumerable.Empty<KeyValuePair<string, int>>());
							sessionLease.Exclusive = agentLease.Exclusive;
							session.Leases.Add(sessionLease);
						}
					}
				}

				// Apply the updates
				if (session != newSession && await _scheduler.TryUpdateSessionAsync(session, newSession, cancellationToken: cancellationToken) == null)
				{
					return false;
				}
			}
			return true;
		}

#pragma warning restore CS0618

		async ValueTask<List<AgentDocument>> PostLoadAsync(List<AgentDocument> documents, CancellationToken cancellationToken)
		{
			List<AgentDocument> newDocuments = new List<AgentDocument>();
			foreach (AgentDocument document in documents)
			{
				AgentDocument? newDocument = await PostLoadAsync(document, cancellationToken);
				if (newDocument != null)
				{
					newDocuments.Add(newDocument);
				}
			}
			return newDocuments;
		}

		ValueTask<Agent?> CreateAgentObjectAsync(AgentDocument? document, CancellationToken cancellationToken)
			=> CreateAgentObjectAsync(document, null, cancellationToken);

		async Task<List<IAgent>> CreateAgentObjectsBatchedAsync(List<AgentDocument> documents, CancellationToken cancellationToken)
		{
			// Collect all session IDs for a single batched Redis MGET
			List<SessionId> sessionIds = new (documents.Count);
			foreach (AgentDocument document in documents)
			{
				if (document.SessionId != null)
				{
					sessionIds.Add(document.SessionId.Value);
				}
			}

			// Batch fetch all sessions in one round-trip
			Dictionary<SessionId, RpcSession> sessions = await _scheduler.TryGetSessionsAsync(sessionIds, cancellationToken);

			// Create agent objects using pre-fetched sessions
			List<IAgent> agents = new (documents.Count);
			foreach (AgentDocument document in documents)
			{
				Agent? agent;
				if (document.SessionId == null)
				{
					agent = new Agent(this, document, null);
				}
				else if (sessions.TryGetValue(document.SessionId.Value, out RpcSession? session))
				{
					// Safe to bypass the reconciliation loop in CreateAgentObjectAsync; the dictionary is keyed by
					// document.SessionId so session.SessionId == document.SessionId is guaranteed.
					agent = new Agent(this, document, session);
				}
				else
				{
					// Session missing from Redis — fall back to sequential handling which clears the stale session
					agent = await CreateAgentObjectAsync(document, cancellationToken);
				}

				if (agent != null)
				{
					agents.Add(agent);
				}
			}
			return agents;
		}

		async ValueTask<Agent?> CreateAgentObjectAsync(AgentDocument? document, RpcSession? session, CancellationToken cancellationToken)
		{
			if (document == null)
			{
				return null;
			}

			AgentId agentId = document.Id;
			for (; ; )
			{
				// If the document doesn't have a session, we don't need to fetch it
				if (document.SessionId == null && session == null)
				{
					return new Agent(this, document, null);
				}
				if (document.SessionId != null && session != null && session.SessionId == document.SessionId.Value)
				{
					return new Agent(this, document, session);
				}

				// Reconcile the differences
				for (; ; )
				{
					// Fetch the document again if update failed below.
					// On first iteration, document is already set.
					if (document == null)
					{
						document = await GetDocumentAsync(agentId, cancellationToken);
						if (document == null)
						{
							return null;
						}
					}

					SessionId? sessionId = document.SessionId;
					if (sessionId == null)
					{
						session = null;
						break;
					}

					// Fetch the session
					session = await _scheduler.TryGetSessionAsync(sessionId.Value, cancellationToken);
					if (session != null)
					{
						break;
					}

					// If there's no matching session, try to remove it from the document. We can expect the document to be authoritative here, since we always create the session before
					// updating the document; if it no longer exists, the document is invalid.
					document = await TryUpdateAsync(document, Builders<AgentDocument>.Update.Unset(x => x.SessionId).Set(x => x.LastOnlineTime, _clock.UtcNow), cancellationToken);
					if (document != null)
					{
						_logger.LogInformation("Cleared session state from agent {AgentId}; no matching document for {SessionId} in Redis.", document.Id, sessionId.Value);
						break;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<IAgent> AddAsync(CreateAgentOptions options, CancellationToken cancellationToken)
		{
			AgentDocument document = new (options.Id, options.Ephemeral, options.EnrollmentKey)
			{
				DocumentVersion = 2,
				ServerDefinedProperties = options.ServerDefinedProperties?.ToList(),
				LastOnlineTime = _clock.UtcNow,
			};
			
			await _agentCollection.InsertOneAsync(document, null, cancellationToken);
			return new Agent(this, document, null);
		}

		/// <inheritdoc/>
		async Task<Agent?> TryResetAsync(Agent agent, bool ephemeral, string enrollmentKey, CancellationToken cancellationToken = default)
		{
			// Don't allow resetting while a session is active
			if (agent.Document.SessionId != null)
			{
				return null;
			}

			// Try to update the document
			UpdateDefinition<AgentDocument> update = Builders<AgentDocument>.Update
				.Set(x => x.Ephemeral, ephemeral)
				.Set(x => x.EnrollmentKey, enrollmentKey)
				.Unset(x => x.Deleted);

			AgentDocument? newDocument = await TryUpdateAsync(agent.Document, update, cancellationToken);
			if (newDocument == null)
			{
				return null;
			}

			// Create the agent instance
			return new Agent(this, newDocument, null);
		}

		/// <inheritdoc/>
		async Task<Agent?> TryDeleteAsync(Agent agent, CancellationToken cancellationToken)
		{
			if (agent.Document.Deleted)
			{
				return null;
			}

			// Terminate the current session
			if (agent.Session != null)
			{
				Agent? newAgent = await TryTerminateSessionAsync(agent, cancellationToken);
				if (newAgent == null)
				{
					return null;
				}
				agent = newAgent;
			}

			// Update the document
			UpdateDefinition<AgentDocument> update = Builders<AgentDocument>.Update
				.Set(x => x.Deleted, true)
				.Set(x => x.EnrollmentKey, "");

			AgentDocument? newDocument = await TryUpdateAsync(agent.Document, update, cancellationToken);
			return await CreateAgentObjectAsync(newDocument, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IAgent?> GetAsync(AgentId agentId, CancellationToken cancellationToken)
		{
			AgentDocument? document = await GetDocumentAsync(agentId, cancellationToken);
			return await CreateAgentObjectAsync(document, cancellationToken);
		}

		async Task<AgentDocument?> GetDocumentAsync(AgentId agentId, CancellationToken cancellationToken)
		{
			AgentDocument? document = await _agentCollection.Find<AgentDocument>(x => x.Id == agentId).FirstOrDefaultAsync(cancellationToken);
			document = await PostLoadAsync(document, cancellationToken);
			return document;
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<IAgent>> GetManyAsync(List<AgentId> agentIds, CancellationToken cancellationToken)
		{
			List<AgentDocument> documents = await _agentCollection.Find(p => agentIds.Contains(p.Id)).ToListAsync(cancellationToken);
			documents = await PostLoadAsync(documents, cancellationToken);
			return await CreateAgentObjectsBatchedAsync(documents, cancellationToken);
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<IAgent> FindAsync(PoolId? poolId, DateTime? modifiedAfter, string? property, AgentStatus? status, bool? enabled, bool includeDeleted, bool consistentRead, [EnumeratorCancellation] CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<AgentDocument> filterBuilder = new FilterDefinitionBuilder<AgentDocument>();

			FilterDefinition<AgentDocument> filter = filterBuilder.Empty;
			if (!includeDeleted)
			{
				filter &= filterBuilder.Ne(x => x.Deleted, true);
			}

			if (poolId != null)
			{
				filter &= filterBuilder.AnyEq(x => x.Pools, poolId.Value);
			}

			if (modifiedAfter != null)
			{
				filter &= filterBuilder.Gt(x => x.UpdateTime, modifiedAfter.Value);
			}

			if (property != null)
			{
				filter &= filterBuilder.AnyEq(x => x.Properties, property);
			}

			if (enabled != null)
			{
				filter &= filterBuilder.Eq(x => x.Enabled, enabled.Value);
			}

			IMongoCollection<AgentDocument> collection = consistentRead ? _agentCollection : _agentCollection.WithReadPreference(ReadPreference.SecondaryPreferred);

			// Use a large batch size to minimize getMore commands when reading from secondaries.
			// The agents collection is updated frequently, making multi-batch cursor reads error-prone.
			FindOptions findOptions = new() { BatchSize = 10000 };
			List<AgentDocument> allDocuments = await collection.Find(filter, findOptions).ToListAsync(cancellationToken);
			allDocuments = await PostLoadAsync(allDocuments, cancellationToken);
			foreach (IAgent agent in await CreateAgentObjectsBatchedAsync(allDocuments, cancellationToken))
			{
				if (status == null || agent.Status == status.Value)
				{
					yield return agent;
				}
			}
		}

		async Task<SessionDocument?> GetSessionAsync(AgentId agentId, SessionId sessionId, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<SessionDocument> filterBuilder = Builders<SessionDocument>.Filter;

			FilterDefinition<SessionDocument> filter = filterBuilder.Eq(x => x.Id, sessionId) & filterBuilder.Eq(x => x.AgentId, agentId);
			return await _sessionCollection.Find(filter).FirstOrDefaultAsync(cancellationToken);
		}

		async Task<List<SessionDocument>> FindSessionsAsync(AgentId agentId, DateTime? startTime, DateTime? finishTime, int index, int count, CancellationToken cancellationToken = default)
		{
			FilterDefinitionBuilder<SessionDocument> filterBuilder = Builders<SessionDocument>.Filter;

			FilterDefinition<SessionDocument> filter = filterBuilder.Eq(x => x.AgentId, agentId);
			if (startTime != null)
			{
				filter &= filterBuilder.Gte(x => x.StartTime, startTime.Value);
			}
			if (finishTime != null)
			{
				filter &= filterBuilder.Or(filterBuilder.Eq(x => x.FinishTime, null), filterBuilder.Lte(x => x.FinishTime, finishTime.Value));
			}

			return await _sessionCollection.Find(filter).SortByDescending(x => x.StartTime).Skip(index).Limit(count).ToListAsync(cancellationToken);
		}

		/// <summary>
		/// Update a single document
		/// </summary>
		/// <param name="current">The document to update</param>
		/// <param name="update">The update definition</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The updated agent document or null if update failed</returns>
		private async Task<AgentDocument?> TryUpdateAsync(AgentDocument current, UpdateDefinition<AgentDocument> update, CancellationToken cancellationToken)
		{
			uint prevUpdateIndex = current.UpdateIndex++;
			current.UpdateTime = DateTime.UtcNow;

			Expression<Func<AgentDocument, bool>> filter = x => x.Id == current.Id && x.UpdateIndex == prevUpdateIndex;
			UpdateDefinition<AgentDocument> updateWithIndex = update.Set(x => x.UpdateIndex, current.UpdateIndex).Set(x => x.UpdateTime, current.UpdateTime);

			return await _agentCollection.FindOneAndUpdateAsync<AgentDocument>(filter, updateWithIndex, new FindOneAndUpdateOptions<AgentDocument, AgentDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken);
		}

		async Task<Agent?> TryUpdateSettingsAsync(Agent agent, UpdateAgentOptions options, CancellationToken cancellationToken)
		{
			// Update the database
			UpdateDefinitionBuilder<AgentDocument> updateBuilder = new UpdateDefinitionBuilder<AgentDocument>();

			List<UpdateDefinition<AgentDocument>> updates = new List<UpdateDefinition<AgentDocument>>();
			if (options.ExplicitPools != null)
			{
				List<PoolId> pools = CreatePoolsList(agent.Document.DynamicPools, options.ExplicitPools, agent.Document.Properties);
				updates.Add(updateBuilder.Set(x => x.Pools, pools));

				List<PoolId> explicitPools = CreatePoolsList(options.ExplicitPools).ToList();
				updates.Add(updateBuilder.Set(x => x.ExplicitPools, explicitPools));
			}
			if (options.Enabled != null)
			{
				updates.Add(updateBuilder.Set(x => x.Enabled, options.Enabled.Value));
			}
			if (options.RequestConform != null)
			{
				updates.Add(updateBuilder.Set(x => x.RequestConform, options.RequestConform.Value));
				updates.Add(updateBuilder.Unset(x => x.ConformAttemptCount));
			}
			if (options.RequestFullConform != null)
			{
				updates.Add(updateBuilder.Set(x => x.RequestFullConform, options.RequestFullConform.Value));
				updates.Add(updateBuilder.Unset(x => x.ConformAttemptCount));
			}
			if (options.RequestRestart != null)
			{
				if (options.RequestRestart.Value)
				{
					updates.Add(updateBuilder.Set(x => x.RequestRestart, true));
				}
				else
				{
					updates.Add(updateBuilder.Unset(x => x.RequestRestart));
				}
			}
			if (options.RequestShutdown != null)
			{
				if (options.RequestShutdown.Value)
				{
					updates.Add(updateBuilder.Set(x => x.RequestShutdown, true));
				}
				else
				{
					updates.Add(updateBuilder.Unset(x => x.RequestShutdown));
				}
			}
			if (options.RequestForceRestart != null)
			{
				if (options.RequestForceRestart.Value)
				{
					updates.Add(updateBuilder.Set(x => x.RequestForceRestart, true));
				}
				else
				{
					updates.Add(updateBuilder.Unset(x => x.RequestForceRestart));
				}
			}
			if (options.ShutdownReason != null)
			{
				updates.Add(updateBuilder.Set(x => x.LastShutdownReason, options.ShutdownReason));
			}
			if (options.Comment != null)
			{
				updates.Add(updateBuilder.Set(x => x.Comment, options.Comment));
			}

			// Apply the update
			AgentDocument? newDocument = await TryUpdateAsync(agent.Document, updateBuilder.Combine(updates), cancellationToken);
			return await CreateAgentObjectAsync(newDocument, agent.Session, cancellationToken);
		}

		async Task<Agent?> TryUpdateSessionAsync(Agent agent, UpdateSessionOptions options, CancellationToken cancellationToken)
		{
			AgentId agentId = agent.Document.Id;
			if (agent.Session == null)
			{
				_logger.LogWarning("Agent {AgentId} does not have an active session; cannot update", agent.Id);
				return null;
			}

			// Try to update the pools and capabilities first
			AgentDocument? newDocument = agent.Document;
			{
				UpdateDefinitionBuilder<AgentDocument> updateBuilder = Builders<AgentDocument>.Update;
				List<UpdateDefinition<AgentDocument>> updates = new List<UpdateDefinition<AgentDocument>>();

				List<string> newProperties = newDocument.Properties ?? [];
				Dictionary<string, int>? newResources = null;
				if (options.Capabilities != null)
				{
					newProperties = [.. options.Capabilities.Properties];
					newResources = options.Capabilities.Resources.ToDictionary();
				}

				if (options.CapabilityOverrides != null)
				{
					if (options.CapabilityOverrides.Properties.Count > 0)
					{
						newProperties = MergeOverrideProperties(agentId, newProperties, options.CapabilityOverrides.Properties);
					}
					if (options.CapabilityOverrides.Resources.Count > 0)
					{
						newResources = MergeOverrideResources(agentId, newResources ?? newDocument.Resources, options.CapabilityOverrides.Resources);
					}
				}

				if (options.DynamicPools != null)
				{
					List<PoolId> dynamicPools = CreatePoolsList(options.DynamicPools).ToList();
					if (!Enumerable.SequenceEqual(dynamicPools, newDocument.DynamicPools))
					{
						updates.Add(updateBuilder.Set(x => x.DynamicPools, dynamicPools));
					}
				}

				List<PoolId> pools = CreatePoolsList(options.DynamicPools ?? newDocument.DynamicPools, newDocument.ExplicitPools, newProperties);
				if (!Enumerable.SequenceEqual(pools, newDocument.Pools))
				{
					updates.Add(updateBuilder.Set(x => x.Pools, pools));
				}

				SetPoolProperties(newProperties, pools);

				if (newResources != null && !ResourcesEqual(newResources, newDocument.Resources))
				{
					updates.Add(updateBuilder.Set(x => x.Resources, newResources));
				}

				if (!Enumerable.SequenceEqual(newDocument.Properties ?? Enumerable.Empty<string>(), newProperties, StringComparer.Ordinal))
				{
					updates.Add(updateBuilder.Set(x => x.Properties, newProperties));
				}

				// Apply the updates
				if (updates.Count > 0)
				{
					newDocument = await TryUpdateAsync(newDocument, updateBuilder.Combine(updates), cancellationToken);
					if (newDocument == null)
					{
						_logger.LogDebug("Update of agent document {AgentId} did not succeed; will retry", agent.Document.Id);
						return null;
					}
				}
			}

			// Apply the updates to the session
			RpcSession? newSession = new RpcSession(agent.Session);
			if (options.Status != null)
			{
				newSession.Status = (RpcAgentStatus)options.Status;
			}
			if (options.Leases != null)
			{
				Dictionary<LeaseId, RpcLease> idToRemoteLease = options.Leases.ToDictionary(x => x.Id, x => x);

				// Remove any completed leases from the agent
				RepeatedField<RpcSessionLease> leases = newSession.Leases;
				for (int idx = 0; idx < leases.Count; idx++)
				{
					RpcSessionLease lease = leases[idx];
					if (!idToRemoteLease.TryGetValue(lease.Id, out RpcLease? remoteLease))
					{
						if (lease.State != RpcLeaseState.Pending)
						{
							leases.RemoveAt(idx--);
						}
					}
					else
					{
						if (remoteLease.State == RpcLeaseState.Cancelled || remoteLease.State == RpcLeaseState.Completed)
						{
							leases.RemoveAt(idx--);
						}
						else if (newSession.Status == RpcAgentStatus.Stopping || newSession.Status == RpcAgentStatus.Busy)
						{
							lease.State = RpcLeaseState.Cancelled;
						}
						else if (remoteLease.State == RpcLeaseState.Active && lease.State == RpcLeaseState.Pending)
						{
							lease.State = RpcLeaseState.Active;
						}
					}
				}
			}

			// Apply the updates in redis
			newSession = await _scheduler.TryUpdateSessionAsync(agent.Session, newSession, options.Capabilities, cancellationToken);
			if (newSession == null)
			{
				_logger.LogDebug("Update of agent {AgentId} session {SessionId} did not succeed; will retry", agent.Document.Id, agent.Session.SessionId);
				return null;
			}

			// If the session is stopped, remove it from the agent document
			if (agent.Session.Status != RpcAgentStatus.Stopped && newSession.Status == RpcAgentStatus.Stopped)
			{
				// Get the session expiry time. We'll use this to update all the other documents.
				DateTime expiryTime = newSession.ExpiryTime;

				// Update the agent document
				while (newDocument != null && newDocument.SessionId == newSession.SessionId)
				{
					UpdateDefinition<AgentDocument> update = new BsonDocument();
					update = update.Unset(x => x.SessionId);
					update = update.Set(x => x.LastOnlineTime, expiryTime);

					if (agent.Document.Ephemeral)
					{
						update = update.Set(x => x.Deleted, true);
					}

					newDocument = await TryUpdateAsync(agent.Document, update, cancellationToken);
					if (newDocument != null)
					{
						break;
					}

					newDocument = await GetDocumentAsync(agent.Id, cancellationToken);
				}

				// Update the session document
				ILogger agentLogger = GetLogger(agent.Id);
				agentLogger.LogInformation("Terminated session {SessionId}", newSession.SessionId);
				await _sessionCollection.UpdateOneAsync(x => x.Id == newSession.SessionId, Builders<SessionDocument>.Update.Set(x => x.FinishTime, expiryTime), cancellationToken: CancellationToken.None);
			}

			// Remove any complete leases
			if (newSession.Leases != null)
			{
				// Merges outcome reasons if lease was already finished by another process (race condition)
				async Task TryMergeOutcomeReasonsAsync(bool updated, LeaseId leaseId, LeaseOutcomeReason reason)
				{
					if (!updated && reason != LeaseOutcomeReason.None)
					{
						_logger.LogInformation("Merging outcome reasons {Reason} for already-finished lease {LeaseId}", reason, leaseId);
						await _leaseCollection.TryAddOutcomeReasonsAsync(leaseId, reason, CancellationToken.None);
					}
				}

				HashSet<LeaseId> newLeaseIds = new HashSet<LeaseId>(newSession.Leases?.Select(x => x.Id) ?? Enumerable.Empty<LeaseId>());
				foreach (RpcSessionLease oldLease in agent.Session.Leases.Where(x => !newLeaseIds.Contains(x.Id)))
				{
					RpcLease? rpcLease = options.Leases?.FirstOrDefault(x => x.Id == oldLease.Id);
					if (rpcLease is { State: RpcLeaseState.Completed })
					{
						LeaseOutcomeReason reason = rpcLease.ToNativeOutcomeReason();
						_logger.LogInformation("Setting outcome {Outcome} for completed lease {LeaseId}", oldLease.Id, rpcLease.Outcome);
						bool updated = await _leaseCollection.TrySetOutcomeAsync(oldLease.Id, _clock.UtcNow, (LeaseOutcome)rpcLease.Outcome, reason, rpcLease.Output.ToArray(), ToNativeLeaseMetrics(rpcLease.Metrics), CancellationToken.None);
						await TryMergeOutcomeReasonsAsync(updated, oldLease.Id, reason);
					}
					else
					{
						LeaseOutcomeReason reason = rpcLease?.ToNativeOutcomeReason() ?? options.DefaultLeaseOutcomeReason ?? LeaseOutcomeReason.NoReasonReported;
						_logger.LogInformation("Setting outcome {Outcome} with reason {Reason} for lease {LeaseId} ({State})", rpcLease?.Outcome, reason, oldLease.Id, rpcLease?.State);
						bool updated = await _leaseCollection.TrySetOutcomeAsync(oldLease.Id, _clock.UtcNow, LeaseOutcome.Cancelled, reason, null, ToNativeLeaseMetrics(rpcLease?.Metrics), CancellationToken.None);
						await TryMergeOutcomeReasonsAsync(updated, oldLease.Id, reason);
					}
				}
			}

			return await CreateAgentObjectAsync(newDocument, newSession, cancellationToken);
		}
		
		[SuppressMessage("Maintainability", "CA1508:Avoid dead conditional code", Justification = "Protobuf generated code may contain null fields")]
		static LeaseMetrics? ToNativeLeaseMetrics(RpcLeaseMetrics? metrics)
		{
			if (metrics == null)
			{
				return null;
			}
			
			LeaseCpuTime cpuTime = metrics.GlobalCpuTime != null
				? new LeaseCpuTime(
					TimeSpan.FromMilliseconds(metrics.GlobalCpuTime.UserMs),
					TimeSpan.FromMilliseconds(metrics.GlobalCpuTime.SystemMs),
					TimeSpan.FromMilliseconds(metrics.GlobalCpuTime.IdleMs))
				: new LeaseCpuTime(TimeSpan.Zero,  TimeSpan.Zero, TimeSpan.Zero);

			return new LeaseMetrics(
				TimeSpan.FromMilliseconds(metrics.SetupTimeMs),
				TimeSpan.FromMilliseconds(metrics.TeardownTimeMs),
				metrics.CpuCount,
				metrics.GlobalCpuUtilization,
				cpuTime);
		}

		static void SetPoolProperties(List<string> properties, List<PoolId> pools)
		{
			properties.RemoveAll(x => x.StartsWith($"{KnownPropertyNames.Pool}=", StringComparison.OrdinalIgnoreCase));
			properties.AddRange(pools.Select(x => $"{KnownPropertyNames.Pool}={x}"));
			properties.Sort(StringComparer.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Merges override properties into a property list. For each key present in both the overrides
		/// and the existing properties, existing entries with that key are replaced by the override entries.
		/// Override entries with keys not present in the existing properties are ignored.
		/// </summary>
		List<string> MergeOverrideProperties(AgentId agentId, IEnumerable<string> properties, IReadOnlyList<string> overrides)
		{
			List<string> newProperties = [.. properties];

			HashSet<string> existingKeys = new(StringComparer.OrdinalIgnoreCase);
			foreach (string prop in newProperties)
			{
				existingKeys.Add(GetPropertyKey(prop));
			}

			HashSet<string> overrideKeys = new(StringComparer.OrdinalIgnoreCase);
			foreach (string prop in overrides)
			{
				string key = GetPropertyKey(prop);
				if (existingKeys.Contains(key))
				{
					overrideKeys.Add(key);
				}
				else
				{
					_logger.LogWarning("Agent {AgentId}: capability override property {Key} ignored - key not present in existing properties", agentId, key);
				}
			}

			newProperties.RemoveAll(x => overrideKeys.Contains(GetPropertyKey(x)));
			newProperties.AddRange(overrides.Where(x => overrideKeys.Contains(GetPropertyKey(x))));
			newProperties.Sort(StringComparer.OrdinalIgnoreCase);
			return newProperties;
		}

		/// <summary>
		/// Merges override resources into an existing resource dictionary, replacing values for existing keys.
		/// Override entries with keys not present in the existing resources are ignored.
		/// </summary>
		Dictionary<string, int>? MergeOverrideResources(AgentId agentId, IReadOnlyDictionary<string, int>? resources, IReadOnlyDictionary<string, int> overrides)
		{
			if (resources == null)
			{
				return null;
			}
			Dictionary<string, int> newResources = new(resources, StringComparer.OrdinalIgnoreCase);
			foreach ((string key, int value) in overrides)
			{
				if (newResources.ContainsKey(key))
				{
					newResources[key] = value;
				}
				else
				{
					_logger.LogWarning("Agent {AgentId}: capability override resource {Key} ignored - key not present in existing resources", agentId, key);
				}
			}
			return newResources;
		}

		/// <summary>
		/// Extracts the key portion from a "KEY=VALUE" property string.
		/// Returns the full string if no '=' is found.
		/// </summary>
		static string GetPropertyKey(string property)
		{
			int eqIdx = property.IndexOf('=', StringComparison.Ordinal);
			return eqIdx >= 0 ? property[..eqIdx] : property;
		}

		static List<PoolId> CreatePoolsList(IEnumerable<PoolId> pools)
			=> pools.Distinct().OrderBy(x => x.Id.Text).ToList();

		static List<PoolId> CreatePoolsList(IEnumerable<PoolId> dynamicPools, IEnumerable<PoolId> explicitPools, IEnumerable<string>? properties)
		{
			List<PoolId> pools = new List<PoolId>();
			pools.AddRange(dynamicPools);
			pools.AddRange(explicitPools);

			if (properties != null)
			{
				foreach (string property in properties)
				{
					const string Key = KnownPropertyNames.RequestedPools + "=";
					if (property.StartsWith(Key, StringComparison.Ordinal))
					{
						try
						{
							pools.AddRange(property[Key.Length..].Split(",").Select(x => new PoolId(x)));
						}
						catch
						{
							// Ignored
						}
					}
				}
			}

			return CreatePoolsList(pools);
		}

		static bool ResourcesEqual(IReadOnlyDictionary<string, int>? dictA, IReadOnlyDictionary<string, int>? dictB)
		{
			dictA ??= ReadOnlyDictionary<string, int>.Empty;
			dictB ??= ReadOnlyDictionary<string, int>.Empty;

			if (dictA.Count != dictB.Count)
			{
				return false;
			}

			foreach (KeyValuePair<string, int> pair in dictA)
			{
				int value;
				if (!dictB.TryGetValue(pair.Key, out value) || value != pair.Value)
				{
					return false;
				}
			}

			return true;
		}

		/// <inheritdoc/>
		async Task<Agent?> TryUpdateWorkspacesAsync(Agent agent, List<AgentWorkspaceInfo> workspaces, bool requestConform, CancellationToken cancellationToken)
		{
			DateTime lastConformTime = DateTime.UtcNow;

			// Set the new workspaces
			UpdateDefinition<AgentDocument> update = Builders<AgentDocument>.Update.Set(x => x.Workspaces, workspaces);
			update = update.Set(x => x.LastConformTime, lastConformTime);
			update = update.Unset(x => x.ConformAttemptCount);
			if (!requestConform)
			{
				update = update.Unset(x => x.RequestConform);
				update = update.Unset(x => x.RequestFullConform);
			}

			// Update the agent
			AgentDocument? newDocument = await TryUpdateAsync(agent.Document, update, cancellationToken);
			return await CreateAgentObjectAsync(newDocument, agent.Session, cancellationToken);
		}

		/// <inheritdoc/>
		async Task<Agent?> TryCreateSessionAsync(Agent agent, CreateSessionOptions options, CancellationToken cancellationToken)
		{
			// Fail if there's already an active session
			if (agent.Session != null)
			{
				return null;
			}

			// Create the new settings for the agent
			AgentId agentId = agent.Id;
			SessionId sessionId = SessionIdUtils.GenerateNewId();

			List<string> newProperties = options.Capabilities.Properties.ToList();
			Dictionary<string, int> newResources = options.Capabilities.Resources.ToDictionary();

			List<PoolId> newDynamicPools = new(options.DynamicPools);
			List<PoolId> newPools = CreatePoolsList(agent.Document.ExplicitPools, newDynamicPools, newProperties);
			SetPoolProperties(newProperties, newPools);

			RpcAgentCapabilities capabilities = new RpcAgentCapabilities();
			capabilities.Properties.Add(newProperties);
			capabilities.Resources.Add(options.Capabilities.Resources);

			// Attempt to create the session in Redis
			RpcSession? newSession = await _scheduler.TryCreateSessionAsync(agentId, sessionId, capabilities, cancellationToken);
			if (newSession == null)
			{
				return null;
			}

			// Update the agent with the new session id
			UpdateDefinition<AgentDocument> update = Builders<AgentDocument>.Update
				.Set(x => x.SessionId, sessionId)
				.Unset(x => x.LastOnlineTime)
				.Unset(x => x.Deleted)
				.Set(x => x.Properties, newProperties)
				.Set(x => x.Resources, newResources)
				.Set(x => x.Pools, newPools)
				.Set(x => x.DynamicPools, newDynamicPools)
				.Set(x => x.Version, options.Version)
				.Unset(x => x.RequestRestart)
				.Unset(x => x.RequestShutdown)
				.Unset(x => x.RequestForceRestart)
				.Set(x => x.LastShutdownReason, "Unexpected");

			if (String.Equals(options.Version, agent.Document.LastUpgradeVersion, StringComparison.Ordinal))
			{
				update = update.Unset(x => x.UpgradeAttemptCount);
			}

			AgentDocument? newDocument = await TryUpdateAsync(agent.Document, update, cancellationToken);
			if (newDocument == null)
			{
				// If this fails, stop the session we create. It will expire naturally anyway, but we can avoid the delay on anything being requeued by just ending it now.
				await _scheduler.TryUpdateSessionAsync(newSession, new RpcSession(newSession) { Status = RpcAgentStatus.Stopped }, cancellationToken: cancellationToken);
				return null;
			}

			// Create a new session document
			SessionDocument newSessionDocument = new SessionDocument(sessionId, agentId, _clock.UtcNow, options.Version);
			await _sessionCollection.InsertOneAsync(newSessionDocument, null, cancellationToken);

			// Update the agent is updated with the session
			return new Agent(this, newDocument, newSession);
		}

		/// <inheritdoc/>
		async Task<Agent?> TryTerminateSessionAsync(Agent agent, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Terminating session {SessionId} for {AgentId}", agent.Session?.SessionId ?? SessionId.Empty, agent.Id);
			return await TryUpdateSessionAsync(agent, new UpdateSessionOptions(AgentStatus.Stopped, DefaultLeaseOutcomeReason: LeaseOutcomeReason.ServerInitiated | LeaseOutcomeReason.AgentSessionTerminated), cancellationToken);
		}

		/// <inheritdoc/>
		async Task<Agent?> TryCreateLeaseAsync(Agent agent, CreateLeaseOptions options, CancellationToken cancellationToken)
		{
			if (agent.Session == null)
			{
				return null;
			}

			AgentId agentId = agent.Document.Id;

			RpcSessionLease newLease = new RpcSessionLease();
			newLease.Id = options.Id;
			newLease.State = RpcLeaseState.Pending;
			newLease.Payload = Any.Pack(options.Payload);
			newLease.ParentId = options.ParentId;
			newLease.Exclusive = options.Exclusive;
			newLease.Resources.Add(options.Resources ?? Enumerable.Empty<KeyValuePair<string, int>>());

			RpcSession? newSession = new RpcSession(agent.Session);
			newSession.Leases.Add(newLease);

			newSession = await _scheduler.TryUpdateSessionAsync(agent.Session, newSession, cancellationToken: cancellationToken);
			if (newSession == null)
			{
				return null;
			}

			try
			{
				DateTime startTime = _clock.UtcNow;
				await _leaseCollection.AddAsync(options.Id, options.ParentId, options.Name, agentId, newSession.SessionId, options.StreamId, options.PoolId, options.LogId, startTime, Any.Pack(options.Payload).ToByteArray(), cancellationToken);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to create lease {LeaseId} for agent {AgentId}; lease already exists?", options.Id, agentId);
			}

			// Update the agent state
			AgentDocument? document = agent.Document;
			while (document != null)
			{
				List<UpdateDefinition<AgentDocument>> updates = new List<UpdateDefinition<AgentDocument>>();
				GetNewLeaseUpdates(document, Any.Pack(options.Payload).ToByteArray(), updates);

				if (updates.Count == 0)
				{
					break;
				}

				UpdateDefinition<AgentDocument> update = Builders<AgentDocument>.Update.Combine(updates);

				document = await TryUpdateAsync(document, update, cancellationToken);
				if (document != null)
				{
					break;
				}

				document = await GetDocumentAsync(agentId, cancellationToken);
				if (document == null)
				{
					return null;
				}
			}

			return await CreateAgentObjectAsync(document, newSession, cancellationToken);
		}

		static void GetNewLeaseUpdates(AgentDocument agent, ReadOnlySpan<byte> payloadData, List<UpdateDefinition<AgentDocument>> updates)
		{
			Any payload = Any.Parser.ParseFrom(payloadData);
			if (payload.TryUnpack(out ConformTask conformTask))
			{
				int newConformAttemptCount = (agent.ConformAttemptCount ?? 0) + 1;
				updates.Add(Builders<AgentDocument>.Update.Set(x => x.ConformAttemptCount, newConformAttemptCount));
				updates.Add(Builders<AgentDocument>.Update.Set(x => x.LastConformTime, DateTime.UtcNow));
			}
			else if (payload.TryUnpack(out UpgradeTask upgradeTask))
			{
				string newVersion = upgradeTask.SoftwareId;

				int versionIdx = newVersion.IndexOf(':', StringComparison.Ordinal);
				if (versionIdx != -1)
				{
					newVersion = newVersion.Substring(versionIdx + 1);
				}

				int newUpgradeAttemptCount = (agent.UpgradeAttemptCount ?? 0) + 1;
				updates.Add(Builders<AgentDocument>.Update.Set(x => x.LastUpgradeVersion, newVersion));
				updates.Add(Builders<AgentDocument>.Update.Set(x => x.UpgradeAttemptCount, newUpgradeAttemptCount));
				updates.Add(Builders<AgentDocument>.Update.Set(x => x.LastUpgradeTime, DateTime.UtcNow));
			}
		}

		/// <inheritdoc/>
		async Task<Agent?> TryCancelLeaseAsync(Agent agent, int leaseIdx, LeaseOutcomeReason? reason, CancellationToken cancellationToken)
		{
			if (agent.Session == null)
			{
				return null;
			}

			RpcSession? newSession = new RpcSession(agent.Session);
			LeaseId leaseId = newSession.Leases[leaseIdx].Id;

			if (newSession.Leases[leaseIdx].State == RpcLeaseState.Pending)
			{
				newSession.Leases.RemoveAt(leaseIdx);

				// Record the outcome immediately for pending leases since they'll be removed from the session
				if (reason.HasValue)
				{
					await _leaseCollection.TrySetOutcomeAsync(leaseId, _clock.UtcNow, LeaseOutcome.Cancelled, reason.Value, null, null, cancellationToken);
				}
			}
			else
			{
				newSession.Leases[leaseIdx].State = RpcLeaseState.Cancelled;

				// Pre-set the outcome for active leases - this ensures the reason is captured
				// When the agent reports completion, TryMergeOutcomeReasonsAsync will merge additional reasons
				if (reason.HasValue)
				{
					bool updated = await _leaseCollection.TrySetOutcomeAsync(leaseId, _clock.UtcNow, LeaseOutcome.Cancelled, reason.Value, null, null, cancellationToken);
					if (!updated)
					{
						// Outcome was already set by another path, add our reasons to it
						await _leaseCollection.TryAddOutcomeReasonsAsync(leaseId, reason.Value, cancellationToken);
					}
				}
			}

			newSession = await _scheduler.TryUpdateSessionAsync(agent.Session, newSession, null, cancellationToken);
			if (newSession == null)
			{
				return null;
			}

			return await CreateAgentObjectAsync(agent.Document, newSession, cancellationToken);
		}

		void OnSessionUpdate(SessionId sessionId)
		{
			if (_sessionIdToTcs.TryGetValue(sessionId, out TaskCompletionSource? tcs))
			{
				_logger.LogDebug("Session {SessionId} was updated.", sessionId);
				tcs.TrySetResult();
			}
		}

		async Task<Agent?> WaitForSessionUpdateAsync(Agent agent, CancellationToken cancellationToken)
		{
			if (agent.Session == null)
			{
				return null;
			}

			RpcSession? newSession = await WaitForSessionUpdateAsync(agent.Session, cancellationToken);
			return await CreateAgentObjectAsync(agent.Document, newSession, cancellationToken);
		}

		async Task<RpcSession?> WaitForSessionUpdateAsync(RpcSession session, CancellationToken cancellationToken)
		{
			TaskCompletionSource newTcs = new TaskCompletionSource();
			try
			{
				TaskCompletionSource tcs = _sessionIdToTcs.GetOrAdd(session.SessionId, newTcs);

				RpcSession? newSession = await _scheduler.TryGetSessionAsync(session.SessionId, cancellationToken);
				if (newSession != null && newSession.UpdateTicks == session.UpdateTicks)
				{
					await tcs.Task.WaitAsync(cancellationToken);
					newSession = await _scheduler.TryGetSessionAsync(session.SessionId, cancellationToken);
				}

				return newSession;
			}
			finally
			{
				newTcs.TrySetResult();
				_sessionIdToTcs.TryRemove(new KeyValuePair<SessionId, TaskCompletionSource>(session.SessionId, newTcs));
			}
		}

		/// <inheritdoc/>
		public IAuditLogChannel<AgentId> GetLogger(AgentId agentId)
		{
			return _auditLog[agentId];
		}
	}
}
