// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Redis;
using Google.Protobuf;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Server;
using Microsoft.Extensions.Caching.Memory;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using StackExchange.Redis;

namespace HordeServer.Agents
{
	internal class AgentScheduler : IAgentScheduler, IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Time after which we expire filters
		/// </summary>
		public static TimeSpan ExpireFiltersTime { get; } = TimeSpan.FromMinutes(30.0);

#pragma warning disable CA1822 // Make property static
		static class Keys
		{
			public static FilterCollectionKeys Filters { get; }
				= new();

			public static SessionCollectionKeys Sessions { get; }
				= new();

			public static LeaseCollectionKeys Leases { get; }
				= new();
		}

		record struct FilterCollectionKeys
		{
			public RedisHashKey<IoHash, long> Current // Maps requirements hash to last-touched ticks
				=> new($"compute:filters:current");

			public FilterKeys this[IoHash requirementsHash]
				=> new(requirementsHash);
		}

		record struct FilterKeys(IoHash RequirementsHash)
		{
			public RedisSetKey<SessionId> AvailableSessions
				=> new($"compute:filters:{RequirementsHash}:available");

			public RedisSetKey<SessionId> PotentialSessions
				=> new($"compute:filters:{RequirementsHash}:potential");

			public RedisStringKey<RpcAgentRequirements> Requirements
				=> new($"compute:filters:{RequirementsHash}:reqs");
		}

		record struct SessionCollectionKeys
		{
			public RedisHashKey<SessionId, long> Current // Maps session id to last update ticks
				=> new($"compute:sessions:current");

			public SessionKeys this[SessionId sessionId]
				=> new(sessionId);
		}

		record struct SessionKeys(SessionId SessionId)
		{
			public RedisStringKey<RpcAgentCapabilities> Capabilities
				=> new($"compute:sessions:{SessionId}:caps");

			public RedisHashKey<IoHash, bool> Filters // Maps filter requirements hash to a flag indicating whether it's currently available
				=> new($"compute:sessions:{SessionId}:filters");

			public RedisStringKey<RpcSession> State
				=> new($"compute:sessions:{SessionId}:state");
		}

		internal static string SessionStateKey(string sessionId) => $"compute:sessions:{sessionId}:state";

		SessionId FromSessionStateKey(string sessionStateKey)
		{
			string[] parts = sessionStateKey.Split(":");
			return parts.Length == 4 ? SessionId.Parse(parts[2]) : throw new ArgumentException($"Unable to extract session ID from key: {sessionStateKey}");
		}

		record struct LeaseCollectionKeys
		{
			public RedisSetKey<LeaseId> Current
				=> new($"compute:leases:active");

			public LeaseKeys this[LeaseId leaseId]
				=> new(leaseId);
		}

		record struct LeaseKeys(LeaseId LeaseId)
		{
			public RedisSetKey<LeaseId> Children
				=> new($"compute:leases:{LeaseId}:children");
		}

#pragma warning restore CA1822

		record class LocalFilter(IoHash Hash, RpcAgentRequirements Requirements);

		record class LocalCapabilities(IoHash Hash, RpcAgentCapabilities Message, IReadOnlySet<string> Properties)
		{
			public LocalCapabilities(IoHash hash, RpcAgentCapabilities message)
				: this(hash, message, new HashSet<string>(message.Properties, StringComparer.Ordinal)) { }
		}

		record class LocalSession(long UpdateTicks, LocalCapabilities Capabilities, IReadOnlyDictionary<IoHash, bool> Filters);

		public event Action<SessionId>? SessionUpdated;

		readonly IRedisService _redisService;
		readonly IClock _clock;
		readonly ITicker _updateCachedFiltersTicker;
		readonly ITicker _expireFiltersTicker;
		readonly ILogger _logger;
		readonly MemoryCache _localSessionCache;
		readonly MemoryCache _requirementsCache;

		LocalFilter[] _cachedFilters = Array.Empty<LocalFilter>();

		static readonly RedisChannel<SessionId> s_sessionUpdateChannel = new RedisChannel<SessionId>(RedisChannel.Literal("compute:sessions:update"));

		void AddLocalSessionToCache(SessionId sessionId, LocalSession localSession)
		{
			using (ICacheEntry cacheEntry = _localSessionCache.CreateEntry(sessionId))
			{
				cacheEntry.SetSlidingExpiration(TimeSpan.FromMinutes(5.0));
				cacheEntry.SetValue(localSession);
			}
		}

		async ValueTask<LocalSession?> TryGetLocalSessionAsync(RpcSession session, CancellationToken cancellationToken)
		{
			// Try to get the current cached value
			LocalSession? localSession;
			if (!_localSessionCache.TryGetValue(session.SessionId, out localSession) || localSession == null || localSession.UpdateTicks != session.UpdateTicks)
			{
				IDatabase database = _redisService.GetDatabase();

				// Update the capabilites
				LocalCapabilities? capabilities;
				if (localSession != null && localSession.Capabilities.Hash == session.CapabilitiesHash)
				{
					capabilities = localSession.Capabilities;
				}
				else
				{
					byte[]? data = (byte[]?)await database.StringGetAsync(Keys.Sessions[session.SessionId].Capabilities.Inner).WaitAsync(cancellationToken);
					if (data == null)
					{
						_logger.LogDebug("Missing capabilities key for session {SessionId}; can retry.", session.SessionId);
						return null;
					}

					IoHash hash = IoHash.Compute(data);
					if (session.CapabilitiesHash != hash)
					{
						ITransaction transaction = database.CreateTransaction();
						transaction.AddCondition(Keys.Sessions[session.SessionId].State.StringEqual(session));
						transaction.AddCondition(Condition.StringEqual(Keys.Sessions[session.SessionId].Capabilities.Inner, data));
						_ = transaction.StringSetAsync(Keys.Sessions[session.SessionId].State, new RpcSession(session) { CapabilitiesHash = hash, UpdateTicks = session.UpdateTicks + 1 }, flags: CommandFlags.FireAndForget);

						if (await transaction.ExecuteAsync())
						{
							_logger.LogWarning("Repaired mismatch between capabilities and session hash for agent {AgentId} session {SessionId}. Was {OldHash}, now {NewHash}.", session.AgentId, session.SessionId, session.CapabilitiesHash, hash);  
						}
						else
						{
							_logger.LogDebug("Hash mismatch for capabilities of session {SessionId}; can retry.", session.SessionId);
						}
						return null;
					}

					capabilities = new LocalCapabilities(hash, RpcAgentCapabilities.Parser.ParseFrom(data));
				}

				// Update the filters
				HashEntry<IoHash, bool>[] entries = await database.HashGetAllAsync(Keys.Sessions[session.SessionId].Filters).WaitAsync(cancellationToken);
				Dictionary<IoHash, bool> filters = entries.ToDictionary(x => x.Name, x => x.Value);

				// Create the new session state
				localSession = new LocalSession(session.UpdateTicks, capabilities, filters);
				AddLocalSessionToCache(session.SessionId, localSession);
			}
			return localSession;
		}

		IAsyncDisposable? _updateEventSubscription;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentScheduler(IRedisService redisService, IClock clock, ILogger<AgentScheduler> logger)
		{
			_redisService = redisService;
			_clock = clock;
			_updateCachedFiltersTicker = clock.AddTicker<AgentScheduler>(TimeSpan.FromSeconds(30.0), UpdateCachedFiltersTickAsync, logger);
			_expireFiltersTicker = clock.AddSharedTicker($"{nameof(AgentScheduler)}.{nameof(ExpireFiltersAsync)}", TimeSpan.FromMinutes(5.0), ExpireFiltersAsync, logger);
			_localSessionCache = new MemoryCache(new MemoryCacheOptions());
			_requirementsCache = new MemoryCache(new MemoryCacheOptions());
			_logger = logger;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _updateCachedFiltersTicker.DisposeAsync();
			await _expireFiltersTicker.DisposeAsync();

			if (_updateEventSubscription != null)
			{
				await _updateEventSubscription.DisposeAsync();
				_updateEventSubscription = null;
			}

			_requirementsCache.Dispose();
			_localSessionCache.Dispose();
		}

		void OnSessionUpdate(SessionId sessionId)
			=> SessionUpdated?.Invoke(sessionId);

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_updateEventSubscription = await _redisService.GetDatabase().Multiplexer.SubscribeAsync(s_sessionUpdateChannel, OnSessionUpdate);

			await _updateCachedFiltersTicker.StartAsync();
			await _expireFiltersTicker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _expireFiltersTicker.StopAsync();
			await _updateCachedFiltersTicker.StopAsync();

			if (_updateEventSubscription != null)
			{
				await _updateEventSubscription.DisposeAsync();
				_updateEventSubscription = null;
			}
		}

		#region Sessions

		/// <inheritdoc/>
		public async Task<RpcSession?> TryCreateSessionAsync(AgentId agentId, SessionId sessionId, RpcAgentCapabilities capabilities, CancellationToken cancellationToken = default)
		{
			DateTime utcNow = _clock.UtcNow;

			byte[] capabilitiesData = capabilities.ToByteArray();
			IoHash capabilitiesHash = IoHash.Compute(capabilitiesData);

			RpcSession newSession = new RpcSession();
			newSession.AgentId = agentId;
			newSession.SessionId = sessionId;
			newSession.CapabilitiesHash = capabilitiesHash;
			newSession.UpdateTicks = utcNow.Ticks;
			newSession.Status = RpcAgentStatus.Ok;

			ITransaction transaction = _redisService.GetDatabase().CreateTransaction();
			transaction.AddCondition(Keys.Sessions.Current.HashNotExists(sessionId));
			_ = transaction.HashSetAsync(Keys.Sessions.Current, sessionId, newSession.UpdateTicks, flags: CommandFlags.FireAndForget);
			_ = transaction.StringSetAsync(Keys.Sessions[sessionId].Capabilities.Inner, capabilitiesData, flags: CommandFlags.FireAndForget);
			_ = transaction.StringSetAsync(Keys.Sessions[sessionId].State, newSession, flags: CommandFlags.FireAndForget);

			LocalCapabilities localCapabilities = new LocalCapabilities(capabilitiesHash, capabilities);
			Dictionary<IoHash, bool> filters = FindMatchingFilters(localCapabilities, newSession);
			UpdateFilters(transaction, sessionId, new Dictionary<IoHash, bool>(), filters);

			if (!await transaction.ExecuteAsync().WaitAsync(cancellationToken))
			{
				_logger.LogDebug("Unable to create session {SessionId}; can retry", sessionId);
				return null;
			}

			LocalSession localSession = new LocalSession(newSession.UpdateTicks, localCapabilities, filters);
			AddLocalSessionToCache(sessionId, localSession);

			return newSession;
		}

		/// <inheritdoc/>
		public async Task<RpcSession?> TryGetSessionAsync(SessionId sessionId, CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			return await database.StringGetAsync(Keys.Sessions[sessionId].State);
		}

		/// <inheritdoc/>
		public async Task<Dictionary<SessionId, RpcSession>> TryGetSessionsAsync(IReadOnlyList<SessionId> sessionIds, CancellationToken cancellationToken = default)
		{
			Dictionary<SessionId, RpcSession> result = new (sessionIds.Count);
			if (sessionIds.Count == 0)
			{
				return result;
			}

			IDatabase database = _redisService.GetDatabase();

			// Use .State.Inner to get the raw RedisKey; the multi-key StringGetAsync overload requires RedisKey[]
			// rather than the typed key wrapper used by the single-key overload in TryGetSessionAsync.
			RedisKey[] keys = new RedisKey[sessionIds.Count];
			for (int idx = 0; idx < sessionIds.Count; idx++)
			{
				keys[idx] = Keys.Sessions[sessionIds[idx]].State.Inner;
			}

			RedisValue[] values = await database.StringGetAsync(keys).WaitAsync(cancellationToken);
			for (int idx = 0; idx < sessionIds.Count; idx++)
			{
				if (!values[idx].IsNull)
				{
					result[sessionIds[idx]] = RedisSerializer.Deserialize<RpcSession>(values[idx]);
				}
			}

			return result;
		}

		/// <inheritdoc/>
		public async Task<RpcAgentCapabilities?> TryGetCapabilitiesAsync(RpcSession session, CancellationToken cancellationToken = default)
		{
			LocalSession? localSession = await TryGetLocalSessionAsync(session, cancellationToken);
			return localSession?.Capabilities.Message;
		}

		/// <inheritdoc/>
		public async Task<RpcSession?> TryUpdateSessionAsync(RpcSession session, RpcSession? newSession, RpcAgentCapabilities? newCapabilities, CancellationToken cancellationToken)
		{
			SessionId sessionId = session.SessionId;

			// Get the local session state. This includes cached values for everything in Redis.
			LocalSession? localSession = await TryGetLocalSessionAsync(session, cancellationToken);
			if (localSession == null)
			{
				return null;
			}

			// Copy the session document if a new one was not specified, and update the last modified time
			newSession ??= new RpcSession(session);

			// Start building the update transaction
			IDatabase database = _redisService.GetDatabase();
			ITransaction transaction = database.CreateTransaction();
			transaction.AddCondition(Keys.Sessions.Current.HashEqual(newSession.SessionId, session.UpdateTicks));

			// Handle agents transitioning to the stopping state
			if (newSession.Status == RpcAgentStatus.Stopped)
			{
				newSession.UpdateTicks = Math.Max(newSession.UpdateTicks + 1, Math.Min(newSession.UpdateTicks + RpcSession.ExpireAfterTime.Ticks, _clock.UtcNow.Ticks));

				newSession.Leases.Clear(); // Need to ensure we remove items from the active set below

				_ = transaction.HashDeleteAsync(Keys.Sessions.Current, newSession.SessionId, flags: CommandFlags.FireAndForget);
				_ = transaction.KeyDeleteAsync(Keys.Sessions[sessionId].Capabilities, flags: CommandFlags.FireAndForget);
				_ = transaction.KeyDeleteAsync(Keys.Sessions[sessionId].State, flags: CommandFlags.FireAndForget);
				UpdateFilters(transaction, sessionId, localSession.Filters, new Dictionary<IoHash, bool>());

				localSession = new LocalSession(newSession.UpdateTicks, localSession.Capabilities, new Dictionary<IoHash, bool>());
			}
			else
			{
				DateTime utcNow = _clock.UtcNow;

				// Extend the expiry time, ensuring it only ever increases
				newSession.UpdateTicks = Math.Max(session.UpdateTicks + 1, utcNow.Ticks);
				_ = transaction.HashSetAsync(Keys.Sessions.Current, session.SessionId, newSession.UpdateTicks, flags: CommandFlags.FireAndForget);

				// Update the capabilities
				LocalCapabilities capabilities = localSession.Capabilities;
				if (newCapabilities != null)
				{
					byte[] newCapabilitiesData = newCapabilities.ToByteArray();
					IoHash newCapabilitiesHash = IoHash.Compute(newCapabilitiesData);

					if (newCapabilitiesHash != capabilities.Hash)
					{
						capabilities = new LocalCapabilities(newCapabilitiesHash, newCapabilities);
						_ = transaction.StringSetAsync(Keys.Sessions[session.SessionId].Capabilities.Inner, newCapabilitiesData, flags: CommandFlags.FireAndForget);
					}
				}
				newSession.CapabilitiesHash = capabilities.Hash;

				// Update the session state
				_ = transaction.StringSetAsync(Keys.Sessions[session.SessionId].State, newSession, flags: CommandFlags.FireAndForget);

				// Update the filters for this session
				Dictionary<IoHash, bool> filters = FindMatchingFilters(capabilities, newSession);
				UpdateFilters(transaction, sessionId, localSession.Filters, filters);

				// Create the new local session object
				localSession = new LocalSession(newSession.UpdateTicks, capabilities, filters);
			}

			// Add any new leases to the global state
			foreach (RpcSessionLease newLease in newSession.Leases)
			{
				if (!session.Leases.Any(x => x.Id == newLease.Id))
				{
					_ = transaction.SetAddAsync(Keys.Leases.Current, newLease.Id, flags: CommandFlags.FireAndForget);
					if (newLease.ParentId != null)
					{
						_ = transaction.SetAddAsync(Keys.Leases[newLease.ParentId.Value].Children, newLease.Id, flags: CommandFlags.FireAndForget);
					}
				}
			}

			// Remove any complete leases from the global state
			foreach (RpcSessionLease oldLease in session.Leases)
			{
				if (!newSession.Leases.Any(x => x.Id == oldLease.Id))
				{
					_ = transaction.SetRemoveAsync(Keys.Leases.Current, oldLease.Id, flags: CommandFlags.FireAndForget);
					if (oldLease.ParentId != null)
					{
						_ = transaction.SetRemoveAsync(Keys.Leases[oldLease.ParentId.Value].Children, oldLease.Id, flags: CommandFlags.FireAndForget);
					}
				}
			}

			// Execute the transaction
			if (!await transaction.ExecuteAsync().WaitAsync(cancellationToken))
			{
				_logger.LogDebug("Redis transaction failed for update of session {SessionId}", sessionId);
				return null;
			}

			// Update the cached session state
			AddLocalSessionToCache(sessionId, localSession);

			// Trace the new expiry time for debugging
			_logger.LogDebug("Updated session {SessionId} expiry time to {ExpiryTime}", sessionId, newSession.ExpiryTime);

			// Notify watchers that the session state has changed
			if (newSession.Status != session.Status || newSession.CapabilitiesHash != session.CapabilitiesHash || !newSession.Leases.Equals(session.Leases))
			{
				_ = database.PublishAsync(s_sessionUpdateChannel, sessionId, CommandFlags.FireAndForget);
			}
			return newSession;
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<RpcSession> FindExpiredSessionsAsync([EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			long expireTicks = (_clock.UtcNow - RpcSession.ExpireAfterTime).Ticks;
			IDatabase database = _redisService.GetDatabase();

			HashEntry<SessionId, long>[] entries = await database.HashGetAllAsync(Keys.Sessions.Current);
			foreach ((SessionId sessionId, long updateTicks) in entries)
			{
				if (updateTicks < expireTicks)
				{
					RpcSession? session = await TryGetSessionAsync(sessionId, cancellationToken);
					if (session != null && session.UpdateTicks == updateTicks)
					{
						yield return session;
					}
				}
			}
		}

		/// <inheritdoc/>
		public async Task<List<SessionId>> RemoveOrphanedSessionsAsync(CancellationToken cancellationToken = default)
		{
			IServer server = _redisService.GetConnection().GetServers().First();
			IDatabase database = _redisService.GetDatabase();

			// Sessions with a state key
			HashSet<SessionId> sessionStates = await server
				.KeysAsync(database: database.Database, pattern: SessionStateKey("*"))
				.Select(key => FromSessionStateKey(key.ToString()))
				.ToHashSetAsync(cancellationToken: cancellationToken);
			
			// Sessions registered in the "compute:sessions:current" hash key
			HashSet<SessionId> registeredSessions = (await database.HashGetAllAsync(Keys.Sessions.Current))
				.Select(x => x.Name).ToHashSet();

			List<SessionId> sessionIds = sessionStates.Except(registeredSessions).ToList();
			foreach (SessionId sid in sessionIds)
			{
				_logger.LogInformation("Deleting orphaned session ID {SessionId}", sid);
				await database.KeyDeleteAsync(sid.ToString());
			}

			return sessionIds;
		}

		#endregion
		#region Filters

		/// <inheritdoc/>
		public async Task<IoHash> CreateFilterAsync(RpcAgentRequirements requirements, CancellationToken cancellationToken = default)
		{
			byte[] requirementsData = requirements.ToByteArray();
			IoHash requirementsHash = IoHash.Compute(requirementsData);

			for (; ; )
			{
				IDatabase database = _redisService.GetDatabase();
				DateTime utcNow = _clock.UtcNow;

				ITransaction transaction = database.CreateTransaction();
				transaction.AddCondition(Keys.Filters.Current.HashNotExists(requirementsHash));
				_ = transaction.HashSetAsync(Keys.Filters.Current, requirementsHash, utcNow.Ticks);
				_ = transaction.StringSetAsync(Keys.Filters[requirementsHash].Requirements.Inner, requirementsData, flags: CommandFlags.FireAndForget);

				if (await transaction.ExecuteAsync().WaitAsync(cancellationToken))
				{
					break;
				}

				transaction = database.CreateTransaction();
				transaction.AddCondition(Keys.Filters.Current.HashNotExists(requirementsHash));
				_ = transaction.HashSetAsync(Keys.Filters.Current, requirementsHash, utcNow.Ticks, flags: CommandFlags.FireAndForget);

				if (await transaction.ExecuteAsync().WaitAsync(cancellationToken))
				{
					break;
				}
			}

			return requirementsHash;
		}

		/// <summary>
		/// Update all the cached filters
		/// </summary>
		async ValueTask UpdateCachedFiltersTickAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				if (await TryUpdateCachedFiltersAsync(cancellationToken))
				{
					break;
				}
			}
		}

		async Task<bool> TryUpdateCachedFiltersAsync(CancellationToken cancellationToken)
		{
			IDatabase database = _redisService.GetDatabase();

			IoHash[] filterHashes = await database.HashKeysAsync(Keys.Filters.Current);
			LocalFilter[] filters = new LocalFilter[filterHashes.Length];

			for (int idx = 0; idx < filterHashes.Length; idx++)
			{
				IoHash filterHash = filterHashes[idx];

				RpcAgentRequirements? requirements = await TryGetFilterRequirementsAsync(filterHash, cancellationToken);
				if (requirements == null)
				{
					return false;
				}

				filters[idx] = new LocalFilter(filterHash, requirements);
			}

			_cachedFilters = filters;
			return true;
		}

		/// <summary>
		/// Expire all the filters older than the standard expiry time
		/// </summary>
		async ValueTask ExpireFiltersAsync(CancellationToken cancellationToken)
		{
			IDatabase database = _redisService.GetDatabase();

			DateTime expiryTime = _clock.UtcNow - ExpireFiltersTime;

			HashEntry<IoHash, long>[] entries = await database.HashGetAllAsync(Keys.Filters.Current).WaitAsync(cancellationToken);
			foreach ((IoHash requirementsHash, long ticks) in entries)
			{
				cancellationToken.ThrowIfCancellationRequested();
				if (ticks < expiryTime.Ticks)
				{
					ITransaction transaction = database.CreateTransaction();
					transaction.AddCondition(Keys.Filters.Current.HashEqual(requirementsHash, ticks));

					_ = transaction.HashDeleteAsync(Keys.Filters.Current, requirementsHash, flags: CommandFlags.FireAndForget);
					_ = transaction.KeyDeleteAsync(Keys.Filters[requirementsHash].AvailableSessions, flags: CommandFlags.FireAndForget);
					_ = transaction.KeyDeleteAsync(Keys.Filters[requirementsHash].PotentialSessions, flags: CommandFlags.FireAndForget);
					_ = transaction.KeyDeleteAsync(Keys.Filters[requirementsHash].Requirements, flags: CommandFlags.FireAndForget);

					_ = transaction.ExecuteAsync(CommandFlags.FireAndForget);
				}
			}
		}

		/// <inheritdoc/>
		public async Task TouchFilterAsync(IoHash requirementsHash, CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			DateTime utcNow = _clock.UtcNow;
			await database.HashSetAsync(Keys.Filters.Current, requirementsHash, utcNow.Ticks);
		}

		/// <inheritdoc/>
		public async Task<IoHash[]> GetFiltersAsync(CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			return await database.HashKeysAsync(Keys.Filters.Current);
		}

		public async Task ForceUpdateFiltersAsync(CancellationToken cancellationToken)
		{
			IDatabase database = _redisService.GetDatabase();

			await UpdateCachedFiltersTickAsync(cancellationToken);

			SessionId[] sessionIds = await database.HashKeysAsync(Keys.Sessions.Current);
			foreach (SessionId sessionId in sessionIds)
			{
				RpcSession? session = await TryGetSessionAsync(sessionId, cancellationToken);
				if (session != null)
				{
					LocalSession? localSession = await TryGetLocalSessionAsync(session, cancellationToken);
					if (localSession != null)
					{
						Dictionary<IoHash, bool> newFilters = FindMatchingFilters(localSession.Capabilities, session);

						ITransaction transaction = database.CreateTransaction();
						UpdateFilters(transaction, session.SessionId, localSession.Filters, newFilters);
						await transaction.ExecuteAsync().WaitAsync(cancellationToken);

						AddLocalSessionToCache(sessionId, localSession with { Filters = newFilters });
					}
				}
			}
		}

		static void UpdateFilters(ITransaction transaction, SessionId sessionId, IReadOnlyDictionary<IoHash, bool> oldFilters, IReadOnlyDictionary<IoHash, bool> newFilters)
		{
			// Create any new filters
			foreach ((IoHash newFilter, bool newAvailable) in newFilters)
			{
				bool oldAvailable;
				if (!oldFilters.TryGetValue(newFilter, out oldAvailable))
				{
					transaction.AddCondition(Keys.Filters.Current.HashExists(newFilter));
					_ = transaction.HashSetAsync(Keys.Sessions[sessionId].Filters, newFilter, newAvailable, flags: CommandFlags.FireAndForget);
					_ = transaction.SetAddAsync(Keys.Filters[newFilter].PotentialSessions, sessionId, flags: CommandFlags.FireAndForget);
					if (newAvailable)
					{
						_ = transaction.SetAddAsync(Keys.Filters[newFilter].AvailableSessions, sessionId, flags: CommandFlags.FireAndForget);
					}
				}
				else if (newAvailable != oldAvailable)
				{
					transaction.AddCondition(Keys.Filters.Current.HashExists(newFilter));
					_ = transaction.HashSetAsync(Keys.Sessions[sessionId].Filters, newFilter, newAvailable, flags: CommandFlags.FireAndForget);
					if (newAvailable)
					{
						_ = transaction.SetAddAsync(Keys.Filters[newFilter].AvailableSessions, sessionId, flags: CommandFlags.FireAndForget);
					}
					else
					{
						_ = transaction.SetRemoveAsync(Keys.Filters[newFilter].AvailableSessions, sessionId, flags: CommandFlags.FireAndForget);
					}
				}
			}

			// Remove any filters that no longer exist
			foreach ((IoHash oldFilter, bool oldAvailable) in oldFilters)
			{
				if (!newFilters.ContainsKey(oldFilter))
				{
					_ = transaction.HashDeleteAsync(Keys.Sessions[sessionId].Filters, oldFilter, flags: CommandFlags.FireAndForget);
					_ = transaction.SetRemoveAsync(Keys.Filters[oldFilter].PotentialSessions, sessionId, flags: CommandFlags.FireAndForget);
					if (oldAvailable)
					{
						_ = transaction.SetRemoveAsync(Keys.Filters[oldFilter].AvailableSessions, sessionId, flags: CommandFlags.FireAndForget);
					}
				}
			}
		}

		Dictionary<IoHash, bool> FindMatchingFilters(LocalCapabilities capabilities, RpcSession session)
		{
			LocalFilter[] cachedFilters = _cachedFilters;

			Dictionary<IoHash, bool> newFilters = new Dictionary<IoHash, bool>();
			foreach (LocalFilter filter in cachedFilters)
			{
				if (MeetsRequirements(capabilities, filter.Requirements))
				{
					bool available = IsAvailable(capabilities, session, filter.Requirements);
					newFilters.Add(filter.Hash, available);
				}
			}

			return newFilters;
		}

		static bool MeetsRequirements(LocalCapabilities capabilities, RpcAgentRequirements requirements)
		{
			foreach (string property in requirements.Properties)
			{
				if (!capabilities.Properties.Contains(property))
				{
					return false;
				}
			}
			foreach ((string name, int requiredCount) in requirements.Resources)
			{
				if (!capabilities.Message.Resources.TryGetValue(name, out int maxCount) || requiredCount > maxCount)
				{
					return false;
				}
			}
			return true;
		}

		static bool IsAvailable(LocalCapabilities capabilities, RpcSession session, RpcAgentRequirements requirements)
		{
			if (session.Leases.Any(x => x.Exclusive))
			{
				return false;
			}
			if (!requirements.Shared && session.Leases.Any())
			{
				return false;
			}

			foreach ((string name, int requiredCount) in requirements.Resources)
			{
				int availableCount;
				if (!capabilities.Message.Resources.TryGetValue(name, out availableCount))
				{
					return false;
				}
				foreach (RpcSessionLease lease in session.Leases)
				{
					if (lease.Resources.TryGetValue(name, out int count))
					{
						availableCount -= count;
					}
				}
				if (availableCount < requiredCount)
				{
					return false;
				}
			}

			return true;
		}

		void AddRequirementsToCache(IoHash requirementsHash, RpcAgentRequirements requirements)
		{
			using (ICacheEntry entry = _requirementsCache.CreateEntry(requirementsHash))
			{
				entry.SetSlidingExpiration(TimeSpan.FromMinutes(30.0));
				entry.SetValue(requirements);
			}
		}

		/// <inheritdoc/>
		public async ValueTask<RpcAgentRequirements?> TryGetFilterRequirementsAsync(IoHash requirementsHash, CancellationToken cancellationToken)
		{
			RpcAgentRequirements? requirements;
			if (!_requirementsCache.TryGetValue(requirementsHash, out requirements))
			{
				IDatabase database = _redisService.GetDatabase();
				requirements = await database.StringGetAsync(Keys.Filters[requirementsHash].Requirements);

				if (requirements != null)
				{
					AddRequirementsToCache(requirementsHash, requirements);
				}
			}
			return requirements;
		}

		/// <inheritdoc/>
		public async IAsyncEnumerable<SessionId> EnumerateFilteredSessionsAsync(IoHash requirementsHash, SessionFilterType type, [EnumeratorCancellation] CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			_ = database.HashSetAsync(Keys.Filters.Current, requirementsHash, _clock.UtcNow.Ticks, flags: CommandFlags.FireAndForget);

			RedisSetKey<SessionId> key = (type == SessionFilterType.Available)? Keys.Filters[requirementsHash].AvailableSessions : Keys.Filters[requirementsHash].PotentialSessions;
			await foreach (SessionId sessionId in database.SetScanAsync(key).WithCancellation(cancellationToken))
			{
				yield return sessionId;
			}
		}

		#endregion
		#region Stats

		/// <inheritdoc/>
		public async Task<LeaseId[]> FindActiveLeaseIdsAsync(CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			return await database.SetMembersAsync(Keys.Leases.Current).WaitAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<int> GetChildLeaseCountAsync(LeaseId id, CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			return (int)await database.SetLengthAsync(Keys.Leases[id].Children).WaitAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<LeaseId[]> GetChildLeaseIdsAsync(LeaseId id, CancellationToken cancellationToken = default)
		{
			IDatabase database = _redisService.GetDatabase();
			return await database.SetMembersAsync(Keys.Leases[id].Children).WaitAsync(cancellationToken);
		}

		#endregion
	}
}
