// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using System.Security.Claims;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Redis;
using EpicGames.Serialization;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Messages;
using HordeServer.Acls;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Auditing;
using HordeServer.Server;
using HordeServer.Tasks;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using StackExchange.Redis;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents
{
	/// <summary>
	/// Singleton used to store agent costs
	/// </summary>
	[RedisConverter(typeof(RedisCbConverter<>))]
	public class AgentRateTable
	{
		/// <summary>
		/// List of costs for different agent types
		/// </summary>
		[CbField]
		public List<AgentRateConfig> Entries { get; set; } = new List<AgentRateConfig>();
	}

	/// <summary>
	/// Wraps functionality for manipulating agents
	/// </summary>
	public sealed class AgentService : IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Maximum time between updates for an agent to be considered online
		/// </summary>
		public static readonly TimeSpan SessionExpiryTime = RpcSession.ExpireAfterTime;

		/// <summary>
		/// Time before a session expires that we will poll until
		/// </summary>
		public static readonly TimeSpan SessionLongPollTime = TimeSpan.FromSeconds(55);

		/// <summary>
		/// Time after which a session will be renewed
		/// </summary>
		public static readonly TimeSpan SessionRenewTime = TimeSpan.FromSeconds(50);

		/// <summary>
		/// Collection of agent documents
		/// </summary>
		public IAgentCollection Agents { get; }

		readonly ILeaseCollection _leases;
		readonly IAclService _aclService;
		readonly IDowntimeService _downtimeService;
		readonly ITaskSource[] _taskSources;
		readonly IRedisService _redisService;
		readonly IHostApplicationLifetime _applicationLifetime;
		readonly IOptionsMonitor<ComputeConfig> _computeConfig;
		readonly IClock _clock;
		readonly Tracer _tracer;
		readonly ILogger _logger;
		readonly ITicker _ticker;

		/// <summary>Lazily updated list of current pools</summary>
		readonly AsyncCachedValue<IReadOnlyList<IPoolConfig>> _cachedPools;

		/// <summary>Manually updated cached list of current agents</summary>
		IReadOnlyDictionary<AgentId, IAgent>? _cachedAgents;

		/// <summary>OpenTelemetry measurements (gauges)</summary>
		IEnumerable<Measurement<int>> _measurements = new List<Measurement<int>>();

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentService(
			IAgentCollection agents,
			ILeaseCollection leases,
			IAclService aclService,
			IDowntimeService downtimeService,
			IPoolCollection poolCollection,
			IEnumerable<ITaskSource> taskSources,
			IRedisService redisService,
			IHostApplicationLifetime applicationLifetime,
			IOptionsMonitor<ComputeConfig> computeConfig,
			IClock clock,
			Tracer tracer,
			Meter meter,
			ILogger<AgentService> logger)
		{
			Agents = agents;
			_leases = leases;
			_aclService = aclService;
			_downtimeService = downtimeService;
			_cachedPools = new AsyncCachedValue<IReadOnlyList<IPoolConfig>>(poolCollection.GetConfigsAsync, TimeSpan.FromSeconds(30.0));
			_taskSources = taskSources.ToArray();
			_applicationLifetime = applicationLifetime;
			_computeConfig = computeConfig;
			_redisService = redisService;
			_clock = clock;
			_ticker = clock.AddTicker($"{nameof(AgentService)}.{nameof(TickAsync)}", TimeSpan.FromSeconds(30.0), TickAsync, logger);
			_tracer = tracer;
			_logger = logger;

			meter.CreateObservableGauge("horde.agent.count", () => _measurements);
		}

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			await _ticker.StartAsync();
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await _ticker.StopAsync();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _cachedPools.DisposeAsync();
			await _ticker.DisposeAsync();
		}

		/// <summary>
		/// Gets user-readable payload information
		/// </summary>
		/// <param name="payload">The payload data</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Dictionary of key/value pairs for the payload</returns>
		public async ValueTask<Dictionary<string, string>?> GetPayloadDetailsAsync(ReadOnlyMemory<byte>? payload, CancellationToken cancellationToken = default)
		{
			Dictionary<string, string>? details = null;
			if (payload != null)
			{
				Any basePayload = Any.Parser.ParseFrom(payload.Value.ToArray());
				foreach (ITaskSource taskSource in _taskSources)
				{
					if (basePayload.Is(taskSource.Descriptor))
					{
						details = new Dictionary<string, string>();
						await taskSource.GetLeaseDetailsAsync(basePayload, details, cancellationToken);
						break;
					}
				}
			}
			return details;
		}

		/// <summary>
		/// Issues a bearer token for the given session id
		/// </summary>
		/// <param name="agentId">The agent id</param>
		/// <param name="sessionId">The session id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Bearer token for the agent</returns>
		public async ValueTask<string> IssueSessionTokenAsync(AgentId agentId, SessionId sessionId, CancellationToken cancellationToken = default)
		{
			List<AclClaimConfig> claims = new List<AclClaimConfig>();
			claims.Add(HordeClaims.AgentDedicatedRoleClaim);
			claims.Add(HordeClaims.GetAgentClaim(agentId));
			claims.Add(HordeClaims.GetSessionClaim(sessionId));
			return await _aclService.IssueBearerTokenAsync(claims, null, cancellationToken);
		}

		/// <summary>
		/// Register a new agent
		/// </summary>
		/// <param name="options">Parameters for new agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Unique ID for the agent</returns>
		public async Task<IAgent> CreateAgentAsync(CreateAgentOptions options, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				IAgent? agent = await Agents.GetAsync(options.Id, cancellationToken);
				if (agent == null)
				{
					return await Agents.AddAsync(options, cancellationToken);
				}

				agent = await agent.TryResetAsync(options.Ephemeral, options.EnrollmentKey, cancellationToken);
				if (agent != null)
				{
					return agent;
				}
			}
		}

		/// <summary>
		/// Gets an agent by ID
		/// </summary>
		/// <param name="agentId">Unique id of the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The agent document</returns>
		public Task<IAgent?> GetAgentAsync(AgentId agentId, CancellationToken cancellationToken = default)
		{
			return Agents.GetAsync(agentId, cancellationToken);
		}

		/// <summary>
		/// Finds all agents matching certain criteria
		/// </summary>
		/// <param name="poolId">The pool containing the agent</param>
		/// <param name="modifiedAfter">If set, only returns agents modified after this time</param>
		/// <param name="property">If set, only return agents matching this property</param>
		/// <param name="includeDeleted">If set, include agents marked as deleted</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of agents matching the given criteria</returns>
		public IAsyncEnumerable<IAgent> FindAgentsAsync(PoolId? poolId, DateTime? modifiedAfter, string? property, bool includeDeleted, CancellationToken cancellationToken)
		{
			return Agents.FindAsync(poolId, modifiedAfter, property, null, null, includeDeleted, true, cancellationToken);
		}

		/// <summary>
		/// Get all agents from local in-memory cache
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of all agents</returns>
		public async Task<IReadOnlyList<IAgent>> GetCachedAgentsAsync(CancellationToken cancellationToken)
		{
			if (_cachedAgents == null)
			{
				// First time this method is called, cached agents may not be initialized
				// This could lead to multiple concurrent agent refreshes (dog-piling), but that's fine as it should only happen once
				await RefreshCachedAgentsAsync(cancellationToken);
			}

			if (_cachedAgents == null)
			{
				throw new Exception("Cached agents not initialized");
			}

			return _cachedAgents.Values.ToList();
		}

		/// <summary>
		/// Update the current workspaces for an agent.
		/// </summary>
		/// <param name="agent">The agent to update</param>
		/// <param name="workspaces">Current list of workspaces</param>
		/// <param name="pendingConform">Whether the agent still needs to run another conform</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state</returns>
		public async Task<bool> TryUpdateWorkspacesAsync(IAgent agent, List<AgentWorkspaceInfo> workspaces, bool pendingConform, CancellationToken cancellationToken)
		{
			_ = this;

			IAgent? newAgent = await agent.TryUpdateWorkspacesAsync(workspaces, pendingConform, cancellationToken);
			return newAgent != null;
		}

		/// <summary>
		/// Marks the agent as deleted
		/// </summary>
		/// <param name="agent">The agent to delete</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		public async Task DeleteAgentAsync(IAgent? agent, CancellationToken cancellationToken = default)
		{
			if (agent == null)
			{
				return;
			}
			
			int maxRetries = 10;
			int retryCount = 0;
			while (agent is { Deleted: false })
			{
				IAgent? newAgent = await agent.TryDeleteAsync(cancellationToken);
				if (newAgent != null)
				{
					break;
				}
				retryCount++;
				if (retryCount >= maxRetries)
				{
					_logger.LogError("Unable to delete agent {AgentId} after {MaxRetries} retry attempts", agent.Id, maxRetries);
					throw new InvalidOperationException($"Unable to delete agent {agent.Id} after {maxRetries} retry attempts");
				}
				
				await Task.Delay(100, cancellationToken);
				agent = await GetAgentAsync(agent.Id, cancellationToken);
			}
		}

		async ValueTask<List<PoolId>> GetDynamicPoolsAsync(IAgent agent, CancellationToken cancellationToken)
		{
			List<PoolId> newDynamicPools = new List<PoolId>();

			IReadOnlyList<IPoolConfig> pools = await _cachedPools.GetAsync(cancellationToken);
			foreach (IPoolConfig pool in pools)
			{
				if (pool.Condition != null && agent.SatisfiesCondition(pool.Condition))
				{
					newDynamicPools.Add(pool.Id);
				}
			}

			return newDynamicPools;
		}

		/// <summary>
		/// Creates a new agent session
		/// </summary>
		/// <param name="agent">The agent to create a session for</param>
		/// <param name="capabilities">Capabilities for the agent</param>
		/// <param name="version">Version of the software that's running</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state</returns>
		public async Task<IAgent> CreateSessionAsync(IAgent agent, RpcAgentCapabilities capabilities, string? version, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				IAuditLogChannel<AgentId> agentLogger = Agents.GetLogger(agent.Id);

				// Check if there's already a session running for this agent.
				IAgent? newAgent;
				if (agent.SessionId != null)
				{
					// Try to terminate the current session
					await TryTerminateSessionAsync(agent, cancellationToken);
				}
				else
				{
					// Get the new pools for the agent
					List<PoolId> dynamicPools = await GetDynamicPoolsAsync(agent, cancellationToken);
					RpcAgentCapabilities normalizedCaps = NormalizeCapabilities(agent, capabilities);
					
					// Reset the agent to use the new session
					newAgent = await agent.TryCreateSessionAsync(new CreateSessionOptions(normalizedCaps, dynamicPools, version), cancellationToken);
					if (newAgent != null)
					{
						LogPropertyChanges(agentLogger, agent.Properties, newAgent.Properties);
						agent = newAgent;
						agentLogger.LogInformation("Session {SessionId} started", newAgent.SessionId);
						break;
					}
				}

				// Get the current agent state
				newAgent = await GetAgentAsync(agent.Id, cancellationToken);
				if (newAgent == null)
				{
					throw new InvalidOperationException($"Invalid agent id '{agent.Id}'");
				}
				agent = newAgent;
			}
			return agent;
		}
		
		/// <summary>
		/// Filters untrusted resource and property keys from capabilities
		/// Also injects potential server-controlled properties
		/// </summary>
		/// <param name="agent">Agent</param>
		/// <param name="capabilities">Capabilities as reported by agent</param>
		/// <returns>A new and updated RpcAgentCapabilities object</returns>
		private static RpcAgentCapabilities NormalizeCapabilities(IAgent agent, RpcAgentCapabilities capabilities)
		{
			RpcAgentCapabilities caps = new (capabilities);
			caps.Properties.Clear();
			caps.Properties.AddRange(agent.ServerDefinedProperties);
			caps.Properties.AddRange(AgentExtensions.RemoveServerDefinedProperties(capabilities.Properties));
			return caps;
		}

		/// <summary>
		/// Determines whether a task source can currently issue tasks
		/// </summary>
		bool CanUseTaskSource(IAgent agent, ITaskSource taskSource)
		{
			TaskSourceFlags flags = taskSource.Flags;
			if ((flags & TaskSourceFlags.AllowWhenBusy) == 0 && agent.Status == AgentStatus.Busy)
			{
				return false;
			}
			if ((flags & TaskSourceFlags.AllowWhenDisabled) == 0 && !agent.Enabled)
			{
				return false;
			}
			if ((flags & TaskSourceFlags.AllowDuringDowntime) == 0 && _downtimeService.IsDowntimeActive)
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Compare properties and write changes to audit log
		/// </summary>
		private static void LogPropertyChanges(IAuditLogChannel<AgentId> agentLogger, IReadOnlyList<string> before, IReadOnlyList<string> after)
		{
			const string AwsInstanceTypeKey = KnownPropertyNames.AwsInstanceType + "=";
			string beforeProp = before.FirstOrDefault(x => x.StartsWith(AwsInstanceTypeKey, StringComparison.Ordinal), String.Empty);
			string afterProp = after.FirstOrDefault(x => x.StartsWith(AwsInstanceTypeKey, StringComparison.Ordinal), String.Empty);

			if (!String.IsNullOrEmpty(beforeProp) && !String.IsNullOrEmpty(afterProp) && beforeProp != afterProp)
			{
				string oldInstanceType = beforeProp.Replace(AwsInstanceTypeKey, "", StringComparison.Ordinal);
				string newInstanceType = afterProp.Replace(AwsInstanceTypeKey, "", StringComparison.Ordinal);
				agentLogger.LogInformation("AWS EC2 instance type changed from {OldInstanceType} to {NewInstanceType}", oldInstanceType, newInstanceType);
			}
		}

		/// <summary>
		/// Waits for a lease to be assigned to an agent
		/// </summary>
		/// <param name="agent">The agent to assign a lease to</param>
		/// <param name="newLeases">Leases that the agent knows about</param>
		/// <param name="cancellationToken"></param>
		/// <returns>True if a lease was assigned, false otherwise</returns>
		public async Task<IAgent?> WaitForLeaseAsync(IAgent? agent, IList<HordeCommon.Rpc.Messages.RpcLease> newLeases, CancellationToken cancellationToken = default)
		{
			HashSet<LeaseId> knownLeases = new HashSet<LeaseId>(newLeases.Select(x => x.Id));
			while (agent != null && agent.Leases.All(x => knownLeases.Contains(x.Id)))
			{
				if (!agent.SessionExpiresAt.HasValue)
				{
					break;
				}

				// Check we have some time to wait
				DateTime utcNow = _clock.UtcNow;
				TimeSpan maxWaitTime = (agent.SessionExpiresAt.Value - SessionExpiryTime + SessionLongPollTime) - utcNow;
				if (maxWaitTime <= TimeSpan.Zero)
				{
					break;
				}

				// Create a cancellation token that will expire with the session
				using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(_applicationLifetime.ApplicationStopping, cancellationToken);
				cancellationSource.CancelAfter(maxWaitTime);

				// If an agent changes, cancel the update
				_ = agent.WaitForUpdateAsync(cancellationSource.Token).ContinueWith(x => cancellationSource.Cancel(), cancellationSource.Token, TaskContinuationOptions.None, TaskScheduler.Default);

				// Create all the tasks to wait for
				List<Task<(ITaskSource, CreateLeaseOptions)?>> tasks = new List<Task<(ITaskSource, CreateLeaseOptions)?>>();
				foreach (ITaskSource taskSource in _taskSources)
				{
					if (CanUseTaskSource(agent, taskSource) && !cancellationSource.IsCancellationRequested)
					{
						Task<(ITaskSource, CreateLeaseOptions)?> task = await GuardedAssignLeaseAsync(taskSource, agent, cancellationSource);
						tasks.Add(task);
					}
				}

				// If no task source is valid, just add a delay
				if (tasks.Count == 0)
				{
					_logger.LogInformation("No task source valid for agent {AgentId}. Waiting {WaitTimeMs} ms", agent.Id, maxWaitTime.TotalMilliseconds);
					await AsyncUtils.DelayNoThrow(maxWaitTime, cancellationToken);
					break;
				}

				// Wait for all the tasks to complete. Once the first task completes it will set the cancellation source, triggering the 
				// others to terminate.
				await Task.WhenAll(tasks);

				// Find the first result
				(ITaskSource, CreateLeaseOptions)? result = null;
				foreach (Task<(ITaskSource, CreateLeaseOptions)?> task in tasks)
				{
					(ITaskSource, CreateLeaseOptions)? taskResult;
					if (task.TryGetResult(out taskResult) && taskResult != null)
					{
						(ITaskSource taskSource, CreateLeaseOptions taskLease) = taskResult.Value;
						if (result == null)
						{
							result = (taskSource, taskLease);
						}
						else
						{
							await taskSource.CancelLeaseAsync(agent, taskLease.Id, Any.Pack(taskLease.Payload), CancellationToken.None);
						}
					}
				}

				// Exit if we didn't find any work to do. It may be that all the task sources returned null, in which case wait for the time period to expire.
				if (result == null)
				{
					if (!cancellationSource.IsCancellationRequested)
					{
						await cancellationSource.Token.AsTask();
					}
					break;
				}

				// Get the resulting lease
				(ITaskSource source, CreateLeaseOptions lease) = result.Value;

				// Add the new lease to the agent
				IAgent? newAgent = await agent.TryCreateLeaseAsync(lease, cancellationToken);
				if (newAgent != null)
				{
					await source.OnLeaseStartedAsync(newAgent, lease.Id, Any.Pack(lease.Payload), Agents.GetLogger(agent.Id), CancellationToken.None);
					return newAgent;
				}
				else
				{
					_logger.LogInformation("Failed adding lease {LeaseId} for agent {AgentId}", lease.Id.ToString(), agent.Id.ToString());
					await source.CancelLeaseAsync(agent, lease.Id, Any.Pack(lease.Payload), CancellationToken.None);
				}

				// Update the agent
				agent = await GetAgentAsync(agent.Id, cancellationToken);
			}
			return agent;
		}

		async Task<Task<(ITaskSource, CreateLeaseOptions)?>> GuardedAssignLeaseAsync(ITaskSource source, IAgent agent, CancellationTokenSource cancellationSource)
		{
			CancellationToken cancellationToken = cancellationSource.Token;
			try
			{
				Task<CreateLeaseOptions?> task = await source.AssignLeaseAsync(agent, cancellationToken);
				return task.ContinueWith(x => WrapAssignedLease(source, x, cancellationSource), TaskScheduler.Default);
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				return Task.FromResult<(ITaskSource, CreateLeaseOptions)?>(null);
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while trying to assign lease");
				return Task.FromResult<(ITaskSource, CreateLeaseOptions)?>(null);
			}
		}

		(ITaskSource, CreateLeaseOptions)? WrapAssignedLease(ITaskSource source, Task<CreateLeaseOptions?> task, CancellationTokenSource cancellationSource)
		{
			if (task.IsCanceled)
			{
				return null;
			}
			else if (task.TryGetResult(out CreateLeaseOptions? lease))
			{
				if (lease == null)
				{
					return null;
				}
				else
				{
					cancellationSource.Cancel();
					return (source, lease);
				}
			}
			else if (task.IsFaulted)
			{
				_logger.LogError(task.Exception, "Exception while trying to assign lease");
				return null;
			}
			else
			{
				_logger.LogWarning("Unhandled task state: {Status}", task.Status);
				return null;
			}
		}

		/// <summary>
		/// Cancels the specified agent lease
		/// </summary>
		/// <param name="lease">The lease to cancel</param>
		/// <param name="outcomeReason">Reason for the cancellation</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<bool> CancelLeaseAsync(ILease lease, LeaseOutcomeReason outcomeReason, CancellationToken cancellationToken)
		{
			// If it's already complete, don't do anything
			if (lease.FinishTime != null)
			{
				return false;
			}

			// Cancel it on the agent that's currently executing it
			IAgent? agent = await GetAgentAsync(lease.AgentId, cancellationToken);
			if (agent == null)
			{
				_logger.LogWarning("Forcing lease {LeaseId} to cancelled state; agent {AgentId} not found", lease.Id, lease.AgentId);
				await _leases.TrySetOutcomeAsync(lease.Id, _clock.UtcNow, LeaseOutcome.Cancelled, outcomeReason, null, null, cancellationToken);
				return false;
			}

			// Find the index of the lease to remove
			int index = agent.Leases.FindIndex(x => x.Id == lease.Id);
			if (index == -1)
			{
				_logger.LogWarning("Forcing lease {LeaseId} to cancelled state; no longer running on agent {AgentId}", lease.Id, lease.AgentId);
				await _leases.TrySetOutcomeAsync(lease.Id, _clock.UtcNow, LeaseOutcome.Cancelled, outcomeReason, null, null, cancellationToken);
				return false;
			}

			// Update the agent
			await agent.TryCancelLeaseAsync(index, outcomeReason, cancellationToken);
			return true;
		}

		/// <summary>
		///
		/// </summary>
		public async Task<IAgent?> UpdateSessionWithWaitAsync(IAgent inAgent, SessionId sessionId, AgentStatus status, RpcAgentCapabilities? capabilities, IList<HordeCommon.Rpc.Messages.RpcLease> newLeases, RpcAgentCapabilities? capabilityOverrides = null, CancellationToken cancellationToken = default)
		{
			IAgent? agent = inAgent;

			// Capture the current agent update index. This allows us to detect if anything has changed.
			uint updateIndex = agent.UpdateIndex;

			// Update the agent session and return to the caller if anything changes
			agent = await UpdateSessionAsync(agent, sessionId, status, capabilities, newLeases, capabilityOverrides, cancellationToken);
			if (agent != null && agent.UpdateIndex == updateIndex && (agent.Leases.Count > 0 || agent.Status != AgentStatus.Stopping))
			{
				_logger.LogDebug("Waiting for lease update on agent {AgentId} session {SessionId}", agent.Id, sessionId);
				agent = await WaitForLeaseAsync(agent, newLeases, cancellationToken);
			}
			return agent;
		}

		/// <summary>
		/// Updates the state of the current agent session
		/// </summary>
		/// <param name="inAgent">The current agent state</param>
		/// <param name="sessionId">Id of the session</param>
		/// <param name="status">New status for the agent</param>
		/// <param name="capabilities">New agent capabilities</param>
		/// <param name="newLeases">New list of leases for this session</param>
		/// <param name="capabilityOverrides">Capability overrides to merge into existing agent capabilities</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Updated agent state</returns>
		public async Task<IAgent?> UpdateSessionAsync(IAgent inAgent, SessionId sessionId, AgentStatus status, RpcAgentCapabilities? capabilities, IList<HordeCommon.Rpc.Messages.RpcLease> newLeases, RpcAgentCapabilities? capabilityOverrides = null, CancellationToken cancellationToken = default)
		{
			DateTime utcNow = _clock.UtcNow;

			IAgent? agent = inAgent;
			while (agent != null)
			{
				// Check the session id is correct.
				if (agent.SessionId != sessionId)
				{
					if (status == AgentStatus.Stopping)
					{
						break; // Harmless; agent is not doing any work.
					}
					else
					{
						throw new InvalidOperationException($"Invalid agent session {sessionId}");
					}
				}

				// Check the session hasn't expired
				if (!agent.IsSessionValid(utcNow))
				{
					throw new InvalidOperationException("Session has already expired");
				}

				// Get the new dynamic pools for the agent
				List<PoolId> dynamicPools = await GetDynamicPoolsAsync(agent, cancellationToken);

				// Update the agent, and try to create new lease documents if we succeed
				IAgent? newAgent = await agent.TryUpdateSessionAsync(new UpdateSessionOptions(status, capabilities, dynamicPools, newLeases, CapabilityOverrides: capabilityOverrides), cancellationToken);
				if (newAgent != null)
				{
					agent = newAgent;
					break;
				}

				// Fetch the agent again
				agent = await GetAgentAsync(agent.Id, cancellationToken);
			}

			// If the agent is stopping, terminate the session
			while (agent != null && agent.Status == AgentStatus.Stopping && agent.Leases.Count == 0)
			{
				IAgent? terminatedAgent = await TryTerminateSessionAsync(agent, cancellationToken);
				if (terminatedAgent != null)
				{
					_logger.LogInformation("Terminated session {SessionId} for {AgentId}; agent is stopping", agent.SessionId, agent.Id);
					agent = terminatedAgent;
					break;
				}
				agent = await GetAgentAsync(agent.Id, cancellationToken);
			}

			return agent;
		}

		/// <summary>
		/// Terminates an existing session. Does not update the agent itself, if it's currently 
		/// </summary>
		/// <param name="agent">The agent whose current session should be terminated</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>An up-to-date IAgent if the session was terminated</returns>
		private static async Task<IAgent?> TryTerminateSessionAsync(IAgent agent, CancellationToken cancellationToken = default)
		{
			// Make sure the agent has a valid session id
			if (agent.SessionId == null)
			{
				return agent;
			}

			// Clear the current session
			return await agent.TryTerminateSessionAsync(cancellationToken);
		}

		/// <summary>
		/// Finds all leases matching a set of criteria
		/// </summary>
		/// <param name="agentId">Unqiue id of the agent executing this lease</param>
		/// <param name="sessionId">Unique id of the agent session</param>
		/// <param name="startTime">Start of the search window to return results for</param>
		/// <param name="finishTime">End of the search window to return results for</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases matching the given criteria</returns>
		public Task<IReadOnlyList<ILease>> FindLeasesAsync(AgentId? agentId, SessionId? sessionId, DateTime? startTime, DateTime? finishTime, int index, int count, CancellationToken cancellationToken = default)
		{
			return _leases.FindLeasesAsync(null, agentId, sessionId, startTime, finishTime, index, count, cancellationToken: cancellationToken);
		}

		/// <summary>
		/// Finds all leases by finish time
		/// </summary>
		/// <param name="minFinishTime">Start of the search window to return results for</param>
		/// <param name="maxFinishTime">End of the search window to return results for</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases matching the given criteria</returns>
		public Task<IReadOnlyList<ILease>> FindLeasesByFinishTimeAsync(DateTime? minFinishTime, DateTime? maxFinishTime, int? index, int? count, CancellationToken cancellationToken = default)
		{
			return _leases.FindLeasesByFinishTimeAsync(minFinishTime, maxFinishTime, index, count, null, false, cancellationToken);
		}

		/// <summary>
		/// Gets a specific lease
		/// </summary>
		/// <param name="leaseId">Unique id of the lease</param>
		/// <param name="cancellationToken"></param>
		/// <returns>The lease that was found, or null if it does not exist</returns>
		public Task<ILease?> GetLeaseAsync(LeaseId leaseId, CancellationToken cancellationToken = default)
		{
			return _leases.GetAsync(leaseId, cancellationToken);
		}

		/// <summary>
		/// Gets the rate for the given agent
		/// </summary>
		/// <param name="agentId">Agent id to query</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Hourly rate of running the given agent</returns>
		public async ValueTask<double?> GetRateAsync(AgentId agentId, CancellationToken cancellationToken = default)
		{
			RedisKey key = $"compute:agent-rates:{agentId}";

			// Try to get the current value
			RedisValue value = await _redisService.GetDatabase().StringGetAsync(key).WaitAsync(cancellationToken);
			if (!value.IsNull)
			{
				double rate = (double)value;
				if (rate == 0.0)
				{
					return null;
				}
				else
				{
					return rate;
				}
			}
			else
			{
				double rate = 0.0;

				// Get the rate table
				IAgent? agent = await GetAgentAsync(agentId, cancellationToken);
				if (agent != null)
				{
					foreach (AgentRateConfig config in _computeConfig.CurrentValue.Rates)
					{
						if (config.Condition != null && config.Condition.Evaluate(x => agent.GetPropertyValues(x)))
						{
							rate = config.Rate;
							break;
						}
					}
				}

				// Cache it for future reference
				await _redisService.GetDatabase().StringSetAsync(key, rate, TimeSpan.FromMinutes(5.0), flags: CommandFlags.FireAndForget).WaitAsync(cancellationToken);
				return rate;
			}
		}

		internal async ValueTask TickAsync(CancellationToken stoppingToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentService)}.{nameof(TickAsync)}");
			await RefreshCachedAgentsAsync(stoppingToken);
			await CollectMetricsAsync(stoppingToken);
		}

		/// <summary>
		/// Updates the in-memory cache of current agents
		/// Called by ticker to avoid blocking reads. Updates in the background, in favor of slightly more stale agents
		/// </summary>
		/// <param name="cancellationToken">Cancellation token</param>
		public async Task RefreshCachedAgentsAsync(CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentService)}.{nameof(RefreshCachedAgentsAsync)}");

			Dictionary<AgentId, IAgent> agents = new();
			await foreach (IAgent agent in Agents.FindAsync(consistentRead: false, cancellationToken: cancellationToken))
			{
				agents[agent.Id] = agent;
			}

			Interlocked.Exchange(ref _cachedAgents, agents);
		}

		private async Task CollectMetricsAsync(CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentService)}.{nameof(CollectMetricsAsync)}");

			IReadOnlyList<IAgent> agentList = await GetCachedAgentsAsync(cancellationToken);
			int numAgentsTotal = agentList.Count;
			int numAgentsTotalDeleted = agentList.Count(a => a.Deleted);
			int numAgentsTotalEnabled = agentList.Count(a => a.Enabled);
			int numAgentsTotalDisabled = agentList.Count(a => !a.Enabled);
			int numAgentsTotalOk = agentList.Count(a => a.Enabled && a.Status == AgentStatus.Ok);
			int numAgentsTotalStopping = agentList.Count(a => a.Enabled && a.Status == AgentStatus.Stopping);
			int numAgentsTotalUnhealthy = agentList.Count(a => a.Enabled && a.Status == AgentStatus.Unhealthy);
			int numAgentsTotalBusy = agentList.Count(a => a.Enabled && a.Status == AgentStatus.Busy);
			int numAgentsTotalUnspecified = agentList.Count(a => a.Enabled && a.Status == AgentStatus.Unspecified);

			List<Measurement<int>> newMeasurements =
			[
				new Measurement<int>(numAgentsTotal),
				new Measurement<int>(numAgentsTotalDeleted, new KeyValuePair<string, object?>("status", "deleted")),
				new Measurement<int>(numAgentsTotalEnabled, new KeyValuePair<string, object?>("status", "enabled")),
				new Measurement<int>(numAgentsTotalDisabled, new KeyValuePair<string, object?>("status", "disabled")),
				new Measurement<int>(numAgentsTotalOk, new KeyValuePair<string, object?>("status", "ok")),
				new Measurement<int>(numAgentsTotalStopping, new KeyValuePair<string, object?>("status", "stopping")),
				new Measurement<int>(numAgentsTotalUnhealthy, new KeyValuePair<string, object?>("status", "unhealthy")),
				new Measurement<int>(numAgentsTotalBusy, new KeyValuePair<string, object?>("status", "paused")),
				new Measurement<int>(numAgentsTotalUnspecified, new KeyValuePair<string, object?>("status", "unspecified"))
			];

			_measurements = newMeasurements;
		}

		/// <summary>
		/// Determines if the user is authorized to perform an action on a particular agent
		/// </summary>
		/// <param name="agent">The agent to check</param>
		/// <param name="user">The principal to authorize</param>
		/// <param name="reason">Reason for being authorized or not</param>
		/// <returns>True if the action is authorized</returns>
		public bool AuthorizeSession(IAgent agent, ClaimsPrincipal user, out string reason)
		{
			if (agent.SessionId == null)
			{
				reason = $"{nameof(agent.SessionId)} is null";
				return false;
			}

			if (!user.HasSessionClaim(agent.SessionId.Value))
			{
				reason = $"Missing session claim for {agent.SessionId.Value}";
				return false;
			}

			if (!agent.IsSessionValid(_clock.UtcNow))
			{
				reason = $"Session has expired";
				return false;
			}

			reason = "Session is valid";
			return true;
		}
	}
}
