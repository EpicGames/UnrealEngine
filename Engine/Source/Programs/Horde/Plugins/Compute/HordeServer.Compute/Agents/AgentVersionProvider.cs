// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Tools;
using HordeServer.Server;

namespace HordeServer.Agents
{
	/// <summary>
	/// Advertises the version number of the agent tool for the /api/v1/server/info endpoint
	/// </summary>
	class AgentVersionProvider : IAgentVersionProvider
	{
		readonly IToolCollection _toolCollection;
		readonly IClock _clock;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentVersionProvider(IToolCollection toolCollection, IClock clock)
		{
			_toolCollection = toolCollection;
			_clock = clock;
		}

		/// <inheritdoc/>
		public async Task<string?> GetAsync(CancellationToken cancellationToken = default)
		{
			ITool? tool = await _toolCollection.GetAsync(AgentExtensions.AgentToolId, cancellationToken);
			if (tool != null)
			{
				IToolDeployment? deployment = tool.GetCurrentDeployment(1.0, _clock.UtcNow);
				if (deployment != null)
				{
					return deployment.Version;
				}
			}
			return null;
		}
	}
}
