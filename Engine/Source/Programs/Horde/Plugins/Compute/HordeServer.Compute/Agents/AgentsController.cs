// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Common;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Sessions;
using HordeServer.Agents.Telemetry;
using HordeServer.Auditing;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents
{
	/// <summary>
	/// Controller for the /api/v1/agents endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	[Route("[controller]")]
	public class AgentsController : HordeControllerBase
	{
		readonly AgentService _agentService;
		readonly IAgentTelemetryCollection _agentTelemetryCollection;
		readonly ILeaseCollection _leaseCollection;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;
		readonly Tracer _tracer;
		readonly ILogger<AgentsController> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentsController(AgentService agentService, IAgentTelemetryCollection agentTelemetryCollection, ILeaseCollection leaseCollection, IOptionsSnapshot<ComputeConfig> computeConfig, Tracer tracer, ILogger<AgentsController> logger)
		{
			_agentService = agentService;
			_agentTelemetryCollection = agentTelemetryCollection;
			_leaseCollection = leaseCollection;
			_computeConfig = computeConfig;
			_tracer = tracer;
			_logger = logger;
		}

		/// <summary>
		/// Finds the agents matching specified criteria.
		/// </summary>
		/// <param name="poolId">The pool containing the agent</param>
		/// <param name="condition">Arbitrary condition to evaluate against the agents</param>
		/// <param name="includeDeleted">Whether to include agents marked as deleted</param>
		/// <param name="invalidateCache">Whether to refresh the cached agents</param>
		/// <param name="index">First result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="modifiedAfter">If set, only returns agents modified after this time</param>
		/// <param name="propsFilter">If set, what keys to include in the list of agent properties. Separated by commas.</param>
		/// <param name="filter">If set, filter for fields/properties to return</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/agents")]
		[ProducesResponseType(typeof(List<GetAgentResponse>), 200)]
		public async Task<ActionResult<List<object>>> FindAgentsAsync(
			[FromQuery] PoolId? poolId = null,
			[FromQuery] Condition? condition = null,
			[FromQuery] bool includeDeleted = false,
			[FromQuery] bool invalidateCache = false,
			[FromQuery] int? index = null,
			[FromQuery] int? count = null,
			[FromQuery] DateTimeOffset? modifiedAfter = null,
			[FromQuery] string? propsFilter = null,
			[FromQuery] PropertyFilter? filter = null)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentsController)}.{nameof(FindAgentsAsync)}");

			if (!_computeConfig.Value.Authorize(AgentAclAction.ListAgents, User))
			{
				return Forbid(AgentAclAction.ListAgents);
			}

			if (invalidateCache)
			{
				await _agentService.RefreshCachedAgentsAsync(HttpContext.RequestAborted);
			}

			IEnumerable<IAgent> agentsEnumerable = (await _agentService.GetCachedAgentsAsync(HttpContext.RequestAborted))
				.Where(x => poolId == null || x.Pools.Contains(poolId.Value))
				.Where(x => modifiedAfter == null || modifiedAfter.Value.UtcDateTime >= x.UpdateTime);

			if (!includeDeleted)
			{
				agentsEnumerable = agentsEnumerable.Where(x => !x.Deleted);
			}

			agentsEnumerable = index != null ? agentsEnumerable.Skip(index.Value) : agentsEnumerable;
			agentsEnumerable = count != null ? agentsEnumerable.Take(count.Value) : agentsEnumerable;

			List<IAgent> agents = agentsEnumerable.ToList();
			List<object> responses = [];
			using TelemetrySpan filterSpan = _tracer.StartActiveSpan($"FilterAgentResponses");
			{
				foreach (IAgent agent in agents)
				{
					if (condition == null || agent.SatisfiesCondition(condition))
					{
						responses.Add(await GetAgentResponseAsync(agent, propsFilter, filter));
					}
				}
			}

			span.SetAttribute("NumAgents", agents.Count);
			span.SetAttribute("NumResponses", responses.Count);
			return responses;
		}

		/// <summary>
		/// Retrieve information about a specific agent
		/// </summary>
		/// <param name="agentId">Id of the agent to get information about</param>
		/// <param name="filter">Filter for the properties to return</param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}")]
		[ProducesResponseType(typeof(GetAgentResponse), 200)]
		public async Task<ActionResult<object>> GetAgentAsync(AgentId agentId, [FromQuery] PropertyFilter? filter = null)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.ViewAgent, User))
			{
				return Forbid(AgentAclAction.ViewAgent, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			return await GetAgentResponseAsync(agent, null, filter);
		}

		/// <summary>
		/// Gets an individual agent response
		/// </summary>
		async ValueTask<object> GetAgentResponseAsync(IAgent agent, string? propFilter = null, PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			double? rate = null;
			if (_computeConfig.Value.Authorize(ServerAclAction.ViewCosts, User))
			{
				rate = await _agentService.GetRateAsync(agent.Id, cancellationToken);
			}

			List<GetAgentLeaseResponse> leases = new List<GetAgentLeaseResponse>();
			foreach (IAgentLease agentLease in agent.Leases)
			{
				try
				{
					ILease? lease = await _leaseCollection.GetAsync(agentLease.Id, cancellationToken);
					if (lease != null)
					{
						Dictionary<string, string>? details = await _agentService.GetPayloadDetailsAsync(lease.Payload, cancellationToken);
						leases.Add(CreateGetAgentLeaseResponse(lease, details, null));
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Failed getting payload details for agent lease {LeaseId}", agentLease.Id);
				}
			}

			return CreateGetAgentResponse(agent, leases, rate, propFilter).ApplyFilter(filter);
		}

		internal static GetAgentLeaseResponse CreateGetAgentLeaseResponse(ILease lease, Dictionary<string, string>? details, double? agentRate)
		{
			string[] outcomeReasons = ToFlagNames(lease.OutcomeReason);
			return new GetAgentLeaseResponse(
				lease.Id, lease.ParentId, lease.AgentId, agentRate, lease.Name, lease.LogId, lease.StartTime,
				lease.FinishTime, lease.FinishTime == null, details, lease.Outcome, outcomeReasons, (long)lease.OutcomeReason, null);
		}
		
		private static string[] ToFlagNames(LeaseOutcomeReason reason)
		{
			if (reason == LeaseOutcomeReason.None)
			{
				return [];
			}
    
			return Enum.GetValues(typeof(LeaseOutcomeReason))
				.Cast<LeaseOutcomeReason>()
				.Where(flag => flag != LeaseOutcomeReason.None && reason.HasFlag(flag))
				.Select(flag => flag.ToString())
				.ToArray();
		}
		
		private static List<string> FilterProperties(List<string> properties, string? filter)
		{
			if (filter == null)
			{
				return properties;
			}
			
			List<string> filteredProps = new (properties.Count);
			List<string> includedKeys = filter.Split(",").Select(x => x.ToUpperInvariant()).ToList();
			foreach (string property in properties)
			{
				string[] parts = property.Split("=");
				string key = parts[0].ToUpperInvariant();
				foreach (string includedKey in includedKeys)
				{
					if (key == includedKey)
					{
						filteredProps.Add(property);
					}
				}
			}
			
			return filteredProps;
		}

		static GetAgentResponse CreateGetAgentResponse(IAgent agent, List<GetAgentLeaseResponse> leases, double? rate, string? propFilter)
		{
			return new GetAgentResponse(
				agent.Id,
				agent.Id.ToString(),
				agent.Enabled,
				agent.Status,
				rate,
				agent.SessionId,
				agent.Ephemeral,
				agent.IsSessionValid(DateTime.UtcNow),
				agent.Deleted,
				agent.RequestConform,
				agent.RequestFullConform,
				agent.RequestRestart,
				agent.RequestShutdown,
				agent.LastShutdownReason ?? "Unknown",
				agent.LastConformTime,
				agent.ConformAttemptCount,
				agent.LastConformTime,
				agent.Version?.ToString() ?? "Unknown",
				FilterProperties(new List<string>(agent.Properties), propFilter),
				new Dictionary<string, int>(agent.Resources),
				agent.UpdateTime,
				agent.LastOnlineTime,
				agent.Pools.Select(x => x.ToString()).ToList(),
				new { Devices = new[] { new { agent.Properties, agent.Resources } } },
				leases,
				agent.Workspaces.ConvertAll(x => CreateGetAgentWorkspaceResponse(x)),
				agent.Comment);
		}

		internal static GetAgentWorkspaceResponse CreateGetAgentWorkspaceResponse(AgentWorkspaceInfo workspace)
		{
			return new GetAgentWorkspaceResponse(workspace.Cluster, workspace.UserName, workspace.Identifier, workspace.Stream, workspace.View, workspace.Incremental, workspace.Method, workspace.MinScratchSpace, workspace.ConformDiskFreeSpace);
		}

		/// <summary>
		/// Update an agent's properties.
		/// </summary>
		/// <param name="agentId">Id of the agent to update.</param>
		/// <param name="update">Properties on the agent to update.</param>
		/// <returns>Http result code</returns>
		[HttpPut]
		[Route("/api/v1/agents/{agentId}")]
		public async Task<ActionResult> UpdateAgentAsync(AgentId agentId, [FromBody] UpdateAgentRequest update)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentsController)}.{nameof(UpdateAgentAsync)}");
			
			if (!_computeConfig.Value.Authorize(AgentAclAction.UpdateAgent, User))
			{
				return Forbid(AgentAclAction.UpdateAgent, agentId);
			}

			string userName = User.GetUser() ?? "Unknown";
			string? clientIp = HttpContext.Connection.RemoteIpAddress?.ToString();
			span.SetAttribute("claims", User.Claims.Select(x => x.ToString()).ToArray());

			for (; ; )
			{
				IAgent? agent = await _agentService.GetAgentAsync(agentId);
				if (agent == null)
				{
					return NotFound(agentId);
				}

				IAgent? newAgent = await agent.TryUpdateAsync(new UpdateAgentOptions(update.Enabled, update.RequestConform, update.RequestFullConform, update.RequestRestart, update.RequestShutdown, update.RequestForceRestart, $"Manual ({userName})", update.Pools?.ConvertAll(x => new PoolId(x)), update.Comment));
				if (newAgent == null)
				{
					continue;
				}

				IAuditLogChannel<AgentId> logger = _agentService.Agents.GetLogger(agent.Id);
				if (agent.Enabled != newAgent.Enabled)
				{
					logger.LogInformation("Setting changed: Enabled = {State} ({UserName} via IP {Ip})", newAgent.Enabled, userName, clientIp);
				}
				if (agent.RequestConform != newAgent.RequestConform)
				{
					logger.LogInformation("Setting changed: RequestConform = {State} ({UserName} via IP {Ip})", newAgent.RequestConform, userName, clientIp);
				}
				if (agent.RequestFullConform != newAgent.RequestFullConform)
				{
					logger.LogInformation("Setting changed: RequestFullConform = {State} ({UserName} via IP {Ip})", newAgent.RequestFullConform, userName, clientIp);
				}
				if (agent.RequestRestart != newAgent.RequestRestart)
				{
					logger.LogInformation("Setting changed: RequestRestart = {State} ({UserName} via IP {Ip})", newAgent.RequestRestart, userName, clientIp);
				}
				if (agent.RequestShutdown != newAgent.RequestShutdown)
				{
					logger.LogInformation("Setting changed: RequestShutdown = {State} ({UserName} via IP {Ip})", newAgent.RequestShutdown, userName, clientIp);
				}
				if (agent.Comment != newAgent.Comment)
				{
					logger.LogInformation("Setting changed: Comment = \"{Comment}\" ({UserName} via IP {Ip})", update.Comment, userName, clientIp);
				}
				foreach (PoolId addedPool in newAgent.ExplicitPools.Except(agent.ExplicitPools))
				{
					logger.LogInformation("Added to pool {PoolId} ({UserName} via IP {Ip})", addedPool, userName, clientIp);
				}
				foreach (PoolId removedPool in agent.ExplicitPools.Except(newAgent.ExplicitPools))
				{
					logger.LogInformation("Removed from pool {PoolId} ({UserName} via IP {Ip})", removedPool, userName, clientIp);
				}
				break;
			}
			return Ok();
		}

		/// <summary>
		/// Remove a registered agent.
		/// </summary>
		/// <param name="agentId">Id of the agent to delete.</param>
		/// <returns>Http result code</returns>
		[HttpDelete]
		[Route("/api/v1/agents/{agentId}")]
		public async Task<ActionResult> DeleteAgentAsync(AgentId agentId)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.DeleteAgent, User))
			{
				return Forbid(AgentAclAction.DeleteAgent, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			await _agentService.DeleteAgentAsync(agent);
			return new OkResult();
		}

		/// <summary>
		/// Retrieve historical information about a specific agent
		/// </summary>
		/// <param name="agentId">Id of the agent to get information about</param>
		/// <param name="minTime">Minimum time for records to return</param>
		/// <param name="maxTime">Maximum time for records to return</param>
		/// <param name="index">Offset of the first result</param>
		/// <param name="count">Number of records to return</param>
		/// <returns>Information about the requested agent</returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/history")]
		public async Task<ActionResult> GetAgentHistoryAsync(AgentId agentId, [FromQuery] DateTime? minTime = null, [FromQuery] DateTime? maxTime = null, [FromQuery] int index = 0, [FromQuery] int count = 50)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.ViewAgent, User))
			{
				return Forbid(AgentAclAction.ViewAgent, agentId);
			}

			Response.ContentType = "application/json";
			Response.StatusCode = 200;
			await Response.StartAsync();
			await _agentService.Agents.GetLogger(agentId).FindAsync(Response.BodyWriter, minTime, maxTime, index, count);
			await Response.CompleteAsync();

			return Empty;
		}

		/// <summary>
		/// Find all the sessions of a particular agent
		/// </summary>
		/// <param name="agentId">Unique id of the agent to find</param>
		/// <param name="startTime">Start time to include in the search</param>
		/// <param name="finishTime">Finish time to include in the search</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <returns>Sessions </returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/sessions")]
		public async Task<ActionResult<List<GetSessionResponse>>> FindSessionsAsync(AgentId agentId, [FromQuery] DateTimeOffset? startTime, [FromQuery] DateTimeOffset? finishTime, [FromQuery] int index = 0, [FromQuery] int count = 50)
		{
			if (!_computeConfig.Value.Authorize(SessionAclAction.ViewSession, User))
			{
				return Forbid(SessionAclAction.ViewSession, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			IReadOnlyList<ISession> sessions = await agent.FindSessionsAsync(startTime?.UtcDateTime, finishTime?.UtcDateTime, index, count);
			return sessions.ConvertAll(x => CreateGetSessionResponse(x));
		}

		static GetSessionResponse CreateGetSessionResponse(ISession session)
		{
			return new GetSessionResponse(session.Id, session.StartTime, session.FinishTime, session.Version);
		}

		/// <summary>
		/// Find all the sessions of a particular agent
		/// </summary>
		/// <param name="agentId">Unique id of the agent to find</param>
		/// <param name="sessionId">Unique id of the session</param>
		/// <returns>Sessions </returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/sessions/{sessionId}")]
		public async Task<ActionResult<GetSessionResponse>> GetSessionAsync(AgentId agentId, SessionId sessionId)
		{
			if (!_computeConfig.Value.Authorize(SessionAclAction.ViewSession, User))
			{
				return Forbid(SessionAclAction.ViewSession, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			ISession? session = await agent.GetSessionAsync(sessionId, HttpContext.RequestAborted);
			if (session == null)
			{
				return NotFound();
			}

			return CreateGetSessionResponse(session);
		}

		/// <summary>
		/// Finds telemetry for an agent within a time range
		/// </summary>
		/// <param name="agentId">Agent identifier</param>
		/// <param name="minTime">Minimum time to return</param>
		/// <param name="maxTime">Maximum time to return</param>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/telemetry")]
		public async Task<ActionResult<GetAgentTelemetryResponse>> FindTelemetryAsync(AgentId agentId, [FromQuery] DateTime minTime, [FromQuery] DateTime maxTime)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.ViewAgent, User))
			{
				return Forbid(AgentAclAction.ViewAgent, agentId);
			}

			IReadOnlyList<IAgentTelemetry> telemetry = await _agentTelemetryCollection.FindAsync(agentId, minTime, maxTime, HttpContext.RequestAborted);

			List<GetAgentTelemetrySampleResponse> samples = new List<GetAgentTelemetrySampleResponse>();
			samples.AddRange(telemetry.Select(x => new GetAgentTelemetrySampleResponse(x.TimeUtc, x.UserCpu, x.IdleCpu, x.SystemCpu, x.FreeRam, x.UsedRam, x.TotalRam, x.WorkingFreeDisk, x.WorkingTotalDisk, x.SystemFreeDisk, x.SystemTotalDisk)));

			return new GetAgentTelemetryResponse(samples);
		}

		/// <summary>
		/// Find all the leases for a particular agent
		/// </summary>
		/// <param name="agentId">Unique id of the agent to find</param>
		/// <param name="sessionId">The session to query</param>
		/// <param name="startTime">Start of the time window to consider</param>
		/// <param name="finishTime">End of the time window to consider</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="filter">Filter to apply to the properties</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sessions </returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/leases")]
		[ProducesResponseType(200, Type = typeof(List<GetAgentLeaseResponse>))]
		public async Task<ActionResult<List<object>>> FindLeasesAsync(AgentId agentId, [FromQuery] SessionId? sessionId, [FromQuery] DateTimeOffset? startTime, [FromQuery] DateTimeOffset? finishTime, [FromQuery] int index = 0, [FromQuery] int count = 1000, [FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeases, User))
			{
				return Forbid(LeaseAclAction.ViewLeases, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId, cancellationToken);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			IReadOnlyList<ILease> leases = await _agentService.FindLeasesAsync(agentId, sessionId, startTime?.UtcDateTime, finishTime?.UtcDateTime, index, count, cancellationToken);

			double? agentRate = null;
			if (_computeConfig.Value.Authorize(ServerAclAction.ViewCosts, User))
			{
				agentRate = await _agentService.GetRateAsync(agentId, cancellationToken);
			}

			List<object> responses = new List<object>();
			foreach (ILease lease in leases)
			{
				Dictionary<string, string>? details = await _agentService.GetPayloadDetailsAsync(lease.Payload, cancellationToken);
				responses.Add(PropertyFilter.Apply(CreateGetAgentLeaseResponse(lease, details, agentRate), filter));
			}

			return responses;
		}

		/// <summary>
		/// Get info about a particular lease
		/// </summary>
		/// <param name="agentId">Unique id of the agent to find</param>
		/// <param name="leaseId">Unique id of the particular lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Lease matching the given id</returns>
		[HttpGet]
		[Route("/api/v1/agents/{agentId}/leases/{leaseId}")]
		public async Task<ActionResult<GetAgentLeaseResponse>> GetLeaseAsync(AgentId agentId, LeaseId leaseId, CancellationToken cancellationToken = default)
		{
			if (!_computeConfig.Value.Authorize(LeaseAclAction.ViewLeases, User))
			{
				return Forbid(LeaseAclAction.ViewLeases, agentId);
			}

			IAgent? agent = await _agentService.GetAgentAsync(agentId, cancellationToken);
			if (agent == null)
			{
				return NotFound(agentId);
			}

			ILease? lease = await _agentService.GetLeaseAsync(leaseId, cancellationToken);
			if (lease == null || lease.AgentId != agentId)
			{
				return NotFound(agentId, leaseId);
			}

			double? agentRate = null;
			if (_computeConfig.Value.Authorize(ServerAclAction.ViewCosts, User))
			{
				agentRate = await _agentService.GetRateAsync(agentId, cancellationToken);
			}

			Dictionary<string, string>? details = await _agentService.GetPayloadDetailsAsync(lease.Payload, cancellationToken);
			return CreateGetAgentLeaseResponse(lease, details, agentRate);
		}
	}
}
