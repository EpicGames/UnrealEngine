// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents.Enrollment;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.Options;

namespace HordeServer.Agents.Enrollment
{
	/// <summary>
	/// Controller for the /api/v1/enrollment endpoint
	/// </summary>
	[ApiController]
	[Authorize]
	public class EnrollmentController : HordeControllerBase
	{
		readonly EnrollmentService _registrationService;
		readonly IOptionsSnapshot<ComputeConfig> _computeConfig;

		/// <summary>
		/// Constructor
		/// </summary>
		public EnrollmentController(EnrollmentService registrationService, IOptionsSnapshot<ComputeConfig> computeConfig)
		{
			_registrationService = registrationService;
			_computeConfig = computeConfig;
		}

		/// <summary>
		/// Finds the agents matching specified criteria.
		/// </summary>
		/// <param name="filter">Filter for the properties to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of matching agents</returns>
		[HttpGet]
		[Route("/api/v1/enrollment")]
		[ProducesResponseType(typeof(GetPendingAgentsResponse), 200)]
		public async Task<ActionResult<object>> GetPendingAgentsAsync([FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.ListAgents, User))
			{
				return Forbid(AgentAclAction.ListAgents);
			}

			IReadOnlyList<EnrollmentRequest> requests = await _registrationService.FindAsync(cancellationToken);

			GetPendingAgentsResponse response = new GetPendingAgentsResponse(requests.ConvertAll(x => new GetPendingAgentResponse(x.Key, x.HostName, x.Description)));
			return PropertyFilter.Apply(response, filter);
		}

		/// <summary>
		/// Legacy endpoint for enrollment
		/// </summary>
		[HttpGet]
		[Route("/api/v1/registration")]
		[Obsolete("Use api/v1/enrollment instead")]
		[ProducesResponseType(typeof(GetPendingAgentsResponse), 200)]
		public Task<ActionResult<object>> LegacyGetPendingAgentsAsync([FromQuery] PropertyFilter? filter = null, CancellationToken cancellationToken = default)
		{
			return GetPendingAgentsAsync(filter, cancellationToken);
		}

		/// <summary>
		/// Retrieve information about a specific agent
		/// </summary>
		/// <param name="request">List of agents to approve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		[HttpPost]
		[Route("/api/v1/enrollment")]
		public async Task<ActionResult> ApproveAgentsAsync([FromBody] ApproveAgentsRequest request, CancellationToken cancellationToken)
		{
			if (!_computeConfig.Value.Authorize(AgentAclAction.CreateAgent, User))
			{
				return Forbid(AgentAclAction.CreateAgent);
			}

			foreach (ApproveAgentRequest agentRequest in request.Agents)
			{
				await _registrationService.ApproveAsync(agentRequest.Key, agentRequest.AgentId, cancellationToken);
			}

			return Ok();
		}

		/// <summary>
		/// Legacy registration method
		/// </summary>
		[HttpPost]
		[Route("/api/v1/registration")]
		[Obsolete("Use api/v1/enrollment instead")]
		public Task<ActionResult> LegacyApproveAgentsAsync([FromBody] ApproveAgentsRequest request, CancellationToken cancellationToken)
		{
			return ApproveAgentsAsync(request, cancellationToken);
		}
	}
}
