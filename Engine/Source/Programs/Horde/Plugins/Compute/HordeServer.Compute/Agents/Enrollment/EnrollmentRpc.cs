// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using Grpc.Core;
using Horde.Common.Rpc;
using HordeServer.Acls;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.Extensions.Logging;

namespace HordeServer.Agents.Enrollment
{
	/// <summary>
	/// Allow agents to register with the server
	/// </summary>
	[AllowAnonymous]
	public class EnrollmentRpc : Horde.Common.Rpc.EnrollmentRpc.EnrollmentRpcBase
	{
		readonly EnrollmentService _registrationService;
		readonly AgentService _agentService;
		readonly IAclService _aclService;
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public EnrollmentRpc(EnrollmentService registrationService, AgentService agentService, IAclService aclService, ILogger<EnrollmentRpc> logger)
		{
			_registrationService = registrationService;
			_agentService = agentService;
			_aclService = aclService;
			_logger = logger;
		}

		/// <inheritdoc/>
		public override async Task EnrollAgent(IAsyncStreamReader<EnrollAgentRequest> requestStream, IServerStreamWriter<EnrollAgentResponse> responseStream, ServerCallContext context)
		{
			Task<bool> nextRequestTask = requestStream.MoveNext();
			if (await nextRequestTask)
			{
				EnrollAgentRequest request = requestStream.Current;
				using IDisposable? scope = _logger.BeginScope("Attempting to register {HostName} with key {Key}", request.HostName, request.Key);

				using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(context.CancellationToken);
				nextRequestTask = requestStream.MoveNext(cancellationSource.Token);

				Task cancelTask = nextRequestTask.ContinueWith(task => cancellationSource.Cancel(), TaskScheduler.Current);
				try
				{
					AgentId agentId;
					try
					{
						agentId = await _registrationService.WaitForApprovalAsync(request.Key, request.HostName, request.Description, cancellationSource.Token);
					}
					catch (OperationCanceledException) when (nextRequestTask.IsCompleted)
					{
						return;
					}

					IAgent agent = await _agentService.CreateAgentAsync(new CreateAgentOptions(agentId, AgentMode.Dedicated, false, request.Key), cancellationSource.Token);

					List<AclClaimConfig> claims = new List<AclClaimConfig>();
					claims.Add(new AclClaimConfig(HordeClaimTypes.Agent, agentId.ToString()));
					claims.Add(new AclClaimConfig(HordeClaimTypes.AgentEnrollmentKey, request.Key));

					EnrollAgentResponse response = new EnrollAgentResponse();
					response.Id = agentId.ToString();
					response.Token = await _aclService.IssueBearerTokenAsync(claims, null, context.CancellationToken);

					await responseStream.WriteAsync(response);
				}
				finally
				{
					await cancellationSource.CancelAsync();
					await cancelTask.IgnoreCanceledExceptionsAsync();
				}
			}
		}
	}
}
