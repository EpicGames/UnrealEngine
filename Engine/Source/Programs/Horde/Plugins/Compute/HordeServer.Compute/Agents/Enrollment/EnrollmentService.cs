// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Redis;
using HordeServer.Server;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Options;
using StackExchange.Redis;

namespace HordeServer.Agents.Enrollment
{
	/// <summary>
	/// Information about a machine requesting to be registered
	/// </summary>
	[RedisConverter(typeof(RedisJsonConverter<>))]
	public record class EnrollmentRequest(string Key, string HostName, string Description);

	/// <summary>
	/// Service which tracks agents pending enrollment
	/// </summary>
	public sealed class EnrollmentService : IHostedService, IAsyncDisposable
	{
		readonly IRedisService _redisService;
		readonly IClock _clock;
		readonly RedisChannel _updateChannel;
		readonly RedisSortedSetKey<string> _keys = new("agents:registration:expire");
		readonly RedisHashKey<string, EnrollmentRequest> _requests = new("agents:registration:requests");
		readonly RedisHashKey<string, string> _approvals = new("agents:registration:approvals");
		readonly AsyncEvent _approvalEvent = new AsyncEvent();
		readonly IOptionsMonitor<ComputeServerConfig> _staticComputeConfig;

		RedisSubscription? _subscription;

		/// <summary>
		/// Constructor
		/// </summary>
		public EnrollmentService(IRedisService redisService, IClock clock, IOptionsMonitor<ComputeServerConfig> staticComputeConfig)
		{
			_redisService = redisService;
			_clock = clock;
			_updateChannel = RedisChannel.Literal("agents:registration");
			_staticComputeConfig = staticComputeConfig;
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await StopSubscriptionAsync();
		}

		static double GetTimestamp(DateTime time)
			=> (time - DateTime.UnixEpoch).TotalSeconds;

		/// <inheritdoc/>
		public async Task StartAsync(CancellationToken cancellationToken)
		{
			_subscription = await _redisService.GetDatabase().Multiplexer.SubscribeAsync(_updateChannel, _ => _approvalEvent.Set());
		}

		/// <inheritdoc/>
		public async Task StopAsync(CancellationToken cancellationToken)
		{
			await StopSubscriptionAsync();
		}

		async Task StopSubscriptionAsync()
		{
			if (_subscription != null)
			{
				await _subscription.DisposeAsync();
				_subscription = null;
			}
		}

		async Task ExpireKeysAsync(IDatabase redis, CancellationToken cancellationToken)
		{
			SortedSetEntry<string>[] entries = await redis.SortedSetRangeByScoreWithScoresAsync(_keys, stop: GetTimestamp(_clock.UtcNow));
			foreach (SortedSetEntry<string> entry in entries)
			{
				cancellationToken.ThrowIfCancellationRequested();

				ITransaction transaction = redis.CreateTransaction();
				_ = transaction.SortedSetRemoveRangeByScoreAsync(_keys, entry.Score, entry.Score);
				_ = transaction.HashDeleteAsync(_requests, entry.ElementValue!);
				_ = transaction.HashDeleteAsync(_approvals, entry.ElementValue!);
				await transaction.ExecuteAsync(CommandFlags.FireAndForget);
			}
		}

		/// <summary>
		/// Adds an agent as a candidate for the farm
		/// </summary>
		/// <param name="key">Unique id for the request</param>
		/// <param name="hostName">Name of the host</param>
		/// <param name="description">Description for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<bool> AddAsync(string key, string hostName, string description, CancellationToken cancellationToken)
		{
			IDatabase redis = _redisService.GetDatabase();
			await ExpireKeysAsync(redis, cancellationToken);

			ITransaction transaction = redis.CreateTransaction();
			_ = transaction.AddCondition(Condition.HashNotExists(_approvals.Inner, key));
			_ = transaction.SortedSetAddAsync(_keys, key, GetTimestamp(_clock.UtcNow + TimeSpan.FromMinutes(2.0)));
			_ = transaction.HashSetAsync(_requests, key, new EnrollmentRequest(key, hostName, description));

			if (!await transaction.ExecuteAsync().WaitAsync(cancellationToken))
			{
				return false;
			}

			if (_staticComputeConfig.CurrentValue.AutoEnrollAgents)
			{
				await ApproveAsync(key, null, cancellationToken);
			}

			return true;
		}

		/// <summary>
		/// Approve an agent for connecting to the farm
		/// </summary>
		/// <param name="key">Unique id for the agent to approve</param>
		/// <param name="agentId">Identifier to assign to the approved agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<bool> ApproveAsync(string key, AgentId? agentId, CancellationToken cancellationToken = default)
		{
			for (; ; )
			{
				cancellationToken.ThrowIfCancellationRequested();

				IDatabase redis = _redisService.GetDatabase();
				await ExpireKeysAsync(redis, cancellationToken);

				EnrollmentRequest? request = await redis.HashGetAsync(_requests, key).WaitAsync(cancellationToken);
				if (request == null)
				{
					return false;
				}

				AgentId agentIdOrDefault = agentId ?? new AgentId(request.HostName);

				ITransaction transaction = redis.CreateTransaction();
				_ = transaction.AddCondition(Condition.HashExists(_requests.Inner, key));
				_ = transaction.SortedSetAddAsync(_keys, key, GetTimestamp(_clock.UtcNow + TimeSpan.FromMinutes(5.0)));
				_ = transaction.HashDeleteAsync(_requests, key);
				_ = transaction.HashSetAsync(_approvals, key, agentIdOrDefault.ToString());

				if (await transaction.ExecuteAsync().WaitAsync(cancellationToken))
				{
					await redis.PublishAsync(_updateChannel, agentIdOrDefault.ToString());
					return true;
				}
			}
		}

		/// <summary>
		/// Find all candidate machines that we can allow to connect
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<IReadOnlyList<EnrollmentRequest>> FindAsync(CancellationToken cancellationToken = default)
		{
			IDatabase redis = _redisService.GetDatabase();
			await ExpireKeysAsync(redis, cancellationToken);

			HashEntry<string, EnrollmentRequest>[] entries = await redis.HashGetAllAsync(_requests).WaitAsync(cancellationToken);
			return entries.ConvertAll(x => x.Value);
		}

		/// <summary>
		/// Waits until a particular agent is approved
		/// </summary>
		/// <param name="key">Unique id for the request</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<AgentId?> GetApprovalAsync(string key, CancellationToken cancellationToken)
		{
			string? agentId = await _redisService.GetDatabase().HashGetAsync(_approvals, key).WaitAsync(cancellationToken);
			if (agentId == null)
			{
				return null;
			}
			return new AgentId(agentId);
		}

		/// <summary>
		/// Waits until a particular agent is approved
		/// </summary>
		/// <param name="key">Unique id for the request</param>
		/// <param name="hostName">Name of the host</param>
		/// <param name="description">Description for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		public async Task<AgentId> WaitForApprovalAsync(string key, string hostName, string description, CancellationToken cancellationToken)
		{
			for (; ; )
			{
				Task task = _approvalEvent.Task;

				AgentId? agentId = await GetApprovalAsync(key, cancellationToken);
				if (agentId != null)
				{
					return agentId.Value;
				}

				if (await AddAsync(key, hostName, description, cancellationToken))
				{
					await task.WaitAsync(cancellationToken);
				}
			}
		}
	}
}
