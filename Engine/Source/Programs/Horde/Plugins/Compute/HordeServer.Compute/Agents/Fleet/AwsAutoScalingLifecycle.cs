// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Text.Json;
using System.Text.Json.Serialization;
using Amazon.AutoScaling;
using Amazon.AutoScaling.Model;
using Amazon.Runtime;
using Amazon.SQS;
using Amazon.SQS.Model;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.AspNetCore.Authorization;
using Microsoft.AspNetCore.Http;
using Microsoft.AspNetCore.Mvc;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;
using StackExchange.Redis;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents.Fleet;

/// <summary>
/// Service handling callbacks and lifecycle events triggered by EC2 auto-scaling in AWS
/// </summary>
public sealed class AwsAutoScalingLifecycleService : IHostedService, IAsyncDisposable
{
	/// <summary>
	/// Lifecycle action for continue
	/// In the case of terminating instances, setting this result means the instance should *continue* be kept alive
	/// </summary>
	public const string ActionContinue = "CONTINUE";

	/// <summary>
	/// Lifecycle action for abandon
	/// In the case of terminating instances, setting this result means the instance should be *terminated*
	/// </summary>
	public const string ActionAbandon = "ABANDON";

	/// <summary>
	/// Warm pool origin
	/// </summary>
	public const string OriginWarmPool = "WarmPool";

	/// <summary>
	/// Normal auto-scaling group origin
	/// </summary>
	public const string OriginAsg = "AutoScalingGroup";

	/// <summary>
	/// How often instance lifecycles for auto-scaling should be updated with AWS 
	/// </summary>
	public TimeSpan LifecycleUpdaterInterval { get; } = TimeSpan.FromMinutes(1);

	private const string RedisKey = "aws-asg-lifecycle";
	private readonly AgentService _agentService;
	private readonly IRedisService _redisService;
	private readonly IAgentCollection _agents;
	private readonly IClock _clock;
	private readonly Tracer _tracer;
	private readonly ILogger<AwsAutoScalingLifecycleService> _logger;
	private readonly ITicker _updateLifecyclesTicker;
	private readonly List<BackgroundTask> _lifecycleEventListenerTasks = new();
	private readonly string[] _sqsQueueUrls;

#pragma warning disable CA2213 // Disposable fields should be disposed
	private IAmazonAutoScaling? _awsAutoScaling;
	private IAmazonSQS? _awsSqs;
#pragma warning restore CA2213

	/// <summary>
	/// Constructor
	/// </summary>
	public AwsAutoScalingLifecycleService(
		AgentService agentService,
		IRedisService redisService,
		IAgentCollection agents,
		IClock clock,
		IOptionsMonitor<ComputeServerConfig> staticComputeConfig,
		IServiceProvider serviceProvider,
		Tracer tracer,
		ILogger<AwsAutoScalingLifecycleService> logger)
	{
		_agentService = agentService;
		_redisService = redisService;
		_agents = agents;
		_clock = clock;
		_tracer = tracer;
		_logger = logger;
		_awsAutoScaling = serviceProvider.GetService<IAmazonAutoScaling>();
		_awsSqs = serviceProvider.GetService<IAmazonSQS>();

		string tickerName = $"{nameof(AwsAutoScalingLifecycleService)}.{nameof(UpdateLifecyclesAsync)}";
		_updateLifecyclesTicker = clock.AddSharedTicker(tickerName, LifecycleUpdaterInterval, UpdateLifecyclesAsync, logger);
		_sqsQueueUrls = staticComputeConfig.CurrentValue.AwsAutoScalingQueueUrls;

		foreach (string sqsQueueUrl in _sqsQueueUrls)
		{
			_lifecycleEventListenerTasks.Add(new BackgroundTask(ct => ListenForLifecycleEventsAsync(sqsQueueUrl, ct)));
		}
	}

	internal void SetAmazonClientsTesting(IAmazonAutoScaling awsAutoScaling, IAmazonSQS awsSqs)
	{
		_awsAutoScaling = awsAutoScaling;
		_awsSqs = awsSqs;
	}

	/// <inheritdoc/>
	public async Task StartAsync(CancellationToken cancellationToken)
	{
		if (_sqsQueueUrls.Length > 0)
		{
			await _updateLifecyclesTicker.StartAsync();
		}

		_lifecycleEventListenerTasks.ForEach(x => x.Start());
	}

	/// <inheritdoc/>
	public async Task StopAsync(CancellationToken cancellationToken)
	{
		foreach (BackgroundTask bgTask in _lifecycleEventListenerTasks)
		{
			await bgTask.StopAsync(cancellationToken);
		}

		await _updateLifecyclesTicker.StopAsync();
	}

	/// <inheritdoc/>
	public async ValueTask DisposeAsync()
	{
		await _updateLifecyclesTicker.DisposeAsync();
		foreach (BackgroundTask bgTask in _lifecycleEventListenerTasks)
		{
			await bgTask.DisposeAsync();
		}
	}

	/// <summary>
	/// Continuously request and receive messages from SQS
	/// </summary>
	/// <param name="sqsQueueUrl">SQS queue to fetch messages from</param>
	/// <param name="cancellationToken">Cancellation token</param>
	private async Task ListenForLifecycleEventsAsync(string sqsQueueUrl, CancellationToken cancellationToken = default)
	{
		_logger.LogInformation("Listening for lifecycle events on {SqsQueueUrl}...", sqsQueueUrl);
		while (!cancellationToken.IsCancellationRequested)
		{
			try
			{
				await ReceiveLifecycleEventsAsync(sqsQueueUrl, cancellationToken);
			}
			catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
			{
				break;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Exception while receiving lifecycle events: {Message}", ex.Message);
				await Task.Delay(TimeSpan.FromSeconds(15), cancellationToken);
			}

			await Task.Delay(TimeSpan.FromSeconds(1), cancellationToken);
		}
	}

	/// <summary>
	/// Receive any queued messages from SQS.
	/// This is a one-time operation, if more messages are expected, call this method again.
	/// </summary>
	/// <param name="sqsQueueUrl">SQS queue to fetch messages from</param>
	/// <param name="cancellationToken">Cancellation token</param>
	internal async Task ReceiveLifecycleEventsAsync(string sqsQueueUrl, CancellationToken cancellationToken)
	{
		ReceiveMessageRequest request = new()
		{
			QueueUrl = sqsQueueUrl,
			MaxNumberOfMessages = 10,
			VisibilityTimeout = 10,
			WaitTimeSeconds = 10,
		};

		ReceiveMessageResponse response = await GetSqs().ReceiveMessageAsync(request, cancellationToken);
		foreach (Message message in response.Messages)
		{
			try
			{
				LifecycleActionEvent ev = JsonSerializer.Deserialize<LifecycleActionEvent>(message.Body) ?? throw new JsonException("Value is null");
				await InitiateTerminationAsync(ev, cancellationToken);
			}
			catch (Exception e)
			{
				_logger.LogError(e, "Failed initiating termination of EC2 instance. Body={Body}", message.Body);
			}
			finally
			{
				await DeleteMessageAsync(sqsQueueUrl, message, cancellationToken);
			}
		}
	}

	private async Task DeleteMessageAsync(string sqsQueueUrl, Message message, CancellationToken cancellationToken)
	{
		try
		{
			DeleteMessageRequest deleteRequest = new() { QueueUrl = sqsQueueUrl, ReceiptHandle = message.ReceiptHandle };
			await GetSqs().DeleteMessageAsync(deleteRequest, cancellationToken);
		}
		catch (AmazonServiceException e)
		{
			_logger.LogError(e, "Failed to delete message {MessageId} from SQS queue {SqsQueueUrl} after processing it", message.MessageId, sqsQueueUrl);
		}
	}

	private IAmazonSQS GetSqs()
	{
		return _awsSqs ?? throw new Exception("AWS SQS client is not set. Make sure AWS is configured in settings.");
	}

	/// <summary>
	/// Gracefully terminate an EC2 instance running the Horde agent
	/// This call is initiated by an event coming from AWS auto-scaling group and will request the agent to shutdown.
	/// Since the agent can be busy running a job, it's possible it cannot be terminated immediately.
	/// The termination request from AWS is therefore tracked and updated continuously as the agent shutdown progresses. 
	/// </summary>
	/// <param name="e">Lifecycle action event received</param>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>True if handled</returns>
	public async Task<bool> InitiateTerminationAsync(LifecycleActionEvent e, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAutoScalingLifecycleService)}.{nameof(InitiateTerminationAsync)}");
		span.SetAttribute("asgName", e.AutoScalingGroupName);
		span.SetAttribute("origin", e.Origin);
		span.SetAttribute("instanceId", e.Ec2InstanceId);
		span.SetAttribute("lifecycleTransition", e.LifecycleTransition);

		if (e.Origin == OriginAsg)
		{
			string instanceIdProp = KnownPropertyNames.AwsInstanceId + "=" + e.Ec2InstanceId;

			IAgent? agent = await _agentService.FindAgentsAsync(null, null, instanceIdProp, true, cancellationToken).FirstOrDefaultAsync(cancellationToken);
			if (agent == null)
			{
				_logger.LogWarning("Lifecycle action received but no agent with instance ID {InstanceId} found", e.Ec2InstanceId);
				return false;
			}

			await agent.TryUpdateAsync(new UpdateAgentOptions { RequestShutdown = true }, cancellationToken: cancellationToken);
			await TrackAgentLifecycleAsync(agent.Id, e, cancellationToken);
			return true;
		}
		else if (e.Origin == OriginWarmPool)
		{
			// EC2 instance in warm pool and not current in use by Horde
			await SendCompleteLifecycleActionAsync(e, ActionResult.Abandon, cancellationToken);
			return true;
		}
		else
		{
			_logger.LogWarning("Unknown origin {Origin}", e.Origin);
			return false;
		}
	}

	/// <summary>
	/// Get instances available for termination.
	/// An AWS Lambda function is set to handle termination policy queries from an auto-scaling group
	/// That function is configured to call Horde server and invoke this method
	/// </summary>
	/// <param name="e">Event as received from AWS API</param>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>A list of instance IDs that can be terminated</returns>
	public async Task<List<string>> GetInstancesAvailableForTerminationAsync(TerminationPolicyEvent e, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAutoScalingLifecycleService)}.{nameof(GetInstancesAvailableForTerminationAsync)}");
		span.SetAttribute("asgName", e.AutoScalingGroupName);
		span.SetAttribute("instancesCount", e.Instances.Count);
		span.SetAttribute("capacityCount", e.CapacityToTerminate.Count);

		List<string> asgInstanceIds = e.Instances
			.Select(x => x.InstanceId)
			.OfType<string>()
			.ToList();

		bool IsAgentSuggestedByAsg(IAgent agent, [NotNullWhen(true)] out string? instanceId)
		{
			instanceId = asgInstanceIds.FirstOrDefault(asgInstanceId => agent.HasProperty($"{KnownPropertyNames.AwsInstanceId}={asgInstanceId}"));
			return instanceId != null;
		}

		List<string> validInstanceIds = new();
		await foreach (IAgent agent in _agentService.FindAgentsAsync(null, null, null, true, cancellationToken))
		{
			if (IsAgentSuggestedByAsg(agent, out string? instanceId))
			{
				if (agent.Leases.Count == 0)
				{
					validInstanceIds.Add(instanceId);
				}
			}
		}

		span.SetAttribute("validInstanceIds", validInstanceIds.Count);
		return validInstanceIds;
	}

	private async Task TrackAgentLifecycleAsync(AgentId agentId, LifecycleActionEvent e, CancellationToken cancellationToken)
	{
		DateTime utcNow = _clock.UtcNow;
		AgentLifecycleInfo info = new(agentId.ToString(), utcNow, utcNow, e);
		using MemoryStream ms = new(500);
		await JsonSerializer.SerializeAsync(ms, info, cancellationToken: cancellationToken);
		await _redisService.GetDatabase().HashSetAsync(RedisKey, agentId.ToString(), ms.ToArray());
	}

	private async Task SendCompleteLifecycleActionAsync(LifecycleActionEvent laEvent, ActionResult result, CancellationToken cancellationToken)
	{
		if (_awsAutoScaling == null)
		{
			throw new Exception($"{nameof(IAmazonAutoScaling)} client not initialized! Cannot make AWS API call.");
		}

		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAutoScalingLifecycleService)}.{nameof(SendCompleteLifecycleActionAsync)}");
		CompleteLifecycleActionRequest request = new()
		{
			InstanceId = laEvent.Ec2InstanceId,
			LifecycleActionResult = result == ActionResult.Continue ? ActionContinue : ActionAbandon,
			LifecycleActionToken = laEvent.LifecycleActionToken,
			LifecycleHookName = laEvent.LifecycleHookName,
			AutoScalingGroupName = laEvent.AutoScalingGroupName
		};
		span.SetAttribute("instanceId", request.InstanceId);
		span.SetAttribute("lifecycleActionResult", request.LifecycleActionResult);
		span.SetAttribute("lifecycleActionToken", request.LifecycleActionToken);
		span.SetAttribute("lifecycleHookName", request.LifecycleHookName);
		span.SetAttribute("autoScalingGroupName", request.AutoScalingGroupName);

		CompleteLifecycleActionResponse response = await _awsAutoScaling.CompleteLifecycleActionAsync(request, cancellationToken);
		span.SetAttribute("statusCode", response.HttpStatusCode.ToString());
	}

	/// <summary>
	/// Update AWS auto-scaling on the lifecycle status for each agent being tracked for shutdown.
	/// This ensures the EC2 instance is not released for termination until it gracefully finished any outstanding work.
	/// That is, the agent status is set to stopped in Horde server.
	/// </summary>
	/// <param name="cancellationToken">Cancellation token for the async task</param>
	/// <returns>Async task</returns>
	internal async ValueTask UpdateLifecyclesAsync(CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAutoScalingLifecycleService)}.{nameof(UpdateLifecyclesAsync)}");

		Dictionary<AgentId, AgentLifecycleInfo> infos = new();
		IDatabase redis = _redisService.GetDatabase();

		foreach (HashEntry entry in await redis.HashGetAllAsync(RedisKey))
		{
			try
			{
				AgentId agentId = new(entry.Name.ToString());
				AgentLifecycleInfo info = JsonSerializer.Deserialize<AgentLifecycleInfo>(entry.Value.ToString()) ?? throw new Exception("Deserialization returned null");
				infos[agentId] = info;
			}
			catch (Exception e)
			{
				_logger.LogWarning(e, "Unable to deserialize agent lifecycle info object from Redis. {Key} = {Value}", entry.Name.ToString(), entry.Value.ToString());
				await redis.HashDeleteAsync(RedisKey, entry.Name);
			}
		}

		span.SetAttribute("numTrackedAgents", infos.Count);

		List<Task> tasks = new();
		foreach (IAgent agent in await _agents.GetManyAsync(infos.Keys.ToList(), cancellationToken))
		{
			ActionResult result = agent.Status == AgentStatus.Stopped ? ActionResult.Abandon : ActionResult.Continue;
			tasks.Add(Task.Run(async () =>
			{
				bool deleteEvent;
				try
				{
					await SendCompleteLifecycleActionAsync(infos[agent.Id].Event, result, cancellationToken);
					deleteEvent = result == ActionResult.Abandon;
				}
				catch (Exception e)
				{
					_logger.LogError(e, "Unable to complete the AWS ASG lifecycle action. Reason: {Message}", e.Message);
					deleteEvent = true;
				}

				if (deleteEvent)
				{
					await redis.HashDeleteAsync(RedisKey, agent.Id.ToString());
				}
			}, cancellationToken));
		}

		span.SetAttribute("numLifecycleActionsSent", tasks.Count);
		await Task.WhenAll(tasks);
	}

	/// <summary>
	/// Class for agent lifecycle state. Serialized and stored in Redis.
	/// </summary>
	private class AgentLifecycleInfo
	{
		public string AgentId { get; private set; }
		public DateTime CreatedAt { get; private set; }
		public DateTime UpdatedAt { get; private set; }
		public LifecycleActionEvent Event { get; private set; }

		public AgentLifecycleInfo(string agentId, DateTime createdAt, DateTime updatedAt, LifecycleActionEvent @event)
		{
			AgentId = agentId;
			CreatedAt = createdAt;
			UpdatedAt = updatedAt;
			Event = @event;
		}
	}

	private enum ActionResult
	{
		/// <inheritdoc cref="AwsAutoScalingLifecycleService.ActionContinue"/>
		Continue,

		/// <inheritdoc cref="AwsAutoScalingLifecycleService.ActionAbandon"/>
		Abandon
	}
}

/// <summary>
/// Controller handling callbacks for AWS auto-scaling
/// </summary>
[ApiController]
[Authorize]
[Route("[controller]")]
public class AwsAutoScalingLifecycleController : HordeControllerBase
{
	private readonly AwsAutoScalingLifecycleService _lifecycleService;

	/// <summary>
	/// Constructor
	/// </summary>
	public AwsAutoScalingLifecycleController(AwsAutoScalingLifecycleService lifecycleService)
	{
		_lifecycleService = lifecycleService;
	}

	/// <summary>
	/// Called by AWS auto-scaling group to get which instance IDs are valid for termination
	/// <see cref="AwsAutoScalingLifecycleService.GetInstancesAvailableForTerminationAsync" />
	/// </summary>
	/// <param name="tpe">Event</param>
	/// <param name="cancellationToken">Cancellation token</param>
	/// <returns>List of instance IDs the AWS auto-scaling group is allowed to terminate</returns>
	[HttpPost]
	[Route("/api/v1/aws/asg-termination-policy")]
	[ProducesResponseType(StatusCodes.Status200OK)]
	public async Task<ActionResult> TerminationPolicyAsync([FromBody] TerminationPolicyEvent tpe, CancellationToken cancellationToken)
	{
		List<string> instanceIds = await _lifecycleService.GetInstancesAvailableForTerminationAsync(tpe, cancellationToken);
		return new JsonResult(instanceIds);
	}
}

/// <summary>
/// Lifecycle action event from AWS auto-scaling group
/// The property names are explicitly set to highlight these are not controlled by Horde but sent from AWS API.
/// </summary>
public class LifecycleActionEvent
{
	/// <summary>
	/// Name of the lifecycle hook
	/// </summary>
	[JsonPropertyName("LifecycleHookName")] public string LifecycleHookName { get; set; } = "";

	/// <summary>
	/// Token for the lifecycle action
	/// </summary>
	[JsonPropertyName("LifecycleActionToken")] public string LifecycleActionToken { get; set; } = "";

	/// <summary>
	/// Lifecycle transition (e.g EC2_INSTANCE_LAUNCHING, EC2_INSTANCE_TERMINATING)
	/// </summary>
	[JsonPropertyName("LifecycleTransition")] public string LifecycleTransition { get; set; } = "";

	/// <summary>
	/// Name of the auto-scaling group being handled
	/// </summary>
	[JsonPropertyName("AutoScalingGroupName")] public string AutoScalingGroupName { get; set; } = "";

	/// <summary>
	/// Instance ID
	/// </summary>
	[JsonPropertyName("EC2InstanceId")] public string Ec2InstanceId { get; set; } = "";

	/// <summary>
	/// Origin of the instance (e.g an agent in service or in a warm pool)
	/// </summary>
	[JsonPropertyName("Origin")] public string Origin { get; set; } = "";
}

/// <summary>
/// Instance referenced by <see cref="TerminationPolicyEvent" />
/// </summary>
public class TerminationPolicyInstance
{
	/// <summary>
	/// Availability zone of the instance (such as 'us-east-1c')
	/// </summary>
	[JsonPropertyName("AvailabilityZone")] public string AvailabilityZone { get; set; } = "";

	/// <summary>
	/// Capacity to terminate (number of instances)
	/// </summary>
	[JsonPropertyName("Capacity")] public int? Capacity { get; set; }

	/// <summary>
	/// Instance ID (such as 'i-123456789')
	/// </summary>
	[JsonPropertyName("InstanceId")] public string? InstanceId { get; set; }

	/// <summary>
	/// Instance type (such as 'm5.xlarge')
	/// </summary>
	[JsonPropertyName("InstanceType")] public string? InstanceType { get; set; }

	/// <summary>
	/// Instance market options (such as 'on-demand' or 'spot')
	/// </summary>
	[JsonPropertyName("InstanceMarketOption")] public string InstanceMarketOption { get; set; } = "";

	/// <summary>
	/// Constructor
	/// </summary>
	public TerminationPolicyInstance()
	{
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="capacity"></param>
	/// <param name="availabilityZone"></param>
	/// <param name="instanceMarketOption"></param>
	public TerminationPolicyInstance(int capacity, string availabilityZone = "us-east-1a", string instanceMarketOption = "on-demand")
	{
		AvailabilityZone = availabilityZone;
		Capacity = capacity;
		InstanceMarketOption = instanceMarketOption;
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="instanceId"></param>
	/// <param name="availabilityZone"></param>
	/// <param name="instanceType"></param>
	/// <param name="instanceMarketOption"></param>
	public TerminationPolicyInstance(string instanceId, string availabilityZone = "us-east-1a", string instanceType = "m5.large", string instanceMarketOption = "on-demand")
	{
		AvailabilityZone = availabilityZone;
		InstanceId = instanceId;
		InstanceType = instanceType;
		InstanceMarketOption = instanceMarketOption;
	}
}

/// <summary>
/// Termination policy event from AWS auto-scaling group
/// The property names are explicitly set to highlight these are not controlled by Horde but sent from AWS API.
/// </summary>
public class TerminationPolicyEvent
{
	/// <summary>
	/// ARN of the auto-scaling group
	/// </summary>
	[JsonPropertyName("AutoScalingGroupARN")] public string AutoScalingGroupArn { get; set; } = "";

	/// <summary>
	/// Name of the auto-scaling group
	/// </summary>
	[JsonPropertyName("AutoScalingGroupName")] public string AutoScalingGroupName { get; set; } = "";

	/// <summary>
	/// Capacity that's been requested to be terminated
	/// </summary>
	[JsonPropertyName("CapacityToTerminate")] public List<TerminationPolicyInstance> CapacityToTerminate { get; set; } = new();

	/// <summary>
	/// Instances available to terminate
	/// </summary>
	[JsonPropertyName("Instances")] public List<TerminationPolicyInstance> Instances { get; set; } = new();

	/// <summary>
	/// Cause of the termination (such as 'SCALE_IN' or 'INSTANCE_REFRESH')
	/// </summary>
	[JsonPropertyName("Cause")] public string Cause { get; set; } = "";
}

