// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json.Serialization;
using Amazon.CloudWatch;
using Amazon.CloudWatch.Model;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Compute;
using HordeServer.Agents.Pools;
using HordeServer.Compute;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Settings for <see cref="ComputeQueueAwsMetricStrategy" />
	/// </summary>
	public class ComputeQueueAwsMetricSettings
	{
		/// <summary>
		/// Compute cluster ID to observe
		/// </summary>
		public string ComputeClusterId { get; set; } = "default";

		/// <summary>
		/// AWS CloudWatch namespace to write metrics in
		/// </summary>
		public string Namespace { get; set; } = "Horde";

		/// <summary>
		/// Constructor used for JSON serialization
		/// </summary>
		[JsonConstructor]
		public ComputeQueueAwsMetricSettings()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="computeClusterId"></param>
		/// <param name="cloudWatchNamespace"></param>
		public ComputeQueueAwsMetricSettings(string computeClusterId, string cloudWatchNamespace = "Horde")
		{
			ComputeClusterId = computeClusterId;
			Namespace = cloudWatchNamespace;
		}
	}

	static class ComputeServiceExtensions
	{
		public static Task<int> GetNumQueuedTasksForPoolAsync(this ComputeTaskSource computeTaskSource, ClusterId clusterId, IPoolConfig pool)
		{
			// Will need to reimplement this functionality in ComputeService if we want to use this strategy, but conditions on tasks
			// may reference properties that are specific to an agent rather than a pool...
			throw new NotImplementedException();
		}
	}

	/// <summary>
	/// A no-op strategy that reports size of compute task queue for a given pool as AWS CloudWatch metrics
	/// A metric that later can be used as a source for AWS-controlled auto-scaling policies. 
	/// </summary>
	public class ComputeQueueAwsMetricStrategy : IPoolSizeStrategy
	{
		private readonly IAmazonCloudWatch _cloudWatch;
		private readonly ComputeTaskSource _computeTaskSource;
		private readonly ComputeQueueAwsMetricSettings _settings;
		private readonly ILogger<ComputeQueueAwsMetricStrategy> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="cloudWatch"></param>
		/// <param name="computeTaskSource"></param>
		/// <param name="settings"></param>
		/// <param name="logger"></param>
		public ComputeQueueAwsMetricStrategy(IAmazonCloudWatch cloudWatch, ComputeTaskSource computeTaskSource,
			ComputeQueueAwsMetricSettings settings, ILogger<ComputeQueueAwsMetricStrategy> logger)
		{
			_cloudWatch = cloudWatch;
			_computeTaskSource = computeTaskSource;
			_settings = settings;
			_logger = logger;
		}

		/// <inheritdoc/>
		public string Name { get; } = "ComputeQueueAwsMetric";

		/// <inheritdoc/>
		public async Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(ComputeQueueAwsMetricStrategy)}.{nameof(CalculatePoolSizeAsync)}");
			span.SetAttribute(OpenTelemetryTracers.DatadogResourceAttribute, pool.Id.ToString());
			span.SetAttribute("currentAgentCount", agents.Count);

			Dictionary<string, List<MetricDatum>> metricsPerCloudWatchNamespace = new();
			int numAgents = agents.Count;
			int totalCpuCores = agents.Select(x => x.Resources.TryGetValue(KnownResourceNames.LogicalCores, out int numCpuCores) ? numCpuCores : 0).Sum();
			int numQueuedComputeTasks = await _computeTaskSource.GetNumQueuedTasksForPoolAsync(new ClusterId(_settings.ComputeClusterId), pool);

			double numQueuedTasksPerAgent = numQueuedComputeTasks / (double)Math.Max(numAgents, 1);
			double numQueuedTasksPerCores = numQueuedComputeTasks / (double)Math.Max(totalCpuCores, 1);

			DateTime now = DateTime.UtcNow;

			if (!metricsPerCloudWatchNamespace.TryGetValue(_settings.Namespace, out List<MetricDatum>? metricDatums))
			{
				metricDatums = new List<MetricDatum>();
				metricsPerCloudWatchNamespace[_settings.Namespace] = metricDatums;
			}

			List<Dimension> dimensions = new()
			{
				new() { Name = "Pool", Value = pool.Id.ToString() },
				new() { Name = "ComputeCluster", Value = _settings.ComputeClusterId },
			};
			metricDatums.Add(new()
			{
				MetricName = "NumComputeTasksPerAgent",
				Dimensions = dimensions,
				Unit = StandardUnit.Count,
				Value = numQueuedTasksPerAgent,
				TimestampUtc = now
			});
			metricDatums.Add(new()
			{
				MetricName = "NumComputeTasksPerCpuCores",
				Dimensions = dimensions,
				Unit = StandardUnit.Count,
				Value = numQueuedTasksPerCores,
				TimestampUtc = now
			});

			foreach ((string ns, List<MetricDatum> metricDatumsNs) in metricsPerCloudWatchNamespace)
			{
				using TelemetrySpan cwSpan = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(ComputeQueueAwsMetricStrategy)}.{nameof(CalculatePoolSizeAsync)}.PutCloudWatchMetrics");
				cwSpan.SetAttribute("namespace", ns);

				PutMetricDataRequest request = new() { Namespace = ns, MetricData = metricDatumsNs };
				PutMetricDataResponse response = await _cloudWatch.PutMetricDataAsync(request, cancellationToken);
				cwSpan.SetAttribute("res.statusCode", (int)response.HttpStatusCode);
				if (response.HttpStatusCode != HttpStatusCode.OK)
				{
					_logger.LogError("Unable to put CloudWatch metrics");
				}
			}

			// Return the input pool size data as-is since we are only observing and not modifying
			return new PoolSizeResult(numAgents, numAgents);
		}
	}

	/// <summary>
	/// Factory for <see cref="LeaseUtilizationStrategy"/>
	/// </summary>
	public class ComputeQueueAwsMetricStrategyFactory : PoolSizeStrategyFactory<ComputeQueueAwsMetricSettings>
	{
		readonly IAmazonCloudWatch _cloudWatch;
		readonly ComputeTaskSource _computeTaskSource;
		readonly ILogger<ComputeQueueAwsMetricStrategy> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public ComputeQueueAwsMetricStrategyFactory(IAmazonCloudWatch cloudWatch, ComputeTaskSource computeTaskSource, ILogger<ComputeQueueAwsMetricStrategy> logger)
			: base(PoolSizeStrategy.ComputeQueueAwsMetric)
		{
			_cloudWatch = cloudWatch;
			_computeTaskSource = computeTaskSource;
			_logger = logger;
		}

		/// <inheritdoc/>
		public override IPoolSizeStrategy Create(ComputeQueueAwsMetricSettings settings)
			=> new ComputeQueueAwsMetricStrategy(_cloudWatch, _computeTaskSource, settings, _logger);
	}
}
