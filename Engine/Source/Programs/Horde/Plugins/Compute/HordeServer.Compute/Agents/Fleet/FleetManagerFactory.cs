// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using System.Text.Json;
using System.Text.Json.Nodes;
using Amazon.AutoScaling;
using Amazon.EC2;
using EpicGames.Core;
using HordeServer.Agents.Fleet.Providers;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet;

using JsonObject = System.Text.Json.Nodes.JsonObject;

/// <summary>
/// Interface for fleet manager factory
/// </summary>
public interface IFleetManagerFactory
{
	/// <summary>
	/// Create a fleet manager
	/// </summary>
	/// <param name="type">Type of fleet manager</param>
	/// <param name="config">Config as a serialized JSON string</param>
	/// <returns>An instantiated fleet manager with parameters loaded from config</returns>
	/// <exception cref="ArgumentException">If fleet manager could not be instantiated</exception>
	public IFleetManager CreateFleetManager(FleetManagerType type, JsonObject? config = null);
}

/// <summary>
/// Factory creating instances of fleet managers
/// </summary>
public sealed class FleetManagerFactory : IFleetManagerFactory
{
	private readonly IAgentCollection _agentCollection;
	private readonly IClock _clock;
	private readonly Meter _meter;
	private readonly IServiceProvider _provider;
	private readonly IOptionsMonitor<ComputeServerConfig> _staticComputeConfig;
	private readonly Tracer _tracer;
	private readonly ILoggerFactory _loggerFactory;

	private IAmazonAutoScaling? _awsAutoScaling;
	private IAmazonEC2? _awsEc2;

	/// <summary>
	/// Constructor
	/// </summary>
	public FleetManagerFactory(IAgentCollection agentCollection, IClock clock, Meter meter, IServiceProvider provider, IOptionsMonitor<ComputeServerConfig> staticComputeConfig, Tracer tracer, ILoggerFactory loggerFactory)
	{
		_agentCollection = agentCollection;
		_clock = clock;
		_meter = meter;
		_provider = provider;
		_staticComputeConfig = staticComputeConfig;
		_tracer = tracer;
		_loggerFactory = loggerFactory;
	}

	/// <inheritdoc/>
	public IFleetManager CreateFleetManager(FleetManagerType type, JsonObject? config)
	{
		return type switch
		{
			FleetManagerType.Default =>
				CreateFleetManager(_staticComputeConfig.CurrentValue.FleetManagerV2, _staticComputeConfig.CurrentValue.FleetManagerV2Config ?? new JsonObject()),
			FleetManagerType.NoOp =>
				new NoOpFleetManager(_loggerFactory.CreateLogger<NoOpFleetManager>()),
			FleetManagerType.Aws =>
				new AwsFleetManager(GetAwsEc2(type), _agentCollection, DeserializeSettings<AwsFleetManagerSettings>(config), _tracer, _loggerFactory.CreateLogger<AwsFleetManager>()),
			FleetManagerType.AwsReuse =>
				new AwsReuseFleetManager(GetAwsEc2(type), _agentCollection, DeserializeSettings<AwsReuseFleetManagerSettings>(config), _tracer, _loggerFactory.CreateLogger<AwsReuseFleetManager>()),
			FleetManagerType.AwsRecycle =>
				new AwsRecyclingFleetManager(GetAwsEc2(type), _agentCollection, _meter, _clock, DeserializeSettings<AwsRecyclingFleetManagerSettings>(config), _tracer, _loggerFactory.CreateLogger<AwsRecyclingFleetManager>()),
			FleetManagerType.AwsAsg =>
				new AwsAsgFleetManager(GetAwsAutoScaling(type), DeserializeSettings<AwsAsgSettings>(config), _tracer, _loggerFactory.CreateLogger<AwsAsgFleetManager>()),
			_ => throw new ArgumentException("Unknown fleet manager type " + type)
		};
	}

	private static T DeserializeSettings<T>(JsonObject? config)
	{
		if (config == null)
		{
			config = new JsonObject();
		}

		try
		{
			T? settings = JsonSerializer.Deserialize<T>((JsonNode)config, new JsonSerializerOptions { AllowTrailingCommas = true, PropertyNameCaseInsensitive = true, ReadCommentHandling = JsonCommentHandling.Skip });
			if (settings == null)
			{
				throw new InvalidDataException($"Unable to deserialize");
			}
			return settings;
		}
		catch (ArgumentException e)
		{
			throw new ArgumentException($"Unable to deserialize {typeof(T)} config: '{config}'", e);
		}
	}

	private IAmazonEC2 GetAwsEc2(FleetManagerType type)
	{
		if (_awsEc2 != null)
		{
			return _awsEc2;
		}

		_awsEc2 = _provider.GetService<IAmazonEC2>();
		if (_staticComputeConfig.CurrentValue.WithAws == false || _awsEc2 == null)
		{
			throw new ArgumentException($"Unable to create fleet manager {type} requiring AWS specific classes. Check that setting '{nameof(ComputeServerConfig.WithAws)}' is enabled");
		}

		return _awsEc2;
	}

	private IAmazonAutoScaling GetAwsAutoScaling(FleetManagerType type)
	{
		if (_awsAutoScaling != null)
		{
			return _awsAutoScaling;
		}

		_awsAutoScaling = _provider.GetService<IAmazonAutoScaling>();
		if (_staticComputeConfig.CurrentValue.WithAws == false || _awsAutoScaling == null)
		{
			throw new ArgumentException($"Unable to create fleet manager {type} requiring AWS specific classes. Check that setting '{nameof(ComputeServerConfig.WithAws)}' is enabled");
		}

		return _awsAutoScaling;
	}
}
