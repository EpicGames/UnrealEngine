// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents.Pools;
using HordeServer.Server;
using HordeServer.Utilities;
using Microsoft.Extensions.Hosting;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.Options;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Parameters required for calculating pool size
	/// </summary>
	public class PoolWithAgents
	{
		/// <summary>
		/// Pool being resized
		/// </summary>
		public IPoolConfig Pool { get; }

		/// <summary>
		/// All agents currently associated with the pool
		/// </summary>
		public List<IAgent> Agents { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="pool"></param>
		/// <param name="agents"></param>
		public PoolWithAgents(IPoolConfig pool, List<IAgent> agents)
		{
			Pool = pool;
			Agents = agents;
		}
	}

	/// <summary>
	/// Service for managing the autoscaling of agent pools
	/// </summary>
	public sealed class FleetService : IHostedService, IAsyncDisposable
	{
		/// <summary>
		/// Max number of auto-scaling calculations to be done concurrently (sizing calculations and fleet manager calls)
		/// </summary>
		private const int MaxParallelTasks = 10;

		private readonly AgentService _agentService;
		private readonly IPoolCollection _poolCollection;
		private readonly IDowntimeService _downtimeService;
		private readonly IFleetManagerFactory _fleetManagerFactory;
		private readonly IClock _clock;
		private readonly IEnumerable<IPoolSizeStrategyFactory> _poolSizeStrategyFactories;
		private readonly ITicker _ticker;
		private readonly ITicker _tickerHighFrequency;
		private readonly TimeSpan _defaultScaleOutCooldown;
		private readonly TimeSpan _defaultScaleInCooldown;
		private readonly IOptions<ComputeServerConfig> _staticComputeConfig;
		private readonly Tracer _tracer;
		private readonly ILogger<FleetService> _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public FleetService(
			AgentService agentService,
			IPoolCollection poolCollection,
			IDowntimeService downtimeService,
			IFleetManagerFactory fleetManagerFactory,
			IClock clock,
			IEnumerable<IPoolSizeStrategyFactory> poolSizeStrategyFactories,
			IOptions<ComputeServerConfig> staticComputeConfig,
			Tracer tracer,
			ILogger<FleetService> logger)
		{
			_agentService = agentService;
			_poolCollection = poolCollection;
			_downtimeService = downtimeService;
			_fleetManagerFactory = fleetManagerFactory;
			_clock = clock;
			_tracer = tracer;
			_logger = logger;
			_poolSizeStrategyFactories = poolSizeStrategyFactories;
			_staticComputeConfig = staticComputeConfig;
			_defaultScaleOutCooldown = TimeSpan.FromSeconds(staticComputeConfig.Value.AgentPoolScaleOutCooldownSeconds);
			_defaultScaleInCooldown = TimeSpan.FromSeconds(staticComputeConfig.Value.AgentPoolScaleInCooldownSeconds);

			// Only enable auto-scaling when running on a hosting provider supporting it. Right now, that is only AWS.
			bool enableAutoScaling = _staticComputeConfig.Value.WithAws;
			Func<CancellationToken, ValueTask> ticker = enableAutoScaling ? TickLeaderAsync : _ => ValueTask.CompletedTask;
			Func<CancellationToken, ValueTask> tickerHighFreq = enableAutoScaling ? TickHighFrequencyAsync : _ => ValueTask.CompletedTask;
			_ticker = clock.AddSharedTicker<FleetService>(TimeSpan.FromSeconds(30), ticker, _logger);
			_tickerHighFrequency = clock.AddSharedTicker("FleetService.TickHighFrequency", TimeSpan.FromSeconds(30), tickerHighFreq, _logger);
		}

		/// <inheritdoc/>
		public Task StartAsync(CancellationToken cancellationToken)
		{
			return Task.WhenAll(_ticker.StartAsync(), _tickerHighFrequency.StartAsync());
		}

		/// <inheritdoc/>
		public Task StopAsync(CancellationToken cancellationToken)
		{
			return Task.WhenAll(_ticker.StopAsync(), _tickerHighFrequency.StopAsync());
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			await _ticker.DisposeAsync();
			await _tickerHighFrequency.DisposeAsync();
		}

		internal async ValueTask TickLeaderAsync(CancellationToken cancellationToken)
		{
			TelemetrySpan span = _tracer.StartSpan($"{nameof(FleetService)}.{nameof(TickLeaderAsync)}");

			try
			{
				List<PoolWithAgents> poolsWithAgents = await GetPoolsWithAgentsAsync(cancellationToken);

				ParallelOptions options = new() { MaxDegreeOfParallelism = MaxParallelTasks, CancellationToken = cancellationToken };
				await Parallel.ForEachAsync(poolsWithAgents, options, async (input, innerCt) =>
				{
					try
					{
						await CalculateSizeAndScaleAsync(input.Pool, input.Agents, innerCt);
					}
					catch (Exception e)
					{
						_logger.LogError(e, "Failed to scale pool {PoolId}", input.Pool.Id);
					}
				});
			}
			finally
			{
				span.End();
			}
		}

		internal async ValueTask TickHighFrequencyAsync(CancellationToken stoppingToken)
		{
			TelemetrySpan span = _tracer.StartSpan($"{nameof(FleetService)}.{nameof(TickHighFrequencyAsync)}");
			try
			{
				// TODO: Re-enable high frequency scaling (only used for experimental scaling of remote execution agents at the moment)
				await Task.Delay(0, stoppingToken);
			}
			finally
			{
				span.End();
			}
		}

		internal async Task CalculateSizeAndScaleAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken)
		{
			IPoolSizeStrategy sizeStrategy = CreatePoolSizeStrategy(pool);
			PoolSizeResult result = await sizeStrategy.CalculatePoolSizeAsync(pool, agents, cancellationToken);
			await ScalePoolAsync(pool, agents, result, cancellationToken);
		}

		internal async Task<List<PoolWithAgents>> GetPoolsWithAgentsAsync(CancellationToken cancellationToken = default)
		{
			List<IAgent> agents = (await _agentService.GetCachedAgentsAsync(cancellationToken))
				.Where(x => x is { Status: AgentStatus.Ok, Enabled: true, RequestShutdown: false }).ToList();

			List<IAgent> GetAgentsInPool(PoolId poolId) => agents.FindAll(a => a.Pools.Any(p => p == poolId));
			IReadOnlyList<IPoolConfig> pools = await _poolCollection.GetConfigsAsync(cancellationToken);

			return pools.Select(pool => new PoolWithAgents(pool, GetAgentsInPool(pool.Id))).ToList();
		}

		internal async Task<ScaleResult> ScalePoolAsync(IPoolConfig poolConfig, List<IAgent> agents, PoolSizeResult poolSizeResult, CancellationToken cancellationToken)
		{
			if (!poolConfig.EnableAutoscaling)
			{
				return new ScaleResult(FleetManagerOutcome.NoOp, 0, 0, "Auto-scaling disabled");
			}

			IPool? pool = await _poolCollection.GetAsync(poolConfig.Id, cancellationToken);
			if (pool == null)
			{
				return new ScaleResult(FleetManagerOutcome.NoOp, 0, 0, "Pool state not found");
			}

			int currentAgentCount = poolSizeResult.CurrentAgentCount;
			int desiredAgentCount = poolSizeResult.DesiredAgentCount;
			int deltaAgentCount = desiredAgentCount - currentAgentCount;

			IFleetManager fleetManager = CreateFleetManager(pool);

			using TelemetrySpan span = _tracer.StartSpan($"{nameof(FleetService)}.{nameof(ScalePoolAsync)}");
			span.SetAttribute(OpenTelemetryTracers.DatadogResourceAttribute, pool.Id.ToString());
			span.SetAttribute("currentAgentCount", currentAgentCount);
			span.SetAttribute("desiredAgentCount", desiredAgentCount);
			span.SetAttribute("deltaAgentCount", deltaAgentCount);
			span.SetAttribute("fleetManager", fleetManager.GetType().Name);

			Dictionary<string, object> logScopeMetadata = new()
			{
				["FleetManager"] = fleetManager.GetType().Name,
				["PoolSizeStrategy"] = poolSizeResult.Status ?? new Dictionary<string, object>(),
				["PoolId"] = pool.Id,
			};

			using IDisposable? logScope = _logger.BeginScope(logScopeMetadata);
			if (pool.LastAgentCount != currentAgentCount || pool.LastDesiredAgentCount != desiredAgentCount)
			{
				_logger.LogInformation("{PoolName} Current={Current} Target={Target} Delta={Delta}",
					pool.Name, currentAgentCount, desiredAgentCount, deltaAgentCount);
			}

			DateTime? scaleOutTime = null;
			DateTime? scaleInTime = null;

			TimeSpan scaleOutCooldown = pool.ScaleOutCooldown ?? _defaultScaleOutCooldown;
			bool isScaleOutCoolingDown = pool.LastScaleUpTime != null && pool.LastScaleUpTime + scaleOutCooldown > _clock.UtcNow;
			TimeSpan? scaleOutCooldownTimeLeft = pool.LastScaleUpTime + _defaultScaleOutCooldown - _clock.UtcNow;
			span.SetAttribute("scaleOutCooldownTimeLeftSecs", scaleOutCooldownTimeLeft?.TotalSeconds ?? -1);

			TimeSpan scaleInCooldown = pool.ScaleInCooldown ?? _defaultScaleInCooldown;
			bool isScaleInCoolingDown = pool.LastScaleDownTime != null && pool.LastScaleDownTime + scaleInCooldown > _clock.UtcNow;
			TimeSpan? scaleInCooldownTimeLeft = pool.LastScaleDownTime + _defaultScaleInCooldown - _clock.UtcNow;
			span.SetAttribute("scaleInCooldownTimeLeftSecs", scaleInCooldownTimeLeft?.TotalSeconds ?? -1);

			span.SetAttribute("isDowntimeActive", _downtimeService.IsDowntimeActive);

			ScaleResult cooldownActiveResult = new(FleetManagerOutcome.NoOp, 0, 0, "Cooldown active");
			ScaleResult result = new(FleetManagerOutcome.NoOp, 0, 0);
			try
			{
				if (deltaAgentCount > 0)
				{
					if (_downtimeService.IsDowntimeActive)
					{
						result = new(FleetManagerOutcome.NoOp, 0, 0, "Downtime is active");
					}
					else if (isScaleOutCoolingDown)
					{
						result = cooldownActiveResult;
					}
					else
					{
						result = await ExpandWithPendingShutdownsFirstAsync(pool, deltaAgentCount, (agentsToAdd) =>
							fleetManager.ExpandPoolAsync(pool, agents, agentsToAdd, cancellationToken), cancellationToken);

						scaleOutTime = _clock.UtcNow;
					}
				}
				else if (deltaAgentCount < 0)
				{
					if (isScaleInCoolingDown)
					{
						result = cooldownActiveResult;
					}
					else
					{
						result = await fleetManager.ShrinkPoolAsync(pool, agents, -deltaAgentCount, cancellationToken);
						scaleInTime = _clock.UtcNow;
					}
				}
			}
			catch (Exception ex)
			{
				_logger.LogInformation(ex, "Failed to scale {PoolName}. Reason: {Reason}", pool.Name, ex.Message);
				return new ScaleResult(FleetManagerOutcome.Failure, 0, 0, "Exception during scaling. See log.");
			}

			bool isResultDifferentFromLastTime = pool.LastScaleResult == null || !pool.LastScaleResult.Equals(result);
			bool isCooldownResult = ReferenceEquals(result, cooldownActiveResult);
			if (isResultDifferentFromLastTime && !isCooldownResult)
			{
				_logger.LogInformation("Scale result: Outcome={Outcome} AgentsAdded={AgentsAdded} AgentsRemoved={AgentsRemoved} Message={Message}",
					result.Outcome, result.AgentsAddedCount, result.AgentsRemovedCount, result.Message);
			}

			span.SetAttribute("resultOutcome", result.Outcome.ToString());
			span.SetAttribute("resultAgentsAdded", result.AgentsAddedCount);
			span.SetAttribute("resultAgentsRemoved", result.AgentsRemovedCount);
			span.SetAttribute("resultOutcome", result.Message);

			await pool.TryUpdateAsync(
				new UpdatePoolOptions
				{
					LastScaleUpTime = scaleOutTime,
					LastScaleDownTime = scaleInTime,
					LastScaleResult = isCooldownResult ? null : result,
					LastAgentCount = currentAgentCount,
					LastDesiredAgentCount = desiredAgentCount
				},
				cancellationToken);

			return result;
		}

		internal IEnumerable<string> GetPropValues(string name)
		{
			DateTime now = _clock.UtcNow;
			return name switch
			{
				"timeUtcYear" => new List<string> { now.Year.ToString() }, // as 1 to 9999
				"timeUtcMonth" => new List<string> { now.Month.ToString() }, // as 1 to 12
				"timeUtcDay" => new List<string> { now.Day.ToString() }, // as 1 to 31
				"timeUtcDayOfWeek" => new List<string> { now.DayOfWeek.ToString(), now.DayOfWeek.ToString().ToLower() },
				"timeUtcHour" => new List<string> { now.Hour.ToString() }, // as 0 to 23
				"timeUtcMin" => new List<string> { now.Minute.ToString() }, // as 0 to 59
				"timeUtcSec" => new List<string> { now.Second.ToString() }, // as 0 to 59

				"dayOfWeek" => new List<string> { now.DayOfWeek.ToString().ToLower() }, // Deprecated, use timeUtcDayOfWeek
				_ => Array.Empty<string>()
			};
		}

		/// <summary>
		/// Instantiate a fleet manager using the list of conditions/configs in <see cref="IPool" />
		/// </summary>
		/// <param name="pool">Pool to use</param>
		/// <returns>A fleet manager with parameters set, as dictated by the pool argument</returns>
		/// <exception cref="ArgumentException">If fleet manager could not be instantiated</exception>
		public IFleetManager CreateFleetManager(IPool pool)
		{
			if (pool.FleetManagers != null)
			{
				foreach (FleetManagerInfo info in pool.FleetManagers)
				{
					if (info.Condition == null || info.Condition.Evaluate(GetPropValues))
					{
						return _fleetManagerFactory.CreateFleetManager(info.Type, info.Config);
					}
				}
			}

			return _fleetManagerFactory.CreateFleetManager(FleetManagerType.Default, new JsonObject());
		}

		/// <summary>
		/// Instantiate a sizing strategy for the given pool
		/// Can resolve either from the list of strategies or the old legacy way
		/// </summary>
		/// <param name="pool">Pool to use</param>
		/// <returns>A pool sizing strategy with parameters set</returns>
		/// <exception cref="ArgumentException"></exception>
		public IPoolSizeStrategy CreatePoolSizeStrategy(IPoolConfig pool)
		{
			if (pool.SizeStrategies != null && pool.SizeStrategies.Count > 0)
			{
				foreach (PoolSizeStrategyInfo info in pool.SizeStrategies)
				{
					if (info.Condition == null || info.Condition.Evaluate(GetPropValues))
					{
						IPoolSizeStrategyFactory? factory = _poolSizeStrategyFactories.FirstOrDefault(x => x.Type == info.Type);
						if (factory == null)
						{
							throw new ArgumentException("Invalid pool size strategy type " + info.Type);
						}

						IPoolSizeStrategy strategy = factory.Create(info.Config);
						if (info.ExtraAgentCount != 0)
						{
							strategy = new ExtraAgentCountStrategy(strategy, info.ExtraAgentCount);
						}

						return strategy;
					}
				}
			}

			// These is the legacy way of creating and configuring strategies (list-based approach above is preferred)
#pragma warning disable CS0618

			switch (pool.SizeStrategy ?? _staticComputeConfig.Value.DefaultAgentPoolSizeStrategy)
			{
				case PoolSizeStrategy.JobQueue:
					return _poolSizeStrategyFactories.OfType<PoolSizeStrategyFactory<JobQueueSettings>>().First().Create(pool.JobQueueSettings ?? new JobQueueSettings());
				case PoolSizeStrategy.LeaseUtilization:
					LeaseUtilizationSettings luSettings = new();
					if (pool.MinAgents != null)
					{
						luSettings.MinAgents = pool.MinAgents.Value;
					}
					if (pool.NumReserveAgents != null)
					{
						luSettings.NumReserveAgents = pool.NumReserveAgents.Value;
					}
					return _poolSizeStrategyFactories.OfType<PoolSizeStrategyFactory<LeaseUtilizationSettings>>().First().Create(luSettings);
				case PoolSizeStrategy.NoOp:
					return new NoOpPoolSizeStrategy();
				default:
					throw new ArgumentException("Unknown pool size strategy " + pool.SizeStrategy);
			}
#pragma warning restore CS0618
		}

		/// <summary>
		/// Cancel a number of pending agent shutdowns for a pool
		/// 
		/// By preventing a shutdown, the agent can immediately be put back
		/// to service without requiring the provisioning of a new one
		/// </summary>
		/// <param name="pool">Pool to cancel shutdowns in</param>
		/// <param name="count">Number of shutdowns to cancel</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Number of pending shutdown cancelled</returns>
		private async Task<int> CancelPendingShutdownsAsync(IPool pool, int count, CancellationToken cancellationToken)
		{
			IReadOnlyList<IAgent> agents = (await _agentService.GetCachedAgentsAsync(cancellationToken))
				.Where(x => x is { Status: AgentStatus.Ok, Enabled: true })
				.Where(x => x.Pools.Contains(pool.Id))
				.ToList();

			int numShutdownsCancelled = 0;
			foreach (IAgent agent in agents)
			{
				if (agent.RequestShutdown && numShutdownsCancelled < count)
				{
					await agent.TryUpdateAsync(new UpdateAgentOptions { RequestShutdown = false }, cancellationToken: cancellationToken);
					numShutdownsCancelled++;
				}
			}

			return numShutdownsCancelled;
		}

		private async Task<ScaleResult> ExpandWithPendingShutdownsFirstAsync(IPool pool, int agentsToAdd, Func<int, Task<ScaleResult>> scaleOutFunc, CancellationToken cancellationToken)
		{
			int numShutdownsCancelled = await CancelPendingShutdownsAsync(pool, agentsToAdd, cancellationToken);
			agentsToAdd -= numShutdownsCancelled;

			Activity.Current?.SetTag("numShutdownsCancelled", numShutdownsCancelled);

			if (agentsToAdd <= 0)
			{
				return new ScaleResult(FleetManagerOutcome.Success, numShutdownsCancelled, 0, "Scaled out by only cancelling shutdowns");
			}

			ScaleResult result = await scaleOutFunc(agentsToAdd);
			return new ScaleResult(result.Outcome, result.AgentsAddedCount + numShutdownsCancelled, result.AgentsRemovedCount, result.Message);
		}
	}
}
