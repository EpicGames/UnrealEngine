// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Agents.Pools;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Available fleet managers
	/// </summary>
	public enum FleetManagerType
	{
		/// <summary>
		/// Default fleet manager
		/// </summary>
		Default,

		/// <summary>
		/// No-op fleet manager.
		/// </summary>
		NoOp,

		/// <summary>
		/// Fleet manager for handling AWS EC2 instances. Will create and/or terminate instances from scratch.
		/// </summary>
		Aws,

		/// <summary>
		/// Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
		/// </summary>
		AwsReuse,

		/// <summary>
		/// Fleet manager for handling AWS EC2 instances. Will start already existing but stopped instances to reuse existing EBS disks.
		/// </summary>
		AwsRecycle,

		/// <summary>
		/// Fleet manager for handling AWS EC2 instances. Uses an EC2 auto-scaling group for controlling the number of running instances.
		/// </summary>
		AwsAsg
	}

	/// <summary>
	/// Outcome of a scaling operation
	/// </summary>
	public enum FleetManagerOutcome
	{
		/// <summary>
		/// Scaling operation completed as intended.
		/// </summary>
		Success,

		/// <summary>
		/// Scaling operation was only partly fulfilled.
		/// </summary>
		PartialSuccess,

		/// <summary>
		/// Scaling operation failed
		/// </summary>
		Failure,

		/// <summary>
		/// No operation took place (disabled or skipped)
		/// </summary>
		NoOp
	}

	/// <summary>
	/// Result from a scaling operation
	/// </summary>
	public class ScaleResult
	{
		/// <summary>
		/// Outcome
		/// </summary>
		public FleetManagerOutcome Outcome { get; }

		/// <summary>
		/// Agents added as part of operation
		/// </summary>
		public int AgentsAddedCount { get; }

		/// <summary>
		/// Agents added as part of operation
		/// </summary>
		public int AgentsRemovedCount { get; }

		/// <summary>
		/// Human-readable log message
		/// </summary>
		public string Message { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="outcome"></param>
		/// <param name="agentsAddedCount"></param>
		/// <param name="agentsRemovedCount"></param>
		/// <param name="message"></param>
		public ScaleResult(FleetManagerOutcome outcome, int agentsAddedCount, int agentsRemovedCount, string message = "")
		{
			Outcome = outcome;
			AgentsAddedCount = agentsAddedCount;
			AgentsRemovedCount = agentsRemovedCount;
			Message = message;
		}

		/// <inheritdoc/>
		protected bool Equals(ScaleResult other)
		{
			return Outcome == other.Outcome && AgentsAddedCount == other.AgentsAddedCount && AgentsRemovedCount == other.AgentsRemovedCount;
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			if (ReferenceEquals(null, obj))
			{
				return false;
			}
			if (ReferenceEquals(this, obj))
			{
				return true;
			}
			if (obj.GetType() != GetType())
			{
				return false;
			}
			return Equals((ScaleResult)obj);
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return HashCode.Combine((int)Outcome, AgentsAddedCount, AgentsRemovedCount);
		}

		/// <inheritdoc/>
		public override string ToString()
		{
			return $"Outcome={Outcome} Added={AgentsAddedCount} Removed={AgentsRemovedCount}";
		}
	}

	/// <summary>
	/// Service to manage a fleet of machines
	/// </summary>
	public interface IFleetManager
	{
		/// <summary>
		/// Expand the given pool
		/// </summary>
		/// <param name="pool">Pool to resize</param>
		/// <param name="agents">Current list of agents in the pool</param>
		/// <param name="count">Number of agents to add</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async task</returns>
		Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken = default);

		/// <summary>
		/// Shrink the given pool
		/// </summary>
		/// <param name="pool">Pool to resize</param>
		/// <param name="agents">Current list of agents in the pool</param>
		/// <param name="count">Number of agents to remove</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async task</returns>
		Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken = default);

		/// <summary>
		/// Returns the number of stopped instances in the given pool
		/// </summary>
		/// <param name="pool">Pool to resize</param>
		/// <param name="cancellationToken">Cancellation token for the call</param>
		/// <returns>Async task</returns>
		Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken = default);
	}
}

