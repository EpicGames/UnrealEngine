// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Nodes;
using HordeServer.Agents.Pools;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Available pool sizing strategies
	/// </summary>
	public enum PoolSizeStrategy
	{
		/// <summary>
		/// Strategy based on lease utilization
		/// <see cref="LeaseUtilizationStrategy"/>
		/// </summary>
		LeaseUtilization,

		/// <summary>
		/// Strategy based on size of job build queue
		/// </summary>
		JobQueue,

		/// <summary>
		/// No-op strategy used as fallback/default behavior
		/// <see cref="NoOpPoolSizeStrategy"/> 
		/// </summary>
		NoOp,

		/// <summary>
		/// A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet
		/// <see cref="ComputeQueueAwsMetric"/> 
		/// </summary>
		ComputeQueueAwsMetric,

		/// <summary>
		/// A no-op strategy that reports metrics to let an external AWS auto-scaling policy scale the fleet
		/// <see cref="LeaseUtilizationAwsMetric"/> 
		/// </summary>
		LeaseUtilizationAwsMetric
	}

	/// <summary>
	/// Extensions for pool size strategy
	/// </summary>
	public static class PoolSizeStrategyExtensions
	{
		/// <summary>
		/// Checks whether the strategy can be called more often than normal (i.e it's cheap to run)
		/// </summary>
		/// <param name="strategy"></param>
		/// <returns>True if it supports high-frequency</returns>
		public static bool SupportsHighFrequency(this PoolSizeStrategy strategy)
		{
			return strategy switch
			{
				PoolSizeStrategy.ComputeQueueAwsMetric => true,
				_ => false
			};
		}
	}

	/// <summary>
	/// Result from calculating pool size
	/// </summary>
	public class PoolSizeResult
	{
		/// <summary>
		/// The agent count as it appeared when pool sizing was calculated
		/// </summary>
		public int CurrentAgentCount { get; }

		/// <summary>
		/// The desired agent count as calculated by pool sizing strategy
		/// </summary>
		public int DesiredAgentCount { get; }

		/// <summary>
		/// Log-friendly metadata object describing the output of the size calculation through key/values
		/// </summary>
		public IReadOnlyDictionary<string, object>? Status { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="currentAgentCount"></param>
		/// <param name="desiredAgentCount"></param>
		/// <param name="status"></param>
		public PoolSizeResult(int currentAgentCount, int desiredAgentCount, IReadOnlyDictionary<string, object>? status = null)
		{
			CurrentAgentCount = currentAgentCount;
			DesiredAgentCount = desiredAgentCount;
			Status = status;
		}
	}

	/// <summary>
	/// Interface for different agent pool sizing strategies
	/// </summary>
	public interface IPoolSizeStrategy
	{
		/// <summary>
		/// Calculate the adequate number of agents to be online for given pools
		/// </summary>
		/// <param name="pool">Pool to calculate size for</param>
		/// <param name="agents">Available agents</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>A result containing the desired agent count</returns>
		Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken = default);

		/// <summary>
		/// Name of the strategy
		/// </summary>
		string Name { get; }
	}

	/// <summary>
	/// Factory for creating <see cref="IPoolSizeStrategy"/> instances
	/// </summary>
	public interface IPoolSizeStrategyFactory
	{
		/// <summary>
		/// Strategy type
		/// </summary>
		PoolSizeStrategy Type { get; }

		/// <summary>
		/// Create a new <see cref="IPoolSizeStrategy"/> instance from the given configuration
		/// </summary>
		IPoolSizeStrategy Create(JsonObject config);
	}

	/// <summary>
	/// Base implementation of <see cref="IPoolSizeStrategyFactory"/> which deserializes options into a concrete settings type
	/// </summary>
	public abstract class PoolSizeStrategyFactory<TConfig> : IPoolSizeStrategyFactory where TConfig : new()
	{
		/// <inheritdoc/>
		public PoolSizeStrategy Type { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		protected PoolSizeStrategyFactory(PoolSizeStrategy type)
			=> Type = type;

		/// <inheritdoc/>
		public IPoolSizeStrategy Create(JsonObject? config)
			=> Create(DeserializeConfig<TConfig>(config));

		/// <summary>
		/// Creates a new strategy object from the given config
		/// </summary>
		public abstract IPoolSizeStrategy Create(TConfig config);

		private static T DeserializeConfig<T>(JsonObject? json)
		{
			T? config = JsonSerializer.Deserialize<T>(json ?? new JsonObject(), new JsonSerializerOptions { PropertyNameCaseInsensitive = true });
			if (config == null)
			{
				throw new ArgumentException("Unable to deserialize config: " + json);
			}
			return config;
		}
	}

	/// <summary>
	/// No-operation strategy that won't resize pools, just return the existing count.
	/// Used to ensure there's always a strategy available for dependency injection, even if it does nothing.
	/// </summary>
	public class NoOpPoolSizeStrategy : IPoolSizeStrategy
	{
		/// <inheritdoc/>
		public Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken)
		{
			return Task.FromResult(new PoolSizeResult(agents.Count, agents.Count));
		}

		/// <inheritdoc/>
		public string Name { get; } = "NoOp";
	}

	class NoOpPoolSizeStrategyFactory : IPoolSizeStrategyFactory
	{
		public PoolSizeStrategy Type => PoolSizeStrategy.NoOp;

		public IPoolSizeStrategy Create(JsonObject? config)
			=> new NoOpPoolSizeStrategy();
	}

	/// <summary>
	/// Pool size strategy wrapping a normal strategy and
	/// applies an extra agent count to the desired agent count.
	/// </summary>
	public class ExtraAgentCountStrategy : IPoolSizeStrategy
	{
		private readonly IPoolSizeStrategy _backingStrategy;
		private readonly int _extraAgentCount;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="backingStrategy">Strategy to be wrapped</param>
		/// <param name="extraAgentCount">Extra count to apply</param>
		public ExtraAgentCountStrategy(IPoolSizeStrategy backingStrategy, int extraAgentCount)
		{
			_backingStrategy = backingStrategy;
			_extraAgentCount = extraAgentCount;
		}

		/// <inheritdoc/>
		public async Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken)
		{
			PoolSizeResult result = await _backingStrategy.CalculatePoolSizeAsync(pool, agents, cancellationToken);
			return new PoolSizeResult(result.CurrentAgentCount, result.DesiredAgentCount + _extraAgentCount, result.Status);
		}

		/// <inheritdoc/>
		public string Name => _backingStrategy.Name;
	}
}
