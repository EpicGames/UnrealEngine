// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using System.Text.Json.Serialization;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Job queue sizing settings for a pool
	/// </summary>
	public class JobQueueSettings
	{
		/// <summary>
		/// Factor translating queue size to additional agents to grow the pool with
		/// The result is always rounded up to nearest integer. 
		/// Example: if there are 20 jobs in queue, a factor 0.25 will result in 5 new agents being added (20 * 0.25)
		/// </summary>
		public double ScaleOutFactor { get; set; } = 0.25;

		/// <summary>
		/// Factor by which to shrink the pool size with when queue is empty
		/// The result is always rounded up to nearest integer.
		/// Example: when the queue size is zero, a default value of 0.9 will shrink the pool by 10% (current agent count * 0.9)
		/// </summary>
		public double ScaleInFactor { get; set; } = 0.9;

		/// <summary>
		/// How far back in time to look for job batches (that potentially are in the queue)
		/// </summary>
		public int SamplePeriodMin { get; set; } = 60 * 24 * 1; // 1 day

		/// <summary>
		/// Time spent in ready state before considered truly waiting for an agent
		///
		/// A job batch can be in ready state before getting picked up and executed.
		/// This threshold will help ensure only batches that have been waiting longer than this value will be considered.
		/// </summary>
		public int ReadyTimeThresholdSec { get; set; } = 45;

		/// <summary>
		/// Constructor used for JSON serialization
		/// </summary>
		[JsonConstructor]
		public JobQueueSettings()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="scaleOutFactor"></param>
		/// <param name="scaleInFactor"></param>
		public JobQueueSettings(double? scaleOutFactor = null, double? scaleInFactor = null)
		{
			ScaleOutFactor = scaleOutFactor.GetValueOrDefault(ScaleOutFactor);
			ScaleInFactor = scaleInFactor.GetValueOrDefault(ScaleInFactor);
		}

		/// <inheritdoc />
		public override string ToString()
		{
			StringBuilder sb = new(50);
			sb.AppendFormat("{0}={1} ", nameof(ScaleOutFactor), ScaleOutFactor);
			sb.AppendFormat("{0}={1} ", nameof(ScaleInFactor), ScaleInFactor);
			sb.AppendFormat("{0}={1} ", nameof(SamplePeriodMin), SamplePeriodMin);
			sb.AppendFormat("{0}={1} ", nameof(ReadyTimeThresholdSec), ReadyTimeThresholdSec);
			return sb.ToString();
		}
	}
}
