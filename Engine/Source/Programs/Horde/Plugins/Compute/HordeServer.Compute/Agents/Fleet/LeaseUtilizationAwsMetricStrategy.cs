// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json.Serialization;
using Amazon.CloudWatch;
using Amazon.CloudWatch.Model;
using EpicGames.Core;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet;

/// <summary>
/// Settings for <see cref="LeaseUtilizationAwsMetricStrategy" />
/// </summary>
public class LeaseUtilizationAwsMetricSettings
{
	/// <summary>
	/// Time window backwards in time to sample for active leases
	/// A value of 60 sec, will query for active leases the last minute. 
	/// </summary>
	public int SamplePeriodSec { get; set; } = 60;

	/// <summary>
	/// AWS CloudWatch namespace to write metrics in
	/// </summary>
	public string CloudWatchNamespace { get; set; } = "Horde";

	/// <summary>
	/// Constructor used for JSON serialization
	/// </summary>
	[JsonConstructor]
	public LeaseUtilizationAwsMetricSettings()
	{
	}

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="samplePeriodSec"></param>
	/// <param name="cloudWatchNamespace"></param>
	public LeaseUtilizationAwsMetricSettings(int samplePeriodSec, string cloudWatchNamespace)
	{
		SamplePeriodSec = samplePeriodSec;
		CloudWatchNamespace = cloudWatchNamespace;
	}
}

/// <summary>
/// A no-op strategy that reports lease utilization for a given pool as AWS CloudWatch metrics
/// A metric that later can be used as a source for AWS-controlled auto-scaling policies. 
/// </summary>
public class LeaseUtilizationAwsMetricStrategy : IPoolSizeStrategy
{
	internal LeaseUtilizationAwsMetricSettings Settings { get; }

	private readonly ILeaseCollection _leaseCollection;
	private readonly IAmazonCloudWatch _cloudWatch;
	private readonly IClock _clock;
	private readonly ILogger<LeaseUtilizationAwsMetricStrategy> _logger;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="leaseCollection"></param>
	/// <param name="cloudWatch"></param>
	/// <param name="settings"></param>
	/// <param name="clock"></param>
	/// <param name="logger"></param>
	public LeaseUtilizationAwsMetricStrategy(ILeaseCollection leaseCollection, IAmazonCloudWatch cloudWatch, LeaseUtilizationAwsMetricSettings settings, IClock clock, ILogger<LeaseUtilizationAwsMetricStrategy> logger)
	{
		_leaseCollection = leaseCollection;
		_cloudWatch = cloudWatch;
		Settings = settings;
		_clock = clock;
		_logger = logger;
	}

	/// <inheritdoc/>
	public string Name { get; } = "LeaseUtilizationAwsMetric";

	/// <inheritdoc/>
	public async Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken = default)
	{
		using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(LeaseUtilizationAwsMetricStrategy)}.{nameof(CalculatePoolSizeAsync)}");
		span.SetAttribute(OpenTelemetryTracers.DatadogResourceAttribute, pool.Id.ToString());

		IReadOnlyList<ILease> leases = await _leaseCollection.FindLeasesAsync(minTime: _clock.UtcNow - TimeSpan.FromSeconds(Settings.SamplePeriodSec), cancellationToken: cancellationToken);
		leases = leases.Where(lease =>
		{
			IAgent? agent = agents.Find(a => a.Id == lease.AgentId);
			return agent != null && agent.IsInPool(pool.Id);
		}).ToList();

		DateTime now = _clock.UtcNow;
		double leaseUtilization = leases.Count / (double)agents.Count;
		leaseUtilization = Double.IsNaN(leaseUtilization) ? 0.0 : leaseUtilization;
		leaseUtilization = Math.Min(leaseUtilization, 1.0); // Clamp as agents cannot be more utilized than 100%
		leaseUtilization *= 100.0; // Normalize value to 0-100

		span.SetAttribute("cloudWatchNs", Settings.CloudWatchNamespace);
		span.SetAttribute("samplePeriodSec", Settings.SamplePeriodSec);
		span.SetAttribute("agentCount", agents.Count);
		span.SetAttribute("leaseCount", leases.Count);
		span.SetAttribute("leaseUtilizationPct", leaseUtilization);

		List<Dimension> dimensions = new() { new() { Name = "Pool", Value = pool.Id.ToString() } };
		List<MetricDatum> metricDatums = new List<MetricDatum>()
		{
			new ()
			{
				MetricName = "LeaseUtilizationPercentage",
				Dimensions = dimensions,
				Unit = StandardUnit.Percent,
				Value = leaseUtilization,
				TimestampUtc = now
			}
		};

		PutMetricDataRequest request = new() { Namespace = Settings.CloudWatchNamespace, MetricData = metricDatums };
		PutMetricDataResponse response = await _cloudWatch.PutMetricDataAsync(request, cancellationToken);

		if (response.HttpStatusCode != HttpStatusCode.OK)
		{
			_logger.LogError("Unable to put CloudWatch metrics");
		}

		// Return the input pool size data as-is since we are only observing and not modifying
		return new PoolSizeResult(agents.Count, agents.Count);
	}
}

/// <summary>
/// Factory for <see cref="LeaseUtilizationAwsMetricStrategy"/>
/// </summary>
public class LeaseUtilizationAwsMetricStrategyFactory : PoolSizeStrategyFactory<LeaseUtilizationAwsMetricSettings>
{
	readonly ILeaseCollection _leaseCollection;
	readonly IAmazonCloudWatch _cloudWatch;
	readonly IClock _clock;
	readonly ILogger<LeaseUtilizationAwsMetricStrategy> _logger;

	/// <summary>
	/// Constructor
	/// </summary>
	public LeaseUtilizationAwsMetricStrategyFactory(ILeaseCollection leaseCollection, IAmazonCloudWatch cloudWatch, IClock clock, ILogger<LeaseUtilizationAwsMetricStrategy> logger)
		: base(PoolSizeStrategy.LeaseUtilizationAwsMetric)
	{
		_leaseCollection = leaseCollection;
		_cloudWatch = cloudWatch;
		_clock = clock;
		_logger = logger;
	}

	/// <inheritdoc/>
	public override IPoolSizeStrategy Create(LeaseUtilizationAwsMetricSettings settings)
		=> new LeaseUtilizationAwsMetricStrategy(_leaseCollection, _cloudWatch, settings, _clock, _logger);
}

