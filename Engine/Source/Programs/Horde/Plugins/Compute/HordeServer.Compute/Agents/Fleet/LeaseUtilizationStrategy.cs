// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc.Tasks;
using HordeServer.Agents.Leases;
using HordeServer.Agents.Pools;
using HordeServer.Utilities;
using Microsoft.Extensions.Caching.Memory;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet
{
	/// <summary>
	/// Lease utilization sizing settings for a pool
	/// </summary>
	public class LeaseUtilizationSettings
	{
		/// <summary>
		/// Time period for each sample
		/// </summary>
		public int SampleTimeSec { get; set; } = 6 * 60;

		/// <summary>
		/// Number of samples to collect for calculating lease utilization
		/// </summary>
		public int NumSamples { get; set; } = 10;

		/// <summary>
		/// Min number of samples for a valid result
		/// </summary>
		public int NumSamplesForResult { get; set; } = 9;

		/// <summary>
		/// The minimum number of agents to keep in the pool
		/// </summary>
		public int MinAgents { get; set; } = 1;

		/// <summary>
		/// The minimum number of idle agents to hold in reserve
		/// </summary>
		public int NumReserveAgents { get; set; } = 5;

		/// <inheritdoc />
		public override string ToString()
		{
			StringBuilder sb = new(150);
			sb.AppendFormat("{0}={1} ", nameof(SampleTimeSec), SampleTimeSec);
			sb.AppendFormat("{0}={1} ", nameof(NumSamples), NumSamples);
			sb.AppendFormat("{0}={1} ", nameof(NumSamplesForResult), NumSamplesForResult);
			sb.AppendFormat("{0}={1} ", nameof(MinAgents), MinAgents);
			sb.AppendFormat("{0}={1} ", nameof(NumReserveAgents), NumReserveAgents);
			return sb.ToString();
		}
	}

	/// <summary>
	/// Calculate pool size by looking at previously finished leases
	/// </summary>
	public class LeaseUtilizationStrategy : IPoolSizeStrategy
	{
		private const string CacheKey = nameof(LeaseUtilizationStrategy);

		struct UtilizationSample
		{
			public double _jobWork;
			public double _otherWork;
		}

		class AgentData
		{
			public IAgent Agent { get; }
			public UtilizationSample[] Samples { get; }
			public List<ILease> Leases { get; } = new();
			private readonly int _numSamples;

			public AgentData(IAgent agent, int numSamples)
			{
				Agent = agent;
				_numSamples = numSamples;
				Samples = new UtilizationSample[numSamples];
			}

			public void Add(double minT, double maxT, double jobWork, double otherWork)
			{
				int minIdx = Math.Clamp((int)minT, 0, _numSamples - 1);
				int maxIdx = Math.Clamp((int)maxT, minIdx, _numSamples - 1);
				for (int idx = minIdx; idx <= maxIdx; idx++)
				{
					double fraction = Math.Clamp(maxT - idx, 0.0, 1.0) - Math.Clamp(minT - idx, 0.0, 1.0);
					Samples[idx]._jobWork += jobWork * fraction;
					Samples[idx]._otherWork += otherWork * fraction;
				}
			}
		}

		class PoolData
		{
			public IPoolConfig Pool { get; }
			public List<IAgent> Agents { get; } = new();
			public UtilizationSample[] Samples { get; }

			public PoolData(IPoolConfig pool, int numSamples)
			{
				Pool = pool;
				Samples = new UtilizationSample[numSamples];
			}

			public void Add(AgentData agentData)
			{
				Agents.Add(agentData.Agent);
				for (int idx = 0; idx < Samples.Length; idx++)
				{
					Samples[idx]._jobWork += agentData.Samples[idx]._jobWork;
					Samples[idx]._otherWork += agentData.Samples[idx]._otherWork;
				}
			}
		}

		internal LeaseUtilizationSettings Settings { get; }

		private readonly AgentService _agentService;
		private readonly IPoolCollection _poolCollection;
		private readonly ILeaseCollection _leaseCollection;
		private readonly IClock _clock;
		private readonly IMemoryCache _cache;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="agentService"></param>
		/// <param name="poolCollection"></param>
		/// <param name="leaseCollection"></param>
		/// <param name="clock"></param>
		/// <param name="cache"></param>
		/// <param name="settings"></param>
		public LeaseUtilizationStrategy(AgentService agentService, IPoolCollection poolCollection, ILeaseCollection leaseCollection, IClock clock, IMemoryCache cache, LeaseUtilizationSettings settings)
		{
			_agentService = agentService;
			_poolCollection = poolCollection;
			_leaseCollection = leaseCollection;
			_clock = clock;
			_cache = cache;
			Settings = settings;
		}

		private async Task<Dictionary<AgentId, AgentData>> GetAgentDataAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(LeaseUtilizationStrategy)}.{nameof(GetAgentDataAsync)}");

			// Find all the current agents
			IReadOnlyList<IAgent> agents = (await _agentService.GetCachedAgentsAsync(cancellationToken)).Where(x => x.Status == AgentStatus.Ok).ToList();

			// Query leases in last interval
			DateTime maxTime = _clock.UtcNow;
			DateTime minTime = maxTime - TimeSpan.FromSeconds(Settings.SampleTimeSec) * Settings.NumSamples;
			IReadOnlyList<ILease> leases = await _leaseCollection.FindLeasesAsync(minTime, maxTime, cancellationToken: cancellationToken);

			// Add all the leases to a data object for each agent
			Dictionary<AgentId, AgentData> agentIdToData = agents.ToDictionary(x => x.Id, x => new AgentData(x, Settings.NumSamples));
			foreach (ILease lease in leases)
			{
				AgentData? agentData;
				if (agentIdToData.TryGetValue(lease.AgentId, out agentData) && agentData.Agent.SessionId == lease.SessionId)
				{
					agentData.Leases.Add(lease);
				}
			}

			// Compute utilization for each agent
			foreach (AgentData agentData in agentIdToData.Values)
			{
				foreach (ILease lease in agentData.Leases.OrderBy(x => x.StartTime))
				{
					double minT = (lease.StartTime - minTime).TotalSeconds / Settings.SampleTimeSec;
					double maxT = (lease.FinishTime == null)
						? Settings.NumSamples
						: ((lease.FinishTime.Value - minTime).TotalSeconds / Settings.SampleTimeSec);

					Any payload = Any.Parser.ParseFrom(lease.Payload.ToArray());
					if (payload.Is(ExecuteJobTask.Descriptor) || payload.Is(ComputeTask.Descriptor))
					{
						agentData.Add(minT, maxT, 1.0, 0.0);
					}
					else
					{
						agentData.Add(minT, maxT, 0.0, 1.0);
					}
				}
			}

			span.SetAttribute("agentDataCount", agentIdToData.Count);
			return agentIdToData;
		}

		private async Task<Dictionary<PoolId, PoolData>> GetPoolDataAsync(CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(LeaseUtilizationStrategy)}.{nameof(GetPoolDataAsync)}");

			Dictionary<AgentId, AgentData> agentIdToData = await GetAgentDataAsync(cancellationToken);

			// Get all the pools
			IReadOnlyList<IPoolConfig> pools = await _poolCollection.GetConfigsAsync(cancellationToken);
			Dictionary<PoolId, PoolData> poolToData = pools.ToDictionary(x => x.Id, x => new PoolData(x, Settings.NumSamples));

			// Find pool utilization over the query period
			foreach (AgentData agentData in agentIdToData.Values)
			{
				foreach (PoolId poolId in agentData.Agent.Pools)
				{
					PoolData? poolData;
					if (poolToData.TryGetValue(poolId, out poolData))
					{
						poolData.Add(agentData);
					}
				}
			}

			span.SetAttribute("poolDataCount", agentIdToData.Count);
			return poolToData;
		}

		/// <inheritdoc/>
		public string Name { get; } = "LeaseUtilization";

		/// <inheritdoc/>
		public async Task<PoolSizeResult> CalculatePoolSizeAsync(IPoolConfig pool, List<IAgent> agents, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(LeaseUtilizationStrategy)}.{nameof(CalculatePoolSizeAsync)}");
			span.SetAttribute(OpenTelemetryTracers.DatadogResourceAttribute, pool.Id.ToString());
			span.SetAttribute("currentAgentCount", agents.Count);

			Dictionary<PoolId, PoolData>? poolToData;

			// Cache pool data for a short while for faster runs when many pools are scaled
			if (!_cache.TryGetValue(CacheKey, out poolToData) || poolToData == null)
			{
				// Pool sizes haven't been cached, update them (might happen from multiple tasks but that is fine)
				poolToData = await GetPoolDataAsync(cancellationToken);
				_cache.Set(CacheKey, poolToData, TimeSpan.FromSeconds(60));
			}

			PoolData poolData = poolToData[pool.Id];

			double utilization = poolData.Samples.Select(x => x._jobWork).OrderByDescending(x => x).Skip(Settings.NumSamples - Settings.NumSamplesForResult).First();

			// Number of agents in use over the sampling period. Can never be greater than number of agents available in pool.
			int numAgentsUtilized = (int)utilization;

			// Include reserve agent count to ensure pool always can grow
			int desiredAgentCount = Math.Max(numAgentsUtilized + Settings.NumReserveAgents, Settings.MinAgents);

			Dictionary<string, object> status = new()
			{
				["Name"] = GetType().Name,
				["Jobs"] = GetDensityMap(poolData.Samples.Select(x => x._jobWork / Math.Max(1, poolData.Agents.Count))),
				["Total"] = GetDensityMap(poolData.Samples.Select(x => x._otherWork / Math.Max(1, poolData.Agents.Count))),
				["Min"] = poolData.Samples.Min(x => x._jobWork),
				["Max"] = poolData.Samples.Max(x => x._jobWork),
				["Avg"] = utilization,
				["Pct"] = poolData.Samples.Sum(x => x._jobWork) / Settings.NumSamples,
				["SampleTimeSec"] = Settings.SampleTimeSec,
				["NumSamples"] = Settings.NumSamples,
				["NumSamplesForResult"] = Settings.NumSamplesForResult,
				["MinAgents"] = Settings.MinAgents,
				["NumReserveAgents"] = Settings.NumReserveAgents,
			};

			return new PoolSizeResult(agents.Count, desiredAgentCount, status);
		}

		/// <summary>
		/// Creates a string of characters indicating a sequence of 0-1 values over time
		/// </summary>
		/// <param name="values">Sequence of values</param>
		/// <returns>Density map</returns>
		private static string GetDensityMap(IEnumerable<double> values)
		{
			const string Greyscale = " 123456789";
			return new string(values.Select(x => Greyscale[Math.Clamp((int)(x * Greyscale.Length), 0, Greyscale.Length - 1)]).ToArray());
		}
	}

	/// <summary>
	/// Factory for <see cref="LeaseUtilizationStrategy"/>
	/// </summary>
	public class LeaseUtilizationStrategyFactory : PoolSizeStrategyFactory<LeaseUtilizationSettings>
	{
		readonly AgentService _agentService;
		readonly IPoolCollection _poolCollection;
		readonly ILeaseCollection _leaseCollection;
		readonly IClock _clock;
		readonly IMemoryCache _memoryCache;

		/// <summary>
		/// Constructor
		/// </summary>
		public LeaseUtilizationStrategyFactory(AgentService agentService, IPoolCollection poolCollection, ILeaseCollection leaseCollection, IClock clock, IMemoryCache memoryCache)
			: base(PoolSizeStrategy.LeaseUtilization)
		{
			_agentService = agentService;
			_poolCollection = poolCollection;
			_leaseCollection = leaseCollection;
			_clock = clock;
			_memoryCache = memoryCache;
		}

		/// <inheritdoc/>
		public override IPoolSizeStrategy Create(LeaseUtilizationSettings settings)
			=> new LeaseUtilizationStrategy(_agentService, _poolCollection, _leaseCollection, _clock, _memoryCache, settings);
	}
}
