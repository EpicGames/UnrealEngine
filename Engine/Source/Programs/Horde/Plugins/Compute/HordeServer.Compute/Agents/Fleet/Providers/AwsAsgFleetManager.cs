// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using System.Text.Json.Serialization;
using Amazon.AutoScaling;
using Amazon.AutoScaling.Model;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet.Providers
{
	/// <summary>
	/// Settings for controlling an EC2 auto-scaling group (ASG)
	/// </summary>
	public class AwsAsgSettings
	{
		/// <summary>
		/// Name of the auto-scaling group
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name</param>
		[JsonConstructor]
		public AwsAsgSettings(string? name)
		{
			// The JSON deserializer can call the constructor with null parameters
			Name = name ?? throw new ArgumentException($"Parameter {nameof(name)} is null");
		}
	}

	/// <summary>
	/// Fleet manager for handling AWS EC2 instances
	/// Uses an EC2 auto-scaling group for controlling the number of running instances.
	/// </summary>
	public sealed class AwsAsgFleetManager : IFleetManager
	{
		internal AwsAsgSettings Settings { get; }
		private readonly IAmazonAutoScaling _awsAutoScaling;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AwsAsgFleetManager(IAmazonAutoScaling awsAutoScaling, AwsAsgSettings asgSettings, Tracer tracer, ILogger<AwsAsgFleetManager> logger)
		{
			_awsAutoScaling = awsAutoScaling;
			Settings = asgSettings;
			_tracer = tracer;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			int desiredCapacity = agents.Count + count;

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAsgFleetManager)}.{nameof(ExpandPoolAsync)}");
			span.SetAttribute("poolName", pool.Name);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("count", count);
			span.SetAttribute("desiredCapacity", desiredCapacity);

			await UpdateAsgAsync(pool.Id, desiredCapacity, span, cancellationToken);
			return new ScaleResult(FleetManagerOutcome.Success, count, 0);
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			int desiredCapacity = Math.Max(0, agents.Count - count);

			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsAsgFleetManager)}.{nameof(ShrinkPoolAsync)}");
			span.SetAttribute("poolName", pool.Name);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("count", count);
			span.SetAttribute("desiredCapacity", desiredCapacity);

			await UpdateAsgAsync(pool.Id, desiredCapacity, span, cancellationToken);
			return new ScaleResult(FleetManagerOutcome.Success, 0, count);
		}

		private async Task UpdateAsgAsync(PoolId poolId, int desiredCapacity, TelemetrySpan span, CancellationToken cancellationToken)
		{
			UpdateAutoScalingGroupRequest request = new() { AutoScalingGroupName = Settings.Name, DesiredCapacity = desiredCapacity };
			UpdateAutoScalingGroupResponse response = await _awsAutoScaling.UpdateAutoScalingGroupAsync(request, cancellationToken);

			span.SetAttribute("res.statusCode", (int)response.HttpStatusCode);
			if (response.HttpStatusCode != HttpStatusCode.OK)
			{
				_logger.LogError("Unable to update auto-scaling group {AutoScalingGroup} for pool {PoolId}.", Settings.Name, poolId);
			}
		}

		/// <inheritdoc/>
		public Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken)
		{
			return Task.FromResult(0);
		}
	}
}
