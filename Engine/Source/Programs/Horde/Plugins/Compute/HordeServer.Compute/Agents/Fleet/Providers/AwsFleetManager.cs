// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.EC2;
using Amazon.EC2.Model;
using EpicGames.Horde.Agents;
using HordeServer.Agents.Pools;
using HordeServer.Auditing;
using HordeServer.Utilities;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet.Providers
{
	/// <summary>
	/// Settings for the AWS fleet manager
	/// </summary>
	public class AwsFleetManagerSettings
	{
		/// <summary>
		/// AWS region (e.g us-east-1)
		/// </summary>
		public string Region { get; }

		/// <summary>
		/// Amazon Machine Image (AMI) to use (e.g ami-0c2e245e4936a2ab2)
		/// </summary>
		public string ImageId { get; }

		/// <summary>
		/// An EC2 instance type (see https://docs.aws.amazon.com/AWSEC2/latest/UserGuide/instance-types.html for full list)
		/// </summary>
		public string InstanceType { get; }

		/// <summary>
		/// IAM	instance profile 
		/// </summary>
		public string? IamInstanceProfile { get; }

		/// <summary>
		/// Key/value tags for the instance
		/// </summary>
		public IReadOnlyDictionary<string, string> Tags { get; }

		/// <summary>
		/// List of subnet IDs this instance can run in. Will implicitly pick the VPC.
		/// A random subnet ID will be picked during instance creation.
		/// </summary>
		public List<string> SubnetIds { get; }

		/// <summary>
		/// Security group IDs to apply. Must exist in the same VPC as the subnets.
		/// </summary>
		public List<string> SecurityGroupIds { get; }

		/// <summary>
		/// User data to pass to the instance. Commonly used for cloud-init scripts that configures the instance during boot up. 
		/// </summary>
		public string? UserData { get; }

		/// <summary>
		/// Key name for the instance (optional).
		/// If skipped, alternative ways to administer the instance must be enabled, such as the AWS SSM Agent. 
		/// </summary>
		public string? KeyName { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="region"></param>
		/// <param name="imageId"></param>
		/// <param name="instanceType"></param>
		/// <param name="iamInstanceProfile"></param>
		/// <param name="tags"></param>
		/// <param name="subnetIds"></param>
		/// <param name="securityGroupIds"></param>
		/// <param name="userData"></param>
		/// <param name="keyName"></param>
		public AwsFleetManagerSettings(string region, string imageId, string instanceType, string? iamInstanceProfile, IReadOnlyDictionary<string, string> tags,
			List<string> subnetIds, List<string> securityGroupIds, string? userData, string? keyName)
		{
			Region = region;
			ImageId = imageId;
			InstanceType = instanceType;
			IamInstanceProfile = iamInstanceProfile;
			Tags = tags;
			SubnetIds = subnetIds;
			SecurityGroupIds = securityGroupIds;
			UserData = userData;
			KeyName = keyName;
		}

		/// <summary>
		/// Get a random subnet ID. Used for 
		/// </summary>
		/// <returns>A random subnet ID from the list of subnets</returns>
		public string GetRandomSubnetId()
		{
			int index = new Random().Next(SubnetIds.Count);
			return SubnetIds[index];
		}

		/// <summary>
		/// Get user-data as base64
		/// Required when interacting with the EC2 API.
		/// </summary>
		/// <returns>User-data encoded as Base64</returns>
		public string? GetUserDataAsBase64()
		{
			if (UserData == null)
			{
				return UserData;
			}

			byte[] plainTextBytes = System.Text.Encoding.UTF8.GetBytes(UserData);
			return Convert.ToBase64String(plainTextBytes);
		}
	}

	/// <summary>
	/// Fleet manager for handling AWS EC2 instances
	/// Will create and/or terminate instances from scratch.
	/// </summary>
	public sealed class AwsFleetManager : IFleetManager
	{
		/// <summary>
		/// EC2 tag prefix signaling an instance belong to agent pool, through pool name
		/// Deprecated in favor of <see cref="AwsTagPoolId" /> as pool ID is the stable reference to a pool. 
		/// </summary>
		public const string AwsTagPoolName = "Horde_Autoscale_Pool";
		
		/// <summary>
		/// EC2 tag prefix signaling an instance belong to agent pool, through pool ID
		/// </summary>
		public const string AwsTagPoolId = "Horde_Autoscale_PoolId";
		
		private const string AwsTagPropertyName = "aws-tag";

		/// <summary>
		/// Settings for fleet manager
		/// </summary>
		public AwsFleetManagerSettings Settings { get; }

		private readonly IAmazonEC2 _ec2;
		private readonly IAgentCollection _agentCollection;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AwsFleetManager(IAmazonEC2 ec2, IAgentCollection agentCollection, AwsFleetManagerSettings settings, Tracer tracer, ILogger<AwsFleetManager> logger)
		{
			_ec2 = ec2;
			_agentCollection = agentCollection;
			Settings = settings;
			_tracer = tracer;
			_logger = logger;
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsFleetManager)}.{nameof(ExpandPoolAsync)}");
			span.SetAttribute("poolName", pool.Name);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("count", count);

			List<Tag> tags = Settings.Tags.Select(x => new Tag(x.Key, x.Value)).ToList();
			tags.Add(new Tag("Horde:RequestedPools", pool.Name));
			tags.Add(new Tag("Horde:Timestamp:Ec2InstanceLaunchRequested", Convert.ToString(DateTimeOffset.UtcNow.ToUnixTimeMilliseconds())));

			RunInstancesRequest request = new()
			{
				ImageId = Settings.ImageId,
				InstanceType = InstanceType.FindValue(Settings.InstanceType),
				KeyName = Settings.KeyName,
				SecurityGroupIds = Settings.SecurityGroupIds,
				SubnetId = Settings.GetRandomSubnetId(), // Pick randomly to spread evenly among available subnets 
				MinCount = count,
				MaxCount = count,
				UserData = Settings.GetUserDataAsBase64(),
				InstanceInitiatedShutdownBehavior = ShutdownBehavior.Terminate,
				MetadataOptions = new InstanceMetadataOptionsRequest
				{
					HttpEndpoint = InstanceMetadataEndpointState.Enabled,
					InstanceMetadataTags = InstanceMetadataTagsState.Enabled,
				}
			};

			if (Settings.IamInstanceProfile != null)
			{
				request.IamInstanceProfile =
					Settings.IamInstanceProfile.StartsWith("arn:aws", StringComparison.InvariantCulture)
						? new IamInstanceProfileSpecification { Arn = Settings.IamInstanceProfile }
						: new IamInstanceProfileSpecification { Name = Settings.IamInstanceProfile };
			}

			if (tags.Count > 0)
			{
				request.TagSpecifications = new() { new() { ResourceType = ResourceType.Instance, Tags = tags } };
			}

			RunInstancesResponse response = await _ec2.RunInstancesAsync(request, cancellationToken);
			int numStartedInstances = response.Reservation.Instances.Count;
			span.SetAttribute("res.statusCode", (int)response.HttpStatusCode);
			span.SetAttribute("res.numInstances", numStartedInstances);

			foreach (Instance i in response.Reservation.Instances)
			{
				_logger.LogInformation("Created instance {InstanceId} for pool {PoolId}", i.InstanceId, pool.Id);
			}

			if (numStartedInstances == count)
			{
				return new ScaleResult(FleetManagerOutcome.Success, numStartedInstances, 0);
			}

			if (numStartedInstances != count)
			{
				_logger.LogWarning("Unable to create all the requested instances for pool {PoolId}. RequestedCount={RequestedCount} ActualCount={ActualCount}", pool.Id, count, numStartedInstances);
			}

			return new ScaleResult(numStartedInstances == 0 ? FleetManagerOutcome.Failure : FleetManagerOutcome.Success, numStartedInstances, 0);
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			await ShrinkPoolViaAgentShutdownRequestAsync(_agentCollection, pool, agents, count, cancellationToken);
			return new ScaleResult(FleetManagerOutcome.Success, 0, count);
		}

		/// <summary>
		/// Shrink pool by gracefully requesting a shutdown for the agent
		/// Allowing the agent to terminate at next best possible moment (usually when all the leases have finished).
		/// </summary>
		/// <param name="agentCollection">Instance of the AgentCollection</param>
		/// <param name="pool">Pool to resize</param>
		/// <param name="agents">Current list of agents in the pool</param>
		/// <param name="count">Number of agents to remove</param>
		/// <param name="cancellationToken"></param>
		public static async Task ShrinkPoolViaAgentShutdownRequestAsync(IAgentCollection agentCollection, IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			_ = cancellationToken;

			using TelemetrySpan span = OpenTelemetryTracers.Horde.StartActiveSpan($"{nameof(AwsFleetManager)}.{nameof(ShrinkPoolViaAgentShutdownRequestAsync)}");
			span.SetAttribute("poolName", pool.Name);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("count", count);

			string awsTagPoolNameProperty = $"{AwsTagPropertyName}={AwsTagPoolName}:{pool.Name}";
			string awsTagPoolIdProperty = $"{AwsTagPropertyName}={AwsTagPoolId}:{pool.Id.ToString()}";

			// Sort the agents by number of active leases. It's better to shut down agents currently doing nothing.
			List<IAgent> filteredAgents = agents.OrderBy(x => x.Leases.Count).ToList();
			List<IAgent> agentsWithAwsTags = filteredAgents.Where(x => x.HasProperty(awsTagPoolNameProperty) || x.HasProperty(awsTagPoolIdProperty)).ToList();
			List<IAgent> agentsLimitedByCount = agentsWithAwsTags.Take(count).ToList();

			span.SetAttribute("agents.num", agents.Count);
			span.SetAttribute("agents.filtered.num", filteredAgents.Count);
			span.SetAttribute("agents.withAwsTags.num", agentsWithAwsTags.Count);
			span.SetAttribute("agents.limitedByCount.num", agentsLimitedByCount.Count);

			foreach (IAgent agent in agentsLimitedByCount)
			{
				await TryRequestShutdownAsync(agentCollection, pool, agent, cancellationToken);
			}
		}

		private static async Task<bool> TryRequestShutdownAsync(IAgentCollection agentCollection, IPool pool, IAgent agent, CancellationToken cancellationToken)
		{
			IAuditLogChannel<AgentId> agentLogger = agentCollection.GetLogger(agent.Id);

			const int MaxRetries = 5;
			for (int retryCount = 0; retryCount < MaxRetries; retryCount++)
			{
				if (await agent.TryUpdateAsync(new UpdateAgentOptions { RequestShutdown = true, ShutdownReason = "Autoscaler" }, cancellationToken: cancellationToken) != null)
				{
					agentLogger.LogInformation("Marked {AgentId} in pool {PoolName} for shutdown due to autoscaling (currently {NumLeases} leases outstanding, {NumRetries} retries)", agent.Id, pool.Name, agent.Leases.Count, retryCount);
					return true;
				}

				IAgent? updatedAgent = await agentCollection.GetAsync(agent.Id, cancellationToken);
				if (updatedAgent == null)
				{
					agentLogger.LogError("Unable to mark agent {AgentId} in pool {PoolName} for shutdown due to autoscaling. Agent no longer exists", agent.Id, pool.Name);
					return false;
				}

				agent = updatedAgent;
			}

			agentLogger.LogError("Unable to mark agent {AgentId} in pool {PoolName} for shutdown due to autoscaling. Retries exhausted", agent.Id, pool.Name);
			return false;
		}

		/// <inheritdoc/>
		public async Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken)
		{
			return (await GetInstancesInPoolAsync(_ec2, InstanceStateName.Stopped, pool, _tracer, cancellationToken)).Count;
		}
		
		/// <summary>
		/// Get instances that are part of given Horde agent pool
		/// </summary>
		/// <param name="ec2">EC2 client</param>
		/// <param name="instanceState">Optional instance state to filter on</param>
		/// <param name="pool">Pool to filter on</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>A list of EC2 instances</returns>
		public static Task<List<Instance>> GetInstancesInPoolAsync(IAmazonEC2 ec2, InstanceStateName? instanceState, IPoolConfig pool, Tracer tracer, CancellationToken cancellationToken)
		{
			return GetInstancesInPoolAsync(ec2, instanceState, pool.Id.ToString(), pool.Name, tracer, cancellationToken);
		}
		
		/// <summary>
		/// Get instances that are part of given Horde agent pool
		/// </summary>
		/// <param name="ec2">EC2 client</param>
		/// <param name="instanceState">Optional instance state to filter on</param>
		/// <param name="poolId">Pool ID to filter on</param>
		/// <param name="poolName">Pool ID to filter on</param>
		/// <param name="tracer">Tracer</param>
		/// <param name="cancellationToken">Cancellation token</param>
		/// <returns>A list of EC2 instances</returns>
		public static async Task<List<Instance>> GetInstancesInPoolAsync(IAmazonEC2 ec2, InstanceStateName? instanceState, string poolId, string poolName, Tracer tracer, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = tracer.StartActiveSpan($"{nameof(AwsFleetManager)}.{nameof(GetInstancesInPoolAsync)}");
			
			List<Filter> poolIdFilters = [new Filter($"tag:{AwsTagPoolId}", [poolId])];
			List<Filter> poolNameFilters = [new Filter($"tag:{AwsTagPoolName}", [poolName])];
			if (instanceState != null)
			{
				poolIdFilters.Add(new Filter("instance-state-name", [instanceState.Value]));
				poolNameFilters.Add(new Filter("instance-state-name", [instanceState.Value]));
			}
			
			List<Instance> result = [];
			{
				// Get matching instances by pool *name*
				using TelemetrySpan ec2Span = tracer.StartActiveSpan($"DescribeInstances");
				DescribeInstancesRequest request = new() { Filters = poolNameFilters };
				DescribeInstancesResponse response = await ec2.DescribeInstancesAsync(request, cancellationToken);
				result.AddRange(response.Reservations.SelectMany(x => x.Instances));
				ec2Span.SetAttribute("res.statusCode", (int)response.HttpStatusCode);
				ec2Span.SetAttribute("res.numReservations", response.Reservations.Count);
			}
			{
				// Get matching instances by pool *ID*
				using TelemetrySpan ec2Span = tracer.StartActiveSpan($"DescribeInstances");
				DescribeInstancesRequest request = new() { Filters = poolIdFilters };
				DescribeInstancesResponse response = await ec2.DescribeInstancesAsync(request, cancellationToken);
				result.AddRange(response.Reservations.SelectMany(x => x.Instances));
				ec2Span.SetAttribute("res.statusCode", (int)response.HttpStatusCode);
				ec2Span.SetAttribute("res.numReservations", response.Reservations.Count);
			}
			
			return result.DistinctBy(x => x.InstanceId).ToList();
		}
	}
}
