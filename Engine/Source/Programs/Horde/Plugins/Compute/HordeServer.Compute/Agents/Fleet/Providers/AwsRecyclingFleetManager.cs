// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.Metrics;
using System.Net;
using Amazon.EC2;
using Amazon.EC2.Model;
using Amazon.Runtime;
using EpicGames.Core;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet.Providers;

/// <summary>
/// Settings for AWS fleet manager that reuses instances
/// </summary>
public class AwsRecyclingFleetManagerSettings
{
	/// <summary>
	/// Prioritized list of instance types to use waking up an instance
	/// If set to null, the instance will be awakened with its last instance type.
	/// </summary>
	public List<string>? InstanceTypes { get; }

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="instanceTypes"></param>
	public AwsRecyclingFleetManagerSettings(List<string>? instanceTypes = null)
	{
		InstanceTypes = instanceTypes;
	}
}

/// <summary>
/// Base exception for AWS fleet managers
/// </summary>
public class AwsFleetManagerException : Exception
{
	/// <inheritdoc />
	public AwsFleetManagerException() { }

	/// <inheritdoc />
	public AwsFleetManagerException(string message) : base(message) { }
}

/// <summary>
/// Generic HTTP response error from AWS API
/// </summary>
public class AwsBadHttpResponseException : AwsFleetManagerException
{
	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="statusCode"></param>
	public AwsBadHttpResponseException(HttpStatusCode statusCode) : base($"HTTP status code {statusCode}") { }
}

/// <summary>
/// Fleet manager for handling AWS EC2 instances
/// Will start already existing but stopped instances to reuse existing EBS disks.
/// See <see cref="AwsFleetManager" /> for creating and terminating instances from scratch.
/// </summary>
public sealed class AwsRecyclingFleetManager : IFleetManager
{
	private static readonly Random s_random = new();

	/// <summary>
	/// Settings for fleet manager
	/// </summary>
	public AwsRecyclingFleetManagerSettings Settings { get; }

	private readonly IAmazonEC2 _ec2;
	private readonly IAgentCollection _agentCollection;
	private readonly Meter _meter;
	private readonly IClock _clock;
	private readonly Tracer _tracer;
	private readonly ILogger _logger;
	private readonly Counter<int> _ec2StartInstanceCounter;

	/// <summary>
	/// Constructor
	/// </summary>
	public AwsRecyclingFleetManager(IAmazonEC2 ec2, IAgentCollection agentCollection, Meter meter, IClock clock, AwsRecyclingFleetManagerSettings settings, Tracer tracer, ILogger<AwsRecyclingFleetManager> logger)
	{
		_ec2 = ec2;
		_agentCollection = agentCollection;
		_meter = meter;
		_clock = clock;
		_tracer = tracer;
		_logger = logger;
		Settings = settings;
		_ec2StartInstanceCounter = _meter.CreateCounter<int>("horde.fleet.awsRecycling.startInstance");
	}

	/// <inheritdoc/>
	public async Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int requestedInstancesCount, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(ExpandPoolAsync)}");
		span.SetAttribute("poolId", pool.Id.ToString());
		span.SetAttribute("poolName", pool.Name);
		span.SetAttribute("numAgents", agents.Count);
		span.SetAttribute("count", requestedInstancesCount);

		using IDisposable? logScope = _logger.BeginScope(new Dictionary<string, object> { ["PoolId"] = pool.Id });

		await StopStuckPendingInstancesAsync(pool, cancellationToken);
		Dictionary<string, List<Instance>> candidatesPerAz = await GetCandidateInstancesAsync(pool, cancellationToken);
		Dictionary<string, int> instanceCountPerAz = candidatesPerAz.ToDictionary(x => x.Key, x => x.Value.Count);
		Dictionary<string, int> requestCountPerAz = DistributeInstanceRequestsOverAzs(requestedInstancesCount, instanceCountPerAz, out int stoppedInstancesMissingCount);
		Dictionary<string, List<Instance>> stoppedInstancesPerAz = GetInstancesToLaunch(candidatesPerAz, requestCountPerAz);
		List<InstanceType>? instanceTypePriority = Settings.InstanceTypes?.Select(InstanceType.FindValue).ToList();
		int instancesToStartCount = requestCountPerAz.Values.Sum(x => x);

		span.SetAttribute("InstancesToStartCount", instancesToStartCount);
		span.SetAttribute("RequestedInstancesCount", requestedInstancesCount);
		span.SetAttribute("CurrentAgentCount", agents.Count);
		span.SetAttribute("StoppedInstancesMissingCount", stoppedInstancesMissingCount);
		span.SetAttribute("InstancesToStartCount", instancesToStartCount);
		span.SetAttribute("StoppedInstancesMissingCount", stoppedInstancesMissingCount);

		if (instancesToStartCount == 0)
		{
			return new ScaleResult(
				FleetManagerOutcome.Failure,
				0,
				0,
				"Unable to start any instance(s). No stopped instances are available");
		}

		foreach ((string az, List<Instance> instances) in stoppedInstancesPerAz)
		{
			using (_logger.WithProperty("Az", az).BeginScope())
			{
				_logger.LogDebug("Trying to start {InstanceCount} instance(s)", instances.Count);
				await StartInstancesWithRetriesAsync(instances, instanceTypePriority, cancellationToken);
			}
		}

		if (stoppedInstancesMissingCount > 0)
		{
			return new ScaleResult(
				FleetManagerOutcome.PartialSuccess,
				instancesToStartCount,
				0,
				$"Starting {instancesToStartCount} instance(s) but not enough stopped instances to accommodate the full pool scale-out");
		}

		return new ScaleResult(
			FleetManagerOutcome.Success,
			instancesToStartCount,
			0,
			$"Starting {instancesToStartCount} instance(s)");
	}

	private static Dictionary<string, List<Instance>> GetInstancesToLaunch(
		Dictionary<string, List<Instance>> candidatesPerAz,
		Dictionary<string, int> requestCountPerAz)
	{
		Dictionary<string, List<Instance>> result = new();
		foreach ((string az, List<Instance> candidates) in candidatesPerAz)
		{
			result[az] = candidates.OrderByDescending(x => x.LaunchTime).Take(requestCountPerAz[az]).ToList();
		}

		return result;
	}

	/// <summary>
	/// Balance number of instance requests evenly over given availability zones (AZs)
	/// </summary>
	/// <param name="requestedInstancesCount">Number of instances requested</param>
	/// <param name="instanceCountPerAz">Number of instances available per AZ</param>
	/// <param name="remainingInstancesCount">Number of instances that can't fit within the given AZ counts</param>
	/// <param name="randomizeAzOrdering">Spread requests evenly by randomizing AZ order. Disabled during testing</param>
	/// <returns>Number of instances to request from each AZ</returns>
	internal static Dictionary<string, int> DistributeInstanceRequestsOverAzs(
		int requestedInstancesCount,
		IReadOnlyDictionary<string, int> instanceCountPerAz,
		out int remainingInstancesCount,
		bool randomizeAzOrdering = true)
	{
		Dictionary<string, int> availableCountPerAz = new(instanceCountPerAz);
		Dictionary<string, int> requestCountPerAz = availableCountPerAz.ToDictionary(x => x.Key, _ => 0);

		List<string> availabilityZones = availableCountPerAz.Keys.OrderBy(x => x).ToList();
		if (randomizeAzOrdering)
		{
			availabilityZones = availabilityZones.OrderBy(_ => s_random.Next()).ToList();
		}

		int instancesAvailableCount = availableCountPerAz.Sum(x => x.Value);
		int instancesAssignedCount = 0;

		while (instancesAvailableCount > 0 && instancesAssignedCount < requestedInstancesCount)
		{
			foreach (string az in availabilityZones)
			{
				int count = availableCountPerAz[az];
				if (count > 0 && instancesAssignedCount < requestedInstancesCount)
				{
					requestCountPerAz[az]++;
					availableCountPerAz[az]--;
					instancesAssignedCount++;
					instancesAvailableCount--;
				}
			}
		}

		remainingInstancesCount = requestedInstancesCount - instancesAssignedCount;
		return requestCountPerAz;
	}

	/// <summary>
	/// Wake up already stopped instances
	/// If an insufficient capacity error, all instance types will be changed according to the priority type list.
	/// Capacity errors typically occurs per instance type, so retrying using a different one may circumvent the issue. 
	/// </summary>
	/// <param name="instances">List of stopped instances to start</param>
	/// <param name="instanceTypePriority">List of instance types to fallback on</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	private async Task StartInstancesWithRetriesAsync(List<Instance> instances, IReadOnlyList<InstanceType>? instanceTypePriority, CancellationToken cancellationToken)
	{
		List<string> instanceIds = instances.Select(x => x.InstanceId).ToList();
		string instanceTypePriorityString = instanceTypePriority == null ? "" : String.Join(',', instanceTypePriority.Select(x => x.Value));
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(StartInstancesWithRetriesAsync)}");
		span.SetAttribute("instances", String.Join(',', instanceIds));
		span.SetAttribute("instanceTypePriority", instanceTypePriorityString);

		if (instances.Count == 0)
		{
			return;
		}

		List<InstanceType?> instanceTypes = new();
		if (instanceTypePriority == null || instanceTypePriority.Count == 0)
		{
			instanceTypes.Add(null);
		}
		else
		{
			instanceTypes.AddRange(instanceTypePriority);
		}

		using IDisposable? logScope = _logger.BeginScope(new Dictionary<string, object>
		{
			["InstanceIds"] = instanceIds,
			["InstanceTypes"] = instanceTypes
		});

		bool success = false;
		foreach (InstanceType? instanceType in instanceTypes)
		{
			KeyValuePair<string, object?> instanceTypeTag = KeyValuePair.Create("instanceType", (object?)instanceType);
			KeyValuePair<string, object?> azTag = KeyValuePair.Create("az", (object?)instances[0].Placement.AvailabilityZone);

			try
			{
				await StartInstancesAsync(instances, instanceType, cancellationToken);
				success = true;
				KeyValuePair<string, object?> statusTag = KeyValuePair.Create("status", (object?)"success");
				_ec2StartInstanceCounter.Add(1, new[] { instanceTypeTag, azTag, statusTag });
				break;
			}
			catch (AmazonEC2Exception e) when (e.ErrorCode == "InsufficientInstanceCapacity")
			{
				KeyValuePair<string, object?> statusTag = KeyValuePair.Create("status", (object?)"errorCapacity");
				_ec2StartInstanceCounter.Add(1, new[] { instanceTypeTag, azTag, statusTag });
				_logger.LogInformation("Insufficient capacity for {InstanceType}", instanceType?.ToString());
			}
			catch (AmazonEC2Exception e) when (e.ErrorCode == "Unsupported")
			{
				KeyValuePair<string, object?> statusTag = KeyValuePair.Create("status", (object?)"errorUnsupported");
				_ec2StartInstanceCounter.Add(1, new[] { instanceTypeTag, azTag, statusTag });
				_logger.LogInformation("Unsupported for {InstanceType}", instanceType?.ToString());
			}
			catch (Exception)
			{
				KeyValuePair<string, object?> statusTag = KeyValuePair.Create("status", (object?)"error");
				_ec2StartInstanceCounter.Add(1, new[] { instanceTypeTag, azTag, statusTag });
				throw;
			}
		}

		span.SetAttribute("success", success);
		if (!success)
		{
			_logger.LogInformation("Unable to start instances. Insufficient capacity for all instance types tried");
		}
	}

	internal async Task StartInstancesAsync(List<Instance> instances, InstanceType? instanceType, CancellationToken cancellationToken)
	{
		List<string> instanceIds = instances.Select(x => x.InstanceId).ToList();
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(StartInstancesAsync)}");
		span.SetAttribute("instances", String.Join(',', instanceIds));
		span.SetAttribute("instanceType", instanceType);

		if (instances.Count == 0)
		{
			return;
		}
		if (instanceType != null)
		{
			await ChangeInstanceTypeAsync(instances, instanceType, cancellationToken);
		}

		StartInstancesRequest startRequest = new();
		startRequest.InstanceIds.AddRange(instanceIds);

		StartInstancesResponse startResponse;
		{
			using TelemetrySpan ec2Span = _tracer.StartActiveSpan("StartInstancesEc2");
			span.SetAttribute("req.instanceIds", String.Join(',', startRequest.InstanceIds));
			startResponse = await _ec2.StartInstancesAsync(startRequest, cancellationToken);
			span.SetAttribute("res.statusCode", (int)startResponse.HttpStatusCode);
			span.SetAttribute("res.numInstances", startResponse.StartingInstances.Count);
		}
		
		if (!IsResponseSuccessful(startResponse))
		{
			throw new AwsBadHttpResponseException(startResponse.HttpStatusCode);
		}

		foreach (InstanceStateChange instanceChange in startResponse.StartingInstances)
		{
			_logger.LogInformation("Instance {InstanceId} started ({PrevState} -> {CurrentState})", instanceChange.InstanceId, instanceChange.PreviousState.Name, instanceChange.CurrentState.Name);
		}

		if (startResponse.StartingInstances.Count != instances.Count)
		{
			var info = new { NumRequestInstances = instances.Count, NumStartedInstances = startResponse.StartingInstances.Count };
			_logger.LogError("Num started instances does not match requested instances {@info}", info);
		}
	}

	/// <summary>
	/// Change instance type of given instances
	/// </summary>
	/// <param name="instances">Instances to change</param>
	/// <param name="newInstanceType">New instance type to use</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	/// <returns>List of instances grouped by availability zone</returns>
	private async Task ChangeInstanceTypeAsync(List<Instance> instances, InstanceType newInstanceType, CancellationToken cancellationToken)
	{
		foreach (Instance instance in instances)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(ChangeInstanceTypeAsync)}");
			span.SetAttribute("fromInstanceType", instance.InstanceType);
			span.SetAttribute("toInstanceType", newInstanceType);

			if (instance.InstanceType == newInstanceType)
			{
				continue;
			}

			ModifyInstanceAttributeRequest request = new() { InstanceId = instance.InstanceId, InstanceType = newInstanceType };
			ModifyInstanceAttributeResponse response = await _ec2.ModifyInstanceAttributeAsync(request, cancellationToken);
			span.SetAttribute("Response.StatusCode", (int)response.HttpStatusCode);

			if (IsResponseSuccessful(response))
			{
				_logger.LogInformation("Instance type for {InstanceId} modified ({PrevInstanceType} -> {CurrentInstanceType})", instance.InstanceId, instance.InstanceType, newInstanceType);
				instance.InstanceType = newInstanceType; // Update local copy with instance type for sake of logging
			}
			else
			{
				_logger.LogError("Failed to modify instance type for {InstanceId}. Wanted {InstanceType}. Status code {StatusCode}", instance.InstanceId, newInstanceType, response.HttpStatusCode);
			}
		}
	}

	/// <summary>
	/// Stop instances that have been stuck in state pending for longer than X time
	/// </summary>
	/// <param name="pool">Which pool instances must belong to</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	private async Task StopStuckPendingInstancesAsync(IPool pool, CancellationToken cancellationToken)
	{
		TimeSpan pendingStateTimeout = TimeSpan.FromMinutes(5);

		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(StopStuckPendingInstancesAsync)}");
		span.SetAttribute("poolId", pool.Id.ToString());

		// Find pending instances in the correct pool
		List<Instance> stuckInstances = await AwsFleetManager.GetInstancesInPoolAsync(_ec2, InstanceStateName.Pending, pool, _tracer, cancellationToken);
		List<string> stuckInstanceIds = stuckInstances
			.Where(x => x.LaunchTime < _clock.UtcNow - pendingStateTimeout)
			.Select(x => x.InstanceId)
			.ToList();

		if (stuckInstanceIds.Count > 0)
		{
			StopInstancesRequest stopRequest = new() { InstanceIds = stuckInstanceIds };
			StopInstancesResponse stopResponse = await _ec2.StopInstancesAsync(stopRequest, cancellationToken);
			span.SetAttribute("stopRes.statusCode", (int)stopResponse.HttpStatusCode);
			span.SetAttribute("stopRes.numStoppingInstances", stopResponse.StoppingInstances.Count);
		}
	}

	/// <summary>
	/// Get list of stopped instances from EC2 API that can be used scaling out a given pool
	/// </summary>
	/// <param name="pool">Which pool instances must belong to</param>
	/// <param name="cancellationToken">Cancellation token for the call</param>
	/// <returns>List of instances grouped by availability zone</returns>
	private async Task<Dictionary<string, List<Instance>>> GetCandidateInstancesAsync(IPool pool, CancellationToken cancellationToken)
	{
		using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsRecyclingFleetManager)}.{nameof(GetCandidateInstancesAsync)}");
		span.SetAttribute("poolId", pool.Id.ToString());

		// Find stopped instances in the correct pool
		List<Instance> instances = await AwsFleetManager.GetInstancesInPoolAsync(_ec2, InstanceStateName.Stopped, pool, _tracer, cancellationToken);

		// Group instance objects by availability zone
		List<IGrouping<string, Instance>> azInstancePairs = instances.GroupBy(x => x.Placement.AvailabilityZone).ToList();

		return azInstancePairs.ToDictionary(x => x.Key, y => y.ToList());
	}

	/// <inheritdoc/>
	public async Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
	{
		await AwsFleetManager.ShrinkPoolViaAgentShutdownRequestAsync(_agentCollection, pool, agents, count, cancellationToken);
		return new ScaleResult(FleetManagerOutcome.Success, 0, count, "Pool scaled in");
	}

	/// <inheritdoc/>
	public async Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken)
	{
		// Find all instances in the pool
		return (await AwsFleetManager.GetInstancesInPoolAsync(_ec2, InstanceStateName.Stopped, pool, _tracer, cancellationToken)).Count;
	}

	private static bool IsResponseSuccessful(AmazonWebServiceResponse response)
	{
		return (int)response.HttpStatusCode >= 200 && (int)response.HttpStatusCode <= 299;
	}
}
