// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.EC2;
using Amazon.EC2.Model;
using HordeServer.Agents.Pools;
using Microsoft.Extensions.Logging;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Fleet.Providers
{
	/// <summary>
	/// Settings for AWS fleet manager that reuses instances
	/// </summary>
	public class AwsReuseFleetManagerSettings
	{
		/// <summary>
		/// Prioritized list of instance types to use waking up an instance
		/// If set to null, the instance will be awakened with its last instance type.
		/// </summary>
		public List<string>? InstanceTypes { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="instanceTypes"></param>
		public AwsReuseFleetManagerSettings(List<string>? instanceTypes = null)
		{
			InstanceTypes = instanceTypes;
		}
	}

	/// <summary>
	/// Fleet manager for handling AWS EC2 instances
	/// Will start already existing but stopped instances to reuse existing EBS disks.
	/// See <see cref="AwsFleetManager" /> for creating and terminating instances from scratch.
	/// </summary>
	public sealed class AwsReuseFleetManager : IFleetManager
	{
		/// <summary>
		/// Settings for fleet manager
		/// </summary>
		public AwsReuseFleetManagerSettings Settings { get; }

		private readonly IAmazonEC2 _ec2;
		private readonly IAgentCollection _agentCollection;
		private readonly Tracer _tracer;
		private readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		public AwsReuseFleetManager(IAmazonEC2 ec2, IAgentCollection agentCollection, AwsReuseFleetManagerSettings settings, Tracer tracer, ILogger<AwsReuseFleetManager> logger)
		{
			_ec2 = ec2;
			_agentCollection = agentCollection;
			_tracer = tracer;
			_logger = logger;
			Settings = settings;
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int requestedInstancesCount, CancellationToken cancellationToken)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AwsReuseFleetManager)}.{nameof(ExpandPoolAsync)}");
			span.SetAttribute("poolName", pool.Name);
			span.SetAttribute("numAgents", agents.Count);
			span.SetAttribute("count", requestedInstancesCount);

			List<Instance> stoppedInstances = await AwsFleetManager.GetInstancesInPoolAsync(_ec2, InstanceStateName.Stopped, pool, _tracer, cancellationToken);

			if (Settings.InstanceTypes is { Count: >= 1 })
			{
				InstanceType newInstanceType = InstanceType.FindValue(Settings.InstanceTypes[0]);
				foreach (Instance stoppedInstance in stoppedInstances)
				{
					if (stoppedInstance.InstanceType == newInstanceType)
					{
						continue;
					}
					using TelemetrySpan modifySpan = _tracer.StartActiveSpan("ModifyInstanceAttribute");

					ModifyInstanceAttributeRequest request = new() { InstanceId = stoppedInstance.InstanceId, InstanceType = newInstanceType };
					ModifyInstanceAttributeResponse response = await _ec2.ModifyInstanceAttributeAsync(request, cancellationToken);
					if ((int)response.HttpStatusCode >= 200 && (int)response.HttpStatusCode <= 299)
					{
						_logger.LogInformation("Instance type for {InstanceId} in pool {PoolId} modified ({PrevInstanceType} -> {CurrentInstanceType})", stoppedInstance.InstanceId, pool.Id, stoppedInstance.InstanceType, newInstanceType);
					}
					else
					{
						_logger.LogError("Failed to modify instance type for {InstanceId} in pool {PoolId}. Wanted {InstanceType}. Status code {StatusCode}", stoppedInstance.InstanceId, pool.Id, newInstanceType, response.HttpStatusCode);
					}
				}
			}

			using (TelemetrySpan startSpan = _tracer.StartActiveSpan("StartInstances"))
			{
				// Try to start the given instances
				StartInstancesRequest startRequest = new();
				startRequest.InstanceIds.AddRange(stoppedInstances.Select(x => x.InstanceId).Take(requestedInstancesCount));
				int stoppedInstancesCount = startRequest.InstanceIds.Count;

				startSpan.SetAttribute("req.instanceIds", String.Join(",", startRequest.InstanceIds));
				int startedInstancesCount = 0;
				if (startRequest.InstanceIds.Count > 0)
				{
					StartInstancesResponse startResponse = await _ec2.StartInstancesAsync(startRequest, cancellationToken);
					startSpan.SetAttribute("res.statusCode", (int)startResponse.HttpStatusCode);
					startSpan.SetAttribute("res.numInstances", startResponse.StartingInstances.Count);
					if ((int)startResponse.HttpStatusCode >= 200 && (int)startResponse.HttpStatusCode <= 299)
					{
						foreach (InstanceStateChange instanceChange in startResponse.StartingInstances)
						{
							startedInstancesCount++;
							_logger.LogInformation("Starting instance {InstanceId} for pool {PoolId} (prev state {PrevState}, current state {CurrentState}", instanceChange.InstanceId, pool.Id, instanceChange.PreviousState.Name, instanceChange.CurrentState.Name);
						}
					}
				}

				if (startRequest.InstanceIds.Count < requestedInstancesCount)
				{
					string reason = "";
					if (stoppedInstancesCount < requestedInstancesCount)
					{
						reason += " Not enough stopped instances to accommodate the pool expansion.";
					}
					if (startedInstancesCount < stoppedInstancesCount)
					{
						reason += " Not all instances were able to start.";
					}

					_logger.LogInformation("Unable to expand pool {PoolName}.\n" +
										   "Reason={Reason}\n" +
										   "RequestedInstancesCount={RequestedCount}\n" +
										   "StoppedInstancesCount={StoppedInstancesCount}\n" +
										   "StartedInstancesCount={StartedInstancesCount}",
						pool.Name, reason.Trim(), requestedInstancesCount, stoppedInstancesCount, startedInstancesCount);
				}

				if (startedInstancesCount == requestedInstancesCount)
				{
					return new ScaleResult(FleetManagerOutcome.Success, startedInstancesCount, 0);
				}

				if (startedInstancesCount > 0 && startedInstancesCount < requestedInstancesCount)
				{
					return new ScaleResult(FleetManagerOutcome.PartialSuccess, startedInstancesCount, 0);
				}

				return new ScaleResult(FleetManagerOutcome.Failure, startedInstancesCount, 0);
			}
		}

		/// <inheritdoc/>
		public async Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			await AwsFleetManager.ShrinkPoolViaAgentShutdownRequestAsync(_agentCollection, pool, agents, count, cancellationToken);
			return new ScaleResult(FleetManagerOutcome.Success, 0, count);
		}

		/// <inheritdoc/>
		public async Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken)
		{
			return (await AwsFleetManager.GetInstancesInPoolAsync(_ec2, InstanceStateName.Stopped, pool, _tracer, cancellationToken)).Count;
		}
	}
}
