// Copyright Epic Games, Inc. All Rights Reserved.

using HordeServer.Agents.Pools;
using Microsoft.Extensions.Logging;

namespace HordeServer.Agents.Fleet.Providers
{
	/// <summary>
	/// No-op implementation of <see cref="IFleetManager"/>
	/// </summary>
	public class NoOpFleetManager : IFleetManager
	{
		readonly ILogger _logger;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="logger">Logging device</param>
		public NoOpFleetManager(ILogger<NoOpFleetManager> logger)
		{
			_logger = logger;
		}

		/// <inheritdoc/>
		public Task<ScaleResult> ExpandPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Expand pool {PoolId} by {Count} agents", pool.Id, count);
			return Task.FromResult(new ScaleResult(FleetManagerOutcome.Success, count, 0));
		}

		/// <inheritdoc/>
		public Task<ScaleResult> ShrinkPoolAsync(IPool pool, IReadOnlyList<IAgent> agents, int count, CancellationToken cancellationToken)
		{
			_logger.LogInformation("Shrink pool {PoolId} by {Count} agents", pool.Id, count);
			return Task.FromResult(new ScaleResult(FleetManagerOutcome.Success, 0, count));
		}

		/// <inheritdoc/>
		public Task<int> GetNumStoppedInstancesAsync(IPoolConfig pool, CancellationToken cancellationToken) => Task.FromResult(0);
	}
}
