// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics.CodeAnalysis;
using System.Security.Cryptography;
using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Common;
using EpicGames.Horde.Compute;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Tools;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;
using HordeCommon.Rpc;
using HordeCommon.Rpc.Messages;
using HordeServer.Agents.Sessions;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;

#pragma warning disable CA2227 // Change x to be read-only by removing the property setter

namespace HordeServer.Agents
{
	/// <summary>
	/// Defines the operation mode of the agent
	/// </summary>
	public enum AgentMode
	{
		/// <summary>
		/// For backwards compatibility
		/// </summary>
		Unspecified = 0,
		
		/// <summary>
		/// Dedicated mode
		/// - Trusted agent running in a controlled environment (e.g., build farm)
		/// - Does not run any other non-Horde workloads
		/// - Capable of executing all lease types
		/// </summary>
		Dedicated = 1,
		
		/// <summary>
		/// Workstation mode
		/// - Agent authenticates as the current user through interactive authentication via EpicGames.OIDC
		/// - Low trust level, typically used for spare on-premise hardware like workstations
		/// - Lease execution is inherently unreliable as these agents yield to non-Horde workloads
		/// - Limited to compute leases only, primarily to support remote execution
		/// </summary>
		Workstation = 2
	}
	
	/// <summary>
	/// Mirrors an Agent document in the database
	/// </summary>
	public interface IAgent
	{
		/// <summary>
		/// Identifier for this agent.
		/// </summary>
		public AgentId Id { get; }

		/// <summary>
		/// The current session id, if it's online
		/// </summary>
		public SessionId? SessionId { get; }

		/// <summary>
		/// Time at which the current session expires. 
		/// </summary>
		public DateTime? SessionExpiresAt { get; }

		/// <summary>
		/// Current status of this agent
		/// </summary>
		public AgentStatus Status { get; }

		/// <summary>
		/// Last time that the agent was online. Null if the agent is currently online.
		/// </summary>
		public DateTime? LastOnlineTime { get; }

		/// <summary>
		/// Whether the agent is enabled
		/// </summary>
		public bool Enabled { get; }

		/// <summary>
		/// Whether the agent is ephemeral
		/// </summary>
		public bool Ephemeral { get; }
		
		/// <summary>
		/// Operation mode of agent
		/// </summary>
		public AgentMode Mode { get; }

		/// <summary>
		/// Whether the agent should be included on the dashboard. This is set to true for ephemeral agents once they are no longer online, or agents that are explicitly deleted.
		/// </summary>
		public bool Deleted { get; }

		/// <summary>
		/// Version of the software running on this agent
		/// </summary>
		public string? Version { get; }

		/// <summary>
		/// Arbitrary comment for the agent (useful for disable reasons etc)
		/// </summary>
		public string? Comment { get; }
		
		/// <summary>
		/// List of server-defined properties (will overwrite and merge with agent-reported properties)
		/// </summary>
		public IReadOnlyList<string> ServerDefinedProperties { get; }
		
		/// <summary>
		/// List of properties for this agent
		/// </summary>
		public IReadOnlyList<string> Properties { get; }

		/// <summary>
		/// List of resources available to the agent
		/// </summary>
		public IReadOnlyDictionary<string, int> Resources { get; }

		/// <summary>
		/// Last upgrade that was attempted
		/// </summary>
		public string? LastUpgradeVersion { get; }

		/// <summary>
		/// Time that which the last upgrade was attempted
		/// </summary>
		public DateTime? LastUpgradeTime { get; }

		/// <summary>
		/// Number of times an upgrade job has failed
		/// </summary>
		public int? UpgradeAttemptCount { get; }

		/// <summary>
		/// All pools for this agent
		/// </summary>
		public IReadOnlyList<PoolId> Pools { get; }

		/// <summary>
		/// Dynamically applied pools
		/// </summary>
		public IReadOnlyList<PoolId> DynamicPools { get; }

		/// <summary>
		/// List of manually assigned pools for agent
		/// </summary>
		public IReadOnlyList<PoolId> ExplicitPools { get; }

		/// <summary>
		/// Whether a conform is requested
		/// </summary>
		public bool RequestConform { get; }

		/// <summary>
		/// Whether a full conform is requested
		/// </summary>
		public bool RequestFullConform { get; }

		/// <summary>
		/// Whether a machine restart is requested
		/// </summary>
		public bool RequestRestart { get; }

		/// <summary>
		/// Whether the machine should be shutdown
		/// </summary>
		public bool RequestShutdown { get; }

		/// <summary>
		/// Whether a forced machine restart is requested
		/// </summary>
		public bool RequestForceRestart { get; }

		/// <summary>
		/// The reason for the last agent shutdown
		/// </summary>
		public string? LastShutdownReason { get; }

		/// <summary>
		/// List of workspaces currently synced to this machine
		/// </summary>
		public IReadOnlyList<AgentWorkspaceInfo> Workspaces { get; }

		/// <summary>
		/// Time at which the last conform job ran
		/// </summary>
		public DateTime LastConformTime { get; }

		/// <summary>
		/// Number of times a conform job has failed
		/// </summary>
		public int? ConformAttemptCount { get; }

		/// <summary>
		/// Array of active leases.
		/// </summary>
		public IReadOnlyList<IAgentLease> Leases { get; }

		/// <summary>
		/// Key used to validate that a particular enrollment is still valid for this agent
		/// </summary>
		public string EnrollmentKey { get; }

		/// <summary>
		/// Last time that the agent was modified
		/// </summary>
		public DateTime UpdateTime { get; }

		/// <summary>
		/// Update counter for this document. Any updates should compare-and-swap based on the value of this counter, or increment it in the case of server-side updates.
		/// </summary>
		public uint UpdateIndex { get; }

		/// <summary>
		/// Gets a sessions for the agent
		/// </summary>
		/// <param name="sessionId">Identifier for the session</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of sessions matching the given criteria</returns>
		Task<ISession?> GetSessionAsync(SessionId sessionId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Find sessions for the given agent
		/// </summary>
		/// <param name="startTime">Start time to include in the search</param>
		/// <param name="finishTime">Finish time to include in the search</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of sessions matching the given criteria</returns>
		Task<IReadOnlyList<ISession>> FindSessionsAsync(DateTime? startTime = null, DateTime? finishTime = null, int index = 0, int count = 10, CancellationToken cancellationToken = default);

		/// <summary>
		/// Resets an agent to use new settings
		/// </summary>
		/// <param name="ephemeral">Whether the agent is ephemeral or not</param>
		/// <param name="enrollmentKey">Key used to identify a unique enrollment for the agent with this id</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IAgent?> TryResetAsync(bool ephemeral, string enrollmentKey, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes an agent
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task<IAgent?> TryDeleteAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Update an agent's settings
		/// </summary>
		/// <param name="options">Options for updating the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state if update was successful</returns>
		Task<IAgent?> TryUpdateAsync(UpdateAgentOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update the current workspaces for an agent.
		/// </summary>
		/// <param name="workspaces">Current list of workspaces</param>
		/// <param name="requestConform">Whether the agent still needs to run another conform</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state</returns>
		Task<IAgent?> TryUpdateWorkspacesAsync(List<AgentWorkspaceInfo> workspaces, bool requestConform, CancellationToken cancellationToken = default);

		/// <summary>
		/// Sets the current session
		/// </summary>
		/// <param name="options">Options for the new session</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state</returns>
		Task<IAgent?> TryCreateSessionAsync(CreateSessionOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to update the current session
		/// </summary>
		/// <param name="options">Options for updating the session</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the document was updated, false if another writer updated the document first</returns>
		Task<IAgent?> TryUpdateSessionAsync(UpdateSessionOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Terminates the current session
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state if it succeeded, otherwise null</returns>
		Task<IAgent?> TryTerminateSessionAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to add a lease to an agent
		/// </summary>
		/// <param name="options">The new lease document</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state if it succeeded, otherwise null</returns>
		Task<IAgent?> TryCreateLeaseAsync(CreateLeaseOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to cancel a lease
		/// </summary>
		/// <param name="leaseIdx">Index of the lease</param>
		/// <param name="reason">Reason for cancelling the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New agent state if it succeeded, otherwise null</returns>
		Task<IAgent?> TryCancelLeaseAsync(int leaseIdx, LeaseOutcomeReason? reason = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Wait for this agent to be updated
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IAgent?> WaitForUpdateAsync(CancellationToken cancellationToken = default);
	}

	/// <summary>
	/// Information for a new lease
	/// </summary>
	public record CreateLeaseOptions(LeaseId Id, LeaseId? ParentId, string Name, StreamId? StreamId, PoolId? PoolId, LogId? LogId, IReadOnlyDictionary<string, int>? Resources, bool Exclusive, IMessage Payload);
	
	/// <summary>
	/// Options for creating an agent
	/// </summary>
	/// <param name="Id">ID of the agent</param>
	/// <param name="Mode">Operation mode</param>
	/// <param name="Ephemeral">Whether the agent is ephemeral or not</param>
	/// <param name="EnrollmentKey">Key used to identify a unique enrollment for the agent with this ID</param>
	/// <param name="ServerDefinedProperties">Server-defined properties</param>
	public record CreateAgentOptions(AgentId Id, AgentMode Mode, bool Ephemeral, string EnrollmentKey, IReadOnlyList<string>? ServerDefinedProperties = null);
	
	/// <summary>
	/// Options for updating an agent
	/// </summary>
	/// <param name="Enabled">Whether the agent is enabled or not</param>
	/// <param name="RequestConform">Whether to request a conform job be run</param>
	/// <param name="RequestFullConform">Whether to request a full conform job be run</param>
	/// <param name="RequestRestart">Whether to request the machine be restarted</param>
	/// <param name="RequestShutdown">Whether to request the machine be shut down</param>
	/// <param name="RequestForceRestart">Request an immediate restart without waiting for leases to complete</param>
	/// <param name="ShutdownReason">The reason for shutting down agent, ex. Autoscaler/Manual/Unexpected</param>
	/// <param name="ExplicitPools">List of pools for the agent</param>
	/// <param name="Comment">New comment</param>
	public record UpdateAgentOptions(bool? Enabled = null, bool? RequestConform = null, bool? RequestFullConform = null, bool? RequestRestart = null, bool? RequestShutdown = null, bool? RequestForceRestart = null, string? ShutdownReason = null, List<PoolId>? ExplicitPools = null, string? Comment = null);

	/// <summary>
	/// Options for starting a new agent session
	/// </summary>
	/// <param name="Capabilities">Capabilities for the agent</param>
	/// <param name="DynamicPools">New list of dynamic pools for the agent</param>
	/// <param name="Version">Current version of the agent software</param>
	public record CreateSessionOptions(RpcAgentCapabilities Capabilities, IReadOnlyList<PoolId> DynamicPools, string? Version);

	/// <summary>
	/// Options for updating a new agent session
	/// </summary>
	/// <param name="Status">New status of the agent</param>
	/// <param name="Capabilities">Capbilities for the session</param>
	/// <param name="DynamicPools">New list of dynamic pools for the agent</param>
	/// <param name="Leases">New set of leases</param>
	/// <param name="DefaultLeaseOutcomeReason">Default reason used when terminating leases without agent-provided reasons</param>
	/// <param name="CapabilityOverrides">Capability overrides to merge into existing agent capabilities</param>
	public record UpdateSessionOptions(AgentStatus? Status = null, RpcAgentCapabilities? Capabilities = null, IReadOnlyList<PoolId>? DynamicPools = null, IEnumerable<RpcLease>? Leases = null, LeaseOutcomeReason? DefaultLeaseOutcomeReason = null, RpcAgentCapabilities? CapabilityOverrides = null);

	/// <summary>
	/// Extension methods for IAgent
	/// </summary>
	public static class AgentExtensions
	{
		/// <summary>
		/// Default tool ID for agent software (multi-platform, shipped without a .NET runtime)
		/// This is being deprecated in favor of the platform-specific and self-contained versions of the agent below
		/// </summary>
		public static ToolId AgentToolId { get; } = new("horde-agent");

		/// <summary>
		/// Tool ID for Windows-specific and self-contained agent software
		/// </summary>
		public static ToolId AgentWinX64ToolId { get; } = new("horde-agent-win-x64");

		/// <summary>
		/// Tool ID for Linux-specific and self-contained agent software
		/// </summary>
		public static ToolId AgentLinuxX64ToolId { get; } = new("horde-agent-linux-x64");

		/// <summary>
		/// Tool ID for Mac-specific and self-contained agent software
		/// </summary>
		public static ToolId AgentMacX64ToolId { get; } = new("horde-agent-osx-x64");
		
		/// <summary>
		/// Set of property names that can only be set by the server
		/// </summary>
		public static readonly HashSet<string> ServerDefinedPropertyNames = [KnownPropertyNames.Trusted];

		/// <summary>
		/// Gets the tool ID for the software the given agent should be running
		/// </summary>
		/// <param name="agent">Agent to check</param>
		/// <param name="computeConfig">Current compute config</param>
		/// <returns>Identifier for the tool that the agent should be using</returns>
		public static ToolId GetSoftwareToolId(this IAgent agent, ComputeConfig computeConfig)
		{
			ToolId result = AgentToolId;
			
			string? reportedToolId = agent.GetToolId();
			if (reportedToolId != null)
			{
				result = new ToolId(reportedToolId);
			}
			else if (agent.IsSelfContained())
			{
				// Let reported tool ID take precedence, this self-contained path and reporting should be deprecated in favor of that 
				
				// Skip support for condition-based software configs below by returning early when self-contained
				// Getting this wrong can lead to a self-contained agent getting non-self-contained updates and vice versa.
				return agent.GetOsFamily() switch
				{
					RuntimePlatform.Type.Windows => AgentWinX64ToolId,
					RuntimePlatform.Type.Linux => AgentLinuxX64ToolId,
					RuntimePlatform.Type.Mac => AgentMacX64ToolId,
					_ => throw new ArgumentOutOfRangeException("Unknown platform " + agent.GetOsFamily())
				};
			}

			foreach (AgentSoftwareConfig softwareConfig in computeConfig.Software)
			{
				if (softwareConfig.Condition != null && agent.SatisfiesCondition(softwareConfig.Condition))
				{
					result = softwareConfig.ToolId;
					break;
				}
			}
			return result;
		}

		/// <summary>
		/// Determines whether this agent is online
		/// </summary>
		/// <returns></returns>
		public static bool IsSessionValid(this IAgent agent, DateTime utcNow)
		{
			return agent.SessionId.HasValue && agent.SessionExpiresAt.HasValue && utcNow < agent.SessionExpiresAt.Value;
		}

		/// <summary>
		/// Tests whether an agent is in the given pool
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="poolId"></param>
		/// <returns></returns>
		public static bool IsInPool(this IAgent agent, PoolId poolId)
		{
			return agent.Pools.Contains(poolId);
		}

		/// <summary>
		/// Tests whether an agent has reported as being a self-contained .NET package
		/// </summary>
		/// <param name="agent">Agent to query</param>
		/// <returns>True if self-contained</returns>
		public static bool IsSelfContained(this IAgent agent)
		{
			List<string> values = agent.GetPropertyValues(KnownPropertyNames.SelfContained).ToList();
			return values.Count > 0 && values[0].Equals("true", StringComparison.OrdinalIgnoreCase);
		}
		
		/// <summary>
		/// Checks whether an agent wants to auto-update itself
		/// </summary>
		/// <param name="agent">Agent to query</param>
		/// <returns>Default true, unless explicitly set to false</returns>
		public static bool IsAutoUpdateEnabled(this IAgent agent)
		{
			string? value = agent.GetPropertyValues(KnownPropertyNames.AutoUpdate).FirstOrDefault();
			return String.IsNullOrEmpty(value) || !String.Equals(value, "false", StringComparison.OrdinalIgnoreCase);
		}
		
		/// <summary>
		/// Get optional tool ID as reported by agent
		/// <see cref="KnownPropertyNames.ToolId" />
		/// </summary>
		public static string? GetToolId(this IAgent agent)
		{
			List<string> values = agent.GetPropertyValues(KnownPropertyNames.ToolId).ToList();
			return values.Count > 0 ? values[0] : null;
		}

		/// <summary>
		/// Get free disk space on agent (as reported through primary device capabilities)
		/// </summary>
		/// <param name="agent">Agent to query</param>
		/// <returns>Amount of free disk space in bytes</returns>
		public static long? GetDiskFreeSpace(this IAgent agent)
		{
			List<string> values = agent.GetPropertyValues(KnownPropertyNames.WorkingDiskFreeSpace).ToList();
			return values.Count > 0 && Int64.TryParse(values[0], out long amount) ? amount : null;
		}

		/// <summary>
		/// Get operating system family of agent
		/// </summary>
		/// <param name="agent">Agent to query</param>
		/// <returns>Type of OS</returns>
		public static RuntimePlatform.Type? GetOsFamily(this IAgent agent)
		{
			List<string> values = agent.GetPropertyValues(KnownPropertyNames.OsFamily).ToList();
			if (values.Count == 0)
			{
				return null;
			}

			return values[0].ToUpperInvariant() switch
			{
				"WINDOWS" => RuntimePlatform.Type.Windows,
				"LINUX" => RuntimePlatform.Type.Linux,
				"MACOS" => RuntimePlatform.Type.Mac,
				_ => null
			};
		}

		/// <summary>
		/// Tests whether an agent has a particular property
		/// </summary>
		/// <param name="agent"></param>
		/// <param name="property"></param>
		/// <returns></returns>
		public static bool HasProperty(this IAgent agent, string property)
		{
			return agent.Properties.BinarySearch(property, StringComparer.OrdinalIgnoreCase) >= 0;
		}
		
		/// <summary>
		/// Check if a property name (incl value) is controlled by the server
		/// Such properties are protected and cannot be set by the agent.
		/// </summary>
		/// <param name="propName">Name of the property</param>
		/// <returns>True if controlled and enforced by the server</returns>
		public static bool IsPropertyServerDefined(string propName)
		{
			return ServerDefinedPropertyNames.Contains(propName);
		}
		
		/// <summary>
		/// Filters a list of properties from server-defined properties
		/// </summary>
		/// <param name="properties">List to filter</param>
		/// <returns>A new list with any server-defined property removed</returns>
		public static IReadOnlyList<string> RemoveServerDefinedProperties(IReadOnlyList<string> properties)
		{
			return properties.Where(property => !IsPropertyServerDefined(property.Split('=', 2)[0])).ToList();
		}
		
		/// <summary>
		/// Finds property values from a sorted list of Name=Value pairs
		/// </summary>
		/// <param name="agent">The agent to query</param>
		/// <param name="name">Name of the property to find</param>
		/// <returns>Property values</returns>
		public static IEnumerable<string> GetPropertyValues(this IAgent agent, string name)
		{
			if (name.Equals(KnownPropertyNames.Id, StringComparison.OrdinalIgnoreCase))
			{
				yield return agent.Id.ToString();
			}
			else if (name.Equals(KnownPropertyNames.Pool, StringComparison.OrdinalIgnoreCase))
			{
				foreach (PoolId poolId in agent.Pools)
				{
					yield return poolId.ToString();
				}
			}
			else
			{
				int index = agent.Properties.BinarySearch(name, StringComparer.OrdinalIgnoreCase);
				if (index < 0)
				{
					index = ~index;
					for (; index < agent.Properties.Count; index++)
					{
						string property = agent.Properties[index];
						if (property.Length <= name.Length || !property.StartsWith(name, StringComparison.OrdinalIgnoreCase) || property[name.Length] != '=')
						{
							break;
						}
						yield return property.Substring(name.Length + 1);
					}
				}
			}
		}

		/// <summary>
		/// Evaluates a condition against an agent
		/// </summary>
		/// <param name="agent">The agent to evaluate</param>
		/// <param name="condition">The condition to evaluate</param>
		/// <returns>True if the agent satisfies the condition</returns>
		public static bool SatisfiesCondition(this IAgent agent, Condition condition)
		{
			return condition.Evaluate(x => agent.GetPropertyValues(x));
		}

		/// <summary>
		/// Determine whether it's possible to add a lease for the given resources
		/// </summary>
		/// <param name="agent">The agent to create a lease for</param>
		/// <param name="requirements">Requirements for the lease</param>
		/// <param name="assignedResources">Receives the allocated resources</param>
		/// <param name="conditions">Condition to check in addition to those in requirements</param>
		/// <returns>True if the new lease can be granted</returns>
		public static bool MeetsRequirements(this IAgent agent, Requirements requirements, Dictionary<string, int> assignedResources, List<Condition> conditions)
		{
			PoolId? poolId = null;
			if (!String.IsNullOrEmpty(requirements.Pool))
			{
				poolId = new PoolId(requirements.Pool);
			}

			List<Condition> combinedConditions = [];
			if (requirements.Condition != null)
			{
				combinedConditions.Add(requirements.Condition);
			}
			combinedConditions.AddRange(conditions);
			return MeetsRequirements(agent, poolId, combinedConditions, requirements.Properties, requirements.Resources, requirements.Exclusive, assignedResources);
		}

		/// <summary>
		/// Determine whether it's possible to add a lease for the given resources
		/// </summary>
		/// <param name="agent">The agent to create a lease for</param>
		/// <param name="poolId">Pool to take the machine from</param>
		/// <param name="conditions">Conditions to satisfy</param>
		/// <param name="properties">Required properties to match</param>
		/// <param name="resources">Resources required to execute</param>
		/// <param name="exclusive">Whether the lease needs to be executed exclusively on the machine</param>
		/// <param name="assignedResources">Resources allocated to the task</param>
		/// <returns>True if the new lease can be granted</returns>
		public static bool MeetsRequirements(this IAgent agent, PoolId? poolId, List<Condition> conditions, IEnumerable<string>? properties, Dictionary<string, ResourceRequirements>? resources, bool exclusive, Dictionary<string, int> assignedResources)
		{
			if (!agent.Enabled || agent.Status != AgentStatus.Ok)
			{
				return false;
			}
			if (agent.Leases.Any(x => x.Exclusive))
			{
				return false;
			}
			if (exclusive && agent.Leases.Any())
			{
				return false;
			}
			if (poolId.HasValue && !agent.IsInPool(poolId.Value))
			{
				return false;
			}
			if (conditions.Any(condition => !agent.SatisfiesCondition(condition)))
			{
				return false;
			}
			if (properties != null && properties.Any(property => !agent.HasProperty(property)))
			{
				return false;
			}
			if (resources != null)
			{
				foreach ((string name, ResourceRequirements resourceRequirements) in resources)
				{
					int remainingCount;
					if (!agent.Resources.TryGetValue(name, out remainingCount))
					{
						return false;
					}
					foreach (AgentLease lease in agent.Leases)
					{
						if (lease.Resources != null)
						{
							int leaseCount;
							lease.Resources.TryGetValue(name, out leaseCount);
							remainingCount -= leaseCount;
						}
					}
					if (remainingCount < resourceRequirements.Min)
					{
						return false;
					}

					int allocatedCount;
					if (resourceRequirements.Max != null)
					{
						allocatedCount = Math.Min(resourceRequirements.Max.Value, remainingCount);
					}
					else
					{
						allocatedCount = resourceRequirements.Min;
					}
					assignedResources.Add(name, allocatedCount);
				}
			}
			return true;
		}
	}

	/// <summary>
	/// Information about a workspace synced to an agent
	/// </summary>
	public class AgentWorkspaceInfo
	{
		/// <summary>
		/// Name of the Perforce cluster to use
		/// </summary>
		public string? Cluster { get; set; }

		/// <summary>
		/// User to log into Perforce with (eg. buildmachine)
		/// </summary>
		public string? UserName { get; set; }

		/// <summary>
		/// Identifier to distinguish this workspace from other workspaces
		/// </summary>
		public string Identifier { get; set; }

		/// <summary>
		/// The stream to sync
		/// </summary>
		public string Stream { get; set; }

		/// <summary>
		/// Custom view for the workspace
		/// </summary>
		public List<string>? View { get; set; }

		/// <summary>
		/// Whether to use an incremental workspace
		/// </summary>
		public bool Incremental { get; set; }

		/// <summary>
		/// Method to use when syncing/materializing data from Perforce
		/// </summary>
		public string? Method { get; set; }

		/// <summary>
		/// Minimum disk space that must be available *after* syncing this workspace (in megabytes)
		/// </summary>
		public long? MinScratchSpace { get; set; }

		/// <summary>
		/// Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk.
		/// </summary>
		public long? ConformDiskFreeSpace { get; set; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="cluster">Name of the Perforce cluster</param>
		/// <param name="userName">User to log into Perforce with (eg. buildmachine)</param>
		/// <param name="identifier">Identifier to distinguish this workspace from other workspaces</param>
		/// <param name="stream">The stream to sync</param>
		/// <param name="view">Custom view for the workspace</param>
		/// <param name="incremental">Whether to use an incremental workspace</param>
		/// <param name="method">Method to use when syncing/materializing data from Perforce</param>
		/// <param name="minScratchSpace">Minimum disk space that must be available *after* syncing this workspace (in megabytes)</param>
		/// <param name="conformDiskFreeSpace">Threshold for when to trigger an automatic conform of agent. Measured in megabytes free on disk</param>
		public AgentWorkspaceInfo(string? cluster, string? userName, string identifier, string stream, List<string>? view, bool incremental, string? method, long? minScratchSpace = null, long? conformDiskFreeSpace = null)
		{
			if (!String.IsNullOrEmpty(cluster))
			{
				Cluster = cluster;
			}
			if (!String.IsNullOrEmpty(userName))
			{
				UserName = userName;
			}
			Identifier = identifier;
			Stream = stream;
			View = view;
			Incremental = incremental;
			Method = method;
			MinScratchSpace = minScratchSpace;
			ConformDiskFreeSpace = conformDiskFreeSpace;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="workspace">RPC message to construct from</param>
		public AgentWorkspaceInfo(RpcAgentWorkspace workspace)
			: this(workspace.ConfiguredCluster, workspace.ConfiguredUserName, workspace.Identifier, workspace.Stream, (workspace.View.Count > 0) ? workspace.View.ToList() : null, workspace.Incremental, workspace.Method, workspace.MinScratchSpace, workspace.ConformDiskFreeSpace)
		{
			Method = String.IsNullOrEmpty(Method) ? null : Method;

			// Treat zero value as null as Protobuf cannot store null 
			MinScratchSpace = workspace.MinScratchSpace == 0 ? null : workspace.MinScratchSpace;
			ConformDiskFreeSpace = workspace.ConformDiskFreeSpace == 0 ? null : workspace.ConformDiskFreeSpace;
		}

		/// <summary>
		/// Gets a digest of the settings for this workspace
		/// </summary>
		/// <returns>Digest for the workspace settings</returns>
		public string GetDigest()
		{
#pragma warning disable CA5351 // Do Not Use Broken Cryptographic Algorithms
			using (MD5 hasher = MD5.Create())
			{
				byte[] data = BsonExtensionMethods.ToBson(this);
				return BitConverter.ToString(hasher.ComputeHash(data)).Replace("-", "", StringComparison.Ordinal);
			}
#pragma warning restore CA5351
		}

		/// <inheritdoc/>
		public override bool Equals(object? obj)
		{
			AgentWorkspaceInfo? other = obj as AgentWorkspaceInfo;
			if (other == null)
			{
				return false;
			}
			if (Cluster != other.Cluster || 
				UserName != other.UserName ||
				Identifier != other.Identifier ||
				Stream != other.Stream || 
				Incremental != other.Incremental ||
				Method != other.Method ||
				MinScratchSpace != other.MinScratchSpace ||
				ConformDiskFreeSpace != other.ConformDiskFreeSpace)
			{
				return false;
			}
			if (!Enumerable.SequenceEqual(View ?? new List<string>(), other.View ?? new List<string>()))
			{
				return false;
			}
			return true;
		}

		/// <inheritdoc/>
		public override int GetHashCode()
		{
			return HashCode.Combine(Cluster, UserName, Identifier, Stream, Incremental); // Ignore 'View' for now
		}

		/// <summary>
		/// Checks if two workspace sets are equivalent, ignoring order
		/// </summary>
		/// <param name="workspacesA">First list of workspaces</param>
		/// <param name="workspacesB">Second list of workspaces</param>
		/// <returns>True if the sets are equivalent</returns>
		public static bool SetEquals(IReadOnlyList<AgentWorkspaceInfo> workspacesA, IReadOnlyList<AgentWorkspaceInfo> workspacesB)
		{
			HashSet<AgentWorkspaceInfo> workspacesSetA = new HashSet<AgentWorkspaceInfo>(workspacesA);
			return workspacesSetA.SetEquals(workspacesB);
		}
	}

	/// <summary>
	/// Configuration for an AutoSDK workspace
	/// </summary>
	public class AutoSdkConfig
	{
		/// <summary>
		/// Disables use of AutoSDK.
		/// </summary>
		public static AutoSdkConfig None { get; } = new AutoSdkConfig(Array.Empty<string>());

		/// <summary>
		/// Syncs the entire AutoSDK folder.
		/// </summary>
		public static AutoSdkConfig Full { get; } = new AutoSdkConfig(null);

		/// <summary>
		/// Additive filter for paths to include in the workspace
		/// </summary>
		public List<string> View { get; set; } = new List<string>();

		/// <summary>
		/// Constructor
		/// </summary>
		public AutoSdkConfig()
		{
			View = new List<string>();
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="filter">Filter for the workspace</param>
		public AutoSdkConfig(IEnumerable<string>? filter)
		{
			if (filter == null)
			{
				View = new List<string> { "..." };
			}
			else
			{
				View = filter.OrderBy(x => x).Distinct().ToList();
			}
		}

		/// <summary>
		/// Merge two workspaces together
		/// </summary>
		[return: NotNullIfNotNull("lhs")]
		[return: NotNullIfNotNull("rhs")]
		public static AutoSdkConfig? Merge(AutoSdkConfig? lhs, AutoSdkConfig? rhs)
		{
			if (lhs?.View == null || lhs.View.Count == 0)
			{
				return rhs;
			}
			if (rhs?.View == null || rhs.View.Count == 0)
			{
				return lhs;
			}

			return new AutoSdkConfig(Enumerable.Concat(lhs.View, rhs.View));
		}

		/// <summary>
		/// Test whether two views are equal
		/// </summary>
		public static bool Equals(AutoSdkConfig? lhs, AutoSdkConfig? rhs)
		{
			if (lhs?.View == null || lhs.View.Count == 0)
			{
				return rhs?.View == null || rhs.View.Count == 0;
			}
			if (rhs?.View == null || rhs.View.Count == 0)
			{
				return false;
			}

			return Enumerable.SequenceEqual(lhs.View, rhs.View);
		}
	}
	/// <summary>
	/// Document describing an active lease
	/// </summary>
	public interface IAgentLease
	{
		/// <summary>
		/// Name of this lease
		/// </summary>
		LeaseId Id { get; }

		/// <summary>
		/// The parent lease id
		/// </summary>
		LeaseId? ParentId { get; }

		/// <summary>
		/// The current state of the lease
		/// </summary>
		LeaseState State { get; }

		/// <summary>
		/// Resources used by this lease
		/// </summary>
		IReadOnlyDictionary<string, int>? Resources { get; }

		/// <summary>
		/// Whether the lease requires exclusive access to the agent
		/// </summary>
		bool Exclusive { get; }

		/// <summary>
		/// For leases in the pending state, encodes an "any" protobuf containing the payload for the agent to execute the lease.
		/// </summary>
		Any? Payload { get; }
	}

	/// <summary>
	/// Document describing an active lease
	/// </summary>
	public class AgentLease : IAgentLease
	{
		/// <inheritdoc/>
		[BsonRequired]
		public LeaseId Id { get; set; }

		/// <inheritdoc/>
		public LeaseId? ParentId { get; set; }

		/// <inheritdoc/>
		public LeaseState State { get; set; }

		/// <inheritdoc/>
		public IReadOnlyDictionary<string, int>? Resources { get; set; }

		/// <inheritdoc/>
		public bool Exclusive { get; set; }

		/// <inheritdoc/>
		public byte[]? Payload { get; set; }

		Any? IAgentLease.Payload => (Payload != null) ? Any.Parser.ParseFrom(Payload) : null;

		/// <summary>
		/// Private constructor
		/// </summary>
		[BsonConstructor]
		private AgentLease()
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		public AgentLease(IAgentLease other)
			: this(other.Id, other.ParentId, other.State, other.Resources, other.Exclusive, other.Payload?.ToByteArray())
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="id">Identifier for the lease</param>
		/// <param name="parentId">The parent lease id</param>
		/// <param name="state">State for the lease</param>
		/// <param name="resources">Resources required for this lease</param>
		/// <param name="exclusive">Whether to reserve the entire device</param>
		/// <param name="payload">Encoded "any" protobuf describing the contents of the payload</param>
		public AgentLease(LeaseId id, LeaseId? parentId, LeaseState state, IReadOnlyDictionary<string, int>? resources, bool exclusive, byte[]? payload)
		{
			Id = id;
			ParentId = parentId;
			State = state;
			Resources = resources;
			Exclusive = exclusive;
			Payload = payload;
		}

		/// <summary>
		/// Create a lease from 
		/// </summary>
		/// <param name="options"></param>
		/// <returns></returns>
		public AgentLease(CreateLeaseOptions options)
			: this(options.Id, options.ParentId, LeaseState.Pending, options.Resources, options.Exclusive, Any.Pack(options.Payload).ToByteArray())
		{
		}
	}

	static class AgentLeaseExtensions
	{
		/// <summary>
		/// Converts this lease to an RPC message
		/// </summary>
		/// <returns>RPC message</returns>
		public static RpcLease ToRpcMessage(this IAgentLease agentLease)
		{
			RpcLease lease = new RpcLease();
			lease.Id = agentLease.Id;
			lease.Payload = agentLease.Payload;
			lease.State = (RpcLeaseState)agentLease.State;
			return lease;
		}
	}
}
