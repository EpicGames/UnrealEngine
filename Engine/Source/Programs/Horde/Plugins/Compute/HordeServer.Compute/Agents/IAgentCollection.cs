// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Pools;
using HordeServer.Auditing;

#pragma warning disable CA1716 // rename parameter property so that it no longer conflicts with the reserved language keyword 'Property'

namespace HordeServer.Agents
{
	/// <summary>
	/// Interface for a collection of agent documents
	/// </summary>
	public interface IAgentCollection
	{
		/// <summary>
		/// Adds a new agent with the given properties
		/// </summary>
		/// <param name="options">Parameters for new agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IAgent> AddAsync(CreateAgentOptions options, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets an agent by ID
		/// </summary>
		/// <param name="agentId">Unique id of the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The agent document</returns>
		Task<IAgent?> GetAsync(AgentId agentId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets multiple agents by ID
		/// </summary>
		/// <param name="agentIds">List of unique IDs of the agents</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The agent documents</returns>
		Task<IReadOnlyList<IAgent>> GetManyAsync(List<AgentId> agentIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all agents matching certain criteria
		/// </summary>
		/// <param name="poolId">The pool ID in string form containing the agent</param>
		/// <param name="modifiedAfter">If set, only returns agents modified after this time</param>
		/// <param name="property">Property to look for</param>
		/// <param name="status">Status to look for</param>
		/// <param name="enabled">Enabled/disabled status to look for</param>
		/// <param name="includeDeleted">Whether agents marked as deleted should be included</param>
		/// <param name="consistentRead">If the database read should be made to the replica server</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of agents matching the given criteria</returns>
		IAsyncEnumerable<IAgent> FindAsync(PoolId? poolId = null, DateTime? modifiedAfter = null, string? property = null, AgentStatus? status = null, bool? enabled = null, bool includeDeleted = false, bool consistentRead = true, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the log channel for an agent
		/// </summary>
		/// <param name="agentId"></param>
		/// <returns></returns>
		IAuditLogChannel<AgentId> GetLogger(AgentId agentId);
	}
}
