// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Sessions;
using HordeCommon.Rpc.Messages;

namespace HordeServer.Agents
{
	/// <summary>
	/// Type of data to return from enumeration calls
	/// </summary>
	public enum SessionFilterType
	{
		/// <summary>
		/// Return only sessions which can satisfy resource/exclusivity requirements
		/// </summary>
		Available = 0,

		/// <summary>
		/// Return all active sessions in the response, even if they are currently busy
		/// </summary>
		Potential = 1,
	}

	/// <summary>
	/// Interface for a scheduler that manages agent sessions and leases.
	/// </summary>
	public interface IAgentScheduler
	{
		#region Sessions

		/// <summary>
		/// Event raised whenever a session is updated (locally or remotely)
		/// </summary>
		event Action<SessionId>? SessionUpdated;

		/// <summary>
		/// Attempts to create a new session for the given agent. Fails if the agent is already executing a session.
		/// </summary>
		/// <param name="agentId">The agent id</param>
		/// <param name="sessionId">Identifier for the new session</param>
		/// <param name="capabilities">Capabilities for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New session object</returns>
		Task<RpcSession?> TryCreateSessionAsync(AgentId agentId, SessionId sessionId, RpcAgentCapabilities capabilities, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to get the latest state of the given session
		/// </summary>
		/// <param name="sessionId">The session identifier</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Session object, or null if the session is not active</returns>
		Task<RpcSession?> TryGetSessionAsync(SessionId sessionId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Batch fetch the latest state of multiple sessions in a single round-trip
		/// </summary>
		/// <param name="sessionIds">The session identifiers to look up</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Dictionary mapping each found session ID to its session object. Missing sessions are omitted.</returns>
		Task<Dictionary<SessionId, RpcSession>> TryGetSessionsAsync(IReadOnlyList<SessionId> sessionIds, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempt to get capabilities for the given session
		/// </summary>
		/// <param name="session">The session object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Matching capabilities object for the session</returns>
		Task<RpcAgentCapabilities?> TryGetCapabilitiesAsync(RpcSession session, CancellationToken cancellationToken = default);

		/// <summary>
		/// Attempts to create a new session for the given agent. Fails if the agent is already executing a session.
		/// </summary>
		/// <param name="session">Previous session state</param>
		/// <param name="newSession">New session stats</param>
		/// <param name="newCapabilities">New capabilities for the agent</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>New session object</returns>
		Task<RpcSession?> TryUpdateSessionAsync(RpcSession session, RpcSession? newSession = null, RpcAgentCapabilities? newCapabilities = null, CancellationToken cancellationToken = default);

		#endregion
		#region Filters

		/// <summary>
		/// Create a new filter for agents matching  with the given requirements
		/// </summary>
		/// <param name="requirements">Requirements for the filter</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task<IoHash> CreateFilterAsync(RpcAgentRequirements requirements, CancellationToken cancellationToken = default);

		/// <summary>
		/// Update the timestamp on the given filter
		/// </summary>
		/// <param name="requirementsHash">Hash of the requirements for the filter to remove</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		Task TouchFilterAsync(IoHash requirementsHash, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the hash of all current filters
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Array of filter hashes</returns>
		Task<IoHash[]> GetFiltersAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets the requirements for a particular filter
		/// </summary>
		/// <param name="requirementsHash">Hash of the requirements object</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Requirements object</returns>
		ValueTask<RpcAgentRequirements?> TryGetFilterRequirementsAsync(IoHash requirementsHash, CancellationToken cancellationToken);

		/// <summary>
		/// Enumerate all the sessions in a filter
		/// </summary>
		/// <param name="requirementsHash">Requirements hash for the filter</param>
		/// <param name="sessionType">Whether to only include available sessions, or also include any sessions that are busy and cannot meet resource requirements</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Sequence of sessions</returns>
		IAsyncEnumerable<SessionId> EnumerateFilteredSessionsAsync(IoHash requirementsHash, SessionFilterType sessionType = SessionFilterType.Potential, CancellationToken cancellationToken = default);

		#endregion
		#region Stats

		/// <summary>
		/// Enumerates all the sessions which have expired
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of expired sessions</returns>
		IAsyncEnumerable<RpcSession> FindExpiredSessionsAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Remove session states that are not registered in the "current session key"
		/// This can happen when upgrading Horde to a newer version and old agents that created new sessions after that.
		/// Clearing Redis can also fix this issue.
		/// </summary>
		/// <param name="cancellationToken">Cancellation token</param>
		public Task<List<SessionId>> RemoveOrphanedSessionsAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all active agent lease IDs
		/// </summary>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of agent lease IDs</returns>
		Task<LeaseId[]> FindActiveLeaseIdsAsync(CancellationToken cancellationToken = default);

		/// <summary>
		/// Find how many child leases a particular lease has
		/// </summary>
		/// <param name="id">Lease ID</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of agent lease IDs</returns>
		Task<int> GetChildLeaseCountAsync(LeaseId id, CancellationToken cancellationToken = default);

		/// <summary>
		/// Get all child lease IDs
		/// </summary>
		/// <param name="id">Lease ID</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of agent lease IDs</returns>
		Task<LeaseId[]> GetChildLeaseIdsAsync(LeaseId id, CancellationToken cancellationToken = default);

		#endregion
	}
}
