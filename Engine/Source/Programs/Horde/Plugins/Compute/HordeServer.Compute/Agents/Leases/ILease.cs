// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using Google.Protobuf;
using Google.Protobuf.WellKnownTypes;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Document describing a lease. This exists to permanently record a lease; the agent object tracks internal state of any active leases through AgentLease objects.
	/// </summary>
	public interface ILease
	{
		/// <summary>
		/// The unique id of this lease
		/// </summary>
		public LeaseId Id { get; }

		/// <summary>
		/// Identifier for the parent lease. Used to terminate hierarchies of leases.
		/// </summary>
		public LeaseId? ParentId { get; }

		/// <summary>
		/// Name of this lease
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Unique id of the agent 
		/// </summary>
		public AgentId AgentId { get; }

		/// <summary>
		/// Unique id of the agent session
		/// </summary>
		public SessionId SessionId { get; }

		/// <summary>
		/// The stream this lease belongs to
		/// </summary>
		public StreamId? StreamId { get; }

		/// <summary>
		/// Pool for the work being executed
		/// </summary>
		public PoolId? PoolId { get; }

		/// <summary>
		/// The log for this lease, if applicable
		/// </summary>
		public LogId? LogId { get; }

		/// <summary>
		/// Time at which this lease started
		/// </summary>
		public DateTime StartTime { get; }

		/// <summary>
		/// Time at which this lease completed
		/// </summary>
		public DateTime? FinishTime { get; }

		/// <summary>
		/// Payload for this lease. A packed Google.Protobuf.Any object.
		/// </summary>
		public ReadOnlyMemory<byte> Payload { get; }

		/// <summary>
		/// Outcome of the lease
		/// </summary>
		public LeaseOutcome Outcome { get; }

		/// <summary>
		/// Reason for the outcome. Can be a combination of reasons.
		/// </summary>
		public LeaseOutcomeReason OutcomeReason { get; set; }

		/// <summary>
		/// Output from executing the lease
		/// </summary>
		public ReadOnlyMemory<byte> Output { get; }

		/// <summary>
		/// Metrics collected during execution of lease
		/// </summary>
		public LeaseMetrics? Metrics { get; }
	}

	/// <summary>
	/// CPU timings for a lease
	/// </summary>
	/// <param name="User">CPU time spent in user mode</param>
	/// <param name="System">CPU time spent in system/kernel mode</param>
	/// <param name="Idle">CPU time spent idle</param>
	public record LeaseCpuTime(TimeSpan User, TimeSpan System, TimeSpan Idle);

	/// <summary>
	/// Metrics collected during execution of lease
	/// Machine-wide metrics assumes the lease has exclusive access to the agent and other background work is negligible.
	/// </summary>
	/// <param name="SetupTime">Time taken to prepare for lease execution (e.g syncing VCS data)</param>
	/// <param name="TeardownTime">Time taken to clean up after a lease execution (removing files created etc)</param>
	/// <param name="CpuCount">Number of logical CPU cores that were available to the lease</param>
	/// <param name="GlobalCpuUtilization">CPU utilization in percent based on number of CPU seconds that was available during the lease</param>
	/// <param name="GlobalCpuTime">Machine-wide CPU time spent during this lease</param>
	public record LeaseMetrics(
		TimeSpan SetupTime,
		TimeSpan TeardownTime,
		int CpuCount,
		double GlobalCpuUtilization,
		LeaseCpuTime GlobalCpuTime
	);

	/// <summary>
	/// Extension methods for leases
	/// </summary>
	public static class LeaseExtensions
	{
		/// <summary>
		/// Gets the task from a lease, encoded as an Any protobuf object
		/// </summary>
		/// <param name="lease">The lease to query</param>
		/// <returns>The task definition encoded as a protobuf Any object</returns>
		public static Any GetTask(this ILease lease)
		{
			return Any.Parser.ParseFrom(lease.Payload.ToArray());
		}

		/// <summary>
		/// Gets a typed task object from a lease
		/// </summary>
		/// <typeparam name="T">Type of the protobuf message to return</typeparam>
		/// <param name="lease">The lease to query</param>
		/// <returns>The task definition</returns>
		public static T GetTask<T>(this ILease lease) where T : IMessage<T>, new()
		{
			return GetTask(lease).Unpack<T>();
		}
	}
}
