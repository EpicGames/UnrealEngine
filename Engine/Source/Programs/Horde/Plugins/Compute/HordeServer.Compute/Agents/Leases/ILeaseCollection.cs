// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Interface for a collection of lease documents
	/// </summary>
	public interface ILeaseCollection
	{
		/// <summary>
		/// Adds a lease to the collection
		/// </summary>
		/// <param name="id">The lease id</param>
		/// <param name="parentId">The parent lease id</param>
		/// <param name="name">Name of the lease</param>
		/// <param name="agentId">The agent id</param>
		/// <param name="sessionId">The agent session handling the lease</param>
		/// <param name="streamId">Stream for the payload</param>
		/// <param name="poolId">The pool for the lease</param>
		/// <param name="logId">Log id for the lease</param>
		/// <param name="startTime">Start time of the lease</param>
		/// <param name="payload">Payload for the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The new lease</returns>
		Task<ILease> AddAsync(LeaseId id, LeaseId? parentId, string name, AgentId agentId, SessionId sessionId, StreamId? streamId, PoolId? poolId, LogId? logId, DateTime startTime, byte[] payload, CancellationToken cancellationToken = default);

		/// <summary>
		/// Deletes a lease from the collection
		/// </summary>
		/// <param name="leaseId">Unique id of the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Async task</returns>
		Task DeleteAsync(LeaseId leaseId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Gets a specific lease
		/// </summary>
		/// <param name="leaseId">Unique id of the lease</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>The lease that was found, or null if it does not exist</returns>
		Task<ILease?> GetAsync(LeaseId leaseId, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all leases matching a set of criteria
		/// </summary>
		/// <param name="parentId">The parent lease id</param>
		/// <param name="agentId">Unqiue id of the agent executing this lease</param>
		/// <param name="sessionId">Unique id of the agent session</param>
		/// <param name="minTime">Start of the window to include leases</param>
		/// <param name="maxTime">End of the window to include leases</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="indexHint">Name of index to be specified as a hint to the database query planner</param>
		/// <param name="consistentRead">If the database read should be made to the replica server</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases matching the given criteria</returns>
		Task<IReadOnlyList<ILease>> FindLeasesAsync(LeaseId? parentId = null, AgentId? agentId = null, SessionId? sessionId = null, DateTime? minTime = null, DateTime? maxTime = null, int? index = null, int? count = null, string? indexHint = null, bool consistentRead = true, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all leases by finish time
		/// </summary>
		/// <param name="minFinishTime">Start of finish time window</param>
		/// <param name="maxFinishTime">End of finish time window</param>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="indexHint">Name of index to be specified as a hint to the database query planner</param>
		/// <param name="consistentRead">If the database read should be made to the replica server</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases matching the given criteria</returns>
		Task<IReadOnlyList<ILease>> FindLeasesByFinishTimeAsync(DateTime? minFinishTime, DateTime? maxFinishTime, int? index, int? count, string? indexHint = null, bool consistentRead = true, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all leases between min and/or max time
		/// Preferred over the more generic FindLeasesAsync as this one use a better index and relaxed read consistency
		/// </summary>
		/// <param name="minTime">Start of the window to include leases</param>
		/// <param name="maxTime">End of the window to include leases</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases matching the given criteria</returns>
		Task<IReadOnlyList<ILease>> FindLeasesAsync(DateTime? minTime, DateTime? maxTime, CancellationToken cancellationToken = default);

		/// <summary>
		/// Finds all active leases
		/// </summary>
		/// <param name="index">Index of the first result to return</param>
		/// <param name="count">Number of results to return</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>List of leases</returns>
		Task<IReadOnlyList<ILease>> FindActiveLeasesAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default);

		/// <summary>
		/// Sets the outcome of a lease
		/// </summary>
		/// <param name="leaseId">The lease to update</param>
		/// <param name="finishTime">Time at which the lease finished</param>
		/// <param name="outcome">Outcome of the lease</param>
		/// <param name="outcomeReason">Reason for the outcome</param>
		/// <param name="output">Output data from the task</param>
		/// <param name="metrics">Metrics</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the lease was updated, false otherwise</returns>
		Task<bool> TrySetOutcomeAsync(LeaseId leaseId, DateTime finishTime, LeaseOutcome outcome, LeaseOutcomeReason outcomeReason, byte[]? output, LeaseMetrics? metrics, CancellationToken cancellationToken = default);

		/// <summary>
		/// Adds outcome reasons to an existing lease. This can be used to merge reasons even after the lease has finished.
		/// </summary>
		/// <param name="leaseId">The lease to update</param>
		/// <param name="outcomeReason">Additional reasons to add (will be ORed with existing reasons)</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>True if the lease was updated, false if lease doesn't exist</returns>
		Task<bool> TryAddOutcomeReasonsAsync(LeaseId leaseId, LeaseOutcomeReason outcomeReason, CancellationToken cancellationToken = default);
	}
}
