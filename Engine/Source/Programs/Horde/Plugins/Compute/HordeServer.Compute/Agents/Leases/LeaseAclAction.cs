// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Acls;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// ACL actions which apply to leases
	/// </summary>
	public static class LeaseAclAction
	{
		/// <summary>
		/// View all the leases that an agent has worked on
		/// </summary>
		public static AclAction ViewLeases { get; } = new AclAction("ViewLeases");

		/// <summary>
		/// View the task data for a lease
		/// </summary>
		public static AclAction ViewLeaseTasks { get; } = new AclAction("ViewLeaseTasks");
	}
}
