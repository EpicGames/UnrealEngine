// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde.Agents;
using EpicGames.Horde.Agents.Leases;
using EpicGames.Horde.Agents.Pools;
using EpicGames.Horde.Agents.Sessions;
using EpicGames.Horde.Logs;
using EpicGames.Horde.Streams;
using HordeServer.Server;
using HordeServer.Utilities;
using MongoDB.Bson;
using MongoDB.Bson.Serialization.Attributes;
using MongoDB.Driver;
using OpenTelemetry.Trace;

namespace HordeServer.Agents.Leases
{
	/// <summary>
	/// Notification that a lease has been updated
	/// </summary>
	public delegate void OnLeaseCompleteDelegate(ILease lease);

	/// <summary>
	/// Collection of lease documents
	/// </summary>
	public class LeaseCollection : ILeaseCollection
	{
		record LeaseMetricsDocument(
			[property: BsonElement("setup")] int SetupTimeMs,
			[property: BsonElement("teardown")] int TeardownTimeMs,
			[property: BsonElement("cpus")] int CpuCount,
			[property: BsonElement("cpuUtil")] double GlobalCpuUtilization,
			[property: BsonElement("cpuUsr")] long GlobalCpuUserMs,
			[property: BsonElement("cpuSys")] long GlobalCpuSystemMs,
			[property: BsonElement("cpuIdl")] long GlobalCpuIdleMs
		)
		{
			public LeaseMetrics ToLeaseMetrics()
			{
				return new LeaseMetrics(
					TimeSpan.FromMilliseconds(SetupTimeMs),
					TimeSpan.FromMilliseconds(TeardownTimeMs),
					CpuCount,
					GlobalCpuUtilization,
					new LeaseCpuTime(TimeSpan.FromMilliseconds(GlobalCpuUserMs), TimeSpan.FromMilliseconds(GlobalCpuSystemMs), TimeSpan.FromMilliseconds(GlobalCpuIdleMs))
				);
			}

			public static LeaseMetricsDocument FromLeaseMetrics(LeaseMetrics metrics)
			{
				return new LeaseMetricsDocument(
					(int)metrics.SetupTime.TotalMilliseconds,
					(int)metrics.TeardownTime.TotalMilliseconds,
					metrics.CpuCount,
					metrics.GlobalCpuUtilization,
					(long)metrics.GlobalCpuTime.User.TotalMilliseconds,
					(long)metrics.GlobalCpuTime.System.TotalMilliseconds,
					(long)metrics.GlobalCpuTime.Idle.TotalMilliseconds
				);
			}
		}
		
		/// <summary>
		/// Concrete implementation of a lease document
		/// </summary>
		class LeaseDocument : ILease
		{
			public LeaseId Id { get; set; }

			[BsonIgnoreIfNull]
			public LeaseId? ParentId { get; set; }

			public string Name { get; set; }
			public AgentId AgentId { get; set; }
			public SessionId SessionId { get; set; }
			public StreamId? StreamId { get; set; }
			public PoolId? PoolId { get; set; }
			public LogId? LogId { get; set; }
			public DateTime StartTime { get; set; }
			public DateTime? FinishTime { get; set; }
			public byte[] Payload { get; set; }
			public LeaseOutcome Outcome { get; set; }
			public LeaseMetricsDocument? Metrics { get; set; }
			
			[BsonRepresentation(BsonType.Int64)]
			public LeaseOutcomeReason OutcomeReason { get; set; } = LeaseOutcomeReason.None;

			[BsonIgnoreIfNull]
			public byte[]? Output { get; set; }

			ReadOnlyMemory<byte> ILease.Payload => Payload;
			ReadOnlyMemory<byte> ILease.Output => Output ?? Array.Empty<byte>();
			LeaseMetrics? ILease.Metrics => Metrics?.ToLeaseMetrics();

			[BsonConstructor]
			private LeaseDocument()
			{
				Name = String.Empty;
				Payload = null!;
			}

			public LeaseDocument(LeaseId id, LeaseId? parentId, string name, AgentId agentId, SessionId sessionId, StreamId? streamId, PoolId? poolId, LogId? logId, DateTime startTime, byte[] payload)
			{
				Id = id;
				ParentId = parentId;
				Name = name;
				AgentId = agentId;
				SessionId = sessionId;
				StreamId = streamId;
				PoolId = poolId;
				LogId = logId;
				StartTime = startTime;
				Payload = payload;
			}
		}

		readonly IMongoCollection<LeaseDocument> _leases;
		readonly Tracer _tracer;
		readonly MongoIndex<LeaseDocument> _finishTimeStartTimeCompoundIndex;

		/// <inheritdoc/>
		public event OnLeaseCompleteDelegate? OnLeaseComplete;

		/// <summary>
		/// Constructor
		/// </summary>
		public LeaseCollection(IMongoService mongoService, Tracer tracer)
		{
			List<MongoIndex<LeaseDocument>> indexes = new List<MongoIndex<LeaseDocument>>();
			//			indexes.Add(keys => keys.Ascending(x => x.ParentId), sparse: true);
			indexes.Add(keys => keys.Ascending(x => x.AgentId));
			indexes.Add(keys => keys.Ascending(x => x.SessionId));
			indexes.Add(keys => keys.Ascending(x => x.StartTime));
			indexes.Add(keys => keys.Ascending(x => x.FinishTime));
			indexes.Add(_finishTimeStartTimeCompoundIndex = MongoIndex.Create<LeaseDocument>(keys => keys.Ascending(x => x.FinishTime).Descending(x => x.StartTime)));

			_leases = mongoService.GetCollection<LeaseDocument>("Leases", indexes);
			_tracer = tracer;
		}

		/// <inheritdoc/>
		public async Task<ILease> AddAsync(LeaseId id, LeaseId? parentId, string name, AgentId agentId, SessionId sessionId, StreamId? streamId, PoolId? poolId, LogId? logId, DateTime startTime, byte[] payload, CancellationToken cancellationToken)
		{
			LeaseDocument lease = new LeaseDocument(id, parentId, name, agentId, sessionId, streamId, poolId, logId, startTime, payload);
			await _leases.ReplaceOneAsync(x => x.Id == id, lease, new ReplaceOptions { IsUpsert = true }, cancellationToken);
			return lease;
		}

		/// <inheritdoc/>
		public async Task DeleteAsync(LeaseId leaseId, CancellationToken cancellationToken)
		{
			await _leases.DeleteOneAsync(x => x.Id == leaseId, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<ILease?> GetAsync(LeaseId leaseId, CancellationToken cancellationToken)
		{
			return await _leases.Find(x => x.Id == leaseId).FirstOrDefaultAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ILease>> FindLeasesAsync(LeaseId? parentId, AgentId? agentId, SessionId? sessionId, DateTime? minTime, DateTime? maxTime, int? index, int? count, string? indexHint = null, bool consistentRead = true, CancellationToken cancellationToken = default)
		{
			using TelemetrySpan span = _tracer.StartActiveSpan($"{nameof(AgentService)}.{nameof(FindLeasesAsync)}");
			span.SetAttribute("AgentId", agentId?.ToString());
			span.SetAttribute("SessionId", sessionId?.ToString());
			span.SetAttribute("StartTime", minTime?.ToString());
			span.SetAttribute("FinishTime", maxTime?.ToString());
			span.SetAttribute("Index", index);
			span.SetAttribute("Count", count);

			IMongoCollection<LeaseDocument> collection = consistentRead ? _leases : _leases.WithReadPreference(ReadPreference.SecondaryPreferred);

			FilterDefinitionBuilder<LeaseDocument> filterBuilder = Builders<LeaseDocument>.Filter;
			FilterDefinition<LeaseDocument> filter = FilterDefinition<LeaseDocument>.Empty;
			if (parentId != null)
			{
				filter &= filterBuilder.Eq(x => x.ParentId, parentId.Value);
			}
			if (agentId != null)
			{
				filter &= filterBuilder.Eq(x => x.AgentId, agentId.Value);
			}
			if (sessionId != null)
			{
				filter &= filterBuilder.Eq(x => x.SessionId, sessionId.Value);
			}
			if (minTime != null)
			{
				filter &= filterBuilder.Or(filterBuilder.Eq(x => x.FinishTime!, null), filterBuilder.Gt(x => x.FinishTime!, minTime.Value));
			}
			if (maxTime != null)
			{
				filter &= filterBuilder.Lt(x => x.StartTime, maxTime.Value);
			}

			return await collection.FindWithHintAsync(filter, indexHint, x => x.SortByDescending(x => x.StartTime).Range(index, count).ToListAsync(cancellationToken));
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ILease>> FindLeasesByFinishTimeAsync(DateTime? minFinishTime, DateTime? maxFinishTime, int? index, int? count, string? indexHint, bool consistentRead, CancellationToken cancellationToken)
		{
			IMongoCollection<LeaseDocument> collection = consistentRead ? _leases : _leases.WithReadPreference(ReadPreference.SecondaryPreferred);
			FilterDefinitionBuilder<LeaseDocument> filterBuilder = Builders<LeaseDocument>.Filter;
			FilterDefinition<LeaseDocument> filter = FilterDefinition<LeaseDocument>.Empty;

			if (minFinishTime == null && maxFinishTime == null)
			{
				throw new ArgumentException($"Both {nameof(minFinishTime)} and {nameof(maxFinishTime)} cannot be null");
			}

			if (minFinishTime != null)
			{
				filter &= filterBuilder.Gt(x => x.FinishTime, minFinishTime.Value);
			}
			if (maxFinishTime != null)
			{
				filter &= filterBuilder.Lt(x => x.FinishTime, maxFinishTime.Value);
			}

			return await collection.FindWithHintAsync(filter, indexHint, x => x.SortByDescending(x => x.FinishTime).Range(index, count).ToListAsync(cancellationToken));
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ILease>> FindLeasesAsync(DateTime? minTime, DateTime? maxTime, CancellationToken cancellationToken = default)
		{
			return await FindLeasesAsync(null, null, null, minTime, maxTime, null, null, _finishTimeStartTimeCompoundIndex.Name, false, cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<IReadOnlyList<ILease>> FindActiveLeasesAsync(int? index = null, int? count = null, CancellationToken cancellationToken = default)
		{
			return await _leases.Find(x => x.FinishTime == null).Range(index, count).ToListAsync(cancellationToken);
		}

		/// <inheritdoc/>
		public async Task<bool> TrySetOutcomeAsync(LeaseId leaseId, DateTime finishTime, LeaseOutcome outcome, LeaseOutcomeReason outcomeReason, byte[]? output, LeaseMetrics? metrics, CancellationToken cancellationToken)
		{
			FilterDefinitionBuilder<LeaseDocument> filterBuilder = Builders<LeaseDocument>.Filter;
			FilterDefinition<LeaseDocument> filter = filterBuilder.Eq(x => x.Id, leaseId) & filterBuilder.Eq(x => x.FinishTime, null);

			UpdateDefinitionBuilder<LeaseDocument> updateBuilder = Builders<LeaseDocument>.Update;
			UpdateDefinition<LeaseDocument> update = updateBuilder
				.Set(x => x.FinishTime, finishTime)
				.Set(x => x.Outcome, outcome)
				.Set(x => x.OutcomeReason, outcomeReason);

			if (output is { Length: > 0 })
			{
				update = update.Set(x => x.Output, output);
			}

			if (metrics != null)
			{
				update = update.Set(x => x.Metrics, LeaseMetricsDocument.FromLeaseMetrics(metrics));
			}

			LeaseDocument? document = await _leases.FindOneAndUpdateAsync(filter, update, new FindOneAndUpdateOptions<LeaseDocument, LeaseDocument> { ReturnDocument = ReturnDocument.After }, cancellationToken: cancellationToken);
			if (document == null)
			{
				return false;
			}

			OnLeaseComplete?.Invoke(document);
			return true;
		}

		/// <inheritdoc/>
		public async Task<bool> TryAddOutcomeReasonsAsync(LeaseId leaseId, LeaseOutcomeReason outcomeReason, CancellationToken cancellationToken)
		{
			if (outcomeReason == LeaseOutcomeReason.None)
			{
				return true; // Nothing to add
			}

			FilterDefinition<LeaseDocument> filter = Builders<LeaseDocument>.Filter.Eq(x => x.Id, leaseId);
			UpdateDefinition<LeaseDocument> update = Builders<LeaseDocument>.Update.BitwiseOr(x => x.OutcomeReason, outcomeReason);

			UpdateResult result = await _leases.UpdateOneAsync(filter, update, cancellationToken: cancellationToken);
			return result.MatchedCount > 0;
		}
	}
}
